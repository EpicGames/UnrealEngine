// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/NumericLimits.h"
#include "Chaos/ISpatialAcceleration.h"
#include "Chaos/PBDRigidClusteredParticles.h"
#include "Chaos/PBDGeometryCollectionParticles.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/ParticleIterator.h"
#include "Chaos/Properties.h"
#include "Chaos/Collision/CollisionFilterBits.h"
#include "ChaosCheck.h"
#include "Chaos/ChaosDebugDrawDeclares.h"
#include "Chaos/AsyncInitBodyHelper.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#if CHAOS_DEBUG_DRAW
#include "Chaos/ChaosDebugDraw.h"
#endif

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Chaos/ChaosUserEntity.h"
#include "PhysicsInterfaceTypesCore.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#ifndef UE_DEBUG_DANGLING_HANDLES
#define UE_DEBUG_DANGLING_HANDLES 0 // Will deliberately cause a memory leak
#endif

class IPhysicsProxyBase;
class UGeometryCollectionComponent;

namespace Chaos
{
	class FConstraintHandle;
	class FPBDRigidsEvolutionBase;
	
	namespace Private
	{
		class FConvexOptimizer;

		CHAOS_API extern FString EmptyParticleName;
	}

	namespace CVars
	{
		bool ForceDeepCopyOnModifyGeometry();
	}

struct FGeometryParticleParameters
{
	FGeometryParticleParameters()
		: bDisabled(false) {}
	bool bDisabled;
};

template <typename T, int d>
using TGeometryParticleParameters UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FGeometryParticleParameters instead") = FGeometryParticleParameters;

struct FKinematicGeometryParticleParameters : public FGeometryParticleParameters
{
	FKinematicGeometryParticleParameters()
		: FGeometryParticleParameters() {}
};

template <typename T, int d>
using TKinematicGeometryParticleParameters UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FKinematicGeometryParticleParameters instead") = FKinematicGeometryParticleParameters;


struct FPBDRigidParticleParameters : public FKinematicGeometryParticleParameters
{
	FPBDRigidParticleParameters()
		: FKinematicGeometryParticleParameters()
		, bStartSleeping(false)
		, bGravityEnabled(true)
		, bCCDEnabled(false)
	{}
	bool bStartSleeping;
	bool bGravityEnabled;
	bool bCCDEnabled;
};

template <typename T, int d>
using TPBDRigidParticleParameters UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPBDRigidParticleParameters instead") = FPBDRigidParticleParameters;

/** Concrete can either be the game thread or physics representation, but API stays the same. Useful for keeping initialization and other logic the same*/
template <typename T, int d, typename FConcrete>
void GeometryParticleDefaultConstruct(FConcrete& Concrete, const FGeometryParticleParameters& Params)
{
	Concrete.SetX(TVector<T, d>(0));
	Concrete.SetR(TRotation<T, d>::Identity);
	Concrete.SetSpatialIdx(FSpatialAccelerationIdx{ 0,0 });
	Concrete.SetResimType(EResimType::FrozenDuringResim);
	Concrete.SetEnabledDuringResim(true);
}


extern CHAOS_API int32 AccelerationStructureSplitStaticAndDynamic;
template <typename T, int d, typename FConcrete>
void KinematicGeometryParticleDefaultConstruct(FConcrete& Concrete, const FKinematicGeometryParticleParameters& Params)
{
	Concrete.SetV(TVector<T, d>(0));
	Concrete.SetW(TVector<T, d>(0));
	if (AccelerationStructureSplitStaticAndDynamic == 1)
	{
		Concrete.SetSpatialIdx(FSpatialAccelerationIdx{ 0,1 });
	}
	else
	{
		Concrete.SetSpatialIdx(FSpatialAccelerationIdx{ 0,0 });
	}
}
template <typename T, int d, typename FConcrete>
void PBDRigidParticleDefaultConstruct(FConcrete& Concrete, const FPBDRigidParticleParameters& Params)
{
	//don't bother calling parent since the call gets made by the corresponding hierarchy in FConcrete
	Concrete.SetCollisionGroup(0);
	Concrete.SetLinearImpulseVelocity(TVector<T, d>(0));
	Concrete.SetAngularImpulseVelocity(TVector<T, d>(0));
	Concrete.SetMaxLinearSpeedSq(TNumericLimits<float>::Max());
	Concrete.SetMaxAngularSpeedSq(TNumericLimits<float>::Max());
	Concrete.SetInitialOverlapDepenetrationVelocity(-1.0f);
	Concrete.SetSleepThresholdMultiplier(1.0f);
	Concrete.SetM(1);
	Concrete.SetInvM(1);
	Concrete.SetCenterOfMass(TVector<T,d>(0));
	Concrete.SetRotationOfMass(TRotation<T, d>::FromIdentity());
	Concrete.SetI(TVec3<FRealSingle>(1, 1, 1));
	Concrete.SetInvI(TVec3<FRealSingle>(1, 1, 1));
	Concrete.SetLinearEtherDrag(0.f);
	Concrete.SetAngularEtherDrag(0.f);
	Concrete.SetGravityEnabled(Params.bGravityEnabled);
	Concrete.SetGravityGroupIndex(0);
	Concrete.SetCCDEnabled(Params.bCCDEnabled);
	Concrete.SetMACDEnabled(false);
	Concrete.SetPartialIslandSleepAllowed(true);
	Concrete.SetIterationSettings(Private::FIterationSettings(INDEX_NONE, INDEX_NONE, INDEX_NONE));
	Concrete.SetDisabled(Params.bDisabled);
	Concrete.SetSleepType(ESleepType::MaterialSleep);
}



template <typename T, int d, typename FConcrete>
void PBDRigidClusteredParticleDefaultConstruct(FConcrete& Concrete, const FPBDRigidParticleParameters& Params)
{
	//don't bother calling parent since the call gets made by the corresponding hierarchy in FConcrete
}

template <typename FConcrete>
bool GeometryParticleSleeping(const FConcrete& Concrete)
{
	if(auto Rigid = Concrete.CastToRigidParticle())
	{
		return Rigid->Sleeping();
	}
	else
	{
	return Concrete.ObjectState() == EObjectStateType::Sleeping;
	}
}

//Used to filter out at the acceleration structure layer using Query data
//Returns true when there is no way a later PreFilter will succeed. Avoid virtuals etc..
UE_DEPRECATED(5.8, "Use FShapeUnionFilterData::BroadFilter(const FQueryFilterData&) instead")
FORCEINLINE_DEBUGGABLE bool PrePreQueryFilterImp(const FCollisionFilterData& QueryFilterData, const FCollisionFilterData& UnionFilterData)
{
	//HACK: need to replace all these hard-coded values with proper enums, bad modules are not setup for it right now
	//ECollisionQuery QueryType = (ECollisionQuery)QueryFilter.Word0;
	//if (QueryType != ECollisionQuery::ObjectQuery)
	if(QueryFilterData.Word0)
	{
		//since we're taking the union of shapes we can only support trace channel
		//const ECollisionChannel QuerierChannel = GetCollisionChannel(QueryFilter.Word3);
		const uint32 QuerierChannel = (QueryFilterData.Word3 << 6) >> (32 - 5);
	
		//uint32 const QuerierBit = ECC_TO_BITFIELD(QuerierChannel);
		const uint32 QuerierBit = (1 << (QuerierChannel));
	
		// check if Querier wants a hit
		const uint32 TouchOrBlock = (UnionFilterData.Word1 | UnionFilterData.Word2);
		return !(QuerierBit & TouchOrBlock);
	}

	return false;
}

UE_DEPRECATED(5.8, "Use the member functions on FShapeFilterData instead")
FORCEINLINE_DEBUGGABLE uint32 GetChaosCollisionChannelAndExtraFilter(uint32 Word3, uint8& OutMaskFilter)
{
	uint32 ChannelMask = (Word3 << ChaosNumExtraFilterBits) >> (32 - ChaosNumCollisionChannelBits);
	OutMaskFilter = Word3 >> (32 - ChaosNumExtraFilterBits);
	return (uint32)ChannelMask;
}

//Used to filter out at the acceleration structure layer using Simdata
//Returns true when there is no way a later PreFilter will succeed. Avoid virtuals etc..
UE_DEPRECATED(5.8, "Use FShapeUnionFilterData::BroadFilter(const FShapeFilterData&) instead")
FORCEINLINE_DEBUGGABLE bool PrePreSimFilterImp(const FCollisionFilterData& SimFilterData, const FCollisionFilterData& OtherSimFilterData)
{
	//HACK: need to replace all these hard-coded values with proper enums, bad modules are not setup for it right now
	//since we're taking the union of shapes we can only support trace channel
	//const ECollisionChannel QuerierChannel = GetCollisionChannel(QueryFilter.Word3);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	uint8  QuerierMaskFilter;
	const uint32 QuerierChannel = GetChaosCollisionChannelAndExtraFilter(SimFilterData.Word3, QuerierMaskFilter);
	uint8  OtherMaskFilter;
	const uint32 OtherChannel = GetChaosCollisionChannelAndExtraFilter(OtherSimFilterData.Word3, OtherMaskFilter);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	//uint32 const QuerierBit = ECC_TO_BITFIELD(QuerierChannel);
	const uint32 QuerierBit = (1 << (QuerierChannel));
	const uint32 OtherBit = (1 << (OtherChannel));

	// check if they can collide ( same logic as DoCollide in CollisionResolution.cpp )
	const bool CanCollide = (QuerierBit & OtherSimFilterData.Word1) && (OtherBit & SimFilterData.Word1);
	return !CanCollide;
}

/** Wrapper that holds both physics thread data and GT data. It's possible that the physics handle is null if we're doing operations entirely on external threads*/
class FAccelerationStructureHandle
{
public:
	static constexpr bool bHasPayloadOnInternalThread = true;

	FAccelerationStructureHandle(FGeometryParticleHandle* InHandle);

	// @param bUsePrefiltering make sur the prefiltering data is computed. setting it to false may be useful when building a payload for a remove operation
	FAccelerationStructureHandle(FGeometryParticle* InGeometryParticle = nullptr, bool bUsePrefiltering = true);

	template <bool bPersistent>
	FAccelerationStructureHandle(TGeometryParticleHandleImp<FReal, 3, bPersistent>& InHandle);

	//Should only be used by GT and scene query threads where an appropriate lock has been acquired
	FGeometryParticle* GetExternalGeometryParticle_ExternalThread() const { return ExternalGeometryParticle; }

	//Should only be used by PT
	FGeometryParticleHandle* GetGeometryParticleHandle_PhysicsThread() const { return GeometryParticleHandle; }

	bool operator==(const FAccelerationStructureHandle& Rhs) const
	{
		return CachedUniqueIdx == Rhs.CachedUniqueIdx;
	}

	bool operator!=(const FAccelerationStructureHandle& Rhs) const
	{
		return !(*this == Rhs);
	}

	void Serialize(FChaosArchive& Ar);

	FUniqueIdx UniqueIdx() const
	{
		return CachedUniqueIdx;
	}

	UE_DEPRECATED(5.8, "This has been deprecated in favor of PrePreFilter")
	bool PrePreQueryFilter(const void* QueryData) const
	{
		if(bCanPrePreFilter)
		{
			if (const FCollisionFilterData* QueryFilterData = static_cast<const FCollisionFilterData*>(QueryData))
			{
				PRAGMA_DISABLE_INTERNAL_WARNINGS
				const Chaos::Filter::FQueryFilterData QuerierFilter = Chaos::Filter::FQueryFilterBuilder::BuildFromLegacyQueryFilter(*QueryFilterData);
				PRAGMA_ENABLE_INTERNAL_WARNINGS
				return UnionFilter.BroadFilter(QuerierFilter) == Chaos::Filter::EBroadFilterResult::Reject;
			}
		}
		
		return false;
	}

	UE_DEPRECATED(5.8, "This has been deprecated in favor of PrePreFilter")
	bool PrePreSimFilter(const void* SimData) const
	{
		if (bCanPrePreFilter)
		{
			if (const FCollisionFilterData* SimFilterData = static_cast<const FCollisionFilterData*>(SimData))
			{
				PRAGMA_DISABLE_INTERNAL_WARNINGS
				const Chaos::Filter::FShapeFilterData ShapeFilter = Chaos::Filter::FShapeFilterBuilder::BuildFromLegacySimFilter(*SimFilterData);
				PRAGMA_ENABLE_INTERNAL_WARNINGS
				Chaos::Filter::FShapeUnionFilterData SimUnionFilter;
				SimUnionFilter.Combine(ShapeFilter);
				return UnionFilter.BroadFilter(SimUnionFilter) == Chaos::Filter::EBroadFilterResult::Reject;
			}
		}

		return false;
	}

	bool PrePreFilter(const Chaos::Filter::FQueryFilterData& QueryFilterData) const
	{
		if (bCanPrePreFilter)
		{
			return UnionFilter.BroadFilter(QueryFilterData) == Chaos::Filter::EBroadFilterResult::Reject;
		}

		return false;
	}

	bool PrePreFilter(const Chaos::Filter::FShapeUnionFilterData& QuerierFilterData) const
	{
		if (bCanPrePreFilter)
		{
			return UnionFilter.BroadFilter(QuerierFilterData) == Chaos::Filter::EBroadFilterResult::Reject;
		}

		return false;
	}

	bool PrePreFilter(const FAccelerationStructureHandle& OtherHandle) const
	{
		return PrePreFilter(OtherHandle.UnionFilter);
	}

	void UpdateFrom(const FAccelerationStructureHandle& InOther)
	{
		UnionFilter = InOther.UnionFilter;
	}

public:
	/**
	* compute the aggregated query collision filter from all associated shapes.
	* NOTE: may not initialize OutQueryFilterData, so it must be cleared before calling if necessary (e.g., when changing overlapping channels)
	**/
	template <typename TParticle>
	UE_DEPRECATED(5.8, "Use ComputeFilterDataFromShapes instead")
	static void ComputeParticleQueryFilterDataFromShapes(const TParticle& Particle, FCollisionFilterData& OutQueryFilterData);

	// Computes union filter data from the associated shapes.
	template <typename TParticle>
	static void ComputeFilterDataFromShapes(const TParticle& Particle, Chaos::Filter::FShapeUnionFilterData& OutFilter);

private:
	FGeometryParticle* ExternalGeometryParticle;
	FGeometryParticleHandle* GeometryParticleHandle;

	FUniqueIdx CachedUniqueIdx;
	Chaos::Filter::FShapeUnionFilterData UnionFilter;

	bool bCanPrePreFilter;

	template <typename TParticle>
	void UpdatePrePreFilter(const TParticle& Particle);

public:
#if CHAOS_DEBUG_DRAW
	void DebugDraw(const bool bExternal, const bool bHit) const;
#endif
};

template <typename T, int d>
using TAccelerationStructureHandle UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FAccelerationStructureHandle instead") = FAccelerationStructureHandle;

template <typename T, int d>
class TParticleHandleBase
{
public:
	using TType = T;
	static constexpr int D = d;

	TParticleHandleBase()
		: SerializableGeometryParticles(TSerializablePtr<TGeometryParticles<T,d>>())
		, ParticleIdx(0)
		, Type(EParticleType::Static)
	{
	}

	template <typename TParticlesType>
	TParticleHandleBase(TSerializablePtr<TParticlesType> InParticles, int32 InParticleIdx)
	: SerializableGeometryParticles(InParticles)
	, ParticleIdx(InParticleIdx)
	, Type(InParticles ? InParticles->ParticleType() : EParticleType::Static)
	{
	}

	//Should only be used for transient handles - maybe we can protect this better?
	TParticleHandleBase(TGeometryParticles<T,d>* InParticles, int32 InParticleIdx)
		: SerializableGeometryParticles(TSerializablePtr<TGeometryParticles<T, d>>())
		, ParticleIdx(InParticleIdx)
		, Type(InParticles ? InParticles->ParticleType() : EParticleType::Static)
	{
		GeometryParticles = InParticles;
	}

	//NOTE: this is not virtual and only acceptable because we know the children have no extra data that requires destruction. 
	//You must modify the union to extend this class and do not add any member variables
	~TParticleHandleBase() = default;

	void Serialize(FChaosArchive& Ar)
	{
		Ar << ParticleIdx;
		uint8 RawType = (uint8)Type;
		Ar << RawType;
		Type = (EParticleType)RawType;
		Ar << SerializableGeometryParticles;
	}

	//This is needed for post serialization fixup of raw pointer. Should only be called by serialization code which is low level and knows the implementation details
	void SetSOALowLevel(TGeometryParticles<T, d>* InParticles)
	{
		//should not be swapping SOAs
		ensure(GeometryParticles == nullptr || GeometryParticles == InParticles);
		GeometryParticles = InParticles;
	}

	EParticleType GetParticleType() const { return Type; }
protected:
	union
	{
		TSerializablePtr<TGeometryParticles<T, d>> SerializableGeometryParticles;
		TGeometryParticles<T, d>* GeometryParticles;
		TKinematicGeometryParticles<T, d>* KinematicGeometryParticles;
		TPBDRigidParticles<T, d>* PBDRigidParticles;
		TPBDRigidClusteredParticles<T, d>* PBDRigidClusteredParticles;
	};

	template <typename TSOA>
	friend class TConstParticleIterator;
	//todo: maybe make private?
	int32 ParticleIdx;	//Index into the particle struct of arrays. Note the index can change
	EParticleType Type;
};

template <typename T, int d, bool bPersistent>
class TKinematicGeometryParticleHandleImp;

template <typename T, int d, bool bPersistent>
class TPBDRigidParticleHandleImp;

template <typename T, int d>
TGeometryParticleHandle<T, d>* GetHandleHelper(TGeometryParticleHandle<T, d>* Handle) { return Handle; }
template <typename T, int d>
const TGeometryParticleHandle<T, d>* GetHandleHelper(const TGeometryParticleHandle<T, d>* Handle) { return Handle; }
template <typename T, int d>
TGeometryParticleHandle<T, d>* GetHandleHelper(TTransientGeometryParticleHandle<T, d>* Handle);
template <typename T, int d>
const TGeometryParticleHandle<T, d>* GetHandleHelper(const TTransientGeometryParticleHandle<T, d>* Handle);

template <typename T, int d>
class TGeometryParticleHandles;

template <typename T, int d, bool bPersistent>
class TGeometryParticleHandleImp : public TParticleHandleBase<T,d>
{
public:
	using FDynamicParticleHandleType = TPBDRigidParticleHandleImp<T, d, bPersistent>;
	using FKinematicParticleHandleType = TKinematicGeometryParticleHandleImp<T, d, bPersistent>;

	using TTransientHandle = TTransientGeometryParticleHandle<T, d>;
	using THandleBase = TParticleHandleBase<T, d>;
	using THandleBase::GeometryParticles;
	using THandleBase::ParticleIdx;
	using THandleBase::Type;
	using TSOAType = TGeometryParticles<T,d>;

	static constexpr bool AlwaysSerializable = bPersistent;

	static TGeometryParticleHandleImp<T, d, bPersistent>* SerializationFactory(FChaosArchive& Ar, TGeometryParticleHandleImp<T, d, bPersistent>* Handle);

	template <typename TPayloadType>
	TPayloadType GetPayload(int32 Idx)
	{
		return TPayloadType(Handle());
	}


protected:
	//needed for serialization
	TGeometryParticleHandleImp()
		: TParticleHandleBase<T, d>()
	{
		SetConstraintGraphNode(nullptr);
	}

	TGeometryParticleHandleImp(TSerializablePtr<TGeometryParticles<T, d>> InParticles, int32 InParticleIdx, int32 InHandleIdx, const FGeometryParticleParameters& Params)
		: TParticleHandleBase<T, d>(InParticles, InParticleIdx)
		, HandleIdx(InHandleIdx)
	{
		//GeometryParticles->Handle(ParticleIdx) = this;
		//TODO: patch from SOA
		GeometryParticleDefaultConstruct<T, d>(*this, Params);
		SetHasBounds(false);
		SetLightWeightDisabled(false);
		SetConstraintGraphNode(nullptr);
	}

	// For transient handle
	TGeometryParticleHandleImp(TGeometryParticles<T, d>* InParticles, const int32 InParticleIdx)
		: TParticleHandleBase<T, d>(InParticles, InParticleIdx)
	{
	}

	template <typename TParticlesType, typename TParams>
	static TUniquePtr<typename TParticlesType::THandleType> CreateParticleHandleHelper(TSerializablePtr<TParticlesType> InParticles, int32 InParticleIdx, int32 InHandleIdx, const TParams& Params)
	{
		check(bPersistent);	//non persistent should not be going through this path
		auto NewHandle = new typename TParticlesType::THandleType(InParticles, InParticleIdx, InHandleIdx, Params);
		TUniquePtr<typename TParticlesType::THandleType> Unique(NewHandle);
		const_cast<TParticlesType*>(InParticles.Get())->SetHandle(InParticleIdx, NewHandle);	//todo: add non const serializable ptr
		return Unique;
	}
public:

	static TUniquePtr<TGeometryParticleHandleImp<T,d, bPersistent>> CreateParticleHandle(TSerializablePtr<TGeometryParticles<T, d>> InParticles, int32 InParticleIdx, int32 InHandleIdx, const FGeometryParticleParameters& Params = FGeometryParticleParameters())
	{
		return TGeometryParticleHandleImp<T,d,bPersistent>::CreateParticleHandleHelper(InParticles, InParticleIdx, InHandleIdx, Params);
	}

	~TGeometryParticleHandleImp()
	{
		if constexpr (bPersistent)
		{
			// If we weren't removed from the graph, invalid pointer dereferencing is possible
			check(GetConstraintGraphNode() == nullptr);

			GeometryParticles->ResetWeakParticleHandle(ParticleIdx);
			GeometryParticles->DestroyParticle(ParticleIdx);
			if (static_cast<uint32>(ParticleIdx) < GeometryParticles->Size())
			{
				if (GeometryParticles->RemoveParticleBehavior() == ERemoveParticleBehavior::RemoveAtSwap)
				{
					GeometryParticles->Handle(ParticleIdx)->ParticleIdx = ParticleIdx;
				}
				else
				{
					//need to update all handles >= ParticleIdx
					for (int32 Idx = ParticleIdx; static_cast<uint32>(Idx) < GeometryParticles->Size(); ++Idx)
					{
						GeometryParticles->Handle(Idx)->ParticleIdx -= 1;
					}
				}
			}

			// Zero the handle out to detect dangling handles and associated memory corruptions
			HandleIdx = INDEX_NONE;
			GeometryParticles = nullptr;
			ParticleIdx = INDEX_NONE;
			Type = EParticleType::Unknown;

			CVD_TRACE_INVALIDATE_CACHED_PARTICLE_METADATA(this);
		}
	}

	template <typename T2, int d2>
	friend TGeometryParticleHandle<T2, d2>* GetHandleHelper(TTransientGeometryParticleHandle<T2, d2>* Handle);
	template <typename T2, int d2>
	friend const TGeometryParticleHandle<T2, d2>* GetHandleHelper(const TTransientGeometryParticleHandle<T2, d2>* Handle);

	friend class TGeometryParticleHandleImp<T, d, false>;
	friend class TGeometryParticleHandleImp<T, d, true>;

	TGeometryParticleHandleImp<T, d, false> AsTransient()
	{
		return TGeometryParticleHandleImp<T, d, false>(GeometryParticles, ParticleIdx);
	}

	TGeometryParticleHandleImp(const TGeometryParticleHandleImp&) = delete;

	UE_DEPRECATED(5.4, "Use GetX instead")
	const TVector<T, d>& X() const { return GeometryParticles->GetX(ParticleIdx); }
	UE_DEPRECATED(5.4, "Use GetX or SetX instead")
	TVector<T, d>& X() { 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS 
			return GeometryParticles->X(ParticleIdx);  
		PRAGMA_ENABLE_DEPRECATION_WARNINGS }
	const TVector<T, d>& GetX() const { return GeometryParticles->GetX(ParticleIdx); }
	void SetX(const TVector<T, d>& InX, bool bInvalidate = false) { GeometryParticles->SetX(ParticleIdx, InX); }

	FUniqueIdx UniqueIdx() const { return GeometryParticles->UniqueIdx(ParticleIdx); }
	void SetUniqueIdx(const FUniqueIdx UniqueIdx, bool bInvalidate = false) const { GeometryParticles->UniqueIdx(ParticleIdx) = UniqueIdx; }

	UE_DEPRECATED(5.4, "Use GetR instead")
	const TRotation<T, d> R() const { return GeometryParticles->GetR(ParticleIdx); }
	UE_DEPRECATED(5.4, "Use SetR or GetR instead")
	TRotation<T, d> R() { return GeometryParticles->GetR(ParticleIdx); }
	const TRotation<T, d> GetR() const { return GeometryParticles->GetR(ParticleIdx); }
	void SetR(const TRotation<T, d>& InR, bool bInvalidate = false) { GeometryParticles->SetR(ParticleIdx, InR); }
	const TRotation<FRealSingle, d> GetRf() const { return GeometryParticles->GetRf(ParticleIdx); }
	void SetRf(const TRotation<FRealSingle, d>& InR, bool bInvalidate = false) { GeometryParticles->SetRf(ParticleIdx, InR); }

	FRigidTransform3 GetTransformXR() const { return FRigidTransform3(GetX(), GetR()); }

	// Initialize the transform
	void InitTransform(const FVec3& InP, const FRotation3& InQ)
	{
		SetX(InP);
		SetR(InQ);
	}

	void SetXR(const FParticlePositionRotation& XR);
	
	void SetNonFrequentData(const FParticleNonFrequentData& InData)
	{
#if CHAOS_DEBUG_NAME
		SetDebugName(InData.DebugName());
#endif

		SetGeometry(Chaos::FImplicitObjectPtr(InData.GetGeometry()));
		SetUniqueIdx(InData.UniqueIdx());
		SetSpatialIdx(InData.SpatialIdx());
		SetParticleID(InData.ParticleID());
	}

	bool HasCollision() const { return GeometryParticles->HasCollision(ParticleIdx); }

	void SetHasCollision(const bool bHasCollision)
	{
		GeometryParticles->HasCollision(ParticleIdx) = bHasCollision;
	}

	ESyncState SyncState() const
	{
		return GeometryParticles->SyncState(ParticleIdx);
	}

	void SetSyncState(ESyncState State)
	{
		GeometryParticles->SyncState(ParticleIdx) = State;
	}

	const FImplicitObjectRef GetGeometry() const { return GeometryParticles->GetGeometry(ParticleIdx).GetReference(); }
	void SetGeometry(const FImplicitObjectPtr& InGeometry) { GeometryParticles->SetGeometry(ParticleIdx, InGeometry); }
	void MergeGeometry(TArray<Chaos::FImplicitObjectPtr>&& Objects);

	// Set a flag on all Unions to indicate that any add/remove operations are errors.
	// This is called when the shapes are shared between GT/PT
	void LockGeometry();

	UE_DEPRECATED(5.4, "Use GetGeometry instead")
	TSerializablePtr<FImplicitObject> Geometry() const { check(false); return TSerializablePtr<FImplicitObject>(); }
	
	UE_DEPRECATED(5.4, "Use SetGeometry with FImplicitObjectPtr instead")
	void SetGeometry(TSerializablePtr<FImplicitObject> InGeometry) { check(false); }
	
	UE_DEPRECATED(5.4, "Use GeometryRef instead")
	TSharedPtr<const FImplicitObject, ESPMode::ThreadSafe> SharedGeometry() const { check(false); return nullptr; }

	UE_DEPRECATED(5.4, "Use SetGeometry with FImplicitObjectPtr instead")
	void SetSharedGeometry(TSharedPtr<const FImplicitObject, ESPMode::ThreadSafe> InGeometry) { check(false); }

	UE_DEPRECATED(5.4, "Use GetGeometry instead")
	const TSharedPtr<const FImplicitObject, ESPMode::ThreadSafe>& SharedGeometryLowLevel() const { check(false); static TSharedPtr<const FImplicitObject, ESPMode::ThreadSafe> DummyPtr(nullptr); return DummyPtr; }

	UE_DEPRECATED(5.4, "Use GetGeometry instead")
	const TUniquePtr<FImplicitObject>& DynamicGeometry() const { check(false);  static TUniquePtr<FImplicitObject> DummyPtr(nullptr); return DummyPtr; }

	UE_DEPRECATED(5.4, "Use SetGeometry with FImplicitObjectPtr instead")
	void SetDynamicGeometry(TUniquePtr<FImplicitObject>&& Unique) { check(false); }

	const FShapesArray& ShapesArray() const { return GeometryParticles->ShapesArray(ParticleIdx); }

	const FShapeInstanceArray& ShapeInstances() const { return GeometryParticles->ShapeInstances(ParticleIdx); }

	void RemoveShape(FPerShapeData* InShape);
	void RemoveShapesAtSortedIndices(const TArrayView<const int32>& InIndices);

	const TAABB<T, d>& LocalBounds() const { return GeometryParticles->LocalBounds(ParticleIdx); }
	void SetLocalBounds(const TAABB<T, d>& NewBounds) { GeometryParticles->LocalBounds(ParticleIdx) = NewBounds; }

	const TVector<T, d>& CCDAxisThreshold() const { return GeometryParticles->CCDAxisThreshold(ParticleIdx); }

	const TAABB<T, d>& WorldSpaceInflatedBounds() const { return GeometryParticles->WorldSpaceInflatedBounds(ParticleIdx); }

	/**
	 * @brief Update any cached state that depends on world-space transform
	 * This includes the world space bounds for the particle and all its shapes.
	*/
	void UpdateWorldSpaceState(const FRigidTransform3& WorldTransform, const FVec3& BoundsExpansion)
	{
		GeometryParticles->UpdateWorldSpaceState(ParticleIdx, WorldTransform, BoundsExpansion);
	}

	/**
	 * @brief Update any cached state that depends on world-space transform for a sweep between WorldTransform and WorldTransform + DeltaX
	 * @param WorldTransform The transform at the end of the sweep
	 * @param BoundsExpansion A uniform expansion applied to the bounds of the particle, but not the individual shapes
	 * @param DeltaX A directional expansion applied to the bounds of the particle, but not the individual shapes
	 * This includes the world space bounds for the particle and all its shapes. If DeltaX is not zero,
	 * the bounds will be equivalent to a union of the bounds at WorldTransform and WorldTransform + DeltaX.
	*/
	void UpdateWorldSpaceStateSwept(const FRigidTransform3& WorldTransform, const FVec3& BoundsExpansion, const FVec3& DeltaX)
	{
		GeometryParticles->UpdateWorldSpaceStateSwept(ParticleIdx, WorldTransform, BoundsExpansion, DeltaX);
	}

	bool HasBounds() const { return GeometryParticles->HasBounds(ParticleIdx); }
	void SetHasBounds(bool bHasBounds) { GeometryParticles->HasBounds(ParticleIdx) = bHasBounds; }

	FSpatialAccelerationIdx SpatialIdx() const { return GeometryParticles->SpatialIdx(ParticleIdx); }
	void SetSpatialIdx(FSpatialAccelerationIdx Idx) { GeometryParticles->SpatialIdx(ParticleIdx) = Idx; }

#if CHAOS_DEBUG_NAME
	const TSharedPtr<FString, ESPMode::ThreadSafe>& DebugName() const { return GeometryParticles->DebugName(ParticleIdx); }
	void SetDebugName(const TSharedPtr<FString, ESPMode::ThreadSafe>& InDebugName) { GeometryParticles->DebugName(ParticleIdx) = InDebugName; }
#endif
	const FString& GetDebugName() const { return GeometryParticles->GetDebugName(ParticleIdx); }

	EObjectStateType ObjectState() const;

	TGeometryParticle<T, d>* GTGeometryParticle() const { return GeometryParticles->GTGeometryParticle(ParticleIdx); }
	TGeometryParticle<T, d>*& GTGeometryParticle() { return GeometryParticles->GTGeometryParticle(ParticleIdx); }

	const IPhysicsProxyBase* PhysicsProxy() const { return GeometryParticles->PhysicsProxy(ParticleIdx); }
	IPhysicsProxyBase* PhysicsProxy() { return GeometryParticles->PhysicsProxy(ParticleIdx); }
	void SetPhysicsProxy(IPhysicsProxyBase* PhysicsProxy) { GeometryParticles->SetPhysicsProxy(ParticleIdx, PhysicsProxy); }

	const TKinematicGeometryParticleHandleImp<T, d, bPersistent>* CastToKinematicParticle() const;
	TKinematicGeometryParticleHandleImp<T, d, bPersistent>* CastToKinematicParticle();

	const TPBDRigidParticleHandleImp<T, d, bPersistent>* CastToRigidParticle() const;
	TPBDRigidParticleHandleImp<T, d, bPersistent>* CastToRigidParticle();

	const TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>* CastToClustered() const;
	TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>* CastToClustered();

	const TPBDGeometryCollectionParticleHandleImp<T, d, bPersistent>* CastToGeometryCollection() const;
	TPBDGeometryCollectionParticleHandleImp<T, d, bPersistent>* CastToGeometryCollection();

	const TGeometryParticleHandle<T, d>* Handle() const { return GetHandleHelper(this);}
	TGeometryParticleHandle<T, d>* Handle() { return GetHandleHelper(this); }
	
	// Useful for logging to indicate particle (this is locally unique among all particles)
	int32 GetHandleIdx() const { return HandleIdx; }

	bool Sleeping() const { return GeometryParticleSleeping(*this); }

	template <typename Container>
	const auto& AuxilaryValue(const Container& AuxContainer) const
	{
		return AuxContainer[HandleIdx];
	}

	template <typename Container>
	auto& AuxilaryValue(Container& AuxContainer)
	{
		return AuxContainer[HandleIdx];
	}

	EResimType ResimType() const { return GeometryParticles->ResimType(ParticleIdx); }

	void SetResimType(EResimType ResimType) { GeometryParticles->ResimType(ParticleIdx) = ResimType; }


	bool EnabledDuringResim() const { return GeometryParticles->EnabledDuringResim(ParticleIdx); }
	void SetEnabledDuringResim(bool bEnabledDuringResim) { GeometryParticles->EnabledDuringResim(ParticleIdx) = bEnabledDuringResim; }

	bool LightWeightDisabled() const { return GeometryParticles->LightWeightDisabled(ParticleIdx); }
	void SetLightWeightDisabled(bool bLightWeightDisabled) { GeometryParticles->LightWeightDisabled(ParticleIdx) = bLightWeightDisabled; }

	EGeometryParticleListMask ListMask() const
	{
		return GeometryParticles->ListMask(ParticleIdx);
	}

	bool IsInAnyList(const EGeometryParticleListMask ListMask) const
	{
		return (GeometryParticles->ListMask(ParticleIdx) & ListMask) != EGeometryParticleListMask::None;
	}

	bool IsInAllLists(const EGeometryParticleListMask ListMask) const
	{
		return (GeometryParticles->ListMask(ParticleIdx) & ListMask) == ListMask;
	}

	void AddToLists(const EGeometryParticleListMask ListMask)
	{
		GeometryParticles->ListMask(ParticleIdx) |= ListMask;
	}

	void RemoveFromLists(const EGeometryParticleListMask ListMask)
	{
		GeometryParticles->ListMask(ParticleIdx) &= ~ListMask;
	}

#if CHAOS_DETERMINISTIC
	FParticleID ParticleID() const { return GeometryParticles->ParticleID(ParticleIdx); }
	void SetParticleID(const FParticleID& ParticleID)
	{
		//When particles are created they are assigned a unique local index
		//This index cannot be used for replicated particles (which use a global index)
		//The global index (if used) is set on the game thread and comes over in the NonFrequent data
		//However, it's possible that the particle id was never set (not used), so it will come over as -1,-1
		//In this case we should continue to use the local index

		//TODO: find a better way to deal with this, shouldn't be at such a low level API

		if (ParticleID.GlobalID != INDEX_NONE || ParticleID.LocalID != INDEX_NONE)
		{
			GeometryParticles->ParticleID(ParticleIdx) = ParticleID;
		}
	}
#endif

	void MoveToSOA(TGeometryParticles<T, d>& ToSOA)
	{
		static_assert(bPersistent, "Cannot move particles from a transient handle");
		check(ToSOA.ParticleType() == Type);
		if (GeometryParticles != &ToSOA)
		{
			GeometryParticles->MoveToOtherParticles(ParticleIdx, ToSOA);
			if (static_cast<uint32>(ParticleIdx) < GeometryParticles->Size())
			{
				GeometryParticles->Handle(ParticleIdx)->ParticleIdx = ParticleIdx;
			}

			const int32 NewParticleIdx = ToSOA.Size() - 1;
			ParticleIdx = NewParticleIdx;

			// Update the particles' list mask to match its new container
			check(GeometryParticles->GetContainerListMask() != EGeometryParticleListMask::None);
			check(ToSOA.GetContainerListMask() != EGeometryParticleListMask::None);
			ToSOA.ListMask(ParticleIdx) &= ~GeometryParticles->GetContainerListMask();
			ToSOA.ListMask(ParticleIdx) |= ToSOA.GetContainerListMask();

			GeometryParticles = &ToSOA;
		}
	}

	static constexpr EParticleType StaticType() { return EParticleType::Static; }

	FString ToString() const;

	void Serialize(FChaosArchive& Ar)
	{
		THandleBase::Serialize(Ar);
		Ar << HandleIdx;
		GeometryParticles->SetHandle(ParticleIdx, this);
	}

	FWeakParticleHandle& WeakParticleHandle()
	{
		return GeometryParticles->WeakParticleHandle(ParticleIdx);
	}

	/**
	 * Only contains persistent constraints (joints, etc.), not transient constraints (collisions)
	*/
	FConstraintHandleArray& ParticleConstraints()
	{
		return GeometryParticles->ParticleConstraints(ParticleIdx);
	}

	/**
	 * Only contains persistent constraints (joints, etc.), not transient constraints (collisions)
	*/
	const FConstraintHandleArray& ParticleConstraints() const
	{
		return GeometryParticles->ParticleConstraints(ParticleIdx);
	}

	void AddConstraintHandle(FConstraintHandle* InConstraintHandle )
	{
		return GeometryParticles->AddConstraintHandle(ParticleIdx, InConstraintHandle);
	}

	void RemoveConstraintHandle(FConstraintHandle* InConstraintHandle)
	{
		return GeometryParticles->RemoveConstraintHandle(ParticleIdx, InConstraintHandle);
	}

	FParticleCollisions& ParticleCollisions()
	{
		return GeometryParticles->ParticleCollisions(ParticleIdx);
	}

	const FParticleCollisions& ParticleCollisions() const
	{
		return GeometryParticles->ParticleCollisions(ParticleIdx);
	}

	bool IsInConstraintGraph() const
	{
		return (GetConstraintGraphNode() != nullptr);
	}

	Private::FPBDIslandParticle* GetConstraintGraphNode() const
	{ 
		return GeometryParticles->ConstraintGraphNode(ParticleIdx);
	}

	void SetConstraintGraphNode(Private::FPBDIslandParticle* InNode)
	{
		GeometryParticles->ConstraintGraphNode(ParticleIdx) = InNode;
	}

	// Deprecated API
	UE_DEPRECATED(5.3, "Use GetConstraintGraphNode()") int32 ConstraintGraphIndex() const { return INDEX_NONE; }
	UE_DEPRECATED(5.3, "Use GetConstraintGraphNode()") void SetConstraintGraphIndex(const int32 InGraphIndex) {}

protected:

	friend TGeometryParticleHandles<T, d>;
	
	struct FInvalidFromTransient {};
	std::conditional_t<bPersistent, int32, FInvalidFromTransient> HandleIdx;	//Index into the handles array. This is useful for binding external attributes. Note the index can change
};

template<>
template<>
int32 TGeometryParticleHandleImp<FReal, 3, true>::GetPayload<int32>(int32 Idx);

template<>
template<>
int32 TGeometryParticleHandleImp<FReal, 3, false>::GetPayload<int32>(int32 Idx);



template <typename T, int d>
TGeometryParticleHandle<T, d>* GetHandleHelper(TTransientGeometryParticleHandle<T, d>* Handle)
{
	return Handle->GeometryParticles->Handle(Handle->ParticleIdx);
}
template <typename T, int d>
const TGeometryParticleHandle<T, d>* GetHandleHelper(const TTransientGeometryParticleHandle<T, d>* Handle)
{
	return Handle->GeometryParticles->Handle(Handle->ParticleIdx);
}

template <typename T, int d, bool bPersistent>
class TKinematicGeometryParticleHandleImp : public TGeometryParticleHandleImp<T,d, bPersistent>
{
public:
	using TGeometryParticleHandleImp<T, d, bPersistent>::ParticleIdx;
	using TGeometryParticleHandleImp<T, d, bPersistent>::KinematicGeometryParticles;
	using TGeometryParticleHandleImp<T, d, bPersistent>::Type;
	using TGeometryParticleHandleImp<T, d, bPersistent>::CastToRigidParticle;
	using TTransientHandle = TTransientKinematicGeometryParticleHandle<T, d>;
	using TSOAType = TKinematicGeometryParticles<T, d>;
	using TGeometryParticleHandleImp<T, d, bPersistent>::SetXR;
	
protected:
	friend class TGeometryParticleHandleImp<T, d, bPersistent>;
	//needed for serialization
	TKinematicGeometryParticleHandleImp()
		: TGeometryParticleHandleImp<T, d, bPersistent>()
	{
	}

	TKinematicGeometryParticleHandleImp(TSerializablePtr<TKinematicGeometryParticles<T, d>> Particles, int32 InIdx, int32 InGlobalIdx, const FKinematicGeometryParticleParameters& Params)
		: TGeometryParticleHandleImp<T, d, bPersistent>(TSerializablePtr<TGeometryParticles<T, d>>(Particles), InIdx, InGlobalIdx, Params)
	{
		KinematicGeometryParticleDefaultConstruct<T, d>(*this, Params);
	}
public:

	static TUniquePtr<TKinematicGeometryParticleHandleImp<T, d, bPersistent>> CreateParticleHandle(TSerializablePtr<TKinematicGeometryParticles<T, d>> InParticles, int32 InParticleIdx, int32 InHandleIdx, const FKinematicGeometryParticleParameters& Params = FKinematicGeometryParticleParameters())
	{
		return TGeometryParticleHandleImp<T, d, bPersistent>::CreateParticleHandleHelper(InParticles, InParticleIdx, InHandleIdx, Params);
	}

	TSerializablePtr <TKinematicGeometryParticleHandleImp<T, d, bPersistent>> ToSerializable() const
	{
		TSerializablePtr<TKinematicGeometryParticleHandleImp<T, d, bPersistent>> Serializable;
		Serializable.SetFromRawLowLevel(this);	//this is safe because CreateParticleHandle gives back a TUniquePtr
		return Serializable;
	}

	UE_DEPRECATED(5.4, "Use GetV instead")
	const TVector<T, d> V() const { return KinematicGeometryParticles->GetV(ParticleIdx); }
	UE_DEPRECATED(5.4, "Use SetV or GetV instead")
	TVector<T, d> V() { return KinematicGeometryParticles->GetV(ParticleIdx); }
	const TVector<T, d> GetV() const { return KinematicGeometryParticles->GetV(ParticleIdx); }
	void SetV(const TVector<T, d>& InV, bool bInvalidate = false) { KinematicGeometryParticles->SetV(ParticleIdx, InV); }
	const TVector<FRealSingle, d> GetVf() const { return KinematicGeometryParticles->GetVf(ParticleIdx); }
	void SetVf(const TVector<FRealSingle, d>& InV, bool bInvalidate = false) { KinematicGeometryParticles->SetVf(ParticleIdx, InV); }

	UE_DEPRECATED(5.4, "Use GetW instead")
	const TVector<T, d> W() const { return KinematicGeometryParticles->GetW(ParticleIdx); }
	UE_DEPRECATED(5.4, "Use SetW or GetW instead")
	TVector<T, d> W() { return KinematicGeometryParticles->GetW(ParticleIdx); }
	const TVector<T, d> GetW() const { return KinematicGeometryParticles->GetW(ParticleIdx); }
	void SetW(const TVector<T, d>& InW, bool bInvalidate = false) { KinematicGeometryParticles->SetW(ParticleIdx, InW); }
	const TVector<FRealSingle, d> GetWf() const { return KinematicGeometryParticles->GetWf(ParticleIdx); }
	void SetWf(const TVector<FRealSingle, d>& InW, bool bInvalidate = false) { KinematicGeometryParticles->SetWf(ParticleIdx, InW); }

	UE_DEPRECATED(5.4, "Use FPBDRigidsEvolutionGBF::SetParticleVelocitied or (SetV, SetW if that is not appropriate)")
	void SetVelocities(const FParticleVelocities& Velocities)
	{
		SetV(Velocities.V());
		SetW(Velocities.W());
	}

	void SetKinematicTarget(const FKinematicTarget& InKinematicTarget, bool bInvalidate = true)
	{
		KinematicGeometryParticles->KinematicTarget(ParticleIdx) = InKinematicTarget;
	}

	const FKinematicTarget& KinematicTarget() const { return KinematicGeometryParticles->KinematicTarget(ParticleIdx); }
	FKinematicTarget& KinematicTarget() { return KinematicGeometryParticles->KinematicTarget(ParticleIdx); }

	//Really only useful when using a transient handle
	const TKinematicGeometryParticleHandleImp<T, d, true>* Handle() const { return KinematicGeometryParticles->Handle(ParticleIdx); }
	TKinematicGeometryParticleHandleImp<T, d, true>* Handle() { return KinematicGeometryParticles->Handle(ParticleIdx); }

	EObjectStateType ObjectState() const;
	static constexpr EParticleType StaticType() { return EParticleType::Kinematic; }
};

template <typename T, int d, bool bPersistent>
class TPBDRigidParticleHandleImp : public TKinematicGeometryParticleHandleImp<T, d, bPersistent>
{
public:
	using TGeometryParticleHandleImp<T, d, bPersistent>::ParticleCollisions;
	using TGeometryParticleHandleImp<T, d, bPersistent>::ParticleConstraints;
	using TGeometryParticleHandleImp<T, d, bPersistent>::ParticleIdx;
	using TGeometryParticleHandleImp<T, d, bPersistent>::PBDRigidParticles;
	using TGeometryParticleHandleImp<T, d, bPersistent>::SetX;
	using TGeometryParticleHandleImp<T, d, bPersistent>::SetR;
	using TGeometryParticleHandleImp<T, d, bPersistent>::Type;
	using TKinematicGeometryParticleHandleImp<T, d, bPersistent>::GetV;
	using TKinematicGeometryParticleHandleImp<T, d, bPersistent>::GetW;

	using TTransientHandle = TTransientPBDRigidParticleHandle<T, d>;
	using TSOAType = TPBDRigidParticles<T, d>;

protected:
	friend class TGeometryParticleHandleImp<T, d, bPersistent>;

	//needed for serialization
	TPBDRigidParticleHandleImp()
		: TKinematicGeometryParticleHandleImp<T, d, bPersistent>()
	{
	}

	TPBDRigidParticleHandleImp<T, d, bPersistent>(TSerializablePtr<TPBDRigidParticles<T, d>> Particles, int32 InIdx, int32 InGlobalIdx, const FPBDRigidParticleParameters& Params = FPBDRigidParticleParameters())
		: TKinematicGeometryParticleHandleImp<T, d, bPersistent>(TSerializablePtr<TKinematicGeometryParticles<T, d>>(Particles), InIdx, InGlobalIdx, Params)
	{
		PBDRigidParticleDefaultConstruct<T, d>(*this, Params);
		SetCollisionConstraintFlags(0);
		SetDisabled(Params.bDisabled);
		SetPreVf(this->GetVf());
		SetPreWf(this->GetWf());
		SetSolverBodyIndex(INDEX_NONE);
		SetP(this->GetX());
		SetQf(this->GetRf());
		SetVSmooth(this->GetV());
		SetWSmooth(this->GetW());
		SetAcceleration(TVector<T, d>(0));
		SetAngularAcceleration(TVector<T, d>(0));
		SetObjectStateLowLevel(Params.bStartSleeping ? EObjectStateType::Sleeping : EObjectStateType::Dynamic);
		SetPreObjectStateLowLevel(ObjectState());
		SetSleepType(ESleepType::MaterialSleep);
		SetSleepCounter(0);
		SetDisableCounter(0);
		SetInvIConditioning(TVec3<FRealSingle>(1));
	}
public:

	static TUniquePtr<TPBDRigidParticleHandleImp<T, d, bPersistent>> CreateParticleHandle(TSerializablePtr<TPBDRigidParticles<T, d>> InParticles, int32 InParticleIdx, int32 InHandleIdx, const FPBDRigidParticleParameters& Params = FPBDRigidParticleParameters())
	{
		return TGeometryParticleHandleImp<T, d, bPersistent>::CreateParticleHandleHelper(InParticles, InParticleIdx, InHandleIdx, Params);
	}

	TSerializablePtr <TPBDRigidParticleHandleImp<T, d, bPersistent>> ToSerializable() const
	{
		TSerializablePtr<TPBDRigidParticleHandleImp<T, d, bPersistent>> Serializable;
		Serializable.SetFromRawLowLevel(this);	//this is safe because CreateParticleHandle gives back a TUniquePtr
		return Serializable;
	}

	operator TPBDRigidParticleHandleImp<T, d, false>& () { return reinterpret_cast<TPBDRigidParticleHandleImp<T, d, false>&>(*this); }

	bool IsKinematic() const { return ObjectState() == EObjectStateType::Kinematic; }
	bool IsDynamic() const { return (ObjectState() == EObjectStateType::Dynamic) || (ObjectState() == EObjectStateType::Sleeping); }

	const TUniquePtr<TBVHParticles<T, d>>& CollisionParticles() const { return PBDRigidParticles->CollisionParticles(ParticleIdx); }
	TUniquePtr<TBVHParticles<T, d>>& CollisionParticles() { return PBDRigidParticles->CollisionParticles(ParticleIdx); }

	int32 CollisionParticlesSize() const { return PBDRigidParticles->CollisionParticlesSize(ParticleIdx); }
	void CollisionParticlesInitIfNeeded() { PBDRigidParticles->CollisionParticlesInitIfNeeded(ParticleIdx); }
	void SetCollisionParticles(TParticles<T, d>&& Points) { PBDRigidParticles->SetCollisionParticles(ParticleIdx, MoveTemp(Points)); }

	int32 CollisionGroup() const { return PBDRigidParticles->CollisionGroup(ParticleIdx); }
	int32& CollisionGroup() { return PBDRigidParticles->CollisionGroup(ParticleIdx); }
	void SetCollisionGroup(const int32 InCollisionGroup) { PBDRigidParticles->CollisionGroup(ParticleIdx) = InCollisionGroup; }

	bool HasCollisionConstraintFlag(const ECollisionConstraintFlags Flag) const { return  PBDRigidParticles->HasCollisionConstraintFlag(Flag, ParticleIdx); }
	void AddCollisionConstraintFlag(const ECollisionConstraintFlags Flag) { PBDRigidParticles->AddCollisionConstraintFlag(Flag, ParticleIdx); }
	void RemoveCollisionConstraintFlag(const ECollisionConstraintFlags Flag) { PBDRigidParticles->RemoveCollisionConstraintFlag(Flag, ParticleIdx); }
	void SetCollisionConstraintFlags(const uint32 Flags) { PBDRigidParticles->SetCollisionConstraintFlags(ParticleIdx, Flags); }
	uint32 CollisionConstraintFlags() const { return PBDRigidParticles->CollisionConstraintFlags(ParticleIdx); }

	bool Disabled() const { return PBDRigidParticles->Disabled(ParticleIdx); }
	//UE_DEPRECATED(5.3, "This method should not be used anymore. SetDisabled should be used instead.")
	bool& Disabled() { return PBDRigidParticles->DisabledRef(ParticleIdx); }

	// See Comment on TRigidParticle::SetDisabledLowLevel. State changes in Evolution should accompany this call.
	void SetDisabledLowLevel(bool disabled) { PBDRigidParticles->SetDisabledLowLevel(ParticleIdx, disabled); }
	void SetDisabled(const bool InDisabled) { PBDRigidParticles->DisabledRef(ParticleIdx) = InDisabled; }

	UE_DEPRECATED(5.4, "Use GetPreV instead")
	const TVector<T, d> PreV() const { return PBDRigidParticles->GetPreV(ParticleIdx); }
	UE_DEPRECATED(5.4, "Use GetPreV or SetPreV instead")
	TVector<T, d> PreV() { return PBDRigidParticles->GetPreV(ParticleIdx); }
	const TVector<T, d> GetPreV() const { return PBDRigidParticles->GetPreV(ParticleIdx); }
	void SetPreV(const TVector<T, d>& InPreV) { PBDRigidParticles->SetPreV(ParticleIdx, InPreV); }
	const TVector<FRealSingle, d> GetPreVf() const { return PBDRigidParticles->GetPreVf(ParticleIdx); }
	void SetPreVf(const TVector<FRealSingle, d>& InPreV) { PBDRigidParticles->SetPreVf(ParticleIdx, InPreV); }

	UE_DEPRECATED(5.4, "Use GetPreW instead")
	const TVector<T, d> PreW() const { return PBDRigidParticles->GetPreW(ParticleIdx); }
	UE_DEPRECATED(5.4, "Use GetPreW or SetPreW instead")
	TVector<T, d> PreW() { return PBDRigidParticles->GetPreW(ParticleIdx); }
	const TVector<T, d> GetPreW() const { return PBDRigidParticles->GetPreW(ParticleIdx); }
	void SetPreW(const TVector<T, d>& InPreW) { PBDRigidParticles->SetPreW(ParticleIdx, InPreW); }
	const TVector<FRealSingle, d> GetPreWf() const { return PBDRigidParticles->GetPreWf(ParticleIdx); }
	void SetPreWf(const TVector<FRealSingle, d>& InPreW) { PBDRigidParticles->SetPreWf(ParticleIdx, InPreW); }

	int32 SolverBodyIndex() const { return PBDRigidParticles->SolverBodyIndex(ParticleIdx); }
	void SetSolverBodyIndex(const int32 InSolverBodyIndex) { PBDRigidParticles->SetSolverBodyIndex(ParticleIdx, InSolverBodyIndex); }

	UE_DEPRECATED(5.4, "Use GetP instead")
	const TVector<T, d>& P() const { return PBDRigidParticles->GetP(ParticleIdx); }
	UE_DEPRECATED(5.4, "Use GetP or SetP instead")
	TVector<T, d>& P() { 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return PBDRigidParticles->P(ParticleIdx); 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	const TVector<T, d>& GetP() const { return PBDRigidParticles->GetP(ParticleIdx); }
	void SetP(const TVector<T, d>& InP) { PBDRigidParticles->SetP(ParticleIdx, InP); }

	UE_DEPRECATED(5.4, "Use GetQ instead")
	const TRotation<T, d> Q() const { return PBDRigidParticles->GetQ(ParticleIdx); }
	UE_DEPRECATED(5.4, "Use GetQ or SetQ instead")
	TRotation<T, d> Q() { return PBDRigidParticles->GetQ(ParticleIdx); }
	const TRotation<T, d> GetQ() const { return PBDRigidParticles->GetQ(ParticleIdx); }
	void SetQ(const TRotation<T, d>& InQ) { PBDRigidParticles->SetQ(ParticleIdx, InQ); }
	const TRotation<FRealSingle, d> GetQf() const { return PBDRigidParticles->GetQf(ParticleIdx); }
	void SetQf(const TRotation<FRealSingle, d>& InQ) { PBDRigidParticles->SetQf(ParticleIdx, InQ); }

	// World-space center of mass position
	const TVector<T, d> XCom() const { return PBDRigidParticles->XCom(ParticleIdx); }
	const TVector<T, d> PCom() const { return PBDRigidParticles->PCom(ParticleIdx); }

	// World-space center of mass rotation
	const TRotation<T, d> RCom() const { return PBDRigidParticles->RCom(ParticleIdx); }
	const TRotation<T, d> QCom() const { return PBDRigidParticles->QCom(ParticleIdx); }

	// Initialize the transform (sets X,R and  P,Q)
	void InitTransform(const FVec3& InP, const FRotation3& InQ)
	{
		SetX(InP);
		SetR(InQ);
		SetP(InP);
		SetQ(InQ);
	}

	// Set world-space center of mass transform
	void SetTransformPQCom(const TVector<T, d>& InPCom, const TRotation<T, d>& InQCom) { PBDRigidParticles->SetTransformPQCom(ParticleIdx, InPCom, InQCom); }

	FRigidTransform3 GetTransformPQ() const { return FRigidTransform3(GetP(), GetQ()); }

	FRigidTransform3 GetTransformXRCom() const { return FRigidTransform3(XCom(), RCom()); }

	FRigidTransform3 GetTransformPQCom() const { return FRigidTransform3(PCom(), QCom()); }

	const TVector<T, d>& VSmooth() const { return PBDRigidParticles->VSmooth(ParticleIdx); }
	TVector<T, d>& VSmooth() { return PBDRigidParticles->VSmooth(ParticleIdx); }
	void SetVSmooth(const TVector<T, d>& InVSmooth) { PBDRigidParticles->VSmooth(ParticleIdx) = InVSmooth; }

	const TVector<T, d>& WSmooth() const { return PBDRigidParticles->WSmooth(ParticleIdx); }
	TVector<T, d>& WSmooth() { return PBDRigidParticles->WSmooth(ParticleIdx); }
	void SetWSmooth(const TVector<T, d>& InWSmooth) { PBDRigidParticles->WSmooth(ParticleIdx) = InWSmooth; }

	const TVector<T, d>& Acceleration() const { return PBDRigidParticles->Acceleration(ParticleIdx); }
	TVector<T, d>& Acceleration() { return PBDRigidParticles->Acceleration(ParticleIdx); }
	void SetAcceleration(const TVector<T, d>& InAcceleration) { PBDRigidParticles->Acceleration(ParticleIdx) = InAcceleration; }

	void AddForce(const TVector<T, d>& InF, bool bInvalidate = true)
	{
		SetAcceleration(Acceleration() + InF * InvM());
	}

	const TVector<T, d>& AngularAcceleration() const { return PBDRigidParticles->AngularAcceleration(ParticleIdx); }
	TVector<T, d>& AngularAcceleration() { return PBDRigidParticles->AngularAcceleration(ParticleIdx); }
	void SetAngularAcceleration(const TVector<T, d>& InAngularAcceleration) { PBDRigidParticles->AngularAcceleration(ParticleIdx) = InAngularAcceleration; }

	CHAOS_API void AddTorque(const TVector<T, d>& InTorque, bool bInvalidate = true);
	CHAOS_API void SetTorque(const TVector<T, d>& InTorque, bool bInvalidate = true);

	const TVector<T, d>& LinearImpulseVelocity() const { return PBDRigidParticles->LinearImpulseVelocity(ParticleIdx); }
	TVector<T, d>& LinearImpulseVelocity() { return PBDRigidParticles->LinearImpulseVelocity(ParticleIdx); }
	void SetLinearImpulseVelocity(const TVector<T, d>& InLinearImpulseVelocity, bool bInvalidate = false) { PBDRigidParticles->LinearImpulseVelocity(ParticleIdx) = InLinearImpulseVelocity; }

	const TVector<T, d>& AngularImpulseVelocity() const { return PBDRigidParticles->AngularImpulseVelocity(ParticleIdx); }
	TVector<T, d>& AngularImpulseVelocity() { return PBDRigidParticles->AngularImpulseVelocity(ParticleIdx); }
	void SetAngularImpulseVelocity(const TVector<T, d>& InAngularImpulseVelocity, bool bInvalidate = false) { PBDRigidParticles->AngularImpulseVelocity(ParticleIdx) = InAngularImpulseVelocity; }

	void SetDynamics(const FParticleDynamics& Dynamics)
	{
		SetAcceleration(Dynamics.Acceleration());
		SetAngularAcceleration(Dynamics.AngularAcceleration());
		SetLinearImpulseVelocity(Dynamics.LinearImpulseVelocity());
		SetAngularImpulseVelocity(Dynamics.AngularImpulseVelocity());
	}

	void SetMassProps(const FParticleMassProps& Props)
	{
		SetCenterOfMass(Props.CenterOfMass());
		SetRotationOfMass(Props.RotationOfMass());
		SetI(Props.I());
		SetInvI(Props.InvI());
		SetM(Props.M());
		SetInvM(Props.InvM());

		SetInertiaConditioningDirty();
	}

	UE_DEPRECATED(5.2, "Moved to FPBDRigidsSolver")
	void SetDynamicMisc(const FParticleDynamicMisc& DynamicMisc, FPBDRigidsEvolutionBase& Evolution) {}

	void ResetSmoothedVelocities()
	{
		SetVSmooth(GetV());
		SetWSmooth(GetW());
	}

	// Get the raw inertia. @see ConditionedInvI()
	const TVec3<FRealSingle>& I() const { return PBDRigidParticles->I(ParticleIdx); }
	TVec3<FRealSingle>& I() { return PBDRigidParticles->I(ParticleIdx); }
	void SetI(const TVec3<FRealSingle>& InI) { PBDRigidParticles->I(ParticleIdx) = InI; }

	// Get the raw inverse inertia. @see ConditionedInvI()
	const TVec3<FRealSingle>& InvI() const { return PBDRigidParticles->InvI(ParticleIdx); }
	TVec3<FRealSingle>& InvI() { return PBDRigidParticles->InvI(ParticleIdx); }
	void SetInvI(const TVec3<FRealSingle>& InInvI) { PBDRigidParticles->InvI(ParticleIdx) = InInvI; }

	T M() const { return PBDRigidParticles->M(ParticleIdx); }
	T& M() { return PBDRigidParticles->M(ParticleIdx); }
	void SetM(const T& InM) { PBDRigidParticles->M(ParticleIdx) = InM; }

	T InvM() const { return PBDRigidParticles->InvM(ParticleIdx); }
	T& InvM() { return PBDRigidParticles->InvM(ParticleIdx); }
	void SetInvM(const T& InInvM) { PBDRigidParticles->InvM(ParticleIdx) = InInvM; }

	const TVector<T,d>& CenterOfMass() const { return PBDRigidParticles->CenterOfMass(ParticleIdx); }
	void SetCenterOfMass(const TVector<T,d>& InCenterOfMass, bool bInvalidate = false) { PBDRigidParticles->CenterOfMass(ParticleIdx) = InCenterOfMass; }

	const TRotation<T,d>& RotationOfMass() const { return PBDRigidParticles->RotationOfMass(ParticleIdx); }
	void SetRotationOfMass(const TRotation<T,d>& InRotationOfMass, bool bInvalidate = false) { PBDRigidParticles->RotationOfMass(ParticleIdx) = InRotationOfMass; }

	// Get the inertia conditioning scales. This is a scale applied to the inverse inertia for use by the constraint solvers to improve stability
	// and is calculated based on the attached joints and potential collision positions (approximated by object size)
	const TVec3<FRealSingle>& InvIConditioning() const { return PBDRigidParticles->InvIConditioning(ParticleIdx); }
	void SetInvIConditioning(const TVec3<FRealSingle>& InInvIConditioning) { PBDRigidParticles->InvIConditioning(ParticleIdx) = InInvIConditioning; }

	// Get the conditioned inertia for use in constraint solvers
	TVec3<FRealSingle> ConditionedInvI() const { return InvIConditioning() * InvI(); }
	TVec3<FRealSingle> ConditionedI() const { return I() / InvIConditioning(); }

	T LinearEtherDrag() const { return PBDRigidParticles->LinearEtherDrag(ParticleIdx); }
	T& LinearEtherDrag() { return PBDRigidParticles->LinearEtherDrag(ParticleIdx); }
	void SetLinearEtherDrag(const T& InLinearEtherDrag) { PBDRigidParticles->LinearEtherDrag(ParticleIdx) = InLinearEtherDrag; }

	T AngularEtherDrag() const { return PBDRigidParticles->AngularEtherDrag(ParticleIdx); }
	T& AngularEtherDrag() { return PBDRigidParticles->AngularEtherDrag(ParticleIdx); }
	void SetAngularEtherDrag(const T& InAngularEtherDrag) { PBDRigidParticles->AngularEtherDrag(ParticleIdx) = InAngularEtherDrag; }

	T MaxLinearSpeedSq() const { return PBDRigidParticles->MaxLinearSpeedSq(ParticleIdx); }
	T& MaxLinearSpeedSq() { return PBDRigidParticles->MaxLinearSpeedSq(ParticleIdx); }
	void SetMaxLinearSpeedSq(const T& InMaxLinearSpeed) { PBDRigidParticles->MaxLinearSpeedSq(ParticleIdx) = InMaxLinearSpeed; }

	T MaxAngularSpeedSq() const { return PBDRigidParticles->MaxAngularSpeedSq(ParticleIdx); }
	T& MaxAngularSpeedSq() { return PBDRigidParticles->MaxAngularSpeedSq(ParticleIdx); }
	void SetMaxAngularSpeedSq(const T& InMaxAngularSpeed) { PBDRigidParticles->MaxAngularSpeedSq(ParticleIdx) = InMaxAngularSpeed; }

	FRealSingle InitialOverlapDepenetrationVelocity() const { return PBDRigidParticles->InitialOverlapDepenetrationVelocity(ParticleIdx); }
	void SetInitialOverlapDepenetrationVelocity(FRealSingle InVel) { PBDRigidParticles->InitialOverlapDepenetrationVelocity(ParticleIdx) = InVel; }

	inline FRealSingle SleepThresholdMultiplier() const { return PBDRigidParticles->SleepThresholdMultiplier(ParticleIdx); }
	inline void SetSleepThresholdMultiplier(FRealSingle Multiplier) { PBDRigidParticles->SleepThresholdMultiplier(ParticleIdx) = Multiplier; }

	EObjectStateType ObjectState() const { return PBDRigidParticles->ObjectState(ParticleIdx); }
	EObjectStateType PreObjectState() const { return PBDRigidParticles->PreObjectState(ParticleIdx); }

	void SetObjectStateLowLevel(EObjectStateType InState) { PBDRigidParticles->SetObjectState(ParticleIdx, InState); }
	void SetPreObjectStateLowLevel(EObjectStateType InState) { PBDRigidParticles->PreObjectState(ParticleIdx) = InState; }
	
	bool IsSleeping() const { return ObjectState() == EObjectStateType::Sleeping; }
	bool WasSleeping() const { return PreObjectState() == EObjectStateType::Sleeping; }

	bool Sleeping() const { return PBDRigidParticles->Sleeping(ParticleIdx); }
	void SetSleeping(bool bSleeping) { PBDRigidParticles->SetSleeping(ParticleIdx, bSleeping); }

	//Really only useful when using a transient handle
	const TPBDRigidParticleHandleImp<T, d, true>* Handle() const { return PBDRigidParticles->Handle(ParticleIdx); }
	TPBDRigidParticleHandleImp<T, d, true>* Handle() { return PBDRigidParticles->Handle(ParticleIdx); }

	inline FRigidParticleControlFlags ControlFlags() const
	{ 
		return PBDRigidParticles->ControlFlags(ParticleIdx);
	}
	
	// NOTE: ControlFlags should not be changed by the solver during the tick. These are externally controlled settings.
	inline void SetControlFlags(const FRigidParticleControlFlags Flags)
	{
		PBDRigidParticles->ControlFlags(ParticleIdx) = Flags;
	}

	inline bool GravityEnabled() const
	{ 
		return ControlFlags().GetGravityEnabled();
	}

	inline void SetGravityEnabled(bool bEnabled)
	{ 
		PBDRigidParticles->ControlFlags(ParticleIdx).SetGravityEnabled(bEnabled);
	}

	inline int32 GravityGroupIndex() const
	{
		return ControlFlags().GetGravityGroupIndex();
	}

	inline void SetGravityGroupIndex(int32 GravityGroupIndex)
	{
		PBDRigidParticles->ControlFlags(ParticleIdx).SetGravityGroupIndex(GravityGroupIndex);
	}

	inline bool UpdateKinematicFromSimulation() const
	{
		return ControlFlags().GetUpdateKinematicFromSimulation();
	}

	inline void SetUpdateKinematicFromSimulation(bool bUpdateKinematicFromSimulation)
	{
		PBDRigidParticles->ControlFlags(ParticleIdx).SetUpdateKinematicFromSimulation(bUpdateKinematicFromSimulation);
	}

	inline bool GyroscopicTorqueEnabled() const
	{
		return ControlFlags().GetGyroscopicTorqueEnabled();
	}

	inline void SetGyroscopicTorqueEnabled(bool bEnabled)
	{
		PBDRigidParticles->ControlFlags(ParticleIdx).SetGyroscopicTorqueEnabled(bEnabled);
	}

	inline bool CCDEnabled() const
	{
		return ControlFlags().GetCCDEnabled();
	}

	inline void SetCCDEnabled(bool bEnabled)
	{
		PBDRigidParticles->ControlFlags(ParticleIdx).SetCCDEnabled(bEnabled);
	}

	inline bool MACDEnabled() const
	{
		return ControlFlags().GetMACDEnabled();
	}

	inline void SetMACDEnabled(bool bEnabled)
	{
		PBDRigidParticles->ControlFlags(ParticleIdx).SetMACDEnabled(bEnabled);
	}

	inline bool PartialIslandSleepAllowed() const
	{
		return ControlFlags().GetPartialIslandSleepAllowed();
	}

	inline void SetPartialIslandSleepAllowed(bool bEnabled)
	{
		PBDRigidParticles->ControlFlags(ParticleIdx).SetPartialIslandSleepAllowed(bEnabled);
	}

	inline Private::FIterationSettings IterationSettings() const
	{
		return PBDRigidParticles->ParticleIterationCounts(ParticleIdx);
	}

	inline void SetIterationSettings(const Private::FIterationSettings& IterationSettingsIn)
	{
		PBDRigidParticles->ParticleIterationCounts(ParticleIdx) = IterationSettingsIn;
	}

	inline void SetPositionSolverIterations(const int32 PositionIterationsIn)
	{
		PBDRigidParticles->ParticleIterationCounts(ParticleIdx).SetNumPositionIterations(PositionIterationsIn);
	}

	inline void SetVelocitySolverIterations(const int32 VelocityIterationsIn)
	{
		PBDRigidParticles->ParticleIterationCounts(ParticleIdx).SetNumVelocityIterations(VelocityIterationsIn);
	}

	inline void SetProjectionSolverIterations(const int32 ProjectionIterationsIn)
	{
		PBDRigidParticles->ParticleIterationCounts(ParticleIdx).SetNumProjectionIterations(ProjectionIterationsIn);
	}

	inline bool OneWayInteraction() const
	{ 
		return ControlFlags().GetOneWayInteractionEnabled();
	}

	inline void SetOneWayInteraction(bool bEnabled)
	{ 
		PBDRigidParticles->ControlFlags(ParticleIdx).SetOneWayInteractionEnabled(bEnabled);
	}

	inline bool InertiaConditioningEnabled() const
	{
		return ControlFlags().GetInertiaConditioningEnabled();
	}

	inline void SetInertiaConditioningEnabled(bool bEnabled)
	{
		// NOTE: We still set this flag even for kinematics because they may change to dynamic later. However we
		// won't actually calculate the inertia until it gets changed to dynamic (which will also set the dirty flag)
		if (bEnabled != InertiaConditioningEnabled())
		{
			PBDRigidParticles->ControlFlags(ParticleIdx).SetInertiaConditioningEnabled(bEnabled);
			SetInertiaConditioningDirty();
		}
	}

	inline bool InertiaConditioningDirty()
	{
		return PBDRigidParticles->TransientFlags(ParticleIdx).GetInertiaConditioningDirty();
	}

	inline void SetInertiaConditioningDirty()
	{
		PBDRigidParticles->TransientFlags(ParticleIdx).SetInertiaConditioningDirty();
	}

	inline void ClearInertiaConditioningDirty()
	{
		PBDRigidParticles->TransientFlags(ParticleIdx).ClearInertiaConditioningDirty();
	}

	inline bool UseIgnoreCollisionManager() const
	{
		return PBDRigidParticles->TransientFlags(ParticleIdx).GetUseIgnoreCollisionManager();
	}

	inline void SetUseIgnoreCollisionManager()
	{
		PBDRigidParticles->TransientFlags(ParticleIdx).SetUseIgnoreCollisionManager();
	}

	inline void ClearUseIgnoreCollisionManager()
	{
		PBDRigidParticles->TransientFlags(ParticleIdx).ClearUseIgnoreCollisionManager();
	}

	inline bool IsMovingKinematic() const
	{
		return IsKinematic() && PBDRigidParticles->TransientFlags(ParticleIdx).GetIsMovingKinematic();
	}

	inline void SetIsMovingKinematic()
	{
		PBDRigidParticles->TransientFlags(ParticleIdx).SetIsMovingKinematic();
	}

	inline void ClearIsMovingKinematic()
	{
		PBDRigidParticles->TransientFlags(ParticleIdx).ClearIsMovingKinematic();
	}

	ESleepType SleepType() const { return PBDRigidParticles->SleepType(ParticleIdx);}

	void SetSleepType(ESleepType SleepType){ PBDRigidParticles->SetSleepType(ParticleIdx, SleepType); }

	int8 SleepCounter() const
	{
		return PBDRigidParticles->SleepCounter(ParticleIdx);
	}

	void SetSleepCounter(int8 SleepCounter)
	{
		PBDRigidParticles->SleepCounter(ParticleIdx) = SleepCounter;
	}

	int8 DisableCounter() const
	{
		return PBDRigidParticles->DisableCounter(ParticleIdx);
	}

	void SetDisableCounter(int8 DisableCounter)
	{
		PBDRigidParticles->DisableCounter(ParticleIdx) = DisableCounter;
	}

	static constexpr EParticleType StaticType() { return EParticleType::Rigid; }

	// Deprecated API
	UE_DEPRECATED(5.3, "No longer supported") int32 IslandIndex() const { return INDEX_NONE; }
	UE_DEPRECATED(5.3, "No longer supported") int32& IslandIndex() { static int32 Dummy = INDEX_NONE; return Dummy; }
	UE_DEPRECATED(5.3, "No longer supported") void SetIslandIndex(const int32 InIslandIndex) {}

};

template <typename T, int d, bool bPersistent>
void TGeometryParticleHandleImp<T,d,bPersistent>::SetXR(const FParticlePositionRotation& XR)
{
	SetX(XR.X());
	SetR(XR.R());
	if(auto Rigid = CastToRigidParticle())
	{
		Rigid->SetP(GetX());
		Rigid->SetQf(GetRf());
	}
}

template <typename T, int d, bool bPersistent>
class TPBDRigidClusteredParticleHandleImp : public TPBDRigidParticleHandleImp<T, d, bPersistent>
{
public:
	using TGeometryParticleHandleImp<T, d, bPersistent>::ParticleIdx;
	using TGeometryParticleHandleImp<T, d, bPersistent>::PBDRigidClusteredParticles;
	using TGeometryParticleHandleImp<T, d, bPersistent>::Type;
	using TTransientHandle = TTransientPBDRigidParticleHandle<T, d>;
	using TSOAType = TPBDRigidClusteredParticles<T, d>;

protected:
	friend class TGeometryParticleHandleImp<T, d, bPersistent>;
	//needed for serialization
	TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>()
	: TPBDRigidParticleHandleImp<T, d, bPersistent>()
	{
	}

	TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>(TSerializablePtr<TPBDRigidClusteredParticles<T, d>> Particles, int32 InIdx, int32 InGlobalIdx, const FPBDRigidParticleParameters& Params)
		: TPBDRigidParticleHandleImp<T, d, bPersistent>(TSerializablePtr<TPBDRigidParticles<T, d>>(Particles), InIdx, InGlobalIdx, Params)
	{
		PBDRigidClusteredParticleDefaultConstruct<T, d>(*this, Params);
	}
public:

	static TUniquePtr<TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>> CreateParticleHandle(TSerializablePtr<TPBDRigidClusteredParticles<T, d>> InParticles, int32 InParticleIdx, int32 InHandleIdx, const FPBDRigidParticleParameters& Params = FPBDRigidParticleParameters())
	{
		return TGeometryParticleHandleImp<T, d, bPersistent>::CreateParticleHandleHelper(InParticles, InParticleIdx, InHandleIdx, Params);
	}

	TSerializablePtr <TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>> ToSerializable() const
	{
		TSerializablePtr<TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>> Serializable;
		Serializable.SetFromRawLowLevel(this);	//this is safe because CreateParticleHandle gives back a TUniquePtr
		return Serializable;
	}

	const TSet<IPhysicsProxyBase*>& PhysicsProxies() const { return PBDRigidClusteredParticles->PhysicsProxies(ParticleIdx); }
	void AddPhysicsProxy(IPhysicsProxyBase* PhysicsProxy) { PBDRigidClusteredParticles->PhysicsProxies(ParticleIdx).Add(PhysicsProxy); }
	void RemovePhysicsProxy(IPhysicsProxyBase* PhysicsProxy) { PBDRigidClusteredParticles->PhysicsProxies(ParticleIdx).Remove(PhysicsProxy); }
	void ClearPhysicsProxies() { PBDRigidClusteredParticles->PhysicsProxies(ParticleIdx).Empty(); }
	
	void SetClusterId(const ClusterId& Id) { PBDRigidClusteredParticles->ClusterIds(ParticleIdx) = Id; }
	const ClusterId& ClusterIds() const { return PBDRigidClusteredParticles->ClusterIds(ParticleIdx); }
	ClusterId& ClusterIds() { return PBDRigidClusteredParticles->ClusterIds(ParticleIdx); }

	FPBDRigidClusteredParticleHandle* Parent() const { return (ClusterIds().Id)? ClusterIds().Id->CastToClustered(): nullptr; }

	const TRigidTransform<T,d>& ChildToParent() const { return PBDRigidClusteredParticles->ChildToParent(ParticleIdx); }
	TRigidTransform<T,d>& ChildToParent() { return PBDRigidClusteredParticles->ChildToParent(ParticleIdx); }
	void SetChildToParent(const TRigidTransform<T, d>& Xf) { PBDRigidClusteredParticles->ChildToParent(ParticleIdx) = Xf; }

	const int32& ClusterGroupIndex() const { return PBDRigidClusteredParticles->ClusterGroupIndex(ParticleIdx); }
	int32& ClusterGroupIndex() { return PBDRigidClusteredParticles->ClusterGroupIndex(ParticleIdx); }
	void SetClusterGroupIndex(const int32 Idx) { PBDRigidClusteredParticles->ClusterGroupIndex(ParticleIdx) = Idx; }

	bool InternalCluster() const { return PBDRigidClusteredParticles->RigidClusteredFlags(ParticleIdx).GetInternalCluster(); }
	void SetInternalCluster(bool bValue) { PBDRigidClusteredParticles->RigidClusteredFlags(ParticleIdx).SetInternalCluster(bValue);	}

	const FImplicitObjectUnionClusteredPtr& GetChildrenSpatial() const { return PBDRigidClusteredParticles->GetChildrenSpatial(ParticleIdx); }
	FImplicitObjectUnionClusteredPtr& GetChildrenSpatial() { return PBDRigidClusteredParticles->GetChildrenSpatial(ParticleIdx); }
	void SetChildrenSpatial(FImplicitObjectUnionClusteredPtr& Obj) { PBDRigidClusteredParticles->GetChildrenSpatial(ParticleIdx) = Obj; }

	UE_DEPRECATED(5.4, "Use GetChildrenSpatial instead")
	const TUniquePtr<FImplicitObjectUnionClustered>& ChildrenSpatial() const
	{
		check(false);
		static TUniquePtr<FImplicitObjectUnionClustered> DummyPtr(nullptr);
		return DummyPtr;
	}
	
	UE_DEPRECATED(5.4, "Use GetChildrenSpatial instead")
    TUniquePtr<FImplicitObjectUnionClustered>& ChildrenSpatial()
	{
		check(false);
		static TUniquePtr<FImplicitObjectUnionClustered> DummyPtr(nullptr);
		return DummyPtr;
	}

	UE_DEPRECATED(5.4, "Use SetChildrenSpatial with FImplicitObjectUnionClusteredPtr instead")
    void SetChildrenSpatial(TUniquePtr<FImplicitObjectUnion>& Obj) { check(false); }

	const FRealSingle& CollisionImpulse() const { return PBDRigidClusteredParticles->CollisionImpulses(ParticleIdx); }
	FRealSingle& CollisionImpulse() { return PBDRigidClusteredParticles->CollisionImpulses(ParticleIdx); }
	void SetCollisionImpulse(const FRealSingle Value) { PBDRigidClusteredParticles->CollisionImpulses(ParticleIdx) = Value; }
	const FRealSingle& CollisionImpulses() const { return PBDRigidClusteredParticles->CollisionImpulses(ParticleIdx); }
	FRealSingle& CollisionImpulses() { return PBDRigidClusteredParticles->CollisionImpulses(ParticleIdx); }
	void SetCollisionImpulses(const FRealSingle Value) { PBDRigidClusteredParticles->CollisionImpulses(ParticleIdx) = Value; }
	void ClearCollisionImpulse() { PBDRigidClusteredParticles->CollisionImpulses(ParticleIdx) = static_cast<FRealSingle>(0); }

	FRealSingle GetExternalStrain() const { return PBDRigidClusteredParticles->ExternalStrains(ParticleIdx); }
	UE_DEPRECATED(5.2, "This method should not be used anymore. FRigidClustering::SetExternalStrain should be used instead.")
	void SetExternalStrain(const FRealSingle Value) { PBDRigidClusteredParticles->ExternalStrains(ParticleIdx) = Value; }
	UE_DEPRECATED(5.2, "This method should not be used anymore. FRigidClustering::SetExternalStrain with 0 strain should be used instead.")
	void ClearExternalStrain() { PBDRigidClusteredParticles->ExternalStrains(ParticleIdx) = static_cast<FRealSingle>(0); }
	
	const FRealSingle& GetInternalStrains() const { return PBDRigidClusteredParticles->Strains(ParticleIdx); }
	const FRealSingle& Strain() const { return PBDRigidClusteredParticles->Strains(ParticleIdx); }
	UE_DEPRECATED(5.2, "This method should not be used anymore. FRigidClustering::SetInternalStrain should be used instead.")
	FRealSingle& Strain() { return PBDRigidClusteredParticles->Strains(ParticleIdx); }
	UE_DEPRECATED(5.2, "This method should not be used anymore. FRigidClustering::SetInternalStrain should be used instead.")
	void SetStrain(const FRealSingle Value) { PBDRigidClusteredParticles->Strains(ParticleIdx) = Value; }
	const FRealSingle& Strains() const { return PBDRigidClusteredParticles->Strains(ParticleIdx); }
	UE_DEPRECATED(5.2, "This method should not be used anymore. FRigidClustering::SetInternalStrain should be used instead.")
	FRealSingle& Strains() { return PBDRigidClusteredParticles->Strains(ParticleIdx); }
	UE_DEPRECATED(5.2, "This method should not be used anymore. FRigidClustering::SetInternalStrain should be used instead.")
	void SetStrains(const FRealSingle Value) { PBDRigidClusteredParticles->Strains(ParticleIdx) = Value; }
	void SetMaximumInternalStrain() { PBDRigidClusteredParticles->Strains(ParticleIdx) = MaxStrain; }

	const TArray<TConnectivityEdge<T>>& ConnectivityEdges() const { return PBDRigidClusteredParticles->ConnectivityEdges(ParticleIdx); }
	TArray<TConnectivityEdge<T>>& ConnectivityEdges() { return PBDRigidClusteredParticles->ConnectivityEdges(ParticleIdx); }
	void SetConnectivityEdges(const TArray<TConnectivityEdge<T>>& Edges) { PBDRigidClusteredParticles->ConnectivityEdges(ParticleIdx) = Edges; }
	void SetConnectivityEdges(TArray<TConnectivityEdge<T>>&& Edges) { PBDRigidClusteredParticles->ConnectivityEdges(ParticleIdx) = MoveTemp(Edges); }

	bool IsAnchored() const { return PBDRigidClusteredParticles->RigidClusteredFlags(ParticleIdx).GetAnchored(); }
	void SetIsAnchored(bool bValue) { PBDRigidClusteredParticles->RigidClusteredFlags(ParticleIdx).SetAnchored(bValue); }

	bool Unbreakable() const { return PBDRigidClusteredParticles->RigidClusteredFlags(ParticleIdx).GetUnbreakable(); }
	void SetUnbreakable(bool bValue) { PBDRigidClusteredParticles->RigidClusteredFlags(ParticleIdx).SetUnbreakable(bValue); }

	bool IsChildToParentLocked() const { return PBDRigidClusteredParticles->RigidClusteredFlags(ParticleIdx).GetChildToParentLocked(); }
	void SetChildToParentLocked(bool bValue) { PBDRigidClusteredParticles->RigidClusteredFlags(ParticleIdx).SetChildToParentLocked(bValue); }

	const TPBDRigidClusteredParticleHandleImp<T, d, true>* Handle() const { return PBDRigidClusteredParticles->Handle(ParticleIdx); }
	TPBDRigidClusteredParticleHandleImp<T, d, true>* Handle() { return PBDRigidClusteredParticles->Handle(ParticleIdx); }

	static constexpr EParticleType StaticType() { return EParticleType::Clustered; }

	int32 TransientParticleIndex() const { return ParticleIdx; }

	// Get the clustered particle convex optmizer
	const TPimplPtr<Private::FConvexOptimizer>& ConvexOptimizer() const {return PBDRigidClusteredParticles->ConvexOptimizers(ParticleIdx); }
	TPimplPtr<Private::FConvexOptimizer>& ConvexOptimizer() {return PBDRigidClusteredParticles->ConvexOptimizers(ParticleIdx); }

private:
	void SetInternalStrains(const FRealSingle Value) { PBDRigidClusteredParticles->Strains(ParticleIdx) = Value; }
	void SetExternalStrains(const FRealSingle Value) { PBDRigidClusteredParticles->ExternalStrains(ParticleIdx) = Value; }
	friend class FRigidClustering;
	static constexpr Chaos::FRealSingle MaxStrain = TNumericLimits<Chaos::FRealSingle>::Max() - TNumericLimits<Chaos::FRealSingle>::Min();
};

template <typename T, int d, bool bPersistent = true>
class TPBDGeometryCollectionParticleHandleImp : public TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>
{
public:
	using TGeometryParticleHandleImp<T, d, bPersistent>::ParticleIdx;
	using TGeometryParticleHandleImp<T, d, bPersistent>::PBDRigidParticles;
	using TGeometryParticleHandleImp<T, d, bPersistent>::Type;
	using TTransientHandle = TTransientPBDGeometryCollectionParticleHandle<T, d>;
	using TSOAType = TPBDGeometryCollectionParticles<T, d>;

protected:
	friend class TGeometryParticleHandleImp<T, d, bPersistent>;

	//needed for serialization
	TPBDGeometryCollectionParticleHandleImp()
		: TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>()
	{}

	TPBDGeometryCollectionParticleHandleImp<T, d, bPersistent>(
		TSerializablePtr<TPBDGeometryCollectionParticles<T, d>> Particles, 
		int32 InIdx, 
		int32 InGlobalIdx, 
		const FPBDRigidParticleParameters& Params = FPBDRigidParticleParameters())
		: TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>(
			TSerializablePtr<TPBDRigidClusteredParticles<T, d>>(Particles), InIdx, InGlobalIdx, Params)
	{}
public:

	static TUniquePtr<TPBDGeometryCollectionParticleHandleImp<T, d, bPersistent>> CreateParticleHandle(
		TSerializablePtr<TPBDGeometryCollectionParticles<T, d>> InParticles, 
		int32 InParticleIdx, 
		int32 InHandleIdx, 
		const FPBDRigidParticleParameters& Params = FPBDRigidParticleParameters())
	{
		return TGeometryParticleHandleImp<T, d, bPersistent>::CreateParticleHandleHelper(
			InParticles, InParticleIdx, InHandleIdx, Params);
	}

	TSerializablePtr<TPBDGeometryCollectionParticleHandleImp<T, d, bPersistent>> ToSerializable() const
	{
		TSerializablePtr<TPBDGeometryCollectionParticleHandleImp<T, d, bPersistent>> Serializable;
		Serializable.SetFromRawLowLevel(this);
		return Serializable;
	}

	const TPBDGeometryCollectionParticleHandleImp<T, d, true>* Handle() const { return PBDRigidParticles->Handle(ParticleIdx); }
	TPBDGeometryCollectionParticleHandleImp<T, d, true>* Handle() { return PBDRigidParticles->Handle(ParticleIdx); }

	static constexpr EParticleType StaticType() { return EParticleType::GeometryCollection; }
};

template <typename T, int d, bool bPersistent>
TKinematicGeometryParticleHandleImp<T, d, bPersistent>* TGeometryParticleHandleImp<T, d, bPersistent>::CastToKinematicParticle() { checkSlow(Type <= EParticleType::Clustered || Type == EParticleType::GeometryCollection);  return Type >= EParticleType::Kinematic ? static_cast<TKinematicGeometryParticleHandleImp<T, d, bPersistent>*>(this) : nullptr; }

template <typename T, int d, bool bPersistent>
const TKinematicGeometryParticleHandleImp<T, d, bPersistent>* TGeometryParticleHandleImp<T,d, bPersistent>::CastToKinematicParticle() const { checkSlow(Type <= EParticleType::Clustered || Type == EParticleType::GeometryCollection); return Type >= EParticleType::Kinematic ? static_cast<const TKinematicGeometryParticleHandleImp<T, d, bPersistent>*>(this) : nullptr; }

template <typename T, int d, bool bPersistent>
const TPBDRigidParticleHandleImp<T, d, bPersistent>* TGeometryParticleHandleImp<T, d, bPersistent>::CastToRigidParticle() const { checkSlow(Type <= EParticleType::Clustered || Type == EParticleType::GeometryCollection); return Type >= EParticleType::Rigid ? static_cast<const TPBDRigidParticleHandleImp<T, d, bPersistent>*>(this) : nullptr; }

template <typename T, int d, bool bPersistent>
TPBDRigidParticleHandleImp<T, d, bPersistent>* TGeometryParticleHandleImp<T, d, bPersistent>::CastToRigidParticle() { checkSlow(Type <= EParticleType::Clustered || Type == EParticleType::GeometryCollection); return Type >= EParticleType::Rigid ? static_cast<TPBDRigidParticleHandleImp<T, d, bPersistent>*>(this) : nullptr; }

template <typename T, int d, bool bPersistent>
const TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>* TGeometryParticleHandleImp<T, d, bPersistent>::CastToClustered() const { checkSlow(Type <= EParticleType::Clustered || Type == EParticleType::GeometryCollection); return Type >= EParticleType::Clustered ? static_cast<const TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>*>(this) : nullptr; }

template <typename T, int d, bool bPersistent>
TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>* TGeometryParticleHandleImp<T, d, bPersistent>::CastToClustered() { checkSlow( Type <= EParticleType::Clustered || Type == EParticleType::GeometryCollection); return Type >= EParticleType::Clustered ? static_cast<TPBDRigidClusteredParticleHandleImp<T, d, bPersistent>*>(this) : nullptr; }

template <typename T, int d, bool bPersistent>
const TPBDGeometryCollectionParticleHandleImp<T, d, bPersistent>* TGeometryParticleHandleImp<T, d, bPersistent>::CastToGeometryCollection() const { checkSlow(Type <= EParticleType::GeometryCollection || Type == EParticleType::GeometryCollection); return Type >= EParticleType::GeometryCollection ? static_cast<const TPBDGeometryCollectionParticleHandleImp<T, d, bPersistent>*>(this) : nullptr; }

template <typename T, int d, bool bPersistent>
TPBDGeometryCollectionParticleHandleImp<T, d, bPersistent>* TGeometryParticleHandleImp<T, d, bPersistent>::CastToGeometryCollection() { checkSlow(Type <= EParticleType::GeometryCollection|| Type == EParticleType::GeometryCollection); return Type >= EParticleType::GeometryCollection ? static_cast<TPBDGeometryCollectionParticleHandleImp<T, d, bPersistent>*>(this) : nullptr; }


template <typename T, int d, bool bPersistent>
EObjectStateType TGeometryParticleHandleImp<T,d, bPersistent>::ObjectState() const
{
	const TKinematicGeometryParticleHandleImp<T, d, bPersistent>* Kin = CastToKinematicParticle();
	return Kin ? Kin->ObjectState() : EObjectStateType::Static;
}

template <typename T, int d, bool bPersistent>
EObjectStateType TKinematicGeometryParticleHandleImp<T, d, bPersistent>::ObjectState() const
{
	const TPBDRigidParticleHandleImp<T, d, bPersistent>* Dyn = CastToRigidParticle();
	return Dyn ? Dyn->ObjectState() : EObjectStateType::Kinematic;
}

enum class EWakeEventEntry : uint8
{
	None,
	Awake,
	Sleep
};

template <typename T, int d, bool bPersistent>
FString TGeometryParticleHandleImp<T, d, bPersistent>::ToString() const
{
	switch (Type)
	{
	case EParticleType::Static:
		return FString::Printf(TEXT("Static[%d]"), ParticleIdx);
	case EParticleType::Kinematic:
		return FString::Printf(TEXT("Kinemmatic[%d]"), ParticleIdx);
	case EParticleType::Rigid:
		return FString::Printf(TEXT("Dynamic[%d]"), ParticleIdx);
	case EParticleType::GeometryCollection:
		return FString::Printf(TEXT("GeometryCollection[%d]"), ParticleIdx);
	case EParticleType::Clustered:
		return FString::Printf(TEXT("Clustered[%d]"), ParticleIdx);
	default:
		break;
	}
	return FString();
}

template <typename T, int d, bool bPersistent>
TGeometryParticleHandleImp<T,d,bPersistent>* TGeometryParticleHandleImp<T,d, bPersistent>::SerializationFactory(FChaosArchive& Ar, TGeometryParticleHandleImp<T, d, bPersistent>* Handle)
{
	check(bPersistent);
	static_assert(sizeof(TGeometryParticleHandleImp<T, d, bPersistent>) == sizeof(TGeometryParticleHandleImp<T, d, bPersistent>), "No new members in derived classes");
	return Ar.IsLoading() ? new TGeometryParticleHandleImp<T, d, bPersistent>() : nullptr;
}

class FGenericParticleHandleImp
{
public:
	using FDynamicParticleHandleType = FPBDRigidParticleHandle;
	using FKinematicParticleHandleType = FKinematicGeometryParticleHandle;

	FGenericParticleHandleImp(FGeometryParticleHandle* InHandle) : MHandle(InHandle) {}

	// Check for the exact type of particle (see also AsKinematic etc, which will work on derived types)
	bool IsStatic() const { return (MHandle->ObjectState() == EObjectStateType::Static); }
	bool IsKinematic() const { return (MHandle->ObjectState() == EObjectStateType::Kinematic); }
	bool IsDynamic() const { return (MHandle->ObjectState() == EObjectStateType::Dynamic) || (MHandle->ObjectState() == EObjectStateType::Sleeping); }
	bool IsSleeping() const { return (MHandle->ObjectState() == EObjectStateType::Sleeping); }
	
	bool WasSleeping() const 
	{ 
		if (FPBDRigidParticleHandle* Rigid = MHandle->CastToRigidParticle())
		{
			return Rigid->WasSleeping();
		}
		return IsSleeping();
	}

	const FKinematicGeometryParticleHandle* CastToKinematicParticle() const { return MHandle->CastToKinematicParticle(); }
	FKinematicGeometryParticleHandle* CastToKinematicParticle() { return MHandle->CastToKinematicParticle(); }
	const FPBDRigidParticleHandle* CastToRigidParticle() const { return MHandle->CastToRigidParticle(); }
	FPBDRigidParticleHandle* CastToRigidParticle() { return MHandle->CastToRigidParticle(); }
	const TPBDGeometryCollectionParticleHandleImp<FReal, 3, true>* CastToGeometryCollection() const { return MHandle->CastToGeometryCollection(); }
	TPBDGeometryCollectionParticleHandleImp<FReal, 3, true>* CastToGeometryCollection() { return MHandle->CastToGeometryCollection(); }
	const TPBDRigidClusteredParticleHandleImp<FReal, 3, true>* CastToClustered() const { return MHandle->CastToClustered(); }
	TPBDRigidClusteredParticleHandleImp<FReal, 3, true>* CastToClustered() { return MHandle->CastToClustered(); }

	const FGeometryParticleHandle* GeometryParticleHandle() const { return MHandle; }
	FGeometryParticleHandle* GeometryParticleHandle() { return MHandle; }
	//Needed for templated code to be the same
	const FGeometryParticleHandle* Handle() const { return MHandle; }
	FGeometryParticleHandle* Handle() { return MHandle; }
	int32 GetHandleIdx() const { return MHandle->GetHandleIdx(); }

	void SetTransform(const FVec3& Pos, const FRotation3& Rot)
	{
		MHandle->SetX(Pos);
		MHandle->SetR(Rot);
		if (FPBDRigidParticleHandle* Dynamic = CastToRigidParticle())
		{
			Dynamic->SetP(Pos);
			Dynamic->SetQ(Rot);
		}
	}

	// Static Particles
	UE_DEPRECATED(5.4, "Use GetX or SetX instead")
	FVec3& X() { 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return MHandle->X(); 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	UE_DEPRECATED(5.4, "Use GetX instead")
	const FVec3& X() const { return MHandle->GetX(); }
	void SetX(const FVec3& InX) { MHandle->SetX(InX); }
	const FVec3& GetX() const { return MHandle->GetX(); }
	UE_DEPRECATED(5.4, "Use GetR or SetR instead")
	FRotation3 R() { return MHandle->GetR(); }
	UE_DEPRECATED(5.4, "Use GetR instead")
	const FRotation3 R() const { return MHandle->GetR(); }
	void SetR(const FRotation3& InR) { MHandle->SetR(InR); }
	const FRotation3 GetR() const { return MHandle->GetR(); }
	const FImplicitObjectRef GetGeometry() const { return MHandle->GetGeometry(); }
	bool Sleeping() const { return MHandle->Sleeping(); }
	FString ToString() const { return MHandle->ToString(); }

	UE_DEPRECATED(5.4, "Use GetGeometry instead")
	TSerializablePtr<FImplicitObject> Geometry() const { return TSerializablePtr<FImplicitObject>(); }
	
	UE_DEPRECATED(5.4, "Use GetGeometry instead")
	const TUniquePtr<FImplicitObject>& DynamicGeometry() const { static TUniquePtr<FImplicitObject> DummyPtr(nullptr); return DummyPtr; }

	bool EnabledDuringResim() const { return MHandle->EnabledDuringResim(); }

	template <typename Container>
	const auto& AuxilaryValue(const Container& AuxContainer) const { return MHandle->AuxilaryValue(AuxContainer); }
	template <typename Container>
	auto& AuxilaryValue(Container& AuxContainer) { return MHandle->AuxilaryValue(AuxContainer); }

	// Kinematic Particles
	const FVec3 V() const { return GetV(); }
	const FVec3 W() const { return GetW(); }

	const FVec3 GetV() const { return (MHandle->CastToKinematicParticle()) ? MHandle->CastToKinematicParticle()->GetV() : ZeroVector; }
	const FVec3 GetW() const { return (MHandle->CastToKinematicParticle()) ? MHandle->CastToKinematicParticle()->GetW() : ZeroVector; }

	void SetV(const FVec3& InV) { if (MHandle->CastToKinematicParticle()) { MHandle->CastToKinematicParticle()->SetV(InV); } }
	void SetW(const FVec3& InW) { if (MHandle->CastToKinematicParticle()) { MHandle->CastToKinematicParticle()->SetW(InW); } }

	const FVec3f GetVf() const { return (MHandle->CastToKinematicParticle()) ? MHandle->CastToKinematicParticle()->GetVf() : ZeroVectorf; }
	const FVec3f GetWf() const { return (MHandle->CastToKinematicParticle()) ? MHandle->CastToKinematicParticle()->GetWf() : ZeroVectorf; }

	void SetVf(const FVec3f& InV) { if (MHandle->CastToKinematicParticle()) { MHandle->CastToKinematicParticle()->SetVf(InV); } }
	void SetWf(const FVec3f& InW) { if (MHandle->CastToKinematicParticle()) { MHandle->CastToKinematicParticle()->SetWf(InW); } }

	const FKinematicTarget& KinematicTarget() const { return (MHandle->CastToKinematicParticle())? MHandle->CastToKinematicParticle()->KinematicTarget() : EmptyKinematicTarget; }

	// Dynamic Particles

	// TODO: Make all of these check ObjectState to maintain current functionality
	int32 CollisionParticlesSize() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->CollisionParticlesSize();
		}

		return 0;
	}

	const TUniquePtr<FBVHParticles>& CollisionParticles() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->CollisionParticles();
		}

		return NullBVHParticles;
	}

	int32 CollisionGroup() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->CollisionGroup();
		}

		return 0;
	}

	bool CCDEnabled() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->CCDEnabled();
		}

		return false;
	}

	bool MACDEnabled() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->MACDEnabled();
		}

		return false;
	}

	bool PartialIslandSleepAllowed() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->PartialIslandSleepAllowed();
		}

		return false;
	}

	Private::FIterationSettings IterationSettings() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->IterationSettings();
		}

		return Private::FIterationSettings(0, 0, 0);
	
	}

	bool HasCollisionConstraintFlag(const ECollisionConstraintFlags Flag)  const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->HasCollisionConstraintFlag(Flag);
		}

		return false;
	}


	// @todo(ccaulfield): should be available on all types?
	bool Disabled() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->Disabled();
		}

		return false;
	}

	const FVec3 PreV() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->GetPreV();
		}

		return ZeroVector;
	}

	const FVec3 PreW() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->GetPreW();
		}
		return ZeroVector;
	}

	const FVec3f GetPreVf() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->GetPreVf();
		}

		return ZeroVectorf;
	}

	const FVec3f GetPreWf() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->GetPreWf();
		}
		return ZeroVectorf;
	}

	int32 SolverBodyIndex() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->SolverBodyIndex();
		}
		return INDEX_NONE;
	}

	void SetSolverBodyIndex(const int32 InSolverBodyIndex)
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->SetSolverBodyIndex(InSolverBodyIndex);
		}
	}

	void SetP(const FVec3& InP)
	{
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->SetP(InP);
		}

		return SetX(InP);
	}

	const FVec3& P() const
	{
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->GetP();
		}

		return GetX();
	}

	const FVec3& GetP() const
	{
		return P();
	}

	void SetQ(const FRotation3& InQ)
	{
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->SetQ(InQ);
		}

		SetR(InQ);
	}

	const FRotation3 Q() const
	{
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->GetQ();
		}

		return GetR();
	}

	const FRotation3 GetQ() const
	{
		return Q();
	}

	// World-space center of mass position
	const FVec3 XCom() const
	{
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->XCom();
		}
		return GetX();
	}
	const FVec3 PCom() const
	{
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->PCom();
		}
		return GetX();
	}

	// World-space center of mass rotation
	const FRotation3 RCom() const
	{ 
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->RCom();
		}
		return GetR();
	}
	const FRotation3 QCom() const
	{ 
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->QCom();
		}
		return GetR();
	}

	void InitTransform(const FVec3& InP, const FRotation3& InQ)
	{
		if (MHandle->CastToRigidParticle())
		{
			MHandle->CastToRigidParticle()->InitTransform(InP, InQ);
		}
		MHandle->InitTransform(InP, InQ);
	}

	void SetTransformPQCom(const FVec3& InPCom, const FRotation3& InQCom)
	{ 
		if (IsDynamic())
		{
			MHandle->CastToRigidParticle()->SetTransformPQCom(InPCom, InQCom);
		}
		else
		{
			SetTransform(InPCom, InQCom);
		}
	}

	FRigidTransform3 GetTransformXR() const
	{
		return MHandle->GetTransformXR();
	}

	FRigidTransform3 GetTransformPQ() const
	{
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->GetTransformPQ();
		}
		return MHandle->GetTransformXR();
	}

	FRigidTransform3 GetTransformXRCom() const
	{
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->GetTransformXRCom();
		}
		return GetTransformXR();
	}

	FRigidTransform3 GetTransformPQCom() const
	{
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->GetTransformPQCom();
		}
		return GetTransformXR();
	}

	/**
	 * Convert a particle-relative position into a com-relative position
	 */
	FVec3 GetComRelativePosition(const FVec3& P)
	{
		if (IsDynamic())
		{
			FPBDRigidParticleHandle* Rigid = MHandle->CastToRigidParticle();
			return Rigid->RotationOfMass().UnrotateVector(P - Rigid->CenterOfMass());
		}
		return P;
	}

	/**
	 * Convert a particle-relative rotation into a com-relative rotation
	 */
	FRotation3 GetComRelativeRotation(const FRotation3& Q)
	{
		if (IsDynamic())
		{
			FPBDRigidParticleHandle* Rigid = MHandle->CastToRigidParticle();
			return Rigid->RotationOfMass().Inverse() * Q;
		}
		return Q;
	}

	/**
	 * Convert a particle-relative transform into a com-relative transform
	 */
	FRigidTransform3 GetComRelativeTransform(const FRigidTransform3& T)
	{
		if (IsDynamic())
		{
			return FRigidTransform3(GetComRelativePosition(T.GetLocation()), GetComRelativeRotation(T.GetRotation()));
		}
		return T;
	}

	const FVec3 VSmooth() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->VSmooth();
		}

		return V();
	}

	const FVec3 WSmooth() const
	{
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->WSmooth();
		}

		return W();
	}

	const FVec3& Acceleration() const
	{ 
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->Acceleration();
		}

		return ZeroVector;
	}
	const FVec3& AngularAcceleration() const
	{ 
		if (MHandle->CastToRigidParticle())
		{
			return MHandle->CastToRigidParticle()->AngularAcceleration();
		}

		return ZeroVector;
	}

	const EObjectStateType ObjectState()  const
	{
		return MHandle->ObjectState();
	}

	FParticleID ParticleID() const
	{
		return MHandle->ParticleID();
	}

	FUniqueIdx UniqueIdx() const
	{
		return MHandle->UniqueIdx();
	}

	bool HasBounds() const
	{
		return MHandle->HasBounds();
	}

	const FAABB3& LocalBounds() const
	{
		return MHandle->LocalBounds();
	}

	//Named this way for templated code (GT/PT particles)
	bool HasBoundingBox() const
	{
		return MHandle->HasBounds();
	}

	//Named this way for templated code (GT/PT particles)
	const FAABB3& BoundingBox() const
	{
		return MHandle->WorldSpaceInflatedBounds();
	}

	const FAABB3& WorldSpaceInflatedBounds() const
	{ 
		return MHandle->WorldSpaceInflatedBounds();
	}

	void UpdateWorldSpaceState(const FRigidTransform3& WorldTransform, const FVec3& BoundsExpansion)
	{
		MHandle->UpdateWorldSpaceState(WorldTransform, BoundsExpansion);
	}

	const TVec3<FRealSingle> I() const
	{ 
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->I();
		}

		return TVec3<FRealSingle>(0);
	}

	const TVec3<FRealSingle> InvI() const
	{ 
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->InvI();
		}

		return TVec3<FRealSingle>(0);
	}

	FReal M() const
	{
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->M();
		}

		return (FReal)0;
	}

	FReal InvM() const
	{
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->InvM();
		}

		return (FReal)0;
	}

	FVec3 CenterOfMass() const
	{
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->CenterOfMass();
		}

		return FVec3(0);
	}

	void SetCenterOfMass(const FVec3& InCom)
	{
		if (IsDynamic())
		{
			MHandle->CastToRigidParticle()->SetCenterOfMass(InCom);
		}
	}

	FRotation3 RotationOfMass() const
	{
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->RotationOfMass();
		}

		return FRotation3::FromIdentity();
	}

	void SetRotationOfMass(const FRotation3& InRom)
	{
		if (IsDynamic())
		{
			MHandle->CastToRigidParticle()->SetRotationOfMass(InRom);
		}
	}

	TVec3<FRealSingle> InvIConditioning() const
	{ 
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->InvIConditioning();
		}

		return TVec3<FRealSingle>(1);
	}

	void SetInvIConditioning(const TVec3<FRealSingle>& InInvIConditioning)
	{ 
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			RigidHandle->SetInvIConditioning(InInvIConditioning);
		}
	}

	TVec3<FRealSingle> ConditionedInvI() const
	{ 
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->ConditionedInvI();
		}

		return TVec3<FRealSingle>(0);
	}

	TVec3<FRealSingle> ConditionedI() const
	{ 
		if (IsDynamic())
		{
			return MHandle->CastToRigidParticle()->ConditionedI();
		}

		return TVec3<FRealSingle>(0);
	}

	bool InertiaConditioningEnabled() const
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			return RigidHandle->InertiaConditioningEnabled();
		}
		return false;
	}

	bool InertiaConditioningDirty() const
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			return RigidHandle->InertiaConditioningDirty();
		}
		return false;
	}

	void SetInertiaConditioningDirty()
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			RigidHandle->SetInertiaConditioningDirty();
		}
	}

	void ClearInertiaConditioningDirty()
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			RigidHandle->ClearInertiaConditioningDirty();
		}
	}

	inline bool UseIgnoreCollisionManager() const
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			return RigidHandle->UseIgnoreCollisionManager();
		}
		return false;
	}

	inline void SetUseIgnoreCollisionManager()
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			RigidHandle->SetUseIgnoreCollisionManager();
		}
	}

	inline void ClearUseIgnoreCollisionManager()
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			RigidHandle->ClearUseIgnoreCollisionManager();
		}
	}

	inline bool IsMovingKinematic() const
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			return RigidHandle->IsMovingKinematic();
		}
		return false;
	}

	inline void SetIsMovingKinematic()
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			RigidHandle->SetIsMovingKinematic();
		}
	}

	inline void ClearIsMovingKinematic()
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			RigidHandle->ClearIsMovingKinematic();
		}
	}

	FReal LinearEtherDrag() const
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			return RigidHandle->LinearEtherDrag();
		}
		return FReal(0);
	}

	FReal AngularEtherDrag() const
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			return RigidHandle->AngularEtherDrag();
		}
		return FReal(0);
	}

	FRealSingle InitialOverlapDepenetrationVelocity() const
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			return RigidHandle->InitialOverlapDepenetrationVelocity();
		}
		return -1.0f;	// Use config
	}

	FRealSingle SleepThresholdMultiplier() const
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			return RigidHandle->SleepThresholdMultiplier();
		}
		return 1.0f;
	}

	bool OneWayInteraction() const
	{
		if (auto RigidHandle = MHandle->CastToRigidParticle())
		{
			return RigidHandle->OneWayInteraction();
		}
		return false;
	}

#if CHAOS_DEBUG_NAME
	const TSharedPtr<FString, ESPMode::ThreadSafe>& DebugName() const
	{
		return MHandle->DebugName();
	}
#endif

	const FString& GetDebugName() const
	{
		return MHandle->GetDebugName();
	}


	FORCEINLINE bool IsInConstraintGraph() const
	{
		return MHandle->IsInConstraintGraph();
	}

	FORCEINLINE Private::FPBDIslandParticle* GetConstraintGraphNode() const
	{
		return MHandle->GetConstraintGraphNode();
	}

	FORCEINLINE void SetConstraintGraphNode(Private::FPBDIslandParticle* InNode)
	{
		MHandle->SetConstraintGraphNode(InNode);
	}

	const FShapesArray& ShapesArray() const { return MHandle->ShapesArray(); }
	const FShapeInstanceArray& ShapeInstances() const { return MHandle->ShapeInstances(); }

	static constexpr EParticleType StaticType()
	{
		return EParticleType::Unknown;
	}

	// Deprecated API
	UE_DEPRECATED(5.3, "No longer supported") int32 IslandIndex() const { return INDEX_NONE; }
	UE_DEPRECATED(5.3, "No longer supported") void SetIslandIndex(const int32 IslandIndex) {}
	UE_DEPRECATED(5.3, "No longer supported") int32 ConstraintGraphIndex() const { return INDEX_NONE; }
	UE_DEPRECATED(5.3, "No longer supported") void SetConstraintGraphIndex(const int32 InGraphIndex) {}

private:
	FGeometryParticleHandle* MHandle;

	// So we can return vectors by const ref in cases where there's no real value. E.g., Velocity of a Static.
	static CHAOS_API const FVec3 ZeroVector;
	static CHAOS_API const FVec3f ZeroVectorf;
	static CHAOS_API const FRotation3 IdentityRotation;
	static CHAOS_API const FMatrix33 ZeroMatrix;
	static CHAOS_API const TUniquePtr<FBVHParticles> NullBVHParticles;
	static CHAOS_API const FKinematicTarget EmptyKinematicTarget;
};

template <typename T, int d>
using TGenericParticleHandleHandleImp UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FGenericParticleHandleImp instead") = FGenericParticleHandleImp;

/**
 * A wrapper around any type of particle handle to provide a consistent (read-only) API for all particle types.
 * This can make code simpler because you can write code that is type-agnostic, but it
 * has a cost. Where possible it is better to write code that is specific to the type(s)
 * of particles being operated on. FGenericParticleHandle has pointer semantics, so you can use one wherever
 * you have a particle handle pointer;

 */
class FGenericParticleHandle
{
public:
	FGenericParticleHandle() : Imp(nullptr) {}
	FGenericParticleHandle(FGeometryParticleHandle* InHandle) : Imp(InHandle) {}

	FGenericParticleHandleImp* operator->() const { return const_cast<FGenericParticleHandleImp*>(&Imp); }
	FGenericParticleHandleImp* Get() const { return const_cast<FGenericParticleHandleImp*>(&Imp); }

	bool IsValid() const { return Imp.Handle() != nullptr; }

	friend uint32 GetTypeHash(const FGenericParticleHandle& H)
	{
		return GetTypeHash(H->ParticleID());
	}

	friend bool operator==(const FGenericParticleHandle& L, const FGenericParticleHandle& R)
	{
		return L->ParticleID() == R->ParticleID();
	}

	friend bool operator!=(const FGenericParticleHandle& L, const FGenericParticleHandle& R)
	{
		return !(L->ParticleID() == R->ParticleID());
	}

	friend bool operator<(const FGenericParticleHandle& L, const FGenericParticleHandle& R)
	{
		return L->ParticleID() < R->ParticleID();
	}

private:
	FGenericParticleHandleImp Imp;
};

template <typename T, int d>
using TGenericParticleHandle UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FGenericParticleHandle instead") = FGenericParticleHandle;

class FConstGenericParticleHandle
{
public:
	FConstGenericParticleHandle() : Imp(nullptr) {}
	FConstGenericParticleHandle(const FGeometryParticleHandle* InHandle) : Imp(const_cast<FGeometryParticleHandle*>(InHandle)) {}
	FConstGenericParticleHandle(const FGenericParticleHandle InHandle) : Imp(InHandle->Handle()) {}

	const FGenericParticleHandleImp* operator->() const { return &Imp; }
	const FGenericParticleHandleImp* Get() const { return &Imp; }

	bool IsValid() const { return Imp.Handle() != nullptr; }

	friend uint32 GetTypeHash(const FConstGenericParticleHandle& H)
	{
		return GetTypeHash(H->ParticleID());
	}

	friend bool operator==(const FConstGenericParticleHandle& L, const FConstGenericParticleHandle& R)
	{
		return L->ParticleID() == R->ParticleID();
	}

	friend bool operator<(const FConstGenericParticleHandle& L, const FConstGenericParticleHandle& R)
	{
		return L->ParticleID() < R->ParticleID();
	}

private:
	FGenericParticleHandleImp Imp;
};

template <typename T, int d>
using TConstGenericParticleHandle UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FConstGenericParticleHandle instead") = FConstGenericParticleHandle;

template <typename T, int d>
class TGeometryParticleHandles : public TArrayCollection
{
public:
	TGeometryParticleHandles()
	{
		AddArray(&Handles);
	}

	void AddHandles(int32 NumHandles)
	{
		AddElementsHelper(NumHandles);
	}

	void Reset()
	{
		ResizeHelper(0);
	}

	void DestroyHandleSwap(TGeometryParticleHandle<T,d>* Handle)
	{
		const int32 UnstableIdx = Handle->HandleIdx;

#if UE_DEBUG_DANGLING_HANDLES
		// This helps to detect dangling handles and associated memory corruptions
		// Leaking memory deliberately here, so only use this while debugging
		Handles[UnstableIdx]->~TGeometryParticleHandleImp();
		(void)Handles[UnstableIdx].Release();
#endif

		RemoveAtSwapHelper(UnstableIdx);
		if (static_cast<uint32>(UnstableIdx) < Size())
		{
			Handles[UnstableIdx]->HandleIdx = UnstableIdx;
		}
	}

	void Serialize(FChaosArchive& Ar)
	{
		Ar << Handles;
		ResizeHelper(Handles.Num());
	}

	const TUniquePtr<TGeometryParticleHandle<T, d>>& Handle(int32 Idx) const { return Handles[Idx];}
	TUniquePtr<TGeometryParticleHandle<T, d>>& Handle(int32 Idx) { return Handles[Idx]; }
private:
	TArrayCollectionArray<TUniquePtr<TGeometryParticleHandleImp<T, d, true>>> Handles;
};

template <typename T, int d>
class TKinematicGeometryParticle;

template <typename T, int d>
class TPBDRigidParticle;


/**
 * Base class for transient classes used to communicate simulated particle state 
 * between game and physics threads, which is managed by proxies.
 *
 * Note the lack of virtual api.
 */
class FParticleData
{
public:
	FParticleData(EParticleType InType = EParticleType::Static)
		: Type(InType)
	{}

	void Reset() { Type = EParticleType::Static; }

	virtual ~FParticleData() = default;

	EParticleType Type;
};

template <typename T, int d>
class TGeometryParticle
{
public:
	typedef TGeometryParticleHandle<T, d> FHandle;

	static constexpr bool AlwaysSerializable = true;

protected:

	TGeometryParticle(const FGeometryParticleParameters& StaticParams = FGeometryParticleParameters())
	{
		Type = EParticleType::Static;
		Proxy = nullptr;
		MUserData = nullptr;
		GeometryParticleDefaultConstruct<T, d>(*this, StaticParams);
	}

public:
	static TUniquePtr<TGeometryParticle<T, d>> CreateParticle(const FGeometryParticleParameters& Params = FGeometryParticleParameters())
	{
		return TUniquePtr< TGeometryParticle<T, d>>(new TGeometryParticle<T, d>(Params));
	}

	virtual ~TGeometryParticle() //only virtual for easier memory management. Should generally be a static API
	{
		SetUniqueIdx(FUniqueIdx{}, false); // Set to an invalid index for dangling handle detection
	}

	TGeometryParticle(const TGeometryParticle&) = delete;

	TGeometryParticle& operator=(const TGeometryParticle&) = delete;

	virtual void Serialize(FChaosArchive& Ar)
	{
		Ar << MXR;
		Ar << MNonFrequentData;
		Ar << MShapesArray;
		Ar << Type;
		//Ar << MDirtyFlags;

		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::SerializeShapeWorldSpaceBounds)
		{
			UpdateShapeBounds();
		}
	}

	virtual bool IsParticleValid() const
	{
		auto Geometry = MNonFrequentData.Read().GetGeometry();
		return Geometry && Geometry->IsValidGeometry();	//todo: if we want support for sample particles without geometry we need to adjust this
	}

	static inline TGeometryParticle<T, d>* SerializationFactory(FChaosArchive& Ar, TGeometryParticle<T, d>* Serializable);

	const TVector<T, d>& X() const { return MXR.Read().X(); }
	const TVector<T, d>& GetX() const { return MXR.Read().X(); }
	void SetX(const TVector<T, d>& InX, bool bInvalidate = true);

	FUniqueIdx UniqueIdx() const { return MNonFrequentData.Read().UniqueIdx(); }
	void SetUniqueIdx(const FUniqueIdx UniqueIdx, bool bInvalidate = true)
	{
		MNonFrequentData.Modify(bInvalidate,MDirtyFlags,Proxy,[UniqueIdx](auto& Data){ Data.SetUniqueIdx(UniqueIdx);});
	}

	const FParticleID& ParticleID() const { return MNonFrequentData.Read().ParticleID(); }
	void SetParticleID(const FParticleID& ParticleID, bool bInvalidate = true)
	{
		if (this->ParticleID() == ParticleID)
		{
			return;
		}

		MNonFrequentData.Modify(bInvalidate, MDirtyFlags, Proxy, [ParticleID](auto& Data) { Data.SetParticleID(ParticleID); });
	}

	const TRotation<T, d> R() const { return MXR.Read().R(); }
	const TRotation<T, d> GetR() const { return MXR.Read().R(); }
	void SetR(const TRotation<T, d>& InR, bool bInvalidate = true);

	const FParticlePositionRotation& XR() const { return MXR.Read(); }
	void SetXR(const FParticlePositionRotation& InXR, bool bInvalidate = true)
	{
		MXR.Write(InXR,bInvalidate,MDirtyFlags,Proxy);
	}
	
	//todo: geometry should not be owned by particle
	void SetGeometry(Chaos::FImplicitObjectPtr ImplicitObjectPtr)
	{
		MNonFrequentData.Modify(true,MDirtyFlags,Proxy,[&ImplicitObjectPtr](auto& Data){ Data.SetGeometry(Chaos::FImplicitObjectPtr(ImplicitObjectPtr));});
		UpdateShapesArray();
	}

	CHAOS_API void MergeGeometry(TArray<Chaos::FImplicitObjectPtr>&& Objects);

	const FImplicitObjectRef GetGeometry() const { return MNonFrequentData.Read().GetGeometry(); }

	// Set a flag on all Unions to indicate that any add/remove operations are errors.
	// This is called when the shapes are shared between GT/PT
	void LockGeometry();

	UE_DEPRECATED(5.4, "Use SetGeometry with an array of Chaos::FImplicitObjectPtr instead")
	void SetGeometry(TUniquePtr<FImplicitObject>&& UniqueGeometry) { check(false); }
	
	UE_DEPRECATED(5.4, "Use SetGeometry with an array of Chaos::FImplicitObjectPtr instead")
    void SetGeometry(TSharedPtr<FImplicitObject, ESPMode::ThreadSafe> SharedGeometry) { check(false); }
	
	UE_DEPRECATED(5.4, "Use SetGeometry with an array of Chaos::FImplicitObjectPtr instead")
    void SetGeometry(TSerializablePtr<FImplicitObject> RawGeometry) { check(false); }

	UE_DEPRECATED(5.4, "Use MergeGeometry with an array of Chaos::FImplicitObjectPtr instead")
	void MergeGeometry(TArray<TUniquePtr<FImplicitObject>>&& Objects) { check(false); }

	UE_DEPRECATED(5.4, "Use GetGeometry instead")
	TSharedPtr<const FImplicitObject,ESPMode::ThreadSafe> SharedGeometryLowLevel() const { check(false); return nullptr; }

	UE_DEPRECATED(5.4, "Use GetGeometry instead")
	TSerializablePtr<FImplicitObject> Geometry() const { check(false); return TSerializablePtr<FImplicitObject>(); }

	CHAOS_API void RemoveShape(FPerShapeData* InShape, bool bWakeTouching);
	CHAOS_API void RemoveShapesAtSortedIndices(const TArrayView<const int32>& InIndices);

	void* UserData() const { return MUserData; }
	void SetUserData(void* InUserData)
	{
		MUserData = InUserData;
	}

	void UpdateShapeBounds()
	{
		UpdateShapeBounds(FRigidTransform3(X(), R()));
	}

	void UpdateShapeBounds(const FRigidTransform3& Transform)
	{
		auto GeomShared = MNonFrequentData.Read().GetGeometry();
		if (GeomShared && GeomShared->HasBoundingBox())
		{
			for (auto& Shape : MShapesArray)
			{
				Shape->UpdateShapeBounds(Transform);
			}
		}
	}

	void SetShapeSimCollisionEnabled(int32 InShapeIndex, bool bInEnabled)
	{
		const bool bCurrent = MShapesArray[InShapeIndex]->GetSimEnabled();
		if(bCurrent != bInEnabled)
		{
			MShapesArray[InShapeIndex]->SetSimEnabled(bInEnabled);
		}
	}

	void SetShapeQueryCollisionEnabled(int32 InShapeIndex, bool bInEnabled)
	{
		const bool bCurrent = MShapesArray[InShapeIndex]->GetQueryEnabled();
		if(bCurrent != bInEnabled)
		{
			MShapesArray[InShapeIndex]->SetQueryEnabled(bInEnabled);
		}
	}

	void SetShapeCollisionTraceType(int32 InShapeIndex,EChaosCollisionTraceFlag TraceType)
	{
		const EChaosCollisionTraceFlag Current = MShapesArray[InShapeIndex]->GetCollisionTraceType();
		if(Current != TraceType)
		{
			MShapesArray[InShapeIndex]->SetCollisionTraceType(TraceType);
		}
	}

	UE_DEPRECATED(5.8, "Use SetShapeFilterData instead")
	void SetShapeSimData(int32 InShapeIndex,const FCollisionFilterData& SimData)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FCollisionFilterData& Current = MShapesArray[InShapeIndex]->GetSimData();
		if(Current != SimData)
		{
			MShapesArray[InShapeIndex]->SetSimData(SimData);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void SetShapeFilterData(int32 InShapeIndex, const Chaos::Filter::FShapeFilterData& FilterData)
	{
		const Chaos::Filter::FShapeFilterData& Current = MShapesArray[InShapeIndex]->GetShapeFilterData();
		if (Current != FilterData)
		{
			MShapesArray[InShapeIndex]->SetShapeFilterData(FilterData);
		}
	}

	void SetShapeFilterInstanceData(int32 InShapeIndex, const Chaos::Filter::FInstanceData& InstanceData)
	{
		const Chaos::Filter::FInstanceData& Current = MShapesArray[InShapeIndex]->GetFilterInstanceData();
		if (Current != InstanceData)
		{
			MShapesArray[InShapeIndex]->SetFilterInstanceData(InstanceData);
		}
	}

#if CHAOS_DEBUG_NAME
	const TSharedPtr<FString, ESPMode::ThreadSafe>& DebugName() const { return MNonFrequentData.Read().DebugName(); }
	void SetDebugName(const TSharedPtr<FString, ESPMode::ThreadSafe>& InDebugName)
	{
		MNonFrequentData.Modify(true,MDirtyFlags,Proxy,[&InDebugName](auto& Data){ Data.SetDebugName(InDebugName);});
	}
#endif

	const FString& GetDebugName() const
	{
#if CHAOS_DEBUG_NAME
		const TSharedPtr<FString, ESPMode::ThreadSafe>& DebugNamePtr = DebugName();
		if (DebugNamePtr.IsValid())
		{
			return *(DebugNamePtr.Get());
		}
#endif
		return Private::EmptyParticleName;
	}

	void MergeShapesArray(FShapesArray&& InShapesArray)
	{
		MergeShapeInstances(reinterpret_cast<FShapeInstanceProxyArray&&>(InShapesArray));
	}

	const FShapesArray& ShapesArray() const { return reinterpret_cast<const FShapesArray&>(MShapesArray); }

	const FShapeInstanceProxyArray& ShapeInstances() const
	{ 
		return MShapesArray;
	}

	//Note: this must be called after setting geometry. This API seems bad. Should probably be part of setting geometry
	void SetShapeInstances(FShapeInstanceProxyArray&& InShapes)
	{ 
		// This is only called after setting or merging geometry which resizes the shapes array
		// so we should always have enough elements at this point
		check(InShapes.Num() == MShapesArray.Num());
		MShapesArray = MoveTemp(InShapes);
	}

	// Overwite the last N shapes with the data from InShapes. This is a highly dubious operation
	// provided as a utility for use in FPhysInterface_Chaos::AddGeometry. Ideally we would remove this
	void MergeShapeInstances(FShapeInstanceProxyArray&& InShapes)
	{
		check(InShapes.Num() <= (MShapesArray.Num()));
		int Idx = MShapesArray.Num() - InShapes.Num();
		for (FShapeInstanceProxyPtr& Shape : InShapes)
		{
			ensure(Idx < MShapesArray.Num());
			Shape->ModifyShapeIndex(Idx);
			MShapesArray[Idx++] = MoveTemp(Shape);
		}
	}


	void SetIgnoreAnalyticCollisionsImp(FImplicitObject* Implicit, bool bIgnoreAnalyticCollisions);
	
	CHAOS_API void SetIgnoreAnalyticCollisions(bool bIgnoreAnalyticCollisions);

	EObjectStateType ObjectState() const;

	EParticleType ObjectType() const
	{
		return Type;
	}

	

	const TKinematicGeometryParticle<T, d>* CastToKinematicParticle() const;
	TKinematicGeometryParticle<T, d>* CastToKinematicParticle();

	const TPBDRigidParticle<T, d>* CastToRigidParticle() const;
	TPBDRigidParticle<T, d>* CastToRigidParticle();

	FSpatialAccelerationIdx SpatialIdx() const { return MNonFrequentData.Read().SpatialIdx(); }
	void SetSpatialIdx(FSpatialAccelerationIdx Idx)
	{
		MNonFrequentData.Modify(true,MDirtyFlags,Proxy,[Idx](auto& Data){ Data.SetSpatialIdx(Idx);});
	}

	void SetResimType(EResimType ResimType)
	{
		MNonFrequentData.Modify(true, MDirtyFlags, Proxy, [ResimType](auto& Data) { Data.SetResimType(ResimType); });
	}

	EResimType ResimType() const
	{
		return MNonFrequentData.Read().ResimType();
	}

	void SetEnabledDuringResim(bool bEnabledDuringResim)
	{
		MNonFrequentData.Modify(true, MDirtyFlags, Proxy, [bEnabledDuringResim](auto& Data) { Data.SetEnabledDuringResim(bEnabledDuringResim); });
	}

	bool EnabledDuringResim() const
	{
		return MNonFrequentData.Read().EnabledDuringResim();
	}


	void SetNonFrequentData(const FParticleNonFrequentData& InData)
	{
		MNonFrequentData.Write(InData,true,MDirtyFlags,Proxy);
	}

	bool IsDirty() const
	{
		return MDirtyFlags.IsDirty();
	}

	bool IsClean() const
	{
		return MDirtyFlags.IsClean();
	}

	bool IsDirty(const EChaosPropertyFlags CheckBits) const
	{
		return MDirtyFlags.IsDirty(CheckBits);
	}

	void ForceDirty(EChaosPropertyFlags CheckBits)
	{
		MDirtyFlags.MarkDirty(CheckBits);
	}

	const FDirtyChaosPropertyFlags& DirtyFlags() const
	{
		return MDirtyFlags;
	}

	FDirtyChaosPropertyFlags& DirtyFlags()
	{
		return MDirtyFlags;
	}

	void ClearDirtyFlags()
	{
		MDirtyFlags.Clear();
	}
	
	TGeometryParticleHandle<T, d>* Handle() const
	{
		if (Proxy)
		{
			return static_cast<TGeometryParticleHandle<T, d>*>(Proxy->GetHandleUnsafe());
		}

		return nullptr;
	}


	void SyncRemoteData(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData, const TArray<int32>& ShapeDataIndices, FShapeDirtyData* ShapesRemoteData) const
	{
		RemoteData.SetParticleBufferType(Type);
		RemoteData.SetFlags(MDirtyFlags);
		SyncRemoteDataImp(Manager, DataIdx, RemoteData);

		for(const int32 ShapeDataIdx : ShapeDataIndices)
		{
			FShapeDirtyData& ShapeRemoteData = ShapesRemoteData[ShapeDataIdx];
			const int32 ShapeIdx = ShapeRemoteData.GetShapeIdx();
			MShapesArray[ShapeIdx]->SyncRemoteData(Manager, ShapeDataIdx, ShapeRemoteData);
		}
	}

	// Update the BVH in the geometry hierarchy if we have new geometry
	// This exists so that we don't repeatedly rebuild the BVH when welding, for example,
	// although ideally we would remove this and change the way welding works
	void PrepareBVH()
	{
		if (MNonFrequentData.IsDirty(MDirtyFlags))
		{
			PrepareBVHImpl();
		}
	}

	class IPhysicsProxyBase* GetProxy() const
	{
		return Proxy;
	}

	void SetProxy(IPhysicsProxyBase* InProxy)
	{
		Proxy = InProxy;
		if(Proxy)
		{
			if(MDirtyFlags.IsDirty())
			{
				if(FPhysicsSolverBase* PhysicsSolverBase = Proxy->GetSolver<FPhysicsSolverBase>())
				{
					PhysicsSolverBase->AddDirtyProxy(Proxy);
				}
			}
		}

		for(auto& Shape : MShapesArray)
		{
			Shape->SetProxy(Proxy);
		}
	}

protected:

	// Pointer to any data that the solver wants to associate with this particle
	// TODO: It's important to eventually hide this!
	// Right now it's exposed to lubricate the creation of the whole proxy system.
	class IPhysicsProxyBase* Proxy;

	/*
	* Describes how a call to ModifyGeometry wishes to access the underlying particle geometry
	*/
	enum class EGeometryAccess
	{
		// Grant direct access to the geometry, use with care - if the geometry is in
		// use inside a solver this will likely be unsafe. If the geometry is not
		// in use, all calls to ModifyGeometry will use Direct
		Direct,

		// Create a new root union, but shallow copy all geometries within the union.
		// This is fine if you are only removing shapes, or adding shapes. If you require
		// the ability to modify the actual geometry you must take a deep copy.
		ShallowCopy,

		// Create a new root union and deep copy all geometries within it. This is required
		// if you intend to actually modify the geometry itself and not just grow/shrink
		// the number of shapes on the particle
		DeepCopy
	};

	template <typename Lambda>
	void ModifyGeometry(const Lambda& Func, const bool bDirectAccess = false)
	{
		return ModifyGeometry(bDirectAccess ? EGeometryAccess::Direct : EGeometryAccess::DeepCopy, Func);
	}

	template <typename Lambda>
	void ModifyGeometry(EGeometryAccess AccessType, const Lambda& Func)
	{
		ensure(Chaos::CVars::bEnableAsyncInitBody || IsInGameThread());
		FPhysicsSolverBase* Solver = Proxy ? Proxy->GetSolverBase() : nullptr;

		if(Solver == nullptr)
		{
			//not registered yet so we can still modify geometry safely
			AccessType = EGeometryAccess::Direct;
		}
		else if(AccessType == EGeometryAccess::ShallowCopy && CVars::ForceDeepCopyOnModifyGeometry())
		{
			AccessType = EGeometryAccess::DeepCopy;
		}

		MNonFrequentData.Modify(true, MDirtyFlags, Proxy, [this, Solver, &Func, AccessType](auto& Data)
		{
			FImplicitObjectPtr GeomToModify = nullptr;
			bool bNewGeom = false;
			if(Data.GetGeometry())
			{
				switch(AccessType)
				{
				case EGeometryAccess::Direct:
					GeomToModify = Data.AccessGeometryDangerous();
					break;
				case EGeometryAccess::ShallowCopy:
					GeomToModify = Data.GetGeometry()->CopyGeometry();
					bNewGeom = true;
					break;
				case EGeometryAccess::DeepCopy:
					GeomToModify = Data.GetGeometry()->DeepCopyGeometry();
					bNewGeom = true;
					break;
				default:
					check(false);
					break;
				}

				Func(*GeomToModify);

				if(bNewGeom)
				{
					//must set geometry after because shapes are rebuilt and we want them to know about anything Func did
					Data.SetGeometry(GeomToModify);
				}
				else
				{
					// Only invalidate the geometry if we modified the existing one.
					// New geometry will be always evaluated the first time it is seen
					CVD_TRACE_INVALIDATE_CACHED_GEOMETRY(GeomToModify);
				}
				UpdateShapesArray();
			}
		});
	}
private:

	TChaosProperty<FParticlePositionRotation, EChaosProperty::XR> MXR;
	TChaosProperty<FParticleNonFrequentData,EChaosProperty::NonFrequentData> MNonFrequentData;
	void* MUserData;

	FShapeInstanceProxyArray MShapesArray;

public:
	UE_DEPRECATED(5.4, "Use GetGeometry instead")
	const TSharedPtr<const FImplicitObject, ESPMode::ThreadSafe>& GeometrySharedLowLevel() const
	{
		static TSharedPtr<const FImplicitObject, ESPMode::ThreadSafe> DummyPtr(nullptr);
		return DummyPtr;
	}
	
private:

protected:

	EParticleType Type;
	FDirtyChaosPropertyFlags MDirtyFlags;

	void MarkDirty(const EChaosPropertyFlags DirtyBits, bool bInvalidate = true);

	CHAOS_API void UpdateShapesArray();

	void PrepareBVHImpl();

	virtual void SyncRemoteDataImp(FDirtyPropertiesManager& Manager, int32 DataIdx, const FDirtyChaosProperties& RemoteData) const
	{
		MXR.SyncRemote(Manager, DataIdx, RemoteData);
		MNonFrequentData.SyncRemote(Manager, DataIdx, RemoteData);
	}
};

// Update all the shapes datas including the convexes ones
template<typename ParticleType>
void UpdateParticleShapes(const TArray<ParticleType*>& ShapesParticles, const FImplicitObject* ImplicitObject,
	const FShapesArray& ShapesArray, const int32 ActorId, const int32 ComponentID)
{
	if (!ShapesParticles.IsEmpty() && (ShapesParticles.Num() <= (ShapesArray.Num())))
	{
		for (int32 ParticleIndex = 0; ParticleIndex < ShapesParticles.Num(); ++ParticleIndex)
		{
			const TUniquePtr<Chaos::FPerShapeData>& ShapeData = ShapesArray[ParticleIndex];
			const TUniquePtr<Chaos::FPerShapeData>& TemplateShape = ShapesParticles[ParticleIndex]->ShapesArray()[0];
			
			if (ShapeData && TemplateShape)
			{
				{
					FCollisionData Data = TemplateShape->GetCollisionData();
					Data.UserData = nullptr;
					ShapeData->SetCollisionData(Data);
				}
    
				{
					const Chaos::Filter::FInstanceData InstanceData(ActorId, ComponentID);
					ShapeData->SetFilterInstanceData(InstanceData);
					ShapeData->SetShapeFilterData(TemplateShape->GetShapeFilterData());
				}
    
				ShapeData->SetSimEnabled(TemplateShape->GetSimEnabled());
				ShapeData->SetQueryEnabled(TemplateShape->GetQueryEnabled());
			}
		}
	}
}

template <typename T, int d>
FChaosArchive& operator<<(FChaosArchive& Ar, TGeometryParticle<T, d>& Particle)
{
	Particle.Serialize(Ar);
	return Ar;
}

template <typename T, int d>
class TKinematicGeometryParticle : public TGeometryParticle<T, d>
{
public:
	typedef TKinematicGeometryParticleHandle<T, d> FHandle;

	using TGeometryParticle<T, d>::Type;
	using TGeometryParticle<T, d>::CastToRigidParticle;
	using Base = TGeometryParticle<T,d>;
	using Base::MDirtyFlags;

protected:
	using Base::Proxy;

	friend TGeometryParticle<T,d>* TGeometryParticle<T, d>::SerializationFactory(FChaosArchive& Ar, TGeometryParticle<T, d>* Serializable);
	TKinematicGeometryParticle(const FKinematicGeometryParticleParameters& KinematicParams = FKinematicGeometryParticleParameters())
		: TGeometryParticle<T, d>(KinematicParams)
	{
		Type = EParticleType::Kinematic;
		KinematicGeometryParticleDefaultConstruct<T, d>(*this, KinematicParams);
	}
public:
	static TUniquePtr<TKinematicGeometryParticle<T, d>> CreateParticle(const FKinematicGeometryParticleParameters& Params = FKinematicGeometryParticleParameters())
	{
		return TUniquePtr< TKinematicGeometryParticle<T, d>>(new TKinematicGeometryParticle<T, d>(Params));
	}

	virtual void Serialize(FChaosArchive& Ar) override
	{
		TGeometryParticle<T, d>::Serialize(Ar);
		Ar << MVelocities;
		//Ar << MKinematicTarget; // TODO
	}

	const TVector<T, d> V() const { return MVelocities.Read().V(); }
	const TVector<T, d> GetV() const { return MVelocities.Read().V(); }
	void SetV(const TVector<T, d>& InV, bool bInvalidate = true);

	const TVector<T, d> W() const { return MVelocities.Read().W(); }
	const TVector<T, d> GetW() const { return MVelocities.Read().W(); }
	void SetW(const TVector<T, d>& InW, bool bInvalidate = true);

	const FKinematicTarget KinematicTarget() const {
		return MKinematicTarget.Read();
	}

	void SetKinematicTarget(const FKinematicTarget& KinematicTarget, bool bInvalidate = true)
	{
		MKinematicTarget.Write(KinematicTarget, bInvalidate, MDirtyFlags, Proxy);
	}

	bool IsKinematicTargetDirty() const
	{
		return MKinematicTarget.IsDirty(MDirtyFlags);
	}

	void ClearKinematicTarget()
	{
		MKinematicTarget.Clear(MDirtyFlags, Proxy);
	}

	const FParticleVelocities& Velocities() const { return MVelocities.Read(); }
	void SetVelocities(const FParticleVelocities& InVelocities,bool bInvalidate = true)
	{
		MVelocities.Write(InVelocities,bInvalidate,MDirtyFlags,Proxy);
	}

	EObjectStateType ObjectState() const;

	static TKinematicGeometryParticle<T, d>* Cast(TGeometryParticle<T, d>* Particle)
	{
		return Particle ? Particle->CastToKinematicParticle() : nullptr;
	}

	static const TKinematicGeometryParticle<T, d>* Cast(const TGeometryParticle<T, d>* Particle)
	{
		return Particle ? Particle->CastToKinematicParticle() : nullptr;
	}


private:
	TChaosProperty<FParticleVelocities, EChaosProperty::Velocities> MVelocities;
	TChaosProperty<FKinematicTarget, EChaosProperty::KinematicTarget> MKinematicTarget;

protected:
	virtual void SyncRemoteDataImp(FDirtyPropertiesManager& Manager, int32 DataIdx, const FDirtyChaosProperties& RemoteData) const
	{
		Base::SyncRemoteDataImp(Manager, DataIdx, RemoteData);
		MVelocities.SyncRemote(Manager, DataIdx, RemoteData);
		MKinematicTarget.SyncRemote(Manager, DataIdx, RemoteData);
	}
};

template <typename T, int d>
class TPBDRigidParticle : public TKinematicGeometryParticle<T, d>
{
public:
	typedef TPBDRigidParticleHandle<T, d> FHandle;

	using TGeometryParticle<T, d>::Type;
	using Base = TKinematicGeometryParticle<T,d>;

	using Base::MDirtyFlags;

protected:
	using Base::Proxy;
	friend TGeometryParticle<T, d>* TGeometryParticle<T, d>::SerializationFactory(FChaosArchive& Ar, TGeometryParticle<T, d>* Serializable);
	TPBDRigidParticle<T, d>(const FPBDRigidParticleParameters& DynamicParams = FPBDRigidParticleParameters())
		: TKinematicGeometryParticle<T, d>(DynamicParams), MWakeEvent(EWakeEventEntry::None)
	{
		Type = EParticleType::Rigid;
		MIsland = INDEX_NONE;
		PBDRigidParticleDefaultConstruct<T, d>(*this, DynamicParams);
		ClearForces();
		ClearTorques();
		SetObjectState(DynamicParams.bStartSleeping ? EObjectStateType::Sleeping : EObjectStateType::Dynamic);
		ClearEvents();
		SetInitialized(false);
	}
public:

	static TUniquePtr<TPBDRigidParticle<T, d>> CreateParticle(const FPBDRigidParticleParameters& DynamicParams = FPBDRigidParticleParameters())
	{
		return TUniquePtr< TPBDRigidParticle<T, d>>(new TPBDRigidParticle<T, d>(DynamicParams));
	}

	void Serialize(FChaosArchive& Ar) override
	{
		TKinematicGeometryParticle<T, d>::Serialize(Ar);
		Ar << MDynamics;

		Ar << MIsland;

		// remove on fracture is deprecated and has been removed, so to avoid versioning let's just use a local variable instead
		bool MToBeRemovedOnFracture_deprecated = false;
		Ar << MToBeRemovedOnFracture_deprecated;
	}

	//const TUniquePtr<TBVHParticles<T, d>>& CollisionParticles() const { return MCollisionParticles; }

	int32 CollisionGroup() const { return MMiscData.Read().CollisionGroup(); }
	void SetCollisionGroup(const int32 InCollisionGroup)
	{
		MMiscData.Modify(true,MDirtyFlags,Proxy,[InCollisionGroup](auto& Data){ Data.SetCollisionGroup(InCollisionGroup);});
	}

	/*
	bool Disabled() const { return MMiscData.Read().bDisabled; }
	void SetDisabled(const bool InDisabled)
	{
		MMiscData.Modify(true,MDirtyFlags,Proxy,[InDisabled](auto& Data){ Data.bDisabled = InDisabled;});
	}*/

	bool GravityEnabled() const { return MMiscData.Read().GravityEnabled(); }
	void SetGravityEnabled(const bool bInEnabled)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [bInEnabled](auto& Data) { Data.SetGravityEnabled(bInEnabled); });
	}

	int32 GravityGroupIndex() const { return MMiscData.Read().GravityGroupIndex(); }
	void SetGravityGroupIndex(int32 NewIndex)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [NewIndex](auto& Data) { Data.SetGravityGroupIndex(NewIndex); });
	}
	
	bool UpdateKinematicFromSimulation() const { return MMiscData.Read().UpdateKinematicFromSimulation(); }
	void SetUpdateKinematicFromSimulation(const bool bUpdateKinematicFromSimulation)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [bUpdateKinematicFromSimulation](auto& Data) { Data.SetUpdateKinematicFromSimulation(bUpdateKinematicFromSimulation); });
	}

	bool GyroscopicTorqueEnabled() const { return MMiscData.Read().GyroscopicTorqueEnabled(); }
	void SetGyroscopicTorqueEnabled(const bool bGyroscopicTorqueEnabled)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [bGyroscopicTorqueEnabled](auto& Data) { Data.SetGyroscopicTorqueEnabled(bGyroscopicTorqueEnabled); });
	}

	bool OneWayInteraction() const { return MMiscData.Read().OneWayInteraction(); }
	void SetOneWayInteraction(const bool bInEnabled)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [bInEnabled](auto& Data) { Data.SetOneWayInteraction(bInEnabled); });
	}

	// Enable a single flag
	void AddCollisionConstraintFlag(const ECollisionConstraintFlags Flag)
	{ 
		MMiscData.Modify(true, MDirtyFlags, Proxy, [Flag](auto& Data) { Data.AddCollisionConstraintFlag(Flag); });
	}
	// Disable a single flag
	void RemoveCollisionConstraintFlag(const ECollisionConstraintFlags Flag)
	{ 
		MMiscData.Modify(true, MDirtyFlags, Proxy, [Flag](auto& Data) { Data.RemoveCollisionConstraintFlag(Flag); });
	}
	// A mask of all the active flags
	uint32 CollisionConstraintFlags() const { return MMiscData.Read().CollisionConstraintFlags(); }
	// Replace all flags
	void SetCollisionConstraintFlags(const uint32 Flags)
	{ 
		MMiscData.Modify(true, MDirtyFlags, Proxy, [Flags](auto& Data) { Data.SetCollisionConstraintFlags(Flags); });
	}

	bool CCDEnabled() const { return MMiscData.Read().CCDEnabled(); }
	void SetCCDEnabled(bool bInEnabled)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [bInEnabled](auto& Data) { Data.SetCCDEnabled(bInEnabled); });
	}

	bool MACDEnabled() const { return MMiscData.Read().MACDEnabled(); }
	void SetMACDEnabled(bool bInEnabled)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [bInEnabled](auto& Data) { Data.SetMACDEnabled(bInEnabled); });
	}

	bool PartialIslandSleepAllowed() const { return MMiscData.Read().PartialIslandSleepAllowed(); }
	void SetPartialIslandSleepAllowed(bool bInEnabled)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [bInEnabled](auto& Data) { Data.SetPartialIslandSleepAllowed(bInEnabled); });
	}

	Private::FIterationSettings IterationSettings() const { return MMiscData.Read().IterationSettings(); }

	void SetIterationSettings(const Private::FIterationSettings& SolverIterationSettingsIn)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [SolverIterationSettingsIn](auto& Data) { Data.SetIterationSettings(SolverIterationSettingsIn); });
	}

	void SetPositionSolverIterations(const int32 PositionSolverIterationCount)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [PositionSolverIterationCount](auto& Data) { Data.SetPositionSolverIterationCount(PositionSolverIterationCount); });
	}

	void SetVelocitySolverIterations(const int32 VelocitySolverIterationCount)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [VelocitySolverIterationCount](auto& Data) { Data.SetVelocitySolverIterationCount(VelocitySolverIterationCount); });
	}

	void SetProjectionSolverIterations(const int32 ProjectionSolverIterationCount)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [ProjectionSolverIterationCount](auto& Data) { Data.SetProjectionSolverIterationCount(ProjectionSolverIterationCount); });
	}

	bool InertiaConditioningEnabled() const { return MMiscData.Read().InertiaConditioningEnabled(); }
	void SetInertiaConditioningEnabled(bool bInEnabled)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [bInEnabled](auto& Data) { Data.SetInertiaConditioningEnabled(bInEnabled); });
	}

	bool Disabled() const { return MMiscData.Read().Disabled(); }

	void SetDisabled(bool bInDisabled)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [bInDisabled](auto& Data) {Data.SetDisabled(bInDisabled); });
	}

	//todo: remove this
	bool IsInitialized() const { return MInitialized; }
	void SetInitialized(const bool InInitialized)
	{
		this->MInitialized = InInitialized;
	}

	const TVector<T, d> Acceleration() const { return MDynamics.Read().Acceleration(); }
	void SetAcceleration(const FVec3& Acceleration, bool bInvalidate = true)
	{ 
		MDynamics.Modify(bInvalidate, MDirtyFlags, Proxy, [&Acceleration](auto& Data) { Data.SetAcceleration(Acceleration); });
	}

	void AddForce(const TVector<T, d>& InF, bool bInvalidate = true)
	{
		FReal InvMass = InvM();
		MDynamics.Modify(bInvalidate,MDirtyFlags,Proxy,[&InF, InvMass](auto& Data){ Data.SetAcceleration(InF * InvMass + Data.Acceleration());});
	}

	void ClearForces(bool bInvalidate = true)
	{
		MDynamics.Modify(bInvalidate, MDirtyFlags, Proxy, [](auto& Data) { Data.SetAcceleration(FVec3(0)); });
	}

	void ApplyDynamicsWeight(const FReal DynamicsWeight)
	{
		if (MDynamics.IsDirty(MDirtyFlags))
		{
			MDynamics.Modify(false, MDirtyFlags, Proxy, [DynamicsWeight](auto& Data)
			{
				Data.SetAcceleration(Data.Acceleration() * DynamicsWeight);
				Data.SetAngularAcceleration(Data.AngularAcceleration() * DynamicsWeight);
			});
		}
	}

	const TVector<T, d> AngularAcceleration() const { return MDynamics.Read().AngularAcceleration(); }
	void SetAngularAcceleration(const TVector<T, d>& InTorque, bool bInvalidate = true)
	{
		MDynamics.Modify(bInvalidate, MDirtyFlags, Proxy, [&InTorque](auto& Data) { Data.SetAngularAcceleration(InTorque);});
	}
	CHAOS_API void AddTorque(const TVector<T, d>& InTorque, bool bInvalidate=true);

	void ClearTorques(bool bInvalidate = true)
	{
		MDynamics.Modify(bInvalidate, MDirtyFlags, Proxy, [](auto& Data) { Data.SetAngularAcceleration(FVec3(0)); });
	}

	const TVector<T, d> LinearImpulseVelocity() const { return MDynamics.Read().LinearImpulseVelocity(); }
	void SetLinearImpulseVelocity(const TVector<T, d>& InLinearImpulseVelocity, bool bInvalidate = true)
	{
		MDynamics.Modify(bInvalidate,MDirtyFlags,Proxy,[&InLinearImpulseVelocity](auto& Data){ Data.SetLinearImpulseVelocity(InLinearImpulseVelocity);});
	}

	const TVector<T, d> AngularImpulseVelocity() const { return MDynamics.Read().AngularImpulseVelocity(); }
	void SetAngularImpulseVelocity(const TVector<T, d>& InAngularImpulseVelocity, bool bInvalidate = true)
	{
		MDynamics.Modify(bInvalidate,MDirtyFlags,Proxy,[&InAngularImpulseVelocity](auto& Data){ Data.SetAngularImpulseVelocity(InAngularImpulseVelocity);});
	}

	void SetDynamics(const FParticleDynamics& InDynamics,bool bInvalidate = true)
	{
		MDynamics.Write(InDynamics,bInvalidate,MDirtyFlags,Proxy);
	}

	void ResetSmoothedVelocities()
	{
		// Physics thread only. API required for FGeometryParticleStateBase::SyncToParticle
	}

	const TVec3<FRealSingle>& I() const { return MMassProps.Read().I(); }
	void SetI(const TVec3<FRealSingle>& InI, bool bInvalidate = true)
	{
		MMassProps.Modify(bInvalidate,MDirtyFlags,Proxy,[&InI](auto& Data){ Data.SetI(InI);});
	}

	const TVec3<FRealSingle>& InvI() const { return MMassProps.Read().InvI(); }
	void SetInvI(const TVec3<FRealSingle>& InInvI, bool bInvalidate = true)
	{
		MMassProps.Modify(bInvalidate,MDirtyFlags,Proxy,[&InInvI](auto& Data){ Data.SetInvI(InInvI);});
	}

	T M() const { return MMassProps.Read().M(); }
	void SetM(const T& InM, bool bInvalidate = true)
	{
		MMassProps.Modify(bInvalidate,MDirtyFlags,Proxy,[InM](auto& Data){ Data.SetM(InM);});
	}

	T InvM() const { return MMassProps.Read().InvM(); }
	void SetInvM(const T& InInvM, bool bInvalidate = true)
	{
		MMassProps.Modify(bInvalidate,MDirtyFlags,Proxy,[InInvM](auto& Data){ Data.SetInvM(InInvM);});
	}
	
	const TVector<T,d>& CenterOfMass() const { return MMassProps.Read().CenterOfMass(); }
	void SetCenterOfMass(const TVector<T,d>& InCenterOfMass,bool bInvalidate = true)
	{
		MMassProps.Modify(bInvalidate,MDirtyFlags,Proxy,[&InCenterOfMass](auto& Data){ Data.SetCenterOfMass(InCenterOfMass);});
	}

	const TRotation<T,d>& RotationOfMass() const { return MMassProps.Read().RotationOfMass(); }
	void SetRotationOfMass(const TRotation<T,d>& InRotationOfMass,bool bInvalidate = true)
	{
		MMassProps.Modify(bInvalidate,MDirtyFlags,Proxy,[&InRotationOfMass](auto& Data){ Data.SetRotationOfMass(InRotationOfMass);});
	}

	void SetMassProps(const FParticleMassProps& InProps)
	{
		MMassProps.Write(InProps,true,MDirtyFlags,Proxy);
	}

	void SetDynamicMisc(const FParticleDynamicMisc& DynamicMisc)
	{
		MMiscData.Write(DynamicMisc,true,MDirtyFlags,Proxy);
	}

	T LinearEtherDrag() const { return MMiscData.Read().LinearEtherDrag(); }
	void SetLinearEtherDrag(const T& InLinearEtherDrag)
	{
		MMiscData.Modify(true,MDirtyFlags,Proxy,[&InLinearEtherDrag](auto& Data){ Data.SetLinearEtherDrag(InLinearEtherDrag);});
	}

	T AngularEtherDrag() const { return MMiscData.Read().AngularEtherDrag(); }
	void SetAngularEtherDrag(const T& InAngularEtherDrag)
	{
		MMiscData.Modify(true,MDirtyFlags,Proxy,[&InAngularEtherDrag](auto& Data){ Data.SetAngularEtherDrag(InAngularEtherDrag);});
	}

	T MaxLinearSpeedSq() const { return MMiscData.Read().MaxLinearSpeedSq(); }
	void SetMaxLinearSpeedSq(const T& InLinearSpeed)
	{
		MMiscData.Modify(true,MDirtyFlags,Proxy,[&InLinearSpeed](auto& Data){ Data.SetMaxLinearSpeedSq(InLinearSpeed);});
	}

	T MaxAngularSpeedSq() const { return MMiscData.Read().MaxAngularSpeedSq(); }
	void SetMaxAngularSpeedSq(const T& InAngularSpeed)
	{
		MMiscData.Modify(true,MDirtyFlags,Proxy,[&InAngularSpeed](auto& Data){ Data.SetMaxAngularSpeedSq(InAngularSpeed);});
	}

	FRealSingle InitialOverlapDepenetrationVelocity() const { return MMiscData.Read().InitialOverlapDepenetrationVelocity(); }
	void SetInitialOverlapDepenetrationVelocity(FRealSingle InVel)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [InVel](auto& Data) { Data.SetInitialOverlapDepenetrationVelocity(InVel); });
	}

	FRealSingle SleepThresholdMultiplier() const { return MMiscData.Read().SleepThresholdMultiplier(); }
	void SetSleepThresholdMultiplier(FRealSingle Multiplier)
	{
		MMiscData.Modify(true, MDirtyFlags, Proxy, [Multiplier](auto& Data) { Data.SetSleepThresholdMultiplier(Multiplier); });
	}

	int32 Island() const { return MIsland; }
	// TODO(stett): Make the setter private. It is public right now to provide access to proxies.
	void SetIsland(const int32 InIsland)
	{
		this->MIsland = InIsland;
	}

	EObjectStateType ObjectState() const { return MMiscData.Read().ObjectState(); }
	void SetObjectState(const EObjectStateType InState, bool bAllowEvents=false, bool bInvalidate=true)
	{
		if (bAllowEvents)
		{
			const auto PreState = ObjectState();
			if(PreState == EObjectStateType::Dynamic && InState == EObjectStateType::Sleeping)
			{
				MWakeEvent = EWakeEventEntry::Sleep;
			}
			else if(PreState == EObjectStateType::Sleeping && InState == EObjectStateType::Dynamic)
			{
				MWakeEvent = EWakeEventEntry::Awake;
			}
		}

		if (InState == EObjectStateType::Sleeping)
		{
			// When an object is forced into a sleep state, the velocities must be zeroed and buffered,
			// in case the velocity is queried during sleep, or in case the object is woken up again.
			this->SetV(FVec3(0.f), bInvalidate);
			this->SetW(FVec3(0.f), bInvalidate);

			// Dynamic particle properties must be marked clean in order not to actually apply forces which
			// have been buffered. If another force is added after the object is put to sleep, the old forces
			// will remain and the new ones will accumulate and re-dirty the dynamic properties which will
			// wake the body.
			MDirtyFlags.MarkClean(ChaosPropertyToFlag(EChaosProperty::Dynamics));
		}

		MMiscData.Modify(bInvalidate,MDirtyFlags,Proxy,[&InState](auto& Data){ Data.SetObjectState(InState);});

	}

	void SetSleepType(ESleepType SleepType, bool bAllowEvents=false, bool bInvalidate=true)
	{
		MMiscData.Modify(true,MDirtyFlags,Proxy,[SleepType](auto& Data){ Data.SetSleepType(SleepType);});

		if (SleepType == ESleepType::NeverSleep && ObjectState() == EObjectStateType::Sleeping)
		{
			SetObjectState(EObjectStateType::Dynamic, bAllowEvents, bInvalidate);
		}
	}

	ESleepType SleepType() const
	{
		return MMiscData.Read().SleepType();
	}

	void ClearEvents() { MWakeEvent = EWakeEventEntry::None; }
	EWakeEventEntry GetWakeEvent() { return MWakeEvent; }

	static TPBDRigidParticle<T, d>* Cast(TGeometryParticle<T, d>* Particle)
	{
		return Particle ? Particle->CastToRigidParticle() : nullptr;
	}

	static const TPBDRigidParticle<T, d>* Cast(const TGeometryParticle<T, d>* Particle)
	{
		return Particle ? Particle->CastToRigidParticle() : nullptr;
	}

	FRigidParticleControlFlags ControlFlags() const
	{
		return MMiscData.Read().ControlFlags();
	}

	void SetControlFlags(const FRigidParticleControlFlags& Flags)
	{
		MMiscData.Modify(true,MDirtyFlags,Proxy,[Flags](auto& Data){ Data.SetControlFlags(Flags);});
	}

private:
	TChaosProperty<FParticleMassProps,EChaosProperty::MassProps> MMassProps;
	TChaosProperty<FParticleDynamics, EChaosProperty::Dynamics> MDynamics;
	TChaosProperty<FParticleDynamicMisc,EChaosProperty::DynamicMisc> MMiscData;

	int32 MIsland;
	bool MInitialized;
	EWakeEventEntry MWakeEvent;

protected:
	virtual void SyncRemoteDataImp(FDirtyPropertiesManager& Manager, int32 DataIdx, const FDirtyChaosProperties& RemoteData) const
	{
		Base::SyncRemoteDataImp(Manager,DataIdx,RemoteData);
		MMassProps.SyncRemote(Manager,DataIdx,RemoteData);
		MDynamics.SyncRemote(Manager,DataIdx,RemoteData);
		MMiscData.SyncRemote(Manager,DataIdx,RemoteData);
	}
};

class FPBDGeometryCollectionParticle : public TPBDRigidParticle<FReal, 3>
{
public:
	typedef TPBDGeometryCollectionParticleHandle<FReal, 3> FHandle;

	using FGeometryParticle::Type;
public:
	FPBDGeometryCollectionParticle(const FPBDRigidParticleParameters& DynamicParams = FPBDRigidParticleParameters())
		: FPBDRigidParticle(DynamicParams)
	{
		Type = EParticleType::GeometryCollection;
	}

	static TUniquePtr<FPBDGeometryCollectionParticle> CreateParticle(const FPBDRigidParticleParameters& DynamicParams = FPBDRigidParticleParameters())
	{
		return TUniquePtr<FPBDGeometryCollectionParticle>(new FPBDGeometryCollectionParticle(DynamicParams));
	}
};

// holding on the deprecation for now as inter dependencies require the use of TPBDGeometryCollectionParticle<> 
template <typename T, int d>
using TPBDGeometryCollectionParticle /* UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPBDGeometryCollectionParticle instead") */ = FPBDGeometryCollectionParticle;

template <typename T, int d>
const TKinematicGeometryParticle<T, d>* TGeometryParticle<T, d>::CastToKinematicParticle() const
{
	if (Type >= EParticleType::Kinematic)
	{
		return static_cast<const TKinematicGeometryParticle<T, d>*>(this);
	}

	return nullptr;
}

template <typename T, int d>
TKinematicGeometryParticle<T, d>* TGeometryParticle<T, d>::CastToKinematicParticle()
{
	if (Type >= EParticleType::Kinematic)
	{
		return static_cast<TKinematicGeometryParticle<T, d>*>(this);
	}

	return nullptr;
}

template <typename T, int d>
TPBDRigidParticle<T, d>* TGeometryParticle<T, d>::CastToRigidParticle() 
{
	if (Type >= EParticleType::Rigid)
	{
		return static_cast<TPBDRigidParticle<T, d>*>(this);
	}

	return nullptr;
}


template <typename T, int d>
const TPBDRigidParticle<T, d>* TGeometryParticle<T, d>::CastToRigidParticle()  const
{
	if (Type >= EParticleType::Rigid)
	{
		return static_cast<const TPBDRigidParticle<T, d>*>(this);
	}

	return nullptr;
}

template <typename T, int d>
void TGeometryParticle<T, d>::SetX(const TVector<T, d>& InX, bool bInvalidate)
{
	MXR.Modify(bInvalidate, MDirtyFlags, Proxy, [&InX](auto& Data) { Data.SetX(InX); });
}

template <typename T, int d>
void TGeometryParticle<T, d>::SetR(const TRotation<T, d>& InR, bool bInvalidate)
{
	MXR.Modify(bInvalidate, MDirtyFlags, Proxy, [&InR](auto& Data) { Data.SetR(InR); });
}

template <typename T, int d>
EObjectStateType TGeometryParticle<T, d>::ObjectState() const
{
	const TKinematicGeometryParticle<T, d>* Kin = CastToKinematicParticle();
	return Kin ? Kin->ObjectState() : EObjectStateType::Static;
}

template <typename T, int d>
EObjectStateType TKinematicGeometryParticle<T, d>::ObjectState() const
{
	const TPBDRigidParticle<T, d>* Dyn = CastToRigidParticle();
	return Dyn ? Dyn->ObjectState() : EObjectStateType::Kinematic;
}

template <typename T, int d>
void TKinematicGeometryParticle<T, d>::SetV(const TVector<T, d>& InV, bool bInvalidate)
{
	MVelocities.Modify(bInvalidate, MDirtyFlags, Proxy, [&InV](auto& Data) { Data.SetV(InV); });
}

template <typename T, int d>
void TKinematicGeometryParticle<T, d>::SetW(const TVector<T, d>& InW, bool bInvalidate)
{
	MVelocities.Modify(bInvalidate, MDirtyFlags, Proxy, [&InW](auto& Data) { Data.SetW(InW); });
}

template <typename T, int d>
TGeometryParticle<T, d>* TGeometryParticle<T, d>::SerializationFactory(FChaosArchive& Ar, TGeometryParticle<T, d>* Serializable)
{
	int8 ObjectType = Ar.IsLoading() ? 0 : (int8)Serializable->Type;
	Ar << ObjectType;
	switch ((EParticleType)ObjectType)
	{
	case EParticleType::Static: if (Ar.IsLoading()) { return new TGeometryParticle<T, d>(); } break;
	case EParticleType::Kinematic: if (Ar.IsLoading()) { return new TKinematicGeometryParticle<T, d>(); } break;
	case EParticleType::Rigid: if (Ar.IsLoading()) { return new TPBDRigidParticle<T, d>(); } break;
	case EParticleType::GeometryCollection: if (Ar.IsLoading()) { return new TPBDGeometryCollectionParticle<T, d>(); } break;
	default:
		check(false);
	}
	return nullptr;
}

template <>
CHAOS_API void Chaos::TGeometryParticle<FReal, 3>::MarkDirty(const EChaosPropertyFlags DirtyBits, bool bInvalidate);

FORCEINLINE_DEBUGGABLE FAccelerationStructureHandle::FAccelerationStructureHandle(FGeometryParticleHandle* InHandle)
	: ExternalGeometryParticle(InHandle->GTGeometryParticle())
	, GeometryParticleHandle(InHandle)
	, CachedUniqueIdx(InHandle->UniqueIdx())
	, bCanPrePreFilter(false)
{
	ensure(CachedUniqueIdx.IsValid());
	if (InHandle)
	{
		UpdatePrePreFilter(*InHandle);
	}
}

FORCEINLINE_DEBUGGABLE FAccelerationStructureHandle::FAccelerationStructureHandle(FGeometryParticle* InGeometryParticle, bool bUsePrefiltering)
	: ExternalGeometryParticle(InGeometryParticle)
	, GeometryParticleHandle(InGeometryParticle ? InGeometryParticle->Handle() : nullptr)
	, CachedUniqueIdx(InGeometryParticle ? InGeometryParticle->UniqueIdx() : FUniqueIdx())
	, bCanPrePreFilter(false)
{
	if (InGeometryParticle)
	{
		ensure(CachedUniqueIdx.IsValid());
		ensure(Chaos::CVars::bEnableAsyncInitBody || IsInGameThread());
		if (bUsePrefiltering)
		{
			UpdatePrePreFilter(*InGeometryParticle);
		}
	}
}

template <bool bPersistent>
FORCEINLINE_DEBUGGABLE FAccelerationStructureHandle::FAccelerationStructureHandle(TGeometryParticleHandleImp<FReal, 3, bPersistent>& InHandle)
	: ExternalGeometryParticle(InHandle.GTGeometryParticle())
	, GeometryParticleHandle(InHandle.Handle())
	, CachedUniqueIdx(InHandle.UniqueIdx())
	, bCanPrePreFilter(false)
{
	ensure(CachedUniqueIdx.IsValid());
	UpdatePrePreFilter(InHandle);
}

template <typename TParticle>
/* static */ void FAccelerationStructureHandle::ComputeParticleQueryFilterDataFromShapes(const TParticle& Particle, FCollisionFilterData& OutQueryFilterData)
{
	const auto& Shapes = Particle.ShapesArray();
	for (const auto& Shape : Shapes)
	{
		// For queries we can ignore any shapes that are not query-enabled (e.g., sim- and probe-only)
		if (Shape->GetQueryEnabled())
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			const FCollisionFilterData& ShapeQueryData = Shape->GetQueryData();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			OutQueryFilterData.Word0 |= ShapeQueryData.Word0;
			OutQueryFilterData.Word1 |= ShapeQueryData.Word1;
			OutQueryFilterData.Word2 |= ShapeQueryData.Word2;
			OutQueryFilterData.Word3 |= ShapeQueryData.Word3;
		}
	}
}

template <typename TParticle>
/* static */ void FAccelerationStructureHandle::ComputeFilterDataFromShapes(const TParticle& Particle, Chaos::Filter::FShapeUnionFilterData& OutFilter)
{
	const FShapesArray& Shapes = Particle.ShapesArray();
	OutFilter = Chaos::Filter::FShapeUnionFilterData();
	for (const auto& Shape : Shapes)
	{
		const bool bSimEnabled = Shape->GetSimEnabled() || Shape->GetIsProbe();
		const bool bQueryEnabled = Shape->GetQueryEnabled();
		OutFilter.Combine(Shape->GetShapeFilterData(), bQueryEnabled, bSimEnabled);
	}
}

template <typename TParticle>
FORCEINLINE_DEBUGGABLE void FAccelerationStructureHandle::UpdatePrePreFilter(const TParticle& Particle)
{
	ComputeFilterDataFromShapes(Particle, UnionFilter);
	bCanPrePreFilter = true;
}


FORCEINLINE_DEBUGGABLE void FAccelerationStructureHandle::Serialize(FChaosArchive& Ar)
{
	Ar << AsAlwaysSerializable(ExternalGeometryParticle);
	Ar << AsAlwaysSerializable(GeometryParticleHandle);

	Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
	if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::SerializeHashResult && Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::UniquePayloadIdx)
	{
		uint32 DummyHash;
		Ar << DummyHash;
	}
	
	if(GeometryParticleHandle)
	{
		CachedUniqueIdx = GeometryParticleHandle->UniqueIdx();
	}
	else if(ExternalGeometryParticle)
	{
		CachedUniqueIdx = ExternalGeometryParticle->UniqueIdx();
	}
	
	if (GeometryParticleHandle && ExternalGeometryParticle)
	{
		ensure(GeometryParticleHandle->UniqueIdx() == ExternalGeometryParticle->UniqueIdx());
	}
	
	ensure(!GeometryParticleHandle || CachedUniqueIdx.IsValid());
	ensure(!ExternalGeometryParticle || CachedUniqueIdx.IsValid());
}

FORCEINLINE_DEBUGGABLE FChaosArchive& operator<<(FChaosArchive& Ar, FAccelerationStructureHandle& AccelerationHandle)
{
	AccelerationHandle.Serialize(Ar);
	return Ar;
}

#if CHAOS_DEBUG_DRAW
FORCEINLINE_DEBUGGABLE void FAccelerationStructureHandle::DebugDraw(const bool bExternal, const bool bHit) const
{
	if (ExternalGeometryParticle && bExternal)
	{
		DebugDraw::DrawParticleShapes(FRigidTransform3(), ExternalGeometryParticle, bHit ? FColor::Red : FColor::Green);
	}

	if (GeometryParticleHandle && !bExternal)
	{
		DebugDraw::DrawParticleShapes(FRigidTransform3(), GeometryParticleHandle, bHit ? FColor(200, 100, 100) : FColor(100, 200, 100));
	}
}
#endif

inline void SetObjectStateHelper(IPhysicsProxyBase& Proxy, FPBDRigidParticle& Rigid, EObjectStateType InState, bool bAllowEvents = false, bool bInvalidate = true)
{
	Rigid.SetObjectState(InState, bAllowEvents, bInvalidate);
}

CHAOS_API void SetObjectStateHelper(IPhysicsProxyBase& Proxy, FPBDRigidParticleHandle& Rigid, EObjectStateType InState, bool bAllowEvents = false, bool bInvalidate = true);

#if !UE_MERGED_MODULES
#if PLATFORM_COMPILER_CLANG
extern template class CHAOS_API ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>;
extern template class CHAOS_API ISpatialVisitor<FAccelerationStructureHandle, FReal>;
#else
extern template class ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>;
extern template class ISpatialVisitor<FAccelerationStructureHandle, FReal>;
#endif
#endif

} // namespace Chaos

