// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Real.h"
#include "Framework/ThreadContextEnum.h"
#include <type_traits>

// Use to define out code blocks that need to be adapted to use Particle Handles in a searchable way (better than #if 0)
#define CHAOS_PARTICLEHANDLE_TODO 0

namespace Chaos
{
	// TGeometryParticle 

	template <typename T, int d>
	class TGeometryParticle;

	template <typename T, int d, bool bProcessing>
	class TGeometryParticleHandleImp;

	template <typename T, int d>
	using TGeometryParticleHandle = TGeometryParticleHandleImp<T, d, true>;

	using FGeometryParticleHandle = TGeometryParticleHandle<FReal, 3>;

	template <typename T, int d>
	using TTransientGeometryParticleHandle = TGeometryParticleHandleImp<T, d, false>;

	using FTransientGeometryParticleHandle = TTransientGeometryParticleHandle<FReal, 3>;

	// TKinematicGeometryParticle

	template <typename T, int d>
	class TKinematicGeometryParticle;

	template <typename T, int d, bool bProcessing>
	class TKinematicGeometryParticleHandleImp;

	template <typename T, int d>
	using TKinematicGeometryParticleHandle = TKinematicGeometryParticleHandleImp<T, d, true>;

	using FKinematicGeometryParticleHandle = TKinematicGeometryParticleHandle<FReal, 3>;

	template <typename T, int d>
	using TTransientKinematicGeometryParticleHandle = TKinematicGeometryParticleHandleImp<T, d, false>;

	using FTransientKinematicGeometryParticleHandle = TTransientKinematicGeometryParticleHandle<FReal, 3>;

	// TPBDRigidParticle

	template <typename T, int d>
	class TPBDRigidParticle;

	template <typename T, int d, bool bProcessing>
	class TPBDRigidParticleHandleImp;

	template <typename T, int d>
	using TPBDRigidParticleHandle = TPBDRigidParticleHandleImp<T, d, true>;

	using FPBDRigidParticleHandle = TPBDRigidParticleHandle<FReal, 3>;

	template <typename T, int d>
	using TTransientPBDRigidParticleHandle = TPBDRigidParticleHandleImp<T, d, false>;

	using FTransientPBDRigidParticleHandle = TTransientPBDRigidParticleHandle<FReal, 3>;

	// TPBDRigidClusteredParticleHandle

	template <typename T, int d, bool bProcessing>
	class TPBDRigidClusteredParticleHandleImp;

	template <typename T, int d>
	using TPBDRigidClusteredParticleHandle = TPBDRigidClusteredParticleHandleImp<T, d, true>;

	using FPBDRigidClusteredParticleHandle = TPBDRigidClusteredParticleHandle<FReal, 3>;

	template <typename T, int d>
	using TTransientPBDRigidClusteredParticleHandle = TPBDRigidClusteredParticleHandleImp<T, d, false>;

	using FTransientPBDRigidClusteredParticleHandle = TTransientPBDRigidClusteredParticleHandle<FReal, 3>;

	// TPBDGeometryCollectionParticleHandle

	class FPBDGeometryCollectionParticle;

	template <typename T, int d, bool bPersistent>
	class TPBDGeometryCollectionParticleHandleImp;

	template <typename T, int d>
	using TPBDGeometryCollectionParticleHandle = TPBDGeometryCollectionParticleHandleImp<T, d, true>;

	using FPBDGeometryCollectionParticleHandle = TPBDGeometryCollectionParticleHandle<FReal, 3>;

	template <typename T, int d>
	using TTransientPBDGeometryCollectionParticleHandle = TPBDGeometryCollectionParticleHandleImp<T, d, false>;
	
	// Generic Particle Handle (syntactic sugar wrapping any particle handle type and providing unified API)

	class FGenericParticleHandle;

	class FConstGenericParticleHandle;

	// TParticleIterator

	template <typename TSOA>
	class TParticleIterator;

	template <typename TSOA>
	class TConstParticleIterator;

	// Game Thread particles

	using FGeometryParticle = TGeometryParticle<FReal, 3>;
	using FKinematicGeometryParticle = TKinematicGeometryParticle<FReal, 3>;
	using FPBDRigidParticle = TPBDRigidParticle<FReal, 3>;

	// Thread-templated particles

	template<EThreadContext Id>
	using TThreadParticle = std::conditional_t<Id == EThreadContext::External, FGeometryParticle, FGeometryParticleHandle>;

	template<EThreadContext Id>
	using TThreadKinematicParticle = std::conditional_t<Id == EThreadContext::External, FKinematicGeometryParticle, FKinematicGeometryParticleHandle>;

	template<EThreadContext Id>
	using TThreadRigidParticle = std::conditional_t<Id == EThreadContext::External, FPBDRigidParticle, FPBDRigidParticleHandle>;
}
