// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Framework/Parallel.h"
#include "Chaos/ParallelFor.h"
#include "Chaos/Particles.h"
#include "ChaosStats.h"

DECLARE_CYCLE_STAT(TEXT("ParticlesSequentialFor"), STAT_ParticlesSequentialFor, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("ParticlesParallelFor"), STAT_ParticlesParallelFor, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("ParticleViewParallelForImp"), STAT_ParticleViewParallelForImp, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("HandleViewParallelForImp"), STAT_HandleViewParallelForImp, STATGROUP_Chaos);

namespace Chaos
{
template <typename TSOA>
class TConstParticleView;

template <typename TSOA>
class TParticleView;

template <typename TSOA>
class TConstHandleView;

template <typename TSOA>
class THandleView;

// The function ParticlesParallelFor may be called on ParticleViews, HandleViews,
// or plain old manually curated arrays of either. In each case, the implementation
// can differ. The following set of templates will select for the right case.
//
// Signatures:
// Lambda:				void(auto& Particle, int32 Index, const int32 ContextIndex)
// ContextCreator:		int32 (int32 WorkerIndex, int32 NumWorkers)
//
// There will be one context creation call per worker thread and they are all created prior
// to starting the work. The ContextCreator should return the context index (which is
// usually just the WorkerIndex but, for example, it can also be used to generate a single context
// for use by all workers, assuming the context has appropriate locks in place.)
//

template <typename TParticleView, typename ContextCreatorType, typename Lambda>
void ParticleViewParallelForImp(const TParticleView& Particles, const ContextCreatorType& ContextCreator, const Lambda& Func);

template <typename THandleView, typename ContextCreatorType, typename Lambda>
void HandleViewParallelForImp(const THandleView& HandleView, const ContextCreatorType& ContextCreator, const Lambda& Func);

template <typename TSOA, typename ContextCreatorType, typename Lambda>
void ParticlesParallelForImp(const TConstHandleView<TSOA>& Particles, const ContextCreatorType& ContextCreator, const Lambda& Func)
{
	Chaos::HandleViewParallelForImp(Particles, ContextCreator, Func);
}

template <typename TSOA, typename ContextCreatorType, typename Lambda>
void ParticlesParallelForImp(const THandleView<TSOA>& Particles, const ContextCreatorType& ContextCreator, const Lambda& Func)
{
	Chaos::HandleViewParallelForImp(Particles, ContextCreator, Func);
}

template <typename TSOA, typename ContextCreatorType, typename Lambda>
void ParticlesParallelForImp(const TConstParticleView<TSOA>& Particles, const ContextCreatorType& ContextCreator, const Lambda& Func)
{
	Chaos::ParticleViewParallelForImp(Particles, ContextCreator, Func);
}

template <typename TSOA, typename ContextCreatorType, typename Lambda>
void ParticlesParallelForImp(const TParticleView<TSOA>& Particles, const ContextCreatorType& ContextCreator, const Lambda& Func)
{
	Chaos::ParticleViewParallelForImp(Particles, ContextCreator, Func);
}

template <typename TParticle, typename ContextCreatorType, typename Lambda>
void ParticlesParallelForImp(const TArray<TParticle>& Particles, const ContextCreatorType& ContextCreator, const Lambda& Func)
{
	// When ParticlesParallelFor is called with a plain old TArray,
	// just do normal parallelization.
	const int32 Num = Particles.Num();
	PhysicsParallelFor(Num, [&Func, &Particles](const int32 Index)
	{
		Func(Particles[Index], Index);
	});
}


template <typename TView, typename ContextCreatorType, typename Lambda>
void ParticlesSequentialFor(const TView& Particles, const ContextCreatorType& ContextCreator, const Lambda& Func)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticlesSequentialFor);

	const int32 ContextIndex = ContextCreator(0, 1);
	int32 Index = 0;
	for (auto& Particle : Particles)
	{
		Func(Particle, Index++, ContextIndex);
	}
}

template <typename TView, typename ContextCreatorType, typename Lambda>
void ParticlesParallelFor(const TView& Particles, const ContextCreatorType& ContextCreator, const Lambda& Func, bool bForceSingleThreaded = false)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticlesParallelFor);

	if (!bForceSingleThreaded && !bDisableParticleParallelFor)
	{
		Chaos::ParticlesParallelForImp(Particles, ContextCreator, Func);
	}
	else
	{
		Chaos::ParticlesSequentialFor(Particles, ContextCreator, Func);
	}
}

template <typename TSOA>
class TConstHandleIterator
{
public:
	using THandle = typename TSOA::THandleType;

	TConstHandleIterator()
		: Handles(nullptr)
		, CurIdx(0)
	{
	}

	TConstHandleIterator(const TArray<THandle*>& InHandles)
		: Handles(&InHandles)
		, CurIdx(0)
	{
		while(CurrentIsLightWeightDisabled())
		{
			operator++();
		}
	}

	operator bool() const { return Handles && CurIdx < Handles->Num(); }
	TConstHandleIterator<TSOA>& operator++()
	{
		bool bFindNextHandle = true;
		while(bFindNextHandle)
		{
			++CurIdx;
			bFindNextHandle = CurrentIsLightWeightDisabled();
		}
		return *this;
	}

	const THandle& operator*() const
	{
		return static_cast<const THandle&>(*(*Handles)[CurIdx]);
	}

	const THandle* operator->() const
	{
		return static_cast<const THandle*>((*Handles)[CurIdx]);
	}

	int32 ComputeSize() const
	{
		return Handles->Num();
	}

protected:
	template <typename TSOA2>
	friend class TConstHandleView;

	const TArray<THandle*>* Handles;
	int32 CurIdx;

	bool CurrentIsLightWeightDisabled() const
	{
		return operator bool() && operator*().LightWeightDisabled();
	}
};

template <typename TSOA>
class THandleIterator : public TConstHandleIterator<TSOA>
{
public:
	using Base = TConstHandleIterator<TSOA>;
	using typename Base::THandle;
	using Base::Handles;
	using Base::CurIdx;

	THandleIterator()
		: Base()
	{
	}

	THandleIterator(const TArray<THandle*>& InHandles)
		: Base(InHandles)
	{
	}

	THandle& operator*() const
	{
		return static_cast<THandle&>(*(*Handles)[CurIdx]);
	}

	THandle* operator->() const
	{
		return static_cast<THandle*>((*Handles)[CurIdx]);
	}

	template <typename TSOA2>
	friend class THandleView;
};

template <typename THandle>
TConstHandleIterator<typename THandle::TSOAType> MakeConstHandleIterator(const TArray<THandle*>& Handles)
{
	return TConstHandleIterator<typename THandle::TSOAType>(Handles);
}

template <typename THandle>
THandleIterator<typename THandle::TSOAType> MakeHandleIterator(const TArray<THandle*>& Handles)
{
	return THandleIterator<typename THandle::TSOAType>(Handles);
}

template <typename TSOA>
class TConstHandleView
{
public:
	using THandle = typename TSOA::THandleType;
	TConstHandleView()
	{
	}

	TConstHandleView(const TArray<THandle*>& InHandles)
		: Handles(InHandles)
	{
	}

	int32 Num() const
	{
		return Handles.Num();
	}

	TConstHandleIterator<TSOA> Begin() const { return MakeConstHandleIterator(Handles); }
	TConstHandleIterator<TSOA> begin() const { return Begin(); }

	TConstHandleIterator<TSOA> End() const { return TConstHandleIterator<TSOA>(); }
	TConstHandleIterator<TSOA> end() const { return End(); }

	template <typename TParticleView, typename ContextCreatorType, typename Lambda>
	friend void HandleViewParallelForImp(const TParticleView& Particles, const ContextCreatorType& ContextCreator, const Lambda& Func);

	template <typename ContextCreatorType, typename Lambda>
	void ParallelFor(const ContextCreatorType& ContextCreator, const Lambda& Func) const
	{
		ParticlesParallelFor(*this, ContextCreator, Func);
	}

	template <typename Lambda>
	void ParallelFor(const Lambda& Func) const
	{
		// Dummy context creator and a context-stripping function wrapper
		auto EmptyContextCreator = [](int32 WorkerIndex, int32 NumWorkers) -> int32 { return WorkerIndex; };
		auto ContextFunc = [&Func](auto& Particle, int32 Index, int32 WorkerIndex) { return Func(Particle, Index); };
		ParticlesParallelFor(*this, EmptyContextCreator, ContextFunc);
	}

protected:

	const TArray<THandle*>& Handles;
};

template <typename TSOA>
class THandleView : public TConstHandleView<TSOA>
{
public:
	using Base = TConstHandleView<TSOA>;
	using typename Base::THandle;
	using Base::Handles;
	using Base::Num;

	THandleView()
		: Base()
	{
	}

	THandleView(const TArray<THandle*>& InHandles)
		: Base(InHandles)
	{
	}

	THandleIterator<TSOA> Begin() const { return MakeHandleIterator(Handles); }
	THandleIterator<TSOA> begin() const { return Begin(); }

	THandleIterator<TSOA> End() const { return THandleIterator<TSOA>(); }
	THandleIterator<TSOA> end() const { return End(); }

	template <typename ContextCreatorType, typename Lambda>
	void ParallelFor(const ContextCreatorType& ContextCreator, const Lambda& Func) const
	{
		ParticlesParallelFor(*this, ContextCreator, Func);
	}

	template <typename Lambda>
	void ParallelFor(const Lambda& Func) const
	{
		// Dummy context creator and a context-stripping function wrapper
		auto EmptyContextCreator = [](int32 WorkerIndex, int32 NumWorkers) -> int32 { return WorkerIndex; };
		auto ContextFunc = [&Func](auto& Particle, int32 Index, int32 WorkerIndex) { return Func(Particle, Index); };
		ParticlesParallelFor(*this, EmptyContextCreator, ContextFunc);
	}
};

template <typename THandle>
TConstHandleView<typename THandle::TSOAType> MakeConstHandleView(const TArray<THandle*>& Handles)
{
	return TConstHandleView<typename THandle::TSOAType>(Handles);
}

template <typename THandle>
THandleView<typename THandle::TSOAType> MakeHandleView(const TArray<THandle*>& Handles)
{
	return THandleView<typename THandle::TSOAType>(Handles);
}

template <typename TSOA>
struct TSOAView
{
	using THandle = typename TSOA::THandleType;

	TSOAView()
		: SOA(nullptr)
		, HandlesArray(nullptr)
	{}

	TSOAView(TSOA* InSOA)
		: SOA(InSOA)
		, HandlesArray(nullptr)
	{}

	template <typename TDerivedHandle>
	TSOAView(TArray<TDerivedHandle*>* Handles)
		: SOA(nullptr)
	{
		static_assert(TIsDerivedFrom< TDerivedHandle, THandle >::IsDerived, "Trying to create a derived view on a base type");

		//This is safe because the view is strictly read only in terms of the pointers
		//I.e. we will never be in a case where we create a new base type and assign it to a derived pointer
		//We are only using this to read data from the pointers and so having a more base API (which cannot modify the pointer) is safe
		HandlesArray = reinterpret_cast<TArray<THandle*>*>(Handles);
	}

	TSOA* SOA;
	TArray<THandle*>* HandlesArray;

	int32 Size() const
	{
		return HandlesArray ? HandlesArray->Num() : SOA->Size();
	}
};

template <typename TSOA>
class TConstParticleIterator
{
public:
	using THandle = typename TSOA::THandleType;
	using THandleBase = typename THandle::THandleBase;
	using TTransientHandle = typename THandle::TTransientHandle;

	TConstParticleIterator()
	{
		MoveToEnd();
	}

	TConstParticleIterator(const TArray<TSOAView<TSOA>>& InSOAViews)
		: SOAViews(&InSOAViews)
		, CurHandlesArray(nullptr)
		, SOAIdx(0)
#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
		, DirtyValidationCount(INDEX_NONE)
#endif
	{
		SeekNonEmptySOA();
		while(CurrentIsLightWeightDisabled())
		{
			operator++();
		}
	}

	TConstParticleIterator(const TConstParticleIterator& Rhs) = default;

	operator bool() const { return TransientHandle.GeometryParticles != nullptr; }

	// The non-parallel implementation of iteration should not deviate in behavior
	// from the parallel implementation in \c ParticleViewParallelForImp().  They
	// must be kept in sync.
	TConstParticleIterator<TSOA>& operator++()
	{
		RangedForValidation();
		bool bFindNextEnabledParticle = true;
		while (bFindNextEnabledParticle)
		{
			if (CurHandlesArray == nullptr)
			{
				//SOA is packed efficiently for iteration
				++TransientHandle.ParticleIdx;
				if (TransientHandle.ParticleIdx >= static_cast<int32>(CurSOASize))
				{
					IncSOAIdx();
				}
			}
			else
			{
				++CurHandleIdx;
				if (CurHandleIdx < CurHandlesArray->Num())
				{
					// Reconstruct the TransientHandle so that it has a chance to update
					// other data members, like the particle type in the case of geometry 
					// particles.
					THandle* HandlePtr = (*CurHandlesArray)[CurHandleIdx];
					THandleBase Handle(HandlePtr->GeometryParticles, HandlePtr->ParticleIdx);
					TransientHandle = static_cast<TTransientHandle&>(Handle);
				}
				else
				{
					IncSOAIdx();
				}
			}

			bFindNextEnabledParticle = CurrentIsLightWeightDisabled();
		}

		return *this;
	}
	
	const TTransientHandle& operator*() const
	{
		RangedForValidation();
		return static_cast<const TTransientHandle&>(TransientHandle);
	}

	const TTransientHandle* operator->() const
	{
		RangedForValidation();
		return static_cast<const TTransientHandle*>(&TransientHandle);
	}

protected:

	bool CurrentIsLightWeightDisabled() const
	{
		return operator bool() && (operator*()).LightWeightDisabled();
	}

	void MoveToEnd()
	{
		SOAIdx = 0;
		CurSOASize = 0;
		CurHandleIdx = 0;
		TransientHandle = THandleBase();
		CurHandlesArray = nullptr;
	}

	void SeekNonEmptySOA()
	{
		while (SOAIdx < SOAViews->Num() && ((*SOAViews)[SOAIdx].Size() == 0))
		{
			++SOAIdx;
		}
		CurHandleIdx = 0;
		if (SOAIdx < SOAViews->Num())
		{
			CurHandlesArray = (*SOAViews)[SOAIdx].HandlesArray;
			TransientHandle = CurHandlesArray ? THandleBase((*CurHandlesArray)[0]->GeometryParticles, (*CurHandlesArray)[0]->ParticleIdx) : THandleBase((*SOAViews)[SOAIdx].SOA, 0);
			CurSOASize = TransientHandle.GeometryParticles->Size();
		}
		else
		{
			MoveToEnd();
		}
		SyncDirtyValidationCount();
		RangedForValidation();
	}

	const TArray<TSOAView<TSOA>>* SOAViews;
	const TArray<THandle*>* CurHandlesArray;
	THandleBase TransientHandle;
	int32 SOAIdx;
	int32 CurSOASize;
	int32 CurHandleIdx;

	void RangedForValidation() const
	{
#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
		if (CurHandlesArray)
		{
			check(DirtyValidationCount == CurHandlesArray->Num());
		}
		else if (TransientHandle.GeometryParticles)
		{
			check(DirtyValidationCount != INDEX_NONE);
			check(TransientHandle.GeometryParticles->DirtyValidationCount() == DirtyValidationCount && TEXT("Iterating over particles while modifying the underlying SOA. Consider delaying any operations that require a Handle*"));
		}
#endif
	}

	void IncSOAIdx()
	{
		++SOAIdx;
		SeekNonEmptySOA();
	}

	void SyncDirtyValidationCount()
	{

#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
		if (CurHandlesArray)
		{
			DirtyValidationCount = CurHandlesArray->Num();
		}
		else
		{
			DirtyValidationCount = TransientHandle.GeometryParticles ? TransientHandle.GeometryParticles->DirtyValidationCount() : INDEX_NONE;
		}
#endif
	}

#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
	int32 DirtyValidationCount;
#endif
};

template <typename TSOA>
class TParticleIterator : public TConstParticleIterator<TSOA>
{
public:
	using Base = TConstParticleIterator<TSOA>;
	using TTransientHandle = typename Base::TTransientHandle;
	using Base::TransientHandle;
	using Base::RangedForValidation;

	TParticleIterator()
		: Base()
	{
	}

	TParticleIterator(const TArray<TSOAView<TSOA>>& InSOAs)
		: Base(InSOAs)
	{
	}

	TTransientHandle& operator*() const
	{
		RangedForValidation();
		//const_cast ok because the const is operation is really about the iterator.The transient handle cannot change
		return const_cast<TTransientHandle&>(static_cast<const TTransientHandle&>(TransientHandle));
	}

	TTransientHandle* operator->() const
	{
		RangedForValidation();
		//const_cast ok because the const is operation is really about the iterator. The transient handle cannot change
		return const_cast<TTransientHandle*>(static_cast<const TTransientHandle*>(&TransientHandle));
	}
};

template <typename TSOA>
TConstParticleIterator<TSOA> MakeConstParticleIterator(const TArray<TSOAView<TSOA>>& SOAs)
{
	return TConstParticleIterator<TSOA>(SOAs);
}

template <typename TSOA>
TParticleIterator<TSOA> MakeParticleIterator(const TArray<TSOAView<TSOA>>& SOAs)
{
	return TParticleIterator<TSOA>(SOAs);
}


template <typename TSOA>
class TConstParticleView
{
public:
	using THandle = typename TSOA::THandleType;
	TConstParticleView()
		: Size(0)
	{
	}

	TConstParticleView(TArray<TSOAView<TSOA>>&& InSOAViews)
		: SOAViews(MoveTemp(InSOAViews))
		, Size(0)
	{
		for (const auto& SOAView : SOAViews)
		{
			Size += SOAView.Size();
		}
	}

	TConstParticleView(TSOAView<TSOA>&& InSOAView)
		: Size(InSOAView.Size())
	{
		SOAViews.Add(MoveTemp(InSOAView));
	}

	int32 Num() const
	{
		return Size;
	}

	TConstParticleIterator<TSOA> Begin() const { return MakeConstParticleIterator(SOAViews); }
	TConstParticleIterator<TSOA> begin() const { return Begin(); }

	TConstParticleIterator<TSOA> End() const { return TConstParticleIterator<TSOA>(); }
	TConstParticleIterator<TSOA> end() const { return End(); }

	template <typename ContextCreatorType, typename Lambda>
	void ParallelFor(const ContextCreatorType& ContextCreator, const Lambda& Func, bool bForceSingleThreaded = false) const
	{
		ParticlesParallelFor(*this, ContextCreator, Func, bForceSingleThreaded);
	}

	template <typename Lambda>
	void ParallelFor(const Lambda& Func, bool bForceSingleThreaded = false) const
	{
		// Dummy context creator and a context-stripping function wrapper
		auto EmptyContextCreator = [](int32 WorkerIndex, int32 NumWorkers) -> int32 { return WorkerIndex; };
		auto ContextFunc = [&Func](auto& Particle, int32 Index, int32 WorkerIndex) { return Func(Particle, Index); };

		ParticlesParallelFor(*this, EmptyContextCreator, ContextFunc, bForceSingleThreaded);
	}

	template <typename TParticleView, typename ContextCreatorType, typename Lambda>
	friend void ParticleViewParallelForImp(const TParticleView& Particles, const ContextCreatorType& ContextCreator, const Lambda& Func);

protected:

	TArray<TSOAView<TSOA>> SOAViews;
	int32 Size;
};

template <typename TSOAIn>
class TParticleView : public TConstParticleView<TSOAIn>
{
public:
	using TSOA = TSOAIn;
	using Base = TConstParticleView<TSOA>;
	using Base::SOAViews;
	using Base::Num;
	using TIterator = TParticleIterator<TSOA>;

	TParticleView()
		: Base()
	{
	}

	TParticleView(TArray<TSOAView<TSOA>>&& InSOAViews)
		: Base(MoveTemp(InSOAViews))
	{
	}

	TParticleIterator<TSOA> Begin() const { return MakeParticleIterator(SOAViews); }
	TParticleIterator<TSOA> begin() const { return Begin(); }

	TParticleIterator<TSOA> End() const { return TParticleIterator<TSOA>(); }
	TParticleIterator<TSOA> end() const { return End(); }

	template <typename ContextCreatorType, typename Lambda>
	void ParallelFor(const ContextCreatorType& ContextCreator, const Lambda& Func, bool bForceSingleThreaded = false) const
	{
		ParticlesParallelFor(*this, ContextCreator, Func, bForceSingleThreaded);
	}

	template <typename Lambda>
	void ParallelFor(const Lambda& Func, bool bForceSingleThreaded = false) const
	{
		// Dummy context creator and a context-stripping function wrapper
		auto EmptyContextCreator = [](int32 WorkerIndex, int32 NumWorkers) -> int32 { return WorkerIndex; };
		auto ContextFunc = [&Func](auto& Particle, int32 Index, int32 WorkerIndex) { return Func(Particle, Index); };

		ParticlesParallelFor(*this, EmptyContextCreator, ContextFunc, bForceSingleThreaded);
	}
};

template <typename TSOA>
TConstParticleView<TSOA> MakeConstParticleView(TArray<TSOAView<TSOA>>&& SOAViews)
{
	return TConstParticleView<TSOA>(MoveTemp(SOAViews));
}

template <typename TSOA>
TConstParticleView<TSOA> MakeConstParticleView(TSOAView<TSOA>&& SOAView)
{
	return TConstParticleView<TSOA>(MoveTemp(SOAView));
}

template <typename TSOA>
TParticleView<TSOA> MakeParticleView(TArray<TSOAView<TSOA>>&& SOAViews)
{
	return TParticleView<TSOA>(MoveTemp(SOAViews));
}

template <typename TSOA>
TConstParticleView<TSOA> MakeConstParticleView(TSOA* SOA)
{
	TArray<TSOAView<TSOA>> SOAs;
	SOAs.Add({ SOA });
	return TConstParticleView<TSOA>(MoveTemp(SOAs));
}

template <typename TSOA>
TParticleView<TSOA> MakeParticleView(TSOA* SOA)
{
	TArray<TSOAView<TSOA>> SOAs;
	SOAs.Add({ SOA });
	return TParticleView<TSOA>(MoveTemp(SOAs));
}

// The non-parallel implementation of iteration should not deviate in behavior from
// this parallel implementation.  They must be kept in sync.
template <typename TParticleView, typename ContextCreatorType, typename Lambda>
void ParticleViewParallelForImp(const TParticleView& Particles, const ContextCreatorType& ContextCreator, const Lambda& UserFunc)
{
	SCOPE_CYCLE_COUNTER(STAT_ParticleViewParallelForImp);

	using TSOA = typename TParticleView::TSOA;
	using THandle = typename TSOA::THandleType;
	using THandleBase = typename THandle::THandleBase;
	using TTransientHandle = typename THandle::TTransientHandle;

	auto Func = [&UserFunc](auto& Handle, const int32 Idx, const int32 ContextIndex)
	{
		if (!Handle.LightWeightDisabled())
		{
			UserFunc(Handle, Idx, ContextIndex);
		}
	};

	// Loop over every SOA in this view, skipping empty ones
	int32 ParticleIdxOff = 0;
	for (int32 ViewIndex = 0; ViewIndex < Particles.SOAViews.Num(); ++ViewIndex)
	{
		const TSOAView<TSOA>& SOAView = Particles.SOAViews[ViewIndex];
		const int32 ParticleCount = SOAView.Size();
		if (ParticleCount == 0)
		{
			continue;
		}

		// Iterate over each element using normal parallel for
		if (const TArray<THandle*>* CurHandlesArray = SOAView.HandlesArray)
		{
			// Do a regular parallel for over the handles in this SOA view
			const int32 HandleCount = CurHandlesArray->Num();
			PhysicsParallelForWithContext(HandleCount, ContextCreator, [&Func, CurHandlesArray, ParticleIdxOff](const int32 HandleIdx, const int32 ContextIndex)
			{
				// Reconstruct the TransientHandle so that it has a chance to update
				// other data members, like the particle type in the case of geometry 
				// particles.
				THandle* HandlePtr = (*CurHandlesArray)[HandleIdx];
				THandleBase Handle(HandlePtr->GeometryParticles, HandlePtr->ParticleIdx);
				Func(static_cast<TTransientHandle&>(Handle), ParticleIdxOff + HandleIdx, ContextIndex);
			});
			ParticleIdxOff += HandleCount;
		}
		else
		{
			// Do a regular parallel for over the particles in this SOA view
			PhysicsParallelForWithContext(ParticleCount, ContextCreator, [&Func, &SOAView, ParticleIdxOff](const int32 ParticleIdx, const int32 ContextIndex)
			{
				// Reconstruct the TransientHandle so that it has a chance to update
				// other data members, like the particle type in the case of geometry 
				// particles.
				THandleBase Handle(SOAView.SOA, ParticleIdx);
				Func(static_cast<TTransientHandle&>(Handle), ParticleIdxOff + ParticleIdx, ContextIndex);
			});
			ParticleIdxOff += ParticleCount;
		}
	}
}

template <typename THandleView, typename ContextCreatorType, typename Lambda>
void HandleViewParallelForImp(const THandleView& HandleView, const ContextCreatorType& ContextCreator, const Lambda& Func)
{
	SCOPE_CYCLE_COUNTER(STAT_HandleViewParallelForImp);

	using THandle = typename THandleView::THandle;
	const int32 HandleCount = HandleView.Handles.Num();
	PhysicsParallelForWithContext(HandleCount, ContextCreator, [&HandleView, &Func](const int32 Index, const int32 ContextIndex)
	{
		if (!HandleView.Handles[Index]->LightWeightDisabled())
		{
			Func(static_cast<THandle&>(*HandleView.Handles[Index]), Index, ContextIndex);
		}
	});
}

}
