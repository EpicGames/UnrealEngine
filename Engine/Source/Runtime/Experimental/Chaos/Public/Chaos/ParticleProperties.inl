// Copyright Epic Games, Inc. All Rights Reserved.

//Make sure to include PhysicsProxyBase.h before using this inl file

#undef CHAOS_PROPERTY_CHECKED
#if CHAOS_CHECKED
#define CHAOS_PROPERTY_CHECKED(x, Type, ProxyType) CHAOS_PROPERTY(x, Type, ProxyType)
#else
#define CHAOS_PROPERTY_CHECKED(x, Type, ProxyType)
#endif

CHAOS_PROPERTY(XR, FParticlePositionRotation, EPhysicsProxyType::SingleParticleProxy)
CHAOS_PROPERTY(Velocities, FParticleVelocities, EPhysicsProxyType::SingleParticleProxy)
CHAOS_PROPERTY(Dynamics, FParticleDynamics, EPhysicsProxyType::SingleParticleProxy)
CHAOS_PROPERTY(DynamicMisc, FParticleDynamicMisc, EPhysicsProxyType::SingleParticleProxy)
CHAOS_PROPERTY(NonFrequentData, FParticleNonFrequentData, EPhysicsProxyType::SingleParticleProxy)
CHAOS_PROPERTY(MassProps, FParticleMassProps, EPhysicsProxyType::SingleParticleProxy)
CHAOS_PROPERTY(KinematicTarget,FKinematicTarget, EPhysicsProxyType::SingleParticleProxy)

CHAOS_PROPERTY(JointSettings, FPBDJointSettings, EPhysicsProxyType::JointConstraintType)
CHAOS_PROPERTY(JointParticleProxies, FProxyBasePairProperty, EPhysicsProxyType::JointConstraintType)
CHAOS_PROPERTY(JointPhysicsObjects, FPhysicsObjectPairProperty, EPhysicsProxyType::JointConstraintType)

CHAOS_PROPERTY(SuspensionSettings, FPBDSuspensionSettings, EPhysicsProxyType::SuspensionConstraintType)
CHAOS_PROPERTY(SuspensionParticleProxy, FParticleProxyProperty, EPhysicsProxyType::SuspensionConstraintType)
CHAOS_PROPERTY(SuspensionLocation, FSuspensionLocation, EPhysicsProxyType::SuspensionConstraintType)
CHAOS_PROPERTY(SuspensionPhysicsObject, FPhysicsObjectProperty, EPhysicsProxyType::SuspensionConstraintType)
CHAOS_PROPERTY(GroundPhysicsObject, FPhysicsObjectProperty, EPhysicsProxyType::SuspensionConstraintType)


CHAOS_PROPERTY(CharacterGroundConstraintSettings, FCharacterGroundConstraintSettings, EPhysicsProxyType::CharacterGroundConstraintType)
CHAOS_PROPERTY(CharacterGroundConstraintDynamicData, FCharacterGroundConstraintDynamicData, EPhysicsProxyType::CharacterGroundConstraintType)
CHAOS_PROPERTY(CharacterParticleProxy, FParticleProxyProperty, EPhysicsProxyType::CharacterGroundConstraintType)
CHAOS_PROPERTY(GroundParticleProxy, FParticleProxyProperty, EPhysicsProxyType::CharacterGroundConstraintType)

CHAOS_PROPERTY(ClusterXR, FParticlePositionRotation, EPhysicsProxyType::ClusterUnionProxy)
CHAOS_PROPERTY(ClusterVelocities, FParticleVelocities, EPhysicsProxyType::ClusterUnionProxy)

#undef CHAOS_PROPERTY