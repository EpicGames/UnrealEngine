// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Defines.h"
#include "Chaos/DynamicParticles.h"
#include "Chaos/PBDParticles.h"
#include "Chaos/PBDRigidParticles.h"

namespace Chaos
{
	template<typename T, int d>
	class TParticles;
	using FParticles = TParticles<FReal, 3>;

	template<typename T, int d>
	class TDynamicParticles;
	using FDynamicParticles = TDynamicParticles<FReal, 3>;

	template<typename T, int d>
	class TPBDParticles;
	using FPBDParticles = TPBDParticles<FReal, 3>;

	template<typename T, int d>
	class TRigidParticles;

	template<typename T, int d>
	class TPBDRigidParticles;

/**
 * Apply an effect to all particles.
 */
class FParticleRule
{
  public:
	virtual void Apply(FParticles& InParticles, const FReal Dt) const { check(0); }
	virtual void Apply(FDynamicParticles& InParticles, const FReal Dt) const { Apply(static_cast<FParticles&>(InParticles), Dt); }
	virtual void Apply(FPBDParticles& InParticles, const FReal Dt) const { Apply(static_cast<FDynamicParticles&>(InParticles), Dt); }
};

template<class T, int d>
using TParticleRule UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FParticleRule instead") = FParticleRule;

}
