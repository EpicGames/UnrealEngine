// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayCollection.h"
#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/Core.h"
#include "Chaos/Particle/ObjectState.h"
#include "Chaos/Vector.h"
#include "ChaosArchive.h"
#include "HAL/LowLevelMemTracker.h"


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#define PARTICLE_ITERATOR_RANGED_FOR_CHECK 1
#else
#define PARTICLE_ITERATOR_RANGED_FOR_CHECK 0
#endif

namespace Chaos
{
	template<class T, int d>
	class TPBDRigidsEvolution;

	enum class ERemoveParticleBehavior : uint8
	{
		RemoveAtSwap,	//O(1) but reorders particles relative to one another
		Remove			//Keeps particles relative to one another, but O(n)
	};

	template<class T, int d>
	class TParticles : public TArrayCollection
	{
	public:
		TParticles()
			: MRemoveParticleBehavior(ERemoveParticleBehavior::RemoveAtSwap)
		{
			AddArray(&MX);
#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
			MDirtyValidationCount = 0;
#endif
		}
		TParticles(const TParticles<T, d>& Other) = delete;
		TParticles(TParticles<T, d>&& Other)
		    : TArrayCollection(), MX(MoveTemp(Other.MX))
			, MRemoveParticleBehavior(Other.MRemoveParticleBehavior)
		{
			AddParticles(Other.Size());
			AddArray(&MX);
			Other.MSize = 0;
#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
			MDirtyValidationCount = 0;
#endif
		}

		TParticles(TArray<TVector<T, d>>&& Positions)
			: TArrayCollection(), MX(MoveTemp(Positions))
			, MRemoveParticleBehavior(ERemoveParticleBehavior::RemoveAtSwap)
		{
			AddParticles(MX.Num());
			AddArray(&MX);

#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
			MDirtyValidationCount = 0;
#endif
		}

		virtual ~TParticles() = default;

		void AddParticles(const int32 Num)
		{
			AddElementsHelper(Num);
			IncrementDirtyValidation();
		}

		void DestroyParticle(const int32 Idx)
		{
			if (MRemoveParticleBehavior == ERemoveParticleBehavior::RemoveAtSwap)
			{
				RemoveAtSwapHelper(Idx);
			}
			else
			{
				RemoveAtHelper(Idx, 1);
			}
			IncrementDirtyValidation();
		}

		ERemoveParticleBehavior RemoveParticleBehavior() const { return MRemoveParticleBehavior; }
		ERemoveParticleBehavior& RemoveParticleBehavior() { return MRemoveParticleBehavior; }

		void MoveToOtherParticles(const int32 Idx, TParticles<T,d>& Other)
		{
			MoveToOtherArrayCollection(Idx, Other);
			IncrementDirtyValidation();
		}
		
		/** Move particle from one SOA to another one given a list of particle indices */
		void MoveToOtherParticles(const TArrayView<int32>& Indices, TParticles<T, d>& Other)
		{
			MoveToOtherArrayCollection(Indices, Other);
			AddDirtyValidation(Indices.Num());
		}

		void Resize(const int32 Num)
		{
			ResizeHelper(Num);
			IncrementDirtyValidation();
		}

		TParticles& operator=(TParticles<T, d>&& Other)
		{
			MX = MoveTemp(Other.MX);
			ResizeHelper(Other.Size());
			Other.MSize = 0;

#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
			MDirtyValidationCount = 0;
			++Other.MDirtyValidationCount;
#endif
			return *this;
		}

		inline const TArrayCollectionArray<TVector<T, d>>& X() const
		{
			return MX;
		}
		
		void Serialize(FArchive& Ar)
		{
			bool bSerialize = true;	//leftover from when we had view support
			Ar << bSerialize;
			if (ensureMsgf(bSerialize, TEXT("Cannot serialize shared views. Refactor needed to reduce memory")))
			{
				Ar << MX;
				ResizeHelper(MX.Num());
			}
			IncrementDirtyValidation();
		}

		const TArrayCollectionArray<TVector<T, d>>& XArray() const
		{
			return MX;
		}

		TArrayCollectionArray<TVector<T, d>>& XArray()
		{
			return MX;
		}
		
		UE_DEPRECATED(5.4, "Use GetX instead")
		const TVector<T, d>& X(const int32 Index) const
		{
			return MX[Index];
		}

		UE_DEPRECATED(5.4, "Use GetX or SetX instead")
		TVector<T, d>& X(const int32 Index)
		{
			return MX[Index];
		}

		const TVector<T, d>& GetX(const int32 Index) const
		{
			return MX[Index];
		}

		void SetX(const int32 Index, const TVector<T, d>& InX)
		{
			MX[Index] = InX;
		}

		FString ToString(int32 index) const
		{
			return FString::Printf(TEXT("MX:%s"), *GetX(index).ToString());
		}

		uint32 GetTypeHash() const
		{
			uint32 OutHash = 0;
			const int32 NumXEntries = MX.Num();

			if(NumXEntries > 0)
			{
				OutHash = UE::Math::GetTypeHash(MX[0]);

				for(int32 XIndex = 1; XIndex < NumXEntries; ++XIndex)
				{
					OutHash = HashCombine(OutHash, UE::Math::GetTypeHash(MX[XIndex]));
				}
			}

			return OutHash;
		}

		SIZE_T GetAllocatedSize() const
		{
			return MX.GetAllocatedSize();
		}

#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
		int32 DirtyValidationCount() const { return MDirtyValidationCount; }
#endif

		FORCEINLINE TArray<TVector<T, d>>& AllX() { return MX; }
		FORCEINLINE const TArray<TVector<T, d>>& AllX() const { return MX; }

	private:
		TArrayCollectionArray<TVector<T, d>> MX;

		ERemoveParticleBehavior MRemoveParticleBehavior;

#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
		int32 MDirtyValidationCount;
#endif

		void IncrementDirtyValidation()
		{
#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
			++MDirtyValidationCount;
#endif
		}

		/** Increase the dirty validation count by NumDirty */
		void AddDirtyValidation(const int32 NumDirty)
		{
#if PARTICLE_ITERATOR_RANGED_FOR_CHECK
			MDirtyValidationCount += NumDirty;
#endif
		}

		inline friend FArchive& operator<<(FArchive& Ar, TParticles<T, d>& InParticles)
		{
			InParticles.Serialize(Ar);
			return Ar;
		}
	};

	template<typename T, int d>
	static uint32 GetTypeHash(const TParticles<T, d>& InParticles)
	{
		return InParticles.GetTypeHash();
	}
	
	enum class EChaosCollisionTraceFlag : int8
	{
		/** Use project physics settings (DefaultShapeComplexity) */
		Chaos_CTF_UseDefault,
		/** Create both simple and complex shapes. Simple shapes are used for regular scene queries and collision tests. Complex shape (per poly) is used for complex scene queries.*/
		Chaos_CTF_UseSimpleAndComplex,
		/** Create only simple shapes. Use simple shapes for all scene queries and collision tests.*/
		Chaos_CTF_UseSimpleAsComplex,
		/** Create only complex shapes (per poly). Use complex shapes for all scene queries and collision tests. Can be used in simulation for static shapes only (i.e can be collided against but not moved through forces or velocity.) */
		Chaos_CTF_UseComplexAsSimple,
		/** */
		Chaos_CTF_MAX,
	};

	using FParticles = TParticles<FReal, 3>;
} // namespace Chaos
