// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Evolution/SolverBody.h"
#include "Chaos/Particles.h"
#include "Containers/ArrayView.h"

namespace Chaos::Softs
{

struct FParticleRangeIndex
{
	int32 RangeId;
	int32 ParticleIndex;

	FParticleRangeIndex() = default;
	~FParticleRangeIndex() = default;
	FParticleRangeIndex(const FParticleRangeIndex&) = default;
	FParticleRangeIndex(FParticleRangeIndex&&) = default;
	FParticleRangeIndex& operator=(const FParticleRangeIndex&) = default;
	FParticleRangeIndex& operator=(FParticleRangeIndex&&) = default;

	FParticleRangeIndex(const int32 InRangeId, const int32 InParticleIndex)
		: RangeId(InRangeId)
		, ParticleIndex(InParticleIndex)
	{}

	bool operator<(const FParticleRangeIndex& Other) const
	{
		if (RangeId != Other.RangeId)
		{
			return RangeId < Other.RangeId;
		}
		return ParticleIndex < Other.ParticleIndex;
	}
	bool operator==(const FParticleRangeIndex&) const = default;
};

inline uint32 GetTypeHash(const FParticleRangeIndex& Index)
{
	return HashCombine(Index.RangeId, Index.ParticleIndex);
}

template<typename ParticlesType, typename = typename TEnableIf<TIsDerivedFrom<ParticlesType, TParticles<FSolverReal, 3>>::IsDerived>::Type>
class TParticlesRange
{
public:

	TParticlesRange() = default;
	virtual ~TParticlesRange() = default;

	TParticlesRange(ParticlesType* InParticles, int32 InOffset, int32 InRangeSize, int32 InRangeId = INDEX_NONE)
		: Particles(InParticles)
		, Offset(InOffset)
		, RangeSize(InRangeSize)
		, RangeId(InRangeId)
	{
	}

	static TParticlesRange AddParticleRange(ParticlesType& InParticles, const int32 InRangeSize, const int32 InRangeId = INDEX_NONE)
	{
		const int32 Offset = (int32)InParticles.Size();
		InParticles.AddParticles(InRangeSize);
		return TParticlesRange(&InParticles, Offset, InRangeSize, InRangeId);
	}

	bool IsValid() const
	{
		return Particles && Offset >= 0 && ((uint32)(Offset + RangeSize) <= Particles->Size());
	}

	template<typename T>
	TConstArrayView<T> GetConstArrayView(const TArray<T>& Array) const
	{
		check(Offset >= 0 && Offset + RangeSize <= Array.Num());
		return TConstArrayView<T>(Array.GetData() + Offset, RangeSize);
	}

	template<typename T>
	TArrayView<T> GetArrayView(TArray<T>& Array) const
	{
		check(Offset >= 0 && Offset + RangeSize <= Array.Num());
		return TArrayView<T>(Array.GetData() + Offset, RangeSize);
	}

	const ParticlesType& GetParticles() const { check(Particles);  return *Particles; }
	ParticlesType& GetParticles() { check(Particles); return *Particles; }
	int32 GetOffset() const { return Offset; }
	int32 GetRangeSize() const { return RangeSize; }
	int32 Size() const { return RangeSize; } // So this has same interface as Particles
	bool IsValidIndex(int32 Index) const 
	{
		return (Index >= 0) && (Index < RangeSize);
	}
	int32 GetRangeId() const { return RangeId; }

protected:

	ParticlesType* Particles = nullptr;
	int32 Offset = INDEX_NONE;
	int32 RangeSize = 0;
	int32 RangeId = INDEX_NONE; // Can be used to identify a Range. 
};
}
