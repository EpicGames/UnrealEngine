// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ParticleHandle.h"
#include "Chaos/Transform.h"
#include "Chaos/ISpatialAcceleration.h"
#include "Chaos/Defines.h"
#include "Tasks/Task.h"
#include "Containers/BitArray.h"


namespace Chaos
{ 
enum EPendingSpatialDataOperation : uint8 
{	
	Delete,
	// Note: Updates and Adds are treated the same right now. TODO: Distinguish between them
	Add, // Use this if the element does not exist in the acceleration structure
	Update // Use this when it is known that the element already exists in the acceleration structure
	
};
	
/** Used for updating intermediate spatial structures when they are finished */
struct FPendingSpatialData
{
	FAccelerationStructureHandle AccelerationHandle;
	FSpatialAccelerationIdx SpatialIdx;
	int32 SyncTimestamp;	//indicates the inputs timestamp associated with latest change. Only relevant for external queue
	EPendingSpatialDataOperation Operation;

	FPendingSpatialData()
	: SyncTimestamp(0)
	, Operation(Add)
	{}

	void Serialize(FChaosArchive& Ar)
	{
		/*Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::SerializeHashResult)
		{
			Ar << UpdateAccelerationHandle;
			Ar << DeleteAccelerationHandle;
		}
		else
		{
			Ar << UpdateAccelerationHandle;
			DeleteAccelerationHandle = UpdateAccelerationHandle;
		}

		Ar << bUpdate;
		Ar << bDelete;

		Ar << UpdatedSpatialIdx;
		Ar << DeletedSpatialIdx;*/
		ensure(false);	//Serialization of transient data like this is currently broken. Need to reevaluate
	}

	FUniqueIdx UniqueIdx() const
	{
		return AccelerationHandle.UniqueIdx();
	}
};

struct FPendingSpatialInternalDataQueue
{
	// PendingDataArrays store in 
	// - [0] all elements added in push data
	// - [1 -> KinematicBatchStartIndex] all dynamic particle contained in GetActiveParticlesArray
	// - [KinematicBatchStartIndex -> PendingDataArrays.Num()] all kinematic particle in GetActiveMovingKinematicParticlesView
	TArray<TArray<FPendingSpatialData>> PendingDataArrays;
	// ParticleToPendingData ensure all particle in index [0] are unique
	TArrayAsMap<FUniqueIdx, int32> ParticleToPendingData;
	// This array is access to mark duplicated data from array [0], not we cannot use bit array because it could be used from different thread.
	TArray<bool> Duplicated;
	int32 KinematicBatchStartIndex = 0;

	void Reset()
	{
		KinematicBatchStartIndex = 0;
		for (TArray<FPendingSpatialData>& PendingData : PendingDataArrays)
		{
			PendingData.Reset();
		}
		PendingDataArrays.Reset();
		ParticleToPendingData.Reset();
		Duplicated.Reset();
	}

	int32 Num() const
	{
		int32 NumPendingData = 0;
		for (const TArray<FPendingSpatialData>& PendingData : PendingDataArrays)
		{
			NumPendingData += PendingData.Num();
		}
		return NumPendingData;
	}

	FPendingSpatialData& Add(int32 BatchIndex, const FUniqueIdx UniqueIdx, EPendingSpatialDataOperation Operation = EPendingSpatialDataOperation::Add)
	{
		if (PendingDataArrays.IsEmpty())
		{
			check(BatchIndex == 0);
			PendingDataArrays.SetNum(1);
		}

		check(PendingDataArrays[0].Num() == Duplicated.Num());
		// Check if duplicate
		if (BatchIndex != 0)
		{
			if (int32* Existing = ParticleToPendingData.Find(UniqueIdx))
			{
				Duplicated[*Existing] = true;
			}
			const int32 NewIdx = PendingDataArrays[BatchIndex].AddDefaulted(1);
			PendingDataArrays[BatchIndex][NewIdx].Operation = Operation;
			check(PendingDataArrays[0].Num() == Duplicated.Num());
			return PendingDataArrays[BatchIndex][NewIdx];
		}
		else // Adding at Index 0 would be always in Single thread
		{
			if (int32* Existing = ParticleToPendingData.Find(UniqueIdx))
			{
				check(PendingDataArrays[0].Num() == Duplicated.Num());
				return PendingDataArrays[0][*Existing];
			}
			const int32 NewIdx = PendingDataArrays[0].AddDefaulted(1);
			ParticleToPendingData.Add(UniqueIdx, NewIdx);
			PendingDataArrays[0][NewIdx].Operation = Operation;
			Duplicated.Add(false);
			check(PendingDataArrays[0].Num() == Duplicated.Num());
			return PendingDataArrays[0][NewIdx];
		}
	}

	void Remove(const FUniqueIdx UniqueIdx)
	{
		if (!PendingDataArrays.IsEmpty())
		{
			check(PendingDataArrays[0].Num() == Duplicated.Num());
			TArray<FPendingSpatialData>& PendingData = PendingDataArrays[0];
			if (int32* Existing = ParticleToPendingData.Find(UniqueIdx))
			{
				const int32 SlotIdx = *Existing;
				if (SlotIdx + 1 < PendingData.Num())
				{
					const FUniqueIdx LastElemUniqueIdx = PendingData.Last().UniqueIdx();
					ParticleToPendingData.FindChecked(LastElemUniqueIdx) = SlotIdx;	//We're going to swap elements so the last element is now in the position of the element we removed
				}

				Duplicated.RemoveAtSwap(SlotIdx);
				PendingData.RemoveAtSwap(SlotIdx);
				ParticleToPendingData.RemoveChecked(UniqueIdx);
			}
			check(PendingDataArrays[0].Num() == Duplicated.Num());
		}
	}

	void CleanUpDuplicated()
	{
		if (!PendingDataArrays.IsEmpty())
		{
			check(PendingDataArrays[0].Num() == Duplicated.Num());
			ParticleToPendingData.Reset();
			TArray<FPendingSpatialData> NoDuplicatedData;
			for (int32 Index = 0; Index < Duplicated.Num(); Index++)
			{
				const bool bIsDuplicated = Duplicated[Index];
				if (!bIsDuplicated)
				{
					NoDuplicatedData.Add(PendingDataArrays[0][Index]);
					ParticleToPendingData.Add(PendingDataArrays[0][Index].UniqueIdx(), Index);
				}
			}
			Duplicated.Init(false, NoDuplicatedData.Num());
			PendingDataArrays[0] = MoveTemp(NoDuplicatedData);
			check(PendingDataArrays[0].Num() == Duplicated.Num());
		}
	}

};

struct FPendingSpatialDataQueue
{
	TArray<FPendingSpatialData> PendingData;
	TArrayAsMap<FUniqueIdx,int32> ParticleToPendingData;

	void Reset()
	{
		PendingData.Reset();
		ParticleToPendingData.Reset();
	}

	int32 Num() const
	{
		return PendingData.Num();
	}

	FPendingSpatialData& FindOrAdd(const FUniqueIdx UniqueIdx, EPendingSpatialDataOperation Operation = EPendingSpatialDataOperation::Add)
	{
		if(int32* Existing = ParticleToPendingData.Find(UniqueIdx))
		{
			return PendingData[*Existing];
		} else
		{
			const int32 NewIdx = PendingData.AddDefaulted(1);
			ParticleToPendingData.Add(UniqueIdx,NewIdx);
			PendingData[NewIdx].Operation = Operation;
			return PendingData[NewIdx];
		}
	}

	void Remove(const FUniqueIdx UniqueIdx)
	{
		if(int32* Existing = ParticleToPendingData.Find(UniqueIdx))
		{
			const int32 SlotIdx = *Existing;
			if(SlotIdx + 1 < PendingData.Num())
			{
				const FUniqueIdx LastElemUniqueIdx = PendingData.Last().UniqueIdx();
				ParticleToPendingData.FindChecked(LastElemUniqueIdx) = SlotIdx;	//We're going to swap elements so the last element is now in the position of the element we removed
			}

			PendingData.RemoveAtSwap(SlotIdx);
			ParticleToPendingData.RemoveChecked(UniqueIdx);
		}
	}
};

}
