// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "Chaos/KinematicGeometryParticles.h"
#include "Chaos/Transform.h"

namespace Chaos
{
template<class T, int d>
class TPerCellBoundaryConditions
{
  public:
	TPerCellBoundaryConditions(const TKinematicGeometryParticles<T, d>& CollisionParticles, const TKinematicGeometryParticles<T, d>& SourceParticles)
	    : MParticles(CollisionParticles), MSources(SourceParticles) {}
	~TPerCellBoundaryConditions() = default;

	void ApplyNeumann(const TUniformGrid<T, d>& Grid, TArrayFaceND<bool, d>& BoundaryConditions, TArrayFaceND<T, d>& Velocity, const T Dt, const Pair<int32, TVector<int32, d>>& Index)
	{
		BoundaryConditions(Index) = false;
		for (uint32 i = 0; i < MParticles.Size(); ++i)
		{
			const TVector<T, d>& X = Grid.Location(Index);
			TRigidTransform<T, d> Frame(MParticles.GetX(i), MParticles.GetR(i));
			if (MParticles.GetGeometry(i)->SignedDistance(Frame.InverseTransformPosition(X)) < 0)
			{
				BoundaryConditions(Index) = true;
				Velocity(Index) = MParticles.GetV(i)[Index.First];
				break;
			}
		}
		for (uint32 i = 0; i < MSources.Size(); ++i)
		{
			const TVector<T, d>& X = Grid.Location(Index);
			TRigidTransform<T, d> Frame(MSources.GetX(i), MSources.GetR(i));
			if (MSources.GetGeometry(i)->SignedDistance(Frame.InverseTransformPosition(X)) < 0)
			{
				BoundaryConditions(Index) = true;
				Velocity(Index) = MSources.GetV(i)[Index.First];
				break;
			}
		}
	}

	void ApplyDirichlet(const TUniformGrid<T, d>& Grid, TArrayND<bool, d>& BoundaryConditions, TArrayND<T, d>& Density, const T Dt, const TVector<int32, d>& Index)
	{
		BoundaryConditions(Index) = false;
		// This is not a real boundary condition; just a convinient place to set density from sources
		for (uint32 i = 0; i < MSources.Size(); ++i)
		{
			const TVector<T, d>& X = Grid.Location(Index);
			TRigidTransform<T, d> Frame(MSources.GetX(i), MSources.GetR(i));
			if (MSources.GetGeometry(i)->SignedDistance(Frame.InverseTransformPosition(X)) < 0)
			{
				Density(Index) = 1.f;
				break;
			}
		}
	}

  private:
	const TKinematicGeometryParticles<T, d>& MParticles;
	const TKinematicGeometryParticles<T, d>& MSources;
};
}
