// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayND.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Vector.h"

namespace Chaos
{
template<class T, int d>
class TPerCellBuoyancy
{
  public:
	TPerCellBuoyancy(const TArrayND<T, 3>& Density, const TVector<T, d>& Direction = TVector<T, d>(0, 1, 0), const T Magnitude = (T)9.8)
	    : MDensity(Density), MAcceleration(Magnitude * Direction) {}
	virtual ~TPerCellBuoyancy() = default;

	inline void Apply(const TUniformGrid<T, d>& Grid, TArrayFaceND<T, d>& Velocity, const T Dt, const Pair<int32, TVector<int32, d>> Index) const
	{
		T DensitySum = MDensity(Grid.ClampIndex(Index.Second + TVector<int32, d>::AxisVector(Index.First))) + MDensity(Index.Second);
		Velocity(Index) += (T)0.5 * DensitySum * MAcceleration[Index.First] * Dt;
	}

  private:
	const TArrayND<T, d>& MDensity;
	const TVector<T, d> MAcceleration;
};
}
