// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayFaceND.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Vector.h"

namespace Chaos
{
template<class T, int d>
class TPerCellGravity
{
  public:
	TPerCellGravity(const TVector<T, d>& Direction = TVector<T, d>(0, -1, 0), const T Magnitude = (T)9.8)
	    : MAcceleration(Direction * Magnitude) {}
	virtual ~TPerCellGravity() = default;

	inline void Apply(const TUniformGrid<T, d>& Grid, TArrayFaceND<T, d>& Velocity, const T Dt, const Pair<int32, TVector<int32, d>> Index) const
	{
		Velocity(Index) += MAcceleration[Index.First] * Dt;
	}

  private:
	TVector<T, d> MAcceleration;
};
}
