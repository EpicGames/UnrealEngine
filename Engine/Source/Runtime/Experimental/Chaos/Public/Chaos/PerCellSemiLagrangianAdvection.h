// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayND.h"
#include "Chaos/UniformGrid.h"

namespace Chaos
{
template<class T, int d>
class TPerCellSemiLagrangianAdvection
{
  public:
	TPerCellSemiLagrangianAdvection() {}
	~TPerCellSemiLagrangianAdvection() = default;

	template<class T_SCALAR>
	void Apply(const TUniformGrid<T, d>& Grid, TArrayND<T_SCALAR, d>& Scalar, const TArrayND<T_SCALAR, d>& ScalarN, const TArrayFaceND<T, d>& VelocityN, const T Dt, const TVector<int32, d>& Index)
	{
		TVector<T, d> Location = Grid.Location(Index);
		TVector<T, d> X = Grid.ClampMinusHalf(Location - Dt * Grid.LinearlyInterpolate(VelocityN, Location));
		Scalar(Index) = Grid.LinearlyInterpolate(ScalarN, X);
	}
};
}
