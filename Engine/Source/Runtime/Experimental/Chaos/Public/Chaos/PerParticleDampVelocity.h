// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/SoftsSimulationSpace.h"
#include "Chaos/Vector.h"
#include "Chaos/PBDSoftsSolverParticles.h"

namespace Chaos::Softs
{

class FPerParticleDampVelocity final
{
public:

	FPerParticleDampVelocity(const FSolverReal InCoefficient = (FSolverReal)0.01)
		: LinearCoefficient(InCoefficient)
		, AngularCoefficient(InCoefficient)
		, LocalDampingCoefficientIndex(ForceInit)
		, LocalDampingLinearCoefficientIndex(ForceInit)
		, LocalDampingAngularCoefficientIndex(ForceInit)
		, LocalDampingSpaceIndex(ForceInit)
	{
	}

	~FPerParticleDampVelocity() = default;

	void UpdatePositionBasedState(const FSolverParticles& Particles, const int32 Offset, const int32 Range);
	UE_INTERNAL CHAOS_API void UpdatePositionBasedState(const FSolverParticlesRange& Particles);

	// Apply damping without first checking for kinematic particles
	void ApplyFast(FSolverParticles& Particles, const FSolverReal /*Dt*/, const int32 Index) const
	{
		Apply(Particles.GetX(Index), Particles.V(Index));
	}

	void Apply(const FSolverVec3& X, FSolverVec3& V) const
	{
		const FSolverVec3 R = X - Xcm;
		const FSolverVec3 Dv = LinearCoefficient * (Vcm - V) + AngularCoefficient * FSolverVec3::CrossProduct(R, Omega);
		V += Dv;
	}

	bool IsEnabled() const
	{
		return LinearCoefficient > 0.f || AngularCoefficient > 0.f;
	}

	bool RequiresUpdatePositionBasedState() const
	{
		return LocalDampingSpace == EChaosSoftsLocalDampingSpace::CenterOfMass;
	}

	UE_INTERNAL CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const FSolverVec3& ReferenceLocation,
		const FSolverVec3& ReferenceVelocity,
		const FSolverVec3& ReferenceAngularVelocity);

private:
	EChaosSoftsLocalDampingSpace LocalDampingSpace = EChaosSoftsLocalDampingSpace::CenterOfMass;
	bool bPropertyIndicesInitialized = false;
	FSolverReal LinearCoefficient;
	FSolverReal AngularCoefficient;
	FSolverVec3 Xcm, Vcm, Omega;

	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(LocalDampingCoefficient, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(LocalDampingLinearCoefficient, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(LocalDampingAngularCoefficient, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(LocalDampingSpace, int32);
};

}  // End namespace Chaos::Softs

#if !defined(CHAOS_DAMP_VELOCITY_ISPC_ENABLED_DEFAULT)
#define CHAOS_DAMP_VELOCITY_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if INTEL_ISPC
#if UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING
static constexpr bool bChaos_DampVelocity_ISPC_Enabled = CHAOS_DAMP_VELOCITY_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_DampVelocity_ISPC_Enabled;
#endif // UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING
#endif
