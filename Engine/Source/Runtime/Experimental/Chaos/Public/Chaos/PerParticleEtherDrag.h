// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PerParticleRule.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Utilities.h"

namespace Chaos
{
	extern FRealSingle LinearEtherDragOverride;
	extern FRealSingle AngularEtherDragOverride;

	class FPerParticleEtherDrag : public FPerParticleRule
	{
	public:
		FPerParticleEtherDrag() {}
		virtual ~FPerParticleEtherDrag() = default;

		inline void ApplyHelper(FVec3& V, FVec3& W, FReal LinearDamp, FReal AngularDamp, FReal LinearLimitSq, FReal AngularLimitSq, FReal Dt) const
		{
			const FReal LinearDrag = LinearEtherDragOverride >= 0 ? LinearEtherDragOverride : LinearDamp * Dt;
			const FReal LinearMultiplier = FMath::Max(FReal(0), FReal(1) - LinearDrag);
			V *= LinearMultiplier;

			const FReal AngularDrag = AngularEtherDragOverride >= 0 ? AngularEtherDragOverride : AngularDamp * Dt;
			const FReal AngularMultiplier = FMath::Max(FReal(0), FReal(1) - AngularDrag);
			W *= AngularMultiplier;

			const FReal LinearSpeedSq = V.SizeSquared();
			const FReal AngularSpeedSq = W.SizeSquared();

			if(LinearSpeedSq > LinearLimitSq)
			{
				V *= FMath::Sqrt(LinearLimitSq / LinearSpeedSq);
			}

			if(AngularSpeedSq > AngularLimitSq)
			{
				W *= FMath::Sqrt(AngularLimitSq / AngularSpeedSq);
			}
		}

		inline void Apply(FDynamicParticles& InParticles, const FReal Dt, const int32 Index) const override
		{
			ensure(false);
		}

		inline void Apply(TRigidParticles<FReal, 3>& InParticles, const FReal Dt, const int32 Index) const override
		{
			FVec3 V = InParticles.GetV(Index);
			FVec3 W = InParticles.GetW(Index);
			ApplyHelper(V, W, InParticles.LinearEtherDrag(Index), InParticles.AngularEtherDrag(Index), InParticles.MaxLinearSpeedSq(Index), InParticles.MaxAngularSpeedSq(Index), Dt);
			InParticles.SetV(Index, V);
			InParticles.SetW(Index, W);
		}

		inline void Apply(TTransientPBDRigidParticleHandle<FReal, 3>& Particle, const FReal Dt) const override
		{
			FVec3 V = Particle.GetV();
			FVec3 W = Particle.GetW();
			ApplyHelper(V, W, Particle.LinearEtherDrag(), Particle.AngularEtherDrag(), Particle.MaxLinearSpeedSq(), Particle.MaxAngularSpeedSq(), Dt);
			Particle.SetV(V);
			Particle.SetW(W);
		}
	};

	template<class T, int d>
	using TPerParticleEtherDrag UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPerParticleEtherDrag instead") = FPerParticleEtherDrag;
}
