// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PerParticleRule.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/Utilities.h"

namespace Chaos
{
class FPerParticleEulerStepVelocity : public FPerParticleRule
{
  public:
	FPerParticleEulerStepVelocity() {}
	virtual ~FPerParticleEulerStepVelocity() = default;

	template<class T_PARTICLES>
	inline void ApplyHelper(T_PARTICLES& InParticles, const FReal Dt, const int32 Index) const
	{
		InParticles.SetV(Index, InParticles.GetV(Index) + InParticles.Acceleration(Index) * Dt);
	}

	inline void Apply(FDynamicParticles& InParticles, const FReal Dt, const int32 Index) const override
	{
		if (InParticles.InvM(Index) == 0)
			return;
		ApplyHelper(InParticles, Dt, Index);
	}

	inline void Apply(TRigidParticles<FReal, 3>& InParticles, const FReal Dt, const int32 Index) const override
	{
		if (InParticles.InvM(Index) == 0 || InParticles.Disabled(Index) || InParticles.Sleeping(Index))
			return;
		ApplyHelper(InParticles, Dt, Index);

		//
		// TODO: This is the first-order approximation.
		//       If needed, we might eventually want to do a second order Euler's Equation,
		//       but if we do that we'll need to do a transform into a rotating reference frame.
		//       Just using W += InvI * (Torque - W x (I * W)) * dt is not correct, since Torque
		//		 and W are in an inertial frame.
		//
		InParticles.SetW(Index, InParticles.GetW(Index) + InParticles.AngularAcceleration(Index) * Dt);
	}
	
	inline void Apply(TTransientPBDRigidParticleHandle<FReal, 3>& Particle, const FReal Dt) const override
	{
		Particle.SetV(Particle.GetV() + Particle.Acceleration() * Dt);
		Particle.SetW(Particle.GetW() + Particle.AngularAcceleration() * Dt);
	}
};

template<class T, int d>
using TPerParticleEulerStepVelocity UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPerParticleEulerStepVelocity instead") = FPerParticleEulerStepVelocity;

}
