// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ParticleHandle.h"
#include "Chaos/PerParticleRule.h"

namespace Chaos
{
	class FPerParticleExternalForces : public FPerParticleRule
	{
	public:
		FPerParticleExternalForces() {}

		virtual ~FPerParticleExternalForces() = default;

		inline void Apply(TTransientPBDRigidParticleHandle<FReal, 3>& HandleIn, const FReal Dt) const override
		{
			if (TPBDRigidParticleHandleImp<FReal, 3, true>* Handle = HandleIn.Handle())
			{
#if CHAOS_PARTICLEHANDLE_TODO
				Handle->F() += Handle->ExternalForce();
				Handle->Torque() += Handle->ExternalTorque();
#endif
			}
		}
	};

	template<class T, int d>
	using TPerParticleExternalForces UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPerParticleExternalForces instead") = FPerParticleExternalForces;

}
