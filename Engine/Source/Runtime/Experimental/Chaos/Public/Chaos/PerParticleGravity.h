// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/DynamicParticles.h"
#include "Chaos/PerParticleRule.h"
#include "Chaos/ParticleHandle.h"

namespace Chaos
{
	class FPerParticleGravity : public FPerParticleRule
	{
	public:
		static constexpr uint32 MaxGravityGroups = 8;

		FPerParticleGravity()
		{
			SetAllGravities(FVec3(0, 0, (FReal)-980.665));
		}

		FPerParticleGravity(const FVec3& G)
		{
			SetAllGravities(G);
		}
		FPerParticleGravity(const FVec3& Direction, const FReal Magnitude)
		{
			SetAllGravities(Direction * Magnitude);
		}
		virtual ~FPerParticleGravity() = default;

		void Apply(TTransientPBDRigidParticleHandle<FReal, 3>& Particle, const FReal Dt) const override
		{
			if(Particle.GravityEnabled())
			{
				Particle.Acceleration() += MAccelerations[Particle.GravityGroupIndex()];
			}
		}

		void SetAcceleration(const FVec3& Acceleration, int32 GroupIndex)
		{ 
			MAccelerations[GroupIndex] = Acceleration;
		}

		const FVec3& GetAcceleration(int32 GroupIndex) const
		{
			return MAccelerations[GroupIndex];
		}

	private:
		void SetAllGravities(const FVec3& Gravity)
		{
			for (int32 Index = 0; Index < MaxGravityGroups; Index++)
			{
				MAccelerations[Index] = Gravity;
			}
		}

	private:
		FVec3 MAccelerations[MaxGravityGroups];
	};

	template<class T, int d>
	using TPerParticleGravity UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPerParticleGravity instead") = FPerParticleGravity;
}
