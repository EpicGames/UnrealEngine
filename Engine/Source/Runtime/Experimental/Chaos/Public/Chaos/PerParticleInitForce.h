// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ParticleHandle.h"
#include "Chaos/PerParticleRule.h"

namespace Chaos
{

class FPerParticleInitForce : public FPerParticleRule
{
  public:
	FPerParticleInitForce() {}
	virtual ~FPerParticleInitForce() = default;

	inline void Apply(FDynamicParticles& InParticles, const FReal Dt, const int Index) const override
	{
		InParticles.Acceleration(Index) = FVec3(0);
	}

	inline void Apply(TRigidParticles<FReal, 3>& InParticles, const FReal Dt, const int Index) const override
	{
		InParticles.Acceleration(Index) = FVec3(0);
		InParticles.AngularAcceleration(Index) = FVec3(0);
	}

	inline void Apply(TTransientPBDRigidParticleHandle<FReal, 3>& Particle, const FReal Dt) const override
	{
		Particle.Acceleration() = FVec3(0);
		Particle.AngularAcceleration() = FVec3(0);
	}
};

template<class T, int d>
using TPerParticleInitForce UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPerParticleInitForce instead") = FPerParticleInitForce;
}
