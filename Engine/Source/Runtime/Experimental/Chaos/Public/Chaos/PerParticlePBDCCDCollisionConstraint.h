// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Internal

#include "Chaos/SoftsSolverCollisionParticles.h"
#include "Chaos/Transform.h"
#include "Chaos/PBDActiveView.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "Chaos/Levelset.h"
#include "Misc/ScopeLock.h"

namespace Chaos::Softs
{

class FPerParticlePBDCCDCollisionConstraint final
{
public:
	FPerParticlePBDCCDCollisionConstraint(
		const TPBDActiveView<FSolverCollisionParticles>& InCollisionParticlesActiveView,
		TArray<FSolverRigidTransform3>& InCollisionTransforms,
		TArray<bool>& InCollided,
		TArray<FSolverVec3>& InContacts,
		TArray<FSolverVec3>& InNormals,
		TArray<uint32>& InDynamicGroupIds,
		TArray<uint32>& InKinematicGroupIds,
		const TArray<FSolverReal>& InPerGroupThicknesses,
		const TArray<FSolverReal>& InPerGroupFriction,
		bool bWriteCCDContacts)
		: CollisionParticlesActiveView(InCollisionParticlesActiveView)
		, CollisionTransforms(InCollisionTransforms)
		, Collided(InCollided)
		, Contacts(InContacts)
		, Normals(InNormals)
		, DynamicGroupIds(InDynamicGroupIds)
		, KinematicGroupIds(InKinematicGroupIds)
		, PerGroupThicknesses(InPerGroupThicknesses)
		, PerGroupFriction(InPerGroupFriction)
		, Mutex(bWriteCCDContacts ? new FCriticalSection : nullptr)
	{
	}

	~FPerParticlePBDCCDCollisionConstraint()
	{
		delete Mutex;
	}

	inline void ApplyRange(FSolverParticles& Particles, const FSolverReal Dt, const int32 Offset, const int32 Range) const
	{
		if (Mutex)
		{
			ApplyRangeHelper<true>(Particles, Dt, Offset, Range);
		}
		else
		{
			ApplyRangeHelper<false>(Particles, Dt, Offset, Range);
		}
	}

private:
	template<bool bLockAndWriteContacts>
	inline void ApplyRangeHelper(FSolverParticles& Particles, const FSolverReal Dt, const int32 Offset, const int32 Range) const
	{
		const uint32 DynamicGroupId = DynamicGroupIds[Offset];  // Particle group Id, must be the same across the entire range
		const FSolverReal Friction = PerGroupFriction[DynamicGroupId];
		const FSolverReal Thickness = PerGroupThicknesses[DynamicGroupId];

		PhysicsParallelFor(Range - Offset, [this, &Particles, Offset, DynamicGroupId, Thickness, Friction, Dt](int32 i)
		{
			const int32 Index = Offset + i;

			if (Particles.InvM(Index) == (FSolverReal)0.)
			{
				return;  // Continue
			}

			CollisionParticlesActiveView.SequentialFor([this, &Particles, &Index, DynamicGroupId, Thickness, Friction, Dt](FSolverCollisionParticles& CollisionParticles, int32 CollisionIndex)
			{
				const uint32 KinematicGroupId = KinematicGroupIds[CollisionIndex];  // Collision group Id

				if (KinematicGroupId != (uint32)INDEX_NONE && DynamicGroupId != KinematicGroupId)
				{
					return; // Bail out if the collision groups doesn't match the particle group id, or use INDEX_NONE (= global collision that affects all particle)
				}

				const FSolverRigidTransform3 Frame(CollisionParticles.GetX(CollisionIndex), CollisionParticles.GetR(CollisionIndex));

				const Pair<FVec3, bool> PointPair = CollisionParticles.GetGeometry(CollisionIndex)->FindClosestIntersection(  // Geometry operates in FReal
					FVec3(CollisionTransforms[CollisionIndex].InverseTransformPositionNoScale(Particles.GetX(Index))),        // hence the back and forth
					FVec3(Frame.InverseTransformPositionNoScale(Particles.P(Index))), (FReal)Thickness);                   // FVec3/FReal conversions

				if (PointPair.Second)
				{
					Collided[CollisionIndex] = true;

					const FSolverVec3 Normal = FSolverVec3(CollisionParticles.GetGeometry(CollisionIndex)->Normal(PointPair.First));
					const FSolverVec3 NormalWorld = Frame.TransformVectorNoScale(Normal);
					const FSolverVec3 ContactWorld = Frame.TransformPositionNoScale(UE::Math::TVector<FSolverReal>(PointPair.First));

					if (bLockAndWriteContacts)
					{
						check(Mutex);
						FScopeLock Lock(Mutex);
						Contacts.Emplace(ContactWorld);
						Normals.Emplace(NormalWorld);
					}
					const FSolverVec3 Direction = ContactWorld - Particles.P(Index);
					const FSolverReal Penetration = FMath::Max((FSolverReal)0., FSolverVec3::DotProduct(NormalWorld, Direction)) + (FSolverReal)UE_THRESH_POINT_ON_PLANE;

					Particles.P(Index) += Penetration * NormalWorld;

					// Friction
					const FSolverVec3 VectorToPoint = Particles.P(Index) - CollisionParticles.GetX(CollisionIndex);
					const FSolverVec3 RelativeDisplacement = (Particles.P(Index) - Particles.GetX(Index)) - (CollisionParticles.V(CollisionIndex) + FSolverVec3::CrossProduct(CollisionParticles.W(CollisionIndex), VectorToPoint)) * Dt;  // This corresponds to the tangential velocity multiplied by dt (friction will drive this to zero if it is high enough)
					const FSolverVec3 RelativeDisplacementTangent = RelativeDisplacement - FSolverVec3::DotProduct(RelativeDisplacement, NormalWorld) * NormalWorld;  // Project displacement into the tangential plane
					const FSolverReal RelativeDisplacementTangentLength = RelativeDisplacementTangent.Size();
					if (RelativeDisplacementTangentLength >= UE_SMALL_NUMBER)
					{
						const FSolverReal PositionCorrection = FMath::Min<FSolverReal>(Penetration * Friction, RelativeDisplacementTangentLength);
						const FSolverReal CorrectionRatio = PositionCorrection / RelativeDisplacementTangentLength;
						Particles.P(Index) -= CorrectionRatio * RelativeDisplacementTangent;
					}
				}
			});
		});
	}

private:
	// TODO(mlentine): Need a bb hierarchy
	const TPBDActiveView<FSolverCollisionParticles>& CollisionParticlesActiveView;
	const TArray<FSolverRigidTransform3>& CollisionTransforms;
	TArray<bool>& Collided;
	TArray<FSolverVec3>& Contacts;
	TArray<FSolverVec3>& Normals;
	const TArray<uint32>& DynamicGroupIds;
	const TArray<uint32>& KinematicGroupIds;
	const TArray<FSolverReal>& PerGroupThicknesses;
	const TArray<FSolverReal>& PerGroupFriction;
	FCriticalSection* const Mutex;
};

}  // End namespace Chaos::Softs
