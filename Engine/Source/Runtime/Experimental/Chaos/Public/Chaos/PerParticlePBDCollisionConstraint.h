// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/PBDActiveView.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverCollisionParticles.h"
#include "Chaos/WeightedLatticeImplicitObject.h"
#include "Chaos/Levelset.h"
#include "HAL/PlatformMath.h"

#if !defined(CHAOS_PER_PARTICLE_COLLISION_ISPC_ENABLED_DEFAULT)
#define CHAOS_PER_PARTICLE_COLLISION_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || (UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
const bool bChaos_PerParticleCollision_ISPC_Enabled = INTEL_ISPC && CHAOS_PER_PARTICLE_COLLISION_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_PerParticleCollision_ISPC_Enabled;
#endif

namespace Chaos::Softs
{

class FPerParticlePBDCollisionConstraint final
{
	struct FVelocityConstraint
	{
		FSolverVec3 Velocity;
		FSolverVec3 Normal;
	};

public:
	FPerParticlePBDCollisionConstraint(const TPBDActiveView<FSolverCollisionParticles>& InParticlesActiveView, TArray<bool>& Collided, TArray<uint32>& DynamicGroupIds, TArray<uint32>& KinematicGroupIds, const TArray<FSolverReal>& PerGroupThickness, const TArray<FSolverReal>& PerGroupFriction)
	: bFastPositionBasedFriction(true)
	, MCollisionParticlesActiveView(InParticlesActiveView)
	, MCollided(Collided), MDynamicGroupIds(DynamicGroupIds)
	, MKinematicGroupIds(KinematicGroupIds), MPerGroupThickness(PerGroupThickness)
	, MPerGroupFriction(PerGroupFriction) {}

	FPerParticlePBDCollisionConstraint(const TPBDActiveView<FSolverCollisionParticles>& InParticlesActiveView, TArray<bool>& Collided,
		TArray<FSolverVec3>& InContacts,
		TArray<FSolverVec3>& InNormals, 
		TArray<FSolverReal>& InPhis,
		TArray<uint32>& DynamicGroupIds, TArray<uint32>& KinematicGroupIds, const TArray<FSolverReal>& PerGroupThickness, const TArray<FSolverReal>& PerGroupFriction,
		bool bWriteCCDContacts)
	    : bFastPositionBasedFriction(true), MCollisionParticlesActiveView(InParticlesActiveView), MCollided(Collided)
		, Contacts(&InContacts)
		, Normals(&InNormals)
		, Phis(&InPhis)
		, MDynamicGroupIds(DynamicGroupIds), MKinematicGroupIds(KinematicGroupIds), MPerGroupThickness(PerGroupThickness), MPerGroupFriction(PerGroupFriction)
		, Mutex(bWriteCCDContacts ? new FCriticalSection : nullptr) {}

	~FPerParticlePBDCollisionConstraint() 
	{
		delete Mutex;
	}

	CHAOS_API void ApplyRange(FSolverParticles& Particles, const FSolverReal Dt, const int32 Offset, const int32 Range) const;

	void ApplyFriction(FSolverParticles& Particles, const FSolverReal Dt, const int32 Index) const
	{
		check(!bFastPositionBasedFriction);  // Do not call this function if this is setup to run with fast PB friction

		if (!MVelocityConstraints.Contains(Index))
		{
			return;
		}
		const FSolverReal VN = FSolverVec3::DotProduct(Particles.V(Index), MVelocityConstraints[Index].Normal);
		const FSolverReal VNBody = FSolverVec3::DotProduct(MVelocityConstraints[Index].Velocity, MVelocityConstraints[Index].Normal);
		const FSolverVec3 VTBody = MVelocityConstraints[Index].Velocity - VNBody * MVelocityConstraints[Index].Normal;
		const FSolverVec3 VTRelative = Particles.V(Index) - VN * MVelocityConstraints[Index].Normal - VTBody;
		const FSolverReal VTRelativeSize = VTRelative.Size();
		const FSolverReal VNMax = FMath::Max(VN, VNBody);
		const FSolverReal VNDelta = VNMax - VN;
		const FSolverReal CoefficientOfFriction = MPerGroupFriction[MDynamicGroupIds[Index]];
		check(CoefficientOfFriction > 0);
		const FSolverReal Friction = CoefficientOfFriction * VNDelta < VTRelativeSize ? CoefficientOfFriction * VNDelta / VTRelativeSize : 1;
		Particles.V(Index) = VNMax * MVelocityConstraints[Index].Normal + VTBody + VTRelative * (1 - Friction);
	}

private:
	template<bool bLockAndWriteContacts>
	void ApplyHelper(FSolverParticles& Particles, const FSolverReal Dt, const int32 Offset, const int32 Range) const;
	void ApplyHelperISPC(FSolverParticles& Particles, const FSolverReal Dt, const int32 Offset, const int32 Range) const;

private:
	bool bFastPositionBasedFriction;
	// TODO(mlentine): Need a bb hierarchy
	const TPBDActiveView<FSolverCollisionParticles>& MCollisionParticlesActiveView;
	TArray<bool>& MCollided;
	TArray<FSolverVec3>* const Contacts = nullptr;
	TArray<FSolverVec3>* const Normals = nullptr;
	TArray<FSolverReal>* const Phis = nullptr;
	const TArray<uint32>& MDynamicGroupIds;
	const TArray<uint32>& MKinematicGroupIds;
	mutable TMap<int32, FVelocityConstraint> MVelocityConstraints;
	const TArray<FSolverReal>& MPerGroupThickness;
	const TArray<FSolverReal>& MPerGroupFriction;
	FCriticalSection* const Mutex = nullptr;
};

}  // End namespace Chaos::Softs
