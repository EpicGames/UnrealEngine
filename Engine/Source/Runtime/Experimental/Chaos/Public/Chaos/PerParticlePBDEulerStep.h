// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PerParticleRule.h"

namespace Chaos
{
/**
 * Update position and rotation based on velocity and angular velocity.
 */
class FPerParticlePBDEulerStep : public FPerParticleRule
{
  public:
	  FPerParticlePBDEulerStep() {}
	virtual ~FPerParticlePBDEulerStep() = default;

	template<class T_PARTICLES>
	inline void ApplyHelper(T_PARTICLES& InParticles, const FReal Dt, const int32 Index) const
	{
		InParticles.SetP(Index, InParticles.GetX(Index) + InParticles.GetV(Index) * Dt);
	}

	inline void Apply(FPBDParticles& InParticles, const FReal Dt, const int32 Index) const override
	{
		ApplyHelper(InParticles, Dt, Index);
	}

	inline void Apply(TPBDRigidParticles<FReal, 3>& InParticles, const FReal Dt, const int32 Index) const override
	{
		FVec3 PCoM = InParticles.XCom(Index);
		FRotation3 QCoM = InParticles.RCom(Index);

		PCoM = PCoM + InParticles.GetV(Index) * Dt;
		QCoM = FRotation3::IntegrateRotationWithAngularVelocity(QCoM, InParticles.GetW(Index), Dt);

		InParticles.SetTransformPQCom(Index, PCoM, QCoM);
	}

	inline void Apply(TTransientPBDRigidParticleHandle<FReal, 3>& Particle, const FReal Dt) const override
	{
		FVec3 PCoM = Particle.XCom();
		FRotation3 QCoM = Particle.RCom();

		PCoM = PCoM + Particle.GetV() * Dt;
		QCoM = FRotation3::IntegrateRotationWithAngularVelocity(QCoM, Particle.GetW(), Dt);

		Particle.SetTransformPQCom(PCoM, QCoM);
	}
};

template<class T, int d>
using TPerParticlePBDEulerStep UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPerParticlePBDEulerStep instead") = FPerParticlePBDEulerStep;
}
