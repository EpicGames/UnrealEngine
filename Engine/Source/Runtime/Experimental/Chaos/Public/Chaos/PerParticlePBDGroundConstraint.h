// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "PerParticleRule.h"

namespace Chaos
{
class FPerParticlePBDGroundConstraint : public FPerParticleRule
{
  public:
	  FPerParticlePBDGroundConstraint(const FReal Height = 0)
	    : MHeight(Height) {}
	virtual ~FPerParticlePBDGroundConstraint() = default;

	inline void Apply(FPBDParticles& InParticles, const FReal Dt, const int32 Index) const override
	{
		if (InParticles.P(Index)[1] >= MHeight || InParticles.InvM(Index) == 0)
			return;
		InParticles.P(Index)[1] = MHeight;
	}

  private:
	FReal MHeight;
};

template<class T, int d>
using PerParticlePBDGroundConstraint UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPerParticlePBDGroundConstraint instead") = FPerParticlePBDGroundConstraint;

}
