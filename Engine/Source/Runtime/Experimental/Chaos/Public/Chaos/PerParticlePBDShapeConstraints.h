// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - This file does not compile anymore (Base ctor not matching)

#include "Chaos/PBDShapeConstraintsBase.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"

namespace Chaos::Softs
{

class UE_DEPRECATED(5.0, "Per particle constraint rules are no longer used by the cloth solver. Use FPBDShapeConstraints instead.") FPerParticlePBDShapeConstraints final : public FPBDShapeConstraintsBase
{
	typedef FPBDShapeConstraintsBase Base;

  public:
	FPerParticlePBDShapeConstraints(const FSolverReal Stiffness = (FSolverReal)1.)
	    : Base(Stiffness)
	{
	}
	FPerParticlePBDShapeConstraints(const FSolverParticles& InParticles, const TArray<FSolverVec3>& TargetPositions, const FSolverReal Stiffness = (FSolverReal)1.)
	    : Base(InParticles, TargetPositions, Stiffness)
	{
	}
	virtual ~FPerParticlePBDShapeConstraints() override {}

	// TODO(mlentine): We likely need to use time n positions here
	void Apply(FSolverParticles& InParticles, const FSolverReal Dt, const int32 Index) const
	{
		if (InParticles.InvM(Index) > 0)
		{
			InParticles.P(Index) -= InParticles.InvM(Index) * Base::GetDelta(InParticles, Index);
		}
	}

	void Apply(FSolverParticles& InParticles, const FSolverReal Dt) const
	{
		PhysicsParallelFor(InParticles.Size(), [&](int32 Index) {
			Apply(InParticles, Dt, Index);
		});
	}
};

}  // End namespace Chaos::Softs
