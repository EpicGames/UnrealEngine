// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"

namespace Chaos::Softs
{

class UE_DEPRECATED(5.0, "Per particle constraint rules are no longer used by the cloth solver. Use FPBDSpringConstraints instead.") FPerParticlePBDSpringConstraints : public FPBDSpringConstraintsBase
{
	typedef FPBDSpringConstraintsBase Base;
	using Base::Constraints;
	using Base::Stiffness;

public:
	FPerParticlePBDSpringConstraints(const FSolverParticles& InParticles, const TArray<TVec2<int32>>& InConstraints, const FSolverReal InStiffness = (FSolverReal)1.)
	    : Base(InParticles, 0, 0, InConstraints, TConstArrayView<FRealSingle>(), FSolverVec2(InStiffness))
	{
		for (int32 i = 0; i < Constraints.Num(); ++i)
		{
			const auto& Constraint = Constraints[i];
			int32 i1 = Constraint[0];
			int32 i2 = Constraint[1];
			if (i1 >= ParticleToConstraints.Num())
			{
				ParticleToConstraints.SetNum(i1 + 1);
			}
			if (i2 >= ParticleToConstraints.Num())
			{
				ParticleToConstraints.SetNum(i2 + 1);
			}
			ParticleToConstraints[i1].Add(i);
			ParticleToConstraints[i2].Add(i);
		}
	}
	virtual ~FPerParticlePBDSpringConstraints() override {}

	// TODO(mlentine): We likely need to use time n positions here
	void Apply(FSolverParticles& InParticles, const FSolverReal Dt, const int32 Index) const
	{
		for (int32 i = 0; i < ParticleToConstraints[Index].Num(); ++i)
		{
			int32 CIndex = ParticleToConstraints[Index][i];
			const auto& Constraint = Constraints[CIndex];
			int32 i1 = Constraint[0];
			int32 i2 = Constraint[1];
			if (Index == i1 && InParticles.InvM(i1) > 0)
			{
				InParticles.P(i1) -= InParticles.InvM(i1) * Base::GetDelta(InParticles, CIndex, (FSolverReal)Stiffness);
			}
			else if (InParticles.InvM(i2) > 0)
			{
				check(Index == i2);
				InParticles.P(i2) += InParticles.InvM(i2) * Base::GetDelta(InParticles, CIndex, (FSolverReal)Stiffness);
			}
		}
	}

private:
	TArray<TArray<int32>> ParticleToConstraints;
};

}  // End namespace Chaos::Softs
