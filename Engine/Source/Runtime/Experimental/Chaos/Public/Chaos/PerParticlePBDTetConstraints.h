// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/PBDTetConstraintsBase.h"

namespace Chaos::Softs
{

class UE_DEPRECATED(5.0, "Per particle constraint rules are no longer used by the cloth solver. Use Softs::FPBDTetConstraints instead.") FPerParticlePBDTetConstraints : public Softs::FPBDTetConstraintsBase
{
	typedef Softs::FPBDTetConstraintsBase Base;
	using Base::Constraints;

  public:
	  FPerParticlePBDTetConstraints(const FSolverParticles& InParticles, TArray<TVec4<int32>>&& InConstraints, const FSolverReal InStiffness = (FSolverReal)1.)
	    : Base(InParticles, MoveTemp(InConstraints), InStiffness)
	{
		for (int32 i = 0; i < Constraints.Num(); ++i)
		{
			const TVec4<int32>& Constraint = Constraints[i];
			int32 i1 = Constraint[0];
			int32 i2 = Constraint[1];
			int32 i3 = Constraint[2];
			int32 i4 = Constraint[3];
			if (i1 >= ParticleToConstraints.Num())
			{
				ParticleToConstraints.SetNum(i1 + 1);
			}
			if (i2 >= ParticleToConstraints.Num())
			{
				ParticleToConstraints.SetNum(i2 + 1);
			}
			if (i3 >= ParticleToConstraints.Num())
			{
				ParticleToConstraints.SetNum(i3 + 1);
			}
			if (i4 >= ParticleToConstraints.Num())
			{
				ParticleToConstraints.SetNum(i4 + 1);
			}
			ParticleToConstraints[i1].Add(i);
			ParticleToConstraints[i2].Add(i);
			ParticleToConstraints[i2].Add(i);
			ParticleToConstraints[i2].Add(i);
		}
	}
	virtual ~FPerParticlePBDTetConstraints() override
	{
	}

	void Apply(FSolverParticles& InParticles, const FSolverReal Dt, const int32 Index) const
	{
		for (int i = 0; i < ParticleToConstraints[Index].Num(); ++i)
		{
			int32 CIndex = ParticleToConstraints[Index][i];
			const TVec4<int32>& Constraint = Constraints[CIndex];
			int32 i1 = Constraint[0];
			int32 i2 = Constraint[1];
			int32 i3 = Constraint[2];
			int32 i4 = Constraint[3];
			const TVec4<FSolverVec3> Grads = Base::GetGradients(InParticles, CIndex);
			const FSolverReal S = Base::GetScalingFactor(InParticles, CIndex, Grads);
			if (Index == i1)
			{
				InParticles.P(i1) -= S * InParticles.InvM(i1) * Grads[0];
			}
			else if (Index == i2)
			{
				InParticles.P(i2) -= S * InParticles.InvM(i2) * Grads[1];
			}
			else if (Index == i3)
			{
				InParticles.P(i3) -= S * InParticles.InvM(i3) * Grads[2];
			}
			else
			{
				check(Index == i4);
				InParticles.P(i4) -= S * InParticles.InvM(i4) * Grads[3];
			}
		}
	}

  private:
	TArray<TArray<int32>> ParticleToConstraints;
};

}  // End namespace Chaos::Softs
