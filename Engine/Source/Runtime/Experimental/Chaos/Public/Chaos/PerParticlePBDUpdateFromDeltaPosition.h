// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/PBDParticles.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/PerParticleRule.h"

namespace Chaos
{
class FPerParticlePBDUpdateFromDeltaPosition : public FPerParticleRule
{
  public:
	  FPerParticlePBDUpdateFromDeltaPosition() {}
	virtual ~FPerParticlePBDUpdateFromDeltaPosition() = default;

	template<class T_PARTICLES>
	inline void ApplyHelper(T_PARTICLES& InParticles, const FReal Dt, const int32 Index) const
	{
		InParticles.SetV(Index, (InParticles.GetP(Index) - InParticles.GetX(Index)) / Dt);
	}

	inline void Apply(FPBDParticles& InParticles, const FReal Dt, const int32 Index) const override
	{
		InParticles.V(Index) = (InParticles.GetP(Index) - InParticles.GetX(Index)) / Dt;
		InParticles.SetX(Index,  InParticles.GetP(Index));
	}

	inline void Apply(TPBDRigidParticles<FReal, 3>& InParticles, const FReal Dt, const int32 Index) const override
	{
		ApplyHelper(InParticles, Dt, Index);
		InParticles.SetW(Index, FRotation3f::CalculateAngularVelocity(InParticles.GetRf(Index), InParticles.GetQf(Index), FRealSingle(Dt)));
	}

	inline void Apply(FPBDRigidParticleHandle* Particle, const FReal Dt) const override
	{
		const FVec3& CenterOfMass = Particle->CenterOfMass();
		const FVec3 CenteredX = Particle->XCom();
		const FVec3 CenteredP = Particle->PCom();
		Particle->SetV(FVec3::CalculateVelocity(CenteredX, CenteredP, Dt));
		Particle->SetWf(FRotation3f::CalculateAngularVelocity(Particle->GetRf(), Particle->GetQf(), FRealSingle(Dt)));
	}

	inline void Apply(TTransientPBDRigidParticleHandle<FReal, 3>& Particle, const FReal Dt) const override
	{
		const FVec3& CenterOfMass = Particle.CenterOfMass();
		const FVec3 CenteredX = Particle.XCom();
		const FVec3 CenteredP = Particle.PCom();
		Particle.SetV(FVec3::CalculateVelocity(CenteredX, CenteredP, Dt));
		Particle.SetWf(FRotation3f::CalculateAngularVelocity(Particle.GetRf(), Particle.GetQf(), FRealSingle(Dt)));
	}
};

template<class T, int d>
using TPerParticlePBDUpdateFromDeltaPosition UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPerParticlePBDUpdateFromDeltaPosition instead") = FPerParticlePBDUpdateFromDeltaPosition;
}
