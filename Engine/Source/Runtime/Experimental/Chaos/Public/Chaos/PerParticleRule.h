// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDParticles.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/ParticleRule.h"

namespace Chaos
{

/**
 * A Particle Rule that applies some effect to all particles in parallel.
 * This should only be used if the effect on any particle is independent of
 * all others (i.e., the implementation of ApplySingle only reads/writes to
 * the one particle).
 */
class FPerParticleRule : public FParticleRule
{
  public:
	void Apply(FParticles& InParticles, const FReal Dt) const override
	{
		ApplyPerParticle(InParticles, Dt);
	}

	void Apply(FDynamicParticles& InParticles, const FReal Dt) const override
	{
		ApplyPerParticle(InParticles, Dt);
	}

	void Apply(FPBDParticles& InParticles, const FReal Dt) const override
	{
		ApplyPerParticle(InParticles, Dt);
	}

	template<class T_PARTICLES>
	void ApplyPerParticle(T_PARTICLES& InParticles, const FReal Dt) const
	{
		PhysicsParallelFor(InParticles.Size(), [&](int32 Index) {
			Apply(InParticles, Dt, Index);
		});
	}

	virtual void Apply(FParticles& InParticles, const FReal Dt, const int Index) const { check(0); };
	virtual void Apply(FDynamicParticles& InParticles, const FReal Dt, const int Index) const { Apply(static_cast<FParticles&>(InParticles), Dt, Index); };
	virtual void Apply(FPBDParticles& InParticles, const FReal Dt, const int Index) const { Apply(static_cast<FDynamicParticles&>(InParticles), Dt, Index); };
	virtual void Apply(TRigidParticles<FReal, 3>& InParticles, const FReal Dt, const int Index) const { Apply(static_cast<FParticles&>(InParticles), Dt, Index); };
	virtual void Apply(FPBDRigidParticles& InParticles, const FReal Dt, const int Index) const { Apply(static_cast<TRigidParticles<FReal, 3>&>(InParticles), Dt, Index); };

	virtual void Apply(TPBDRigidParticleHandle<FReal, 3>* Particle, const FReal Dt) const { check(0); }
	virtual void Apply(TTransientPBDRigidParticleHandle<FReal, 3>& Particle, const FReal Dt) const { check(0); }
};

template<class T, int d>
using TPerParticleRule UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FPerParticleRule instead") = FPerParticleRule;

}
