// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Chaos/Defines.h"
#include "Chaos/HandleArray.h"

namespace Chaos
{
	using FMaterialArray = THandleArray<FChaosPhysicsMaterial>;
	using FChaosMaterialHandle = FMaterialArray::FHandle;
	using FChaosConstMaterialHandle = FMaterialArray::FConstHandle;
	using FMaterialMaskArray = THandleArray<FChaosPhysicsMaterialMask>;
	using FChaosMaterialMaskHandle = FMaterialMaskArray::FHandle;
	using FChaosConstMaterialMaskHandle = FMaterialMaskArray::FConstHandle;

	/** 
	 * Helper wrapper to encapsulate the access through the material manager
	 * Handles returned from the material manager are only designed to be used (resolved) on the game thread
	 * by the physics interface.
	 */
	struct FMaterialHandle
	{
		// Get should be only called on the Game Thread, instead use GetInternal on Physics Thread
		CHAOS_API FChaosPhysicsMaterial* Get() const;
		CHAOS_API FChaosPhysicsMaterial* GetInternal(const THandleArray<FChaosPhysicsMaterial>* const SimMaterials) const;
		FChaosMaterialHandle InnerHandle;

		// IsValid should be only called on the Game Thread, instead use IsValidInternal on Physics Thread
		bool IsValid() const { return Get() != nullptr; }
		bool IsValidInternal(const THandleArray<FChaosPhysicsMaterial>* const SimMaterials) const { return GetInternal(SimMaterials) != nullptr; }
		void Reset() { InnerHandle = FChaosMaterialHandle(); }

		friend bool operator ==(const FMaterialHandle& A, const FMaterialHandle& B) { return A.InnerHandle == B.InnerHandle; }

		friend FArchive& operator <<(FArchive& Ar, FMaterialHandle& Value) { Ar << Value.InnerHandle; return Ar; }
	};

	/**
	 * Helper wrapper to encapsulate the access through the material manager
	 * Handles returned from the material manager are only designed to be used (resolved) on the game thread
	 * by the physics interface.
	 */
	struct FConstMaterialHandle
	{
		CHAOS_API const FChaosPhysicsMaterial* Get() const;
		FChaosConstMaterialHandle InnerHandle;

		bool IsValid() const { return Get() != nullptr; }

		friend bool operator ==(const FConstMaterialHandle& A, const FConstMaterialHandle& B) { return A.InnerHandle == B.InnerHandle; }

		friend FArchive& operator <<(FArchive& Ar, FConstMaterialHandle& Value) { Ar << Value.InnerHandle; return Ar; }
	};

	/**
	 * Helper wrapper to encapsulate the access through the material manager
	 * Handles returned from the material manager are only designed to be used (resolved) on the game thread
	 * by the physics interface.
	 */
	struct FMaterialMaskHandle
	{
		CHAOS_API FChaosPhysicsMaterialMask* Get() const;
		FChaosMaterialMaskHandle InnerHandle;

		bool IsValid() const { return Get() != nullptr; }

		friend bool operator ==(const FMaterialMaskHandle& A, const FMaterialMaskHandle& B) { return A.InnerHandle == B.InnerHandle; }

		friend FArchive& operator <<(FArchive& Ar, FMaterialMaskHandle& Value) { Ar << Value.InnerHandle; return Ar; }
	};

	/**
	 * Helper wrapper to encapsulate the access through the material manager
	 * Handles returned from the material manager are only designed to be used (resolved) on the game thread
	 * by the physics interface.
	 */
	struct FConstMaterialMaskHandle
	{
		CHAOS_API const FChaosPhysicsMaterialMask* Get() const;
		FChaosConstMaterialMaskHandle InnerHandle;

		bool IsValid() const { return Get() != nullptr; }

		friend bool operator ==(const FConstMaterialMaskHandle& A, const FConstMaterialMaskHandle& B) { return A.InnerHandle == B.InnerHandle; }

		friend FArchive& operator <<(FArchive& Ar, FConstMaterialMaskHandle& Value) { Ar << Value.InnerHandle; return Ar; }
	};

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMaterialCreated, FMaterialHandle);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMaterialDestroyed, FMaterialHandle);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMaterialUpdated, FMaterialHandle);

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMaterialMaskCreated, FMaterialMaskHandle);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMaterialMaskDestroyed, FMaterialMaskHandle);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMaterialMaskUpdated, FMaterialMaskHandle);

	using FMaterialCreatedDelegate = FOnMaterialCreated::FDelegate;
	using FMaterialDestroyedDelegate = FOnMaterialDestroyed::FDelegate;
	using FMaterialUpdatedDelegate = FOnMaterialUpdated::FDelegate;

	using FMaterialMaskCreatedDelegate = FOnMaterialMaskCreated::FDelegate;
	using FMaterialMaskDestroyedDelegate = FOnMaterialMaskDestroyed::FDelegate;
	using FMaterialMaskUpdatedDelegate = FOnMaterialMaskUpdated::FDelegate;

	/** 
	 * Global manager for physical materials.
	 * Materials are created, updated and destroyed only on the game thread and an immutable
	 * copy of the materials are stored on each solver. The solvers module binds to the updated event
	 * in this manager and enqueues updates to all active solvers when a material is updated.
	 *
	 * The material manager provides handles for the objects which should be stored instead of the
	 * material pointer. When accessing the internal material always use the handle rather than
	 * storing the result of Get()
	 */
	class FPhysicalMaterialManager
	{
	public:

		static CHAOS_API FPhysicalMaterialManager& Get();

		/** Create a new material, returning a stable handle to it - this should be stored and not the actual material pointer */
		CHAOS_API FMaterialHandle Create();
		CHAOS_API FMaterialMaskHandle CreateMask();

		/** Destroy the material referenced by the provided handle */
		CHAOS_API void Destroy(FMaterialHandle InHandle);
		CHAOS_API void Destroy(FMaterialMaskHandle InHandle);

		/** Get the actual material from a handle */
		CHAOS_API FChaosPhysicsMaterial* Resolve(FChaosMaterialHandle InHandle) const;
		CHAOS_API const FChaosPhysicsMaterial* Resolve(FChaosConstMaterialHandle InHandle) const;

		CHAOS_API FChaosPhysicsMaterialMask* Resolve(FChaosMaterialMaskHandle InHandle) const;
		CHAOS_API const FChaosPhysicsMaterialMask* Resolve(FChaosConstMaterialMaskHandle InHandle) const;

		/** Signals stakeholders that the stored material for the provided handle has changed */
		CHAOS_API void UpdateMaterial(FMaterialHandle InHandle);
		CHAOS_API void UpdateMaterialMask(FMaterialMaskHandle InHandle);

		/** Gets the internal list of primary materials representing the current user state of the material data */
		UE_DEPRECATED(5.1, "GetMasterMaterials_External is deprecated, please use GetPrimaryMaterials_External instead")
		CHAOS_API const THandleArray<FChaosPhysicsMaterial>& GetMasterMaterials_External() const;
		UE_DEPRECATED(5.1, "GetMasterMaterialMasks_External is deprecated, please use GetPrimaryMaterialMasks_External instead")
		CHAOS_API const THandleArray<FChaosPhysicsMaterialMask>& GetMasterMaterialMasks_External() const;

		CHAOS_API const THandleArray<FChaosPhysicsMaterial>& GetPrimaryMaterials_External() const;
		CHAOS_API const THandleArray<FChaosPhysicsMaterialMask>& GetPrimaryMaterialMasks_External() const;

		/** Events */
		FOnMaterialUpdated OnMaterialUpdated;
		FOnMaterialCreated OnMaterialCreated;
		FOnMaterialDestroyed OnMaterialDestroyed;

		FOnMaterialMaskUpdated OnMaterialMaskUpdated;
		FOnMaterialMaskCreated OnMaterialMaskCreated;
		FOnMaterialMaskDestroyed OnMaterialMaskDestroyed;

	private:

		/** Initial size for the handle-managed array */
		static constexpr int32 InitialCapacity = 4;

		/** Material manager is a singleton - access with Get() */
		FPhysicalMaterialManager();

		/** Handle-managed array of global materials. This is pushed to all solvers who all maintain a copy */
		THandleArray<FChaosPhysicsMaterial> Materials;

		/** Handle-managed array of global material masks. This is pushed to all solvers who all maintain a copy */
		THandleArray<FChaosPhysicsMaterialMask> MaterialMasks;

		mutable FRWLock MaterialsLock;
		mutable FRWLock MaterialMasksLock;
	};
}

