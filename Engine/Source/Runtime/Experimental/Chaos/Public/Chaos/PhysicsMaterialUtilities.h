// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/HandleArray.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/Serializable.h"
#include "Chaos/ShapeInstanceFwd.h"
#include "Templates/UniquePtr.h"

namespace Chaos
{
	class FChaosPhysicsMaterial;

	namespace Private
	{
		// Get the physical material for the specified Shape on the particle.
		// NOTE: each shape may have its own material, but the particle may also have an override.
		extern CHAOS_API const FChaosPhysicsMaterial* GetPhysicsMaterial(
			const FGeometryParticleHandle* Particle,
			const FShapeInstance* Shape,
			const int32 FaceIndex,
			const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>* PhysicsMaterials,
			const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>* PerParticlePhysicsMaterials,
			const THandleArray<FChaosPhysicsMaterial>* const SimMaterials);

		// Get the first physical material on the particle (a multi-shape particle may have many)
		extern CHAOS_API const FChaosPhysicsMaterial* GetFirstPhysicsMaterial(
			const FGeometryParticleHandle* Particle,
			const TArrayCollectionArray<TSerializablePtr<FChaosPhysicsMaterial>>* PhysicsMaterials,
			const TArrayCollectionArray<TUniquePtr<FChaosPhysicsMaterial>>* PerParticlePhysicsMaterials,
			const THandleArray<FChaosPhysicsMaterial>* const SimMaterials);
	}
}
