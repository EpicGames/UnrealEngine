// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"
#include "UObject/NameTypes.h"

class IPhysicsProxyBase;
class FGeometryCollectionPhysicsProxy;

namespace Chaos
{
	/**
	 * The FPhysicsObject is effectively a reference to a single particle in the solver.
	 * It maintains this reference indirectly via the physics proxy. This object is meant to be usable on both the game thread and physics thread.
	 */
	struct FPhysicsObject;
	using FPhysicsObjectHandle = FPhysicsObject*;
	using FConstPhysicsObjectHandle = const FPhysicsObject*;
	using FPhysicsObjectId = int32;

	struct FPhysicsObjectDeleter
	{
		CHAOS_API void operator()(FPhysicsObjectHandle p);
	};
	using FPhysicsObjectUniquePtr = TUniquePtr<FPhysicsObject, FPhysicsObjectDeleter>;

} // namespace Chaos
