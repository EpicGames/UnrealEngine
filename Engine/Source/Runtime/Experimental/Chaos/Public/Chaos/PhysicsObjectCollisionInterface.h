// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Interface/SQTypes.h"
#include "Chaos/GeometryQueries.h"
#include "Chaos/PhysicsObject.h"
#include "Chaos/PhysicsObjectInterface.h"
#include "Framework/Threading.h"

namespace Chaos
{
	struct FSweepParameters
	{
		bool bSweepComplex = false;
		bool bComputeMTD = false;
	};

	template<EThreadContext Id>
	class FPhysicsObjectCollisionInterface
	{
	public:
		explicit FPhysicsObjectCollisionInterface(FReadPhysicsObjectInterface<Id>& InInterface) :
			Interface(InInterface)
		{}

		// This function will not compute any overlap heuristic.
		CHAOS_API bool PhysicsObjectOverlap(const FConstPhysicsObjectHandle ObjectA, const FTransform& InTransformA, const FConstPhysicsObjectHandle ObjectB, const FTransform& InTransformB, bool bTraceComplex);

		// Returns all the overlaps within A given a shape B.
		template<typename TOverlapHit>
		bool PhysicsObjectOverlap(const FConstPhysicsObjectHandle ObjectA, const FTransform& InTransformA, const FConstPhysicsObjectHandle ObjectB, const FTransform& InTransformB, bool bTraceComplex, TArray<TOverlapHit>& OutOverlaps)
		{
			static_assert(std::is_same_v<TOverlapHit, ChaosInterface::TThreadOverlapHit<Id>>);
			return PairwiseShapeOverlapHelper(
				ObjectA,
				InTransformA,
				ObjectB,
				InTransformB,
				bTraceComplex,
				false,
				FVector::Zero(),
				[this, ObjectA, &OutOverlaps](const FShapeOverlapData& A, const FShapeOverlapData& B, const FMTDInfo&)
				{
					TOverlapHit Overlap;
					Overlap.Shape = A.Shape;
					Overlap.Actor = Interface.GetParticle(ObjectA);
					OutOverlaps.Add(Overlap);
					return false;
				}
			);
		}

		// This function does the same as GetPhysicsObjectOverlap but also computes the MTD metric.
		CHAOS_API bool PhysicsObjectOverlapWithMTD(const FConstPhysicsObjectHandle ObjectA, const FTransform& InTransformA, const FConstPhysicsObjectHandle ObjectB, const FTransform& InTransformB, bool bTraceComplex, FMTDInfo& OutMTD);

		// This function does the same as GetPhysicsObjectOverlap but also computes the AABB overlap metric.
		CHAOS_API bool PhysicsObjectOverlapWithAABB(const FConstPhysicsObjectHandle ObjectA, const FTransform& InTransformA, const FConstPhysicsObjectHandle ObjectB, const FTransform& InTransformB, bool bTraceComplex, const FVector& Tolerance, FBox& OutOverlap);
		CHAOS_API bool PhysicsObjectOverlapWithAABBSize(const FConstPhysicsObjectHandle ObjectA, const FTransform& InTransformA, const FConstPhysicsObjectHandle ObjectB, const FTransform& InTransformB, bool bTraceComplex, const FVector& Tolerance, FVector& OutOverlapSize);
		CHAOS_API bool PhysicsObjectOverlapWithAABBIntersections(const FConstPhysicsObjectHandle ObjectA, const FTransform& InTransformA, const FConstPhysicsObjectHandle ObjectB, const FTransform& InTransformB, bool bTraceComplex, const FVector& Tolerance, TArray<FBox>& Intersections);

		template<typename TRaycastHit>
		bool LineTrace(TArrayView<const FConstPhysicsObjectHandle> InObjects, const FVector& WorldStart, const FVector& WorldEnd, bool bTraceComplex, TRaycastHit& OutBestHit)
		{
			static_assert(std::is_same_v<TRaycastHit, ChaosInterface::TThreadRaycastHit<Id>>);
			bool bHit = false;
			OutBestHit.Distance = TNumericLimits<float>::Max();

			const FVector Delta = WorldEnd - WorldStart;
			const FReal DeltaMag = Delta.Size();
			if (DeltaMag < UE_KINDA_SMALL_NUMBER)
			{
				return false;
			}

			FTransform BestWorldTM = FTransform::Identity;

			for (const FConstPhysicsObjectHandle Object : InObjects)
			{
				const FTransform WorldTM = Interface.GetTransform(Object);
				const FVector LocalStart = WorldTM.InverseTransformPositionNoScale(WorldStart);
				const FVector LocalDelta = WorldTM.InverseTransformVectorNoScale(Delta);

				Interface.VisitEveryShape(
					{ &Object, 1 },
					[this, &bHit, &WorldTM, &LocalStart, &LocalDelta, &Delta, DeltaMag, &BestWorldTM, bTraceComplex, &OutBestHit](const FConstPhysicsObjectHandle IterObject, TThreadShapeInstance<Id>* Shape)
					{
						check(Shape);

						const Filter::FShapeFilterData& FilterData = Shape->GetShapeFilterData();
						const bool bShapeIsComplex = FilterData.HasFlag(Chaos::EFilterFlags::ComplexCollision);
						const bool bShapeIsSimple = FilterData.HasFlag(Chaos::EFilterFlags::SimpleCollision);
						if ((bTraceComplex && bShapeIsComplex) || (!bTraceComplex && bShapeIsSimple))
						{
							FReal Distance;
							FVec3 LocalPosition;
							FVec3 LocalNormal;
							int32 FaceIndex;

							const bool bRaycastHit = Shape->GetGeometry()->Raycast(
								LocalStart,
								LocalDelta / DeltaMag,
								DeltaMag,
								0,
								Distance,
								LocalPosition,
								LocalNormal,
								FaceIndex
							);

							if (bRaycastHit)
							{
								if (Distance < OutBestHit.Distance)
								{
									bHit = true;
									BestWorldTM = WorldTM;
									OutBestHit.Distance = static_cast<float>(Distance);
									OutBestHit.WorldNormal = LocalNormal;
									OutBestHit.WorldPosition = LocalPosition;
									OutBestHit.Shape = Shape;
									OutBestHit.Actor = Interface.GetParticle(IterObject);
									OutBestHit.FaceIndex = FaceIndex;
								}
							}
						}
						return false;
					}
				);
			}

			if (bHit)
			{
				OutBestHit.WorldNormal = BestWorldTM.TransformVectorNoScale(OutBestHit.WorldNormal);
				OutBestHit.WorldPosition = BestWorldTM.TransformPositionNoScale(OutBestHit.WorldPosition);
			}

			return bHit;
		}

		template<typename TOverlapHit>
		bool ShapeOverlap(TArrayView<const FConstPhysicsObjectHandle> InObjects, const Chaos::FImplicitObject& InGeom, const FTransform& GeomTransform, TArray<TOverlapHit>& OutOverlaps)
		{
			return ShapeOverlapWithMTD(InObjects, InGeom, GeomTransform, OutOverlaps, nullptr);
		}

		template<typename TOverlapHit>
		bool ShapeOverlapWithMTD(TArrayView<const FConstPhysicsObjectHandle> InObjects, const Chaos::FImplicitObject& InGeom, const FTransform& GeomTransform, TArray<TOverlapHit>& OutOverlaps, FMTDInfo* MTD)
		{
			static_assert(std::is_same_v<TOverlapHit, ChaosInterface::TThreadOverlapHit<Id>>);
			bool bHasOverlap = false;
			for (const FConstPhysicsObjectHandle Object : InObjects)
			{
				const FTransform WorldTM = Interface.GetTransform(Object);

				Interface.VisitEveryShape(
					{ &Object, 1 },
					[this, &bHasOverlap, &WorldTM, &InGeom, &GeomTransform, &OutOverlaps, MTD](const FConstPhysicsObjectHandle IterObject, TThreadShapeInstance<Id>* Shape)
					{
						check(Shape);
						const bool bOverlap = Chaos::Utilities::CastHelper(
							InGeom,
							GeomTransform,
							[Shape, &WorldTM, MTD](const auto& Downcast, const auto& FullTransformB)
							{
								return Chaos::OverlapQuery(*Shape->GetGeometry(), WorldTM, Downcast, FullTransformB, 0, MTD);
							}
						);

						if (bOverlap)
						{
							bHasOverlap = true;

							TOverlapHit Overlap;
							Overlap.Shape = Shape;
							Overlap.Actor = Interface.GetParticle(IterObject);
							OutOverlaps.Add(Overlap);
						}
						return false;
					}
				);
			}
			return bHasOverlap;
		}

		template<typename TSweepHit>
		bool ShapeSweep(TArrayView<const FConstPhysicsObjectHandle> InObjects, const Chaos::FImplicitObject& InGeom, const FTransform& StartTM, const FVector& EndPos, const FSweepParameters& Params, TSweepHit& OutBestHit)
		{
			static_assert(std::is_same_v<TSweepHit, ChaosInterface::TThreadSweepHit<Id>>);
			bool bHit = false;
			const FVector StartPos = StartTM.GetTranslation();
			const FVector Delta = EndPos - StartPos;
			const FReal DeltaMag = Delta.Size();

			const bool bSweepHasLength = DeltaMag > 0.f;
			const FVector Dir = bSweepHasLength ? (Delta / DeltaMag) : FVector(1, 0, 0);

			OutBestHit.Distance = TNumericLimits<float>::Max();
			for (const FConstPhysicsObjectHandle Object : InObjects)
			{
				const FTransform WorldTM = Interface.GetTransform(Object);

				Interface.VisitEveryShape(
					{ &Object, 1 },
					[this, &WorldTM, &InGeom, &StartTM, &bHit, &Delta, DeltaMag, &Dir, &OutBestHit, &Params](const FConstPhysicsObjectHandle IterObject, TThreadShapeInstance<Id>* Shape)
					{
						check(Shape);

						const Filter::FShapeFilterData& FilterData = Shape->GetShapeFilterData();
						const bool bShapeIsComplex = FilterData.HasFlag(Chaos::EFilterFlags::ComplexCollision);
						const bool bShapeIsSimple = FilterData.HasFlag(Chaos::EFilterFlags::SimpleCollision);
						if ((Params.bSweepComplex && bShapeIsComplex) || (!Params.bSweepComplex && bShapeIsSimple))
						{
							FVec3 WorldPosition;
							FVec3 WorldNormal;
							FReal Distance;
							int32 FaceIdx;
							FVec3 FaceNormal;

							const bool bShapeHit = Chaos::Utilities::CastHelper(
								InGeom,
								StartTM,
								[Shape, &WorldTM, &Dir, DeltaMag, &Distance, &WorldPosition, &WorldNormal, &FaceIdx, &FaceNormal, &Params](const auto& Downcast, const auto& FullTransformB)
								{
									// Set bComputeMTD to true to better match internal SQ Visitor behavior.
									return Chaos::SweepQuery(*Shape->GetGeometry(), WorldTM, Downcast, FullTransformB, Dir, DeltaMag, Distance, WorldPosition, WorldNormal, FaceIdx, FaceNormal, 0.f, Params.bComputeMTD);
								}
							);

							const float FloatDistance = static_cast<float>(Distance);
							if (bShapeHit && FloatDistance < OutBestHit.Distance)
							{
								bHit = true;

								OutBestHit.Shape = Shape;
								OutBestHit.WorldPosition = WorldPosition;
								OutBestHit.WorldNormal = WorldNormal;
								OutBestHit.Distance = FloatDistance;
								OutBestHit.FaceIndex = FaceIdx;
								OutBestHit.FaceNormal = FaceNormal;

								// Always compute the best face index. This used to check for non-initial overlap hits, however there's no reason to do that as we should always have valid data.
								const FVector LocalPosition = WorldTM.InverseTransformPositionNoScale(OutBestHit.WorldPosition);
								const FVector LocalUnitDir = WorldTM.InverseTransformVectorNoScale(Dir);
								OutBestHit.FaceIndex = Shape->GetGeometry()->FindMostOpposingFace(LocalPosition, LocalUnitDir, OutBestHit.FaceIndex, 1);

								OutBestHit.Actor = Interface.GetParticle(IterObject);
							}
						}
						return false;
					}
				);

				// Previously, when a zero length sweep was performed, normal was scaled by the penetration. 
				// This preserves that old behavior until we determine if this is needed.
				if (bHit && !bSweepHasLength)
				{
					OutBestHit.WorldNormal = OutBestHit.WorldNormal * -OutBestHit.Distance;
				}
			}
			return bHit;
		}


	private:
		FReadPhysicsObjectInterface<Id>& Interface;

		struct FShapeOverlapData
		{
			TThreadShapeInstance<Id>* Shape;
			FAABB3 BoundingBox;
		};

		/**
		 * For every pair of shapes that overlap, allows the caller to perform some computation. If additional pairs of shapes need to be examined, the input TFunction should return true.
		 */
		CHAOS_API bool PairwiseShapeOverlapHelper(const FConstPhysicsObjectHandle ObjectA, const FTransform& InTransformA, const FConstPhysicsObjectHandle ObjectB, const FTransform& InTransformB, bool bTraceComplex, bool bComputeMTD, const FVector& Tolerance, const TFunction<bool(const FShapeOverlapData&, const FShapeOverlapData&, const FMTDInfo&)>& Lambda);
	};

	using FPhysicsObjectCollisionInterface_External = FPhysicsObjectCollisionInterface<EThreadContext::External>;
	using FPhysicsObjectCollisionInterface_Internal = FPhysicsObjectCollisionInterface<EThreadContext::Internal>;
}
