// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ClusterCreationParameters.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PhysicsObject.h"
#include "Chaos/ShapeInstanceFwd.h"
#include "Chaos/ISpatialAcceleration.h"
#include "Framework/Threading.h"
#include "Containers/ArrayView.h"
#include "Math/MathFwd.h"
#include "UObject/ObjectMacros.h"

#include "PhysicsObjectInterface.generated.h"

class FChaosScene;
class FChaosUserDefinedEntity;
class IPhysicsProxyBase;

USTRUCT(BlueprintType)
struct FClosestPhysicsObjectResult
{
	GENERATED_BODY()

	Chaos::FPhysicsObjectHandle PhysicsObject = nullptr;
	FVector ClosestLocation;
	double ClosestDistance = 0.0;
	operator bool() const
	{
		return IsValid();
	}
	bool IsValid() const
	{
		return PhysicsObject != nullptr;
	}

	CHAOS_API FName HitName() const;
};

enum ERadialImpulseFalloff : int;

namespace Chaos
{
	class FPBDRigidsSolver;
	class FPerShapeData;
	struct FMTDInfo;

	/**
	 * FReadPhysicsObjectInterface will assume that these operations are safe to call (i.e. the relevant scenes have been read locked on the game thread).
	 */
	template<EThreadContext Id>
	class FReadPhysicsObjectInterface
	{
	public:
		CHAOS_API FPhysicsObjectHandle GetRootObject(const FConstPhysicsObjectHandle Object);
		CHAOS_API bool HasChildren(const FConstPhysicsObjectHandle Object);
		CHAOS_API FChaosUserDefinedEntity* GetUserDefinedEntity(const FConstPhysicsObjectHandle Object);
		CHAOS_API int32 GetClusterHierarchyLevel(const FConstPhysicsObjectHandle Object);

		CHAOS_API FTransform GetTransform(const FConstPhysicsObjectHandle Object);
		CHAOS_API FVector GetX(const FConstPhysicsObjectHandle Object);
		CHAOS_API FVector GetCoM(const FConstPhysicsObjectHandle Object);
		CHAOS_API FVector GetWorldCoM(const FConstPhysicsObjectHandle Object);
		CHAOS_API FQuat GetR(const FConstPhysicsObjectHandle Object);
		CHAOS_API FVector GetV(const FConstPhysicsObjectHandle Object);
		CHAOS_API FVector GetVAtPoint(const FConstPhysicsObjectHandle Object, const FVector& Point);
		CHAOS_API FVector GetW(const FConstPhysicsObjectHandle Object);
		CHAOS_API FSpatialAccelerationIdx GetSpatialIndex(const FConstPhysicsObjectHandle Object);

		CHAOS_API TThreadParticle<Id>* GetParticle(const FConstPhysicsObjectHandle Object);
		CHAOS_API TThreadKinematicParticle<Id>* GetKinematicParticle(const FConstPhysicsObjectHandle Object);
		CHAOS_API TThreadRigidParticle<Id>* GetRigidParticle(const FConstPhysicsObjectHandle Object);
		CHAOS_API TArray<TThreadParticle<Id>*> GetAllParticles(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		CHAOS_API TArray<TThreadRigidParticle<Id>*> GetAllRigidParticles(TArrayView<const FConstPhysicsObjectHandle> InObjects, bool bIncludeNulls = false);

		UE_DEPRECATED(5.3, "GetAllShapes has been deprecated. Please use GetAllThreadShapes instead.")
		CHAOS_API TArray<FPerShapeData*> GetAllShapes(TArrayView<const FConstPhysicsObjectHandle> InObjects);

		CHAOS_API TArray<TThreadShapeInstance<Id>*> GetAllThreadShapes(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		CHAOS_API FImplicitObjectRef GetGeometry(const FConstPhysicsObjectHandle Handle);

		// Returns true if a shape is found and we can stop iterating.
		CHAOS_API void VisitEveryShape(TArrayView<const FConstPhysicsObjectHandle> InObjects, TFunctionRef<bool(const FConstPhysicsObjectHandle, TThreadShapeInstance<Id>*)> Lambda);

		CHAOS_API bool AreAllValid(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		CHAOS_API bool AreAllKinematic(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		CHAOS_API bool AreAllSleeping(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		CHAOS_API bool AreAllRigidBody(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		CHAOS_API bool AreAllDynamic(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		CHAOS_API bool AreAllDynamicOrSleeping(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		CHAOS_API bool AreAllDisabled(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		CHAOS_API bool AreAllShapesQueryEnabled(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		CHAOS_API float GetMass(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		CHAOS_API FBox GetBounds(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		CHAOS_API FBox GetWorldBounds(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		CHAOS_API FClosestPhysicsObjectResult GetClosestPhysicsBodyFromLocation(TArrayView<const FConstPhysicsObjectHandle> InObjects, const FVector& WorldLocation);
		CHAOS_API FAccelerationStructureHandle CreateAccelerationStructureHandle(const FConstPhysicsObjectHandle Handle);

		friend class FPhysicsObjectInterface;
	protected:
		FReadPhysicsObjectInterface() = default;
	};

	using FReadPhysicsObjectInterface_External = FReadPhysicsObjectInterface<EThreadContext::External>;
	using FReadPhysicsObjectInterface_Internal = FReadPhysicsObjectInterface<EThreadContext::Internal>;

	/**
	 * FReadPhysicsObjectInterface will assume that these operations are safe to call (i.e. the relevant scenes have been read locked on the physics thread).
	 */
	template<EThreadContext Id>
	class FWritePhysicsObjectInterface: public FReadPhysicsObjectInterface<Id>
	{
	public:
		CHAOS_API void SetUserDefinedEntity(TArrayView<const FPhysicsObjectHandle> InObjects, FChaosUserDefinedEntity* UserDefinedEntity); // Set the user defined entity, use nullptr to remove the Entity and release the memory
		CHAOS_API void PutToSleep(TArrayView<const FPhysicsObjectHandle> InObjects);
		CHAOS_API void WakeUp(TArrayView<const FPhysicsObjectHandle> InObjects);
		CHAOS_API void ForceKinematic(TArrayView<const FPhysicsObjectHandle> InObjects);
		CHAOS_API void AddForce(TArrayView<const FPhysicsObjectHandle> InObjects, const FVector& Force, bool bInvalidate);
		UE_DEPRECATED(5.5, "This version of AddTorque has been deprecated. Please use the version with @param bAccelChange")
		CHAOS_API void AddTorque(TArrayView<const FPhysicsObjectHandle> InObjects, const FVector& Torque, bool bInvalidate);
		CHAOS_API void AddTorque(TArrayView<const FPhysicsObjectHandle> InObjects, const FVector& Torque, bool bInvalidate, bool bAccelChange = false);
		CHAOS_API void SetLinearImpulseVelocity(TArrayView<const FPhysicsObjectHandle> InObjects, const FVector& Impulse, bool bVelChange);
		CHAOS_API void SetLinearVelocity(TArrayView<const FPhysicsObjectHandle> InObjects, const FVector& InVelocity, bool bAddToCurrent = false);
		CHAOS_API void SetAngularVelocityInRadians(TArrayView<const FPhysicsObjectHandle> InObjects, const FVector& InAngularVelocity, bool bAddToCurrent = false);

		UE_DEPRECATED(5.4, "This version of AddRadialImpulse has been deprecated. Please use the version where the strain value is passed explicitly")
		CHAOS_API void AddRadialImpulse(TArrayView<const FPhysicsObjectHandle> InObjects, FVector Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff, bool bApplyStrain, bool bInvalidate, bool bVelChange = false);

		CHAOS_API void AddRadialImpulse(TArrayView<const FPhysicsObjectHandle> InObjects, FVector Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff, bool bApplyStrain, float Strain, bool bInvalidate, bool bVelChange = false, float MinValue = 0.f, float MaxValue = 1.f);
		CHAOS_API void SetLinearEtherDrag(TArrayView<const FPhysicsObjectHandle> InObjects, float InLinearDrag);
		CHAOS_API void SetAngularEtherDrag(TArrayView<const FPhysicsObjectHandle> InObjects, float InAngularDrag);

		CHAOS_API void UpdateShapeCollisionFlags(TArrayView<const FPhysicsObjectHandle> InObjects, bool bSimCollision, bool bQueryCollision);
		UE_DEPRECATED(5.7, "Use UpdateCombinedShapeFilterData instead")
		CHAOS_API void UpdateShapeFilterData(TArrayView<const FPhysicsObjectHandle> InObjects, const FCollisionFilterData& QueryData, const FCollisionFilterData& SimData);
		CHAOS_API void UpdateCombinedShapeFilterData(TArrayView<const FPhysicsObjectHandle> InObjects, const Chaos::Filter::FCombinedShapeFilterData& CombinedShapeFilter);

		template<typename TPayloadType, typename T, int d>
		void AddToSpatialAcceleration(TArrayView<const FPhysicsObjectHandle> InObjects, ISpatialAcceleration<TPayloadType, T, d>* SpatialAcceleration)
		{
			if (!SpatialAcceleration)
			{
				return;
			}

			for (const FConstPhysicsObjectHandle Handle : InObjects)
			{
				const FBox WorldBounds = this->GetWorldBounds({ &Handle, 1 });
				const FAABB3 ChaosWorldBounds{ WorldBounds.Min, WorldBounds.Max };
				FAccelerationStructureHandle AccelerationHandle = this->CreateAccelerationStructureHandle(Handle);
				SpatialAcceleration->UpdateElementIn(AccelerationHandle, ChaosWorldBounds, true, this->GetSpatialIndex(Handle));
			}
		}

		template<typename TPayloadType, typename T, int d>
		void RemoveFromSpatialAcceleration(TArrayView<const FPhysicsObjectHandle> InObjects, ISpatialAcceleration<TPayloadType, T, d>* SpatialAcceleration)
		{
			if (!SpatialAcceleration)
			{
				return;
			}

			for (const FConstPhysicsObjectHandle Handle : InObjects)
			{
				FAccelerationStructureHandle AccelerationHandle = this->CreateAccelerationStructureHandle(Handle);
				SpatialAcceleration->RemoveElementFrom(AccelerationHandle, this->GetSpatialIndex(Handle));
			}
		}

		CHAOS_API void AddConnectivityEdgesBetween(TArrayView<const FPhysicsObjectHandle> FromObjects, TArrayView<const FPhysicsObjectHandle> ToObjects, const FClusterCreationParameters& Parameters);

		friend class FPhysicsObjectInterface;
	protected:
		FWritePhysicsObjectInterface() = default;
	};


	using FWritePhysicsObjectInterface_External = FWritePhysicsObjectInterface<EThreadContext::External>;
	using FWritePhysicsObjectInterface_Internal = FWritePhysicsObjectInterface<EThreadContext::Internal>;

	/**
	 * The FPhysicsObjectInterface is primarily used to perform maintenance operations on the FPhysicsObject.
	 * Any operations on the underlying particle/particle handle should use the FReadPhysicsObjectInterface and
	 * FWritePhysicsObjectInterface.
	 */
	class FPhysicsObjectInterface
	{
	public:
		static CHAOS_API void SetName(const FPhysicsObjectHandle Object, const FName& InName);
		static CHAOS_API FName GetName(const FConstPhysicsObjectHandle Object);

#if CHAOS_DEBUG_NAME
		/** Returns the debug name of the underlying particle */
		static CHAOS_API const FString& GetDebugName(const FConstPhysicsObjectHandle Object);
#endif
		static CHAOS_API void SetId(const FPhysicsObjectHandle Object, int32 InId);
		static CHAOS_API int32 GetId(const FConstPhysicsObjectHandle Object);

		static CHAOS_API FPBDRigidsSolver* GetSolver(TArrayView<const FConstPhysicsObjectHandle> InObjects);
		static CHAOS_API FPBDRigidsSolver* GetSolver(const FConstPhysicsObjectHandle InObject);
		static CHAOS_API IPhysicsProxyBase* GetProxy(TArrayView<const FConstPhysicsObjectHandle> InObjects);

	protected:
		// This function should not be called without an appropriate read-lock on the relevant scene.
		template<EThreadContext Id>
		static FReadPhysicsObjectInterface<Id> CreateReadInterface() { return FReadPhysicsObjectInterface<Id>{}; }

		// This function should not be called without an appropriate write-lock on the relevant scene.
		template<EThreadContext Id>
		static FWritePhysicsObjectInterface<Id> CreateWriteInterface() { return FWritePhysicsObjectInterface<Id>{}; }
	};
}
