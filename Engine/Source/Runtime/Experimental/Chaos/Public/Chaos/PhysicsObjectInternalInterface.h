// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/PhysicsObjectInterface.h"

namespace Chaos
{
	class FPhysicsObjectInternalInterface : public Chaos::FPhysicsObjectInterface
	{
	public:
		static CHAOS_API FReadPhysicsObjectInterface_Internal GetRead();
		static CHAOS_API FWritePhysicsObjectInterface_Internal GetWrite();
	};
}
