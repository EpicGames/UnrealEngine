// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Templates/SharedPointer.h"

namespace Chaos
{
	// Handle to a IPhysicsProxyBase, that cross references its creation timestamp to make sure the original proxy has not been deleted
	// and we are truly dealing with the one used to make the handle initially (the pointer could have been reused)
	struct FPhysicsProxyHandle
	{
	public:
		FPhysicsProxyHandle() = default;
		FPhysicsProxyHandle(const FPhysicsProxyHandle&) = default;
		FPhysicsProxyHandle& operator=(const FPhysicsProxyHandle& Other) = default;
		FPhysicsProxyHandle(FPhysicsProxyHandle&&) = default;
		FPhysicsProxyHandle& operator=(FPhysicsProxyHandle&& Other) = default;
		~FPhysicsProxyHandle() = default;

		FPhysicsProxyHandle(IPhysicsProxyBase* InProxy)
		{
			Set(InProxy);
		}
		[[nodiscard]] FORCEINLINE bool operator==(const FPhysicsProxyHandle& RHS) const
		{
			return Proxy == RHS.Proxy && Timestamp == RHS.Timestamp;
		}
		[[nodiscard]] FORCEINLINE bool operator!=(const FPhysicsProxyHandle& RHS) const
		{
			return !(*this == RHS);
		}

		/** Sets this handle to point to InProxy, or resets this handle if InProxy is null. */
		FORCEINLINE void Set(IPhysicsProxyBase* InProxy)
		{
			if (InProxy != nullptr)
			{
				Proxy = InProxy;
				Timestamp = InProxy->GetSyncTimestamp();
			}
			else
			{
				Reset();
			}
		}

		/** Sets this handle to point to the physics proxy corresponding to a particle handle */
		CHAOS_API void Set(FGeometryParticleHandle& ParticleHandle);
		
		/** Resets this handle, which will now be invalid */
		FORCEINLINE void Reset()
		{
			Proxy = nullptr;
			Timestamp.Reset();
		}

		/** Returns true if this handle points to the same non deleted non null physics proxy as that used when setting or initializing the handle */
		[[nodiscard]] FORCEINLINE bool IsValid() const
		{
			// Make sure the timestamp is still valid, the proxy also, and it is the proxy corresponding to the original one used at Init time (its current timestamp should match Timestamp we established in Set)
			return (Timestamp.IsValid() && !Timestamp->bDeleted && Proxy && Proxy->GetSyncTimestamp() == Timestamp);
		}

		/** Returns the underlying physics proxy pointer, or null if this handle is invalid (see IsValid()) */
		[[nodiscard]] FORCEINLINE IPhysicsProxyBase* Get() const
		{
			return IsValid() ? Proxy : nullptr;
		}

		/** Returns the underlying IPhysicsProxyBase pointer, without performing any validity checks. Use caution. */
		[[nodiscard]] FORCEINLINE IPhysicsProxyBase* GetUnsafe() const
		{
			return Proxy;
		}

		/** Returns true if the handle was set to a proxy but that proxy was deleted since then */
		[[nodiscard]] FORCEINLINE bool IsStale() const
		{
			return (Proxy != nullptr) && (Timestamp.IsValid() && Timestamp->bDeleted);
		}

	private:
		IPhysicsProxyBase* Proxy = nullptr;
		TSharedPtr<FProxyTimestampBase, ESPMode::ThreadSafe> Timestamp;
	};
} // namespace Chaos

