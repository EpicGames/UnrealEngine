// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/GCObject.h"

#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Chaos/PullPhysicsDataImp.h"
#include "PhysicsProxy/CharacterGroundConstraintProxy.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "PhysicsProxy/JointConstraintProxy.h"
#include "Chaos/Framework/ChaosResultsManager.h"

#include <type_traits>

namespace Chaos
{
	namespace Private
	{
		// This lets us add support for performing per-proxy operations in PullPhysicsStateForEachDirtyProxy_External
		// without the need to keep adding function parameters to keep taking in lambdas. We assume that the incoming
		// user-specified TDispatcher has various operator() overridden to take in a specific proxy pointer. We require
		// the TPullPhysicsStateDispatchHelper then to wrap the TDispatcher so that we can check if the incoming TDispatcher
		// has a override for the proxy we want to pass to it. If it does not, we do nothing. This lets us maintain backward
		// compatability as we keep adding in new proxy types while not requiring any additional computation at runtime since
		// this dispatch is all done at compile time.
		template<typename TDispatcher>
		struct TPullPhysicsStateDispatchHelper
		{
			template<typename TProxy>
			static void Apply(TDispatcher& Dispatcher, TProxy* Proxy)
			{
				if constexpr (std::is_invocable_v<TDispatcher, TProxy*>)
				{
					Dispatcher(Proxy);
				}
			}
		};
	}

	template<typename TDispatcher>
	void FPhysicsSolverBase::PullPhysicsStateForEachDirtyProxy_External(TDispatcher& Dispatcher)
	{
		using namespace Chaos;

		FPullPhysicsData* LatestData = nullptr;
		if (IsUsingAsyncResults() && UseAsyncInterpolation)
		{
			SCOPE_CYCLE_COUNTER(STAT_AsyncInterpolateResults);

			const FReal ResultsTime = GetPhysicsResultsTime_External();
			//we want to interpolate between prev and next. There are a few cases to consider:
			//case 1: dirty data exists in both prev and next. In this case continuous data is interpolated, state data is a step function from prev to next
			//case 2: prev has dirty data and next doesn't. in this case take prev as it means nothing to interpolate, just a constant value
			//case 3: prev has dirty data and next has overwritten data. In this case we do nothing as the overwritten data wins (came from GT), and also particle may be deleted
			//case 4: prev has no dirty data and next does. In this case interpolate from gt data to next
			//case 5: prev has no dirty data and next was overwritten. In this case do nothing as the overwritten data wins, and also particle may be deleted

			TArray<const FChaosInterpolationResults*> ResultsPerChannel;
			
			{
				SCOPE_CYCLE_COUNTER(STAT_AsyncPullResults);
				ResultsPerChannel = PullResultsManager->PullAsyncPhysicsResults_External(ResultsTime);
			}

			for (int32 ChannelIdx = 0; ChannelIdx < ResultsPerChannel.Num(); ++ChannelIdx)
			{
				const FChaosInterpolationResults& Results = *ResultsPerChannel[ChannelIdx];
				LatestData = Results.Next;
				//todo: go wide
				const int32 SolverTimestamp = Results.Next ? Results.Next->SolverTimestamp : INDEX_NONE;

				// single particles
				{
					SCOPE_CYCLE_COUNTER(STAT_ProcessSingleProxy);
					for (const FChaosRigidInterpolationData& RigidInterp : Results.RigidInterpolations)
					{
						if (FSingleParticlePhysicsProxy* Proxy = RigidInterp.Prev.GetProxy())
						{
							const FDirtyRigidParticleReplicationErrorData* ErrorData = LatestData->DirtyRigidErrors.Find(Proxy);
							if (Proxy->PullFromPhysicsState(RigidInterp.Prev, SolverTimestamp, &RigidInterp.Next, &Results.Alpha, ErrorData, GetAsyncDeltaTime()))
							{
								Private::TPullPhysicsStateDispatchHelper<TDispatcher>::Apply(Dispatcher, Proxy);
							}
							LatestData->DirtyRigidErrors.Remove(Proxy);
						}
					}
				}

				// geometry collections
				{
					SCOPE_CYCLE_COUNTER(STAT_ProcessGCProxy);
					for (const FChaosGeometryCollectionInterpolationData& GCInterp : Results.GeometryCollectionInterpolations)
					{
						if (FGeometryCollectionPhysicsProxy* Proxy = GCInterp.Prev.GetProxy())
						{
							const FDirtyRigidParticleReplicationErrorData* ErrorData = LatestData->DirtyRigidErrors.Find(Proxy);
							if (Proxy->PullFromPhysicsState(GCInterp.Prev, SolverTimestamp, &GCInterp.Next, &Results.Alpha, ErrorData, GetAsyncDeltaTime()))
							{
								Private::TPullPhysicsStateDispatchHelper<TDispatcher>::Apply(Dispatcher, Proxy);
							}
							LatestData->DirtyRigidErrors.Remove(Proxy);
						}
					}
				}

				// cluster unions
				{
					SCOPE_CYCLE_COUNTER(STAT_ProcessClusterUnionProxy)
					for(const FChaosClusterUnionInterpolationData& ClusterInterp : Results.ClusterUnionInterpolations)
					{
						if(FClusterUnionPhysicsProxy* Proxy = ClusterInterp.Prev.GetProxy())
						{
							const FDirtyRigidParticleReplicationErrorData* ErrorData = LatestData->DirtyRigidErrors.Find(Proxy);
							if (Proxy->PullFromPhysicsState(ClusterInterp.Prev, SolverTimestamp, &ClusterInterp.Next, &Results.Alpha, ErrorData, GetAsyncDeltaTime()))
							{
								Private::TPullPhysicsStateDispatchHelper<TDispatcher>::Apply(Dispatcher, Proxy);
							}
							LatestData->DirtyRigidErrors.Remove(Proxy);
						}
					}
				}
			}
		}
		else
		{
			SCOPE_CYCLE_COUNTER(STAT_SyncPullResults);

			//no interpolation so just use latest, in non-substepping modes this will just be the next result
			// available in the queue - however if we substepped externally we need to consume the whole
			// queue by telling the sync pull that we expect multiple results.

			const FChaosInterpolationResults& Results = PullResultsManager->PullSyncPhysicsResults_External();
			LatestData = Results.Next;
			//todo: go wide
			const int32 SolverTimestamp = Results.Next ? Results.Next->SolverTimestamp : INDEX_NONE;

			// Single particles
			{
				SCOPE_CYCLE_COUNTER(STAT_ProcessSingleProxy);
				for(const FChaosRigidInterpolationData& RigidInterp : Results.RigidInterpolations)
				{
					if(FSingleParticlePhysicsProxy* Proxy = RigidInterp.Prev.GetProxy())
					{
						if(Proxy->PullFromPhysicsState(RigidInterp.Next, SolverTimestamp))
						{
							Private::TPullPhysicsStateDispatchHelper<TDispatcher>::Apply(Dispatcher, Proxy);
						}
					}
				}
			}

			// geometry collections
			{
				SCOPE_CYCLE_COUNTER(STAT_ProcessGCProxy);
				for(const FChaosGeometryCollectionInterpolationData& GCInterp : Results.GeometryCollectionInterpolations)
				{
					if(FGeometryCollectionPhysicsProxy* Proxy = GCInterp.Prev.GetProxy())
					{
						if(Proxy->PullFromPhysicsState(GCInterp.Next, SolverTimestamp))
						{
							Private::TPullPhysicsStateDispatchHelper<TDispatcher>::Apply(Dispatcher, Proxy);
						}
					}
				}
			}

			// cluster unions
			{
				SCOPE_CYCLE_COUNTER(STAT_ProcessClusterUnionProxy)
				for(const FChaosClusterUnionInterpolationData& ClusterInterp : Results.ClusterUnionInterpolations)
				{
					if(FClusterUnionPhysicsProxy* Proxy = ClusterInterp.Prev.GetProxy())
					{
						if(Proxy->PullFromPhysicsState(ClusterInterp.Prev, SolverTimestamp))
						{
							Private::TPullPhysicsStateDispatchHelper<TDispatcher>::Apply(Dispatcher, Proxy);
						}
					}
				}
			}
		}

		//no interpolation for GC or joints at the moment
		if (LatestData)
		{
			SCOPE_CYCLE_COUNTER(STAT_PullConstraints);
			const int32 SyncTimestamp = LatestData->SolverTimestamp;

			//
			// @todo(chaos) : Add Dirty Constraints Support
			//
			// This is temporary constraint code until the DirtyParticleBuffer
			// can be updated to support constraints. In summary : The 
			// FDirtyPropertiesManager is going to be updated to support a 
			// FDirtySet that is specific to a TConstraintProperties class.
			//
			for (const FDirtyJointConstraintData& DirtyData : LatestData->DirtyJointConstraints)
			{
				if (auto Proxy = DirtyData.GetProxy())
				{
					if (Proxy->PullFromPhysicsState(DirtyData, SyncTimestamp))
					{
						Private::TPullPhysicsStateDispatchHelper<TDispatcher>::Apply(Dispatcher, Proxy);
					}

				}
			}

			//latest data may be used multiple times during interpolation, so for non interpolated joints we clear it
			LatestData->DirtyJointConstraints.Reset();

			for (const FDirtyCharacterGroundConstraintData& DirtyData : LatestData->DirtyCharacterGroundConstraints)
			{
				if (auto Proxy = DirtyData.GetProxy())
				{
					// todo: Do we need to add a callback?
					Proxy->PullFromPhysicsState(DirtyData, SyncTimestamp);
				}
			}
			LatestData->DirtyCharacterGroundConstraints.Reset();
		}
	}

	// Pulls physics state for each dirty particle and allows caller to do additional work if needed
	template <typename RigidLambda, typename ConstraintLambda, typename GeometryCollectionLambda>
	void FPhysicsSolverBase::PullPhysicsStateForEachDirtyProxy_External(const RigidLambda& RigidFunc, const ConstraintLambda& ConstraintFunc, const GeometryCollectionLambda& GeometryCollectionFunc)
	{
		struct FDispatcher
		{
			void operator()(FSingleParticlePhysicsProxy* Proxy)
			{
				RigidFunc(Proxy);
			}

			void operator()(FJointConstraintPhysicsProxy* Proxy)
			{
				ConstraintFunc(Proxy);
			}

			void operator()(FGeometryCollectionPhysicsProxy* Proxy)
			{
				GeometryCollectionFunc(Proxy);
			}
		} Dispatcher;
		PullPhysicsStateForEachDirtyProxy_External(Dispatcher);
	}

	template <typename RigidLambda, typename ConstraintLambda>
	void FPhysicsSolverBase::PullPhysicsStateForEachDirtyProxy_External(const RigidLambda& RigidFunc, const ConstraintLambda& ConstraintFunc)
	{
		struct FDispatcher
		{
			void operator()(FSingleParticlePhysicsProxy* Proxy)
			{
				RigidFunc(Proxy);
			}

			void operator()(FJointConstraintPhysicsProxy* Proxy)
			{
				ConstraintFunc(Proxy);
			}
		} Dispatcher;
		PullPhysicsStateForEachDirtyProxy_External(Dispatcher);
	}
}
