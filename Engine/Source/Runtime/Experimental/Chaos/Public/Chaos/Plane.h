// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/CorePlane.h"
#include "Chaos/ImplicitObject.h"
#include "ChaosArchive.h"
#include "ChaosCheck.h"

namespace Chaos
{

template<class T, int d>
class TPlane final : public FImplicitObject
{
  public:
	using FImplicitObject::GetTypeName;


	TPlane() : FImplicitObject(0, ImplicitObjectType::Plane) {}	//needed for serialization
	TPlane(const TVector<T, d>& InX, const TVector<T, d>& InNormal)
	    : FImplicitObject(0, ImplicitObjectType::Plane)
		, MPlaneConcrete(InX, InNormal)
	{
	}
	TPlane(const TPlane<T, d>& Other)
	    : FImplicitObject(0, ImplicitObjectType::Plane)
	    , MPlaneConcrete(Other.MPlaneConcrete)
	{
	}
	TPlane(TPlane<T, d>&& Other)
	    : FImplicitObject(0, ImplicitObjectType::Plane)
	    , MPlaneConcrete(MoveTemp(Other.MPlaneConcrete))
	{
	}
	virtual ~TPlane() = default;

	static constexpr EImplicitObjectType StaticType()
	{
		return ImplicitObjectType::Plane;
	}

	virtual FReal GetRadius() const override
	{
		return 0.0f;
	}

	/**
	 * Phi is positive on the side of the normal, and negative otherwise.
	 */
	T SignedDistance(const TVector<T, d>& x) const
	{
		return MPlaneConcrete.SignedDistance(x);
	}

	/**
	 * Phi is positive on the side of the normal, and negative otherwise.
	 */
	virtual FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const override
	{
		return MPlaneConcrete.PhiWithNormal(x,Normal);
	}

	TVector<T, d> FindClosestPoint(const TVector<T, d>& x, const T Thickness = (T)0) const
	{
		return MPlaneConcrete.FindClosestPoint(x,Thickness);
	}

	virtual bool Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const override
	{
		return MPlaneConcrete.Raycast(StartPoint,Dir,Length,Thickness,OutTime,OutPosition,OutNormal,OutFaceIndex);
	}

	virtual Pair<FVec3, bool> FindClosestIntersectionImp(const FVec3& StartPoint, const FVec3& EndPoint, const FReal Thickness) const override
 	{
		return MPlaneConcrete.FindClosestIntersection(StartPoint,EndPoint,Thickness);
	}

	const TVector<T,d>& X() const { return MPlaneConcrete.X(); }
	const TVector<T,d>& Normal() const { return MPlaneConcrete.Normal(); }
	const TVector<T, d>& Normal(const TVector<T, d>&) const { return MPlaneConcrete.Normal(); }
	
	FORCEINLINE void SerializeImp(FArchive& Ar)
	{
		FImplicitObject::SerializeImp(Ar);
		MPlaneConcrete.Serialize(Ar);
	}

	virtual void Serialize(FChaosArchive& Ar) override
	{
		FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName());
		SerializeImp(Ar);
	}

	virtual void Serialize(FArchive& Ar) override
	{
		SerializeImp(Ar);
	}

	virtual uint32 GetTypeHash() const override
	{
		return MPlaneConcrete.GetTypeHash();
	}

	const TPlaneConcrete<T>& PlaneConcrete() const { return MPlaneConcrete; }

  private:
	  TPlaneConcrete<T> MPlaneConcrete;
};

template<typename T, int d>
TVector<T, 2> ComputeBarycentricInPlane(const TVector<T, d>& P0, const TVector<T, d>& P1, const TVector<T, d>& P2, const TVector<T, d>& P)
{
	TVector<T, 2> Bary;
	TVector<T, d> P10 = P1 - P0;
	TVector<T, d> P20 = P2 - P0;
	TVector<T, d> PP0 = P - P0;
	T Size10 = P10.SizeSquared();
	T Size20 = P20.SizeSquared();
	T ProjSides = TVector<T, d>::DotProduct(P10, P20);
	T ProjP1 = TVector<T, d>::DotProduct(PP0, P10);
	T ProjP2 = TVector<T, d>::DotProduct(PP0, P20);
	T Denom = Size10 * Size20 - ProjSides * ProjSides;
	using FVec2Real = decltype(Bary.X);
	Bary.X = FVec2Real((Size20 * ProjP1 - ProjSides * ProjP2) / Denom);
	Bary.Y = FVec2Real((Size10 * ProjP2 - ProjSides * ProjP1) / Denom);
	return Bary;
}

template<typename T, int d>
const TVector<T, d> FindClosestPointAndAlphaOnLineSegment(const TVector<T, d>& P0, const TVector<T, d>& P1, const TVector<T, d>& P, T& OutAlpha)
{
	const TVector<T, d> P10 = P1 - P0;
	const TVector<T, d> PP0 = P - P0;
	const T Proj = TVector<T, d>::DotProduct(P10, PP0);
	if (Proj < (T)0) //first check we're not behind
	{
		OutAlpha = (T)0;
		return P0;
	}

	const T Denom2 = P10.SizeSquared();
	if (Denom2 < (T)1e-4)
	{
		OutAlpha = (T)0;
		return P0;
	}

	//do proper projection
	const T NormalProj = Proj / Denom2;
	if (NormalProj > (T)1) //too far forward
	{
		OutAlpha = (T)1;
		return P1;
	}

	OutAlpha = NormalProj;
	return P0 + NormalProj * P10; //somewhere on the line
}

template<typename T, int d>
const TVector<T, d> FindClosestPointOnLineSegment(const TVector<T, d>& P0, const TVector<T, d>& P1, const TVector<T, d>& P)
{
	T AlphaUnused;
	return FindClosestPointAndAlphaOnLineSegment(P0, P1, P, AlphaUnused);
}

template<typename T, int d>
TVector<T, d> FindClosestPointOnTriangle(const TVector<T, d>& ClosestPointOnPlane, const TVector<T, d>& P0, const TVector<T, d>& P1, const TVector<T, d>& P2, const TVector<T, d>& P)
{
	const T Epsilon = 1e-4f;

	const TVector<T, 2> Bary = ComputeBarycentricInPlane(P0, P1, P2, ClosestPointOnPlane);

	if (Bary[0] >= -Epsilon && Bary[0] <= 1 + Epsilon && Bary[1] >= -Epsilon && Bary[1] <= 1 + Epsilon && (Bary[0] + Bary[1]) <= (1 + Epsilon))
	{
		return ClosestPointOnPlane;
	}

	const TVector<T, d> P10Closest = FindClosestPointOnLineSegment(P0, P1, P);
	const TVector<T, d> P20Closest = FindClosestPointOnLineSegment(P0, P2, P);
	const TVector<T, d> P21Closest = FindClosestPointOnLineSegment(P1, P2, P);

	const T P10Dist2 = (P - P10Closest).SizeSquared();
	const T P20Dist2 = (P - P20Closest).SizeSquared();
	const T P21Dist2 = (P - P21Closest).SizeSquared();

	if (P10Dist2 < P20Dist2)
	{
		if (P10Dist2 < P21Dist2)
		{
			return P10Closest;
		}
		else
		{
			return P21Closest;
		}
	}
	else
	{
		if (P20Dist2 < P21Dist2)
		{
			return P20Closest;
		}
		else
		{
			return P21Closest;
		}
	}
}

template<typename T, int d>
TVector<T, d> FindClosestPointOnTriangle(const TPlane<T, d>& TrianglePlane, const TVector<T, d>& P0, const TVector<T, d>& P1, const TVector<T, d>& P2, const TVector<T, d>& P)
{
	const TVector<T, d> PointOnPlane = TrianglePlane.FindClosestPoint(P);
	return FindClosestPointOnTriangle(PointOnPlane, P0, P1, P2, P);
}

// This method follows FindClosestPointOnTriangle but does less duplicate work and can handle degenerate triangles. It also returns the barycentric coordinates for the returned point.
template<typename T, int d>
TVector<T, d> FindClosestPointAndBaryOnTriangle(const TVector<T, d>& P0, const TVector<T, d>& P1, const TVector<T, d>& P2, const TVector<T, d>& P, TVector<T, 3>& Bary)
{
	const TVector<T, d> P10 = P1 - P0;
	const TVector<T, d> P20 = P2 - P0;
	const TVector<T, d> P21 = P2 - P1;
	const TVector<T, d> PP0 = P - P0;
	
	const T Size10 = P10.SizeSquared();
	const T Size20 = P20.SizeSquared();
	const T Size21 = P21.SizeSquared();

	const T ProjP1 = PP0.Dot(P10);
	const T ProjP2 = PP0.Dot(P20);


	auto ProjectToP01 = [&Bary, &P0, &P1, Size10, ProjP1]() -> TVector<T,d>
	{
		Bary.Z = 0.f;
		Bary.Y = FMath::Clamp(ProjP1 / Size10, (T)0., (T)1.);
		Bary.X = 1.f - Bary.Y;
		const TVector<T,d> Result = Bary.X * P0 + Bary.Y * P1;
		return Result;
	};

	auto ProjectToP02 = [&Bary, &P0, &P2, Size20, ProjP2]() -> TVector<T, d>
	{
		Bary.Y = 0.f;
		Bary.Z = FMath::Clamp(ProjP2 / Size20, (T)0., (T)1.);
		Bary.X = 1.f - Bary.Z;
		const TVector<T, d> Result = Bary.X * P0 + Bary.Z * P2;
		return Result;
	};

	auto ProjectToP12 = [&Bary, &P1, &P2, &P]() -> TVector<T, d>
	{
		const TVector<T,d> P2P1 = P2 - P1;

		Bary.X = 0.f;
		Bary.Z = FMath::Clamp(P2P1.Dot(P - P1) / P2P1.SizeSquared(), (T)0., (T)1.);
		Bary.Y = 1.f - Bary.Z;
		const TVector<T, d> Result = Bary.Y * P1 + Bary.Z * P2;
		return Result;
	};


	// Degenerate triangles
	if (Size10 < (T)UE_DOUBLE_SMALL_NUMBER)
	{
		if (Size20 < (T)UE_DOUBLE_SMALL_NUMBER)
		{
			// Triangle is (nearly) a single point.
			Bary.X = (T)1.;
			Bary.Y = Bary.Z = (T)0.;
			return P0;
		}

		// Triangle is a line segment from P0(=P1) to P2. Project to that line segment.
		return ProjectToP02();
	}
	if (Size20 < (T)UE_DOUBLE_SMALL_NUMBER)
	{
		// Triangle is a line segment from P0(=P2) to P1. Project to that line segment.
		return ProjectToP01();
	}


	const T ProjSides = P10.Dot(P20);
	const T Denom = Size10 * Size20 - ProjSides * ProjSides;

	if (Denom < (T)UE_DOUBLE_SMALL_NUMBER)
	{
		// Triangle is a line segment from P0 to P1(=P2), or otherwise the 3 points are (nearly) colinear. Project to the longest edge.
		if (Size21 > Size20)
		{
			return ProjectToP12();
		}
		else if (Size20 > Size10)
		{
			return ProjectToP02();
		}
		else
		{
			return ProjectToP01();
		}
	}

	// Non-degenerate triangle

	// Numerators of barycentric coordinates if P is inside triangle
	const T BaryYNum = (Size20 * ProjP1 - ProjSides * ProjP2);
	const T BaryZNum = (Size10 * ProjP2 - ProjSides * ProjP1);

	// Using this specific epsilon to have parity with FindClosestPointOnTriangle
	constexpr T Epsilon = (T)1e-4;
	const T EpsilonScaledByDenom = Epsilon * Denom;

	if (BaryYNum + BaryZNum > Denom * (1 + Epsilon)) // i.e., Bary.X < -Epsilon.
	{
		if (BaryYNum < -EpsilonScaledByDenom)
		{
			// Bary.X < 0, Bary.Y < 0, (Bary.Z > 1)

			// We're outside the triangle in the region opposite angle P2. (if angle at P2 is > 90 degrees, closest point could be on one of these segments)
			// Should project onto line segment P02 or P12
			if (ProjP2 < Size20)
			{
				// Closer to P02 
				// Project to line segment from P02
				return ProjectToP02();
			}
		} 
		else if (BaryZNum < -EpsilonScaledByDenom)
		{
			// Bary.X < 0, Bary.Y > 1, Bary.Z < 0
			
			// We're outside the triangle in the region opposite the angle P1.
			// Should project onto line segment P01 or P12
			if (ProjP1 < Size10)
			{
				// Closer to P01.
				// Project to line segment from P01
				return ProjectToP01();
			}
		}

		// Bary.X < 0, Bary.Y >=0, Bary.Z >= 0

		// We're outside the triangle in the region closest to P12
		// Project to line segment from P12
		return ProjectToP12();
	}
	else if (BaryYNum < -EpsilonScaledByDenom)
	{
		if (BaryZNum < -EpsilonScaledByDenom)
		{
			// Bary.X > 1, Bary.Y < 0, Bary.Z < 0

			// We're outside the triangle in the region opposite P0.
			// Should project onto line segment P01 or P02 
			if (ProjP1 > 0.f)
			{
				// Closer to P01.
				// Project to line segment from P01
				return ProjectToP01();
			}
		}

		// Bary.X >= 0, Bary.Y < 0, Bary.Z >= 0
		// We're outside the triangle in region closest to P02.
		// Project to line segment from P02
		return ProjectToP02();
	}
	else if (BaryZNum < -EpsilonScaledByDenom)
	{
		// Bary.X >=0, Bary.Y >=0, Bary.Z < 0
		// We're outside the triangle in the region closest to P01
		// Project to line segment from P0 to P1
		return ProjectToP01();
	}
	
	// Bary.X >= 0, Bary.Y >= 0, Bary.Z >= 0
	// Interior of triangle!
	Bary.Y = BaryYNum / Denom;
	Bary.Z = BaryZNum / Denom;
	Bary.X = (T)1. - Bary.Y - Bary.Z;
	const TVector<T,d> Result = Bary.X * P0 + Bary.Y * P1 + Bary.Z * P2;
	return Result;
}

template<typename T, int d>
bool IntersectPlanes2(TVector<T,d>& I, TVector<T,d>& D, const TPlane<T,d>& P1, const TPlane<T,d>& P2)
{
	FVector LI = I, LD = D;
	FPlane LP1(P1.X(), P1.Normal()), LP2(P2.X(), P2.Normal());
	bool RetVal = FMath::IntersectPlanes2(LI,LD,LP1,LP2);
	I = LI; D = LD;
	return RetVal;
}

// Returns the closest distance between two edges (X0,X1) and (Y0,Y1) and their barycentric coordinates of closest point pair
template<typename T, int d>
T FindClosestPointAndBaryBetweenEdges(const TVector<T, d>& X0, const TVector<T, d>& X1, const TVector<T, d>& Y0, const TVector<T, d>& Y1, 
	TVector<T, 2>& BaryX, TVector<T, 2>& BaryY, T Eps = UE_SMALL_NUMBER)
{
	// Edge directions
	const TVector<T, d> U = X1 - X0;
	const TVector<T, d> V = Y1 - Y0;
	const TVector<T, d> W = X0 - Y0;

	const T a = U.SizeSquared();
	const T b = U.Dot(V);
	const T c = V.SizeSquared();
	const T dval = U.Dot(W);
	const T eval = V.Dot(W);

	const T D = a * c - b * b;

	T sN, sD = D;
	T tN, tD = D;

	// Initial solution
	if (D < Eps) // nearly parallel
	{
		sN = (T)0;
		sD = (T)1;
		tN = eval;
		tD = c;
	}
	else
	{
		sN = (b * eval - c * dval);
		tN = (a * eval - b * dval);
	}

	// Clamp s to [0,1]
	if (sN <= (T)0)
	{
		sN = (T)0;
		tN = eval;
		tD = c;
	}
	else if (sN >= sD)
	{
		sN = sD;
		tN = eval + b;
		tD = c;
	}

	// Clamp t to [0,1]
	if (tN <= (T)0)
	{
		tN = (T)0;
		if (-dval <= (T)0)
		{
			sN = (T)0;
			sD = (T)1;
		}
		else if (-dval >= a)
		{
			sN = sD;
		}
		else
		{
			sN = -dval;
			sD = a;
		}
	}
	else if (tN >= tD)
	{
		tN = tD;
		if ((-dval + b) <= (T)0)
		{
			sN = (T)0;
			sD = (T)1;
		}
		else if ((-dval + b) >= a)
		{
			sN = sD;
		}
		else
		{
			sN = (-dval + b);
			sD = a;
		}
	}

	// Barycentric weights
	const T s = (FMath::Abs(sN) < Eps) ? (T)0 : sN / sD;
	const T t = (FMath::Abs(tN) < Eps) ? (T)0 : tN / tD;
	BaryX[0] = (T)1 - s;
	BaryX[1] = s;
	BaryY[0] = (T)1 - t;
	BaryY[1] = t;

	// Closest points
	const TVector<T, d> ClosestX = X0 + U * s;
	const TVector<T, d> ClosestY = Y0 + V * t;

	return (ClosestX - ClosestY).Size();
}

}
