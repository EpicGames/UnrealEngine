// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayFaceND.h"
#include "Chaos/ArrayND.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/UniformGrid.h"

namespace Chaos
{
template<class T, int d>
class TPressureProjection
{
  public:
	TPressureProjection() {}
	~TPressureProjection() = default;

	void Apply(const TUniformGrid<T, d>& Grid, TArrayFaceND<T, d>& Velocity, const TArrayND<bool, d>& Dirichlet, const TArrayFaceND<bool, d>& Neumann, const T dt)
	{
		TArrayND<T, d> Pressure(Grid), Divergence(Grid);
		// compute divergence
		PhysicsParallelFor(Grid.GetNumCells(), [&](int32 Index) {
			const TVector<int32, d> CellIndex = Grid.GetIndex(Index);
			Divergence(CellIndex) = 0;
			for (int32 Axis = 0; Axis < d; ++Axis)
			{
				Divergence(CellIndex) += (Velocity(MakePair(Axis, CellIndex + TVector<int32, d>::AxisVector(Axis))) - Velocity(MakePair(Axis, CellIndex))) / Grid.Dx()[Axis];
			}
		});
		check(false);
		MPressureRule(Grid, Pressure, Divergence, Dirichlet, Neumann, dt);
		PhysicsParallelFor(Grid.GetNumFaces(), [&](int32 Index) {
			const auto FaceIndex = Grid.GetFaceIndex(Index);
			const auto Axis = FaceIndex.First;
			const auto PrevIndex = FaceIndex.Second - TVector<int32, d>::AxisVector(Axis);
			const auto NextIndex = FaceIndex.Second;
			// Currently we assume that if no boundary is set on the boarder it must be dirichlet
			T PrevPressure = 0;
			T NextPressure = 0;
			if (NextIndex[Axis] < Grid.Counts()[Axis])
			{
				NextPressure = Pressure(NextIndex);
			}
			if (PrevIndex[Axis] >= 0)
			{
				PrevPressure = Pressure(PrevIndex);
			}
			if (!Neumann(FaceIndex))
			{
				Velocity(FaceIndex) -= (NextPressure - PrevPressure) / Grid.Dx()[Axis];
			}
		});
	}

  private:
	TFunction<void(const TUniformGrid<T, d>&, TArrayND<T, d>&, const TArrayND<T, d>&, const TArrayND<bool, d>&, const TArrayFaceND<bool, d>&, const T)> MPressureRule;
};
}
