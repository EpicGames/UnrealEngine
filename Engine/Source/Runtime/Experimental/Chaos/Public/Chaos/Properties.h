// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ParticleDirtyFlags.h"
#include "Framework/PhysicsProxyBase.h"
#include "Framework/PhysicsSolverBase.h"

namespace Chaos
{

template <typename T, EChaosProperty PropName>
class TChaosProperty
{
public:
	TChaosProperty() = default;
	TChaosProperty(const T& Val)
		: Property(Val)
	{
	}

	//we don't support this because it could lead to bugs with values not being properly written to remote
	TChaosProperty(const TChaosProperty<T,PropName>& Rhs) = delete;

	bool IsDirty(const FDirtyChaosPropertyFlags& Flags) const
	{
		return Flags.IsDirty(PropertyFlag);
	}

	const T& Read() const { return Property; }
	void Write(const T& Val, bool bInvalidate, FDirtyChaosPropertyFlags& Dirty, IPhysicsProxyBase* Proxy)
	{
		Property = Val;
		MarkDirty(bInvalidate, Dirty, Proxy);
	}

	template <typename Lambda>
	void Modify(bool bInvalidate, FDirtyChaosPropertyFlags& Dirty,IPhysicsProxyBase* Proxy,const Lambda& LambdaFunc)
	{
		LambdaFunc(Property);
		MarkDirty(bInvalidate, Dirty, Proxy);
	}

	void Clear(FDirtyChaosPropertyFlags& Dirty, IPhysicsProxyBase* Proxy)
	{
		Dirty.MarkClean(PropertyFlag);
		if (Proxy && Dirty.IsClean())
		{
			if (FPhysicsSolverBase* PhysicsSolverBase = Proxy->GetSolver<FPhysicsSolverBase>())
			{
				PhysicsSolverBase->RemoveDirtyProxyIfNoShapesAreDirty(Proxy);
			}
		}
	}

	void SyncRemote(FDirtyPropertiesManager& Manager, int32 DataIdx, const FDirtyChaosProperties& Remote) const
	{
		Remote.SyncRemote<T,PropName>(Manager, DataIdx, Property);
	}

	void Serialize(FChaosArchive& Ar)
	{
		Ar << Property;
	}

private:
	T Property;
	static constexpr EChaosPropertyFlags PropertyFlag = ChaosPropertyToFlag(PropName);

	void MarkDirty(bool bInvalidate, FDirtyChaosPropertyFlags& Dirty, IPhysicsProxyBase* Proxy)
	{
		if(bInvalidate)
		{
			Dirty.MarkDirty(PropertyFlag);
			
			if(Proxy)
			{
				if(FPhysicsSolverBase* PhysicsSolverBase = Proxy->GetSolver<FPhysicsSolverBase>())
				{
					PhysicsSolverBase->AddDirtyProxy(Proxy);
				}
			}
		}
	}

	//we don't support this because it could lead to bugs with values not being properly written to remote
	TChaosProperty<T,PropName>& operator=(const TChaosProperty<T,PropName>& Rhs) = delete;
};

template <typename T, EChaosProperty PropName>
FChaosArchive& operator<<(FChaosArchive& Ar, TChaosProperty<T, PropName>& Prop)
{
	//TODO: should this only work with dirty flag? Not sure if this path really matters at this point
	Prop.Serialize(Ar);
	return Ar;
}


template <typename T,EShapeProperty PropName>
class TShapeProperty
{
public:
	TShapeProperty() = default;
	TShapeProperty(const T& Val)
		: Property(Val)
	{
	}
	explicit TShapeProperty(TShapeProperty<T, PropName>&& Other)
		: Property(MoveTemp(Other.Property))
	{}

	//we don't support this because it could lead to bugs with values not being properly written to remote
	TShapeProperty(const TShapeProperty<T,PropName>& Rhs) = delete;

	const T& Read() const { return Property; }
	void Write(const T& Val,bool bInvalidate,FShapeDirtyFlags& Dirty,IPhysicsProxyBase* Proxy, int32 ShapeIdx)
	{
		Property = Val;
		MarkDirty(bInvalidate,Dirty,Proxy, ShapeIdx);
	}

	template <typename Lambda>
	void Modify(bool bInvalidate,FShapeDirtyFlags& Dirty,IPhysicsProxyBase* Proxy, int32 ShapeIdx, const Lambda& LambdaFunc)
	{
		LambdaFunc(Property);
		MarkDirty(bInvalidate,Dirty,Proxy, ShapeIdx);
	}

	void SyncRemote(FDirtyPropertiesManager& Manager, int32 DataIdx, FShapeDirtyData& Remote) const
	{
		Remote.SyncRemote<T,PropName>(Manager, DataIdx, Property);
	}

	void Serialize(FChaosArchive& Ar)
	{
		Ar << Property;
	}

private:
	T Property;
	static constexpr EShapeFlags PropertyFlag = ShapePropToFlag(PropName);

	void MarkDirty(bool bInvalidate, FShapeDirtyFlags& Dirty,IPhysicsProxyBase* Proxy, int32 ShapeIdx)
	{
		if(bInvalidate)
		{
			const bool bFirstDirty = Dirty.IsClean();
			Dirty.MarkDirty(PropertyFlag);

			if(bFirstDirty && Proxy)
			{
				if(FPhysicsSolverBase* PhysicsSolverBase = Proxy->GetSolver<FPhysicsSolverBase>())
				{
					PhysicsSolverBase->AddDirtyProxyShape(Proxy, ShapeIdx);
				}
			}
		}
	}

	//we don't support this because it could lead to bugs with values not being properly written to remote
	TShapeProperty<T, PropName>& operator=(const TShapeProperty<T, PropName>& Rhs) = delete;
};

template <typename T,EShapeProperty PropName>
FChaosArchive& operator<<(FChaosArchive& Ar,TShapeProperty<T,PropName>& Prop)
{
	//TODO: should this use dirty flags? not sure if this path even matters
	Prop.Serialize(Ar);
	return Ar;
}
}