// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Queue.h"
#include "Chaos/Defines.h"
#include "Chaos/ParticleDirtyFlags.h"
#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Chaos/ParallelFor.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"
#include "GeometryCollectionProxyData.h"
#include "PBDRigidsEvolutionFwd.h"

namespace Chaos
{

class FClusterUnionPhysicsProxy;
class FCharacterGroundConstraintProxy;
class FJointConstraintPhysicsProxy;

template <typename TProxy, typename TTimeStamp>
struct TBasePullData
{
public:
	void SetProxy(TProxy& InProxy)
	{
		ensure(Timestamp.Get() == nullptr);
		Timestamp = InProxy.GetSyncTimestamp();
		Proxy = &InProxy;
	}

	TProxy* GetProxy() const
	{
		if( Timestamp )
		{
			return !Timestamp->bDeleted ? Proxy : nullptr;
		}
		return nullptr;
	}
	
	const TTimeStamp* GetTimestamp() const { return static_cast<TTimeStamp*>(Timestamp.Get()); }
	
protected:
	TBasePullData() : Proxy(nullptr){}
	~TBasePullData() = default;
	
private:
	TProxy* Proxy;
	TSharedPtr<FProxyTimestampBase,ESPMode::ThreadSafe> Timestamp;	//question: is destructor expensive now? might need a better way
};

//Simple struct for when the simulation dirties a particle. Copies all properties regardless of which changed since they tend to change together
struct FDirtyRigidParticleData : public TBasePullData<FSingleParticlePhysicsProxy, FSingleParticleProxyTimestamp>
{
	FVec3 X;
	FQuat R;
	FVec3 V;
	FVec3 W;
	EObjectStateType ObjectState;
};

struct FDirtyRigidParticleReplicationErrorData : public TBasePullData<FSingleParticlePhysicsProxy, FSingleParticleProxyTimestamp>
{
	FVec3 ErrorX;
	FQuat ErrorR;
};

struct FDirtyGeometryCollectionData : public TBasePullData<FGeometryCollectionPhysicsProxy, FProxyTimestampBase>
{
public:
	bool HasResults() const { return ResultPtr.IsValid(); }

	FDirtyGeometryCollectionData()
		: ResultPtr(new FGeometryCollectionResults)
	{}

	FDirtyGeometryCollectionData(const FDirtyGeometryCollectionData& Other) = default;
	FDirtyGeometryCollectionData(FDirtyGeometryCollectionData&& Other) = default;
	FDirtyGeometryCollectionData& operator=(const FDirtyGeometryCollectionData& Other) = default;
	FDirtyGeometryCollectionData& operator=(FDirtyGeometryCollectionData&& Other) = default;

	FGeometryCollectionResults& Results()
	{
		check(ResultPtr);
		return *ResultPtr;
	}

	const FGeometryCollectionResults& Results() const 
	{
		check(ResultPtr);
		return *ResultPtr;
	}

private:
	TRefCountPtr<FGeometryCollectionResults> ResultPtr;
};

struct FDirtyClusterUnionParticleData
{
	FUniqueIdx ParticleIdx;
	FRigidTransform3 ChildToParent;
	IPhysicsProxyBase* Proxy = nullptr;
	void* CachedOwner = nullptr;
	int32 BoneId = INDEX_NONE;
};

struct FDirtyClusterUnionData : public TBasePullData<FClusterUnionPhysicsProxy, FClusterUnionProxyTimestamp>
{
	FVec3 X;
	FQuat R;
	FVec3 V;
	FVec3 W;
	FRealSingle Mass;
	FVec3f Inertia;
	EObjectStateType ObjectState = EObjectStateType::Dynamic;
	bool bIsAnchored = false;
	TArray<FDirtyClusterUnionParticleData> ChildParticles;
	Chaos::FImplicitObjectPtr Geometry = nullptr;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FDirtyClusterUnionData(){}
	FDirtyClusterUnionData(const FDirtyClusterUnionData&) = default;
	~FDirtyClusterUnionData() = default;
	Chaos::FDirtyClusterUnionData& operator =(const Chaos::FDirtyClusterUnionData &) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.4, "Please use Geometry instead")
	TSharedPtr<FImplicitObject, ESPMode::ThreadSafe> SharedGeometry = nullptr;
	
	TArray<FCollisionData> CollisionData;
	UE_DEPRECATED(5.8, "Use ShapeInstanceData and ShapeFilterData instead")
	TArray<FCollisionFilterData> QueryData;
	UE_DEPRECATED(5.8, "Use ShapeInstanceData and ShapeFilterData instead")
	TArray<FCollisionFilterData> SimData;
	TArray<Chaos::Filter::FShapeFilterData> ShapeFilterData;
	TArray<Chaos::Filter::FInstanceData> ShapeInstanceData;
};

struct FJointConstraintOutputData {
	bool bIsBreaking = false;
	bool bIsBroken = false;
	bool bIsViolating = false;
	bool bDriveTargetChanged = false;
	FVector Force = FVector(0);
	FVector Torque = FVector(0);
	float LinearViolation = 0.f;
	float AngularViolation = 0.f;
};

class FJointConstraint;

struct FDirtyJointConstraintData : public TBasePullData<FJointConstraintPhysicsProxy, FProxyTimestampBase>
{
	FJointConstraintOutputData OutputData;
};

struct FDirtyCharacterGroundConstraintData : public TBasePullData<FCharacterGroundConstraintProxy, FProxyTimestampBase>
{
	FVector Force = FVector(0.0);
	FVector Torque = FVector(0.0);
	FCharacterGroundConstraintSettings Settings;
	FCharacterGroundConstraintDynamicData Data;
	FGeometryParticleHandle* GroundParticle = nullptr;
	bool bSettingsChanged = false;
	bool bDataChanged = false;
	bool bGroundParticleChanged = false;
};

//A simulation frame's result of dirty particles. These are all the particles that were dirtied in this particular sim step
class FPullPhysicsData
{
public:
	TArray<FDirtyRigidParticleData> DirtyRigids;
	TMap<const IPhysicsProxyBase*, FDirtyRigidParticleReplicationErrorData> DirtyRigidErrors;
	TArray<FDirtyGeometryCollectionData> DirtyGeometryCollections;
	TArray<FDirtyClusterUnionData> DirtyClusterUnions;
	TArray<FDirtyJointConstraintData> DirtyJointConstraints;
	TArray<FDirtyCharacterGroundConstraintData> DirtyCharacterGroundConstraints;

	int32 SolverTimestamp;
	FReal ExternalStartTime;	//The start time associated with this result. The time is synced using the external time
	FReal ExternalEndTime;		//The end time associated with this result. The time is synced using the external time

	void Reset();
};

}; // namespace Chaos
