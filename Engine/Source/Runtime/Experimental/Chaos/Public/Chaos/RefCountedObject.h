// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <atomic>
#include "AutoRTFM.h"
#include "Misc/AssertionMacros.h"

namespace Chaos
{

// Chaos ref-counted object
//  * @note In AutoRTFM, the return value of AddRef/Release may be higher than expected, because the refcount won't decrease
//          until the transaction is committed. This is fine for use with TRefCountPtr, as it doesn't use the refcount directly.
class FChaosRefCountedObject
{
public:
	FChaosRefCountedObject() {}
	virtual ~FChaosRefCountedObject()
	{
		// We want to report an error if we attempt to destroy a ref-counted object with leaked references.
		// Sometimes, these objects exist ephemerally on the stack. If so, it should have a refcount of zero right now.
		if (GetRefCount() != 0)
		{
			// If not, it might still have references that are queued up to Release at ONCOMMIT time. So, we check a second
			// time during OnCommit. (If an ephemeral stack object is destroyed a non-zero refcount, this is user error;
			// we might report this by check-failing here with a garbage value for its refcount, as its stack representation
			// might already be overwritten.)
			UE_AUTORTFM_ONCOMMIT(this)
			{
				check(GetRefCount() == 0);
			};
		}
	}
	FChaosRefCountedObject(const FChaosRefCountedObject& Rhs) = delete;
	FChaosRefCountedObject& operator=(const FChaosRefCountedObject& Rhs) = delete;
	
	uint32 AddRef() const
	{
		bool bIsFirstReference;

		UE_AUTORTFM_OPEN
		{
			bIsFirstReference = (++NumRefs == 1);
		};
		UE_AUTORTFM_ONABORT(=, this)
		{
			if (bIsFirstReference)
			{
				// We took the first reference, and then aborted. This should undo the taking of
				// the reference, but shouldn't delete the object if it is transient.
				--NumRefs;
			}
			else
			{
				// After an object gains its initial reference, an AddRef call can be balanced
				// out with a matching Release.
				Release();
			}
		};
		// TRefCountPtr doesn't use the return value.
		return 0;
	}

	uint32 Release() const
	{
		UE_AUTORTFM_ONCOMMIT(this)
		{
			if (--NumRefs == 0 && RefCountMode == ERCM_Transient)
			{
				delete this;
			}
		};
		// TRefCountPtr doesn't use the return value.
		return 0;
	}

	uint32 GetRefCount() const
	{
		uint32 Ret;
		UE_AUTORTFM_OPEN
		{
			Ret = uint32(NumRefs.load());
		};
		return Ret;
	}

	void MakePersistent() const
	{
		ERCM_RefCountMode OriginalMode = RefCountMode;

		UE_AUTORTFM_OPEN
		{
			RefCountMode = ERCM_Persistent;
		};
		UE_AUTORTFM_ONABORT(=, this)
		{
			RefCountMode = OriginalMode;
		};
	}

private:
	// Number of refs onto the object.
	mutable std::atomic<int32> NumRefs = 0;
	
	enum ERCM_RefCountMode {
		// An object is considered Transient by default.
		// After an initial AddRef, when the reference count reaches zero, it automatically deletes itself.
		ERCM_Transient,
		// Calling MakePersistent will convert an object to Persistent. 
		// A Persistent object no longer deletes itself when the reference count reaches zero; the caller
		// is responsible for deletion. (Basically, this opts out of the reference-counting mechanism.)
		ERCM_Persistent,
	};
	mutable std::atomic<ERCM_RefCountMode> RefCountMode = ERCM_Transient;
};

}  // namespace Chaos
