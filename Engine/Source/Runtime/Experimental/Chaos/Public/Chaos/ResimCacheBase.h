// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "ParticleHandle.h"

namespace Chaos
{

class IResimCacheBase
{
public:
	IResimCacheBase()
	: bIsResimming(false)
	{
	}

	virtual ~IResimCacheBase() = default;
	virtual void ResetCache() = 0;
	bool IsResimming() const { return bIsResimming; }
	void SetResimming(bool bInResimming) { bIsResimming = bInResimming; }

	const TParticleView<TGeometryParticles<FReal,3>>& GetDesyncedView() const
	{
		return DesyncedView;
	}

	void SetDesyncedParticles(TArray<TGeometryParticleHandle<FReal,3>*>&& InDesyncedParticles)
	{
		DesyncedParticles = MoveTemp(InDesyncedParticles);
		TArray<TSOAView<TGeometryParticles<FReal,3>>> TmpArray = { {&DesyncedParticles} };
		DesyncedView = MakeParticleView(MoveTemp(TmpArray));
	}
private:
	bool bIsResimming;
	TParticleView<TGeometryParticles<FReal,3>> DesyncedView;
	TArray<TGeometryParticleHandle<FReal,3>*> DesyncedParticles;
};

} // namespace Chaos
