// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/Core.h"

namespace Chaos
{
	namespace ResimConsoleVars
	{
		extern CHAOS_API bool bIsBubbleResimEnabled;
		extern CHAOS_API bool bCanFrozenParticlesModifyContacts;
	}
}