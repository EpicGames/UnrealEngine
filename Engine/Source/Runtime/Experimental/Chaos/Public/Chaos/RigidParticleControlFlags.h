// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class FString;

namespace Chaos
{
	class FChaosArchive;

	/**
	 * @brief Enable/Disable the features on a particle
	 * These flags are externally controlled and should not be changed by the solver during the tick. The may be
	 * bound directly to settings or game-side object state. These flags should be treated as read-only to the solver.
	*/
	class FRigidParticleControlFlags
	{
	public:
		using FStorage = uint16;

		FRigidParticleControlFlags()
			: Bits(0)
		{
		}

		FStorage GetFlags() const { return Bits; }
		void SetFlags(const FStorage InBits) { Bits = InBits; }

		bool GetGravityEnabled() const { return Flags.bGravityEnabled; }
		FRigidParticleControlFlags& SetGravityEnabled(const bool bEnabled) { Flags.bGravityEnabled = bEnabled; return *this; }

		bool GetUpdateKinematicFromSimulation() const { return Flags.bUpdateKinematicFromSimulation; }
		FRigidParticleControlFlags& SetUpdateKinematicFromSimulation(const bool bUpdateKinematicFromSimulation) { Flags.bUpdateKinematicFromSimulation = bUpdateKinematicFromSimulation; return *this; }

		bool GetGyroscopicTorqueEnabled() const { return Flags.bGyroscopicTorqueEnabled; }
		FRigidParticleControlFlags& SetGyroscopicTorqueEnabled(const bool bEnabled) { Flags.bGyroscopicTorqueEnabled = bEnabled; return *this; }

		int32 GetGravityGroupIndex() const { return Flags.GravityGroupIndex; }
		FRigidParticleControlFlags& SetGravityGroupIndex(const int32 GravityGroupIndex) 
		{ 
			ensure(GravityGroupIndex < 8);
			Flags.GravityGroupIndex = static_cast<FStorage>(GravityGroupIndex);
			return *this; 
		}

		bool GetCCDEnabled() const { return Flags.bCCDEnabled; }
		FRigidParticleControlFlags& SetCCDEnabled(const bool bEnabled) { Flags.bCCDEnabled = bEnabled; return *this; }

		bool GetMACDEnabled() const { return Flags.bMACDEnabled; }
		FRigidParticleControlFlags& SetMACDEnabled(const bool bEnabled) { Flags.bMACDEnabled = bEnabled; return *this; }

		bool GetPartialIslandSleepAllowed() const { return Flags.bPartialIslandSleepAllowed; }
		FRigidParticleControlFlags& SetPartialIslandSleepAllowed(const bool bEnabled) { Flags.bPartialIslandSleepAllowed = bEnabled; return *this; }

		bool GetOneWayInteractionEnabled() const { return Flags.bOneWayInteractionEnabled; }
		FRigidParticleControlFlags& SetOneWayInteractionEnabled(const bool bEnabled) { Flags.bOneWayInteractionEnabled = bEnabled; return *this; }

		// If enabled, inertia may be increased to improve stability
		bool GetInertiaConditioningEnabled() const { return Flags.bInertiaConditioningEnabled; }
		FRigidParticleControlFlags& SetInertiaConditioningEnabled(const bool bEnabled) { Flags.bInertiaConditioningEnabled = bEnabled; return *this; }

		// Used by rewind system debuging
		FString ToString() const;

		// Used by MarshallingManager and dirty data system
		friend bool operator==(const FRigidParticleControlFlags& L, const FRigidParticleControlFlags& R) { return L.GetFlags() == R.GetFlags(); }
		friend bool operator!=(const FRigidParticleControlFlags& L, const FRigidParticleControlFlags& R) { return L.GetFlags() != R.GetFlags(); }

		// Serialization
		friend FChaosArchive& operator<<(FChaosArchive& Ar, FRigidParticleControlFlags& Data);

		UE_DEPRECATED(5.4, "Not used")
		bool GetMaxDepenetrationVelocityOverrideEnabled() const { return false; }

		UE_DEPRECATED(5.4, "Not used")
		FRigidParticleControlFlags& SetMaxDepenetrationVelocityOverrideEnabled(const bool bEnabled) { return *this; }

	private:
		struct FFlags
		{
			FStorage bGravityEnabled : 1;
			FStorage bCCDEnabled : 1;
			FStorage bOneWayInteractionEnabled : 1;
			FStorage bEnableInitialOverlapDepenetration : 1;
			FStorage bInertiaConditioningEnabled : 1;
			FStorage GravityGroupIndex : 3;
			FStorage bUpdateKinematicFromSimulation : 1;
			FStorage bMACDEnabled : 1;
			FStorage bPartialIslandSleepAllowed : 1;
			FStorage bGyroscopicTorqueEnabled : 1;
			// Add new properties above this line
			// Change FStorage typedef if we exceed the max bits
		};
		union
		{
			FFlags Flags;
			FStorage Bits;
		};
	};

	// If we add more bits and exceed the storage size...
	static_assert(sizeof(FRigidParticleControlFlags) == sizeof(FRigidParticleControlFlags::FStorage), "Change FRigidParticleControlFlags::FStorage to be larger");


	/**
	 * @brief Transient flags for indicating somethings needs to be updated based on a change to the particle
	 * Typically some of these flags will be set when something on the particle changes (e.g., when initially created,
	 * a joint is connected, etc etc). The flags will be checked and reset by some system during the tick.
	 * 
	 * Transient flags are for use in the solver only. They should not be directly controlled by settings, or
	 * sent back outside of the solver (e.g., not to the game thread). @see FRigidParticleControlFlags for
	 * externally controlled flags.
	 *
	 * E.g., the bInertiaConditioningDirty flags is set whenever the particle changes in such a way that the
	 * inertia conditioning needs to be recalculated and is reset by the evolution when it has recalculated it.
	 * 
	 * @note In general the transient flags can be set from multiple locations, but should only ever be cleared in one.
	*/
	class FRigidParticleTransientFlags
	{
	public:
		using FStorage = uint8;

		FRigidParticleTransientFlags()
			: Bits(0)
		{
		}

		FStorage GetFlags() const { return Bits; }
		void SetFlags(const FStorage InFlags) { Bits = InFlags; }

		// Set to true when some property of the particle changes and we should regenerate the inertia conditioning vector
		bool GetInertiaConditioningDirty() const { return Flags.bInertiaConditioningDirty; }
		void SetInertiaConditioningDirty() { Flags.bInertiaConditioningDirty = true; }
		void ClearInertiaConditioningDirty() { Flags.bInertiaConditioningDirty = false; }

		// Set to true when the particle has one or more entries in the CollisionIgnoreManager
		bool GetUseIgnoreCollisionManager() const { return Flags.bUseIgnoreCollisionManager; }
		void SetUseIgnoreCollisionManager() { Flags.bUseIgnoreCollisionManager = true; }
		void ClearUseIgnoreCollisionManager() { Flags.bUseIgnoreCollisionManager = false; }

		// Is this particle kinematic and moving (velocity and angular velocity are non-zero). This is updated in ApplyKinematicTargets
		// to help avoid checking velocity and angular velocity (especially for kinematics).
		bool GetIsMovingKinematic() const { return Flags.bIsMovingKinematic; }
		void SetIsMovingKinematic() { Flags.bIsMovingKinematic = true; }
		void ClearIsMovingKinematic() { Flags.bIsMovingKinematic = false; }

	private:
		struct FFlags
		{
			FStorage bInertiaConditioningDirty : 1;
			FStorage bUseIgnoreCollisionManager : 1;
			FStorage bIsMovingKinematic : 1;
			// Add new properties above this line
			// Change FStorage typedef if we exceed the max bits
		};
		union
		{
			FFlags Flags;
			FStorage Bits;
		};
	};

	// If we add more bits and exceed the storage size...
	static_assert(sizeof(FRigidParticleTransientFlags) == sizeof(FRigidParticleTransientFlags::FStorage), "Change FRigidParticleTransientFlags::FStorage to be larger");
}