// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Queue.h"
#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/Collision/CollisionConstraintFlags.h"
#include "Chaos/BVHParticles.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/KinematicGeometryParticles.h"
#include "Chaos/Matrix.h"
#include "Chaos/Particles.h"
#include "Chaos/Rotation.h"
#include "Chaos/RigidParticleControlFlags.h"
#include "HAL/LowLevelMemTracker.h"
#include "Chaos/Evolution/IterationSettings.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

namespace Chaos
{

enum class ESleepType : uint8
{
	MaterialSleep,	//physics material determines sleep threshold
	NeverSleep		//never falls asleep
};

template<class T, int d>
struct TSleepData
{
	TSleepData()
		: Particle(nullptr)
		, Sleeping(true)
	{}

	TSleepData(
		TGeometryParticleHandle<T, d>* InParticle, bool InSleeping)
		: Particle(InParticle)
		, Sleeping(InSleeping)
	{}

	TGeometryParticleHandle<T, d>* Particle;
	bool Sleeping; // if !Sleeping == Awake
};


// Counts the number of bits needed to represent an int with a max
constexpr int8 NumBitsNeeded(const int8 MaxValue)
{
	return MaxValue == 0 ? 0 : 1 + NumBitsNeeded(MaxValue >> 1);
}

// Make a bitmask which covers the lowest NumBits bits with 1's.
constexpr int8 LowBitsMask(const int8 NumBits)
{
	return NumBits == 0 ? 0 : (int8)((1 << (NumBits - 1)) | LowBitsMask(NumBits - 1));
}

// Count N, the number of bits needed to store an object state
static constexpr int8 ObjectStateBitCount = NumBitsNeeded((int8)EObjectStateType::Count - (int8)1);

// RigidParticle data that is commonly accessed together.
// This contains all properties accessed in the broadphase filtering (FSpatialAccelerationBroadPhase)
// NOTE: not a member class for easier natvis
struct FRigidParticleCoreData
{
	int32 CollisionGroup							= INDEX_NONE;						// 4 bytes
	uint32 CollisionConstraintFlags					= 0;								// 4 bytes
	FRigidParticleControlFlags ControlFlags			= {};								// 2 bytes
	FRigidParticleTransientFlags TransientFlags		= {};								// 1 byte
	EObjectStateType ObjectState					= EObjectStateType::Uninitialized;	// 1 byte
	EObjectStateType PreObjectState					= EObjectStateType::Uninitialized;	// 1 byte
	bool bDisabled									= false;							// 1 byte
};

template<class T, int d>
class TRigidParticles : public TKinematicGeometryParticles<T, d>
{
public:
	using TArrayCollection::Size;
    using TParticles<T, d>::X;
    using TGeometryParticles<T, d>::R;

	TRigidParticles()
	    : TKinematicGeometryParticles<T, d>()
	{
		RegisterArrays();
	}

	TRigidParticles(const TRigidParticles<T, d>& Other) = delete;
	TRigidParticles(TRigidParticles<T, d>&& Other)
	    : TKinematicGeometryParticles<T, d>(MoveTemp(Other))
		, CoreData(MoveTemp(Other.CoreData))
		, MVSmooth(MoveTemp(Other.MVSmooth))
		, MWSmooth(MoveTemp(Other.MWSmooth))
		, MAcceleration(MoveTemp(Other.MAcceleration))
		, MAngularAcceleration(MoveTemp(Other.MAngularAcceleration))
		, MLinearImpulseVelocity(MoveTemp(Other.MLinearImpulseVelocity))
		, MAngularImpulseVelocity(MoveTemp(Other.MAngularImpulseVelocity))
		, MI(MoveTemp(Other.MI))
		, MInvI(MoveTemp(Other.MInvI))
		, MInvIConditioning(MoveTemp(Other.MInvIConditioning))
		, MM(MoveTemp(Other.MM))
		, MInvM(MoveTemp(Other.MInvM))
		, MCenterOfMass(MoveTemp(Other.MCenterOfMass))
		, MRotationOfMass(MoveTemp(Other.MRotationOfMass))
		, MLinearEtherDrag(MoveTemp(Other.MLinearEtherDrag))
		, MAngularEtherDrag(MoveTemp(Other.MAngularEtherDrag))
		, MaxLinearSpeedsSq(MoveTemp(Other.MaxLinearSpeedsSq))
		, MaxAngularSpeedsSq(MoveTemp(Other.MaxAngularSpeedsSq))
		, MInitialOverlapDepenetrationVelocity(MoveTemp(Other.MInitialOverlapDepenetrationVelocity))
		, MSleepThresholdMultiplier(MoveTemp(Other.MSleepThresholdMultiplier))
		, MCollisionParticles(MoveTemp(Other.MCollisionParticles))
		, MSleepType(MoveTemp(Other.MSleepType))
		, MSleepCounter(MoveTemp(Other.MSleepCounter))
		, MDisableCounter(MoveTemp(Other.MDisableCounter))
		, MParticleIterationCounts(MoveTemp(Other.MParticleIterationCounts))
	{
		RegisterArrays();
	}

	void RegisterArrays()
	{
		TArrayCollection::AddArray(&CoreData);

		TArrayCollection::AddArray(&MVSmooth);
		TArrayCollection::AddArray(&MWSmooth);
		TArrayCollection::AddArray(&MAcceleration);
		TArrayCollection::AddArray(&MAngularAcceleration);
		TArrayCollection::AddArray(&MLinearImpulseVelocity);
		TArrayCollection::AddArray(&MAngularImpulseVelocity);
		TArrayCollection::AddArray(&MI);
		TArrayCollection::AddArray(&MInvI);
		TArrayCollection::AddArray(&MInvIConditioning);
		TArrayCollection::AddArray(&MM);
		TArrayCollection::AddArray(&MInvM);
		TArrayCollection::AddArray(&MCenterOfMass);
		TArrayCollection::AddArray(&MRotationOfMass);
		TArrayCollection::AddArray(&MLinearEtherDrag);
		TArrayCollection::AddArray(&MAngularEtherDrag);
		TArrayCollection::AddArray(&MaxLinearSpeedsSq);
		TArrayCollection::AddArray(&MaxAngularSpeedsSq);
		TArrayCollection::AddArray(&MInitialOverlapDepenetrationVelocity);
		TArrayCollection::AddArray(&MSleepThresholdMultiplier);
		TArrayCollection::AddArray(&MCollisionParticles);
		TArrayCollection::AddArray(&MSleepType);
		TArrayCollection::AddArray(&MSleepCounter);
		TArrayCollection::AddArray(&MDisableCounter);
		TArrayCollection::AddArray(&MParticleIterationCounts);

	}

	virtual ~TRigidParticles() = default;

	FORCEINLINE const TVector<T, d>& VSmooth(const int32 Index) const { return MVSmooth[Index]; }
	FORCEINLINE TVector<T, d>& VSmooth(const int32 Index) { return MVSmooth[Index]; }

	FORCEINLINE const TVector<T, d>& WSmooth(const int32 Index) const { return MWSmooth[Index]; }
	FORCEINLINE TVector<T, d>& WSmooth(const int32 Index) { return MWSmooth[Index]; }

	FORCEINLINE const TVector<T, d>& AngularAcceleration(const int32 Index) const { return MAngularAcceleration[Index]; }
	FORCEINLINE TVector<T, d>& AngularAcceleration(const int32 Index) { return MAngularAcceleration[Index]; }

	FORCEINLINE const TVector<T, d>& Acceleration(const int32 Index) const { return MAcceleration[Index]; }
	FORCEINLINE TVector<T, d>& Acceleration(const int32 Index) { return MAcceleration[Index]; }

	FORCEINLINE const TVector<T, d>& LinearImpulseVelocity(const int32 Index) const { return MLinearImpulseVelocity[Index]; }
	FORCEINLINE TVector<T, d>& LinearImpulseVelocity(const int32 Index) { return MLinearImpulseVelocity[Index]; }

	FORCEINLINE const TVector<T, d>& AngularImpulseVelocity(const int32 Index) const { return MAngularImpulseVelocity[Index]; }
	FORCEINLINE TVector<T, d>& AngularImpulseVelocity(const int32 Index) { return MAngularImpulseVelocity[Index]; }

	FORCEINLINE const TVec3<FRealSingle>& I(const int32 Index) const { return MI[Index]; }
	FORCEINLINE TVec3<FRealSingle>& I(const int32 Index) { return MI[Index]; }

	FORCEINLINE const TVec3<FRealSingle>& InvI(const int32 Index) const { return MInvI[Index]; }
	FORCEINLINE TVec3<FRealSingle>& InvI(const int32 Index) { return MInvI[Index]; }

	FORCEINLINE const TVec3<FRealSingle>& InvIConditioning(const int32 Index) const { return MInvIConditioning[Index]; }
	FORCEINLINE TVec3<FRealSingle>& InvIConditioning(const int32 Index) { return MInvIConditioning[Index]; }

	FORCEINLINE const T M(const int32 Index) const { return MM[Index]; }
	FORCEINLINE T& M(const int32 Index) { return MM[Index]; }

	FORCEINLINE const T InvM(const int32 Index) const { return MInvM[Index]; }
	FORCEINLINE T& InvM(const int32 Index) { return MInvM[Index]; }

	FORCEINLINE const TVector<T,d>& CenterOfMass(const int32 Index) const { return MCenterOfMass[Index]; }
	FORCEINLINE TVector<T,d>& CenterOfMass(const int32 Index) { return MCenterOfMass[Index]; }

	FORCEINLINE const TRotation<T,d>& RotationOfMass(const int32 Index) const { return MRotationOfMass[Index]; }
	FORCEINLINE TRotation<T,d>& RotationOfMass(const int32 Index) { return MRotationOfMass[Index]; }

	FORCEINLINE const T& LinearEtherDrag(const int32 index) const { return MLinearEtherDrag[index]; }
	FORCEINLINE T& LinearEtherDrag(const int32 index) { return MLinearEtherDrag[index]; }

	FORCEINLINE const T& AngularEtherDrag(const int32 index) const { return MAngularEtherDrag[index]; }
	FORCEINLINE T& AngularEtherDrag(const int32 index) { return MAngularEtherDrag[index]; }

	FORCEINLINE const T& MaxLinearSpeedSq(const int32 index) const { return MaxLinearSpeedsSq[index]; }
	FORCEINLINE T& MaxLinearSpeedSq(const int32 index) { return MaxLinearSpeedsSq[index]; }

	FORCEINLINE const T& MaxAngularSpeedSq(const int32 index) const { return MaxAngularSpeedsSq[index]; }
	FORCEINLINE T& MaxAngularSpeedSq(const int32 index) { return MaxAngularSpeedsSq[index]; }

	FORCEINLINE const FRealSingle& InitialOverlapDepenetrationVelocity(const int32 index) const { return MInitialOverlapDepenetrationVelocity[index]; }
	FORCEINLINE FRealSingle& InitialOverlapDepenetrationVelocity(const int32 index) { return MInitialOverlapDepenetrationVelocity[index]; }

	FORCEINLINE const FRealSingle& SleepThresholdMultiplier(const int32 Index) const { return MSleepThresholdMultiplier[Index]; }
	FORCEINLINE FRealSingle& SleepThresholdMultiplier(const int32 Index) { return MSleepThresholdMultiplier[Index]; }

	FORCEINLINE int32 CollisionParticlesSize(int32 Index) const { return MCollisionParticles[Index] == nullptr ? 0 : MCollisionParticles[Index]->Size(); }

	void CollisionParticlesInitIfNeeded(const int32 Index);
	void SetCollisionParticles(const int32 Index, TParticles<T, d>&& Particles);
	
	FORCEINLINE const TUniquePtr<TBVHParticles<T, d>>& CollisionParticles(const int32 Index) const { return MCollisionParticles[Index]; }
	FORCEINLINE TUniquePtr<TBVHParticles<T, d>>& CollisionParticles(const int32 Index) { return MCollisionParticles[Index]; }

	FORCEINLINE const int32 CollisionGroup(const int32 Index) const { return CoreData[Index].CollisionGroup; }
	FORCEINLINE int32& CollisionGroup(const int32 Index) { return CoreData[Index].CollisionGroup; }

	FORCEINLINE bool HasCollisionConstraintFlag(const ECollisionConstraintFlags Flag, const int32 Index) const { return (CoreData[Index].CollisionConstraintFlags & (uint32)Flag) != 0; }
	FORCEINLINE void AddCollisionConstraintFlag(const ECollisionConstraintFlags Flag, const int32 Index) { CoreData[Index].CollisionConstraintFlags |= (uint32)Flag; }
	FORCEINLINE void RemoveCollisionConstraintFlag(const ECollisionConstraintFlags Flag, const int32 Index) { CoreData[Index].CollisionConstraintFlags &= ~(uint32)Flag; }
	FORCEINLINE void SetCollisionConstraintFlags(const int32 Index, const uint32 Flags) { CoreData[Index].CollisionConstraintFlags = Flags; }
	FORCEINLINE uint32 CollisionConstraintFlags(const int32 Index) const { return CoreData[Index].CollisionConstraintFlags; }

	FORCEINLINE const bool Disabled(const int32 Index) const { return CoreData[Index].bDisabled; }

	FORCEINLINE bool& DisabledRef(const int32 Index) { return CoreData[Index].bDisabled; }

	// DisableParticle/EnableParticle on Evolution should be used. Don't disable particles with this.
    // Using this will break stuff. This is for solver's use only, and possibly some particle construction/copy code.
	FORCEINLINE void SetDisabledLowLevel(const int32 Index, bool InDisabled) { CoreData[Index].bDisabled = InDisabled; }

	FORCEINLINE const FRigidParticleControlFlags& ControlFlags(const int32 Index) const { return CoreData[Index].ControlFlags; }
	FORCEINLINE FRigidParticleControlFlags& ControlFlags(const int32 Index) { return CoreData[Index].ControlFlags; }

	FORCEINLINE const FRigidParticleTransientFlags& TransientFlags(const int32 Index) const { return CoreData[Index].TransientFlags; }
	FORCEINLINE FRigidParticleTransientFlags& TransientFlags(const int32 Index) { return CoreData[Index].TransientFlags; }

	FORCEINLINE ESleepType SleepType(const int32 Index) const { return MSleepType[Index]; }
	FORCEINLINE ESleepType& SleepType(const int32 Index) { return MSleepType[Index]; }

	FORCEINLINE int8 SleepCounter(const int32 Index) const { return MSleepCounter[Index]; }
	FORCEINLINE int8& SleepCounter(const int32 Index) { return MSleepCounter[Index]; }

	FORCEINLINE int8 DisableCounter(const int32 Index) const { return MDisableCounter[Index]; }
	FORCEINLINE int8& DisableCounter(const int32 Index) { return MDisableCounter[Index]; }

	FORCEINLINE Private::FIterationSettings ParticleIterationCounts(const int32 Index) const { return MParticleIterationCounts[Index]; }
	FORCEINLINE Private::FIterationSettings& ParticleIterationCounts(const int32 Index) { return MParticleIterationCounts[Index]; }

	// @todo(chaos): This data should be marshalled via the proxies like everything else. There is probably a particle recycling bug here.
	FORCEINLINE TArray<TSleepData<T, d>>& GetSleepData() { return MSleepData; }
	FORCEINLINE	void AddSleepData(TGeometryParticleHandle<T, d>* Particle, bool Sleeping)
	{ 
		TSleepData<T, d> SleepData;
		SleepData.Particle = Particle;
		SleepData.Sleeping = Sleeping;

		SleepDataLock.WriteLock();
		MSleepData.Add(SleepData);
		SleepDataLock.WriteUnlock();
	}
	void ClearSleepData()
	{
		SleepDataLock.WriteLock();
		MSleepData.Empty();
		SleepDataLock.WriteUnlock();
	}
	FORCEINLINE FTransactionallySafeRWLock& GetSleepDataLock() { return SleepDataLock; }

	FORCEINLINE const EObjectStateType ObjectState(const int32 Index) const { return CoreData[Index].ObjectState; }
	FORCEINLINE EObjectStateType& ObjectState(const int32 Index) { return CoreData[Index].ObjectState; }

	FORCEINLINE const EObjectStateType PreObjectState(const int32 Index) const { return CoreData[Index].PreObjectState; }
	FORCEINLINE EObjectStateType& PreObjectState(const int32 Index) { return CoreData[Index].PreObjectState; }

	FORCEINLINE const bool Dynamic(const int32 Index) const { return ObjectState(Index) == EObjectStateType::Dynamic; }

	FORCEINLINE const bool Sleeping(const int32 Index) const { return ObjectState(Index) == EObjectStateType::Sleeping; }

	FORCEINLINE const bool HasInfiniteMass(const int32 Index) const { return MInvM[Index] == (T)0; }

	FORCEINLINE FString ToString(int32 Index) const
	{
		FString BaseString = TKinematicGeometryParticles<T, d>::ToString(Index);
		return FString::Printf(TEXT("%s, MAcceleration:%s, MAngularAcceleration:%s, MLinearImpulseVelocity:%s, MAngularImpulseVelocity:%s, MI:%s, MInvI:%s, MM:%f, MInvM:%f, MCenterOfMass:%s, MRotationOfMass:%s, MCollisionParticles(num):%d, MCollisionGroup:%d, MDisabled:%d, MSleeping:%d"),
			*BaseString, *Acceleration(Index).ToString(), *AngularAcceleration(Index).ToString(), *LinearImpulseVelocity(Index).ToString(), *AngularImpulseVelocity(Index).ToString(),
			*I(Index).ToString(), *InvI(Index).ToString(), M(Index), InvM(Index), *CenterOfMass(Index).ToString(), *RotationOfMass(Index).ToString(), CollisionParticlesSize(Index),
			CollisionGroup(Index), Disabled(Index), Sleeping(Index));
	}

	virtual void Serialize(FChaosArchive& Ar) override
	{
		TKinematicGeometryParticles<T,d>::Serialize(Ar);

		// To avoid bumping file version, serialize to/from previous structures
		// If we aren't loading (i.e., we are saving or copying) CoreData will be valid, so copy that to the legacy structure
		// Also, Particles do not know their island index and it should not be serialized (but it used to be)
		// @todo(chaos): I think its time to bump the version number and clean up particle serialization!
		FLegacyData LegacyData;
		TArrayCollectionArray<int32> LegacyIslandIndex;

		if (!Ar.IsLoading())
		{
			LegacyData.CopyFromCoreData(CoreData);
			LegacyIslandIndex.SetNumZeroed(MCollisionParticles.Num());
		}

		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		if(Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::KinematicCentersOfMass)
		{
			Ar << MCenterOfMass;
			Ar << MRotationOfMass;
		}

		Ar << MAcceleration << MAngularAcceleration << MLinearImpulseVelocity << MAngularImpulseVelocity;

		Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
		Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
		if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ChaosInertiaConvertedToVec3
			&& Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::ChaosInertiaConvertedToVec3)
		{
			TArray<PMatrix<T, d, d>> IArray;
			TArray<PMatrix<T, d, d>> InvIArray;
			Ar << IArray << InvIArray;

			for (int32 Idx = 0; Idx < IArray.Num(); ++Idx)
			{
				MI.Add(IArray[Idx].GetDiagonal());
				MInvI.Add(InvIArray[Idx].GetDiagonal());
			}
		}
		else
		{
			Ar << MI << MInvI;
		}

		MInvIConditioning.Resize(MInvI.Num());
		for (int32 Index = 0; Index < MInvI.Num(); ++Index)
		{
			MInvIConditioning[Index] = TVec3<FRealSingle>(1);
		}
		
		Ar << MM << MInvM;

		Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
		if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::AddDampingToRigids)
		{
			Ar << MLinearEtherDrag << MAngularEtherDrag;
		}

		Ar << MCollisionParticles << LegacyData.MCollisionGroup << LegacyIslandIndex << LegacyData.MDisabled << LegacyData.MObjectState << MSleepType;
		// @todo(chaos): what about ControlFlags, TransientFlags, PreObjectState, SleepCounter, ..?

		// If we loaded into the legacy structure, copy to CoreData
		if (Ar.IsLoading())
		{
			LegacyData.CopyToCoreData(CoreData);
		}
	}

	// Deprecated API
	UE_DEPRECATED(5.3, "No longer supported") const int32 IslandIndex(const int32 Index) const { return INDEX_NONE; }
	UE_DEPRECATED(5.3, "No longer supported") int32& IslandIndex(const int32 Index) { static int32 Dummy = INDEX_NONE; return Dummy; }

private:
	// Used during serialization to avoid bumping the file version as we switch to aggregated strunctures like FRigidParticleCoreData.
	// Note: Only serialized data is needed here and not all data is serialized so some elements in the aggregates are not represented.
	struct FLegacyData
	{
		TArrayCollectionArray<int32> MCollisionGroup;
		TArrayCollectionArray<EObjectStateType> MObjectState;
		TArrayCollectionArray<bool> MDisabled;

		void CopyFromCoreData(const TArrayCollectionArray<FRigidParticleCoreData>& Source)
		{
			MCollisionGroup.Resize(Source.Num());
			MObjectState.Resize(Source.Num());
			MDisabled.Resize(Source.Num());

			for (int32 Index = 0; Index < Source.Num(); ++Index)
			{
				MCollisionGroup[Index] = Source[Index].CollisionGroup;
				MObjectState[Index] = Source[Index].ObjectState;
				MDisabled[Index] = Source[Index].bDisabled;
			}
		}

		void CopyToCoreData(TArrayCollectionArray<FRigidParticleCoreData>& Dest)
		{
			Dest.Resize(MCollisionGroup.Num());

			for (int32 Index = 0; Index < Dest.Num(); ++Index)
			{
				Dest[Index].CollisionGroup = MCollisionGroup[Index];
				Dest[Index].ObjectState = MObjectState[Index];
				Dest[Index].bDisabled = MDisabled[Index];
			}
		}
	};

	TArrayCollectionArray<FRigidParticleCoreData> CoreData;

	TArrayCollectionArray<TVector<T, d>> MVSmooth;
	TArrayCollectionArray<TVector<T, d>> MWSmooth;
	TArrayCollectionArray<TVector<T, d>> MAcceleration;
	TArrayCollectionArray<TVector<T, d>> MAngularAcceleration;
	TArrayCollectionArray<TVector<T, d>> MLinearImpulseVelocity;
	TArrayCollectionArray<TVector<T, d>> MAngularImpulseVelocity;
	TArrayCollectionArray<TVec3<FRealSingle>> MI;
	TArrayCollectionArray<TVec3<FRealSingle>> MInvI;
	TArrayCollectionArray<TVec3<FRealSingle>> MInvIConditioning;
	TArrayCollectionArray<T> MM;
	TArrayCollectionArray<T> MInvM;
	TArrayCollectionArray<TVector<T,d>> MCenterOfMass;
	TArrayCollectionArray<TRotation<T,d>> MRotationOfMass;
	TArrayCollectionArray<T> MLinearEtherDrag;
	TArrayCollectionArray<T> MAngularEtherDrag;
	TArrayCollectionArray<T> MaxLinearSpeedsSq;
	TArrayCollectionArray<T> MaxAngularSpeedsSq;
	TArrayCollectionArray<FRealSingle> MInitialOverlapDepenetrationVelocity;
	TArrayCollectionArray<FRealSingle> MSleepThresholdMultiplier;
	TArrayCollectionArray<TUniquePtr<TBVHParticles<T, d>>> MCollisionParticles;
	TArrayCollectionArray<ESleepType> MSleepType;
	TArrayCollectionArray<int8> MSleepCounter;
	TArrayCollectionArray<int8> MDisableCounter;
	TArrayCollectionArray<Private::FIterationSettings> MParticleIterationCounts;

	TArray<TSleepData<T, d>> MSleepData;
	FTransactionallySafeRWLock SleepDataLock;
};



template <typename T, int d>
FChaosArchive& operator<<(FChaosArchive& Ar, TRigidParticles<T, d>& Particles)
{
	Particles.Serialize(Ar);
	return Ar;
}

}
