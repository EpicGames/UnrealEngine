// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Plane.h"
#include "Chaos/Utilities.h"

namespace Chaos
{
	// The feature type returned by SATPenetration
	enum class ESATFeatureType
	{
		None,
		Plane,
		Edge,
		Vertex,
	};

	// The results from SATPenetration
	struct FSATResult
	{
		FSATResult()
			: FeatureTypes{ ESATFeatureType::None, ESATFeatureType::None }
			, FeatureIndices{ INDEX_NONE, INDEX_NONE }
			, SignedDistance(TNumericLimits<FReal>::Lowest())
		{}

		bool IsValid() const
		{
			return FeatureTypes[0] != ESATFeatureType::None;	// No need to check feature 1
		}

		bool IsEdgeContact() const
		{
			return (FeatureTypes[0] == ESATFeatureType::Edge);	// No need to check feature 1
		}

		bool IsPlaneContact() const
		{
			return (FeatureTypes[0] == ESATFeatureType::Plane) || (FeatureTypes[0] == ESATFeatureType::Vertex);	// No need to check feature 1
		}

		FSATResult& SwapShapes()
		{
			Swap(FeatureTypes[0], FeatureTypes[1]);
			Swap(FeatureIndices[0], FeatureIndices[1]);
			return *this;
		}

		ESATFeatureType FeatureTypes[2];
		int32 FeatureIndices[2];
		FReal SignedDistance;
	};

	// Parameters for SATPenetartion
	struct FSATSettings
	{
		FSATSettings()
			: PlaneBias(0)
			, ObjectBias(0)
		{}

		// Bias to select Plane-Vertex contacts over Edge-Edge contacts with similar separation
		FReal PlaneBias;

		// Bias to select first (+ve) or second (-ve) object as the Plane owner when both report similar Plane-Vertex distances
		FReal ObjectBias;
	};

	// Check whether the two edges of two convex shapes contribute to the Minkowski sum.
	// A and B are the face normals for the faces of the edge convex 1
	// C and D are the negated face normals for the faces of the edge convex 2
	inline bool IsMinkowskiSumFace(const FVec3& A, const FVec3& B, const FVec3& C, const FVec3& D)
	{
		const FVec3 BA = FVec3::CrossProduct(B, A);
		const FVec3 DC = FVec3::CrossProduct(D, C);
		const FReal CBA = FVec3::DotProduct(C, BA);
		const FReal DBA = FVec3::DotProduct(D, BA);
		const FReal ADC = FVec3::DotProduct(A, DC);
		const FReal BDC = FVec3::DotProduct(B, DC);

		const FReal Tolerance = 1.e-2f;
		return ((CBA * DBA) < -Tolerance) && ((ADC * BDC) < -Tolerance) && ((CBA * BDC) > Tolerance);
	}

	// Find the nearest Plane-Vertex pair by looking at the Vertices of Convex1 and the Planes of Convex2
	template <typename ConvexImplicitType1, typename ConvexImplicitType2>
	FSATResult SATPlaneVertex(
		const ConvexImplicitType1& Convex1,
		const FRigidTransform3& Convex1Transform,
		const ConvexImplicitType2& Convex2,
		const FRigidTransform3& Convex2Transform,
		const FReal CullDistance)
	{
		FSATResult Result;

		const FRigidTransform3 Convex2ToConvex1Transform = Convex2Transform.GetRelativeTransformNoScale(Convex1Transform);

		const int32 NumPlanes2 = Convex2.NumPlanes();
		for (int32 PlaneIndex2 = 0; PlaneIndex2 < NumPlanes2; ++PlaneIndex2)
		{
			const TPlaneConcrete<FReal, 3> Plane2 = Convex2.GetPlane(PlaneIndex2);
			const FVec3 PlaneN2In1 = Convex2ToConvex1Transform.TransformVectorNoScale(Plane2.Normal());
			const FVec3 PlaneX2In1 = Convex2ToConvex1Transform.TransformPositionNoScale(Plane2.X());

			FReal NearestVertexDistance = TNumericLimits<FReal>::Max();
			int32 NearestVertexIndex1 = INDEX_NONE;

			// @todo(chaos): Use support method that returns vertex index. Use hill climbing
			const int32 NumVertices1 = Convex1.NumVertices();
			for (int32 VertexIndex1 = 0; VertexIndex1 < NumVertices1; ++VertexIndex1)
			{
				const FVec3 VertexX1 = Convex1.GetVertex(VertexIndex1);
				const FReal VertexDistance = FVec3::DotProduct(VertexX1 - PlaneX2In1, PlaneN2In1);
				if (VertexDistance < NearestVertexDistance)
				{
					NearestVertexDistance = VertexDistance;
					NearestVertexIndex1 = VertexIndex1;
				}
			}

			// We can stop if all verts are farther than CullDistance from any plane
			if (NearestVertexDistance > CullDistance)
			{
				return FSATResult();
			}

			// Is this the new best separating axis?
			if (NearestVertexDistance > Result.SignedDistance)
			{
				Result.FeatureTypes[0] = ESATFeatureType::Vertex;
				Result.FeatureTypes[1] = ESATFeatureType::Plane;
				Result.FeatureIndices[0] = NearestVertexIndex1;
				Result.FeatureIndices[1] = PlaneIndex2;
				Result.SignedDistance = NearestVertexDistance;
			}
		}

		return Result;
	}

	// Find the nearest Edge-Edge pair that contributes to the minkowski surface
	template <typename ConvexImplicitType1, typename ConvexImplicitType2>
	FSATResult SATEdgeEdge(
		const ConvexImplicitType1& Convex1,
		const FRigidTransform3& Convex1Transform,
		const ConvexImplicitType2& Convex2,
		const FRigidTransform3& Convex2Transform,
		const FReal CullDistance)
	{
		FSATResult Result;

		const FRigidTransform3 Convex2ToConvex1Transform = Convex2Transform.GetRelativeTransformNoScale(Convex1Transform);

		// Center of the convex shape, used to enforce correct normal direction
		const FVec3 Centroid2 = Convex2.GetCenterOfMass();
		const FVec3 Centroid2In1 = Convex2ToConvex1Transform.TransformPositionNoScale(Centroid2);

		const int32 NumEdges1 = Convex1.NumEdges();
		const int32 NumEdges2 = Convex2.NumEdges();

		// Loop over the edges in Convex2
		for (int32 EdgeIndex2 = 0; EdgeIndex2 < NumEdges2; ++EdgeIndex2)
		{
			// Edge Vertices in other object space
			const int32 EdgeVertexIndex2A = Convex2.GetEdgeVertex(EdgeIndex2, 0);
			const int32 EdgeVertexIndex2B = Convex2.GetEdgeVertex(EdgeIndex2, 1);
			const FVec3 EdgeVertex2AIn1 = Convex2ToConvex1Transform.TransformPositionNoScale(FVector(Convex2.GetVertex(EdgeVertexIndex2A)));
			const FVec3 EdgeVertex2BIn1 = Convex2ToConvex1Transform.TransformPositionNoScale(FVector(Convex2.GetVertex(EdgeVertexIndex2B)));

			// Planes that use the edge
			const int32 EdgePlaneIndex2A = Convex2.GetEdgePlane(EdgeIndex2, 0);
			const int32 EdgePlaneIndex2B = Convex2.GetEdgePlane(EdgeIndex2, 1);
			const FVec3 EdgePlaneNormal2AIn1 = Convex2ToConvex1Transform.TransformVectorNoScale(Convex2.GetPlane(EdgePlaneIndex2A).Normal());
			const FVec3 EdgePlaneNormal2BIn1 = Convex2ToConvex1Transform.TransformVectorNoScale(Convex2.GetPlane(EdgePlaneIndex2B).Normal());

			// Loop over the edges in Convex1
			for (int32 EdgeIndex1 = 0; EdgeIndex1 < NumEdges1; ++EdgeIndex1)
			{
				// Edge Vertices
				const int32 EdgeVertexIndex1A = Convex1.GetEdgeVertex(EdgeIndex1, 0);
				const int32 EdgeVertexIndex1B = Convex1.GetEdgeVertex(EdgeIndex1, 1);
				const FVec3 EdgeVertex1A = Convex1.GetVertex(EdgeVertexIndex1A);
				const FVec3 EdgeVertex1B = Convex1.GetVertex(EdgeVertexIndex1B);

				// Planes that use the edge
				const int32 EdgePlaneIndex1A = Convex1.GetEdgePlane(EdgeIndex1, 0);
				const int32 EdgePlaneIndex1B = Convex1.GetEdgePlane(EdgeIndex1, 1);
				const FVec3 EdgePlaneNormal1A = Convex1.GetPlane(EdgePlaneIndex1A).Normal();
				const FVec3 EdgePlaneNormal1B = Convex1.GetPlane(EdgePlaneIndex1B).Normal();

				// Does this edge pair contribute to the Minkowski sum?
				if (!IsMinkowskiSumFace(EdgePlaneNormal1A, EdgePlaneNormal1B, -EdgePlaneNormal2AIn1, -EdgePlaneNormal2BIn1))
				{
					continue;
				}

				// Separating normal (always points away from convex 2)
				// @todo(chaos): we can perform the distance culling with a non-normalized axis and defer the sqrt
				FVec3 EdgeNormal = FVec3::CrossProduct(EdgeVertex1B - EdgeVertex1A, EdgeVertex2BIn1 - EdgeVertex2AIn1);
				if (!Utilities::NormalizeSafe(EdgeNormal))
				{
					continue;
				}
				if (FVec3::DotProduct(EdgeNormal, EdgeVertex2AIn1 - Centroid2In1) < 0)
				{
					EdgeNormal = -EdgeNormal;
				}

				// Signed separating distance
				const FReal EdgeDistance = FVec3::DotProduct(EdgeVertex1A - EdgeVertex2AIn1, EdgeNormal);

				// We can stop if any edge pair on the Minkowski surface is a separating axis
				if (EdgeDistance > CullDistance)
				{
					return FSATResult();
				}

				// Should we use this edge pair?
				if (EdgeDistance > Result.SignedDistance)
				{
					Result.FeatureTypes[0] = ESATFeatureType::Edge;
					Result.FeatureTypes[1] = ESATFeatureType::Edge;
					Result.FeatureIndices[0] = EdgeIndex1;
					Result.FeatureIndices[1] = EdgeIndex2;
					Result.SignedDistance = EdgeDistance;
				}
			}
		}

		return Result;
	}

	// Separating Axis Test
	// Find the pair of features with the minimum separation distance or the minimum depenetration distance
	template <typename ConvexImplicitType1, typename ConvexImplicitType2>
	FSATResult SATPenetration(
		const ConvexImplicitType1& Convex1,
		const FRigidTransform3& Convex1Transform,
		const ConvexImplicitType2& Convex2,
		const FRigidTransform3& Convex2Transform,
		const FReal CullDistance,
		const FSATSettings& Settings)
	{
		// Find the closest vertex of Convex1 to a plane of Convex2
		FSATResult PlaneResult1 = SATPlaneVertex(Convex1, Convex1Transform, Convex2, Convex2Transform, CullDistance);
		if (PlaneResult1.SignedDistance > CullDistance)
		{
			return FSATResult();
		}

		// Find the closest vertex of Convex2 to a plane of Convex1
		FSATResult PlaneResult2 = SATPlaneVertex(Convex2, Convex2Transform, Convex1, Convex1Transform, CullDistance).SwapShapes();
		if (PlaneResult2.SignedDistance > CullDistance)
		{
			return FSATResult();
		}

		// Find the closest edge pair
		FSATResult EdgeResult = SATEdgeEdge(Convex1, Convex1Transform, Convex2, Convex2Transform, CullDistance);
		if (EdgeResult.SignedDistance > CullDistance)
		{
			return FSATResult();
		}

		// Select the best contact. Prefer face contacts to edge contacts (for +ve bias)
		const FReal MaxPlaneDistance = FMath::Max(PlaneResult1.SignedDistance, PlaneResult2.SignedDistance);
		const bool bUseEdgeResult = (EdgeResult.SignedDistance > MaxPlaneDistance + Settings.PlaneBias);
		if (bUseEdgeResult)
		{
			return EdgeResult;
		}

		// Prefer planes on Convex2 over Convex1 (for +ve bias, and vice-versa for -ve bias) to prevent flip-flopping
		const bool bUsePlaneResult1 = (PlaneResult1.SignedDistance > PlaneResult2.SignedDistance - Settings.ObjectBias);
		if (bUsePlaneResult1)
		{
			return PlaneResult1;
		}

		return PlaneResult2;
	}

}