// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Array.h"
#include "Chaos/Map.h"
#include "Chaos/Vector.h"

namespace Chaos
{
	template<class T, int d>
	class TParticles;

	/**
	* Mesh structure of connected particles via edges.
	*/
	class FSegmentMesh
	{
	public:
		FSegmentMesh()
		{}
		CHAOS_API FSegmentMesh(TArray<TVec2<int32>>&& Elements);
		FSegmentMesh(const FSegmentMesh& Other) = delete;
		FSegmentMesh(FSegmentMesh&& Other)
		    : MElements(MoveTemp(Other.MElements))
		    , MPointToEdgeMap(MoveTemp(Other.MPointToEdgeMap))
		    , MPointToNeighborsMap(MoveTemp(Other.MPointToNeighborsMap))
		{}
		CHAOS_API ~FSegmentMesh();

		CHAOS_API void Init(const TArray<TVec2<int32>>& Elements);
		CHAOS_API void Init(TArray<TVec2<int32>>&& Elements);

		int32 GetNumElements() const
		{
			return MElements.Num();
		}

		const TArray<TVec2<int32>>& GetElements() const
		{
			return MElements;
		}

		/**
		 * @ret The set of neighbor nodes, or nullptr if \p index is not found.
		*/
		const TSet<int32>* GetNeighbors(const int32 index) const
		{
			return GetPointToNeighborsMap().Find(index);
		}

		/**
		* @ret A map of each point index to the list of connected points.
		*/
		CHAOS_API const TMap<int32, TSet<int32>>& GetPointToNeighborsMap() const;

		/**
		* @ret A map of each point index to the list of connected edges.
		*/
		CHAOS_API const TMap<int32, TArray<int32>>& GetPointToEdges() const;

		/**
		* @ret Lengths (or lengths squared) of all edges.
		* @param InParticles - The particle positions to use.  This routine assumes it's sized appropriately.
		* @param lengthSquared - If true, the squared length is returned, which is faster.
		*/
		CHAOS_API TArray<FReal> GetEdgeLengths(
			const TParticles<FReal, 3>& InParticles, 
			const bool lengthSquared = false) const;

	private:
		CHAOS_API void _ClearAuxStructures();

		CHAOS_API void _UpdatePointToNeighborsMap() const;

		CHAOS_API void _UpdatePointToEdgesMap() const;

	private:
		// We use TVector rather than FEdge to represent connectivity because
		// sizeof(TVec2<int32>) < sizeof(FEdge).  FEdge has an extra int32
		// member called Count, which we don't currently have a use for.
		TArray<TVec2<int32>> MElements;

		// Members are mutable so they can be generated on demand by const API.
		mutable TMap<int32, TArray<int32>> MPointToEdgeMap;
		mutable TMap<int32, TSet<int32>> MPointToNeighborsMap;
	};

	template <typename T>
	using TSegmentMesh = FSegmentMesh;

} // namespace Chaos
