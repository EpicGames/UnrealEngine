// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace Chaos
{
template <typename T>
class TSerializablePtr
{
public:
	TSerializablePtr() : Ptr(nullptr) {}
	template <typename R>
	explicit TSerializablePtr(const TUniquePtr<R>& Unique) : Ptr(Unique.Get()) {}
	explicit TSerializablePtr(TUniquePtr<T>&& Unique) = delete;

	//template <typename R>
	//TSerializablePtr(const TSerializablePtr<R>& Other) : Ptr(Other.Get()) {}
	
	template<ESPMode TESPMode>
	explicit TSerializablePtr(const TSharedPtr<T, TESPMode>& Shared) : Ptr(Shared.Get()) {}
	
	explicit TSerializablePtr(const TRefCountPtr<T>& RefCount) : Ptr(RefCount.GetReference()) {}
	
	const T* operator->() const { return Ptr; }
	const T* Get() const { return Ptr; }
	const T* GetReference() const { return Ptr; }
	const T& operator*() const { return *Ptr; }
	void Reset() { Ptr = nullptr; }
	bool operator!() const { return Ptr == nullptr; }
	bool operator==(const TSerializablePtr<T>& Serializable) const { return Ptr == Serializable.Ptr; }
	bool operator!=(const TSerializablePtr<T>& Serializable) const { return Ptr != Serializable.Ptr; }
	operator bool() const { return Ptr != nullptr; }
	bool IsValid() const { return Ptr != nullptr; }

	template <typename R>
	operator TSerializablePtr<R>() const
	{
		const R* RCast = Ptr;
		TSerializablePtr<R> Ret;
		Ret.SetFromRawLowLevel(RCast);
		return Ret;
	}

	//NOTE: this is needed for serialization. This should NOT be used directly
	void SetFromRawLowLevel(const T* InPtr)
	{
		Ptr = InPtr;
	}

private:
	TSerializablePtr(T* InPtr) : Ptr(InPtr) {}
	const T* Ptr;
};

template <typename T>
inline uint32 GetTypeHash(const TSerializablePtr<T>& Ptr) { return ::GetTypeHash(Ptr.Get()); }

template <typename T>
TSerializablePtr<T> MakeSerializable(const TUniquePtr<T>& Unique)
{
	return TSerializablePtr<T>(Unique);
}

template <typename Ret, typename T>
TSerializablePtr<Ret> MakeSerializable(const TUniquePtr<T>& Unique)
{
	return TSerializablePtr<Ret>(Unique);
}

template <typename T>
TSerializablePtr<T> MakeSerializable(const TSerializablePtr<T>& P)
{
	return P;
}

template <typename T>
TSerializablePtr<T> MakeSerializable(const TUniquePtr<T>&& Unique) = delete;

template <typename Ret, typename T>
TSerializablePtr<T> MakeSerializable(const TUniquePtr<T>&& Unique) = delete;

template<typename T, ESPMode TESPMode>
TSerializablePtr<T> MakeSerializable(const TSharedPtr<T, TESPMode>& Shared)
{
	return TSerializablePtr<T>(Shared);
}
	
template<typename T>
TSerializablePtr<T> MakeSerializable(const TRefCountPtr<T>& RefCount)
{
	return TSerializablePtr<T>(RefCount);
}

//This is only available for types that are guaranteed to be serializable. This is done by having a factory that returns unique pointers for example
template <typename T>
typename TEnableIf<T::AlwaysSerializable, TSerializablePtr<T>>::Type& AsAlwaysSerializable(T*& Ptr)
{
	return reinterpret_cast<TSerializablePtr<T>&>(Ptr);
}


template <typename T>
typename TEnableIf<T::AlwaysSerializable, TArray<TSerializablePtr<T>>>::Type& AsAlwaysSerializableArray(TArray<T*>& Ptrs)
{
	return reinterpret_cast<TArray<TSerializablePtr<T>>&>(Ptrs);
}

class FChaosArchive;

}