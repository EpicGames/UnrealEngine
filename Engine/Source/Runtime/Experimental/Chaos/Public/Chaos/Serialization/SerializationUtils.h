// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/ParticleHandleFwd.h"
#include "Containers/Array.h"

namespace Chaos
{
	class FConstraintHandle;
}

namespace Chaos
{
	namespace Serialization::Private
	{
		/** Lightweight unversioned ustruct serializer */
		UE_INTERNAL CHAOS_API void FastStructSerialize(UScriptStruct* Struct, void* SourceData, FArchive& Ar, void* Defaults = nullptr);

		/** Lightweight unversioned ustruct serializer */
		template<typename StructType>
		UE_INTERNAL void FastStructSerialize(FArchive& Ar, StructType* Data)
		{
			StructType DefaultStruct = StructType();
			FastStructSerialize(StructType::StaticStruct(), reinterpret_cast<void*>(Data), Ar, &DefaultStruct);
		}
	}
}
