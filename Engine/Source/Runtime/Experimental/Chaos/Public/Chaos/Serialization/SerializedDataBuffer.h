// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Array.h"
#include "Serialization/Archive.h"
#include "Templates/SharedPointer.h"

namespace Chaos
{
	/** Wrapper around a data buffer used for serialization of chaos data */
	struct FSerializedDataBuffer
	{
		FSerializedDataBuffer(const TConstArrayView<uint8> ConstView) : DataBuffer(ConstView.GetData(), ConstView.Num())
		{	
		}

		FSerializedDataBuffer(TArray<uint8>&& InByteArray) : DataBuffer(MoveTemp(InByteArray))
		{	
		}

		FSerializedDataBuffer(const TArray<uint8>& InByteArray) : DataBuffer(InByteArray)
		{	
		}

		FSerializedDataBuffer()
		{
		}

		TArray<uint8>& GetDataAsByteArrayRef()
		{
			return DataBuffer;
		}

		int32 GetSize() const
		{
			return DataBuffer.Num();
		}

		void Serialize(FArchive& Ar)
		{
			Ar << DataBuffer;
		}

	private:
		TArray<uint8> DataBuffer;
	};

	inline FArchive& operator<<(FArchive& Ar, FSerializedDataBuffer& Data)
	{
		Data.Serialize(Ar);
		return Ar;
	}
}
