// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Array.h"
#include "SolverSerializer.h"
#include "Serialization/Archive.h"

namespace Chaos
{
	struct UE_INTERNAL FSerializedMultiPhysicsState
	{
		FSerializedMultiPhysicsState()
		{
		}

		FSerializedMultiPhysicsState(ESerializedDataContext DataContext)
		{
			Header.DataContext = DataContext;
		}

		CHAOS_API void ReadElementDataIntoBuffer(FSerializedDataBuffer& TargetBuffer);

		template<typename ElementType>
		void AppendDataFromElement(ElementType& InElement)
		{
			if (Header.DataContext == ESerializedDataContext::Both)
			{
				AppendDataFromElement_Internal(InElement, ESerializedDataContext::Internal);
				AppendDataFromElement_Internal(InElement, ESerializedDataContext::External);
				return;
			}
			
			AppendDataFromElement_Internal(InElement, Header.DataContext);
		}

		template<typename ElementType>
		void ApplyDataToElement(ElementType& InElement)
		{
			if (Header.DataContext == ESerializedDataContext::Both)
			{
				ApplyDataToElement_Internal(InElement, ESerializedDataContext::Internal);
				ApplyDataToElement_Internal(InElement, ESerializedDataContext::External);
				return;
			}
			
			ApplyDataToElement_Internal(InElement, Header.DataContext);
		}

	private:
		template<typename ElementType>
		void AppendDataFromElement_Internal(ElementType& InElement, ESerializedDataContext DataContext)
		{
			FDataEntryTag DataTag;
			DataTag.DataOffset = GetSize();
			
			InElement.SerializeState(MigratedStateAsBytes, DataContext);

			DataTag.DataSize = GetSize() - DataTag.DataOffset;

			Header.DataTagPerElementIndex.Emplace(DataTag);
		}

		template<typename ElementType>
		void ApplyDataToElement_Internal(ElementType& InElement, ESerializedDataContext DataContext)
		{
			FSerializedDataBufferPtr DataBuffer = MakeUnique<FSerializedDataBuffer>();
			ReadElementDataIntoBuffer(*DataBuffer);
			InElement.ApplySerializedState(MoveTemp(DataBuffer), DataContext);
		}

	public:

		CHAOS_API void Serialize(FArchive& Ar);

		int32 GetSize() const
		{
			return MigratedStateAsBytes.GetSize();
		}

	private:

		FSerializedDataBuffer MigratedStateAsBytes;

		struct FDataEntryTag
		{
			int32 DataOffset = INDEX_NONE;
			int32 DataSize = INDEX_NONE;

			friend FArchive& operator<<(FArchive& Ar, FDataEntryTag& Data)
			{
				Ar << Data.DataOffset;
				Ar << Data.DataSize;
				return Ar;
			}

			bool IsValid() const
			{
				return DataOffset != INDEX_NONE && DataSize != INDEX_NONE;
			}
		};
	public:
		struct FHeader
		{
			ESerializedDataContext DataContext = Chaos::ESerializedDataContext::Invalid;
			int32 NumBodies = 0;
			int32 NumConstraints = 0;

			TArray<FDataEntryTag> DataTagPerElementIndex;
				
			friend FArchive& operator<<(FArchive& Ar, FHeader& Data)
			{
				Ar << Data.DataContext;
				Ar << Data.NumBodies;
				Ar << Data.NumConstraints;
				Ar << Data.DataTagPerElementIndex;

				return Ar;
			}

			bool IsValid() const
			{
				const int32 CurrentDataElementsNum = DataTagPerElementIndex.Num();
				const int32 CurrentBodiesAndConstraintsNum = NumBodies + NumConstraints;
				const int32 ExpectedDataElementsNum = DataContext == ESerializedDataContext::Both ? CurrentBodiesAndConstraintsNum * 2 : CurrentBodiesAndConstraintsNum;
				return DataContext != ESerializedDataContext::Invalid && CurrentDataElementsNum == ExpectedDataElementsNum;
			}
		};

		void SetNumOfBodies(int32 NumBodies)
		{
			Header.NumBodies = NumBodies;
		}

		void SetNumOfConstraints(int32 NumConstraints)
		{
			Header.NumConstraints = NumConstraints;
		}

		const FHeader& GetHeader() const
		{
			return Header;
		}

	private:
		FHeader Header;
		int32 CurrentReadElementIndex = 0;
	};
}


