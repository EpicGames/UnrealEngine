// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/Serialization/SerializedDataBuffer.h"
#include "Containers/StripedMap.h"
#include "UObject/ObjectMacros.h"

struct FChaosVDJointConstraint;
struct FChaosVDParticleDataWrapper;
class IPhysicsProxyBase;

namespace Chaos
{
	class FConstraintBase;
	class FConstraintHandle;
	class FJointConstraint;
	class FPBDJointConstraintHandle;
	class FPBDRigidsSolver;
	struct FSerializedDataBufferHandle;

	typedef TUniquePtr<FSerializedDataBuffer> FSerializedDataBufferPtr;

	/** Enum used to indicate where we should read/write the data in serialize request */
	enum class ESerializedDataContext : uint8
	{
		Invalid,
		/** The data is owned by the solver */
		Internal,
		/** The data is owned by the game thread */
		External,
		/** Serialize the data from both, the GT and PT*/
		Both
	};
	
	CHAOS_API const TCHAR* LexToString(ESerializedDataContext Value);

	/* Object capable of serializing total or partially a rigid solver instance **/
	class FSolverSerializer
	{
	public:
	
		FSolverSerializer(FPBDRigidsSolver* InSolver);

		void SerializeToBuffer(FSerializedDataBuffer& OutSerializedData);
		void PopulateFromSerializedBuffer(const FSerializedDataBuffer& InSerializedData);

		CHAOS_API void SerializeParticleStateToBuffer(FGeometryParticleHandle* InParticleHandle, FSerializedDataBuffer& OutSerializedData);
		CHAOS_API void SerializeParticleStateToBuffer(FGeometryParticle* GTParticle, FSerializedDataBuffer& OutSerializedData);

		CHAOS_API void SerializeConstraintStateToBuffer(FConstraintHandle* ConstraintHandlePtr, FSerializedDataBuffer& OutSerializedData);
		CHAOS_API void SerializeConstraintStateToBuffer(FConstraintBase* GTConstraintPtr, FSerializedDataBuffer& OutSerializedData);

		CHAOS_API void ApplySerializedStateToParticle(FGeometryParticleHandle* InParticleHandle, FSerializedDataBuffer& InSerializedData);
		CHAOS_API void ApplySerializedStateToParticle(FGeometryParticleHandle* InParticleHandle, const FChaosVDParticleDataWrapper& InParticleState);

		CHAOS_API void ApplySerializedStateToParticle(FGeometryParticle* GTParticle, FSerializedDataBuffer& InSerializedData);
		CHAOS_API void ApplySerializedStateToParticle(FGeometryParticle* GTParticle, const FChaosVDParticleDataWrapper& InParticleState);

		CHAOS_API void ApplySerializedStateToJointConstraint(FPBDJointConstraintHandle* ConstraintHandlePtr, FSerializedDataBuffer& InSerializedData);
		CHAOS_API void ApplySerializedStateToJointConstraint(FJointConstraint* ConstraintPtr, FSerializedDataBuffer& InSerializedData);

		CHAOS_API void ApplySerializedStateToConstraint(FConstraintHandle* ConstraintHandlePtr, FSerializedDataBuffer& InSerializedData);
		CHAOS_API void ApplySerializedStateToConstraint(FConstraintBase* ConstraintPtr, FSerializedDataBuffer& InSerializedData);

		CHAOS_API void PushPendingInternalSerializedStateForProxy(IPhysicsProxyBase* Proxy, FSerializedDataBufferPtr&& InState);
		CHAOS_API FSerializedDataBufferPtr PopPendingInternalSerializedStateForProxy(IPhysicsProxyBase* Proxy);


	private:
		FPBDRigidsSolver* SolverInstance;

		static constexpr int32 StripeCount = 32;
		TStripedMap<StripeCount, IPhysicsProxyBase*, FSerializedDataBufferPtr> PendingMigratedPhysicsStateByProxy;
	};
}

