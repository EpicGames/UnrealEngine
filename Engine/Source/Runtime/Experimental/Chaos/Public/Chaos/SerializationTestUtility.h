// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ChaosArchive.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/CustomVersion.h"
#include "HAL/FileManager.h"

#if UE_BUILD_SHIPPING == 0
namespace Chaos
{
	/**
	 * Serializes and loads ObjectToSave to memory and to disk, returning loaded versions in array for testing.
	 *   also loads and returns all binaries in test's subdirectory in SerializedBinaryDirectory. Used to test backwards compatibility for previous serialization formats.
	 * @param ObjectToSave - Data being tested. Will be saved and loaded, loaded copies returned for testing.
	 * @param SerializedBinaryDirectory - Path to directory containing subfolders containing binaries to load for testing.
	 * @param BinaryFolderName - Name of folder in SerializedBinarYDirectory for this test. Should not match name of other tests.
	 * @param bSave - If true, ObjectToSave will be saved to SerialziedBinarY folder for testing in future. Should be false, temporarily flip to true to save.
	 * @param ObjectsToTest - Returned Objects that were deserialized.
	 * @return - False if fails to load a binary file, otherwse true. Should fail test on false.
	 */
	template <class T, class U>
	bool SaveLoadUtility(U &ObjectToSave, TCHAR const * SerializedBinaryDirectory, TCHAR const * BinaryFolderName, bool bSave, TArray<U>& ObjectsToTest)
	{
		// Get Test object for memory serialization
		{
			// Write to memory
			TArray<uint8> Data;
			{
				FMemoryWriter Ar(Data);
				FChaosArchive Writer(Ar);

				Writer << ObjectToSave;
			}

			// Read from Memory
			U ReadFromMemory;
			{
				FMemoryReader Ar(Data);
				FChaosArchive Reader(Ar);


				Reader << ReadFromMemory;
			}

			ObjectsToTest.Emplace(MoveTemp(ReadFromMemory));
		}

#if 0	// TODO: Fix archive level versioning? Remove?

		// Get files from binary folder.
		TArray<FString> FilesFound;
		IFileManager &FileManager = IFileManager::Get();
		FString TestBinaryFolder = FString(SerializedBinaryDirectory) / BinaryFolderName;
		FileManager.FindFiles(FilesFound, *TestBinaryFolder);

		// Add binaries in folder to results
		{
			for (FString File : FilesFound)
			{
				U ReadFromDisk;

				File = TestBinaryFolder / File;
				FArchive *Ar = FileManager.CreateFileReader(*File, FILEREAD_None);
				if (Ar)
				{
					TArray<uint8> Data;
					FMemoryReader MemoryAr(Data);
					FChaosArchive Reader(MemoryAr);

					// Retrieve version and set on archive.
					FCustomVersionContainer Version;
					Version.Serialize(*Ar);
					Reader.SetCustomVersions(Version);

					// Retrieve memory blob from disk.
					*Ar << Data;

					// Serialize into object.
					Reader << ReadFromDisk;
					Ar->Close();
				}
				else
				{
					// Fail on failing to read test binaries.
					return false;
				}

				ObjectsToTest.Emplace(MoveTemp(ReadFromDisk));
			}
		}

		if(bSave)
		{
			FString FilePath = TestBinaryFolder / BinaryFolderName;
			FilePath.AppendInt(FilesFound.Num());
			FilePath.Append(TEXT(".bin"));

			FArchive *Ar = FileManager.CreateFileWriter(*FilePath, FILEWRITE_None);
			if (Ar)
			{
				TArray<uint8> Data;
				FMemoryWriter MemoryAr(Data);
				FChaosArchive Writer(MemoryAr);


				// Serialize into Data array first, then serialize that to disk.
				// Cannot use FArchive directly as FChaosArchive's proxy archive because it does not implement
				// some required serialization functions. (FMemoryArchive does implement these).


				// Serialize object to memory buffer
				Writer << ObjectToSave;

				// Get version from serialization, and serialize to file archive.
				FCustomVersionContainer Version = Writer.GetCustomVersions();
				Version.Serialize(*Ar);

				// Serialize object from memory buffer to file archive.
				*Ar << Data;
				Ar->Close();
			}
		}
#endif

		return true;
	}
}

#endif
