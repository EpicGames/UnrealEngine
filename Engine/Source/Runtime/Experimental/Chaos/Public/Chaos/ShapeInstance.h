// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

#include "Chaos/CollisionFilterData.h"
#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ParticleDirtyFlags.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PhysicalMaterials.h"
#include "Chaos/Properties.h"
#include "Chaos/Serializable.h"
#include "Chaos/ShapeInstanceFwd.h"

namespace Chaos
{
	class FImplicitObject;
	class FShapeInstance;
	class FShapeInstanceProxy;

	namespace Private
	{
		class FShapeInstanceExtended;
	}


	/**
	 * FPerShapeData is going to be deprecated. See FShapeInstance and FShapeInstanceProxy
	 * 
	 * @todo(chaos): 
	 * - change ShapesArray() and all code using it to use ShapeInstance or ShapeInstanceProxy as appropriate
	 * - deprecate FPerShapeData
	 */
	class FPerShapeData
	{
	protected:
		enum class EPerShapeDataType : uint32
		{
			Proxy,
			Sim,
			SimExtended,
		};

		EPerShapeDataType GetType() const
		{
			return Type;
		}

		// Call a function on the concrete type
		template<typename TLambda> decltype(auto) DownCast(const TLambda& Lambda);
		template<typename TLambda> decltype(auto) DownCast(const TLambda& Lambda) const;

		Private::FShapeInstanceExtended* AsShapeInstanceExtended();
		CHAOS_API const Private::FShapeInstanceExtended* AsShapeInstanceExtended() const;

	public:
		// Downcasts exposed until we deprecate and remove FPerShapeData
		FShapeInstanceProxy* AsShapeInstanceProxy();
		const FShapeInstanceProxy* AsShapeInstanceProxy() const;
		FShapeInstance* AsShapeInstance();
		const FShapeInstance* AsShapeInstance() const;

	public:
		static constexpr bool AlwaysSerializable = true;

		UE_DEPRECATED(5.3, "Not used")
		static bool RequiresCachedLeafInfo(const FImplicitObject* Geometry) { return false; }

		UE_DEPRECATED(5.3, "Call FShapeInstanceProxy::Make for game thread objects, FShapeInstance::Make for physics thread objects")
		static CHAOS_API TUniquePtr<FPerShapeData> CreatePerShapeData(int32 InShapeIdx, TSerializablePtr<FImplicitObject> InGeometry);

		UE_DEPRECATED(5.3, "Call FShapeInstanceProxy::UpdateGeometry for game thread objects, FShapeInstance::UpdateGeometry for physics thread objects")
		static CHAOS_API void UpdateGeometry(TUniquePtr<FPerShapeData>& InOutShapePtr, TSerializablePtr<FImplicitObject> InGeometry);


		static CHAOS_API FPerShapeData* SerializationFactory(FChaosArchive& Ar, FPerShapeData*);

		virtual ~FPerShapeData() = default;

		CHAOS_API virtual void Serialize(FChaosArchive& Ar);

		void UpdateShapeBounds(const FRigidTransform3& WorldTM);

		UE_DEPRECATED(5.4, "Bounds no longer expanded. Use UpdateShapeBounds without BoundsExpansion")
		void UpdateShapeBounds(const FRigidTransform3& WorldTM, const FVec3& BoundsExpansion) { UpdateShapeBounds(WorldTM); }

		void* GetUserData() const;
		void SetUserData(void* InUserData);

		UE_DEPRECATED(5.8, "Use GetCombinedShapeFilterData instead")  FCollisionFilterData GetQueryData() const;
		UE_DEPRECATED(5.8, "Use SetCombinedShapeFilterData instead")  void SetQueryData(const FCollisionFilterData& InQueryData);

		UE_DEPRECATED(5.8, "Use GetCombinedShapeFilterData instead") FCollisionFilterData GetSimData() const;
		UE_DEPRECATED(5.8, "Use SetCombinedShapeFilterData instead") void SetSimData(const FCollisionFilterData& InSimData);

		CHAOS_API const Filter::FShapeFilterData GetShapeFilterData() const;
		CHAOS_API void SetShapeFilterData(const Filter::FShapeFilterData& InShapeFilter);
		CHAOS_API const Filter::FInstanceData GetFilterInstanceData() const;
		CHAOS_API void SetFilterInstanceData(const Filter::FInstanceData& InInstanceData);
		CHAOS_API const Filter::FCombinedShapeFilterData GetCombinedShapeFilterData() const;
		CHAOS_API void SetCombinedShapeFilterData(const Filter::FCombinedShapeFilterData& InCombinedShapeFilter);

		FImplicitObjectRef GetGeometry() const;

		const TAABB<FReal, 3>& GetWorldSpaceShapeBounds() const;

		void UpdateWorldSpaceState(const FRigidTransform3& WorldTransform);

		UE_DEPRECATED(5.4, "Bounds no longer expanded. Use GetWorldSpaceShapeBounds()")
		const TAABB<FReal, 3>& GetWorldSpaceInflatedShapeBounds() const { return GetWorldSpaceShapeBounds(); }
		
		UE_DEPRECATED(5.4, "Bounds no longer expanded. Use UpdateWorldSpaceState() without BoundsExpansion")
		void UpdateWorldSpaceState(const FRigidTransform3& WorldTransform, const FVec3& BoundsExpansion) { UpdateWorldSpaceState(WorldTransform); }

		// The leaf shape (with transformed and implicit wrapper removed).
		const FImplicitObject* GetLeafGeometry() const;

		// The actor-relative transform of the leaf geometry.
		FRigidTransform3 GetLeafRelativeTransform() const;

		// The world-space transform of the leaf geometry.
		// If we have non-identity leaf relative transform, is cached from the last call to UpdateWorldSpaceState.
		// If not cahced, is constructed from arguments.
		FRigidTransform3 GetLeafWorldTransform(const FGeometryParticleHandle* Particle) const;

		void UpdateLeafWorldTransform(FGeometryParticleHandle* Particle);

		int32 NumMaterials() const;
		const FMaterialHandle& GetMaterial(const int32 Index) const;

		const TArray<FMaterialHandle>& GetMaterials() const;
		void SetMaterial(FMaterialHandle InMaterial);
		void SetMaterials(const TArray<FMaterialHandle>& InMaterials);
		void SetMaterials(TArray<FMaterialHandle>&& InMaterials);

		const TArray<FMaterialMaskHandle>& GetMaterialMasks() const;
		void SetMaterialMasks(const TArray<FMaterialMaskHandle>& InMaterialMasks);
		void SetMaterialMasks(TArray<FMaterialMaskHandle>&& InMaterialMasks);

		const TArray<uint32>& GetMaterialMaskMaps() const;
		void SetMaterialMaskMaps(const TArray<uint32>& InMaterialMaskMaps);
		void SetMaterialMaskMaps(TArray<uint32>&& InMaterialMaskMaps);

		const TArray<FMaterialHandle>& GetMaterialMaskMapMaterials() const;
		void SetMaterialMaskMapMaterials(const TArray<FMaterialHandle>& InMaterialMaskMapMaterials);
		void SetMaterialMaskMapMaterials(TArray<FMaterialHandle>&& InMaterialMaskMapMaterials);

		const FShapeDirtyFlags GetDirtyFlags() const;

		bool GetQueryEnabled() const;
		void SetQueryEnabled(const bool bEnable);

		bool GetSimEnabled() const;
		void SetSimEnabled(const bool bEnable);

		bool GetIsProbe() const;
		void SetIsProbe(const bool bIsProbe);

		EChaosCollisionTraceFlag GetCollisionTraceType() const;
		void SetCollisionTraceType(const EChaosCollisionTraceFlag InTraceFlag);

		const FCollisionData& GetCollisionData() const;
		void SetCollisionData(const FCollisionData& Data);

		const FMaterialData& GetMaterialData() const;
		void SetMaterialData(const FMaterialData& Data);

		void SyncRemoteData(FDirtyPropertiesManager& Manager, int32 ShapeDataIdx, FShapeDirtyData& RemoteData);
		
		void SetProxy(IPhysicsProxyBase* InProxy);
		
		int32 GetShapeIndex() const;
		void ModifyShapeIndex(int32 NewShapeIndex);

		template <typename Lambda>
		UE_DEPRECATED(5.8, "Get/SetCombinedShapeFilterData instead")
		void ModifySimData(const Lambda& LambdaFunc);
		template <typename Lambda> void ModifyMaterials(const Lambda& LambdaFunc);
		template <typename Lambda> void ModifyMaterialMasks(const Lambda& LambdaFunc);
		template <typename Lambda> void ModifyMaterialMaskMaps(const Lambda& LambdaFunc);
		template <typename Lambda> void ModifyMaterialMaskMapMaterials(const Lambda& LambdaFunc);

	protected:
		FPerShapeData(const EPerShapeDataType InType, int32 InShapeIdx)
			: Type(InType)
			, bIsSingleMaterial(false)
			, ShapeIdx(InShapeIdx)
			, Geometry()
			, WorldSpaceShapeBounds(FAABB3(FVec3(0), FVec3(0)))
		{
		}
		
		UE_DEPRECATED(5.4, "Use FPerShapeData with FImplicitObjectPtr instead")
		FPerShapeData(const EPerShapeDataType InType, int32 InShapeIdx, TSerializablePtr<FImplicitObject> InGeometry)
			: Type(InType)
			, bIsSingleMaterial(false)
			, ShapeIdx(InShapeIdx)
			, Geometry()
			, WorldSpaceShapeBounds(FAABB3(FVec3(0), FVec3(0)))
		{
			check(false);
		}

		FPerShapeData(const EPerShapeDataType InType, int32 InShapeIdx, const FImplicitObjectPtr& InGeometry)
			: Type(InType)
			, bIsSingleMaterial(false)
			, ShapeIdx(InShapeIdx)
			, Geometry(InGeometry)
			, WorldSpaceShapeBounds(FAABB3(FVec3(0), FVec3(0)))
		{
		}

		FPerShapeData(const EPerShapeDataType InType, const FPerShapeData& Other)
			: Type(InType)
			, bIsSingleMaterial(Other.bIsSingleMaterial)
			, ShapeIdx(Other.ShapeIdx)
			, Geometry(Other.Geometry)
			, WorldSpaceShapeBounds(Other.WorldSpaceShapeBounds)
		{
		}

		virtual void SerializeMaterials(FChaosArchive& Ar) = 0;

		EPerShapeDataType Type : 2;
		uint32 bIsSingleMaterial : 1;	// For use by FShapeInstance (here because the space is available for free)
		uint32 ShapeIdx : 29;
		FShapeDirtyFlags DirtyFlags;	// For use by FShapeInstanceProxy as there's 4 bytes of padding here
		FImplicitObjectPtr Geometry;
		TAABB<FReal, 3> WorldSpaceShapeBounds;
	};


	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// FShapeInstanceProxy
	// 
	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////

	/*
	 * NOTE: FShapeInstanceProxy is a Game-Thread object. 
	 * See FShapeInstance for the physics-thread equivalent
	 * 
	 * FShapeInstanceProxy contains the per-shape data associated with a single shape on a particle. 
	 * This contains data like the collision / query filters, material properties etc.
	 *
	 * Every particle holds one FShapeInstanceProxy object for each geometry they use.
	 * If the particle has a Union of geometries there will be one FShapeInstanceProxy
	 * for each geometry in the union. (Except ClusteredUnions - these are not flattened
	 * because they contain their own query acceleration structure.)
	 *
	 * NOTE: keep size to a minimum. There can be millions of these in s scene.
	 *
	 * @todo(chaos) : try to remove the GT Proxy pointer - this could easily be passed into the
	 * relevant functions instead.
	 *
	 * @todo(chaos) : reduce the cost of MaterialData for shapes with one materialand no masks etc.
	 */
	class FShapeInstanceProxy : public FPerShapeData
	{
	public:
		friend class FPerShapeData;

		UE_DEPRECATED(5.4, "Use Make with FImplicitObjectPtr instead")
		static TUniquePtr<FShapeInstanceProxy> Make(int32 InShapeIdx, TSerializablePtr<FImplicitObject> InGeometry)
		{
			check(false);
			return nullptr;
		}

		UE_DEPRECATED(5.4, "Use UpdateGeometry with FImplicitObjectPtr instead")
        static void UpdateGeometry(TUniquePtr<FShapeInstanceProxy>& InOutShapePtr, TSerializablePtr<FImplicitObject> InGeometry)
		{
			check(false);
		}

		static CHAOS_API TUniquePtr<FShapeInstanceProxy> Make(int32 InShapeIdx, const FImplicitObjectPtr& InGeometry);
		static CHAOS_API void UpdateGeometry(TUniquePtr<FShapeInstanceProxy>& InOutShapePtr, const FImplicitObjectPtr& InGeometry);
		static CHAOS_API FShapeInstanceProxy* SerializationFactory(FChaosArchive& Ar, FShapeInstanceProxy*);

		CHAOS_API void UpdateShapeBounds(const FRigidTransform3& WorldTM);

		void* GetUserData() const { return CollisionData.Read().UserData; }
		void SetUserData(void* InUserData)
		{
			CollisionData.Modify(true, DirtyFlags, Proxy, ShapeIdx, [InUserData](FCollisionData& Data) { Data.UserData = InUserData; });
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.8, "Use GetCombinedShapeFilterData instead") FCollisionFilterData GetQueryData() const { return CollisionData.Read().GetQueryData(); }
		UE_DEPRECATED(5.8, "Use SetCombinedShapeFilterData instead") void SetQueryData(const FCollisionFilterData& InQueryData)
		{
			CollisionData.Modify(true, DirtyFlags, Proxy, ShapeIdx, [InQueryData](FCollisionData& Data) { Data.SetQueryData(InQueryData); });
		}

		UE_DEPRECATED(5.8, "Use GetCombinedShapeFilterData instead") FCollisionFilterData GetSimData() const { return CollisionData.Read().GetSimData(); }
		UE_DEPRECATED(5.8, "Use SetCombinedShapeFilterData instead") void SetSimData(const FCollisionFilterData& InSimData)
		{
			CollisionData.Modify(true, DirtyFlags, Proxy, ShapeIdx, [InSimData](FCollisionData& Data) { Data.SetSimData(InSimData); });
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		CHAOS_API const Filter::FShapeFilterData GetShapeFilterData() const;
		CHAOS_API void SetShapeFilterData(const Filter::FShapeFilterData& InShapeFilter);
		CHAOS_API const Filter::FInstanceData GetFilterInstanceData() const;
		CHAOS_API void SetFilterInstanceData(const Filter::FInstanceData& InInstanceData);
		CHAOS_API const Filter::FCombinedShapeFilterData GetCombinedShapeFilterData() const;
		CHAOS_API void SetCombinedShapeFilterData(const Filter::FCombinedShapeFilterData& InCombinedShapeFilter);

		CHAOS_API void UpdateWorldSpaceState(const FRigidTransform3& WorldTransform);

		// The leaf shape (with transformed and implicit wrapper removed).
		CHAOS_API const FImplicitObject* GetLeafGeometry() const;

		// The actor-relative transform of the leaf geometry.
		CHAOS_API FRigidTransform3 GetLeafRelativeTransform() const;

		// The world-space transform of the leaf geometry.
		// If we have non-identity leaf relative transform, is cached from the last call to UpdateWorldSpaceState.
		// If not cahced, is constructed from arguments.
		CHAOS_API FRigidTransform3 GetLeafWorldTransform(const FGeometryParticleHandle* Particle) const;
		CHAOS_API void UpdateLeafWorldTransform(FGeometryParticleHandle* Particle);

		int32 NumMaterials() const { return Materials.Read(). Materials.Num(); }
		const FMaterialHandle& GetMaterial(const int32 Index) const { return Materials.Read().Materials[Index]; }

		const TArray<FMaterialHandle>& GetMaterials() const { return Materials.Read().Materials; }
		const TArray<FMaterialMaskHandle>& GetMaterialMasks() const { return Materials.Read().MaterialMasks; }
		const TArray<uint32>& GetMaterialMaskMaps() const { return Materials.Read().MaterialMaskMaps; }
		const TArray<FMaterialHandle>& GetMaterialMaskMapMaterials() const { return Materials.Read().MaterialMaskMapMaterials; }

		const FShapeDirtyFlags GetDirtyFlags() const { return DirtyFlags; }

		void SetMaterial(FMaterialHandle InMaterial)
		{
			Materials.Modify(true, DirtyFlags, Proxy, ShapeIdx, [InMaterial](FMaterialData& Data)
				{
					Data.Materials.Reset(1);
					Data.Materials.Add(InMaterial);
				});
		}

		void SetMaterials(const TArray<FMaterialHandle>& InMaterials)
		{
			Materials.Modify(true, DirtyFlags, Proxy, ShapeIdx, [&InMaterials](FMaterialData& Data)
				{
					Data.Materials = InMaterials;
				});
		}

		void SetMaterials(TArray<FMaterialHandle>&& InMaterials)
		{
			Materials.Modify(true, DirtyFlags, Proxy, ShapeIdx, [&InMaterials](FMaterialData& Data)
				{
					Data.Materials = MoveTemp(InMaterials);
				});
		}

		void SetMaterialMasks(const TArray<FMaterialMaskHandle>& InMaterialMasks)
		{
			Materials.Modify(true, DirtyFlags, Proxy, ShapeIdx, [&InMaterialMasks](FMaterialData& Data)
				{
					Data.MaterialMasks = InMaterialMasks;
				});
		}

		void SetMaterialMaskMaps(const TArray<uint32>& InMaterialMaskMaps)
		{
			Materials.Modify(true, DirtyFlags, Proxy, ShapeIdx, [&InMaterialMaskMaps](FMaterialData& Data)
				{
					Data.MaterialMaskMaps = InMaterialMaskMaps;
				});
		}

		void SetMaterialMaskMapMaterials(const TArray<FMaterialHandle>& InMaterialMaskMapMaterials)
		{
			Materials.Modify(true, DirtyFlags, Proxy, ShapeIdx, [&InMaterialMaskMapMaterials](FMaterialData& Data)
				{
					Data.MaterialMaskMapMaterials = InMaterialMaskMapMaterials;
				});
		}

		bool GetQueryEnabled() const { return CollisionData.Read().bQueryCollision; }
		void SetQueryEnabled(const bool bEnable)
		{
			CollisionData.Modify(true, DirtyFlags, Proxy, ShapeIdx, [bEnable](FCollisionData& Data) { Data.bQueryCollision = bEnable; });
		}

		bool GetSimEnabled() const { return CollisionData.Read().bSimCollision; }
		void SetSimEnabled(const bool bEnable)
		{
			CollisionData.Modify(true, DirtyFlags, Proxy, ShapeIdx, [bEnable](FCollisionData& Data) { Data.bSimCollision = bEnable; });
		}

		bool GetIsProbe() const { return CollisionData.Read().bIsProbe; }
		void SetIsProbe(const bool bIsProbe)
		{
			CollisionData.Modify(true, DirtyFlags, Proxy, ShapeIdx, [bIsProbe](FCollisionData& Data) { Data.bIsProbe = bIsProbe; });
		}

		EChaosCollisionTraceFlag GetCollisionTraceType() const { return CollisionData.Read().CollisionTraceType; }
		void SetCollisionTraceType(const EChaosCollisionTraceFlag InTraceFlag)
		{
			CollisionData.Modify(true, DirtyFlags, Proxy, ShapeIdx, [InTraceFlag](FCollisionData& Data) { Data.CollisionTraceType = InTraceFlag; });
		}

		const FCollisionData& GetCollisionData() const { return CollisionData.Read(); }

		void SetCollisionData(const FCollisionData& Data)
		{
			CollisionData.Write(Data, true, DirtyFlags, Proxy, ShapeIdx);
		}

		const FMaterialData& GetMaterialData() const { return Materials.Read(); }

		void SetMaterialData(const FMaterialData& Data)
		{
			Materials.Write(Data, true, DirtyFlags, Proxy, ShapeIdx);
		}

		void SyncRemoteData(FDirtyPropertiesManager& Manager, int32 ShapeDataIdx, FShapeDirtyData& RemoteData)
		{
			RemoteData.SetFlags(DirtyFlags);
			CollisionData.SyncRemote(Manager, ShapeDataIdx, RemoteData);
			Materials.SyncRemote(Manager, ShapeDataIdx, RemoteData);
			DirtyFlags.Clear();
		}

		void SetProxy(IPhysicsProxyBase* InProxy)
		{
			Proxy = InProxy;
			if (Proxy)
			{
				if (DirtyFlags.IsDirty())
				{
					if (FPhysicsSolverBase* PhysicsSolverBase = Proxy->GetSolver<FPhysicsSolverBase>())
					{
						PhysicsSolverBase->AddDirtyProxyShape(Proxy, ShapeIdx);
					}
				}
			}
		}

		template <typename Lambda>
		UE_DEPRECATED(5.8, "Get/SetCombinedShapeFilterData instead")
		void ModifySimData(const Lambda& LambdaFunc)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			CollisionData.Modify(true, DirtyFlags, Proxy, ShapeIdx, [&LambdaFunc](FCollisionData& Data) { LambdaFunc(Data.SimData); });
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		template <typename Lambda>
		void ModifyMaterials(const Lambda& LambdaFunc)
		{
			Materials.Modify(true, DirtyFlags, Proxy, ShapeIdx, [&LambdaFunc](FMaterialData& Data)
				{
					LambdaFunc(Data.Materials);
				});
		}

		template <typename Lambda>
		void ModifyMaterialMasks(const Lambda& LambdaFunc)
		{
			Materials.Modify(true, DirtyFlags, Proxy, ShapeIdx, [&LambdaFunc](FMaterialData& Data)
				{
					LambdaFunc(Data.MaterialMasks);
				});
		}

		template <typename Lambda>
		void ModifyMaterialMaskMaps(const Lambda& LambdaFunc)
		{
			Materials.Modify(true, DirtyFlags, Proxy, ShapeIdx, [&LambdaFunc](FMaterialData& Data)
				{
					LambdaFunc(Data.MaterialMaskMaps);
				});
		}

		template <typename Lambda>
		void ModifyMaterialMaskMapMaterials(const Lambda& LambdaFunc)
		{
			Materials.Modify(true, DirtyFlags, Proxy, ShapeIdx, [&LambdaFunc](FMaterialData& Data)
				{
					LambdaFunc(Data.MaterialMaskMapMaterials);
				});
		}

	protected:
		explicit FShapeInstanceProxy(int32 InShapeIdx)
			: FPerShapeData(EPerShapeDataType::Proxy, InShapeIdx)
			, Proxy(nullptr)
			, CollisionData()
			, Materials()
		{
		}
		
		UE_DEPRECATED(5.4, "Use FShapeInstanceProxy with FImplicitObjectPtr instead")
		FShapeInstanceProxy(int32 InShapeIdx, TSerializablePtr<FImplicitObject> InGeometry)
			: FPerShapeData(EPerShapeDataType::Proxy, InShapeIdx)
			, Proxy(nullptr)
			, CollisionData()
			, Materials()
		{
			check(false);
		}

		FShapeInstanceProxy(int32 InShapeIdx, const FImplicitObjectPtr& InGeometry)
			: FPerShapeData(EPerShapeDataType::Proxy, InShapeIdx, InGeometry)
			, Proxy(nullptr)
			, CollisionData()
			, Materials()
		{
		}

		CHAOS_API virtual void SerializeMaterials(FChaosArchive& Ar) override final;


		IPhysicsProxyBase* Proxy;

		TShapeProperty<FCollisionData, EShapeProperty::CollisionData> CollisionData;
		TShapeProperty<FMaterialData, EShapeProperty::Materials> Materials;
	};


	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// FShapeInstance
	// 
	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////


	/*
	 * NOTE: FShapeInstance is a Physics-Thread object.
	 * See FShapeInstanceProxy for the game-thread equivalent
	 *
	 * FShapeInstance contains the per-shape data associated with a single shape on a particle.
	 * This contains data like the collision / query filters, material properties etc.
	 *
	 * Every particle holds one FShapeInstance object for each geometry they use.
	 * If the particle has a Union of geometries there will be one FShapeInstance
	 * for each geometry in the union. (Except ClusteredUnions - these are not flattened
	 * because they contain their own query acceleration structure.)
	 *
	 * NOTE: keep size to a minimum. There can be millions of these in s scene.
	 *
	 * NOTE: this version has reduced-memory material storage compared to FShapeInstanceProxy, 
	 * but increased cost when switching from a single-material to a multi-material shape
	 * (though this should not be happening much if at all).
	 * 
	 */
	class FShapeInstance : public FPerShapeData
	{
	public:
		friend class FPerShapeData;

		CHAOS_API static TUniquePtr<FShapeInstance> Make(int32 InShapeIdx, const FImplicitObjectPtr& InGeometry);
		CHAOS_API static void UpdateGeometry(TUniquePtr<FShapeInstance>& InOutShapePtr, const FImplicitObjectPtr& InGeometry);
		CHAOS_API static FShapeInstance* SerializationFactory(FChaosArchive& Ar, FShapeInstance*);

		virtual ~FShapeInstance()
		{
			if (!bIsSingleMaterial && (Material.MaterialData != nullptr))
			{
				delete Material.MaterialData;
			}
		}

		CHAOS_API void UpdateShapeBounds(const FRigidTransform3& WorldTM);

		void* GetUserData() const { return CollisionData.UserData; }
		void SetUserData(void* InUserData) { CollisionData.UserData = InUserData; }

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		UE_DEPRECATED(5.8, "Use GetCombinedShapeFilterData instead") FCollisionFilterData GetQueryData() const { return CollisionData.GetQueryData(); }
		UE_DEPRECATED(5.8, "Use SetCombinedShapeFilterData instead") void SetQueryData(const FCollisionFilterData& InQueryData) { CollisionData.SetQueryData(InQueryData); }

		UE_DEPRECATED(5.8, "Use GetCombinedShapeFilterData instead") FCollisionFilterData GetSimData() const { return CollisionData.GetSimData(); }
		UE_DEPRECATED(5.8, "Use SetCombinedShapeFilterData instead") void SetSimData(const FCollisionFilterData& InSimData) { CollisionData.SetSimData(InSimData); }
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		CHAOS_API const Filter::FShapeFilterData GetShapeFilterData() const;
		CHAOS_API void SetShapeFilterData(const Filter::FShapeFilterData& InShapeFilter);
		CHAOS_API const Filter::FInstanceData GetFilterInstanceData() const;
		CHAOS_API void SetFilterInstanceData(const Filter::FInstanceData& InInstanceData);
		CHAOS_API const Filter::FCombinedShapeFilterData GetCombinedShapeFilterData() const;
		CHAOS_API void SetCombinedShapeFilterData(const Filter::FCombinedShapeFilterData& InCombinedShapeFilter);

		CHAOS_API void UpdateWorldSpaceState(const FRigidTransform3& WorldTransform);

		// The leaf shape (with transformed and implicit wrapper removed).
		CHAOS_API const FImplicitObject* GetLeafGeometry() const;

		// The actor-relative transform of the leaf geometry.
		CHAOS_API FRigidTransform3 GetLeafRelativeTransform() const;

		// The world-space transform of the leaf geometry.
		// If we have non-identity leaf relative transform, is cached from the last call to UpdateWorldSpaceState.
		// If not cahced, is constructed from arguments.
		CHAOS_API FRigidTransform3 GetLeafWorldTransform(const FGeometryParticleHandle* Particle) const;
		CHAOS_API void UpdateLeafWorldTransform(FGeometryParticleHandle* Particle);

		int32 NumMaterials() const
		{
			if (bIsSingleMaterial)
			{
				return Material.MaterialHandle.IsValid() ? 1 : 0;
			}
			else
			{
				return (Material.MaterialData != nullptr) ? Material.MaterialData->Materials.Num() : 0;
			}
		}

		const FMaterialHandle& GetMaterial(const int32 Index) const
		{
			if (bIsSingleMaterial)
			{
				check(Index == 0);
				check(Material.MaterialHandle.IsValid());
				return Material.MaterialHandle;
			}
			else
			{
				return GetMaterialDataImpl().Materials[Index];
			}
		}

		int32 NumMaterialsInternal(const THandleArray<FChaosPhysicsMaterial>*const SimMaterials) const
		{
			if (bIsSingleMaterial)
			{
				return Material.MaterialHandle.IsValidInternal(SimMaterials) ? 1 : 0;
			}
			else
			{
				return (Material.MaterialData != nullptr) ? Material.MaterialData->Materials.Num() : 0;
			}
		}

		const FMaterialHandle& GetMaterialInternal(const int32 Index, const THandleArray<FChaosPhysicsMaterial>*const SimMaterials) const
		{
			if (bIsSingleMaterial)
			{
				check(Index == 0);
				check(Material.MaterialHandle.IsValidInternal(SimMaterials));
				return Material.MaterialHandle;
			}
			else
			{
				return GetMaterialDataImpl().Materials[Index];
			}
		}


		const TArray<FMaterialHandle>& GetMaterials() const { return GetMaterialDataImpl().Materials; }
		const TArray<FMaterialMaskHandle>& GetMaterialMasks() const { return GetMaterialDataImpl().MaterialMasks; }
		const TArray<uint32>& GetMaterialMaskMaps() const { return GetMaterialDataImpl().MaterialMaskMaps; }
		const TArray<FMaterialHandle>& GetMaterialMaskMapMaterials() const { return GetMaterialDataImpl().MaterialMaskMapMaterials; }

		void SetMaterial(FMaterialHandle InMaterial) { SetMaterialImpl(InMaterial); }
		void SetMaterials(const TArray<FMaterialHandle>& InMaterials) { GetMaterialDataImpl().Materials = InMaterials; }
		void SetMaterials(TArray<FMaterialHandle>&& InMaterials) { GetMaterialDataImpl().Materials = InMaterials; }
		void SetMaterialMasks(const TArray<FMaterialMaskHandle>& InMaterialMasks) { GetMaterialDataImpl().MaterialMasks = InMaterialMasks; }
		void SetMaterialMaskMaps(const TArray<uint32>& InMaterialMaskMaps) { GetMaterialDataImpl().MaterialMaskMaps = InMaterialMaskMaps; }
		void SetMaterialMaskMapMaterials(const TArray<FMaterialHandle>& InMaterialMaskMapMaterials) { GetMaterialDataImpl().MaterialMaskMapMaterials = InMaterialMaskMapMaterials; }

		bool GetQueryEnabled() const { return CollisionData.bQueryCollision; }
		void SetQueryEnabled(const bool bEnable) { CollisionData.bQueryCollision = bEnable; }

		bool GetSimEnabled() const { return CollisionData.bSimCollision; }
		void SetSimEnabled(const bool bEnable) { CollisionData.bSimCollision = bEnable; }

		bool GetIsProbe() const { return CollisionData.bIsProbe; }
		void SetIsProbe(const bool bIsProbe) { CollisionData.bIsProbe = bIsProbe; }

		EChaosCollisionTraceFlag GetCollisionTraceType() const { return CollisionData.CollisionTraceType; }
		void SetCollisionTraceType(const EChaosCollisionTraceFlag InTraceFlag) { CollisionData.CollisionTraceType = InTraceFlag; }

		const FCollisionData& GetCollisionData() const { return CollisionData; }
		void SetCollisionData(const FCollisionData& Data) { CollisionData = Data; }

		const FMaterialData& GetMaterialData() const { return GetMaterialDataImpl(); }
		void SetMaterialData(const FMaterialData& Data) { SetMaterialDataImpl(Data); }

		// @todo(chaos): remove when FPerShapeData is removed
		const FShapeDirtyFlags GetDirtyFlags() const { check(false); return FShapeDirtyFlags(); }
		void SyncRemoteData(FDirtyPropertiesManager& Manager, int32 ShapeDataIdx, FShapeDirtyData& RemoteData) { check(false); }
		void SetProxy(IPhysicsProxyBase* InProxy) { check(false); }

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		template <typename Lambda>
		UE_DEPRECATED(5.8, "Get/SetCombinedShapeFilterData instead")
		void ModifySimData(const Lambda& LambdaFunc) { LambdaFunc(CollisionData.SimData); }
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		template <typename Lambda> void ModifyMaterials(const Lambda& LambdaFunc) { LambdaFunc(GetMaterialDataImpl().Materials); }
		template <typename Lambda> void ModifyMaterialMasks(const Lambda& LambdaFunc) { LambdaFunc(GetMaterialDataImpl().MaterialMasks); }
		template <typename Lambda> void ModifyMaterialMaskMaps(const Lambda& LambdaFunc) { LambdaFunc(GetMaterialDataImpl().MaterialMaskMaps); }
		template <typename Lambda> void ModifyMaterialMaskMapMaterials(const Lambda& LambdaFunc) { LambdaFunc(GetMaterialDataImpl().MaterialMaskMapMaterials); }

	protected:
		explicit FShapeInstance(int32 InShapeIdx)
			: FPerShapeData(EPerShapeDataType::Sim, InShapeIdx)
			, CollisionData()
		{
			bIsSingleMaterial = true;
		}

		UE_DEPRECATED(5.4, "Use FShapeInstance with FImplicitObjectPtr instead")
		FShapeInstance(int32 InShapeIdx, TSerializablePtr<FImplicitObject> InGeometry)
			: FPerShapeData(EPerShapeDataType::Sim, InShapeIdx)
			, CollisionData()
		{
			check(false);
		}

		FShapeInstance(int32 InShapeIdx, const FImplicitObjectPtr& InGeometry)
			: FPerShapeData(EPerShapeDataType::Sim, InShapeIdx, InGeometry)
			, CollisionData()
		{
			bIsSingleMaterial = true;
		}

		explicit FShapeInstance(FShapeInstance&& Other)
			: FPerShapeData(EPerShapeDataType::Sim, Other)
			, CollisionData(MoveTemp(Other.CollisionData))
		{
			if (bIsSingleMaterial)
			{
				Material.MaterialHandle = Other.Material.MaterialHandle;
			}
			else
			{
				Material.MaterialData = Other.Material.MaterialData;
				Other.Material.MaterialData = nullptr;
			}
		}
		
		UE_DEPRECATED(5.4, "Use FShapeInstance with FImplicitObjectPtr instead")
		FShapeInstance(const EPerShapeDataType InType, int32 InShapeIdx, TSerializablePtr<FImplicitObject> InGeometry)
			: FPerShapeData(InType, InShapeIdx)
			, CollisionData()
		{
			check(false);
		}

		FShapeInstance(const EPerShapeDataType InType, int32 InShapeIdx, FImplicitObjectPtr InGeometry)
			: FPerShapeData(InType, InShapeIdx, InGeometry)
			, CollisionData()
		{
			bIsSingleMaterial = true;
		}

		FShapeInstance(const EPerShapeDataType InType, FShapeInstance&& Other)
			: FPerShapeData(InType, Other)
			, CollisionData(MoveTemp(Other.CollisionData))
		{
			if (bIsSingleMaterial)
			{
				Material.MaterialHandle = Other.Material.MaterialHandle;
			}
			else
			{
				Material.MaterialData = Other.Material.MaterialData;
				Other.Material.MaterialData = nullptr;
			}
		}

		FMaterialData& GetMaterialDataImpl()
		{
			if (bIsSingleMaterial)
			{
				const FMaterialHandle ExistingMaterial = Material.MaterialHandle;

				Material.MaterialData = new FMaterialData();
				bIsSingleMaterial = false;

				if (ExistingMaterial.IsValid())
				{
					// Ideally we do not get here if we have been setup as a single-material shape.
					// However the API combination of SetMaterial() and GetMaterialData() requires
					// we be able to upgrade from a single material to multi-material shape.
					// We should try to remove this if possible.
					UE_LOGF(LogChaos, Warning, "Perf/memory warning: request for Material Array on single-material ShapeInstance");

					Material.MaterialData->Materials.Add(ExistingMaterial);
				}
			}
			return *Material.MaterialData;
		}

		const FMaterialData& GetMaterialDataImpl() const
		{
			return const_cast<FShapeInstance*>(this)->GetMaterialDataImpl();
		}

		void SetMaterialImpl(const FMaterialHandle& InMaterial)
		{
			if (!bIsSingleMaterial)
			{
				// Destroy existing MaterialData if present
				if (Material.MaterialData != nullptr)
				{
					delete Material.MaterialData;
				}
				bIsSingleMaterial = true;
			}

			Material.MaterialHandle = InMaterial;
		}

		void SetMaterialDataImpl(const FMaterialData& Data)
		{ 
			// If we have only one material and no masks, we can use the single material handle
			if ((Data.Materials.Num() <= 1) && Data.MaterialMasks.IsEmpty() && Data.MaterialMaskMaps.IsEmpty() && Data.MaterialMaskMapMaterials.IsEmpty())
			{
				const FMaterialHandle MaterialHandle = (Data.Materials.Num() > 0) ? Data.Materials[0] : FMaterialHandle();
				SetMaterialImpl(MaterialHandle);
			}
			else
			{
				// This could probably be a Move except it would change the SetMaterialData API
				// As it stands this will duplicate (and eventually discard) the material and mask arrays
				// although this is probably not called on an already-initialized material data very often
				GetMaterialDataImpl() = Data;
			}
		}

		CHAOS_API virtual void SerializeMaterials(FChaosArchive& Ar) override final;

		union FMaterialUnion
		{
			FMaterialUnion() : MaterialHandle() {}		// Default to single-shape mode
			~FMaterialUnion() {}						// Destruction handled by FShapeInstance

			FMaterialHandle MaterialHandle;				// Set if we have only 1 material, no masks etc
			FMaterialData* MaterialData;				// Set if we have multiple materials or any masks
		};

		FCollisionData CollisionData;
		mutable FMaterialUnion Material;

	};

	namespace Private
	{

		///////////////////////////////////////////////////////////////////////////////////////////////
		///////////////////////////////////////////////////////////////////////////////////////////////
		// 
		// FShapeInstanceExtended
		// 
		///////////////////////////////////////////////////////////////////////////////////////////////
		///////////////////////////////////////////////////////////////////////////////////////////////

		/**
		 * An extended version of FShapeInstance (physics-thread shape instance data) that caches
		 * some world-space state of the shape for use in collision detection. This extended data
		 * if only required for shapes that have a transform relative to the particle they are
		 * attached to. It helps by avoiding the need to recalculate the shape transform every
		 * time it is needed in collision detection, which is once for each other shape we
		 * may be in contact with.
		 * 
		 * NOTE: keep size to a minimum. There can be millions of these in s scene.
		 */
		class FShapeInstanceExtended : public FShapeInstance
		{
		public:
			friend class FShapeInstance;

			FRigidTransform3 GetWorldTransform() const
			{
				return FRigidTransform3(WorldPosition, WorldRotation);
			}

			void SetWorldTransform(const FRigidTransform3& LeafWorldTransform)
			{
				WorldPosition = LeafWorldTransform.GetTranslation();
				WorldRotation = LeafWorldTransform.GetRotation();
			}

		protected:
			FShapeInstanceExtended(int32 InShapeIdx, FImplicitObjectPtr InGeometry)
				: FShapeInstance(EPerShapeDataType::SimExtended, InShapeIdx, InGeometry)
			{
			}

			UE_DEPRECATED(5.4, "Use FShapeInstanceExtended with FImplicitObjectPtr instead")
			FShapeInstanceExtended(int32 InShapeIdx, TSerializablePtr<FImplicitObject> InGeometry)
            	: FShapeInstance(InShapeIdx)
            {
				check(false);
            }

			FShapeInstanceExtended(FShapeInstance&& PerShapeData)
				: FShapeInstance(EPerShapeDataType::SimExtended, MoveTemp(PerShapeData))
			{
			}

			// NOTE: FRotation is 16-byte aligned so it goes first
			FRotation3 WorldRotation;
			FVec3 WorldPosition;
		};
	}


	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// Downcasts
	// 
	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////

	inline FShapeInstanceProxy* FPerShapeData::AsShapeInstanceProxy()
	{
		if (Type == EPerShapeDataType::Proxy)
		{
			return static_cast<FShapeInstanceProxy*>(this);
		}
		return nullptr;
	}
	
	inline const FShapeInstanceProxy* FPerShapeData::AsShapeInstanceProxy() const
	{
		if (Type == EPerShapeDataType::Proxy)
		{
			return static_cast<const FShapeInstanceProxy*>(this);
		}
		return nullptr;
	}

	inline FShapeInstance* FPerShapeData::AsShapeInstance()
	{
		if (Type != EPerShapeDataType::Proxy)
		{
			return static_cast<FShapeInstance*>(this);
		}
		return nullptr;
	}

	inline const FShapeInstance* FPerShapeData::AsShapeInstance() const
	{
		if (Type != EPerShapeDataType::Proxy)
		{
			return static_cast<const FShapeInstance*>(this);
		}
		return nullptr;
	}

	inline Private::FShapeInstanceExtended* FPerShapeData::AsShapeInstanceExtended()
	{
		if (Type == EPerShapeDataType::SimExtended)
		{
			return static_cast<Private::FShapeInstanceExtended*>(this);
		}
		return nullptr;
	}

	inline const Private::FShapeInstanceExtended* FPerShapeData::AsShapeInstanceExtended() const
	{
		if (Type == EPerShapeDataType::SimExtended)
		{
			return static_cast<const Private::FShapeInstanceExtended*>(this);
		}
		return nullptr;
	}

	template<typename TLambda>
	inline decltype(auto) FPerShapeData::DownCast(const TLambda& Lambda)
	{
		// NOTE: only FShapeInstanceProxy and FShapeInstance implement the full interface
		// FShapeInstanceExtended is a hidden derived type as far as we are concerned here
		if (Type == EPerShapeDataType::Proxy)
		{
			return Lambda(*AsShapeInstanceProxy());
		}
		else
		{
			return Lambda(*AsShapeInstance());
		}
	}

	template<typename TLambda>
	inline  decltype(auto) FPerShapeData::DownCast(const TLambda& Lambda) const
	{
		// See comments in non-const DownCast()
		if (Type == EPerShapeDataType::Proxy)
		{
			return Lambda(*AsShapeInstanceProxy());
		}
		else
		{
			return Lambda(*AsShapeInstance());
		}
	}


	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// FPerShapeData implementation
	// 
	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////


	inline void FPerShapeData::UpdateShapeBounds(const FRigidTransform3& WorldTM)
	{
		DownCast([&WorldTM](auto& ShapeInstance) { ShapeInstance.UpdateShapeBounds(WorldTM); });
	}

	inline void* FPerShapeData::GetUserData() const
	{
		return DownCast([](auto& ShapeInstance) { return ShapeInstance.GetUserData(); });
	}

	inline void FPerShapeData::SetUserData(void* InUserData)
	{
		DownCast([InUserData](auto& ShapeInstance) { ShapeInstance.SetUserData(InUserData); });
	}

	inline FCollisionFilterData FPerShapeData::GetQueryData() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DownCast([](const auto& ShapeInstance) -> FCollisionFilterData { return ShapeInstance.GetQueryData(); });
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	inline void FPerShapeData::SetQueryData(const FCollisionFilterData& InQueryData)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DownCast([&InQueryData](auto& ShapeInstance) { ShapeInstance.SetQueryData(InQueryData); });
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	inline FCollisionFilterData FPerShapeData::GetSimData() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DownCast([](const auto& ShapeInstance) -> FCollisionFilterData { return ShapeInstance.GetSimData(); });
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	inline void FPerShapeData::SetSimData(const FCollisionFilterData& InSimData)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DownCast([&InSimData](auto& ShapeInstance) { ShapeInstance.SetSimData(InSimData); });
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	inline FImplicitObjectRef FPerShapeData::GetGeometry() const
	{
		return Geometry.GetReference();
	}

	inline const TAABB<FReal, 3>& FPerShapeData::GetWorldSpaceShapeBounds() const
	{
		return WorldSpaceShapeBounds;
	}

	inline void FPerShapeData::UpdateWorldSpaceState(const FRigidTransform3& WorldTransform)
	{
		DownCast([&WorldTransform](auto& ShapeInstance) { ShapeInstance.UpdateWorldSpaceState(WorldTransform); });
	}

	inline const FImplicitObject* FPerShapeData::GetLeafGeometry() const
	{
		return DownCast([](auto& ShapeInstance) { return ShapeInstance.GetLeafGeometry(); });
	}

	inline FRigidTransform3 FPerShapeData::GetLeafRelativeTransform() const
	{
		return DownCast([](auto& ShapeInstance) { return ShapeInstance.GetLeafRelativeTransform(); });
	}

	inline FRigidTransform3 FPerShapeData::GetLeafWorldTransform(const FGeometryParticleHandle* Particle) const
	{
		return DownCast([Particle](auto& ShapeInstance) { return ShapeInstance.GetLeafWorldTransform(Particle); });
	}

	inline void FPerShapeData::UpdateLeafWorldTransform(FGeometryParticleHandle* Particle)
	{
		DownCast([Particle](auto& ShapeInstance) { ShapeInstance.UpdateLeafWorldTransform(Particle); });
	}

	inline int32 FPerShapeData::NumMaterials() const
	{
		return DownCast([](const auto& ShapeInstance) { return ShapeInstance.NumMaterials(); });
	}

	inline const FMaterialHandle& FPerShapeData::GetMaterial(const int32 Index) const
	{
		return DownCast([Index](const auto& ShapeInstance) -> const auto& { return ShapeInstance.GetMaterial(Index); });
	}

	inline const TArray<FMaterialHandle>& FPerShapeData::GetMaterials() const
	{
		return DownCast([](const auto& ShapeInstance) -> const auto& { return ShapeInstance.GetMaterials(); });
	}

	inline const TArray<FMaterialMaskHandle>& FPerShapeData::GetMaterialMasks() const
	{
		return DownCast([](const auto& ShapeInstance) -> const auto& { return ShapeInstance.GetMaterialMasks(); });
	}

	inline const TArray<uint32>& FPerShapeData::GetMaterialMaskMaps() const
	{
		return DownCast([](const auto& ShapeInstance) -> const auto& { return ShapeInstance.GetMaterialMaskMaps(); });
	}

	inline const TArray<FMaterialHandle>& FPerShapeData::GetMaterialMaskMapMaterials() const
	{
		return DownCast([](const auto& ShapeInstance) -> const auto& { return ShapeInstance.GetMaterialMaskMapMaterials(); });
	}

	inline const FShapeDirtyFlags FPerShapeData::GetDirtyFlags() const
	{
		return DownCast([](auto& ShapeInstance) { return ShapeInstance.GetDirtyFlags(); });
	}

	inline void FPerShapeData::SetMaterial(FMaterialHandle InMaterial)
	{
		DownCast([&InMaterial](auto& ShapeInstance) { ShapeInstance.SetMaterial(InMaterial); });
	}

	inline void FPerShapeData::SetMaterials(const TArray<FMaterialHandle>& InMaterials)
	{
		DownCast([&InMaterials](auto& ShapeInstance) { ShapeInstance.SetMaterials(InMaterials); });
	}

	inline void FPerShapeData::SetMaterials(TArray<FMaterialHandle>&& InMaterials)
	{
		DownCast([&InMaterials](auto& ShapeInstance) { ShapeInstance.SetMaterials(MoveTemp(InMaterials)); });
	}

	inline void FPerShapeData::SetMaterialMasks(const TArray<FMaterialMaskHandle>& InMaterialMasks)
	{
		DownCast([&InMaterialMasks](auto& ShapeInstance) { ShapeInstance.SetMaterialMasks(InMaterialMasks); });
	}

	inline void FPerShapeData::SetMaterialMasks(TArray<FMaterialMaskHandle>&& InMaterialMasks)
	{
		DownCast([&InMaterialMasks](auto& ShapeInstance) { ShapeInstance.SetMaterialMasks(MoveTemp(InMaterialMasks)); });
	}

	inline void FPerShapeData::SetMaterialMaskMaps(const TArray<uint32>& InMaterialMaskMaps)
	{
		DownCast([&InMaterialMaskMaps](auto& ShapeInstance) { ShapeInstance.SetMaterialMaskMaps(InMaterialMaskMaps); });
	}

	inline void FPerShapeData::SetMaterialMaskMaps(TArray<uint32>&& InMaterialMaskMaps)
	{
		DownCast([&InMaterialMaskMaps](auto& ShapeInstance) { ShapeInstance.SetMaterialMaskMaps(MoveTemp(InMaterialMaskMaps)); });
	}

	inline void FPerShapeData::SetMaterialMaskMapMaterials(const TArray<FMaterialHandle>& InMaterialMaskMapMaterials)
	{
		DownCast([&InMaterialMaskMapMaterials](auto& ShapeInstance) { ShapeInstance.SetMaterialMaskMapMaterials(InMaterialMaskMapMaterials); });
	}

	inline void FPerShapeData::SetMaterialMaskMapMaterials(TArray<FMaterialHandle>&& InMaterialMaskMapMaterials)
	{
		DownCast([&InMaterialMaskMapMaterials](auto& ShapeInstance) { ShapeInstance.SetMaterialMaskMapMaterials(MoveTemp(InMaterialMaskMapMaterials)); });
	}

	inline bool FPerShapeData::GetQueryEnabled() const
	{
		return DownCast([](auto& ShapeInstance) { return ShapeInstance.GetQueryEnabled(); });
	}

	inline void FPerShapeData::SetQueryEnabled(const bool bEnable)
	{
		DownCast([bEnable](auto& ShapeInstance) { ShapeInstance.SetQueryEnabled(bEnable); });
	}

	inline bool FPerShapeData::GetSimEnabled() const
	{
		return DownCast([](auto& ShapeInstance) { return ShapeInstance.GetSimEnabled(); });
	}

	inline void FPerShapeData::SetSimEnabled(const bool bEnable)
	{
		DownCast([bEnable](auto& ShapeInstance) { ShapeInstance.SetSimEnabled(bEnable); });
	}

	inline bool FPerShapeData::GetIsProbe() const
	{
		return DownCast([](auto& ShapeInstance) { return ShapeInstance.GetIsProbe(); });
	}

	inline void FPerShapeData::SetIsProbe(const bool bIsProbe)
	{
		DownCast([bIsProbe](auto& ShapeInstance) { ShapeInstance.SetIsProbe(bIsProbe); });
	}

	inline EChaosCollisionTraceFlag FPerShapeData::GetCollisionTraceType() const
	{
		return DownCast([](auto& ShapeInstance) { return ShapeInstance.GetCollisionTraceType(); });
	}

	inline void FPerShapeData::SetCollisionTraceType(const EChaosCollisionTraceFlag InTraceFlag)
	{
		DownCast([InTraceFlag](auto& ShapeInstance) { ShapeInstance.SetCollisionTraceType(InTraceFlag); });
	}

	inline const FCollisionData& FPerShapeData::GetCollisionData() const
	{
		return DownCast([](const auto& ShapeInstance) -> const auto& { return ShapeInstance.GetCollisionData(); });
	}

	inline void FPerShapeData::SetCollisionData(const FCollisionData& Data)
	{
		DownCast([&Data](auto& ShapeInstance) { ShapeInstance.SetCollisionData(Data); });
	}

	inline const FMaterialData& FPerShapeData::GetMaterialData() const
	{
		return DownCast([](auto& ShapeInstance) -> const auto& { return ShapeInstance.GetMaterialData(); });
	}

	inline void FPerShapeData::SetMaterialData(const FMaterialData& Data)
	{
		DownCast([&Data](auto& ShapeInstance) { ShapeInstance.SetMaterialData(Data); });
	}

	inline void FPerShapeData::SyncRemoteData(FDirtyPropertiesManager& Manager, int32 ShapeDataIdx, FShapeDirtyData& RemoteData)
	{
		DownCast([&Manager, ShapeDataIdx, &RemoteData](auto& ShapeInstance) { ShapeInstance.SyncRemoteData(Manager, ShapeDataIdx, RemoteData); });
	}

	inline void FPerShapeData::SetProxy(IPhysicsProxyBase* InProxy)
	{
		DownCast([InProxy](auto& ShapeInstance) { ShapeInstance.SetProxy(InProxy); });
	}

	inline int32 FPerShapeData::GetShapeIndex() const
	{
		return ShapeIdx;
	}

	inline void FPerShapeData::ModifyShapeIndex(int32 NewShapeIndex)
	{
		ShapeIdx = NewShapeIndex;
	}

	template <typename Lambda> 
	void FPerShapeData::ModifySimData(const Lambda& LambdaFunc)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		DownCast([&LambdaFunc](auto& ShapeInstance) { ShapeInstance.ModifySimData(LambdaFunc); });
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	template <typename Lambda> 
	void FPerShapeData::ModifyMaterials(const Lambda& LambdaFunc)
	{
		DownCast([&LambdaFunc](auto& ShapeInstance) { ShapeInstance.ModifyMaterials(LambdaFunc); });
	}

	template <typename Lambda>
	void FPerShapeData::ModifyMaterialMasks(const Lambda& LambdaFunc)
	{
		DownCast([&LambdaFunc](auto& ShapeInstance) { ShapeInstance.ModifyMaterialMasks(LambdaFunc); });
	}

	template <typename Lambda> 
	void FPerShapeData::ModifyMaterialMaskMaps(const Lambda& LambdaFunc)
	{
		DownCast([&LambdaFunc](auto& ShapeInstance) { ShapeInstance.ModifyMaterialMaskMaps(LambdaFunc); });
	}

	template <typename Lambda> 
	void FPerShapeData::ModifyMaterialMaskMapMaterials(const Lambda& LambdaFunc)
	{
		DownCast([&LambdaFunc](auto& ShapeInstance) { ShapeInstance.ModifyMaterialMaskMapMaterials(LambdaFunc); });
	}


	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////
	// 
	// Misc stuff
	// 
	///////////////////////////////////////////////////////////////////////////////////////////////
	///////////////////////////////////////////////////////////////////////////////////////////////


	inline FChaosArchive& operator<<(FChaosArchive& Ar, FPerShapeData& Shape)
	{
		Shape.Serialize(Ar);
		return Ar;
	}

	UE_DEPRECATED(5.3, "Not for external use")
	void CHAOS_API UpdateShapesArrayFromGeometry(FShapesArray& ShapesArray, TSerializablePtr<FImplicitObject> Geometry, const FRigidTransform3& ActorTM, IPhysicsProxyBase* Proxy);

}
