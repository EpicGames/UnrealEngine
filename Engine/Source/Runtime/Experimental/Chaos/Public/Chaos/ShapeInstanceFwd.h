// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Framework/ThreadContextEnum.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "Templates/RefCounting.h"
#include "ImplicitFwd.h"
#include <type_traits>

// @todo(chaos): use TRefCountPtr rather than TSharedPtr below - it is smaller (one atomic int instead of 2 atomic ints and 2 pointers)

namespace Chaos
{
	// Legacy names
	class FPerShapeData;
	using FShapesArray = TArray<TUniquePtr<FPerShapeData>, TInlineAllocator<1>>;

	// Game thread ShapeInstance
	class FShapeInstanceProxy;
	using FShapeInstanceProxyPtr = TUniquePtr<FShapeInstanceProxy>;
	using FConstShapeInstanceProxyPtr = TUniquePtr<const FShapeInstanceProxy>;
	using FShapeInstanceProxyArray = TArray<FShapeInstanceProxyPtr, TInlineAllocator<1>>;

	// Physics thread ShapeInstance
	class FShapeInstance;
	using FShapeInstancePtr = TUniquePtr<FShapeInstance>;
	using FConstShapeInstancePtr = TUniquePtr<const FShapeInstance>;
	using FShapeInstanceArray = TArray<FShapeInstancePtr, TInlineAllocator<1>>;

	template<EThreadContext Id>
	using TThreadShapeInstance = std::conditional_t<Id == EThreadContext::External, FShapeInstanceProxy, FShapeInstance>;

	template<EThreadContext Id>
	using TThreadShapeInstanceArray = std::conditional_t<Id == EThreadContext::External, FShapeInstanceProxyArray, FShapeInstanceArray>;
}
