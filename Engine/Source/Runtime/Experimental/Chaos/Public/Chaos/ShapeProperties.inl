// Copyright Epic Games, Inc. All Rights Reserved.

SHAPE_PROPERTY(CollisionData,FCollisionData)
SHAPE_PROPERTY(Materials,FMaterialData)