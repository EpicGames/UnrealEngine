// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Queue.h"
#include "Chaos/Defines.h"
#include "Chaos/Core.h"

class FNetBitReader;
class FNetBitWriter;
class APlayerController;

namespace Chaos
{

class ISimCallbackObject;

struct FSimCallbackOutput
{
	FSimCallbackOutput()
	: InternalTime(-1)
	{
	}

	/** The internal time of the sim when this output was generated */
	FReal InternalTime;
	
protected:

	// Do not delete directly, use FreeOutputData_External
	~FSimCallbackOutput() = default;
};

struct FSimCallbackInput
{
	FSimCallbackInput()
	: NumSteps(0)
	{
	}

	//Called by substep code so we can reuse input for multiple steps
	void SetNumSteps_External(int32 InNumSteps)
	{
		NumSteps = InNumSteps;
	}

	virtual bool NetSendInputCmd(FNetBitWriter& Ar) { return false; }
	virtual bool NetRecvInputCmd(APlayerController* PC, FNetBitReader& Ar) { return false; }

protected:
	// Do not delete directly, use FreeInputData_Internal
	virtual ~FSimCallbackInput() = default;

private:
	int32 NumSteps;	//the number of steps this input belongs to

	void Release_Internal(ISimCallbackObject& CallbackObj);

	friend struct FPushPhysicsData;
};

struct FSimCallbackNoInput : public FSimCallbackInput
{
	void Reset(){}
};

struct FSimCallbackNoOutput : public FSimCallbackOutput
{
	void Reset() {}
};

/** Handle for output that is automatically cleaned up.
	NOTE: this should only be used on external thread as the destructor automatically frees into external structures */
class FSimCallbackOutputHandle
{
public:
	FSimCallbackOutputHandle()
		: SimCallbackOutput(nullptr)
		, SimCallbackObject(nullptr)
	{
	}

	FSimCallbackOutputHandle(FSimCallbackOutput* Output, ISimCallbackObject* InCallbackObject)
	: SimCallbackOutput(Output)
	, SimCallbackObject(InCallbackObject)
	{
		ensure(Output == nullptr || InCallbackObject != nullptr);	//can't have one but not the other
	}
	FSimCallbackOutputHandle(const FSimCallbackOutputHandle& Other) = delete;
	FSimCallbackOutputHandle& operator=(const FSimCallbackOutputHandle& Other) = delete;
	
	FSimCallbackOutputHandle& operator=(FSimCallbackOutputHandle&& Other)
	{
		if(&Other != this)
		{
			Free_External();
			SimCallbackOutput = Other.SimCallbackOutput;
			Other.SimCallbackOutput = nullptr;

			SimCallbackObject = Other.SimCallbackObject;
			Other.SimCallbackObject = nullptr;
		}

		return *this;
	}
	FSimCallbackOutputHandle(FSimCallbackOutputHandle&& Other)
	: SimCallbackOutput(Other.SimCallbackOutput)
	, SimCallbackObject(Other.SimCallbackObject)
	{
		Other.SimCallbackOutput = nullptr;
		Other.SimCallbackObject = nullptr;
	}

	~FSimCallbackOutputHandle()
	{
		Free_External();
	}

	operator bool() const { return SimCallbackOutput != nullptr; }

	FSimCallbackOutput* Get() { return SimCallbackOutput; }
	const FSimCallbackOutput* Get() const { return SimCallbackOutput; }

	FSimCallbackOutput* operator->() { return SimCallbackOutput; }
	const FSimCallbackOutput* operator->() const { return SimCallbackOutput; }

	FSimCallbackOutput& operator*() { return *SimCallbackOutput; }
	const FSimCallbackOutput& operator*() const { return *SimCallbackOutput; }
private:

	void Free_External();

	FSimCallbackOutput* SimCallbackOutput;
	ISimCallbackObject* SimCallbackObject;
};

/** Handle for output that is automatically cleaned up.
	NOTE: this should only be used on external thread as the destructor automatically frees into external structures */

template <typename T>
class TSimCallbackOutputHandle : public FSimCallbackOutputHandle
{
public:

	TSimCallbackOutputHandle() : FSimCallbackOutputHandle(){}

	TSimCallbackOutputHandle(T* Output, ISimCallbackObject* CallbackObject)
	: FSimCallbackOutputHandle(Output, CallbackObject)
	{
	}

	T* Get() { return static_cast<T*>(FSimCallbackOutputHandle::Get());}
	const T* Get() const { return static_cast<const T*>(FSimCallbackOutputHandle::Get()); }

	T* operator->() { return static_cast<T*>(FSimCallbackOutputHandle::Get()); }
	const T* operator->() const { return static_cast<const T*>(FSimCallbackOutputHandle::Get()); }

	T& operator*() { return static_cast<T&>(FSimCallbackOutputHandle::operator*()); }
	const T& operator*() const { return static_cast<const T&>(FSimCallbackOutputHandle::operator*()); }
};

}; // namespace Chaos
