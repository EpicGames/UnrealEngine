// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/TransactionallySafeSpscQueue.h"
#include "Chaos/Defines.h"
#include "Chaos/SimCallbackInput.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "ChaosStats.h"
#include "Chaos/PhysicsObject.h"

// Enable through build or just here in code to cause an untracked callback to fail to compile
// Untracked callbacks will show up a such in profiling sessions and this can help track them down
#ifndef UE_CHAOS_UNTRACKED_CALLBACK_IS_ERROR
#define UE_CHAOS_UNTRACKED_CALLBACK_IS_ERROR 0
#endif

namespace Chaos
{
class FPhysicsSolverBase;
class FMidPhaseModifierAccessor;
class FCCDModifierAccessor;
class FStrainModifierAccessor;
class FCollisionContactModifier;
class FSingleParticlePhysicsProxy;

namespace Utilities
{
	CHAOS_API FReal GetSolverPhysicsResultsTime(const FPhysicsSolverBase*);
}

enum class ESimCallbackOptions : uint32
{
	None					= 0,
	Presimulate				= 1 << 0,
	MidPhaseModification	= 1 << 1,
	CCDModification			= 1 << 2,
	ContactModification		= 1 << 3,
	StrainModification		= 1 << 5,
	ParticleRegister		= 1 << 6,
	ParticleUnregister		= 1 << 7,
	RunOnFrozenGameThread	= 1 << 8,
	Rewind					= 1 << 9,
	PhysicsObjectUnregister	= 1 << 10,
	PreIntegrate			= 1 << 11,
	PostIntegrate			= 1 << 12,
	PreSolve				= 1 << 13,
	PostSolve				= 1 << 14,
	InjectInputsExternal	= 1 << 15,
	ProcessInputsExternal	= 1 << 16,
	ProcessInputsInternal	= 1 << 17,
};
ENUM_CLASS_FLAGS(ESimCallbackOptions)

/**
 * Callback API used for executing code at different points in the simulation.
 * The external thread pushes input data at its own rate (typically once per game thread tick)
 * The internal thread consumes the relevant inputs based on timestamps.
 * For example, if a physics step is 40ms and we tick the game thread at 20ms, the callback would receive 2 inputs per callback (assuming data was pushed every game thread tick)
 * A callback can generate one output to be consumed by the external thread.
 * For example, you could apply a force to an object based on how close the object is to the ground. In this case the game thread may want to know how much force was applied.
 * 
 * This API is also used for resimulating.
 * Because of this, the input data is const and its lifetime is maintained by the internal thread.
 * It is expected that callbacks are "pure" in the sense that they rely only on the input data and affect the simulation in a repeatable and deterministic way.
 * This means that if the same inputs are passed into the callback, we expect the exact same output and that any simulation changes are the same.
 * We rely on this to cache results and skip callbacks when possible during a resim.
 * See functions for more details.
 */
class ISimCallbackObject
{
public:

	//Destructor called on internal thread.
	virtual ~ISimCallbackObject() = default;
	ISimCallbackObject(const ISimCallbackObject&) = delete;	//not copyable

	/** The point in time when this simulation step begins*/
	FReal GetSimTime_Internal() const { return SimTime_Internal; }

	/** The delta time associated with this simulation step */
	FReal GetDeltaTime_Internal() const { return DeltaTime_Internal; }

	virtual bool IsFAsyncObjectManagerCallback() const { return false;}

	virtual void InjectInputs_External(int32 PhysicsStep, int32 NumSteps)
	{
		OnInjectInputs_External(PhysicsStep, NumSteps);
	}

	virtual void ProcessInputs_External(int32 PhysicsStep)
	{
		OnProcessInputs_External(PhysicsStep);
	}

	void PostInitialize_Internal()
	{
		OnPostInitialize_Internal();
	}

	void PostInitialize_External()
	{
		OnPostInitialize_External();
	}

	void PreShutdown_External()
	{
		OnPreShutdown_External();
	}

	virtual void ProcessInputs_Internal(int32 PhysicsStep)
	{
		OnProcessInputs_Internal(PhysicsStep);
	}

	void PreSimulate_Internal()
	{
		OnPreSimulate_Internal();
	}

	void PreIntegrate_Internal()
	{
		OnPreIntegrate_Internal();
	}

	void PostIntegrate_Internal()
	{
		OnPostIntegrate_Internal();
	}

	void MidPhaseModification_Internal(FMidPhaseModifierAccessor& Modifier)
	{
		OnMidPhaseModification_Internal(Modifier);
	}

	void CCDModification_Internal(FCCDModifierAccessor& Modifier)
	{
		OnCCDModification_Internal(Modifier);
	}

	void StrainModification_Internal(FStrainModifierAccessor& Modifier)
	{
		OnStrainModification_Internal(Modifier);
	}

	void ContactModification_Internal(FCollisionContactModifier& Modifier)
	{
		OnContactModification_Internal(Modifier);
	}

	void PreSolve_Internal()
	{
		OnPreSolve_Internal();
	}

	void PostSolve_Internal()
	{
		OnPostSolve_Internal();
	}

	void FinalizeOutputData_Internal()
	{
		if(CurrentOutput_Internal)
		{
			OnFinalizeOutputData_Internal(CurrentOutput_Internal);
			CurrentOutput_Internal = nullptr;
		}
	}

	/**
	 * Free the output data. Note that allocation is done on the internal thread, but freeing is done on the external thread.
	 * A common pattern is to use a single producer single consumer thread safe queue to manage this.
	 */
	virtual void FreeOutputData_External(FSimCallbackOutput* Output) = 0;

	/**
	 * Free the input data. Note that allocation is done on the external thread, but freeing is done on the internal thread.
	 * A common pattern is to use a single producer single consumer thread safe queue to manage this.
	 */
	virtual void FreeInputData_Internal(FSimCallbackInput* Input) = 0;

#if STATS
	/**
	 * Get a stat to describe this callback. Uses GetFNameForStatId() which derived classes should implement to describe themselves
	 */
	TStatId GetStatId() const
	{
		static TStatId StatId = FDynamicStats::CreateStatId<FStatGroup_STATGROUP_Chaos>(GetFNameForStatId());
		return StatId;
	};
#endif

	/**
	 * Get a stat name to describe this callback in external profiling and STATS when Named events are enabled
	 */
#if UE_CHAOS_UNTRACKED_CALLBACK_IS_ERROR
	virtual FName GetFNameForStatId() const = 0;
#else
	virtual FName GetFNameForStatId() const
	{
		const static FLazyName StaticName("Untracked Physics Callback");
		return StaticName;
	}
#endif

	FPhysicsSolverBase* GetSolver()
	{ 
		return Solver; 
	}
	
	const FPhysicsSolverBase* GetSolver() const 
	{ 
		return Solver; 
	}
	
	virtual int32 TriggerRewindIfNeeded_Internal(int32 LastCompletedStep)
	{
		return INDEX_NONE;
	}
	
	virtual void ApplyCorrections_Internal(int32 PhysicsStep, FSimCallbackInput* Input)
	{
		ensure(false);
	}
	
	virtual void FirstPreResimStep_Internal(int32 PhysicsStep)
	{}

	bool HasOption(const ESimCallbackOptions Option) const
	{
		return (Options & Option) == Option;
	}

	UE_DEPRECATED(5.1, "Use HasOption(ESimCallbackOptions::RunOnFrozenGameThread) instead.")
	bool RunOnFrozenGameThread() const 
	{
		return HasOption(ESimCallbackOptions::RunOnFrozenGameThread);
	}

protected:

	ISimCallbackObject(const ESimCallbackOptions InOptions = ESimCallbackOptions::Presimulate)
		: bPendingDelete(false)
		, bPendingDelete_External(false)
		, CurrentExternalInput_External(nullptr)
		, Solver(nullptr)
		, CurrentOutput_Internal(nullptr)
		, CurrentInput_Internal(nullptr)
		, Options(InOptions)
	{ }
	


	/**
	 * Gets the current producer input data. This is what the external thread should be writing to
	 */
	CHAOS_API FSimCallbackInput* GetProducerInputData_External();

	void SetCurrentInput_Internal(FSimCallbackInput* NewInput)
	{
		CurrentInput_Internal = NewInput;
	}

	void SetSimAndDeltaTime_Internal(const FReal InSimTime, const FReal InDeltaTime)
	{
		SimTime_Internal = InSimTime;
		DeltaTime_Internal = InDeltaTime;
	}

private:
	
	/**
	 * Allocate the input data.
	 * A common pattern is to use a single producer single consumer thread safe queue to manage this
	 * Note that allocation is done on the external thread, and freeing is done on the internal one
	 */
	virtual FSimCallbackInput* AllocateInputData_External() = 0;

	/** Called before physics task is dispatched from the game thread, last call to produce async inputs in
	* Register with ESimCallbackOptions::InjectInputsExternal to get callback */
	virtual void OnInjectInputs_External(int32 PhysicsStep, int32 NumSteps)
	{
		check(false);
	}

	/** Called before async inputs are marshaled over to the physics thread, last call to alter already produced async inputs in
	* Register with ESimCallbackOptions::ProcessInputsExternal to get callback */
	virtual void OnProcessInputs_External(int32 PhysicsStep)
	{
		// check(false); // Enable when FNetworkPhysicsCallback no longer calls this
	}

	/**
	* Called once when callback object is registered and after input data have been marshaled
	*/
	virtual void OnPostInitialize_Internal() { }

	/**
	* Called on the external thread after the callback object is registered with the solver.
	* The solver is guaranteed to be available via GetSolver() at this point.
	* Use this for GT-side setup such as registering with solver delegates.
	*/
	virtual void OnPostInitialize_External() { }

	/**
	* Called on the external thread before the callback object is unregistered from the solver.
	* Symmetric to OnPostInitialize_External — use this to clean up any solver-level registrations
	* (e.g. delegates) that were set up during initialization.
	*/
	virtual void OnPreShutdown_External() { }

	/** Called before any sim callbacks are triggered but after async inputs has marshaled over
	* Register with ESimCallbackOptions::ProcessInputsInternal to get callback */
	virtual void OnProcessInputs_Internal(int32 PhysicsStep)
	{
		// check(false); // Enable when FNetworkPhysicsCallback no longer calls this
	}

	/**
	* Called before simulation step (NOTE: not once per sub-step when sub-stepping is enabled)
	*/
	virtual void OnPreSimulate_Internal()
	{
		check(false);
	}

	/**
	* Called once per simulation sub-step, before Integrate. Can be used to modify particle positions, velocities etc.
	*
	* NOTE: you must explicitly request PreIntegrate when registering the callback for this to be called
	*/
	virtual void OnPreIntegrate_Internal()
	{ 
		check(false);
	}

	/**
	* Called once per simulation sub-step. Can be used to modify particle positions, velocities etc.
	*
	* NOTE: you must explicitly request PostIntegrate when registering the callback for this to be called
	*/
	virtual void OnPostIntegrate_Internal()
	{
		check(false);
	}

	/**
	* Called once per simulation step. Allows user to modify midphase pairs
	*
	* NOTE: you must explicitly request midphase modification when registering the callback for this to be called
	*/
	virtual void OnMidPhaseModification_Internal(FMidPhaseModifierAccessor& Modifier)
	{
		check(false);
	}

	/**
	* Called once per simulation step. Allows user to modify CCD results
	*
	* NOTE: you must explicitly request CCD modification when registering the callback for this to be called
	*/
	virtual void OnCCDModification_Internal(FCCDModifierAccessor& Modifier)
	{
		check(false);
	}

	virtual void OnStrainModification_Internal(FStrainModifierAccessor& Modifier)
	{
		check(false);
	}

	/**
	* Called once per simulation step. Allows user to modify contacts
	*
	* NOTE: you must explicitly request contact modification when registering the callback for this to be called
	*/
	virtual void OnContactModification_Internal(FCollisionContactModifier& Modifier)
	{
		//registered for contact modification, but implementation is missing
		check(false);
	}

	/**
	* Called once per simulation sub-step. Can be used to modify particle positions, velocities etc.
	*
	* NOTE: you must explicitly request PreSolve when registering the callback for this to be called
	*/
	virtual void OnPreSolve_Internal()
	{
		check(false);
	}

	/**
	* Called once per simulation sub-step. Can be used to modify particle positions, velocities etc.
	*
	* NOTE: you must explicitly request PostSolve when registering the callback for this to be called
	*/
	virtual void OnPostSolve_Internal()
	{
		check(false);
	}

	/**
	* Called once in a simulation step if any new particles were registered. Occurs after UniqueIdxs
	* are valid.
	*/
	virtual void OnParticlesRegistered_Internal(TArray<FSingleParticlePhysicsProxy*>& RegisteredProxies)
	{
		check(false);
	}

	/**
	* Called once in a simulation step if any particles were unregistered. Occurs immediately before
	* UniqueIdxs become invalid.
	*/
	virtual void OnParticleUnregistered_Internal(TArray<TTuple<Chaos::FUniqueIdx, FSingleParticlePhysicsProxy*>>& UnregisteredProxies)
	{
		check(false);
	}

	/** Called when physics proxies owning FPhysicsObjects unregister */
	virtual void OnPhysicsObjectUnregistered_Internal(FConstPhysicsObjectHandle PhysicsObject)
	{
		check(false);
	}

	/** If we've already allocated an output in this simulation step, use it again. This way multiple sim callbacks only generate one output. Otherwise allocate an output for us and mark it as pending for external thread */
	virtual void OnFinalizeOutputData_Internal(FSimCallbackOutput* CurOutput)
	{
		check(false);	//wrote to output but not finalizing it. Typically this is pushed into some kind of thread safe queue
	}

	friend class FPBDRigidsSolver;
	friend class FPhysicsSolverBase;
	friend class FChaosMarshallingManager;
	friend struct FPushPhysicsData;

	bool bPendingDelete;	//used internally for more efficient deletion. Callbacks do not need to check this
	bool bPendingDelete_External;	//used for efficient deletion. Callbacks do not need to check this

	FSimCallbackInput* CurrentExternalInput_External;	//the input currently being filled out by external thread
	FPhysicsSolverBase* Solver;

	//putting this here so that user classes don't have to bother with non-default constructor
	void SetSolver_External(FPhysicsSolverBase* InSolver)
	{
		Solver = InSolver;
	}
	
	UE_DEPRECATED(5.1, "Do not change options after creation of the callback object - instead, specify them using the TOptions template parameter.")
	void SetContactModification(bool InContactModification)
	{
		Options = Options | ESimCallbackOptions::ContactModification;
	}

protected:
	FSimCallbackOutput* CurrentOutput_Internal;	//the output currently being written to in this sim step

	const FSimCallbackInput* GetCurrentInput_Internal() const 
	{ 
		return CurrentInput_Internal; 
	}

private:
	FSimCallbackInput* CurrentInput_Internal;	        //the input associated with the step we are executing.

	FReal SimTime_Internal;
	FReal DeltaTime_Internal;

	// TODO: Make this const and remove the "friend class FPhysicsSolverBase"
	// once FPhysicsSolverBase::CreateAndRegisterSimCallbackObject_External(bool, bool)
	// has been deprecated.
	ESimCallbackOptions Options;
	friend class FPhysicsSolverBase;
};

/** Simple callback command object. Commands are typically passed in as lambdas and there's no need for data management. Should not be used directly, see FPhysicsSolverBase::EnqueueCommand */
class FSimCallbackCommandObject : public ISimCallbackObject
{
public:
	FSimCallbackCommandObject(TUniqueFunction<void()>&& InFunc, int32 Step = INDEX_NONE)
		: Func(MoveTemp(InFunc))
		, Func2(nullptr)
		, bFuncHasTimeParameters(false)
		, ExecuteOnStep(Step)
	{
		ensureMsgf(Func, TEXT("Created a sim callback object with an unbound function. This command will not be executed."));
	}

	FSimCallbackCommandObject(TUniqueFunction<void(FReal DeltaTime, FReal SimTime)>&& InFunc, int32 Step = INDEX_NONE)
		: Func(nullptr)
		, Func2(MoveTemp(InFunc))
		, bFuncHasTimeParameters(true)
		, ExecuteOnStep(Step)
	{
		ensureMsgf(Func2, TEXT("Created a sim callback object with an unbound function. This command will not be executed."));
	}

	virtual void FreeOutputData_External(FSimCallbackOutput* Output)
	{
		//data management handled by command passed in (data should be copied by value as commands run async and memory lifetime is hard to predict)
		check(false);
	}

	virtual FName GetFNameForStatId() const override
	{
		const static FLazyName StaticName("FSimCallbackCommandObject");
		return StaticName;
	}

	const bool ShouldExecute(const int32 Step, const bool bIsResim) const
	{
		return bIsResim ? ExecuteOnStep == Step : ExecuteOnStep <= Step;
	}

	const bool ShouldDelay(const int32 Step, const bool bIsResim) const
	{
		return !bIsResim && ExecuteOnStep > Step;
	}

	const int32 GetExecuteOnStep() const
	{
		return ExecuteOnStep;
	}

private:

	virtual FSimCallbackInput* AllocateInputData_External()
	{
		//data management handled by command passed in (data should be copied by value as commands run async and memory lifetime is hard to predict)
		check(false);
		return nullptr;
	}

	virtual void FreeInputData_Internal(FSimCallbackInput* Input)
	{
		//data management handled by command passed in (data should be copied by value as commands run async and memory lifetime is hard to predict)
		check(false);
	}

	virtual void OnPreSimulate_Internal() override
	{
		if (!bFuncHasTimeParameters)
		{
			if (ensureMsgf(Func, TEXT("The function of this sim callback object became unbound. This should not happen. This command will not be executed.")))
			{
				Func();
			}
		}
		else
		{
			if (ensureMsgf(Func2, TEXT("The function of this sim callback object became unbound. This should not happen. This command will not be executed.")))
			{
				Func2(GetDeltaTime_Internal(), GetSimTime_Internal());
			}
		}
		
	}

	// Those two function could be in an union 
	TUniqueFunction<void()> Func;
	TUniqueFunction<void(FReal deltaTime, FReal SimTime)> Func2;

	bool bFuncHasTimeParameters;
	int32 ExecuteOnStep;

	friend struct FSimCallbackInput;
};

/** Simple templated implementation that uses lock free queues to manage memory */
template <typename TInputType = FSimCallbackNoInput, typename TOutputType = FSimCallbackNoOutput, ESimCallbackOptions TOptions = ESimCallbackOptions::Presimulate>
class TSimCallbackObject : public ISimCallbackObject
{
public:

	TSimCallbackObject()
		: ISimCallbackObject(TOptions)
		, CurrentOutput_External(nullptr)
	{ }

	UE_DEPRECATED(5.1, "Use default constructor instead and specify RunFrozenOnGameThread using TOptions template parameter.")
	TSimCallbackObject(bool InRunOnFrozenGameThread)
		: ISimCallbackObject(InRunOnFrozenGameThread ? (TOptions | ESimCallbackOptions::RunOnFrozenGameThread) : TOptions)
		, CurrentOutput_External(nullptr)
	{ }

	virtual void FreeOutputData_External(FSimCallbackOutput* Output) override
	{
		auto Concrete = static_cast<TOutputType*>(Output);
		Concrete->Reset();
		OutputPool.Enqueue(Concrete);
	}

	/**
	 * Gets the current producer input data. This is what the external thread should be writing to
	 */
	TInputType* GetProducerInputData_External()
	{
		return static_cast<TInputType*>(ISimCallbackObject::GetProducerInputData_External());
	}

	/** 
	* Get the input associated with the current sim step. This input was provided by the external thread. Note the data could be from a few frames ago
	*/
	const TInputType* GetConsumerInput_Internal() const
	{
		return static_cast<const TInputType*>(GetCurrentInput_Internal());
	}

	/**
	* Gets the output data produced in order up to and including ResultsTime. Typical usage is:
	* while(auto Output = PopOutputData_External()) { //process output }
	*/
	TSimCallbackOutputHandle<TOutputType> PopOutputData_External()
	{
		const FReal ResultsTime = Utilities::GetSolverPhysicsResultsTime(GetSolver());
		if(!CurrentOutput_External)
		{
			OutputQueue.Dequeue(CurrentOutput_External);
		}

		if(CurrentOutput_External && CurrentOutput_External->InternalTime <= ResultsTime)
		{
			TOutputType* Output = CurrentOutput_External;
			CurrentOutput_External = nullptr;
			return TSimCallbackOutputHandle<TOutputType>(Output, this);
		}
		else
		{
			return TSimCallbackOutputHandle<TOutputType>();
		}
	}

	/**
	* Pop up to the latest output, even if it is in the future.
	* NOTE: It's up to the user to check the internal time of the outputs
	* that this produces. See GetSolver()->GetPhysicsResultsTime_External()
	* for the interpolation time.
	* 
	* A typical example is to pop all of these into a queue, and to
	* interpolate them manually.
	*/
	TSimCallbackOutputHandle<TOutputType> PopFutureOutputData_External()
	{
		if (!CurrentOutput_External)
		{
			OutputQueue.Dequeue(CurrentOutput_External);
		}
		
		TOutputType* Output = CurrentOutput_External;
		CurrentOutput_External = nullptr;
		return TSimCallbackOutputHandle<TOutputType>(Output, this);
	}

	/**
	* Check if the output queue is empty of data
	* Can be used while iterating the queue through PopOutputData_External and PopFutureOutputData_External to check if the current data is the last data.
	*/
	bool IsOutputQueueEmpty_External() const
	{
		return OutputQueue.IsEmpty();
	}

	/**
	* Gets the current producer output data. This is what the callback generates. If multiple callbacks are triggered in one step, the same output is used
	*/

	TOutputType& GetProducerOutputData_Internal()
	{
		if(!CurrentOutput_Internal)
		{
			CurrentOutput_Internal = NewOutputData_Internal(GetSimTime_Internal());
		}

		return static_cast<TOutputType&>(*CurrentOutput_Internal);
	}

private:

	TOutputType* NewOutputData_Internal(const FReal InternalTime)
	{
		auto NewOutput = NewDataHelper(OutputBacking, OutputPool);
		NewOutput->InternalTime = InternalTime;
		return NewOutput;
	}

	template <typename T>
	T* NewDataHelper(TArray<TUniquePtr<T>>& Backing, TTransactionallySafeSpscQueue<T*>& Queue)
	{
		T* Result;
		if (!Queue.Dequeue(Result))
		{
			Backing.Emplace(new T());
			Result = Backing.Last().Get();
		}

		return Result;
	}

	virtual void FreeInputData_Internal(FSimCallbackInput* Input) override
	{
		auto Concrete = static_cast<TInputType*>(Input);
		Concrete->Reset();
		InputPool.Enqueue(Concrete);
	}

	TInputType* NewInputData_External()
	{
		return NewDataHelper(InputBacking, InputPool);
	}

	virtual FSimCallbackInput* AllocateInputData_External() override
	{
		return NewInputData_External();
	}

	void OnFinalizeOutputData_Internal(FSimCallbackOutput* BaseOutput) override
	{
		OutputQueue.Enqueue(static_cast<TOutputType*>(BaseOutput));
	}

	TTransactionallySafeSpscQueue<TInputType*> InputPool;
	TArray<TUniquePtr<TInputType>> InputBacking;

	TTransactionallySafeSpscQueue<TOutputType*> OutputPool;
	TArray<TUniquePtr<TOutputType>> OutputBacking;
	TTransactionallySafeSpscQueue<TOutputType*> OutputQueue;	//holds the outputs in order

	TOutputType* CurrentOutput_External;	//the earliest output we can consume
};

struct FSimCallbackInputAndObject
{
	ISimCallbackObject* CallbackObject;
	FSimCallbackInput* Input;
};

inline void FSimCallbackInput::Release_Internal(ISimCallbackObject& CallbackObj)
{
	//free once all steps are done with this input
	ensure(NumSteps);
	if(--NumSteps == 0)
	{
		CallbackObj.FreeInputData_Internal(this);
	}
}

inline void FSimCallbackOutputHandle::Free_External()
{
	if (SimCallbackOutput)
	{
		SimCallbackObject->FreeOutputData_External(SimCallbackOutput);
	}
}

}; // namespace Chaos
