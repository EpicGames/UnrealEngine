// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/Particles.h"
#include "Chaos/Rotation.h"
#include "UObject/FortniteValkyrieBranchObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"


namespace Chaos
{
	template<class T, int d>
	class TSimpleGeometryParticles : public TParticles<T, d>
	{
	public:

		using TArrayCollection::Size;
		using TParticles<T,d>::GetX;
		
		TSimpleGeometryParticles()
		    : TParticles<T, d>()
		{
			TArrayCollection::AddArray(&MR);
			TArrayCollection::AddArray(&MGeometry);
		}
		TSimpleGeometryParticles(const TSimpleGeometryParticles<T, d>& Other) = delete;
		TSimpleGeometryParticles(TSimpleGeometryParticles<T, d>&& Other)
		    : TParticles<T, d>(MoveTemp(Other))
			, MR(MoveTemp(Other.MR))
			, MGeometry(MoveTemp(Other.MGeometry))
		{
			TArrayCollection::AddArray(&MR);
			TArrayCollection::AddArray(&MGeometry);
		}
		TSimpleGeometryParticles& operator=(const TSimpleGeometryParticles<T, d>& Other) = delete;
		TSimpleGeometryParticles& operator=(TSimpleGeometryParticles<T, d>&& Other) = delete;

		TSimpleGeometryParticles(TParticles<T, d>&& Other)
		    : TParticles<T, d>(MoveTemp(Other))
		{
			TArrayCollection::AddArray(&MR);
			TArrayCollection::AddArray(&MGeometry);
		}

		virtual ~TSimpleGeometryParticles() override
		{}

		UE_DEPRECATED(5.4, "Use GetR instead")
		FORCEINLINE const TRotation<T, d> R(const int32 Index) const { return TRotation<T, d>(MR[Index]); }
		UE_DEPRECATED(5.4, "Use GetR or SetR instead.")
		FORCEINLINE TRotation<T, d> R(const int32 Index) { return MR[Index]; }
		FORCEINLINE const TRotation<T, d> GetR(const int32 Index) const { return TRotation<T, d>(MR[Index]); }
		FORCEINLINE void SetR(const int32 Index, const TRotation<T, d>& InR) { MR[Index] = TRotation<FRealSingle, d>(InR); }
		FORCEINLINE const TRotation<FRealSingle, d> GetRf(const int32 Index) const { return MR[Index]; }
		FORCEINLINE void SetRf(const int32 Index, const TRotation<FRealSingle, d>& InR) { MR[Index] = InR; }
		const TArrayCollectionArray<TRotation<FRealSingle, d>>& GetR() const { return MR; }
		TArrayCollectionArray<TRotation<FRealSingle, d>>& GetR() { return MR; }

		FORCEINLINE const FImplicitObjectPtr& GetGeometry(const int32 Index) const { return MGeometry[Index]; }
		void SetGeometry(const int32 Index, const FImplicitObjectPtr& InGeometry)
		{
			SetGeometryImpl(Index, InGeometry);
		}

		FORCEINLINE const TArray<FImplicitObjectPtr>& GetAllGeometry() const { return MGeometry; }
		FORCEINLINE TArray<TRotation<FRealSingle, d>>& AllR() { return MR; }

		virtual void Serialize(FChaosArchive& Ar)
		{
			TParticles<T, d>::Serialize(Ar);
			
			Ar.UsingCustomVersion(FFortniteValkyrieBranchObjectVersion::GUID);
			if (Ar.CustomVer(FFortniteValkyrieBranchObjectVersion::GUID) < FFortniteValkyrieBranchObjectVersion::RefCountedOImplicitObjects)
			{
				TArrayCollectionArray<TSerializablePtr<FImplicitObject>> LGeometry;
				TArrayCollectionArray<TUniquePtr<Chaos::FImplicitObject>> LDynamicGeometry;
				Ar << LGeometry << LDynamicGeometry;

				if(Ar.IsLoading())
				{
					MGeometry.SetNumUninitialized(LGeometry.Num());
					uint32 ImplicitIndex = 0;
					for(const TSerializablePtr<FImplicitObject>& ImplicitObjectPtr : LGeometry)
					{
						MGeometry[ImplicitIndex++] = ImplicitObjectPtr->CopyGeometry();
					}
				}
			}
			else
			{
				Ar << MGeometry;
			}

			Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
			if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) >= FUE5ReleaseStreamObjectVersion::SinglePrecisonParticleDataPT)
			{
				Ar << MR;
			}
			else
			{
				TArrayCollectionArray<TRotation<FReal, d>> RDouble;
				RDouble.Resize(MR.Num());
				for (int32 Index = 0; Index < MR.Num(); ++Index)
				{
					RDouble[Index] = TRotation<FReal, d >(MR[Index]);
				}

				Ar << RDouble;

				MR.Resize(RDouble.Num());
				for (int32 Index = 0; Index < RDouble.Num(); ++Index)
				{
					MR[Index] = TRotation<FRealSingle, d >(RDouble[Index]);
				}
			}
		}

	protected:
		virtual void SetGeometryImpl(const int32 Index, const FImplicitObjectPtr& InGeometry)
		{
			MGeometry[Index] = InGeometry;
		}

	private:
		TArrayCollectionArray<TRotation<FRealSingle, d>> MR;
		// MGeometry contains raw ptrs to every entry in both MSharedGeometry and MDynamicGeometry.
		// It may also contain raw ptrs to geometry which is managed outside of Chaos.
		TArrayCollectionArray<FImplicitObjectPtr> MGeometry;
	};

	template <typename T, int d>
	FChaosArchive& operator<<(FChaosArchive& Ar, TSimpleGeometryParticles<T, d>& Particles)
	{
		Particles.Serialize(Ar);
		return Ar;
	}
}

