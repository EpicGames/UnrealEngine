// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos
{
	template <typename T, int d>
	TVector<T, d> LineSimplexFindOrigin(const TVector<T, d>* Simplex, int32* Idxs, int32& NumVerts, T* OutBarycentric)
	{
		const TVector<T, d>& X0 = Simplex[Idxs[0]];
		const TVector<T, d>& X1 = Simplex[Idxs[1]];
		const TVector<T, d> X0ToX1 = X1 - X0;

		//Closest Point = (-X0 dot X1-X0) / ||(X1-X0)||^2 * (X1-X0)

		const TVector<T, d> X0ToOrigin = -X0;
		const T Dot = TVector<T, d>::DotProduct(X0ToOrigin, X0ToX1);

		if (Dot <= 0)
		{
			NumVerts = 1;
			OutBarycentric[Idxs[0]] = 1;
			return X0;
		}

		const T X0ToX1Squared = X0ToX1.SizeSquared();

		if (X0ToX1Squared <= Dot || X0ToX1Squared <= std::numeric_limits<T>::min())	//if dividing gives 1+ or the line is degenerate
		{
			NumVerts = 1;
			Idxs[0] = Idxs[1];
			OutBarycentric[Idxs[1]] = 1;
			return X1;
		}

		const T Ratio = FMath::Clamp(Dot / X0ToX1Squared, T(0), T(1));
		const TVector<T, d> Closest = Ratio * (X0ToX1)+X0;	//note: this could pass X1 by machine epsilon, but doesn't seem worth it for now
		OutBarycentric[Idxs[0]] = 1 - Ratio;
		OutBarycentric[Idxs[1]] = Ratio;
		return Closest;
	}

	template <typename T>
	TVec3<T> LineSimplexFindOrigin2(TVec3<T>* Simplex, int32& NumVerts, T* OutBarycentric, TVec3<T>* A, TVec3<T>* B)
	{
		const TVec3<T>& X0 = Simplex[0];
		const TVec3<T>& X1 = Simplex[1];
		const TVec3<T> X0ToX1 = X1 - X0;

		//Closest Point = (-X0 dot X1-X0) / ||(X1-X0)||^2 * (X1-X0)

		const TVec3<T> X0ToOrigin = -X0;
		const T Dot = TVec3<T>::DotProduct(X0ToOrigin, X0ToX1);

		if (Dot <= 0)
		{
			NumVerts = 1;
			OutBarycentric[0] = 1;
			return X0;
		}

		const T X0ToX1Squared = X0ToX1.SizeSquared();

		if (X0ToX1Squared <= Dot || X0ToX1Squared <= std::numeric_limits<T>::min())	//if dividing gives 1+ or the line is degenerate
		{
			NumVerts = 1;
			OutBarycentric[0] = 1;
			Simplex[0] = Simplex[1];
			A[0] = A[1];
			B[0] = B[1];
			return X1;
		}

		const T Ratio = FMath::Clamp(Dot / X0ToX1Squared, T(0), T(1));
		const TVec3<T> Closest = Ratio * (X0ToX1)+X0;	//note: this could pass X1 by machine epsilon, but doesn't seem worth it for now
		OutBarycentric[0] = 1 - Ratio;
		OutBarycentric[1] = Ratio;
		return Closest;
	}


	struct FSimplex
	{
		int32 NumVerts;
		int32 Idxs[4];
		int32 operator[](int32 Idx) const { return Idxs[Idx]; }
		int32& operator[](int32 Idx) { return Idxs[Idx]; }

		FSimplex(std::initializer_list<int32> InIdxs = {})
			: NumVerts(InIdxs.size())
		{
			check(NumVerts <= 4);
			int32 i = 0;
			for (int32 Idx : InIdxs)
			{
				Idxs[i++] = Idx;
			}
			while (i < 4)
			{
				Idxs[i++] = 0;	//some code uses these for lookup regardless of NumVerts. Makes for faster code so just use 0 to make lookup safe
			}
		}
	};

	template <typename T>
	bool SignMatch(T A, T B)
	{
		return (A > 0 && B > 0) || (A < 0 && B < 0);
	}

	template <typename T>
	TVec3<T> TriangleSimplexFindOrigin(const TVec3<T>* Simplex, FSimplex& Idxs, T* OutBarycentric)
	{
		/* Project the origin onto the triangle plane:
		   Let n = (b-a) cross (c-a)
		   Let the distance from the origin dist = ((-a) dot n / ||n||^2)
		   Then the projection p = 0 - dist * n = (a dot n) / ||n||^2
		*/

		const int32 Idx0 = Idxs[0];
		const int32 Idx1 = Idxs[1];
		const int32 Idx2 = Idxs[2];

		const TVec3<T>& X0 = Simplex[Idx0];
		const TVec3<T>& X1 = Simplex[Idx1];
		const TVec3<T>& X2 = Simplex[Idx2];

		const TVec3<T> X0ToX1 = X1 - X0;
		const TVec3<T> X0ToX2 = X2 - X0;
		const TVec3<T> TriNormal = TVec3<T>::CrossProduct(X0ToX1, X0ToX2);

		/*
		   We want |(a dot n) / ||n||^2| < 1 / eps to avoid inf. But note that |a dot n| <= ||a||||n|| and so
		   |(a dot n) / ||n||^2| <= ||a|| / ||n| < 1 / eps requires that ||eps*a||^2 < ||n||^2
		*/
		const T TriNormal2 = TriNormal.SizeSquared();
		const T Eps2 = (X0*std::numeric_limits<T>::min()).SizeSquared();
		if (Eps2 >= TriNormal2)	//equality fixes case where both X0 and TriNormal2 are 0
		{
			//degenerate triangle so return previous line results
			Idxs.NumVerts = 2;
			return LineSimplexFindOrigin(Simplex, Idxs.Idxs, Idxs.NumVerts, OutBarycentric);
		}

		const TVec3<T> TriNormalOverSize2 = TriNormal / TriNormal2;
		const T SignedDistance = TVec3<T>::DotProduct(X0, TriNormalOverSize2);
		const TVec3<T> ProjectedOrigin = TriNormal * SignedDistance;

		/*
			Let p be the origin projected onto the triangle plane. We can represent the point p in a 2d subspace spanned by the triangle
			|a_u, b_u, c_u| |lambda_1| = |p_u|
			|a_v, b_v, c_v| |lambda_2| = |p_v|
			|1,   1,   1  | |lambda_3| = |1  |

			Cramer's rule gives: lambda_i = det(M_i) / det(M)
			To choose u and v we simply test x,y,z to see if any of them are linearly independent
		*/

		T DetM = 0;	//not needed but fixes compiler warning
		int32 BestAxisU = INDEX_NONE;
		int32 BestAxisV = 0;	//once best axis is chosen use the axes that go with it in the right order

		{
			T MaxAbsDetM = 0;	//not needed but fixes compiler warning
			int32 AxisU = 1;
			int32 AxisV = 2;
			for (int32 CurAxis = 0; CurAxis < 3; ++CurAxis)
			{
				T TmpDetM = X1[AxisU] * X2[AxisV] - X2[AxisU] * X1[AxisV]
					+ X2[AxisU] * X0[AxisV] - X0[AxisU] * X2[AxisV]
					+ X0[AxisU] * X1[AxisV] - X1[AxisU] * X0[AxisV];
				const T AbsDetM = FMath::Abs(TmpDetM);
				if (BestAxisU == INDEX_NONE || AbsDetM > MaxAbsDetM)
				{
					MaxAbsDetM = AbsDetM;
					DetM = TmpDetM;
					BestAxisU = AxisU;
					BestAxisV = AxisV;
				}
				AxisU = AxisV;
				AxisV = CurAxis;
			}
		}

		/*
			Now solve for the cofactors (i.e. the projected origin replaces the column of each cofactor).
			Notice that this is really the area of each sub triangle with the projected origin.
			If the sign of the determinants is different than the sign of the entire triangle determinant then we are outside of the triangle.
			The conflicting signs indicate which voronoi regions to search

			Cofactor_a =    |p_u b_u c_u|  Cofactor_b =    |a_u p_u c_u|  Cofactor_c = |a_u b_u p_u|
						 det|p_v b_v c_v|               det|a_v p_v c_v|            det|a_v c_v p_v|
							|1   1  1   |                  |1   1  1   |               |1   1  1   |
		*/
		const TVec3<T>& P0 = ProjectedOrigin;
		const TVec3<T> P0ToX0 = X0 - P0;
		const TVec3<T> P0ToX1 = X1 - P0;
		const TVec3<T> P0ToX2 = X2 - P0;

		const T Cofactors[3] = {
			P0ToX1[BestAxisU] * P0ToX2[BestAxisV] - P0ToX2[BestAxisU] * P0ToX1[BestAxisV],
			-P0ToX0[BestAxisU] * P0ToX2[BestAxisV] + P0ToX2[BestAxisU] * P0ToX0[BestAxisV],
			P0ToX0[BestAxisU] * P0ToX1[BestAxisV] - P0ToX1[BestAxisU] * P0ToX0[BestAxisV]
		};

		bool bSignMatch[3];
		FSimplex SubSimplices[3] = { {Idx1,Idx2}, {Idx0,Idx2}, {Idx0,Idx1} };
		TVec3<T> ClosestPointSub[3];
		T SubBarycentric[3][4];
		int32 ClosestSubIdx = INDEX_NONE;
		T MinSubDist2 = 0;	//not needed
		bool bInside = true;
		for (int32 Idx = 0; Idx < 3; ++Idx)
		{
			bSignMatch[Idx] = SignMatch(DetM, Cofactors[Idx]);
			if (!bSignMatch[Idx])
			{
				bInside = false;
				ClosestPointSub[Idx] = LineSimplexFindOrigin(Simplex, SubSimplices[Idx].Idxs, SubSimplices[Idx].NumVerts, SubBarycentric[Idx]);

				const T Dist2 = ClosestPointSub[Idx].SizeSquared();
				if (ClosestSubIdx == INDEX_NONE || Dist2 < MinSubDist2)
				{
					MinSubDist2 = Dist2;
					ClosestSubIdx = Idx;
				}
			}
		}

		if (bInside)
		{
			//SignMatch ensures that DetM is not 0. The Det_i / Det_m ratio is always between 0-1 because it represents the ratio of areas and Det_m is the total area
			const T InvDetM = 1 / DetM;
			T Lambda0 = Cofactors[0] * InvDetM;
			T Lambda1 = Cofactors[1] * InvDetM;
			//T Lambda2 = 1 - Lambda1 - Lambda0;
			T Lambda2 = Cofactors[2] * InvDetM;
			OutBarycentric[Idx0] = Lambda0;
			OutBarycentric[Idx1] = Lambda1;
			OutBarycentric[Idx2] = Lambda2;

			// We know that we are inside the triangle so we can use the projected point we calculated above. 
			// The closest point can also be derived from the barycentric coordinates, but it will contain 
			// numerical error from the determinant calculation  and can cause GJK to terminate with a poor solution.
			// (E.g., this caused jittering when walking on box with dimensions of 100000cm or more).
			// return X0 * Lambda0 + X1 * Lambda1 + X2 * Lambda2;
			return ProjectedOrigin;
		}
		else
		{
			check(Idx0 >= 0 && Idx0 < 4);
			check(Idx1 >= 0 && Idx1 < 4);
			check(Idx2 >= 0 && Idx2 < 4);
			Idxs = SubSimplices[ClosestSubIdx];
			OutBarycentric[Idx0] = SubBarycentric[ClosestSubIdx][Idx0];
			OutBarycentric[Idx1] = SubBarycentric[ClosestSubIdx][Idx1];
			OutBarycentric[Idx2] = SubBarycentric[ClosestSubIdx][Idx2];
			return ClosestPointSub[ClosestSubIdx];
		}
	}

	template <typename T>
	TVec3<T> TriangleSimplexFindOrigin2(TVec3<T>* Simplex, int32& NumVerts, T* OutBarycentric, TVec3<T>* As, TVec3<T>* Bs)
	{
		const TVec3<T>& A = Simplex[0];
		const TVec3<T>& B = Simplex[1];
		const TVec3<T>& C = Simplex[2];

		// Vertex region A
		const TVec3<T> AB = B - A;
		const TVec3<T> AC = C - A;
		const TVec3<T> AO = -A;

		const T d1 = TVec3<T>::DotProduct(AB, AO);
		const T d2 = TVec3<T>::DotProduct(AC, AO);

		const bool bIsD1LEZero = (d1 <= T(0));
		const bool bIsD2LEZero = (d2 <= T(0));
		const bool bIsA = bIsD1LEZero && bIsD2LEZero;
		if (bIsA)
		{
			NumVerts = 1;
			OutBarycentric[0] = T(1);
			return A;
		}

		//Vertex region B
		const TVec3<T> BO = -B;
		const T d3 = TVec3<T>::DotProduct(AB, BO);
		const T d4 = TVec3<T>::DotProduct(AC, BO);

		const bool bIsD3GEZero = (d3 >= T(0));
		const bool bIsD3GED4 = (d3 >= d4);
		const bool bIsB = bIsD3GEZero && bIsD3GED4;
		if (bIsB)
		{
			NumVerts = 1;
			OutBarycentric[0] = T(1);
			Simplex[0] = B;
			As[0] = As[1];
			Bs[0] = Bs[1];
			return B;
		}

		// Edge AB
		const T d1d4 = d1 * d4;
		const T vc = d1d4 - d2 * d3;
		const T NormalizationDenominatorAB = d1 - d3;

		const bool bIsZeroGEvc = (vc <= T(0));
		const bool bIsD1GEZero = (d1 >= T(0));
		const bool bIsZeroGED3 = (d3 <= T(0));
		const bool bIsNDABGTZero = (NormalizationDenominatorAB > T(0));
		const bool bIsAB = bIsZeroGEvc && bIsD1GEZero && bIsZeroGED3 && bIsNDABGTZero;

		if (bIsAB)
		{
			NumVerts = 2;
			const T V = d1 / NormalizationDenominatorAB;
			const T OneMinusV = T(1) - V;
			OutBarycentric[0] = OneMinusV;
			OutBarycentric[1] = V;
			return A + V * AB;
		}

		// Vertex C
		const TVec3<T> CO = -C;
		const T d5 = TVec3<T>::DotProduct(AB, CO);
		const T d6 = TVec3<T>::DotProduct(AC, CO);
		const bool bIsD6GEZero = (d6 >= T(0));
		const bool bIsD6GED5 = (d6 >= d5);
		const bool bIsC = bIsD6GEZero && bIsD6GED5;
		if (bIsC)
		{
			NumVerts = 1;
			OutBarycentric[0] = T(1);
			Simplex[0] = C;
			As[0] = As[2];
			Bs[0] = Bs[2];
			return C;
		}

		// Edge AC
		const T d5d2 = d5 * d2;
		const T vb = d5d2 - d1 * d6;
		const T NormalizationDenominatorAC = d2 - d6;

		const bool bIsZeroGEvb = (vb <= T(0));
		const bool bIsD2GEZero = (d2 >= T(0));
		const bool bIsZeroGED6 = (d6 <= T(0));
		const bool bIsNDACGTZero = (NormalizationDenominatorAC > T(0));
		const bool bIsAC = bIsZeroGEvb && bIsD2GEZero && bIsZeroGED6 && bIsNDACGTZero;
		if (bIsAC)
		{
			const T W = d2 / NormalizationDenominatorAC;
			const T OneMinusW = T(1) - W;
			NumVerts = 2;
			OutBarycentric[0] = OneMinusW;
			OutBarycentric[1] = W;
			Simplex[1] = C;
			As[1] = As[2];
			Bs[1] = Bs[2];
			return A + W * AC;
		}

		// Edge BC
		const T d3d6 = d3 * d6;
		const T va = d3d6 - d5 * d4;
		const T d4MinusD3 = d4 - d3;
		const T d5MinusD6 = d5 - d6;
		const T NormalizationDenominatorBC = d4MinusD3 + d5MinusD6;

		const bool bIsZeroGEva = (va <= T(0));
		const bool bIsD4MinusD3GEZero = (d4MinusD3 >= T(0));
		const bool bIsD5MinusD6GEZero = (d5MinusD6 >= T(0));
		const bool bIsNDBCGTZero = (NormalizationDenominatorBC > T(0));
		const bool bIsBC = bIsZeroGEva && bIsD4MinusD3GEZero && bIsD5MinusD6GEZero && bIsNDBCGTZero;
		if (bIsBC)
		{
			const T W = d4MinusD3 / NormalizationDenominatorBC;
			const T OneMinusW = T(1) - W;
			NumVerts = 2;
			OutBarycentric[0] = OneMinusW;
			OutBarycentric[1] = W;
			const TVec3<T> CMinusB = C - B;
			const TVec3<T> Result = B + W * CMinusB;
			Simplex[0] = B;
			Simplex[1] = C;
			As[0] = As[1];
			Bs[0] = Bs[1];
			As[1] = As[2];
			Bs[1] = Bs[2];
			return Result;
		}

		// Inside triangle
		const T denom = T(1) / (va + vb + vc);
		const T U = va * denom;
		const T V = vb * denom;
		const T W = vc * denom;
		NumVerts = 3;
		OutBarycentric[0] = U;
		OutBarycentric[1] = V;
		OutBarycentric[2] = W;

		// We know that we are inside the triangle so we project the origin onto the plane
		// The closest point can also be derived from the barycentric coordinates, but it will contain 
		// numerical error from the determinant calculation  and can cause GJK to terminate with a poor solution.
		// (E.g., this caused jittering when walking on box with dimensions of 100000cm or more).
		// This fix the unit test TestSmallCapsuleLargeBoxGJKRaycast_Vertical
		// Previously was return VectorMultiplyAdd(AC, w, VectorMultiplyAdd(AB, v, A));
		const TVec3<T> TriNormal = TVec3<T>::CrossProduct(AB, AC);
		const T TriNormal2 = TVec3<T>::DotProduct(TriNormal, TriNormal);
		if (TriNormal2 > std::numeric_limits<T>::min())
		{
			const TVec3<T> TriNormalOverSize2 = TriNormal / TriNormal2;
			const T SignedDistance = TVec3<T>::DotProduct(A, TriNormalOverSize2);
			return TriNormal * SignedDistance;
		}

		// If we get here we hit a degenerate Simplex (all 3 verts in a line/point)
		// Let's just exit GJK with whatever normal and distance it had last iteration...
		return FVec3(0);
	}

	template <typename T>
	TVec3<T> TetrahedronSimplexFindOrigin(const TVec3<T>* Simplex, FSimplex& Idxs, T* OutBarycentric)
	{
		const int32 Idx0 = Idxs[0];
		const int32 Idx1 = Idxs[1];
		const int32 Idx2 = Idxs[2];
		const int32 Idx3 = Idxs[3];

		const TVec3<T>& X0 = Simplex[Idx0];
		const TVec3<T>& X1 = Simplex[Idx1];
		const TVec3<T>& X2 = Simplex[Idx2];
		const TVec3<T>& X3 = Simplex[Idx3];

		//Use signed volumes to determine if origin is inside or outside
		/*
			M = [X0x X1x X2x X3x;
				 X0y X1y X2y X3y;
				 X0z X1z X2z X3z;
				 1   1   1   1]
		*/

		T Cofactors[4];
		Cofactors[0] = -TVec3<T>::DotProduct(X1, TVec3<T>::CrossProduct(X2, X3));
		Cofactors[1] = TVec3<T>::DotProduct(X0, TVec3<T>::CrossProduct(X2, X3));
		Cofactors[2] = -TVec3<T>::DotProduct(X0, TVec3<T>::CrossProduct(X1, X3));
		Cofactors[3] = TVec3<T>::DotProduct(X0, TVec3<T>::CrossProduct(X1, X2));
		T DetM = (Cofactors[0] + Cofactors[1]) + (Cofactors[2] + Cofactors[3]);

		bool bSignMatch[4];
		FSimplex SubIdxs[4] = { {1,2,3}, {0,2,3}, {0,1,3}, {0,1,2} };
		TVec3<T> ClosestPointSub[4];
		T SubBarycentric[4][4];
		int32 ClosestTriangleIdx = INDEX_NONE;
		T MinTriangleDist2 = 0;

		bool bInside = true;
		for (int Idx = 0; Idx < 4; ++Idx)
		{
			bSignMatch[Idx] = SignMatch(DetM, Cofactors[Idx]);
			if (!bSignMatch[Idx])
			{
				bInside = false;
				ClosestPointSub[Idx] = TriangleSimplexFindOrigin(Simplex, SubIdxs[Idx], SubBarycentric[Idx]);

				const T Dist2 = ClosestPointSub[Idx].SizeSquared();
				if (ClosestTriangleIdx == INDEX_NONE || Dist2 < MinTriangleDist2)
				{
					MinTriangleDist2 = Dist2;
					ClosestTriangleIdx = Idx;
				}
			}
		}

		if (bInside)
		{
			OutBarycentric[Idx0] = Cofactors[0] / DetM;
			OutBarycentric[Idx1] = Cofactors[1] / DetM;
			OutBarycentric[Idx2] = Cofactors[2] / DetM;
			OutBarycentric[Idx3] = Cofactors[3] / DetM;

			return TVec3<T>(0);
		}

		Idxs = SubIdxs[ClosestTriangleIdx];

		OutBarycentric[Idx0] = SubBarycentric[ClosestTriangleIdx][Idx0];
		OutBarycentric[Idx1] = SubBarycentric[ClosestTriangleIdx][Idx1];
		OutBarycentric[Idx2] = SubBarycentric[ClosestTriangleIdx][Idx2];
		OutBarycentric[Idx3] = SubBarycentric[ClosestTriangleIdx][Idx3];

		return ClosestPointSub[ClosestTriangleIdx];
	}

	template <typename T>
	TVec3<T> TetrahedronSimplexFindOrigin2(TVec3<T>* Simplex, int32& NumVerts, T* OutBarycentric, TVec3<T>* A, TVec3<T>* B)
	{
		const TVec3<T>& X0 = Simplex[0];
		const TVec3<T>& X1 = Simplex[1];
		const TVec3<T>& X2 = Simplex[2];
		const TVec3<T>& X3 = Simplex[3];

		//Use signed volumes to determine if origin is inside or outside
		/*
			M = [X0x X1x X2x X3x;
				 X0y X1y X2y X3y;
				 X0z X1z X2z X3z;
				 1   1   1   1]
		*/

		T Cofactors[4];
		Cofactors[0] = -TVec3<T>::DotProduct(X1, TVec3<T>::CrossProduct(X2, X3));
		Cofactors[1] = TVec3<T>::DotProduct(X0, TVec3<T>::CrossProduct(X2, X3));
		Cofactors[2] = -TVec3<T>::DotProduct(X0, TVec3<T>::CrossProduct(X1, X3));
		Cofactors[3] = TVec3<T>::DotProduct(X0, TVec3<T>::CrossProduct(X1, X2));
		T DetM = (Cofactors[0] + Cofactors[1]) + (Cofactors[2] + Cofactors[3]);

		bool bSignMatch[4];
		int32 SubNumVerts[4] = { 3, 3, 3, 3 };
		TVec3<T> SubSimplices[4][3] = { {Simplex[1], Simplex[2], Simplex[3]}, {Simplex[0], Simplex[2], Simplex[3]}, {Simplex[0], Simplex[1], Simplex[3]}, {Simplex[0], Simplex[1], Simplex[2]} };
		TVec3<T> SubAs[4][3] = { {A[1], A[2], A[3]}, {A[0], A[2], A[3]}, {A[0], A[1], A[3]}, {A[0], A[1], A[2]} };
		TVec3<T> SubBs[4][3] = { {B[1], B[2], B[3]}, {B[0], B[2], B[3]}, {B[0], B[1], B[3]}, {B[0], B[1], B[2]} };
		TVec3<T> ClosestPointSub[4];
		T SubBarycentric[4][4];
		int32 ClosestTriangleIdx = INDEX_NONE;
		T MinTriangleDist2 = 0;

		bool bInside = true;
		for (int Idx = 0; Idx < 4; ++Idx)
		{
			bSignMatch[Idx] = SignMatch(DetM, Cofactors[Idx]);
			if (!bSignMatch[Idx])
			{
				bInside = false;
				ClosestPointSub[Idx] = TriangleSimplexFindOrigin2(SubSimplices[Idx], SubNumVerts[Idx], SubBarycentric[Idx], SubAs[Idx], SubBs[Idx]);

				const T Dist2 = ClosestPointSub[Idx].SizeSquared();
				if (ClosestTriangleIdx == INDEX_NONE || Dist2 < MinTriangleDist2)
				{
					MinTriangleDist2 = Dist2;
					ClosestTriangleIdx = Idx;
				}
			}
		}

		if (bInside)
		{
			OutBarycentric[0] = Cofactors[0] / DetM;
			OutBarycentric[1] = Cofactors[1] / DetM;
			OutBarycentric[2] = Cofactors[2] / DetM;
			OutBarycentric[3] = Cofactors[3] / DetM;

			return TVec3<T>(0);
		}

		NumVerts = SubNumVerts[ClosestTriangleIdx];
		for (int i = 0; i < 3; i++)
		{
			OutBarycentric[i] = SubBarycentric[ClosestTriangleIdx][i];
			Simplex[i] = SubSimplices[ClosestTriangleIdx][i];
			A[i] = SubAs[ClosestTriangleIdx][i];
			B[i] = SubBs[ClosestTriangleIdx][i];
		}

		return ClosestPointSub[ClosestTriangleIdx];
	}

	template <typename T>
	void ReorderGJKArray(T* Data, FSimplex& Idxs)
	{
		const T D0 = Data[Idxs[0]];
		const T D1 = Data[Idxs[1]];
		const T D2 = Data[Idxs[2]];
		const T D3 = Data[Idxs[3]];
		Data[0] = D0;
		Data[1] = D1;
		Data[2] = D2;
		Data[3] = D3;
	}

	template <typename T>
	TVec3<T> SimplexFindClosestToOrigin(TVec3<T>* Simplex, FSimplex& Idxs, T* OutBarycentric, TVec3<T>* A = nullptr, TVec3<T>* B = nullptr)
	{
		TVec3<T> ClosestPoint;
		switch (Idxs.NumVerts)
		{
		case 1:
			OutBarycentric[Idxs[0]] = 1;
			ClosestPoint = Simplex[Idxs[0]]; break;
		case 2:
		{
			ClosestPoint = LineSimplexFindOrigin(Simplex, Idxs.Idxs, Idxs.NumVerts, OutBarycentric);
			break;
		}
		case 3:
		{
			ClosestPoint = TriangleSimplexFindOrigin(Simplex, Idxs, OutBarycentric);
			break;
		}
		case 4:
		{
			ClosestPoint = TetrahedronSimplexFindOrigin(Simplex, Idxs, OutBarycentric);
			break;
		}
		default:
			ensure(false);
			ClosestPoint = TVec3<T>(0);
		}

		ReorderGJKArray(Simplex, Idxs);
		ReorderGJKArray(OutBarycentric, Idxs);
		if (A)
		{
			ReorderGJKArray(A, Idxs);
		}

		if (B)
		{
			ReorderGJKArray(B, Idxs);
		}

		Idxs[0] = 0;
		Idxs[1] = 1;
		Idxs[2] = 2;
		Idxs[3] = 3;

		return ClosestPoint;
	}

	template <typename T>
	TVec3<T> SimplexFindClosestToOrigin2(TVec3<T>* Simplex, int32& NumVerts, T* OutBarycentric, TVec3<T>* A, TVec3<T>* B)
	{
		switch (NumVerts)
		{
		case 1:
			OutBarycentric[0] = T(1);
			return Simplex[0];
		case 2:
		{
			return LineSimplexFindOrigin2(Simplex, NumVerts, OutBarycentric, A, B);
		}
		case 3:
		{
			return TriangleSimplexFindOrigin2(Simplex, NumVerts, OutBarycentric, A, B);
		}
		case 4:
		{
			return TetrahedronSimplexFindOrigin2(Simplex, NumVerts, OutBarycentric, A, B);
		}
		default:
			ensure(false);
			return TVec3<T>(0);
		}
	}

}
