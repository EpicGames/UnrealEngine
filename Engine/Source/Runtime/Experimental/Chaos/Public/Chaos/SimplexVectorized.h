// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Math/VectorRegister.h"
#include "Chaos/VectorUtility.h"

namespace Chaos
{

	template <typename T, bool CalculatExtraInformation>
	FORCEINLINE_DEBUGGABLE T VectorLineSimplexFindOrigin(T* RESTRICT Simplex, int32& RESTRICT NumVerts, T& RESTRICT OutBarycentric, T* RESTRICT A, T* RESTRICT B)
	{
		const T& X0 = Simplex[0];
		const T& X1 = Simplex[1];
		const T X0ToX1 = VectorSubtract(X1, X0);

		//Closest Point = (-X0 dot X1-X0) / ||(X1-X0)||^2 * (X1-X0)

		const T X0ToOrigin = VectorNegate(X0);
		const T Dot = VectorDot3(X0ToOrigin, X0ToX1);

		const T IsX0 = VectorCompareGE(TVectorZero<T>(), Dot);

		constexpr T OutBarycentricIfX0OrX1 = TMakeVectorRegisterConstant<T>(1, 0, 0, 0);
		const T X0ToX1Squared = VectorDot3(X0ToX1, X0ToX1);
		const T DotBigger = VectorCompareGE(Dot, X0ToX1Squared);

		const T MinLimt = TMakeVectorRegisterConstant<T>(std::numeric_limits<FRealSingle>::min() , std::numeric_limits<FRealSingle>::min(), std::numeric_limits<FRealSingle>::min(), std::numeric_limits<FRealSingle>::min());
		const T X0ToX1SquaredSmall = VectorCompareGE(MinLimt, X0ToX1Squared);
		const T IsX1 = VectorBitwiseOr(DotBigger, X0ToX1SquaredSmall);

		Simplex[0] = VectorSelect(IsX1, Simplex[1], Simplex[0]);

		if constexpr(CalculatExtraInformation)
		{
			A[0] = VectorSelect(IsX1, A[1], A[0]);
			B[0] = VectorSelect(IsX1, B[1], B[0]);
		}

		T Ratio = VectorDivide(Dot, X0ToX1Squared);

		Ratio = VectorMax(Ratio, TVectorZero<T>());
		Ratio = VectorMin(Ratio, TMakeVectorRegisterConstant<T>(1, 1, 1, 1));

		T Closest = VectorMultiplyAdd(Ratio, X0ToX1, X0);

		const T OneMinusRatio = VectorSubtract(TMakeVectorRegisterConstant<T>(1, 1, 1, 1), Ratio);
		const T OutBarycentricOtherwise = VectorUnpackLo(OneMinusRatio, Ratio);

		Closest = VectorSelect(IsX0, X0, VectorSelect(IsX1, X1, Closest));

		const T IsX0OrX1 = VectorBitwiseOr(IsX0, IsX1);
		NumVerts =  ::VectorMaskBits(IsX0OrX1) ? 1 : NumVerts;

		if constexpr (CalculatExtraInformation)
		{
			OutBarycentric = VectorSelect(IsX0OrX1, OutBarycentricIfX0OrX1, OutBarycentricOtherwise);
		}

		return Closest;
	}

	// Based on an algorithm in Real Time Collision Detection - Ericson (very close to that)
	// Using the same variable name conventions for easy reference
	template <typename T, bool CalculatExtraInformation>
	FORCEINLINE_DEBUGGABLE T TriangleSimplexFindOriginFast(T* RESTRICT Simplex, int32& RESTRICT NumVerts, T& RESTRICT OutBarycentric, T* RESTRICT As, T* RESTRICT Bs)
	{
		const T& A = Simplex[0];
		const T& B = Simplex[1];
		const T& C = Simplex[2];

		const T AB = VectorSubtract(B, A);
		const T AC = VectorSubtract(C, A);

		const T TriNormal = Private::VectorCrossNoFMA(AB, AC);
		const T TriNormal2 = VectorDot3(TriNormal, TriNormal);
		const T MinDouble = TMakeVectorRegisterConstant<T>(std::numeric_limits<float>::min(), std::numeric_limits<float>::min(), std::numeric_limits<float>::min(), std::numeric_limits<float>::min());
		const T AMin = VectorMultiply(A, MinDouble);
		const T Eps2 = VectorDot3(AMin, AMin);
		const T Eps2GENormal2 = VectorCompareGE(Eps2, TriNormal2);
	

		if (::VectorMaskBits(Eps2GENormal2))
		{
			NumVerts = 2;
			return VectorLineSimplexFindOrigin<T, CalculatExtraInformation>(Simplex, NumVerts, OutBarycentric, As, Bs);
		}

		// Vertex region A
		const T AO = VectorNegate(A);

		const T d1 = VectorDot3(AB, AO);
		const T d2 = VectorDot3(AC, AO);

		const T IsD1SEZero = VectorCompareGE(TVectorZero<T>(), d1);
		const T IsD2SEZero = VectorCompareGE(TVectorZero<T>(), d2);
		const T IsA = VectorBitwiseAnd(IsD1SEZero, IsD2SEZero);

		if (::VectorMaskBits(IsA))
		{
			NumVerts = 1;
			if constexpr (CalculatExtraInformation)
			{
				OutBarycentric = TMakeVectorRegisterConstant<T>(1, 0, 0, 0);
			}
			return A;
		}

		//Vertex region B
		const T BO = VectorNegate(B);
		const T d3 = VectorDot3(AB, BO);
		const T d4 = VectorDot3(AC, BO);

		const T IsD3GEZero = VectorCompareGE(d3, TVectorZero<T>());
		const T IsD3GED4 = VectorCompareGE(d3, d4);
		const T IsB = VectorBitwiseAnd(IsD3GEZero, IsD3GED4);

		if (::VectorMaskBits(IsB))
		{
			NumVerts = 1;
			if constexpr (CalculatExtraInformation)
			{
				OutBarycentric = TMakeVectorRegisterConstant<T>(1, 0, 0, 0); 
			}
			Simplex[0] = Simplex[1];
			if constexpr (CalculatExtraInformation)
			{
				As[0] = As[1];
				Bs[0] = Bs[1];
			}
			return B;
		}

		// Edge AB
		const T d1d4 = VectorMultiply(d1, d4);
		const T vc = VectorNegateMultiplyAdd(d3, d2, d1d4);
		const T NormalizationDenominatorAB = VectorSubtract(d1, d3);

		const T IsZeroGEvc = VectorCompareGE(TVectorZero<T>(), vc);
		const T IsD1GEZero = VectorCompareGE(d1, TVectorZero<T>());
		const T IsZeroGED3= VectorCompareGE(TVectorZero<T>(), d3);
		const T IsNDABGTZero = VectorCompareGT(NormalizationDenominatorAB, TVectorZero<T>());
		const T IsAB = VectorBitwiseAnd(VectorBitwiseAnd(IsZeroGEvc, IsD1GEZero), VectorBitwiseAnd(IsZeroGED3, IsNDABGTZero));

		if (::VectorMaskBits(IsAB))
		{
			NumVerts = 2;

			const T v = VectorDivide(d1, NormalizationDenominatorAB);
			const T OneMinusV = VectorSubtract(TMakeVectorRegisterConstant<T>(1, 1, 1, 1), v);
			// b0	a1	a2	a3
			if constexpr (CalculatExtraInformation)
			{
				OutBarycentric = VectorUnpackLo(OneMinusV, v);
			}
			return VectorMultiplyAdd(v, AB, A);
		}

		// Vertex C
		const T CO = VectorNegate(C);
		const T d5 = VectorDot3(AB, CO);
		const T d6 = VectorDot3(AC, CO);
		const T IsD6GEZero = VectorCompareGE(d6, TVectorZero<T>());
		const T IsD6GED5 = VectorCompareGE(d6, d5);
		const T IsC = VectorBitwiseAnd(IsD6GEZero, IsD6GED5);

		if (::VectorMaskBits(IsC))
		{
			NumVerts = 1;
			if constexpr (CalculatExtraInformation)
			{
				OutBarycentric = TMakeVectorRegisterConstant<T>(1, 0, 0, 0);
			}

			Simplex[0] = Simplex[2];
			if constexpr (CalculatExtraInformation)
			{
				As[0] = As[2];
				Bs[0] = Bs[2];
			}
			return C;
		}

		// Edge AC
		const T d5d2 = VectorMultiply(d5, d2);
		const T vb = VectorNegateMultiplyAdd(d1, d6, d5d2);
		const T NormalizationDenominatorAC = VectorSubtract(d2, d6);

		const T IsZeroGEvb = VectorCompareGE(TVectorZero<T>(), vb);
		const T IsD2GEZero = VectorCompareGE(d2, TVectorZero<T>());
		const T IsZeroGED6 = VectorCompareGE(TVectorZero<T>(), d6);
		const T IsNDACGTZero = VectorCompareGT(NormalizationDenominatorAC, TVectorZero<T>());
		const T IsAC = VectorBitwiseAnd(VectorBitwiseAnd(IsZeroGEvb, IsD2GEZero), VectorBitwiseAnd(IsZeroGED6, IsNDACGTZero));

		if (::VectorMaskBits(IsAC))
		{
			const T w = VectorDivide(d2, NormalizationDenominatorAC);
			NumVerts = 2;
			const T OneMinusW = VectorSubtract(TMakeVectorRegisterConstant<T>(1, 1, 1, 1), w);
			// b0	a1	a2	a3
			if constexpr (CalculatExtraInformation)
			{
				OutBarycentric = VectorUnpackLo(OneMinusW, w);
			}
			Simplex[1] = Simplex[2];
			if constexpr (CalculatExtraInformation)
			{
				As[1] = As[2];
				Bs[1] = Bs[2];
			}
			return VectorMultiplyAdd(w, AC, A);
		}

		// Edge BC
		const T d3d6 = VectorMultiply(d3, d6);
		const T va = VectorNegateMultiplyAdd(d5, d4, d3d6);
		const T d4MinusD3 = VectorSubtract(d4, d3);
		const T d5MinusD6 = VectorSubtract(d5, d6);
		const T NormalizationDenominatorBC = VectorAdd(d4MinusD3, d5MinusD6);

		const T IsZeroGEva = VectorCompareGE(TVectorZero<T>(), va);
		const T IsD4MinusD3GEZero = VectorCompareGE(d4MinusD3, TVectorZero<T>());
		const T IsD5MinusD6GEZero = VectorCompareGE(d5MinusD6, TVectorZero<T>());
		const T IsNDBCGTZero = VectorCompareGT(NormalizationDenominatorBC, TVectorZero<T>());
		const T IsBC = VectorBitwiseAnd(VectorBitwiseAnd(IsZeroGEva, IsD4MinusD3GEZero), VectorBitwiseAnd(IsD5MinusD6GEZero, IsNDBCGTZero));

		if (::VectorMaskBits(IsBC))
		{
			NumVerts = 2;
			const T w = VectorDivide(d4MinusD3, NormalizationDenominatorBC);
			if constexpr (CalculatExtraInformation)
			{
				const T OneMinusW = VectorSubtract(TMakeVectorRegisterConstant<T>(1, 1, 1, 1), w);
				// b0	a1	a2	a3
				OutBarycentric = VectorUnpackLo(OneMinusW, w);
			}
			const T CMinusB = VectorSubtract(C, B);
			const T Result = VectorMultiplyAdd(w, CMinusB, B);
			Simplex[0] = Simplex[1];
			Simplex[1] = Simplex[2];
			if constexpr (CalculatExtraInformation)
			{
				As[0] = As[1];
				Bs[0] = Bs[1];
				As[1] = As[2];
				Bs[1] = Bs[2];
			}
			return Result;
		}

		// Inside triangle
		const T denom = VectorDivide(VectorOne(), VectorAdd(va, VectorAdd(vb, vc)));
		T v = VectorMultiply(vb, denom);
		T w = VectorMultiply(vc, denom);
		NumVerts = 3;

		if constexpr (CalculatExtraInformation)
		{
			const T OneMinusVMinusW = VectorSubtract(VectorSubtract(TMakeVectorRegisterConstant<T>(1, 1, 1, 1), v), w);
			// b0	a1	a2	a3
			const T OneMinusVMinusW_W = VectorUnpackLo(OneMinusVMinusW, w);
			// a0	b0	a1	b1
			OutBarycentric = VectorUnpackLo(OneMinusVMinusW_W, v);
		}

		// We know that we are inside the triangle so we can use the projected point we calculated above. 
		// The closest point can also be derived from the barycentric coordinates, but it will contain 
		// numerical error from the determinant calculation  and can cause GJK to terminate with a poor solution.
		// (E.g., this caused jittering when walking on box with dimensions of 100000cm or more).
		// This fix the unit test TestSmallCapsuleLargeBoxGJKRaycast_Vertical
		// Previously was return VectorMultiplyAdd(AC, w, VectorMultiplyAdd(AB, v, A));
		const T TriNormalOverSize2 = VectorDivide(TriNormal, TriNormal2);
		const T SignedDistance = VectorDot3(T(A), TriNormalOverSize2);
		return VectorMultiply(TriNormal, SignedDistance);
	}

	template<typename T>
	FORCEINLINE bool VectorSignMatch(T A, T B)
	{
		T OneIsZero = VectorMultiply(A, B);
		OneIsZero = ::VectorCompareEQ(OneIsZero, TVectorZero<T>());

		const bool IsZero = static_cast<bool>(::VectorMaskBits(OneIsZero));
		const int32 MaskA = ::VectorMaskBits(A);
		const int32 MaskB = ::VectorMaskBits(B);
		return (MaskA == MaskB) && !IsZero;
	}

	template <typename T, bool CalculatExtraInformation>
	FORCEINLINE_DEBUGGABLE T VectorTetrahedronSimplexFindOrigin(T* RESTRICT Simplex, int32& RESTRICT NumVerts, T& RESTRICT OutBarycentric, T* RESTRICT A, T* RESTRICT B)
	{
		const T& X0 = Simplex[0];
		const T& X1 = Simplex[1];
		const T& X2 = Simplex[2];
		const T& X3 = Simplex[3];

		//Use signed volumes to determine if origin is inside or outside
		/*
			M = [X0x X1x X2x X3x;
				 X0y X1y X2y X3y;
				 X0z X1z X2z X3z;
				 1   1   1   1]
		*/

		T Cofactors[4];
		Cofactors[0] = VectorNegate(VectorDot3(X1, Private::VectorCrossNoFMA(X2, X3)));
		Cofactors[1] = VectorDot3(X0, Private::VectorCrossNoFMA(X2, X3));
		Cofactors[2] = VectorNegate(VectorDot3(X0, Private::VectorCrossNoFMA(X1, X3)));
		Cofactors[3] = VectorDot3(X0, Private::VectorCrossNoFMA(X1, X2));
		T DetM = VectorAdd(VectorAdd(Cofactors[0], Cofactors[1]), VectorAdd(Cofactors[2], Cofactors[3]));

		bool bSignMatch[4];
		// constexpr int32 SubIdxs[4][3] = { {1,2,3}, {0,2,3}, {0,1,3}, {0,1,2} };
		int32 SubNumVerts[4] = { 3, 3, 3, 3};
		T SubSimplices[4][3] = { {Simplex[1], Simplex[2], Simplex[3]}, {Simplex[0], Simplex[2], Simplex[3]}, {Simplex[0], Simplex[1], Simplex[3]}, {Simplex[0], Simplex[1], Simplex[2]} };
		T SubAs[4][3];
		T SubBs[4][3];
		if constexpr (CalculatExtraInformation)
		{
			//SubAs = { {A[1], A[2], A[3]}, {A[0], A[2], A[3]}, {A[0], A[1], A[3]}, {A[0], A[1], A[2]} };
			//SubBs = { {B[1], B[2], B[3]}, {B[0], B[2], B[3]}, {B[0], B[1], B[3]}, {B[0], B[1], B[2]} };
			SubAs[0][0] = A[1]; SubAs[0][1] = A[2]; SubAs[0][2] = A[3];
			SubAs[1][0] = A[0]; SubAs[1][1] = A[2]; SubAs[1][2] = A[3];
			SubAs[2][0] = A[0]; SubAs[2][1] = A[1]; SubAs[2][2] = A[3];
			SubAs[3][0] = A[0]; SubAs[3][1] = A[1]; SubAs[3][2] = A[2];

			SubBs[0][0] = B[1]; SubBs[0][1] = B[2]; SubBs[0][2] = B[3];
			SubBs[1][0] = B[0]; SubBs[1][1] = B[2]; SubBs[1][2] = B[3];
			SubBs[2][0] = B[0]; SubBs[2][1] = B[1]; SubBs[2][2] = B[3];
			SubBs[3][0] = B[0]; SubBs[3][1] = B[1]; SubBs[3][2] = B[2];
		}
		T ClosestPointSub[4];
		T SubBarycentric[4];
		int32 ClosestTriangleIdx = INDEX_NONE;
		T MinTriangleDist2 = TVectorZero<T>();


		constexpr int32 IdxSimd[4] = { 0, 1, 2, 3 };

		T Eps = VectorMultiply(GlobalVectorConstants::KindaSmallNumber, VectorDivide(DetM, VectorSet1(4.0f)));

		bool bInside = true;
		for (int Idx = 0; Idx < 4; ++Idx)
		{
			bSignMatch[Idx] = VectorSignMatch<T>(DetM, Cofactors[Idx]);
			if (!bSignMatch[Idx])
			{
				bInside = false;
				ClosestPointSub[Idx] = TriangleSimplexFindOriginFast<T, CalculatExtraInformation>(SubSimplices[Idx], SubNumVerts[Idx], SubBarycentric[Idx], SubAs[Idx], SubBs[Idx]);

				const T Dist2 = VectorDot3(ClosestPointSub[Idx], ClosestPointSub[Idx]);

				const T MinGTDist = VectorCompareGT(MinTriangleDist2, Dist2);

				const bool bFindClosest = static_cast<bool>(::VectorMaskBits(MinGTDist)) || (ClosestTriangleIdx == INDEX_NONE);

				MinTriangleDist2 = bFindClosest ? Dist2 :  MinTriangleDist2;
				ClosestTriangleIdx = bFindClosest ? IdxSimd[Idx] :  ClosestTriangleIdx;
			}
		}

		if (bInside)
		{
			if constexpr (CalculatExtraInformation)
			{
				T OutBarycentricVectors[4];
				const T  InvDetM = VectorDivide(TMakeVectorRegisterConstant<T>(1, 1, 1, 1), DetM);
				OutBarycentricVectors[0] = VectorMultiply(Cofactors[0], InvDetM);
				OutBarycentricVectors[1] = VectorMultiply(Cofactors[1], InvDetM);
				OutBarycentricVectors[2] = VectorMultiply(Cofactors[2], InvDetM);
				OutBarycentricVectors[3] = VectorMultiply(Cofactors[3], InvDetM);
				// a0	b0	a1	b1
				const T OutBarycentric0101 = VectorUnpackLo(OutBarycentricVectors[0], OutBarycentricVectors[1]);
				const T OutBarycentric2323 = VectorUnpackLo(OutBarycentricVectors[2], OutBarycentricVectors[3]);
				// a0	a1	b0	b1
				OutBarycentric = VectorCombineLow(OutBarycentric0101, OutBarycentric2323);
			}

			return TVectorZero<T>();
		}

		NumVerts = SubNumVerts[ClosestTriangleIdx];
		if constexpr (CalculatExtraInformation)
		{
			OutBarycentric = SubBarycentric[ClosestTriangleIdx];
		}

		for (int i = 0; i < 3; i++)
		{
			Simplex[i] = SubSimplices[ClosestTriangleIdx][i];
			if constexpr (CalculatExtraInformation)
			{
				A[i] = SubAs[ClosestTriangleIdx][i];
				B[i] = SubBs[ClosestTriangleIdx][i];
			}
		}

		return ClosestPointSub[ClosestTriangleIdx];
	}


	// CalculatExtraHitInformation : Should we calculate the BaryCentric coordinates, As and Bs?
	template <typename T, bool CalculatExtraInformation = true>
	FORCEINLINE_DEBUGGABLE T VectorSimplexFindClosestToOrigin(T* RESTRICT Simplex, int32& RESTRICT NumVerts, T& RESTRICT OutBarycentric, T* RESTRICT A, T* RESTRICT B)
	{
		switch (NumVerts)
		{
		case 1:
			if constexpr (CalculatExtraInformation)
			{
				OutBarycentric = TMakeVectorRegisterConstant<T>(1, 0, 0 , 0);
			}
			return Simplex[0]; 
		case 2:
		{
			return VectorLineSimplexFindOrigin<T, CalculatExtraInformation>(Simplex, NumVerts, OutBarycentric, A, B);
		}
		case 3:
		{
			return TriangleSimplexFindOriginFast<T, CalculatExtraInformation>(Simplex, NumVerts, OutBarycentric, A, B);
		}
		case 4:
		{
			return VectorTetrahedronSimplexFindOrigin<T, CalculatExtraInformation>(Simplex, NumVerts, OutBarycentric, A, B);
			break;
		}
		default:
			ensure(false);
			return TVectorZero<T>();
		}
	}
}
