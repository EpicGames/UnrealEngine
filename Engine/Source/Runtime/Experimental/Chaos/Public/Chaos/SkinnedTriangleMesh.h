// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ImplicitObject.h"
#include "Chaos/AABB.h"
#include "Chaos/HierarchicalSpatialHash.h"
#include "Chaos/TriangleMesh.h"

struct FKSkinnedTriangleMeshElem;

namespace Chaos
{
	struct FWeightedInfluenceData
	{
		FWeightedInfluenceData()
			: NumInfluences(0)
		{
			FMemory::Memset(BoneIndices, (uint8)INDEX_NONE, sizeof(BoneIndices));
			FMemory::Memset(BoneWeights, 0, sizeof(BoneWeights));
		}

		uint32 GetTypeHash() const
		{
			uint32 Result = ::GetTypeHash(NumInfluences);
			for (uint8 Index = 0; Index < NumInfluences; ++Index)
			{
				Result = HashCombine(Result, ::GetTypeHash(BoneIndices[Index]));
				Result = HashCombine(Result, ::GetTypeHash(BoneWeights[Index]));
			}
			return Result;
		}

		static const uint8 MaxTotalInfluences = 12;
		uint8 NumInfluences;
		uint16 BoneIndices[MaxTotalInfluences]; // This is index into FSkinnedTriangleMesh::UsedBones (which then maps into skeletalmesh bones)
		float BoneWeights[MaxTotalInfluences];
	};

	inline FArchive& operator<<(FArchive& Ar, FWeightedInfluenceData& Value)
	{
		Ar << Value.NumInfluences;
		for (int32 Index = 0; Index < Value.NumInfluences; ++Index)
		{
			Ar << Value.BoneIndices[Index];
			Ar << Value.BoneWeights[Index];
		}
		return Ar;
	}

	/**
	 * Skinned Triangle Mesh
	 */
	class FSkinnedTriangleMesh : public FImplicitObject
	{
	public:
		using ObjectType = FImplicitObjectPtr;
		using FImplicitObject::GetTypeName;

		CHAOS_API FSkinnedTriangleMesh(FTriangleMesh&& TriangleMesh, TArray<FVec3f>&& InReferencePositions, TArray<FWeightedInfluenceData>&& InBoneData, 
			TArray<FName>&& InUsedBones, FTransform&& InReferenceRootTransform, TArray<FTransform>&& InReferenceRelativeTransforms);

		CHAOS_API FSkinnedTriangleMesh(FSkinnedTriangleMesh&& Other);

		virtual ~FSkinnedTriangleMesh() override = default;

		static constexpr EImplicitObjectType StaticType()
		{
			return ImplicitObjectType::SkinnedTriangleMesh;
		}

		virtual const FAABB3 BoundingBox() const override 
		{
			return LocalBoundingBox.IsEmpty() ? FAABB3() : FAABB3(LocalBoundingBox); 
		}
		const TArray<FWeightedInfluenceData>& GetBoneData() const 
		{
			return BoneData; 
		}
		const TArray<FName>& GetUsedBones() const 
		{
			return UsedBones;
		}
		const FTriangleMesh& GetTriangleMesh() const 
		{
			return TriangleMesh;
		}
		// Original reference positions
		const TArray<FVec3f>& GetReferencePositions() const 
		{
			return ReferencePositions;
		}
		// Current skinned positions
		const TArray<FVec3f>& GetLocalPositions() const 
		{
			return LocalPositions;
		}
		// Current Skinned positions for writing
		TArrayView<FVec3f> GetLocalPositions() 
		{
			return TArrayView<FVec3f>(LocalPositions);
		}
		const FTriangleMesh::TSpatialHashType<FRealSingle>& GetSpatialHierarchy() const 
		{
			return SpatialHash;
		}

		// Skin positions
		CHAOS_API void SkinPositions(const TArray<FTransform>& RelativeTransforms, const TArrayView<FVec3f>& Positions) const;
		void SkinPositions(const TArray<FTransform>& RelativeTransforms)
		{
			SkinPositions(RelativeTransforms, GetLocalPositions());
			UpdateLocalBoundingBox();
		}
		void UpdateLocalBoundingBox()
		{
			LocalBoundingBox = CalculateBoundingBox(LocalPositions);
		}
		CHAOS_API void UpdateSpatialHierarchy(const FRealSingle MinLodSize = 0.f);

		virtual void Serialize(FChaosArchive& Ar) override
		{
			SerializeImp(Ar);
		}

		void SerializeImp(FArchive& Ar)
		{
			FImplicitObject::SerializeImp(Ar);
			TriangleMesh.Serialize(Ar);
			Ar << BoneData;
			Ar << UsedBones;
			Ar << ReferenceRootTransform;
			Ar << ReferenceRelativeTransforms;
			Ar << ReferencePositions;
			TBox<FRealSingle, 3>::SerializeAsAABB(Ar, ReferenceBoundingBox);
			if (Ar.IsLoading())
			{
				FinalizeConstruction();
			}
		}

		CHAOS_API virtual FImplicitObjectPtr CopyGeometry() const override;
		CHAOS_API virtual FImplicitObjectPtr CopyGeometryWithScale(const FVec3& Scale) const override;
		CHAOS_API virtual uint32 GetTypeHash() const override;
		virtual FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const override
		{
			// Currently no users of PhiWithNormal. Only used for proximity queries.
			ensure(false);
			return TNumericLimits<FReal>::Max();
		}

	private:
		friend FKSkinnedTriangleMeshElem;

		FSkinnedTriangleMesh()
			:FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::SkinnedTriangleMesh)
		{}

		CHAOS_API TAABB<FRealSingle,3> CalculateBoundingBox(const TConstArrayView<FVec3f>& Positions) const;
		CHAOS_API void FinalizeConstruction();

		// Serialized data. Only non-const because of serialization
		FTriangleMesh TriangleMesh;
		TArray<FWeightedInfluenceData> BoneData;
		TArray<FName> UsedBones;
		FTransform ReferenceRootTransform;
		TArray<FTransform> ReferenceRelativeTransforms; // ReferenceRootTransform * RefBaseMatrixInv(UsedBoneIdx)
		TArray<FVec3f> ReferencePositions;
		TAABB<FRealSingle, 3> ReferenceBoundingBox;

		// Calculated data
		TAABB<FRealSingle, 3> LocalBoundingBox;
		TArray<FVec3f> LocalPositions;
		FTriangleMesh::TSpatialHashType<FRealSingle> SpatialHash;
	};
}