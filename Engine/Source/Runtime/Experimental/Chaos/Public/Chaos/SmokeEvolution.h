// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ArrayFaceND.h"
#include "Chaos/ArrayND.h"
#include "Chaos/KinematicGeometryParticles.h"
#include "Chaos/UniformGrid.h"

#include <cmath>

namespace Chaos
{
template<class T, int d>
class TSmokeEvolution
{
  public:
	TSmokeEvolution(const TUniformGrid<T, d>& Grid, TKinematicGeometryParticles<T, d>&& GeometryParticles, TKinematicGeometryParticles<T, d>&& SourceParticles)
	    : MGrid(Grid), MVelocity(MGrid), MDensity(MGrid), MDirichlet(MGrid), MNeumann(MGrid), MCollisionParticles(MoveTemp(GeometryParticles)), MSourceParticles(MoveTemp(SourceParticles))
	{
		MVelocity.Fill(0);
		MDensity.Fill(0);
	}
	~TSmokeEvolution() = default;

	void AdvanceOneTimeStep(const T Dt);

	inline void AddForceFunction(TFunction<void(const TUniformGrid<T, d>&, TArrayFaceND<T, d>&, const T, const Pair<int32, TVector<int32, d>>)> ForceFunction) { MForceRules.Add(ForceFunction); }
	inline void SetAdvectionFunction(TFunction<void(const TUniformGrid<T, d>&, TArrayND<T, d>&, const TArrayND<T, d>&, const TArrayFaceND<T, d>&, const T, const TVector<int32, d>)> AdvectionFunction) { MAdvectionRule = AdvectionFunction; }
	inline void SetConvectionFunction(TFunction<void(const TUniformGrid<T, d>&, TArrayND<T, d>&, const TArrayND<T, d>&, const TArrayFaceND<T, d>&, const T, const TVector<int32, d>)> ConvectionFunction) { MConvectionRule = ConvectionFunction; }
	inline void SetProjectionFunction(TFunction<void(const TUniformGrid<T, d>&, TArrayFaceND<T, d>&, const TArrayND<bool, d>&, const TArrayFaceND<bool, d>&, const T)> ProjectionFunction) { MProjectionRule = ProjectionFunction; }

	inline const TUniformGrid<T, d>& Grid() const { return MGrid; }
	inline const TArrayFaceND<T, d>& Velocity() const { return MVelocity; }
	inline const TArrayND<T, d>& Density() const { return MDensity; }
	inline TArrayND<T, d>& Density() { return MDensity; }

	const T ComputeDivergence()
	{
		T Divergence = 0;
		for (int32 i = 0; i < MGrid.GetNumCells(); ++i)
		{
			T LocalDivergence = 0;
			for (int32 j = 0; j < d; ++j)
			{
				const TVector<int32, d> index = MGrid.GetIndex(i);
				LocalDivergence += (MVelocity(MakePair(j, MGrid.ClampIndex(index + TVector<int32, d>::AxisVector(j)))) - MVelocity(MakePair(j, index))) / MGrid.Dx()[j];
			}
			Divergence += FGenericPlatformMath::Abs(LocalDivergence);
		}
		return Divergence / (T)MGrid.Counts().Product();
	}

  private:
	TUniformGrid<T, d> MGrid;
	TArrayFaceND<T, d> MVelocity;
	TArrayND<T, d> MDensity;
	TArrayND<bool, d> MDirichlet;
	TArrayFaceND<bool, d> MNeumann;
	TKinematicGeometryParticles<T, d> MCollisionParticles, MSourceParticles;

	TArray<TFunction<void(const TUniformGrid<T, d>&, TArrayFaceND<T, d>&, const T, const Pair<int32, TVector<int32, d>>)>> MForceRules;
	TFunction<void(const TUniformGrid<T, d>&, TArrayND<T, d>&, const TArrayND<T, d>&, const TArrayFaceND<T, d>&, const T, const TVector<int32, d>)> MAdvectionRule;
	TFunction<void(const TUniformGrid<T, d>&, TArrayND<T, d>&, const TArrayND<T, d>&, const TArrayFaceND<T, d>&, const T, const TVector<int32, d>)> MConvectionRule;
	TFunction<void(const TUniformGrid<T, d>&, TArrayFaceND<T, d>&, const TArrayND<bool, d>&, const TArrayFaceND<bool, d>&, const T)> MProjectionRule;
};
}
