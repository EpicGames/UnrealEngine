// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Defines.h"
#include "Chaos/Math/Poisson.h"

namespace Chaos {

	template <class T>
	bool SmoothProject(
		const TConstArrayView<FVec3>& Points,
		const TArray<TVec3<int32>>& Tris,
		const TArray<FVec3>& PointNormals,
		const FVec3& Pos,
		const int32 TriIdx,
		FVec3& Weights,
		TArray<TArray<FVec3>>& TangentBases,
		T* B,
		T* C,
		TVec3<T>& C0,
		TVec3<T>& C1,
		TVec3<T>& C2,
		TVec3<T>& W0,
		TVec3<T>& W1,
		TVec3<T>& W2)
	{
		// Construct orthogonal tangents from point normal
		for (int32 i = 0; i < 3; i++)
		{
			const int32 PtIdx = Tris[TriIdx][i];
			const FVec3& PointNormal = PointNormals[PtIdx];
			TangentBases[i][0] = PointNormal.GetOrthogonalVector().GetSafeNormal();
			TangentBases[i][1] = FVec3::CrossProduct(TangentBases[i][0], PointNormal);
		}
		// Solve
		for (int32 i = 0; i < 3; i++)
		{
			for (int j = 0; j < 3; j++)
			{
				const int32 PtIdx = Tris[TriIdx][j];
				const FVec3& Pt = Points[PtIdx];
				const FVec3& TangentBase0 = TangentBases[i][0];
				const FVec3& TangentBase1 = TangentBases[i][1];
				FReal B0j = FVec3::DotProduct(TangentBase0, Pt);
				FReal B1j = FVec3::DotProduct(TangentBase1, Pt);
				RowMaj3x3Set(B, 0, j, static_cast<T>(B0j));
				RowMaj3x3Set(B, 1, j, static_cast<T>(B1j));
				RowMaj3x3Set(B, 2, j, static_cast<T>(1.0));
			}
			RowMaj3x3Inverse(B, B); // Sets B
			FVec3 PV(
				FVec3::DotProduct(TangentBases[i][0], Pos),
				FVec3::DotProduct(TangentBases[i][1], Pos),
				static_cast<T>(1.0));
			RowMaj3x3SetCol(C, i, RowMaj3x3Multiply(B, PV));
		}

		RowMaj3x3Get(C, 0, 0) -= static_cast<T>(1.0);
		RowMaj3x3Get(C, 1, 1) -= static_cast<T>(1.0);
		RowMaj3x3Get(C, 2, 2) -= static_cast<T>(1.0);

		RowMaj3x3GetRow(C, 0, C0);
		RowMaj3x3GetRow(C, 1, C1);
		RowMaj3x3GetRow(C, 2, C2);

		W0 = FVec3::CrossProduct(C0, C1);
		W1 = FVec3::CrossProduct(C0, C2);
		W2 = FVec3::CrossProduct(C1, C2);

		if (W0[0] >= 0.0 && W0[1] >= 0.0 && W0[2] >= 0.0)
		{
			Weights = W0;
		}
		else if (W1[0] >= 0.0 && W1[1] >= 0.0 && W1[2] >= 0.0)
		{
			Weights = W1;
		}
		else if (W2[0] >= 0.0 && W2[1] >= 0.0 && W2[2] >= 0.0)
		{
			Weights = W2;
		}
		else
		{
			// We're outside of the cone implied by the normals,
			// so the projection doesn't exist.
			return false;
		}

		FReal WeightsSum = Weights[0] + Weights[1] + Weights[2];
		if (fabs(WeightsSum) < 1.0e-6)
		{
			// The point is at the vertex of the normal cone, so the
			// projection isn't unique.  Any face location will do.
			static const T OneThird = static_cast<T>(1.0 / 3);
			Weights = FVec3(OneThird, OneThird, OneThird);
			return true;
		}
		Weights /= WeightsSum;
		return true;
	}

	template <class T>
	bool SmoothProject(
		const TConstArrayView<FVec3>& Points,
		const TArray<TVec3<int32>>& Tris,
		const TArray<FVec3>& PointNormals,
		const FVec3& Pos,
		const int32 TriIdx,
		FVec3& Weights)
	{
		TArray<TArray<FVec3>> TangentBases;
		TangentBases.SetNum(3);
		for (int32 i = 0; i < 3; i++)
		{
			TangentBases[i].SetNum(2);
		}
		T B[9];
		T C[9];
		TVec3<T> C0;
		TVec3<T> C1;
		TVec3<T> C2;
		TVec3<T> W0;
		TVec3<T> W1;
		TVec3<T> W2;

		return SmoothProject(Points, Tris, PointNormals, Pos, TriIdx, Weights,
			TangentBases, &B[0], &C[0], C0, C1, C2, W0, W1, W2);
	}

	template <class T>
	TArray<bool> SmoothProject(
		const TConstArrayView<FVec3>& Points,
		const TArray<TVec3<int32>>& Tris,
		const TArray<FVec3>& PointNormals,
		const FVec3& Pos,
		const TArray<int32>& TriIdx,
		TArray<FVec3>& Weights,
		const bool UseFirstFound = false)
	{
		TArray<TArray<FVec3>> TangentBases;
		TangentBases.SetNum(3);
		for (int32 i = 0; i < 3; i++)
		{
			TangentBases[i].SetNum(2);
		}
		T B[9];
		T C[9];
		TVec3<T> C0;
		TVec3<T> C1;
		TVec3<T> C2;
		TVec3<T> W0;
		TVec3<T> W1;
		TVec3<T> W2;

		TArray<bool> RetVal;
		RetVal.SetNumZeroed(TriIdx.Num());
		Weights.SetNum(TriIdx.Num());
		for (int i = 0; i < TriIdx.Num(); i++)
		{
			RetVal[i] = SmoothProject(Points, Tris, PointNormals, Pos, TriIdx[i], Weights[i],
				TangentBases, &B[0], &C[0], C0, C1, C2, W0, W1, W2);
			if (UseFirstFound && RetVal[i])
			{
				return RetVal;
			}
		}
		return RetVal;
	}

} // namespace Chaos