// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/ArrayCollection.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/PerParticleDampVelocity.h"
#include "Chaos/SoftsEvolutionLinearSystem.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Chaos/SoftsSolverCollisionParticles.h"
#include "Chaos/SoftsSolverCollisionParticlesRange.h"
#include "Chaos/VelocityField.h"
#include "Misc/EnumClassFlags.h"

enum struct EChaosSoftsLocalDampingSpace : uint8;

namespace Chaos::Softs
{

enum struct ESolverMode : uint8
{
	None = 0,
	PBD = 1 << 0,
	ForceBased = 1 << 1
};
ENUM_CLASS_FLAGS(ESolverMode);

/**
 * Per Group context with information about the current solver configuration.
 */
struct FEvolutionGroupContext
{
	ESolverMode SolverMode = ESolverMode::None;
	FSolverReal Dt = (FSolverReal)0.;

	int32 NumPBDIterations = 0;
	int32 CurrentPBDIteration = INDEX_NONE;

	int32 NumNewtonIterations = 0;
	int32 CurrentNewtonIteration = INDEX_NONE;

	void Reset()
	{
		SolverMode = ESolverMode::None;
		Dt = (FSolverReal)0.;

		NumPBDIterations = 0;
		CurrentPBDIteration = INDEX_NONE;

		NumNewtonIterations = 0;
		CurrentNewtonIteration = INDEX_NONE;
	}

	void Init(ESolverMode InSolverMode, FSolverReal InDt, int32 InNumPBDIterations, int32 InNumNewtonIterations)
	{
		SolverMode = InSolverMode;
		Dt = InDt;
		NumPBDIterations = InNumPBDIterations;
		CurrentPBDIteration = INDEX_NONE;
		NumNewtonIterations = InNumNewtonIterations;
		CurrentNewtonIteration = INDEX_NONE;
	}
};

/**
 * Solver can contain multiple "Groups". Groups do not interact with each other. 
 * They may be in different spaces. They may be solved in parallel, completely independently of each other. 
 * The only reason why they're in the same evolution is because they share the same solver
 * settings and step together in time. 
 * 
 * A Group can contain multiple "SoftBodies". SoftBodies can interact but have different 
 * constraint rules/forces.
 */
class FEvolution
{
public:

	CHAOS_API FEvolution(const FCollectionPropertyConstFacade& Properties);
	~FEvolution() = default;

	/** Reset/empty everything.
	 * @param bResetGlobalRules whether or not to reset global rules (e.g., KinematicUpdate) */
	CHAOS_API void Reset(bool bResetGlobalRules = true);

	/** Move forward in time */
	CHAOS_API void AdvanceOneTimeStep(const FSolverReal Dt, const FSolverReal TimeDependentIterationMultiplier);

	/** Add custom collection arrays */
	void AddGroupArray(TArrayCollectionArrayBase* Array) { Groups.AddArray(Array); }
	void AddParticleArray(TArrayCollectionArrayBase* Array) { Particles.AddArray(Array); }
	void AddCollisionParticleArray(TArrayCollectionArrayBase* Array) { CollisionParticles.AddArray(Array); }

	const FSolverParticles& GetParticles() const { return Particles; }
	// Giving non-const access so data can be set freely, but do not add or remove particles here. Use AddSoftBody
	FSolverParticles& GetParticles() { return Particles; }

	UE_DEPRECATED(5.6, "Use GetActiveGroupsArray instead")
	TSet<uint32> GetActiveGroups() const 
	{
		return TSet<uint32>(GetActiveGroupsArray());
	}
	CHAOS_API TArray<uint32> GetActiveGroupsArray() const;
	CHAOS_API int32 NumActiveParticles() const;

	const FEvolutionGroupContext& GetGroupContext(uint32 GroupId) const
	{
		return Groups.SolverContexts[GroupId];
	}

	// Convenience method to get by SoftBodyId
	const FEvolutionGroupContext& GetGroupContextForSoftBody(int32 SoftBodyId) const
	{
		return Groups.SolverContexts[SoftBodies.GroupId[SoftBodyId]];
	}

	/** 
	 * Add a SoftBody to a Group. Adding and removing soft bodies is not threadsafe.
	 * 
	 * @return SoftBodyId
	 */
	CHAOS_API int32 AddSoftBody(uint32 GroupId, int32 NumParticles, bool bEnable);
	// TODO: add garbage collection. Currently soft bodies and their particles are recycled only if another softbody with the exact same number of particles is requested. 
	// This is the common use case with cloth collision ranges, but we will likely need something more sophisticated.
	// (Or we could have each SoftBody own its own SolverParticles--we don't take advantage of a single Particle list anywhere)
	CHAOS_API void RemoveSoftBody(int32 SoftBodyId);
	int32 GetSoftBodyParticleNum(int32 SoftBodyId) const { return SoftBodies.ParticleRanges[SoftBodyId].GetRangeSize(); }
	int32 GetSoftBodyGroupId(int32 SoftBodyId) const { return SoftBodies.GroupId[SoftBodyId]; }
	CHAOS_API void SetSoftBodyProperties(int32 SoftBodyId, const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FSolverVec3& ReferenceSpaceLocation,
		const FSolverVec3& ReferenceSpaceVelocity,
		const FSolverVec3& ReferenceSpaceAngularVelocity
	);
	UE_DEPRECATED(5.6, "Use version with ReferenceSpace parameters. Otherwise, LocalSpace damping with ReferenceBone space will not work correctly.")
	void SetSoftBodyProperties(int32 SoftBodyId, const FCollectionPropertyConstFacade& PropertyCollection,
			const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
	{
		SetSoftBodyProperties(SoftBodyId, PropertyCollection, WeightMaps, FSolverVec3(0.f), FSolverVec3(0.f), FSolverVec3(0.f));
	}
	// Activating/deactivating Soft Bodies in different groups is threadsafe. Activations within a group is not threadsafe.
	CHAOS_API void ActivateSoftBody(int32 SoftBodyId, bool bActivate);
	bool IsSoftBodyActive(int32 SoftBodyId) const { return SoftBodies.Status[SoftBodyId] == FSoftBodies::EStatus::Active; }
	FSolverParticlesRange& GetSoftBodyParticles(int32 SoftBodyId) { return SoftBodies.ParticleRanges[SoftBodyId]; }
	const FSolverParticlesRange& GetSoftBodyParticles(int32 SoftBodyId) const { return SoftBodies.ParticleRanges[SoftBodyId]; }
	const TArray<int32>& GetGroupSoftBodies(uint32 GroupId) const { return Groups.SoftBodies[GroupId]; }
	const TSet<int32>& GetGroupActiveSoftBodies(uint32 GroupId) const { return Groups.ActiveSoftBodies[GroupId]; }
	int32 GetLastLinearSolveIterations(int32 SoftBodyId) const { return SoftBodies.LinearSystems[SoftBodyId].GetLastSolveIterations(); }
	FSolverReal GetLastLinearSolveError(int32 SoftBodyId) const { return SoftBodies.LinearSystems[SoftBodyId].GetLastSolveError(); }
	
	/**
	 * Add Collision particle range to a group. Adding and removing collision particle ranges is not threadsafe.
	 * 
	 * @return Particle range offset (unique id for this range)
	 */
	CHAOS_API int32 AddCollisionParticleRange(uint32 GroupId, int32 NumParticles, bool bEnable);
	CHAOS_API void RemoveCollisionParticleRange(int32 CollisionRangeId);
	// Activating/deactivating Collision Particle ranges in different groups is threadsafe. Activations within a group is not threadsafe.
	CHAOS_API void ActivateCollisionParticleRange(int32 CollisionRangeId, bool bEnable);
	const TSet<int32>& GetGroupActiveCollisionParticleRanges(uint32 GroupId) const { return Groups.ActiveCollisionParticleRanges[GroupId]; }
	CHAOS_API TArray<FSolverCollisionParticlesRange> GetActiveCollisionParticles(uint32 GroupId) const;

	bool IsValidCollisionParticleRange(int32 CollisionRangeId) const { return CollisionRanges.ParticleRanges.IsValidIndex(CollisionRangeId); }
	FSolverCollisionParticlesRange& GetCollisionParticleRange(int32 CollisionRangeId) { return CollisionRanges.ParticleRanges[CollisionRangeId]; }
	const FSolverCollisionParticlesRange& GetCollisionParticleRange(int32 CollisionRangeId) const { return CollisionRanges.ParticleRanges[CollisionRangeId]; }
	
	/** Global Rules*/
	typedef TFunction<void(FSolverParticlesRange&, const FSolverReal Dt, const FSolverReal Time)> KinematicUpdateFunc;
	typedef TFunction<void(FSolverCollisionParticlesRange&, const FSolverReal Dt, const FSolverReal Time)> CollisionKinematicUpdateFunc;
	void SetKinematicUpdateFunction(KinematicUpdateFunc Func) { KinematicUpdate = Func; }
	void SetCollisionKinematicUpdateFunction(CollisionKinematicUpdateFunc Func) { CollisionKinematicUpdate = Func; }

	/** Soft Body Rules. */
	typedef TFunction<void(const FSolverParticlesRange&, const FSolverReal Dt, const ESolverMode)> ParallelInitFunc;
	typedef TFunction<void(FSolverParticlesRange&, const FSolverReal Dt, const ESolverMode)> ConstraintRuleFunc;
	typedef TFunction<void(FSolverParticlesRange&, const FSolverReal Dt)> PBDConstraintRuleFunc;
	typedef TFunction<void(FSolverParticlesRange&, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>&)> PBDCollisionConstraintRuleFunc;
	typedef TFunction<void(const FSolverParticlesRange&, const FSolverReal Dt, FEvolutionLinearSystem&)> UpdateLinearSystemFunc;
	typedef TFunction<void(const FSolverParticlesRange&, const FSolverReal Dt, const TArray<FSolverCollisionParticlesRange>&, FEvolutionLinearSystem&)> UpdateLinearSystemCollisionsFunc;

	/* Add Ranges to allocate space for your rules. */

	// Presubstep init methods (always run at beginning of substep)
	void AllocatePreSubstepParallelInitRange(int32 SoftBodyId, int32 NumRules)
	{
		return AllocateRules(SoftBodyId, NumRules, SoftBodies.PreSubstepParallelInits);
	}
	// PBD Rules that apply external forces (only run if doing PBD)
	void AllocatePBDExternalForceRulesRange(int32 SoftBodyId, int32 NumRules)
	{
		return AllocateRules(SoftBodyId, NumRules, SoftBodies.PBDExternalForceRules);
	}
	// Post initial guess init methods (always run after kinematic and initial guess update, before any solving.) 
	void AllocatePostInitialGuessParallelInitRange(int32 SoftBodyId, int32 NumRules)
	{
		return AllocateRules(SoftBodyId, NumRules, SoftBodies.PostInitialGuessParallelInits);
	}
	// Rules that run once per substep after all initial guess and initialization is done.
	void AllocatePreSubstepConstraintRulesRange(int32 SoftBodyId, int32 NumRules)
	{
		return AllocateRules(SoftBodyId, NumRules, SoftBodies.PreSubstepConstraintRules);
	}
	// Normal per-iteration PBD rules (only run if doing PBD)
	void AllocatePerIterationPBDConstraintRulesRange(int32 SoftBodyId, int32 NumRules)
	{
		return AllocateRules(SoftBodyId, NumRules, SoftBodies.PerIterationPBDConstraintRules);
	}
	// Collision per-iteration PBD rules (only run if doing PBD)
	void AllocatePerIterationCollisionPBDConstraintRulesRange(int32 SoftBodyId, int32 NumRules)
	{
		return AllocateRules(SoftBodyId, NumRules, SoftBodies.PerIterationCollisionPBDConstraintRules);
	}
	// Normal per-iteration PBD rules that run after collisions (only run if doing PBD)
	void AllocatePerIterationPostCollisionsPBDConstraintRulesRange(int32 SoftBodyId, int32 NumRules)
	{
		return AllocateRules(SoftBodyId, NumRules, SoftBodies.PerIterationPostCollisionsPBDConstraintRules);
	}
	// Linear system rules (only run if doing ForceBased)
	void AllocateUpdateLinearSystemRulesRange(int32 SoftBodyId, int32 NumRules)
	{
		return AllocateRules(SoftBodyId, NumRules, SoftBodies.UpdateLinearSystemRules);
	}
	// Linear system collision rules (only run if doing ForceBased)
	void AllocateUpdateLinearSystemCollisionsRulesRange(int32 SoftBodyId, int32 NumRules)
	{
		return AllocateRules(SoftBodyId, NumRules, SoftBodies.UpdateLinearSystemCollisionsRules);
	}
	// Post substep rules (always run at end of substep)
	void AllocatePostSubstepConstraintRulesRange(int32 SoftBodyId, int32 NumRules)
	{
		return AllocateRules(SoftBodyId, NumRules, SoftBodies.PostSubstepConstraintRules);
	}

	// Presubstep init methods (always run at beginning of substep)
	TArrayView<ParallelInitFunc> GetPreSubstepParallelInitRange(int32 SoftBodyId)
	{
		return GetRules(SoftBodyId, SoftBodies.PreSubstepParallelInits);
	}
	// PBD Rules that apply external forces (only run if doing PBD)
	TArrayView<PBDConstraintRuleFunc> GetPBDExternalForceRulesRange(int32 SoftBodyId)
	{
		return GetRules(SoftBodyId, SoftBodies.PBDExternalForceRules);
	}
	// Post initial guess init methods (always run after kinematic and initial guess update, before any solving.) 
	TArrayView<ParallelInitFunc> GetPostInitialGuessParallelInitRange(int32 SoftBodyId)
	{
		return GetRules(SoftBodyId, SoftBodies.PostInitialGuessParallelInits);
	}
	// Rules that run once per substep after all initial guess and initialization is done.
	TArrayView<ConstraintRuleFunc> GetPreSubstepConstraintRulesRange(int32 SoftBodyId)
	{
		return GetRules(SoftBodyId, SoftBodies.PreSubstepConstraintRules);
	}
	// Normal per-iteration PBD rules (only run if doing PBD)
	TArrayView<PBDConstraintRuleFunc> GetPerIterationPBDConstraintRulesRange(int32 SoftBodyId)
	{
		return GetRules(SoftBodyId, SoftBodies.PerIterationPBDConstraintRules);
	}
	// Collision per-iteration PBD rules (only run if doing PBD)
	TArrayView<PBDCollisionConstraintRuleFunc> GetPerIterationCollisionPBDConstraintRulesRange(int32 SoftBodyId)
	{
		return GetRules(SoftBodyId, SoftBodies.PerIterationCollisionPBDConstraintRules);
	}
	// Normal per-iteration PBD rules that run after collisions (only run if doing PBD)
	TArrayView<PBDConstraintRuleFunc> GetPerIterationPostCollisionsPBDConstraintRulesRange(int32 SoftBodyId)
	{
		return GetRules(SoftBodyId, SoftBodies.PerIterationPostCollisionsPBDConstraintRules);
	}
	// Linear system rules (only run if doing ForceBased)
	TArrayView<UpdateLinearSystemFunc> GetUpdateLinearSystemRulesRange(int32 SoftBodyId)
	{
		return GetRules(SoftBodyId, SoftBodies.UpdateLinearSystemRules);
	}
	// Linear system collision rules (only run if doing ForceBased)
	TArrayView<UpdateLinearSystemCollisionsFunc> GetUpdateLinearSystemCollisionsRulesRange(int32 SoftBodyId)
	{
		return GetRules(SoftBodyId, SoftBodies.UpdateLinearSystemCollisionsRules);
	}
	// Post substep rules (always run at end of substep)
	TArrayView<ConstraintRuleFunc> GetPostSubstepConstraintRulesRange(int32 SoftBodyId)
	{
		return GetRules(SoftBodyId, SoftBodies.PostSubstepConstraintRules);
	}

	/** Solver settings */
	FSolverReal GetTime() const { return Time; }
	int32 GetIterations() const { return NumIterations; }
	int32 GetMaxIterations() const { return MaxNumIterations; }
	int32 GetNumUsedIterations() const { return NumUsedIterations; }
	bool GetDisableTimeDependentNumIterations() const { return bDisableTimeDependentNumIterations; }
	bool GetDoQuasistatics() const { return bDoQuasistatics; }
	void SetDisableTimeDependentNumIterations(bool bDisable) { bDisableTimeDependentNumIterations = bDisable; }
	CHAOS_API void SetSolverProperties(const FCollectionPropertyConstFacade& PropertyCollection);

private:

	template<typename ElementType>
	struct TArrayRange
	{
		static TArrayRange AddRange(TArray<ElementType>& InArray, int32 InRangeSize)
		{
			TArrayRange Range;
			Range.Offset = InArray.Num();
			Range.Array = &InArray;
			Range.Array->AddDefaulted(InRangeSize);
			Range.RangeSize = InRangeSize;
			return Range;
		}

		bool IsValid() const
		{
			return RangeSize == 0 || (Array && Offset >= 0 && Offset + RangeSize <= Array->Num());
		}

		TConstArrayView<ElementType> GetConstArrayView() const
		{
			check(IsValid());
			return TConstArrayView<ElementType>(Array->GetData() + Offset, RangeSize);
		}

		TArrayView<ElementType> GetArrayView()
		{
			check(IsValid());
			return TArrayView<ElementType>(Array->GetData() + Offset, RangeSize);
		}

		bool IsEmpty() const { return RangeSize == 0; }
		int32 GetRangeSize() const { return RangeSize; }
	private:
		TArray<ElementType>* Array = nullptr;
		int32 Offset = INDEX_NONE;
		int32 RangeSize = 0;
	};

	// SoftBody SOA
	struct FSoftBodies : public TArrayCollection
	{
		FSoftBodies()
		{
			TArrayCollection::AddArray(&Status);
			TArrayCollection::AddArray(&GroupId);
			TArrayCollection::AddArray(&ParticleRanges);
			TArrayCollection::AddArray(&GlobalDampings);
			TArrayCollection::AddArray(&LocalDamping);
			TArrayCollection::AddArray(&UsePerParticleDamping);
			TArrayCollection::AddArray(&LinearSystems);

			TArrayCollection::AddArray(&PreSubstepParallelInits);
			TArrayCollection::AddArray(&PBDExternalForceRules);
			TArrayCollection::AddArray(&PostInitialGuessParallelInits);
			TArrayCollection::AddArray(&PreSubstepConstraintRules);
			TArrayCollection::AddArray(&PerIterationPBDConstraintRules);
			TArrayCollection::AddArray(&PerIterationCollisionPBDConstraintRules);
			TArrayCollection::AddArray(&PerIterationPostCollisionsPBDConstraintRules);
			TArrayCollection::AddArray(&UpdateLinearSystemRules);
			TArrayCollection::AddArray(&UpdateLinearSystemCollisionsRules);
			TArrayCollection::AddArray(&PostSubstepConstraintRules);
		}

		void Reset()
		{
			ResizeHelper(0);
		}

		int32 AddSoftBody()
		{
			const int32 Offset = Size();
			AddElementsHelper(1);
			return Offset;
		}

		enum struct EStatus : uint8
		{
			Invalid = 0,
			Active = 1,
			Inactive = 2,
			Free = 3 // Available for recycling
		};
		TArrayCollectionArray<EStatus> Status;
		TArrayCollectionArray<uint32> GroupId;
		TArrayCollectionArray<FSolverParticlesRange> ParticleRanges;
		TArrayCollectionArray<FSolverReal> GlobalDampings;
		TArrayCollectionArray<FPerParticleDampVelocity> LocalDamping;
		TArrayCollectionArray<bool> UsePerParticleDamping;
		TArrayCollectionArray<FEvolutionLinearSystem> LinearSystems;

		TArrayCollectionArray<TArray<ParallelInitFunc>> PreSubstepParallelInits;
		TArrayCollectionArray<TArray<PBDConstraintRuleFunc>> PBDExternalForceRules;
		TArrayCollectionArray<TArray<ParallelInitFunc>> PostInitialGuessParallelInits;
		TArrayCollectionArray<TArray<ConstraintRuleFunc>> PreSubstepConstraintRules;
		TArrayCollectionArray<TArray<PBDConstraintRuleFunc>> PerIterationPBDConstraintRules;
		TArrayCollectionArray<TArray<PBDCollisionConstraintRuleFunc>> PerIterationCollisionPBDConstraintRules;
		TArrayCollectionArray<TArray<PBDConstraintRuleFunc>> PerIterationPostCollisionsPBDConstraintRules;
		TArrayCollectionArray<TArray<UpdateLinearSystemFunc>> UpdateLinearSystemRules;
		TArrayCollectionArray<TArray<UpdateLinearSystemCollisionsFunc>> UpdateLinearSystemCollisionsRules;
		TArrayCollectionArray<TArray<ConstraintRuleFunc>> PostSubstepConstraintRules;
	};

	// CollisionBodyRange SOA
	struct FCollisionBodyRanges : public TArrayCollection
	{
		FCollisionBodyRanges()
		{
			TArrayCollection::AddArray(&Status);
			TArrayCollection::AddArray(&GroupId);
			TArrayCollection::AddArray(&ParticleRanges);
		}

		void Reset()
		{
			ResizeHelper(0);
		}

		int32 AddRange()
		{
			const int32 Offset = Size();
			AddElementsHelper(1);
			return Offset;
		}

		enum struct EStatus : uint8
		{
			Invalid = 0,
			Active = 1,
			Inactive = 2,
			Free = 3 // Available for recycling
		};
		TArrayCollectionArray<EStatus> Status;
		TArrayCollectionArray<uint32> GroupId;
		TArrayCollectionArray<FSolverCollisionParticlesRange> ParticleRanges;
	};

	struct FGroups : public TArrayCollection
	{
		FGroups()
		{
			TArrayCollection::AddArray(&SoftBodies);
			TArrayCollection::AddArray(&ActiveSoftBodies);
			TArrayCollection::AddArray(&ActiveCollisionParticleRanges);
			TArrayCollection::AddArray(&SolverContexts);
		}
		
		void Reset()
		{
			ResizeHelper(0);
		}

		void AddGroupsToSize(uint32 DesiredSize)
		{
			if (ensure(DesiredSize >= Size()))
			{
				ResizeHelper((int32)DesiredSize);
			}
		}

		TArrayCollectionArray<TArray<int32>> SoftBodies;
		TArrayCollectionArray<TSet<int32>> ActiveSoftBodies;
		TArrayCollectionArray<TSet<int32>> ActiveCollisionParticleRanges;
		TArrayCollectionArray<FEvolutionGroupContext> SolverContexts;
	};

	// Wrapper around FEvolutionLinearSystemSolverParameters that knows how to read a property collection
	struct FLinearSystemParameters : public FEvolutionLinearSystemSolverParameters
	{
		typedef FEvolutionLinearSystemSolverParameters Base;

		FLinearSystemParameters()
			: Base()
		{}

		FLinearSystemParameters(const FCollectionPropertyConstFacade& PropertyCollection, bool bInXPBDInitialGuess)
			: Base(GetDoQuasistatics(PropertyCollection, false)
				, bInXPBDInitialGuess
				, GetMaxNumCGIterations(PropertyCollection, DefaultMaxNumCGIterations)
				, GetCGResidualTolerance(PropertyCollection, DefaultCGTolerance)
				, GetCheckCGResidual(PropertyCollection, bDefaultCheckCGResidual))
		{}

		void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection, bool bInXPBDInitialGuess)
		{
			bXPBDInitialGuess = bInXPBDInitialGuess;
			bDoQuasistatics = GetDoQuasistatics(PropertyCollection, false);
			MaxNumCGIterations = GetMaxNumCGIterations(PropertyCollection, DefaultMaxNumCGIterations);
			CGResidualTolerance = GetCGResidualTolerance(PropertyCollection, DefaultCGTolerance);
			bCheckCGResidual = GetCheckCGResidual(PropertyCollection, bDefaultCheckCGResidual);
		}

		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(DoQuasistatics, bool);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(MaxNumCGIterations, int32);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(CGResidualTolerance, float);
		UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(CheckCGResidual, bool);
	};

	template<typename RuleFunc>
	void AllocateRules(int32 SoftBodyId, int32 NumRules, TArrayCollectionArray<TArray<RuleFunc>>& RuleArray)
	{
		check(RuleArray[SoftBodyId].IsEmpty());
		RuleArray[SoftBodyId].SetNum(NumRules);
	}

	template<typename RuleFunc>
	TArrayView<RuleFunc> GetRules(int32 SoftBodyId, TArrayCollectionArray<TArray<RuleFunc>>& RuleArray)
	{
		return TArrayView<RuleFunc>(RuleArray[SoftBodyId]);
	}

	void AdvanceOneTimeStepInternal(const FSolverReal Dt, const int32 TimeDependentNumIterations, uint32 GroupId);

	// Solver data
	FSolverReal Time;
	bool bEnableForceBasedSolver = false;
	int32 MaxNumIterations; // Used for time-dependent iteration counts
	int32 NumIterations; // PBD iterations
	int32 NumUsedIterations = 0; // Last actual time-dependent iteration count
	int32 NumNewtonIterations; // Implicit force-based solve
	bool bDisableTimeDependentNumIterations = false;
	bool bDoQuasistatics;
	FSolverReal SolverFrequency; 
	FLinearSystemParameters LinearSystemParameters; // Per-solver parameters that need to be passed to the linear system solver


	// Per-Particle data
	FSolverParticles Particles;
	TArrayCollectionArray<FSolverReal> ParticleDampings;

	// Per-Collision particle data
	FSolverCollisionParticles CollisionParticles;

	// Per-SoftBody data
	FSoftBodies SoftBodies;

	// SoftBody free-list
	TMap<int32, TArray<int32>> SoftBodyFreeList; // Key = NumParticles, Value = SoftBodyId(s)

	// Per-CollisionBodyRange data
	FCollisionBodyRanges CollisionRanges;

	// Collision Range free-list
	TMap<int32, TArray<int32>> CollisionRangeFreeList; // Key = NumParticles, Value = CollisionRangeId(s)

	// Per-Group data
	FGroups Groups;

	KinematicUpdateFunc KinematicUpdate;
	CollisionKinematicUpdateFunc CollisionKinematicUpdate;

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(DampingCoefficient, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(LocalDampingCoefficient, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(MaxNumIterations, int32);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(NumIterations, int32);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(DoQuasistatics, bool);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(SolverFrequency, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(EnableForceBasedSolver, bool);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(NumNewtonIterations, int32);
};
}