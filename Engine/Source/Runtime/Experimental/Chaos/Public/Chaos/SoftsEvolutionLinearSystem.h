// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/Math/BlockSparseLinearSystem.h"

namespace Chaos::Softs
{
struct FEvolutionLinearSystemSolverParameters
{
	static constexpr int32 DefaultMaxNumCGIterations = 50;
	static constexpr FSolverReal DefaultCGTolerance = (FSolverReal)1e-4;
	static constexpr bool bDefaultCheckCGResidual = false;

	FEvolutionLinearSystemSolverParameters() = default;
	FEvolutionLinearSystemSolverParameters(
		bool bInDoQuasistatics,
		bool bInXPBDInitialGuess,
		int32 InMaxNumCGIterations,
		FSolverReal InCGResidualTolerance,
		bool bInCheckResidual)
		: bDoQuasistatics(bInDoQuasistatics)
		, bXPBDInitialGuess(bInXPBDInitialGuess)
		, MaxNumCGIterations(InMaxNumCGIterations)
		, CGResidualTolerance(InCGResidualTolerance)
		, bCheckCGResidual(bInCheckResidual)
	{}
	FEvolutionLinearSystemSolverParameters(const FEvolutionLinearSystemSolverParameters&) = default;
	FEvolutionLinearSystemSolverParameters(FEvolutionLinearSystemSolverParameters&&) = default;
	~FEvolutionLinearSystemSolverParameters() = default;
	FEvolutionLinearSystemSolverParameters& operator=(const FEvolutionLinearSystemSolverParameters&) = default;
	FEvolutionLinearSystemSolverParameters& operator=(FEvolutionLinearSystemSolverParameters&&) = default;

	bool bDoQuasistatics = false;
	bool bXPBDInitialGuess = false;
	int32 MaxNumCGIterations = DefaultMaxNumCGIterations;
	FSolverReal CGResidualTolerance = DefaultCGTolerance;
	bool bCheckCGResidual = bDefaultCheckCGResidual;
};

// Assemble and solve the linear system generated by the force based solver in FEvolution.
// Currently just supports using CG to solve the linear system generated by using newton's method with backward euler.
// The matrix format and CG is controlled by BlockSparseLinearSystem. 
class FEvolutionLinearSystem
{
public:
	CHAOS_API FEvolutionLinearSystem(const FSolverParticlesRange& Particles);
	FEvolutionLinearSystem() = default;
	FEvolutionLinearSystem(const FEvolutionLinearSystem&) = delete;
	FEvolutionLinearSystem(FEvolutionLinearSystem&& Other)
		: Parameters(Other.Parameters)
		, NumCompactIndices(Other.NumCompactIndices)
		, CompactifiedIndices(MoveTemp(Other.CompactifiedIndices))
		, RHS(MoveTemp(Other.RHS))
		, Matrix(MoveTemp(Other.Matrix))
		, bDfDxTimesVTerm(Other.bDfDxTimesVTerm)
	{}
	FEvolutionLinearSystem& operator=(const FEvolutionLinearSystem&) = delete;
	FEvolutionLinearSystem& operator=(FEvolutionLinearSystem&& Other)
	{
		Parameters = Other.Parameters;
		NumCompactIndices = Other.NumCompactIndices;
		CompactifiedIndices = MoveTemp(Other.CompactifiedIndices);
		RHS = MoveTemp(Other.RHS);
		Matrix = MoveTemp(Other.Matrix);
		bDfDxTimesVTerm = Other.bDfDxTimesVTerm;
		return *this;
	}

	~FEvolutionLinearSystem() = default;

	CHAOS_API void Init(const FSolverParticlesRange& Particles, const FSolverReal Dt, bool bInFirstNewtonIteration, const FEvolutionLinearSystemSolverParameters& Params);

	void ReserveForParallelAdd(const int32 NumDiagonalEntries, const int32 NumOffDiagonalEntries)
	{
		Matrix.ReserveForParallelAdd(NumDiagonalEntries, NumOffDiagonalEntries);
	}

	CHAOS_API void AddForce(const FSolverParticlesRange& Particles, const FSolverVec3& Force, int32 ParticleIndex, const FSolverReal Dt);

	// Df1Dx2 is Derivative of F for ParticleIndex1 with respect to X of Particle2.
	// Components of Df1Dx2 are Df1Dx2[j][i] = DF1_i / DX2_j
	// This is consistent with UnrealMath which assumes row-major matrices and treats positions as row vectors (left multiply)
	// 
	// We're assuming all forces are symmetric. This will add Df1Dfx2 and Df2Dfx1 = Df1Dfx2^T (and same for Df2Dv1 = Df1Dv2^T) when ParticleIndex1 != ParticleIndex2
	CHAOS_API void AddSymmetricForceDerivative(const FSolverParticlesRange& Particles, const FSolverMatrix33* const Df1Dx2, const FSolverMatrix33* const Df1Dv2, int32 ParticleIndex1, int32 ParticleIndex2, const FSolverReal Dt);

	bool Solve(FSolverParticlesRange& Particles, const FSolverReal Dt);

	int32 GetLastSolveIterations() const 
	{
		return LastSolveIterations;
	}

	FSolverReal GetLastSolveError() const
	{
		return LastSolveError;
	}

	bool RequiresSPDForceDerivatives() const
	{
		// we're currently just doing CG to solve the primal equations, so we need to be symmetric positive definite
		return true;
	}
private:
	void CalculateCompactIndices(const FSolverParticlesRange& Particles);

	const FEvolutionLinearSystemSolverParameters* Parameters = nullptr;
	int32 NumCompactIndices;
	TArray<int32> CompactifiedIndices; // ParticleRangeIndex -> Dense array with kinematic points removed.
	TArray<FSolverVec3> RHS;
	TBlockSparseSymmetricLinearSystem<FSolverReal,3> Matrix;
	bool bDfDxTimesVTerm = false;

	// Debug reporting
	int32 LastSolveIterations = 0;
	FSolverReal LastSolveError = (FSolverReal)0.;
};

}  // End namespace Chaos::Softs
