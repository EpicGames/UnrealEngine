// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDFlatWeightMap.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsEvolutionLinearSystem.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Chaos/CollectionPropertyFacade.h"

namespace Chaos::Softs
{
class FExternalForcesBase
{
public:
	FExternalForcesBase(const FSolverParticlesRange& Particles,
		const FSolverVec3& InGravity,
		const FSolverVec2& InGravityScale,
		const TConstArrayView<FRealSingle>& InGravityScaleMultipliers,
		const TArray<FSolverVec3>& InNormals)
		: Gravity(InGravity)
		, bApplyGravityScale(true)
		, GravityScale(InGravityScale,
			InGravityScaleMultipliers,
			Particles.GetRangeSize())
		, FictitiousAngularVelocity(0.f)
		, ReferenceSpaceLocation(0.f)
		, bUsePointBasedWindModel(false)
		, PointBasedWind(0.f)
		, LegacyWindAdaptation(0.f)
		, Normals(InNormals)
	{}

	virtual ~FExternalForcesBase() = default;

	void Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FExternalForcesBase_Apply);
		FSolverVec3* const Acceleration = Particles.GetAcceleration().GetData();
		const FSolverReal* const InvM = Particles.GetInvM().GetData();
		const FSolverVec3* const X = Particles.XArray().GetData();
		const FSolverVec3* const V = Particles.GetV().GetData();
		const FSolverVec3* const N = Particles.GetConstArrayView(Normals).GetData();

		const bool bHasFictitiousForces = !FictitiousAngularVelocity.IsNearlyZero();

		const bool bHasPerParticleGravity = HasPerParticleGravity();
		const FSolverReal GravityScaleConstant = bApplyGravityScale ? (FSolverReal)GravityScale : (FSolverReal)1.f;
		const FSolverVec2& GravityScaleOffsetRange = GravityScale.GetOffsetRange();
		const FSolverReal* const GravityScaleMultipliers = GravityScale.GetMapValues().GetData();
		check(!bHasPerParticleGravity || (GravityScale.GetMapValues().Num() == Particles.GetRangeSize()));

		for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
		{
			if (InvM[Index] != (FSolverReal)0.)
			{
				const FSolverReal PerParticleGravityScale = bHasPerParticleGravity ? GravityScaleOffsetRange[0] + GravityScaleOffsetRange[1] * GravityScaleMultipliers[Index] :
					GravityScaleConstant;
				Acceleration[Index] = PerParticleGravityScale * Gravity;

				if (bHasFictitiousForces)
				{
					// Centrifugal force (*InvM to get acceleration)
					Acceleration[Index] -= FSolverVec3::CrossProduct(FictitiousAngularVelocity, FSolverVec3::CrossProduct(FictitiousAngularVelocity, X[Index] - ReferenceSpaceLocation));
				}
				if (bUsePointBasedWindModel)
				{
					const FSolverVec3 VelocityDelta = PointBasedWind - V[Index];
					FSolverVec3 Direction = VelocityDelta;
					if (Direction.Normalize())
					{
						// Scale by angle
						const FSolverReal DirectionDot = FSolverVec3::DotProduct(Direction, N[Index]);
						const FSolverReal ScaleFactor = FMath::Min(1.f, FMath::Abs(DirectionDot) * LegacyWindAdaptation);
						Acceleration[Index] += VelocityDelta * ScaleFactor;
					}
				}
			}
		}
	}

	void UpdateLinearSystem(const FSolverParticlesRange& Particles, const FSolverReal Dt, FEvolutionLinearSystem& LinearSystem) const
	{
		const FSolverReal* const InvM = Particles.GetInvM().GetData();
		const FSolverReal* const M = Particles.GetM().GetData();
		const FSolverVec3* const X = Particles.XArray().GetData();
		const FSolverVec3* const V = Particles.GetV().GetData();
		const FSolverVec3* const N = Particles.GetConstArrayView(Normals).GetData();

		const bool bHasFictitiousForces = !FictitiousAngularVelocity.IsNearlyZero();

		const bool bHasPerParticleGravity = HasPerParticleGravity();
		const FSolverReal GravityScaleConstant = bApplyGravityScale ? (FSolverReal)GravityScale : (FSolverReal)1.f;
		const FSolverVec2& GravityScaleOffsetRange = GravityScale.GetOffsetRange();
		const FSolverReal* const GravityScaleMultipliers = GravityScale.GetMapValues().GetData();
		check(!bHasPerParticleGravity || (GravityScale.GetMapValues().Num() == Particles.GetRangeSize()));

		for (int32 Index = 0; Index < Particles.GetRangeSize(); ++Index)
		{
			if (InvM[Index] != (FSolverReal)0.)
			{
				const FSolverReal PerParticleGravityScale = bHasPerParticleGravity ? GravityScaleOffsetRange[0] + GravityScaleOffsetRange[1] * GravityScaleMultipliers[Index] :
					GravityScaleConstant;
				FSolverVec3 Force = Gravity * M[Index] * PerParticleGravityScale;

				if (bHasFictitiousForces)
				{
					// Centrifugal force (*InvM to get acceleration)
					Force -= FSolverVec3::CrossProduct(FictitiousAngularVelocity, FSolverVec3::CrossProduct(FictitiousAngularVelocity, X[Index] - ReferenceSpaceLocation)) * M[Index];
				}
				if (bUsePointBasedWindModel)
				{
					const FSolverVec3 VelocityDelta = PointBasedWind - V[Index];
					FSolverVec3 Direction = VelocityDelta;
					if (Direction.Normalize())
					{
						// Scale by angle
						const FSolverReal DirectionDot = FSolverVec3::DotProduct(Direction, N[Index]);
						const FSolverReal ScaleFactor = FMath::Min(1.f, FMath::Abs(DirectionDot) * LegacyWindAdaptation);
						Force += VelocityDelta * ScaleFactor * M[Index];
					}
				}
				LinearSystem.AddForce(Particles, Force, Index, Dt);
			}
		}
	}

	bool UsePointBasedWindModel() const { return bUsePointBasedWindModel; }
	const FSolverVec3& GetGravity() const { return Gravity; }
	bool HasPerParticleGravity() const { return bApplyGravityScale && GravityScale.HasWeightMap(); }
	FSolverVec3 GetScaledGravity(int32 ParticleIndex) const
	{
		const FSolverReal ParticleGravityScale = bApplyGravityScale ? (GravityScale.HasWeightMap() ? GravityScale[ParticleIndex] : GravityScale.GetLow()) : (FSolverReal)1.f;
		return Gravity * ParticleGravityScale;
	}
	const FSolverVec3& GetFictitiousAngularVelocity() const { return FictitiousAngularVelocity; }
	const FSolverVec3& GetReferenceSpaceLocation() const { return ReferenceSpaceLocation; }
protected:
	FSolverVec3 Gravity; 
	bool bApplyGravityScale;
	FPBDFlatWeightMap GravityScale;
	FSolverVec3 FictitiousAngularVelocity;
	FSolverVec3 ReferenceSpaceLocation;
	bool bUsePointBasedWindModel;
	FSolverVec3 PointBasedWind;
	FSolverReal LegacyWindAdaptation;
	const TArray<FSolverVec3>& Normals;
};

class FExternalForces : public FExternalForcesBase
{
	typedef FExternalForcesBase Base;
public:
	static constexpr FSolverReal DefaultGravityScale = (FSolverReal)1.f;
	static constexpr FSolverReal DefaultGravityZOverride = (FSolverReal)-980.665f;
	static constexpr bool bDefaultUseGravityOverride = false;
	static constexpr FSolverReal DefaultFictitiousAngularScale = (FSolverReal)1.f;
	static constexpr bool bDefaultUsePointBasedWindModel = false;

	// We don't have the solver values at the time this is constructed. Need to SetProperties at least once after setting 
	// data from the solver.
	FExternalForces(const FSolverParticlesRange& Particles,
		const TArray<FSolverVec3>& InNormals,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection)
		: Base(Particles,
			FSolverVec3((FSolverReal)0.f, (FSolverReal)0.f, DefaultGravityZOverride),
			FSolverVec2(GetWeightedFloatGravityScale(PropertyCollection, DefaultGravityScale)),
			WeightMaps.FindRef(GetGravityScaleString(PropertyCollection, GravityScaleName.ToString())),
			InNormals
			)
		, ParticleCount(Particles.GetRangeSize())
		, WorldGravityMultiplier((FSolverReal)1.f)
		, SolverGravity((FSolverReal)0.f, (FSolverReal)0.f, DefaultGravityZOverride)
		, bPerSoftBodyGravityOverrideEnabled(true)
		, FictitiousAngularVelocityNoScale(0.f)
		, UseGravityOverrideIndex(PropertyCollection)
		, GravityOverrideIndex(PropertyCollection)
		, GravityScaleIndex(PropertyCollection)
		, FictitiousAngularScaleIndex(PropertyCollection)
		, UsePointBasedWindModelIndex(PropertyCollection)
	{}

	void SetWorldGravityMultiplier(FSolverReal InWorldGravityMultiplier) { WorldGravityMultiplier = InWorldGravityMultiplier; }
	void SetSolverGravityProperties(const FSolverVec3& InSolverGravity, bool bInPerSoftBodyGravityOverrideEnabled)
	{
		SolverGravity = InSolverGravity;
		bPerSoftBodyGravityOverrideEnabled = bInPerSoftBodyGravityOverrideEnabled;
	}
	void SetFictitiousForcesData(const FSolverVec3& InFictitiousAngularVelocityNoScale, const FSolverVec3& InReferenceSpaceLocation)
	{
		FictitiousAngularVelocityNoScale = InFictitiousAngularVelocityNoScale;
		ReferenceSpaceLocation = InReferenceSpaceLocation;
	}
	void SetSolverWind(const FSolverVec3& SolverWind, const FSolverReal InLegacyWindAdaptation)
	{
		PointBasedWind = SolverWind;
		LegacyWindAdaptation = InLegacyWindAdaptation;
	}

	/** This should be called after all other data is set as it will populate the final base class values.*/
	void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
	{
		const bool bUseGravityOverride = UseGravityOverrideIndex != INDEX_NONE ? GetUseGravityOverride(PropertyCollection) : bDefaultUseGravityOverride;
		const FSolverVec3 GravityOverride = GravityOverrideIndex != INDEX_NONE ? FSolverVec3(GetGravityOverride(PropertyCollection)) :
			FSolverVec3(0.f, 0.f, DefaultGravityZOverride);
		CalculateGravity(bUseGravityOverride, GravityOverride);

		if (IsGravityScaleMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatGravityScale(PropertyCollection));
			if (IsGravityScaleStringDirty(PropertyCollection))
			{
				const FString& WeightMapName = GetGravityScaleString(PropertyCollection);
				GravityScale = FPBDFlatWeightMap(WeightedValue, WeightMaps.FindRef(WeightMapName), ParticleCount);
			}
			else
			{
				GravityScale.SetWeightedValue(WeightedValue);
			}
		}

		const FSolverReal FictitiousAngularScale = FMath::Min((FSolverReal)2., FictitiousAngularScaleIndex != INDEX_NONE ? GetFictitiousAngularScale(PropertyCollection) : DefaultFictitiousAngularScale);

		FictitiousAngularVelocity = FictitiousAngularVelocityNoScale * FictitiousAngularScale;

		bUsePointBasedWindModel = UsePointBasedWindModelIndex != INDEX_NONE ? GetUsePointBasedWindModel(PropertyCollection) : bDefaultUsePointBasedWindModel;
	}
private:
	void CalculateGravity(bool bUseGravityOverride, const FSolverVec3& GravityOverride)
	{
		bApplyGravityScale = !(bPerSoftBodyGravityOverrideEnabled && bUseGravityOverride);
		Gravity = (bApplyGravityScale ? SolverGravity : GravityOverride) * WorldGravityMultiplier;
	}
	const int32 ParticleCount;

	/** Gravity */
	// "World" level properties (this comes from a world-level CVar)
	FSolverReal WorldGravityMultiplier;

	// "Solver" level properties (set from Solver-level BP)
	FSolverVec3 SolverGravity;
	bool bPerSoftBodyGravityOverrideEnabled;

	/** Fictitious Forces */
	FSolverVec3 FictitiousAngularVelocityNoScale; // Without Scale parameter applied.

	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(UseGravityOverride, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(GravityOverride, FVector3f);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(GravityScale, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(FictitiousAngularScale, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(UsePointBasedWindModel, bool);
};
}
