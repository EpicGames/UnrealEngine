// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDStiffness.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/CollectionPropertyFacade.h"

namespace Chaos::Softs
{
class FMultiResConstraints final
{
public:
	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsMultiResStiffnessEnabled(PropertyCollection, false);
	}

	static constexpr FSolverReal MinStiffness = (FSolverReal)0; // We're not checking against MinStiffness (except when it's constant and == 0)
	static constexpr FSolverReal MaxStiffness = (FSolverReal)1e9;

	FMultiResConstraints(
		const FSolverParticlesRange& FineParticles,
		const int32 InCoarseSoftBodyId,
		const FTriangleMesh& InCoarseMesh,
		TArray<TVec4<FSolverReal>>&& InCoarseToFinePositionBaryCoordsAndDist,
		TArray<TVec3<int32>>&& InCoarseToFineSourceMeshVertIndices,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection)
		: NumParticles(FineParticles.Size())
		, CoarseSoftBodyId(InCoarseSoftBodyId)
		, CoarseMesh(InCoarseMesh)
		, CoarseToFinePositionBaryCoordsAndDist(MoveTemp(InCoarseToFinePositionBaryCoordsAndDist))
		, CoarseToFineSourceMeshVertIndices(MoveTemp(InCoarseToFineSourceMeshVertIndices))
		, bUseXPBD(GetMultiResUseXPBD(PropertyCollection, false))
		, Stiffness(
			FSolverVec2(GetWeightedFloatMultiResStiffness(PropertyCollection, 1.f)),
			WeightMaps.FindRef(GetMultiResStiffnessString(PropertyCollection, MultiResStiffnessName.ToString())),
			NumParticles,
			FPBDStiffness::DefaultTableSize,
			FPBDStiffness::DefaultParameterFitBase,
			bUseXPBD ? MaxStiffness : (FSolverReal)1.f)
		, VelocityTargetStiffness(
			FSolverVec2(GetWeightedFloatMultiResVelocityTargetStiffness(PropertyCollection, 1.f)),
			WeightMaps.FindRef(GetMultiResVelocityTargetStiffnessString(PropertyCollection, MultiResVelocityTargetStiffnessName.ToString())),
			NumParticles,
			FPBDStiffness::DefaultTableSize,
			FPBDStiffness::DefaultParameterFitBase,
			bUseXPBD ? MaxStiffness : (FSolverReal)1.f)
		, MultiResUseXPBDIndex(PropertyCollection)
		, MultiResStiffnessIndex(PropertyCollection)
		, MultiResVelocityTargetStiffnessIndex(PropertyCollection)
	{
		check(NumParticles == CoarseToFinePositionBaryCoordsAndDist.Num());
		check(NumParticles == CoarseToFineSourceMeshVertIndices.Num());
	}

	~FMultiResConstraints() = default;

	void Init() const 
	{
		if (bUseXPBD)
		{
			Lambdas.Reset();
			Lambdas.SetNumZeroed(NumParticles);
		}
	}

	void ApplyProperties(const FSolverReal Dt, const int32 NumIterations)
	{
		if (bUseXPBD)
		{
			Stiffness.ApplyXPBDValues(MaxStiffness);
			VelocityTargetStiffness.ApplyXPBDValues(MaxStiffness);
		}
		else
		{
			Stiffness.ApplyPBDValues(Dt, NumIterations);
			VelocityTargetStiffness.ApplyPBDValues(Dt, NumIterations);
		}
	}

	CHAOS_API void UpdateFineTargets(const FSolverParticlesRange& CoarseParticles);

	// Update stiffness values
	CHAOS_API void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);

	CHAOS_API void Apply(FSolverParticlesRange& FineParticles, const FSolverReal Dt) const;

	const TArray<FSolverVec3>& GetFineTargetPositions() const { return FineTargetPositions; }
	int32 GetCoarseSoftBodyId() const { return CoarseSoftBodyId; }
	const FTriangleMesh& GetCoarseMesh() const { return CoarseMesh; }
	bool IsConstraintActive(const int32 ParticleIndex) const
	{
		const FSolverReal StiffnessValue = Stiffness.HasWeightMap() ? Stiffness[ParticleIndex] : (FSolverReal)Stiffness;
		const FSolverReal VelocityStiffnessValue = VelocityTargetStiffness.HasWeightMap() ? VelocityTargetStiffness[ParticleIndex] : (FSolverReal)VelocityTargetStiffness;
		return StiffnessValue > 0.f || VelocityStiffnessValue > 0.f;
	}

private:
	void ApplyHelper(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue, const FSolverReal ExpVelocityTargetStiffnessValue) const;

	const int32 NumParticles;
	const int32 CoarseSoftBodyId;
	const FTriangleMesh& CoarseMesh;
	const TArray<TVec4<FSolverReal>> CoarseToFinePositionBaryCoordsAndDist;
	const TArray<TVec3<int32>> CoarseToFineSourceMeshVertIndices;

	bool bUseXPBD;
	FPBDStiffness Stiffness;
	FPBDStiffness VelocityTargetStiffness;

	mutable TArray<FSolverReal> Lambdas;
	TArray<FSolverVec3> FineTargetPositions;
	TArray<FSolverVec3> FineTargetVelocities;
	TArray<FSolverReal> CoarseBarycentricMass;

	int32 NonZeroStiffnessMin = INDEX_NONE;
	int32 NonZeroStiffnessMax = INDEX_NONE;
	bool bStiffnessEntriesInitialized = false;

	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(MultiResUseXPBD, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(MultiResStiffness, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(MultiResVelocityTargetStiffness, float);
};

}  // End namespace Chaos::Softs

#if !defined(CHAOS_MULTIRES_ISPC_ENABLED_DEFAULT)
#define CHAOS_MULTIRES_ISPC_ENABLED_DEFAULT 1
#endif

// Support ISPC enable/disable in non-shipping builds
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bChaos_MultiRes_ISPC_Enabled = INTEL_ISPC && CHAOS_MULTIRES_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_MultiRes_ISPC_Enabled;
#endif
