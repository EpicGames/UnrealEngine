// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "SoftsSimulationSpace.generated.h"

UENUM()
enum struct EChaosSoftsSimulationSpace : uint8
{
	// World space
	WorldSpace = 0,

	// Component space
	ComponentSpace,

	// Top level bone for each cloth.
	ReferenceBoneSpace,
};

UENUM()
enum struct EChaosSoftsLocalDampingSpace : uint8
{
	// Calculated CoM (expensive)
	CenterOfMass = 0,

	// Top level bone for each cloth.
	ReferenceBoneSpace,

};