// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SimpleGeometryParticles.h"
#include "PBDSoftsEvolutionFwd.h"

namespace Chaos::Softs
{
class FSolverCollisionParticles : public TSimpleGeometryParticles<FSolverReal, 3>
{
	typedef TSimpleGeometryParticles<FSolverReal, 3> Base;

  public:
	FSolverCollisionParticles()
	    : TSimpleGeometryParticles<FSolverReal, 3>()
	{
		TArrayCollection::AddArray(&MV);
		TArrayCollection::AddArray(&MW);
	}
	FSolverCollisionParticles(const FSolverCollisionParticles& Other) = delete;
	FSolverCollisionParticles(FSolverCollisionParticles&& Other)
	    : TSimpleGeometryParticles<FSolverReal, 3>(MoveTemp(Other)), MV(MoveTemp(Other.MV)), MW(MoveTemp(Other.MW))
	{
		TArrayCollection::AddArray(&MV);
		TArrayCollection::AddArray(&MW);
	}
	virtual ~FSolverCollisionParticles() override = default;
	FSolverCollisionParticles& operator=(const FSolverCollisionParticles& Other) = delete;
	FSolverCollisionParticles& operator=(FSolverCollisionParticles&& Other) = delete;

	const TVector<FSolverReal, 3>& X(const int32 Index) const	{ 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Base::X(Index);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	TVector<FSolverReal, 3>& X(const int32 Index) {
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Base::X(Index);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

	const FSolverVec3& V(const int32 Index) const { return MV[Index]; }
	FSolverVec3& V(const int32 Index) { return MV[Index]; }
	const TArrayCollectionArray<FSolverVec3>& GetV() const { return MV; }
	TArrayCollectionArray<FSolverVec3>& GetV() { return MV; }

	const FSolverVec3& W(const int32 Index) const { return MW[Index]; }
	FSolverVec3& W(const int32 Index) { return MW[Index]; }
	const TArrayCollectionArray<FSolverVec3>& GetW() const { return MW; }
	TArrayCollectionArray<FSolverVec3>& GetW() { return MW; }

	virtual void Serialize(FChaosArchive& Ar) override
	{
		TSimpleGeometryParticles<FSolverReal, 3>::Serialize(Ar);
		Ar << MV << MW;
	}

	FORCEINLINE TArray<FSolverVec3>& AllV() { return MV; }
	FORCEINLINE TArray<FSolverVec3>& AllW() { return MW; }

  private:
	TArrayCollectionArray<FSolverVec3> MV;
	TArrayCollectionArray<FSolverVec3> MW;
};
}

