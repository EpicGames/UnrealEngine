// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/SoftsSolverCollisionParticles.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/ParticlesRange.h"

namespace Chaos::Softs
{
class FSolverCollisionParticlesRange final : public TParticlesRange<FSolverCollisionParticles>
{
public:
	FSolverCollisionParticlesRange() = default;
	~FSolverCollisionParticlesRange() = default;
	FSolverCollisionParticlesRange(const FSolverCollisionParticlesRange&) = default;
	FSolverCollisionParticlesRange(FSolverCollisionParticlesRange&&) = default;
	FSolverCollisionParticlesRange& operator=(const FSolverCollisionParticlesRange&) = default;
	FSolverCollisionParticlesRange& operator=(FSolverCollisionParticlesRange&&) = default;

	FSolverCollisionParticlesRange(TParticlesRange<FSolverCollisionParticles>&& Other)
		:TParticlesRange<FSolverCollisionParticles>(MoveTemp(Other))
	{}

	// SolverCollisionParticles data
	const FSolverVec3& V(const int32 Index) const { return GetParticles().V(Index + Offset); }
	FSolverVec3& V(const int32 Index) { return GetParticles().V(Index + Offset); }
	TConstArrayView<FSolverVec3> GetV() const { return GetConstArrayView(GetParticles().GetV()); }
	TArrayView<FSolverVec3> GetV() { return GetArrayView(GetParticles().GetV()); }
	const FSolverVec3& W(const int32 Index) const { return GetParticles().W(Index + Offset); }
	FSolverVec3& W(const int32 Index) { return GetParticles().W(Index + Offset); }
	TConstArrayView<FSolverVec3> GetW() const { return GetConstArrayView(GetParticles().GetW()); }
	TArrayView<FSolverVec3> GetW() { return GetArrayView(GetParticles().GetW()); }

	// SimpleGeometryParticles data
	const FSolverRotation3 R(const int32 Index) const { return GetParticles().GetR(Index + Offset); }
	void SetR(const int32 Index, const FSolverRotation3& InR) { GetParticles().SetR(Index + Offset, InR); }
	UE_DEPRECATED(5.4, "Use GetR or SetR instead")
	FSolverRotation3 R(const int32 Index) { return GetParticles().GetR(Index + Offset); }
	TConstArrayView<FSolverRotation3> GetR() const { return GetConstArrayView(GetParticles().GetR()); }
	TArrayView<FSolverRotation3> GetR() { return GetArrayView(GetParticles().GetR()); }

	const FImplicitObjectPtr& GetGeometry(const int32 Index) const { return GetParticles().GetGeometry(Index + Offset); }
	void SetGeometry(const int32 Index, const FImplicitObjectPtr& InGeometry) { return GetParticles().SetGeometry(Index + Offset, InGeometry); }
	TConstArrayView<FImplicitObjectPtr> GetAllGeometry() const { return GetConstArrayView(GetParticles().GetAllGeometry()); }

	// Particles data
	const FSolverVec3& X(const int32 Index) const { return GetParticles().GetX(Index + Offset); }
	FSolverVec3& X(const int32 Index) { 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GetParticles().X(Index + Offset); 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	const FSolverVec3& GetX(const int32 Index) const { return GetParticles().GetX(Index + Offset); }
	void SetX(const int32 Index, const FSolverVec3& InX) { GetParticles().SetX(Index + Offset, InX); }
	TConstArrayView<FSolverVec3> XArray() const { return GetConstArrayView(GetParticles().XArray()); }
	TArrayView<FSolverVec3> XArray() { return GetArrayView(GetParticles().XArray()); }
};
}
