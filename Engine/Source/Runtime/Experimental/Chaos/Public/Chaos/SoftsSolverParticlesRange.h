// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/ParticlesRange.h"

namespace Chaos::Softs
{

class FSolverParticlesRange final : public TParticlesRange<FSolverParticles>
{
public:
	FSolverParticlesRange() = default;
	~FSolverParticlesRange() = default;
	FSolverParticlesRange(const FSolverParticlesRange&) = default;
	FSolverParticlesRange(FSolverParticlesRange&&) = default;
	FSolverParticlesRange& operator=(const FSolverParticlesRange&) = default;
	FSolverParticlesRange& operator=(FSolverParticlesRange&&) = default;

	FSolverParticlesRange(FSolverParticles* InParticles, const int32 InOffset, const int32 InRange)
		: TParticlesRange<FSolverParticles>(InParticles, InOffset, InRange)
	{}

	explicit FSolverParticlesRange(FSolverParticles& InParticles)
		: TParticlesRange<FSolverParticles>(&InParticles, 0, InParticles.Size())
	{}

	explicit FSolverParticlesRange(const FSolverParticles& InParticles)
		: TParticlesRange<FSolverParticles>(const_cast<FSolverParticles*>(&InParticles), 0, InParticles.Size())
	{}

	FSolverParticlesRange(TParticlesRange<FSolverParticles>&& Other)
		: TParticlesRange<FSolverParticles>(MoveTemp(Other))
	{}

	// SolverParticles data
	const FSolverVec3& P(const int32 Index) const { return GetParticles().P(Index + Offset); }
	FSolverVec3& P(const int32 Index) { return GetParticles().P(Index + Offset); }
	const FSolverVec3& GetP(const int32 Index) const { return GetParticles().P(Index + Offset); }
	void SetP(const int32 Index, const FSolverVec3& InP) { GetParticles().P(Index + Offset) = InP; }
	const FPAndInvM& PAndInvM(const int32 Index) const { return GetParticles().PAndInvM(Index + Offset); }
	FPAndInvM& PAndInvM(const int32 Index) { return GetParticles().PAndInvM(Index + Offset); }
	TConstArrayView<FPAndInvM> GetPAndInvM() const { return GetConstArrayView(GetParticles().GetPAndInvM()); }
	TArrayView<FPAndInvM> GetPAndInvM() { return GetArrayView(GetParticles().GetPAndInvM()); }
	const FSolverVec3& VPrev(const int32 Index) const { return GetParticles().VPrev(Index + Offset); }
	FSolverVec3& VPrev(const int32 Index) { return GetParticles().VPrev(Index + Offset); }
	TConstArrayView<FSolverVec3> GetVPrev() const { return GetConstArrayView(GetParticles().GetVPrev()); }
	TArrayView<FSolverVec3> GetVPrev() { return GetArrayView(GetParticles().GetVPrev()); }

	// DynamicParticles data
	const FSolverVec3& V(const int32 Index) const { return GetParticles().V(Index + Offset); }
	FSolverVec3& V(const int32 Index) { return GetParticles().V(Index + Offset); }
	TConstArrayView<FSolverVec3> GetV() const { return GetConstArrayView(GetParticles().GetV()); }
	TArrayView<FSolverVec3> GetV() { return GetArrayView(GetParticles().GetV()); }
	const FSolverVec3& Acceleration(const int32 Index) const { return GetParticles().Acceleration(Index + Offset); }
	FSolverVec3& Acceleration(const int32 Index) { return GetParticles().Acceleration(Index + Offset); }
	TConstArrayView<FSolverVec3> GetAcceleration() const { return GetConstArrayView(GetParticles().GetAcceleration()); }
	TArrayView<FSolverVec3> GetAcceleration() { return GetArrayView(GetParticles().GetAcceleration()); }
	FSolverReal M(const int32 Index) const { return GetParticles().M(Index + Offset); }
	FSolverReal& M(const int32 Index) { return GetParticles().M(Index + Offset); }
	TConstArrayView<FSolverReal> GetM() const { return GetConstArrayView(GetParticles().GetM()); }
	TArrayView<FSolverReal> GetM() { return GetArrayView(GetParticles().GetM()); }
	FSolverReal InvM(const int32 Index) const { return GetParticles().InvM(Index + Offset); }
	FSolverReal& InvM(const int32 Index) { return GetParticles().InvM(Index + Offset); }
	TConstArrayView<FSolverReal> GetInvM() const { return GetConstArrayView(GetParticles().GetInvM()); }
	TArrayView<FSolverReal> GetInvM() { return GetArrayView(GetParticles().GetInvM()); }

	// Particles data
	const FSolverVec3& X(const int32 Index) const { return GetParticles().GetX(Index + Offset); }
	FSolverVec3& X(const int32 Index) { 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return GetParticles().X(Index + Offset); 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	const FSolverVec3& GetX(const int32 Index) const { return GetParticles().GetX(Index + Offset); }
	void SetX(const int32 Index, const FSolverVec3& InX) { GetParticles().SetX(Index + Offset, InX); }

	TConstArrayView<FSolverVec3> XArray() const { return GetConstArrayView(GetParticles().XArray()); }
	TArrayView<FSolverVec3> XArray() { return GetArrayView(GetParticles().XArray()); }
};
}
