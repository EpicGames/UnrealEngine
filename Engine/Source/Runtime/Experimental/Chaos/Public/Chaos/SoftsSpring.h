// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/SoftsEvolutionLinearSystem.h"
#include "Chaos/SoftsSolverParticlesRange.h"

namespace Chaos::Softs
{

namespace Spring
{

// Spring without damping
template<typename SolverParticlesOrRange>
FSolverVec3 GetXPBDSpringDelta(const SolverParticlesOrRange& Particles, const FSolverReal Dt,
	const TVec2<int32>& Constraint, const FSolverReal RestLength, FSolverReal& Lambda,
	const FSolverReal StiffnessValue)
{
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];

	const FSolverReal CombinedInvMass = Particles.InvM(Index2) + Particles.InvM(Index1);

	const FSolverVec3& P1 = Particles.P(Index1);
	const FSolverVec3& P2 = Particles.P(Index2);
	FSolverVec3 Direction = P1 - P2;
	const FSolverReal Distance = Direction.SafeNormalize();
	const FSolverReal Offset = Distance - RestLength;

	const FSolverReal AlphaInv = StiffnessValue * Dt * Dt;

	const FSolverReal DLambda = (-AlphaInv * Offset - Lambda) / (AlphaInv * CombinedInvMass + (FSolverReal)1.);
	const FSolverVec3 Delta = DLambda * Direction;
	Lambda += DLambda;

	return Delta;
}

// This is a following the original XPBD paper using a single lambda for stretch and damping.
template<typename SolverParticlesOrRange>
FSolverVec3 GetXPBDSpringDeltaWithDamping(const SolverParticlesOrRange& Particles, const FSolverReal Dt,
	const TVec2<int32>& Constraint, const FSolverReal RestLength, FSolverReal& Lambda,
	const FSolverReal StiffnessValue, const FSolverReal DampingRatioValue)
{
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];

	const FSolverReal CombinedInvMass = Particles.InvM(Index2) + Particles.InvM(Index1);
	check(CombinedInvMass > (FSolverReal)0);

	const FSolverReal Damping = DampingRatioValue * 2.f * FMath::Sqrt(StiffnessValue / CombinedInvMass) * (RestLength > UE_SMALL_NUMBER ? (FSolverReal)1. / RestLength : (FSolverReal)1.);

	const FSolverVec3& P1 = Particles.P(Index1);
	const FSolverVec3& P2 = Particles.P(Index2);
	FSolverVec3 Direction = P1 - P2;
	const FSolverReal Distance = Direction.SafeNormalize();
	const FSolverReal Offset = Distance - RestLength;

	const FSolverVec3& X1 = Particles.GetX(Index1);
	const FSolverVec3& X2 = Particles.GetX(Index2);

	const FSolverVec3 RelativeVelocityTimesDt = P1 - X1 - P2 + X2;

	const FSolverReal AlphaInv = StiffnessValue * Dt * Dt;
	const FSolverReal BetaDt = Damping * Dt;

	const FSolverReal DLambda = (-AlphaInv * Offset - Lambda - BetaDt * FSolverVec3::DotProduct(Direction, RelativeVelocityTimesDt)) / ((AlphaInv + BetaDt) * CombinedInvMass + (FSolverReal)1.);
	const FSolverVec3 Delta = DLambda * Direction;
	Lambda += DLambda;

	return Delta;
}

// Spring damping constraint (separate from spring stretching)
template<typename SolverParticlesOrRange>
FSolverVec3 GetXPBDSpringDampingDelta(const SolverParticlesOrRange& Particles, const FSolverReal Dt,
	const TVec2<int32>& Constraint, const FSolverReal RestLength, FSolverReal& Lambda,
	const FSolverReal StiffnessValue, const FSolverReal DampingRatioValue)
{
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];
	const FSolverReal CombinedInvMass = Particles.InvM(Index2) + Particles.InvM(Index1);
	check(CombinedInvMass > (FSolverReal)0);

	const FSolverReal Damping = DampingRatioValue * 2.f * FMath::Sqrt(StiffnessValue / CombinedInvMass) * (RestLength > UE_SMALL_NUMBER ? (FSolverReal)1. / RestLength : (FSolverReal)1.);

	const FSolverVec3& P1 = Particles.P(Index1);
	const FSolverVec3& P2 = Particles.P(Index2);
	FSolverVec3 Direction = (P1 - P2);
	Direction.SafeNormalize();

	const FSolverVec3& X1 = Particles.GetX(Index1);
	const FSolverVec3& X2 = Particles.GetX(Index2);
	const FSolverVec3 RelativeVelocityTimesDt = P1 - X1 - P2 + X2;
	const FSolverReal BetaDt = Damping * Dt;
	const FSolverReal DLambda = (-BetaDt * FSolverVec3::DotProduct(Direction, RelativeVelocityTimesDt) - Lambda) / (BetaDt * CombinedInvMass + (FSolverReal)1.);
	const FSolverVec3 Delta = DLambda * Direction;
	Lambda += DLambda;
	return Delta;
}

// Spring without damping
template<typename SolverParticlesOrRange>
FSolverVec3 GetXPBDAxialSpringDelta(const SolverParticlesOrRange& Particles, const FSolverReal Dt,
	const TVec3<int32>& Constraint, const FSolverReal Bary, const FSolverReal RestLength, FSolverReal& Lambda,
	const FSolverReal StiffnessValue)
{
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];
	const int32 Index3 = Constraint[2];

	const FSolverReal PInvMass = Particles.InvM(Index3) * ((FSolverReal)1. - Bary) + Particles.InvM(Index2) * Bary;
	const FSolverReal CombinedInvMass = PInvMass + Particles.InvM(Index1);
	check(CombinedInvMass > (FSolverReal)0);

	const FSolverVec3& P1 = Particles.P(Index1);
	const FSolverVec3& P2 = Particles.P(Index2);
	const FSolverVec3& P3 = Particles.P(Index3);
	const FSolverVec3 P = (P2 - P3) * Bary + P3;
	FSolverVec3 Direction = P1 - P;
	const FSolverReal Distance = Direction.SafeNormalize();
	const FSolverReal Offset = Distance - RestLength;

	const FSolverReal AlphaInv = StiffnessValue * Dt * Dt;

	const FSolverReal DLambda = (-AlphaInv * Offset - Lambda) / (AlphaInv * CombinedInvMass + (FSolverReal)1.);
	const FSolverVec3 Delta = DLambda * Direction;
	Lambda += DLambda;

	return Delta;
}

// This is a following the original XPBD paper using a single lambda for stretch and damping.
template<typename SolverParticlesOrRange>
FSolverVec3 GetXPBDAxialSpringDeltaWithDamping(const SolverParticlesOrRange& Particles, const FSolverReal Dt,
	const TVec3<int32>& Constraint, const FSolverReal Bary, const FSolverReal RestLength, FSolverReal& Lambda,
	const FSolverReal StiffnessValue, const FSolverReal DampingRatioValue)
{
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];
	const int32 Index3 = Constraint[2];

	const FSolverReal PInvMass = Particles.InvM(Index3) * ((FSolverReal)1. - Bary) + Particles.InvM(Index2) * Bary;
	const FSolverReal CombinedInvMass = PInvMass + Particles.InvM(Index1);
	check(CombinedInvMass > (FSolverReal)0);

	const FSolverReal Damping = DampingRatioValue * 2.f * FMath::Sqrt(StiffnessValue / CombinedInvMass) * (RestLength > UE_SMALL_NUMBER ? (FSolverReal)1. / RestLength : (FSolverReal)1.);

	const FSolverVec3& P1 = Particles.P(Index1);
	const FSolverVec3& P2 = Particles.P(Index2);
	const FSolverVec3& P3 = Particles.P(Index3);
	const FSolverVec3 P = (P2 - P3) * Bary + P3;
	FSolverVec3 Direction = P1 - P;
	const FSolverReal Distance = Direction.SafeNormalize();
	const FSolverReal Offset = Distance - RestLength;

	const FSolverVec3& X1 = Particles.GetX(Index1);
	const FSolverVec3& X2 = Particles.GetX(Index2);
	const FSolverVec3& X3 = Particles.GetX(Index3);
	const FSolverVec3 X = (X2 - X3) * Bary + X3;

	const FSolverVec3 RelativeVelocityTimesDt = P1 - X1 - P + X;

	const FSolverReal AlphaInv = StiffnessValue * Dt * Dt;
	const FSolverReal BetaDt = Damping * Dt;

	const FSolverReal DLambda = (-AlphaInv * Offset - Lambda - BetaDt * FSolverVec3::DotProduct(Direction, RelativeVelocityTimesDt)) / ((AlphaInv + BetaDt) * CombinedInvMass + (FSolverReal)1.);
	const FSolverVec3 Delta = DLambda * Direction;
	Lambda += DLambda;

	return Delta;
}

// Spring damping constraint (separate from spring stretching)
template<typename SolverParticlesOrRange>
FSolverVec3 GetXPBDAxialSpringDampingDelta(const SolverParticlesOrRange& Particles, const FSolverReal Dt,
	const TVec3<int32>& Constraint, const FSolverReal Bary, const FSolverReal RestLength, FSolverReal& Lambda,
	const FSolverReal StiffnessValue, const FSolverReal DampingRatioValue)
{
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];
	const int32 Index3 = Constraint[2];

	const FSolverReal PInvMass = Particles.InvM(Index3) * ((FSolverReal)1. - Bary) + Particles.InvM(Index2) * Bary;
	const FSolverReal CombinedInvMass = PInvMass + Particles.InvM(Index1);
	check(CombinedInvMass > (FSolverReal)0);

	const FSolverReal Damping = DampingRatioValue * 2.f * FMath::Sqrt(StiffnessValue / CombinedInvMass) * (RestLength > UE_SMALL_NUMBER ? (FSolverReal)1. / RestLength : (FSolverReal)1.);

	const FSolverVec3& P1 = Particles.P(Index1);
	const FSolverVec3& P2 = Particles.P(Index2);
	const FSolverVec3& P3 = Particles.P(Index3);
	const FSolverVec3 P = (P2 - P3) * Bary + P3;
	FSolverVec3 Direction = P1 - P;
	const FSolverReal Distance = Direction.SafeNormalize();
	const FSolverReal Offset = Distance - RestLength;

	const FSolverVec3& X1 = Particles.GetX(Index1);
	const FSolverVec3& X2 = Particles.GetX(Index2);
	const FSolverVec3& X3 = Particles.GetX(Index3);
	const FSolverVec3 X = (X2 - X3) * Bary + X3;
	const FSolverVec3 RelativeVelocityTimesDt = P1 - X1 - P + X;
	const FSolverReal BetaDt = Damping * Dt;
	const FSolverReal DLambda = (-BetaDt * FSolverVec3::DotProduct(Direction, RelativeVelocityTimesDt) - Lambda) / (BetaDt * CombinedInvMass + (FSolverReal)1.);
	const FSolverVec3 Delta = DLambda * Direction;
	Lambda += DLambda;

	return Delta;
}

// Spring without damping
template<typename SolverParticlesOrRange, int32 N>
FSolverVec3 GetXPBDEmbeddedSpringDelta(const SolverParticlesOrRange& Particles, const FSolverReal Dt,
	const TVector<int32, N>& Constraint, const TVector<FSolverReal, N>& Weights, const FSolverReal RestLength, FSolverReal& Lambda,
	const FSolverReal ExtensionStiffnessValue, const FSolverReal CompressionStiffnessValue)
{
	FSolverReal CombinedInvMass = 0.f;
	FSolverVec3 Direction(0.f);
	for (int32 NIdx = 0; NIdx < N; ++NIdx)
	{
		const int32 Index = Constraint[NIdx];
		CombinedInvMass += Particles.InvM(Index) * FMath::Abs(Weights[NIdx]);
		Direction += Particles.P(Index) * Weights[NIdx];
	}

	const FSolverReal Distance = Direction.SafeNormalize();
	const FSolverReal Offset = Distance - RestLength;

	// Do we need a smoothstep?
	const FSolverReal StiffnessValue = Offset >= 0 ? ExtensionStiffnessValue : CompressionStiffnessValue;
	const FSolverReal AlphaInv = StiffnessValue * Dt * Dt;

	const FSolverReal DLambda = (-AlphaInv * Offset - Lambda) / (AlphaInv * CombinedInvMass + (FSolverReal)1.);
	const FSolverVec3 Delta = DLambda * Direction;
	Lambda += DLambda;

	return Delta;
}

// Spring damping constraint (separate from spring stretching)
template<typename SolverParticlesOrRange, int32 N>
FSolverVec3 GetXPBDEmbeddedSpringDampingDelta(const SolverParticlesOrRange& Particles, const FSolverReal Dt,
	const TVector<int32, N>& Constraint, const TVector<FSolverReal, N>& Weights, const FSolverReal RestLength, FSolverReal& Lambda,
	const FSolverReal ExtensionStiffnessValue, const FSolverReal CompressionStiffnessValue, const FSolverReal DampingRatioValue)
{
	FSolverReal CombinedInvMass = 0.f;
	FSolverVec3 DeltaPDirection(0.f);
	FSolverVec3 DeltaX(0.f);
	for (int32 NIdx = 0; NIdx < N; ++NIdx)
	{
		const int32 Index = Constraint[NIdx];
		CombinedInvMass += Particles.InvM(Index) * FMath::Abs(Weights[NIdx]);
		DeltaPDirection += Particles.P(Index) * Weights[NIdx];
		DeltaX += Particles.X(Index) * Weights[NIdx];
	}
	check(CombinedInvMass > (FSolverReal)0);

	const FSolverVec3 RelativeVelocityTimesDt = DeltaPDirection - DeltaX;

	const FSolverReal Distance = DeltaPDirection.SafeNormalize();
	const FSolverReal Offset = Distance - RestLength;
	const FSolverReal StiffnessValue = Offset >= 0 ? ExtensionStiffnessValue : CompressionStiffnessValue;
	const FSolverReal Damping = DampingRatioValue * 2.f * FMath::Sqrt(StiffnessValue / CombinedInvMass) * (RestLength > UE_SMALL_NUMBER ? (FSolverReal)1. / RestLength : (FSolverReal)1.);

	const FSolverReal BetaDt = Damping * Dt;
	const FSolverReal DLambda = (-BetaDt * FSolverVec3::DotProduct(DeltaPDirection, RelativeVelocityTimesDt) - Lambda) / (BetaDt * CombinedInvMass + (FSolverReal)1.);
	const FSolverVec3 Delta = DLambda * DeltaPDirection;
	Lambda += DLambda;

	return Delta;
}

inline void UpdateSpringLinearSystem(const FSolverParticlesRange& Particles, const FSolverReal Dt,
	const TVec2<int32>& Constraint, const FSolverReal RestLength,
	const FSolverReal StiffnessValue, const FSolverReal MinStiffness, const FSolverReal DampingRatioValue,
	FEvolutionLinearSystem& LinearSystem)
{
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];

	if (StiffnessValue <= MinStiffness || (Particles.InvM(Index2) == (FSolverReal)0. && Particles.InvM(Index1) == (FSolverReal)0.))
	{
		return;
	}

	const FSolverVec3& P1 = Particles.P(Index1);
	const FSolverVec3& P2 = Particles.P(Index2);
	FSolverVec3 Direction = P1 - P2;
	const FSolverReal Distance = Direction.SafeNormalize();
	if (Distance < UE_SMALL_NUMBER)
	{
		// We can't calculate a direction if distance is zero. Just skip
		return;
	}
	const FSolverReal CombinedInvMass = Particles.InvM(Index2) + Particles.InvM(Index1);

	const FSolverReal Damping = DampingRatioValue * 2.f * FMath::Sqrt(StiffnessValue / CombinedInvMass) * (RestLength > UE_SMALL_NUMBER ? (FSolverReal)1. / RestLength : (FSolverReal)1.);
	const FSolverVec3 RelVel = Particles.V(Index1) - Particles.V(Index2);
	const FSolverReal CDot = FSolverVec3::DotProduct(Direction, RelVel);

	const FSolverReal Offset = Distance - RestLength; // C
	// const FSolverVec3 GradC1 = Direction;
	// const FSolverVec3 GradC2 = -GradC1;
	const FSolverReal Scalar = -(StiffnessValue * Offset + Damping * CDot);
	const FSolverVec3 Force1 = Scalar * Direction;
	// const FSolverVec3 Force2 = -Force1

	const FSolverMatrix33 DirDirT = FSolverMatrix33::OuterProduct(Direction, Direction); // = GradC1 GradC1^T
	const FSolverMatrix33 Hess11 = (FSolverReal)1. / Distance * (FSolverMatrix33::Identity - DirDirT); // = Hess22
	// Hess12 = Hess21 = -Hess11

	// Df1Dx1 = -Stiffness * (DirDirT + Offset * Hess11) - Damping * CDot * Hess11
	const FSolverReal ScalarModified = LinearSystem.RequiresSPDForceDerivatives() ? FMath::Min(Scalar, 0) : Scalar;
	const FSolverMatrix33 Df1Dx1 = -StiffnessValue * DirDirT + ScalarModified * Hess11;
	const FSolverMatrix33 Df1Dx2 = -Df1Dx1;
	// Df2Dx2 = Df1Dx1
	const FSolverMatrix33 Df1Dv1 = -Damping * DirDirT;
	const FSolverMatrix33 Df1Dv2 = -Df1Dv1;
	// Df2Dv2 = Df1Dv1

	if (Particles.InvM(Index1) != (FSolverReal)0.)
	{
		LinearSystem.AddForce(Particles, Force1, Index1, Dt);
		LinearSystem.AddSymmetricForceDerivative(Particles, &Df1Dx1, &Df1Dv1, Index1, Index1, Dt);
		LinearSystem.AddSymmetricForceDerivative(Particles, &Df1Dx2, &Df1Dv2, Index1, Index2, Dt);
		if (Particles.InvM(Index2) != (FSolverReal)0.)
		{
			LinearSystem.AddForce(Particles, -Force1, Index2, Dt);
			LinearSystem.AddSymmetricForceDerivative(Particles, &Df1Dx1, &Df1Dv1, Index2, Index2, Dt);
		}
	}
	else
	{
		check(Particles.InvM(Index2) != (FSolverReal)0.);
		LinearSystem.AddForce(Particles, -Force1, Index2, Dt);
		LinearSystem.AddSymmetricForceDerivative(Particles, &Df1Dx1, &Df1Dv1, Index2, Index2, Dt);
		LinearSystem.AddSymmetricForceDerivative(Particles, &Df1Dx2, &Df1Dv2, Index2, Index1, Dt);
	}
}

inline void UpdateAxialSpringLinearSystem(const FSolverParticlesRange& Particles, const FSolverReal Dt,
	const TVec3<int32>& Constraint, const FSolverReal Bary, const FSolverReal RestLength,
	const FSolverReal StiffnessValue, const FSolverReal MinStiffness, const FSolverReal DampingRatioValue,
	FEvolutionLinearSystem& LinearSystem)
{
	const int32 Index1 = Constraint[0];
	const int32 Index2 = Constraint[1];
	const int32 Index3 = Constraint[2];

	const FSolverReal PInvMass = Particles.InvM(Index3) * ((FSolverReal)1. - Bary) + Particles.InvM(Index2) * Bary;
	if (StiffnessValue < MinStiffness || (Particles.InvM(Index2) == (FSolverReal)0. && PInvMass == (FSolverReal)0.))
	{
		return;
	}

	const FSolverVec3& P1 = Particles.P(Index1);
	const FSolverVec3& P2 = Particles.P(Index2);
	const FSolverVec3& P3 = Particles.P(Index3);
	const FSolverVec3 P = Bary * P2 + ((FSolverReal)1. - Bary) * P3;
	FSolverVec3 Direction = P1 - P;
	const FSolverReal Distance = Direction.SafeNormalize();
	if (Distance < UE_SMALL_NUMBER)
	{
		// We can't calculate a direction if distance is zero. Just skip
		return;
	}
	const FSolverReal CombinedInvMass = PInvMass + Particles.InvM(Index1);

	const FSolverReal Damping = DampingRatioValue * 2.f * FMath::Sqrt(StiffnessValue / CombinedInvMass) * (RestLength > UE_SMALL_NUMBER ? (FSolverReal)1. / RestLength : (FSolverReal)1.);

	const FSolverVec3& V1 = Particles.V(Index1);
	const FSolverVec3& V2 = Particles.V(Index2);
	const FSolverVec3& V3 = Particles.V(Index3);
	const FSolverVec3 V = Bary * V2 + ((FSolverReal)1. - Bary) * V3;

	const FSolverVec3 RelVel = V1 - V;
	const FSolverReal CDot = FSolverVec3::DotProduct(Direction, RelVel);

	const FSolverReal Offset = Distance - RestLength; // C
	// const FSolverVec3 GradC1 = Direction;
	// const FSolverVec3 GradC2 = - Bary * GradC1;
	// const FSolverVec3 GradC3 = - (1 - Bary) * GradC1;
	const FSolverReal Scalar = -(StiffnessValue * Offset + Damping * CDot);
	const FSolverVec3 Force1 = Scalar * Direction;
	// const FSolverVec3 Force2 = -Bary * Force1
	// const FSolverVec3 Force3 = -(1 - Bary) * Force1

	const FSolverMatrix33 DirDirT = FSolverMatrix33::OuterProduct(Direction, Direction); // = GradC1 GradC1^T
	const FSolverMatrix33 Hess11 = (FSolverReal)1. / Distance * (FSolverMatrix33::Identity - DirDirT);

	// Df1Dx1 = -Stiffness * (DirDirT + Offset * Hess11) - Damping * CDot * Hess11
	const FSolverReal ScalarModified = LinearSystem.RequiresSPDForceDerivatives() ? FMath::Min(Scalar, 0) : Scalar;
	const FSolverMatrix33 Df1Dx1 = -StiffnessValue * DirDirT + ScalarModified * Hess11;
	const FSolverMatrix33 Df1Dv1 = -Damping * DirDirT;

	// DfiDxj = Bi * Bj * Df1Dx1, where B1 = 1, B2 = -Bary, B3 = -(1-Bary)
	// Same for DfiDvj
	const FSolverReal B3(Bary - (FSolverReal)1.);

	if (Particles.InvM(Index1) != (FSolverReal)0.)
	{
		LinearSystem.AddForce(Particles, Force1, Index1, Dt);
		LinearSystem.AddSymmetricForceDerivative(Particles, &Df1Dx1, &Df1Dv1, Index1, Index1, Dt);
	}
	if (Particles.InvM(Index2) != (FSolverReal)0.)
	{
		const FSolverMatrix33 Df2Dx2 = (Bary * Bary) * Df1Dx1;
		const FSolverMatrix33 Df2Dv2 = (Bary * Bary) * Df1Dv1;
		LinearSystem.AddForce(Particles, -Bary * Force1, Index2, Dt);
		LinearSystem.AddSymmetricForceDerivative(Particles, &Df2Dx2, &Df2Dv2, Index2, Index2, Dt);
	}
	if (Particles.InvM(Index3) != (FSolverReal)0.)
	{
		const FSolverMatrix33 Df3Dx3 = (B3 * B3) * Df1Dx1;
		const FSolverMatrix33 Df3Dv3 = (B3 * B3) * Df1Dv1;
		LinearSystem.AddForce(Particles, -B3 * Force1, Index3, Dt);
		LinearSystem.AddSymmetricForceDerivative(Particles, &Df3Dx3, &Df3Dv3, Index3, Index3, Dt);
	}
	if (Particles.InvM(Index1) != (FSolverReal)0. || Particles.InvM(Index2) != (FSolverReal)0.)
	{
		const FSolverMatrix33 Df1Dx2 = -Bary * Df1Dx1;
		const FSolverMatrix33 Df1Dv2 = -Bary * Df1Dv1;
		LinearSystem.AddSymmetricForceDerivative(Particles, &Df1Dx2, &Df1Dv2, Index1, Index2, Dt);
	}
	if (Particles.InvM(Index1) != (FSolverReal)0. || Particles.InvM(Index3) != (FSolverReal)0.)
	{
		const FSolverMatrix33 Df1Dx3 = -B3 * Df1Dx1;
		const FSolverMatrix33 Df1Dv3 = -B3 * Df1Dv1;
		LinearSystem.AddSymmetricForceDerivative(Particles, &Df1Dx3, &Df1Dv3, Index1, Index3, Dt);
	}
}
}
}
