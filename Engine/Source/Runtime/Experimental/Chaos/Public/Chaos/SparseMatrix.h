// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Map.h"
#include "Chaos/VectorND.h"

namespace Chaos
{
template<class T>
class SparseMatrix
{
  public:
	SparseMatrix(const int32 size)
	    : MSize(size) {}
	~SparseMatrix() = default;

	T operator()(const int32 i, const int32 j)
	{
		int32 Key = i * MSize + j;
		if (!MValues.Contains(Key))
		{
			MValues.Add(Key, T());
			if (!RowToIndicesMap.Contains(i))
			{
				RowToIndicesMap.Add(i, {j});
			}
			else
			{
				RowToIndicesMap[i].Add(j);
			}
		}
		return MValues[Key];
	}
	const T operator()(const int32 i, const int32 j) const
	{
		int32 Key = i * MSize + j;
		check(MValues.Contains(Key));
		return MValues[Key];
	}
	VectorND<T> operator*(const VectorND<T>& vector) const
	{
		check(vector.Num() == MSize);
		VectorND<T> result(MSize);
		for (int32 i = 0; i < MSize; ++i)
		{
			result[i] = 0;
			if (!RowToIndicesMap.Contains(i))
				continue;
			for (auto j : RowToIndicesMap[i])
			{
				result[i] += (*this)(i, j) * vector[j];
			}
		}
		return result;
	}

  private:
	int32 MSize;
	TMap<int32, TArray<int32>> RowToIndicesMap;
	TMap<int32, T> MValues;
};
}
