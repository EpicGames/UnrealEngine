// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/ISpatialAccelerationCollection.h"
#include "Chaos/Box.h"
#include "Chaos/Collision/SpatialAccelerationBroadPhase.h"
#include "Chaos/Collision/StatsData.h"
#include "ChaosStats.h"
#include "GeometryParticlesfwd.h"
#include "Misc/OutputDevice.h"

#include <tuple>

class FChaosVDDataWrapperUtils;

namespace Chaos
{

// Bucket inner indices define for specific use cases		
enum class ESpatialAccelerationCollectionBucketInnerIndex
{
	Default = 0, // If acceleration structures are not split up by body type, they end up in this bucket, otherwise static bodies will go here
	Dynamic = 1, // For dynamic bodies if they use a separate acceleration structure
	DefaultQueryOnly = 2, // Query only bodies if they are separated
	DynamicQueryOnly = 3 // Dynamic Query only bodies if they are separated
};

// Bucket inner indices define for specific use cases		
enum UE_DEPRECATED(5.8, "Use ESpatialAccelerationCollectionBucketInnerIndex instead")
ESpatialAccelerationCollectionBucketInnerIdx
{
	Default_Renamed_To_Avoid_Compile_Errors = 0, // If acceleration structures are not split up by body type, they end up in this bucket, otherwise static bodies will go here
	Dynamic = 1, // For dynamic bodies if they use a separate acceleration structure
	DefaultQueryOnly = 2, // Query only bodies if they are separated
	DynamicQueryOnly = 3 // Dynamic Query only bodies if they are separated
};

template <typename T>
void FreeObjHelper(T*& RawPtr)
{
	RawPtr = nullptr;
}

template <typename TPayloadType, typename T, int d>
struct TSpatialAccelerationBucketEntry
{
	TUniquePtr<ISpatialAcceleration<TPayloadType, T, d>> Acceleration;
	uint16 TypeInnerIdx; // Index in bucket

	void CopyFrom(TSpatialAccelerationBucketEntry<TPayloadType, T, d>& Src)
	{
		Acceleration = Src.Acceleration->Copy();
		TypeInnerIdx = Src.TypeInnerIdx;
	}

	TSpatialAccelerationBucketEntry() = default;
	TSpatialAccelerationBucketEntry(const TSpatialAccelerationBucketEntry<TPayloadType, T, d>& Other) = delete;
	TSpatialAccelerationBucketEntry(TSpatialAccelerationBucketEntry<TPayloadType, T, d>&& Other) = default;
	TSpatialAccelerationBucketEntry<TPayloadType, T, d>& operator=(TSpatialAccelerationBucketEntry<TPayloadType, T, d>&& Other) = default;
	TSpatialAccelerationBucketEntry<TPayloadType, T, d>& operator=(TSpatialAccelerationBucketEntry<TPayloadType, T, d>& Other) = delete;
};

template <typename TPayloadType, typename T, int d>
FChaosArchive& operator<<(FChaosArchive& Ar, TSpatialAccelerationBucketEntry<TPayloadType, T, d>& BucketEntry)
{
	Ar << BucketEntry.Acceleration;
	Ar << BucketEntry.TypeInnerIdx;
	return Ar;
}


template <typename TPayloadType, typename T, int d>
void FreeObjHelper(TSpatialAccelerationBucketEntry<TPayloadType, T, d >& BucketEntry)
{
	BucketEntry.Acceleration = nullptr;
}

template <typename T>
T CopyFromHelper(const T& Src)
{
	return Src;
}

template <typename TPayloadType, typename T, int d>
TSpatialAccelerationBucketEntry<TPayloadType, T, d> CopyFromHelper(const TSpatialAccelerationBucketEntry<TPayloadType, T, d >& Src)
{
	TSpatialAccelerationBucketEntry<TPayloadType, T, d> Copy;
	Copy.Acceleration = Src.Acceleration ? Src.Acceleration->Copy() : nullptr;
	Copy.TypeInnerIdx = Src.TypeInnerIdx;
	return Copy;
}


template <typename TObj>
struct TSpatialCollectionBucket
{
	TSpatialCollectionBucket() = default;
	TSpatialCollectionBucket(const TSpatialCollectionBucket<TObj>& Other) = delete;
	TSpatialCollectionBucket(TSpatialCollectionBucket<TObj>&& Other) = default;
	TSpatialCollectionBucket<TObj>& operator=(const TSpatialCollectionBucket<TObj>& Other) = delete;
	TSpatialCollectionBucket<TObj>& operator=(TSpatialCollectionBucket<TObj>&& Other) = default;

	TArray<TObj> Objects;
	TArray<uint16> FreeIndices;

	int32 Add(TObj&& Obj)
	{
		uint16 Idx;
		if (FreeIndices.Num())
		{
			Idx = FreeIndices.Pop();
			Objects[Idx] = MoveTemp(Obj);
		}
		else
		{
			Idx = static_cast<uint16>(Objects.Add(MoveTemp(Obj)));
		}

		return Idx;
	}

   	void UpdateOrAddAt(uint16 Idx, TObj&& Obj)
	{
		uint16 ObjectNum = static_cast<uint16>(Objects.Num());
		if (ObjectNum <= Idx)
		{
			Objects.SetNum(Idx + 1);
			for (uint16 FreeIndex = ObjectNum; FreeIndex < Idx - 1; FreeIndex++)
			{
				FreeIndices.Add(FreeIndex);
			}
		}
		else
		{
			FreeIndices.Remove(Idx);
		}

		Objects[Idx] = MoveTemp(Obj);
	}

	void Remove(uint16 Idx)
	{
		if (Objects.Num() == Idx + 1)
		{
			Objects.Pop(EAllowShrinking::No);
		}
		else
		{
			FreeObjHelper(Objects[Idx]);
			FreeIndices.Add(Idx);
		}
	}

	void CopyFrom(const TSpatialCollectionBucket& Src)
	{
		Objects.SetNum(Src.Objects.Num());
		for (int32 ObjIdx = 0; ObjIdx < Objects.Num(); ++ObjIdx)
		{
			Objects[ObjIdx] = CopyFromHelper(Src.Objects[ObjIdx]);
		}
		FreeIndices = Src.FreeIndices;
	}

};

template <typename TObj>
FChaosArchive& operator<<(FChaosArchive& Ar, TSpatialCollectionBucket<TObj>& Bucket)
{
	Ar << Bucket.Objects;
	Ar << Bucket.FreeIndices;
	return Ar;
}

template <typename... TRemaining>
struct TSpatialTypeTuple
{
};

template <typename TAcceleration, typename... TRemaining>
struct TSpatialTypeTuple<TAcceleration, TRemaining...>
{
	using FirstType = TAcceleration;
	TSpatialCollectionBucket<TAcceleration*> First;
	TSpatialTypeTuple<TRemaining...> Remaining;

	TSpatialTypeTuple() = default;

	TSpatialTypeTuple(const TSpatialTypeTuple<TAcceleration, TRemaining...>& Other) = delete;	//raw pointers need updating because buckets use a deep copy
};

template<int Idx, typename ... Rest>
struct TSpatialTypeTupleGetter
{

};

template<int Idx, typename First, typename... Rest>
struct TSpatialTypeTupleGetter<Idx, First, Rest...>
{
	static auto& Get(TSpatialTypeTuple<First, Rest...>& Types) { return TSpatialTypeTupleGetter<Idx - 1, Rest...>::Get(Types.Remaining); }
	static const auto& Get(const TSpatialTypeTuple<First, Rest...>& Types) { return TSpatialTypeTupleGetter<Idx - 1, Rest...>::Get(Types.Remaining); }
};

template<typename First, typename... Rest>
struct TSpatialTypeTupleGetter<0, First, Rest...>
{
	static auto& Get(TSpatialTypeTuple<First, Rest...>& Types) { return Types.First; }
	static const auto& Get(const TSpatialTypeTuple<First, Rest...>& Types) { return Types.First; }
};


// Gets a bucket of acceleration structure pointers
template <int Idx, typename First, typename... Rest>
auto& GetAccelerationsPerType(TSpatialTypeTuple<First, Rest...>& Types)
{
	return TSpatialTypeTupleGetter<Idx, First, Rest...>::Get(Types);
}

template <int Idx, typename First, typename... Rest>
const auto& GetAccelerationsPerType(const TSpatialTypeTuple<First, Rest...>& Types)
{
	return TSpatialTypeTupleGetter<Idx, First, Rest...>::Get(Types);
}

template <int TypeIdx, int NumTypes, typename Tuple, typename TPayloadType, typename T, int d>
struct TSpatialAccelerationCollectionHelper
{
	template <typename SQVisitor>
	static bool RaycastFast(const Tuple& Types, const TVector<T, d>& Start, FQueryFastData& CurData, SQVisitor& Visitor, const FVec3& Dir, const FVec3 InvDir, const bool bParallel[3])
	{
		const auto& Accelerations = GetAccelerationsPerType<TypeIdx>(Types).Objects;
		for (const auto& Accelerator : Accelerations)
		{
			if (Accelerator && !Accelerator->RaycastFast(Start, CurData, Visitor, Dir, InvDir, bParallel))
			{
				return false;
			}
		}
		
		constexpr int NextType = TypeIdx + 1;
		if (NextType < NumTypes)
		{
			return TSpatialAccelerationCollectionHelper<NextType < NumTypes ? NextType : 0, NumTypes, Tuple, TPayloadType, T, d>::RaycastFast(Types, Start, CurData, Visitor, Dir, InvDir, bParallel);
		}

		return true;
	}

	template <typename SQVisitor>
	static bool SweepFast(const Tuple& Types, const TVector<T, d>& Start, FQueryFastData& CurData, const TVector<T, d> QueryHalfExtents, SQVisitor& Visitor, const FVec3& Dir, const FVec3 InvDir, const bool bParallel[3])
	{
		const auto& Accelerations = GetAccelerationsPerType<TypeIdx>(Types).Objects;
		for (const auto& Accelerator : Accelerations)
		{
			if (Accelerator && !Accelerator->SweepFast(Start, CurData, QueryHalfExtents, Visitor, Dir, InvDir, bParallel))
			{
				return false;
			}
		}

		constexpr int NextType = TypeIdx + 1;
		if (NextType < NumTypes)
		{
			return TSpatialAccelerationCollectionHelper < NextType < NumTypes ? NextType : 0, NumTypes, Tuple, TPayloadType, T, d>::SweepFast(Types, Start, CurData, QueryHalfExtents, Visitor, Dir, InvDir, bParallel);
		}

		return true;
	}

	template <typename SQVisitor>
	static bool OverlapFast(const Tuple& Types, const TAABB<T, d> QueryBounds, SQVisitor& Visitor)
	{
		const auto& Accelerations = GetAccelerationsPerType<TypeIdx>(Types).Objects;
		for (const auto& Accelerator : Accelerations)
		{
			if (Accelerator && !Accelerator->OverlapFast(QueryBounds, Visitor))
			{
				return false;
			}
		}

		constexpr int NextType = TypeIdx + 1;
		if (NextType < NumTypes)
		{
			return TSpatialAccelerationCollectionHelper < NextType < NumTypes ? NextType : 0, NumTypes, Tuple, TPayloadType, T, d>::OverlapFast(Types, QueryBounds, Visitor);
		}

		return true;
	}

	static void GlobalObjects(const Tuple& Types, TArray<TPayloadBoundsElement<TPayloadType, T>>& ObjList)
	{
		const auto& Accelerations = GetAccelerationsPerType<TypeIdx>(Types).Objects;
		for (const auto& Accelerator : Accelerations)
		{
			if (Accelerator)
			{
				//todo: handle early out
				ObjList.Append(Accelerator->GlobalObjects());
			}
		}

		constexpr int NextType = TypeIdx + 1;
		if (NextType < NumTypes)
		{
			TSpatialAccelerationCollectionHelper < NextType < NumTypes ? NextType : 0, NumTypes, Tuple, TPayloadType, T, d>::GlobalObjects(Types, ObjList);
		}
	}

	static void SetNumFrom(const Tuple& TypesSrc, Tuple& TypesDest)
	{
		const auto& SrcAccelerations = GetAccelerationsPerType<TypeIdx>(TypesSrc).Objects;
		auto& DestAccelerations = GetAccelerationsPerType<TypeIdx>(TypesDest).Objects;
		DestAccelerations.SetNum(SrcAccelerations.Num());

		constexpr int NextType = TypeIdx + 1;
		if (NextType < NumTypes)
		{
			TSpatialAccelerationCollectionHelper < NextType < NumTypes ? NextType : 0, NumTypes, Tuple, TPayloadType, T, d>::SetNumFrom(TypesSrc, TypesDest);
		}
	}

	static void Reset(const Tuple& Types)
	{
		auto& Accelerations = GetAccelerationsPerType<TypeIdx>(Types).Objects;
		for (auto& Accelerator : Accelerations)
		{
			if (Accelerator)
			{
				Accelerator->Reset();
			}
		}

		constexpr int NextType = TypeIdx + 1;
		if (NextType < NumTypes)
		{
			TSpatialAccelerationCollectionHelper < NextType < NumTypes ? NextType : 0, NumTypes, Tuple, TPayloadType, T, d>::Reset(Types);
		}
	}

	/** Compute the overlapping leaves for each leaf */
	static void CacheOverlappingLeaves(const Tuple& Types)
	{
		auto& Accelerations = GetAccelerationsPerType<TypeIdx>(Types).Objects;
		for (auto& Accelerator : Accelerations)
		{
			if (Accelerator)
			{
				Accelerator->CacheOverlappingLeaves();
			}
		}
		constexpr int NextType = TypeIdx + 1;
		if (NextType < NumTypes)
		{
			TSpatialAccelerationCollectionHelper < NextType < NumTypes ? NextType : 0, NumTypes, Tuple, TPayloadType, T, d>::CacheOverlappingLeaves(Types);
		}
	}
	
	static uint16 FindTypeIdx(const Tuple& Types, SpatialAccelerationType Type)
	{
		using AccelType = typename std::remove_pointer<typename decltype(GetAccelerationsPerType<TypeIdx>(Types).Objects)::ElementType>::type;
		if (AccelType::StaticType == Type)
		{
			return TypeIdx;
		}

		constexpr int NextType = TypeIdx + 1;
		if (NextType < NumTypes)
		{
			return TSpatialAccelerationCollectionHelper < NextType < NumTypes ? NextType : 0, NumTypes, Tuple, TPayloadType, T, d>::FindTypeIdx(Types, Type);
		}
		else
		{
			return NumTypes;
		}
	}
};

template <typename SpatialAccelerationCollection>
typename std::enable_if_t<std::is_same_v<typename SpatialAccelerationCollection::TPayloadType, FAccelerationStructureHandle>, void> PBDComputeConstraintsLowLevel_Helper(FReal Dt, const SpatialAccelerationCollection& Accel, FSpatialAccelerationBroadPhase& BroadPhase, Private::FCollisionConstraintAllocator* Allocator, const FCollisionDetectorSettings& Settings, IResimCacheBase* ResimCache)
{
	BroadPhase.ProduceOverlaps(Dt, Accel, Allocator, Settings, ResimCache);
}

template <typename SpatialAccelerationCollection>
typename std::enable_if_t<!std::is_same_v<typename SpatialAccelerationCollection::TPayloadType, FAccelerationStructureHandle>, void> PBDComputeConstraintsLowLevel_Helper(FReal Dt, const SpatialAccelerationCollection& Accel, FSpatialAccelerationBroadPhase& BroadPhase, Private::FCollisionConstraintAllocator* Allocator, const FCollisionDetectorSettings& Settings, IResimCacheBase* ResimCache)
{
}

template <typename ... TSpatialAccelerationTypes>
class TSpatialAccelerationCollection : public
	ISpatialAccelerationCollection<typename std::tuple_element<0, std::tuple< TSpatialAccelerationTypes...>>::type::PayloadType,
	typename std::tuple_element<0, std::tuple< TSpatialAccelerationTypes...>>::type::TType,
	std::tuple_element<0, std::tuple< TSpatialAccelerationTypes...>>::type::D>
{
public:
	using FirstAccelerationType = typename std::tuple_element<0, std::tuple< TSpatialAccelerationTypes...>>::type;
	using TPayloadType = typename FirstAccelerationType::PayloadType;
	using T = typename FirstAccelerationType::TType;
	static constexpr int d = FirstAccelerationType::D;
	using BucketEntryType = TSpatialAccelerationBucketEntry<TPayloadType, T, d>;
	using BucketType = TSpatialCollectionBucket<BucketEntryType>;

	TSpatialAccelerationCollection()
	{
	}

	virtual FSpatialAccelerationIdx AddSubstructure(TUniquePtr<ISpatialAcceleration<TPayloadType, T, d>>&& Substructure, uint16 BucketIdx, uint16 BucketInnerIdx) override
	{
		check(BucketIdx < MaxBuckets);
		FSpatialAccelerationIdx Result;
		Result.Bucket = BucketIdx;
		
		ISpatialAcceleration<TPayloadType, T, d>* AccelPtr = Substructure.Get();
		BucketEntryType BucketEntry;
		BucketEntry.Acceleration = MoveTemp(Substructure);
		
		const int32 TypeIdx = GetTypeIdx(AccelPtr);
		switch (TypeIdx)
		{
		case 0: BucketEntry.TypeInnerIdx = static_cast<uint16>(GetAccelerationsPerType<0>(Types).Add(&AccelPtr->template AsChecked<typename std::tuple_element<0, std::tuple<TSpatialAccelerationTypes...>>::type>())); break;
		case 1: BucketEntry.TypeInnerIdx = static_cast<uint16>(GetAccelerationsPerType<ClampedIdx(1)>(Types).Add(&AccelPtr->template AsChecked<typename std::tuple_element<ClampedIdx(1), std::tuple<TSpatialAccelerationTypes...>>::type>())); break;
		case 2: BucketEntry.TypeInnerIdx = static_cast<uint16>(GetAccelerationsPerType<ClampedIdx(2)>(Types).Add(&AccelPtr->template AsChecked<typename std::tuple_element<ClampedIdx(2), std::tuple<TSpatialAccelerationTypes...>>::type>())); break;
		case 3: BucketEntry.TypeInnerIdx = static_cast<uint16>(GetAccelerationsPerType<ClampedIdx(3)>(Types).Add(&AccelPtr->template AsChecked<typename std::tuple_element<ClampedIdx(3), std::tuple<TSpatialAccelerationTypes...>>::type>())); break;
		case 4: BucketEntry.TypeInnerIdx = static_cast<uint16>(GetAccelerationsPerType<ClampedIdx(4)>(Types).Add(&AccelPtr->template AsChecked<typename std::tuple_element<ClampedIdx(4), std::tuple<TSpatialAccelerationTypes...>>::type>())); break;
		case 5: BucketEntry.TypeInnerIdx = static_cast<uint16>(GetAccelerationsPerType<ClampedIdx(5)>(Types).Add(&AccelPtr->template AsChecked<typename std::tuple_element<ClampedIdx(5), std::tuple<TSpatialAccelerationTypes...>>::type>())); break;
		case 6: BucketEntry.TypeInnerIdx = static_cast<uint16>(GetAccelerationsPerType<ClampedIdx(6)>(Types).Add(&AccelPtr->template AsChecked<typename std::tuple_element<ClampedIdx(6), std::tuple<TSpatialAccelerationTypes...>>::type>())); break;
		case 7: BucketEntry.TypeInnerIdx = static_cast<uint16>(GetAccelerationsPerType<ClampedIdx(7)>(Types).Add(&AccelPtr->template AsChecked<typename std::tuple_element<ClampedIdx(7), std::tuple<TSpatialAccelerationTypes...>>::type>())); break;
		}

		this->ActiveBucketsMask |= (1 << BucketIdx);
		
		Result.InnerIdx = BucketInnerIdx;
		Buckets[BucketIdx].UpdateOrAddAt(BucketInnerIdx, MoveTemp(BucketEntry));
		
		return Result;
	}

	/** Dispatch the compute the overlapping leaves for each collection helper */
	virtual void CacheOverlappingLeaves() override
	{
		TSpatialAccelerationCollectionHelper<0, NumTypes, decltype(Types), TPayloadType, T, d>::CacheOverlappingLeaves(Types);
	}

	virtual TUniquePtr <ISpatialAcceleration<TPayloadType, T, d>> RemoveSubstructure(FSpatialAccelerationIdx Idx) override
	{
		TUniquePtr <ISpatialAcceleration<TPayloadType, T, d>> Removed;
		{
			BucketEntryType& BucketEntry = Buckets[Idx.Bucket].Objects[Idx.InnerIdx];
			const int32 TypeIdx = GetTypeIdx(BucketEntry.Acceleration.Get());
			switch (TypeIdx)
			{
			case 0: GetAccelerationsPerType<0>(Types).Remove(BucketEntry.TypeInnerIdx); break;
			case 1: GetAccelerationsPerType<ClampedIdx(1)>(Types).Remove(BucketEntry.TypeInnerIdx); break;
			case 2: GetAccelerationsPerType<ClampedIdx(2)>(Types).Remove(BucketEntry.TypeInnerIdx); break;
			case 3: GetAccelerationsPerType<ClampedIdx(3)>(Types).Remove(BucketEntry.TypeInnerIdx); break;
			case 4: GetAccelerationsPerType<ClampedIdx(4)>(Types).Remove(BucketEntry.TypeInnerIdx); break;
			case 5: GetAccelerationsPerType<ClampedIdx(5)>(Types).Remove(BucketEntry.TypeInnerIdx); break;
			case 6: GetAccelerationsPerType<ClampedIdx(6)>(Types).Remove(BucketEntry.TypeInnerIdx); break;
			case 7: GetAccelerationsPerType<ClampedIdx(7)>(Types).Remove(BucketEntry.TypeInnerIdx); break;
			}
			Removed = MoveTemp(BucketEntry.Acceleration);
		}
		Buckets[Idx.Bucket].Remove(Idx.InnerIdx);
		if (Buckets[Idx.Bucket].Objects.Num() == 0)
		{
			this->ActiveBucketsMask &= ~(1 << Idx.Bucket);
		}
		return Removed;
	}

	virtual ISpatialAcceleration<TPayloadType, T, d>* GetSubstructure(FSpatialAccelerationIdx Idx) override
	{
		check(Idx.Bucket < MaxBuckets);
		if (Buckets[Idx.Bucket].Objects.Num() > 0)
		{
			return Buckets[Idx.Bucket].Objects[Idx.InnerIdx].Acceleration.Get();
		}

		return nullptr;
	}

	virtual void SwapSubstructure(ISpatialAccelerationCollection<TPayloadType, T, d>& InOther, FSpatialAccelerationIdx Idx) override
	{
		using ThisType = decltype(this);
		ThisType Other = static_cast<ThisType>(&InOther);

		check(Idx.Bucket < MaxBuckets);
		check(Idx.Bucket < Other->MaxBuckets);
		check(Buckets[Idx.Bucket].Objects.Num() > Idx.InnerIdx);
		check(Other->Buckets[Idx.Bucket].Objects.Num() > Idx.InnerIdx);

		TUniquePtr<ISpatialAcceleration<TPayloadType, T, d>> OtherSubStructure = Other->RemoveSubstructure(Idx);
		TUniquePtr<ISpatialAcceleration<TPayloadType, T, d>> ThisSubStructure = RemoveSubstructure(Idx);

		Other->AddSubstructure(MoveTemp(ThisSubStructure), Idx.Bucket, Idx.InnerIdx);
		AddSubstructure(MoveTemp(OtherSubStructure), Idx.Bucket, Idx.InnerIdx);
	}
	
	virtual void Reset() override
	{
		SCOPE_CYCLE_COUNTER(STAT_AccelerationStructureReset);
		TSpatialAccelerationCollectionHelper<0, NumTypes, decltype(Types), TPayloadType, T, d>::Reset(Types);
	}

	virtual void Raycast(const TVector<T, d>& Start, const TVector<T, d>& Dir, const T Length, ISpatialVisitor<TPayloadType, T>& Visitor) const override
	{
		TSpatialVisitor<TPayloadType, T> ProxyVisitor(Visitor);
		Raycast(Start, Dir, Length, ProxyVisitor);
	}

	template <typename SQVisitor>
	void Raycast(const TVector<T, d>& Start, const TVector<T, d>& Dir, const T Length, SQVisitor& Visitor) const
	{
		FQueryFastData QueryFastData(Dir, Length);
		TSpatialAccelerationCollectionHelper<0, NumTypes, decltype(Types), TPayloadType, T, d>::RaycastFast(Types, Start, QueryFastData, Visitor, QueryFastData.Dir, QueryFastData.InvDir, QueryFastData.bParallel);
	}

	void Sweep(const TVector<T, d>& Start, const TVector<T, d>& Dir, const T Length, const TVector<T, d> QueryHalfExtents, ISpatialVisitor<TPayloadType, T>& Visitor) const override
	{
		TSpatialVisitor<TPayloadType, T> ProxyVisitor(Visitor);
		Sweep(Start, Dir, Length, QueryHalfExtents, ProxyVisitor);
	}

	template <typename SQVisitor>
	void Sweep(const TVector<T, d>& Start, const TVector<T, d>& Dir, const T Length, const TVector<T, d> QueryHalfExtents, SQVisitor& Visitor) const
	{
		FQueryFastData QueryFastData(Dir, Length);
		TSpatialAccelerationCollectionHelper<0, NumTypes, decltype(Types), TPayloadType, T, d>::SweepFast(Types, Start, QueryFastData, QueryHalfExtents, Visitor, QueryFastData.Dir, QueryFastData.InvDir, QueryFastData.bParallel);
	}

	virtual void Overlap(const TAABB<T, d>& QueryBounds, ISpatialVisitor<TPayloadType, T>& Visitor) const override
	{
		TSpatialVisitor<TPayloadType, T> ProxyVisitor(Visitor);
		Overlap(QueryBounds, ProxyVisitor);
	}

	template <typename SQVisitor>
	typename std::enable_if_t<!std::is_same_v<SQVisitor, typename Private::FSimOverlapVisitor>, void> Overlap(const TAABB<T, 3>& QueryBounds, SQVisitor& Visitor) const
	{
		TSpatialAccelerationCollectionHelper<0, NumTypes, decltype(Types), TPayloadType, T, d>::OverlapFast(Types, QueryBounds, Visitor);
	}

	template <typename SQVisitor>
	typename std::enable_if_t<std::is_same_v<SQVisitor, typename Private::FSimOverlapVisitor>, void> Overlap(const TAABB<T, 3>& QueryBounds, SQVisitor& Visitor) const
	{		
		const BucketType& Bucket = Buckets[0];
		{
			int BucketInnerIdx = 0;
			for (const BucketEntryType& BucketEntry : Bucket.Objects)
			{
				constexpr uint16 DefaultQueryOnlyAsUInt16 = static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::DefaultQueryOnly);
				constexpr uint16 DynamicQueryOnlyAsUInt16 = static_cast<std::underlying_type_t<ESpatialAccelerationCollectionBucketInnerIndex>>(ESpatialAccelerationCollectionBucketInnerIndex::DynamicQueryOnly);			
				
				if (BucketInnerIdx != DefaultQueryOnlyAsUInt16 && BucketInnerIdx != DynamicQueryOnlyAsUInt16)
				{
					auto AccelStructure = static_cast<Chaos::TAABBTree<TPayloadType, Chaos::TAABBTreeLeafArray<Chaos::FAccelerationStructureHandle, true, T>, true, T>*>(BucketEntry.Acceleration.Get());
					AccelStructure->OverlapFast(QueryBounds, Visitor);
				}
				BucketInnerIdx++;
			}
		}
	}

	TArray<TPayloadBoundsElement<TPayloadType, T>> GlobalObjects() const
	{
		TArray<TPayloadBoundsElement<TPayloadType, T>> ObjList;
		TSpatialAccelerationCollectionHelper<0, NumTypes, decltype(Types), TPayloadType, T, d>::GlobalObjects(Types, ObjList);
		return ObjList;
	}

	// visitor signature is : void Visitor(FSpatialAccelerationIdx SpatialIdx)
	template <typename TVisitor>
	void VisitAllSpatialIndices(TVisitor Visitor) const 
	{
		uint16 BucketIdx = 0;
		for (const BucketType& Bucket : Buckets)
		{
			uint16 InnerIdx = 0;
			for (const BucketEntryType& Entry : Bucket.Objects)
			{
				if (Entry.Acceleration)
				{
					Visitor(FSpatialAccelerationIdx{ BucketIdx, InnerIdx });
				}
				++InnerIdx;
			}
			++BucketIdx;
		}
	}

	virtual TArray<FSpatialAccelerationIdx> GetAllSpatialIndices() const override
	{
		uint16 BucketIdx = 0;
		TArray<FSpatialAccelerationIdx> Indices;
		VisitAllSpatialIndices(
			[&Indices](FSpatialAccelerationIdx Idx)
			{
				Indices.Add(Idx);
			}
		);
		return Indices;
	}

	// Returns true if the element was in fact removed from some SpatialIdx, otherwise returns false if the element was not found anywhere
	virtual bool RemoveElementFrom(const TPayloadType& Payload, FSpatialAccelerationIdx SpatialIdx) override
	{
		const uint16 UseBucket = ((1 << SpatialIdx.Bucket) & this->ActiveBucketsMask) ? SpatialIdx.Bucket : 0;
		bool bSuccess = Buckets[UseBucket].Objects[SpatialIdx.InnerIdx].Acceleration->RemoveElement(Payload);
		if (!bSuccess)
		{
			// Make sure that we remove this Payload even if the SpatialIdx is wrong
			VisitAllSpatialIndices(
				[this, &Payload, &SpatialIdx, &bSuccess](FSpatialAccelerationIdx Idx)
				{
					if (!(Idx == SpatialIdx))
					{
						const uint16 Buckt = ((1 << Idx.Bucket) & this->ActiveBucketsMask) ? Idx.Bucket : 0;
						const bool bRemoved = Buckets[Buckt].Objects[Idx.InnerIdx].Acceleration->RemoveElement(Payload);
						bSuccess |= bRemoved;
					}
				}
			);
		}
		return bSuccess;
	}

	virtual bool NeedUpdateElementIn(const TPayloadType& Payload, const TAABB<T, d>& NewBounds, FSpatialAccelerationIdx SpatialIdx)
	{
		const uint16 UseBucket = ((1 << SpatialIdx.Bucket) & this->ActiveBucketsMask) ? SpatialIdx.Bucket : 0;
		return  Buckets[UseBucket].Objects[SpatialIdx.InnerIdx].Acceleration->NeedUpdateElement(Payload, NewBounds);
	}

	// Returns true if the element was in fact updated or if it was moved from one SpatialIdx to another during the update or false if not found at any SpatialIdx
	virtual bool UpdateElementIn(const TPayloadType& Payload, const TAABB<T, d>& NewBounds, bool bHasBounds, FSpatialAccelerationIdx SpatialIdx)
	{
		const uint16 UseBucket = ((1 << SpatialIdx.Bucket) & this->ActiveBucketsMask) ? SpatialIdx.Bucket : 0;
		bool bElementExisted = Buckets[UseBucket].Objects[SpatialIdx.InnerIdx].Acceleration->UpdateElement(Payload, NewBounds, bHasBounds);
		// In case spatial index changed, remove this element in all other substructures
		if (!bElementExisted)
		{
			VisitAllSpatialIndices(
				[this, &Payload, &SpatialIdx, &bElementExisted](FSpatialAccelerationIdx Idx)
				{
					if (!(Idx == SpatialIdx))
					{
						const uint16 Buckt = ((1 << Idx.Bucket) & this->ActiveBucketsMask) ? Idx.Bucket : 0;
						const bool Removed = Buckets[Buckt].Objects[Idx.InnerIdx].Acceleration->RemoveElement(Payload);
						bElementExisted = bElementExisted || Removed;
					}
				});
		}
		return bElementExisted;
	}

	virtual TUniquePtr<ISpatialAcceleration<TPayloadType, T, d>> Copy() const
	{
		return TUniquePtr<ISpatialAcceleration<TPayloadType, T, d>>(new TSpatialAccelerationCollection<TSpatialAccelerationTypes...>(*this));
	}

	virtual void PBDComputeConstraintsLowLevel(T Dt, FSpatialAccelerationBroadPhase& BroadPhase, Private::FCollisionConstraintAllocator* Allocator, const FCollisionDetectorSettings& Settings, IResimCacheBase* ResimCache) const override
	{
		PBDComputeConstraintsLowLevel_Helper(Dt, *this, BroadPhase, Allocator, Settings, ResimCache);
	}

#if !UE_BUILD_SHIPPING
	virtual void DebugDraw(ISpacialDebugDrawInterface<T>* InInterface) const override
	{
		for (const BucketType& Bucket : Buckets)
		{
			for (const BucketEntryType& Entry : Bucket.Objects)
			{
				if (Entry.Acceleration)
				{
					Entry.Acceleration->DebugDraw(InInterface);
				}
			}
		}
	}
#endif

	virtual void Serialize(FChaosArchive& Ar)
	{
		//todo: let user serialize out bucket params

		//serialize out sub structures
		for (int BucketIdx = 0; BucketIdx < MaxBuckets; ++BucketIdx)
		{
			Ar << Buckets[BucketIdx];

			if (Ar.IsLoading())
			{
				for (const BucketEntryType& Entry : Buckets[BucketIdx].Objects)
				{
					ISpatialAcceleration<TPayloadType, T, d>* RawPtr = Entry.Acceleration.Get();
					const int32 TypeIdx = GetTypeIdx(RawPtr);
					switch (TypeIdx)
					{
					case 0: GetAccelerationsPerType<0>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<0, std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
					case 1: GetAccelerationsPerType<ClampedIdx(1)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(1), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
					case 2: GetAccelerationsPerType<ClampedIdx(2)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(2), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
					case 3: GetAccelerationsPerType<ClampedIdx(3)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(3), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
					case 4: GetAccelerationsPerType<ClampedIdx(4)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(4), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
					case 5: GetAccelerationsPerType<ClampedIdx(5)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(5), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
					case 6: GetAccelerationsPerType<ClampedIdx(6)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(6), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
					case 7: GetAccelerationsPerType<ClampedIdx(7)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(7), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
					}
				}
			}
		}
	}

#if !UE_BUILD_SHIPPING
	void DumpStats() const override
	{
		if(GLog)
		{
			DumpStatsTo(*GLog);
		}
	}

	void DumpStatsTo(FOutputDevice& Ar) const override
	{
		for(int BucketIdx = 0; BucketIdx < MaxBuckets; ++BucketIdx)
		{
			const BucketType& Bucket = Buckets[BucketIdx];
			Ar.Logf(TEXT("Bucket %d (%d entries):"), BucketIdx, Bucket.Objects.Num());

			for(int EntryIdx = 0; EntryIdx < Bucket.Objects.Num(); ++EntryIdx)
			{
				Ar.Logf(TEXT("\tEntry %d"), EntryIdx);
				Bucket.Objects[EntryIdx].Acceleration->DumpStatsTo(Ar);
				Ar.Logf(TEXT(""));
			}
		}
	}
#endif

private:

	TSpatialAccelerationCollection(const TSpatialAccelerationCollection<TSpatialAccelerationTypes...>& Other)
	{
		//need to make a deep copy of the unique ptrs and then update raw ptrs
		TSpatialAccelerationCollectionHelper<0, NumTypes, decltype(Types), TPayloadType, T, d>::SetNumFrom(Other.Types, Types);
		this->ActiveBucketsMask = Other.ActiveBucketsMask;

		for (int BucketIdx = 0; BucketIdx < MaxBuckets; ++BucketIdx)
		{
			Buckets[BucketIdx].CopyFrom(Other.Buckets[BucketIdx]);
			bool bFirst = true;
			for (const BucketEntryType& Entry : Buckets[BucketIdx].Objects)
			{
				ISpatialAcceleration<TPayloadType, T, d>* RawPtr = Entry.Acceleration.Get();
				const int32 TypeIdx = GetTypeIdx(RawPtr);
				switch (TypeIdx)
				{
				case 0: GetAccelerationsPerType<0>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<0, std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
				case 1: GetAccelerationsPerType<ClampedIdx(1)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(1), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
				case 2: GetAccelerationsPerType<ClampedIdx(2)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(2), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
				case 3: GetAccelerationsPerType<ClampedIdx(3)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(3), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
				case 4: GetAccelerationsPerType<ClampedIdx(4)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(4), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
				case 5: GetAccelerationsPerType<ClampedIdx(5)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(5), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
				case 6: GetAccelerationsPerType<ClampedIdx(6)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(6), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
				case 7: GetAccelerationsPerType<ClampedIdx(7)>(Types).Objects[Entry.TypeInnerIdx] = &RawPtr->template AsChecked<typename std::tuple_element<ClampedIdx(7), std::tuple<TSpatialAccelerationTypes...>>::type>(); break;
				}
			}
		}
	}

	TSpatialAccelerationCollection<TSpatialAccelerationTypes...>& operator=(const TSpatialAccelerationCollection<TSpatialAccelerationTypes...>& Other) = delete;
	virtual void DeepAssign(const ISpatialAcceleration<TPayloadType, FReal, 3>& Other) override
	{
		check(false);	//not implemented
	}
	
	static constexpr uint32 ClampedIdx(uint32 Idx)
	{
		return Idx < NumTypes ? Idx : 0;
	}

	static uint8 GetTypeIdx(const ISpatialAcceleration<TPayloadType, T, d>* Accel)
	{
		if (Accel->template As<typename std::tuple_element<0, std::tuple< TSpatialAccelerationTypes...>>::type> ())
		{
			return 0;
		}
		
		if (Accel->template As<typename std::tuple_element<ClampedIdx(1), std::tuple< TSpatialAccelerationTypes...>>::type> ())
		{
			return 1;
		}

		if (Accel->template As<typename std::tuple_element<ClampedIdx(2), std::tuple< TSpatialAccelerationTypes...>>::type> ())
		{
			return 2;
		}

		if (Accel->template As<typename std::tuple_element<ClampedIdx(3), std::tuple< TSpatialAccelerationTypes...>>::type> ())
		{
			return 3;
		}

		if (Accel->template As<typename std::tuple_element<ClampedIdx(4), std::tuple< TSpatialAccelerationTypes...>>::type> ())
		{
			return 4;
		}

		if (Accel->template As<typename std::tuple_element<ClampedIdx(5), std::tuple< TSpatialAccelerationTypes...>>::type> ())
		{
			return 5;
		}

		if (Accel->template As<typename std::tuple_element<ClampedIdx(6), std::tuple< TSpatialAccelerationTypes...>>::type> ())
		{
			return 6;
		}

		if (Accel->template As<typename std::tuple_element<ClampedIdx(7), std::tuple< TSpatialAccelerationTypes...>>::type> ())
		{
			return 7;
		}

		check(false);	//Did you instantiate the collection with the right template arguments?
		return 0;
	}

	static constexpr uint16 MaxBuckets = 8;
	BucketType Buckets[MaxBuckets];
	TSpatialTypeTuple< TSpatialAccelerationTypes...> Types; // Have buckets of acceleration structure pointers
	static constexpr uint32 NumTypes = sizeof...(TSpatialAccelerationTypes);

	friend ::FChaosVDDataWrapperUtils;
};


}
