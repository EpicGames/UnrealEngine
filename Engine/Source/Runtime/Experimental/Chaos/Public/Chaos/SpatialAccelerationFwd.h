// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Real.h"

namespace Chaos
{
	class FAccelerationStructureHandle;
	template <typename TPayload, typename T, int d>
	class ISpatialAcceleration;

	using IDefaultChaosSpatialAcceleration = ISpatialAcceleration<FAccelerationStructureHandle, FReal, 3>;
}