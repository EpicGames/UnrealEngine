// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Box.h"
#include "Chaos/Vector.h"

namespace Chaos
{
template<class T>
class TSpatialHash
{
 public:
	TSpatialHash(const TArray<TVec3<T>>& Particles, const T Radius)
	: MParticles(Particles)
	{
		Init(Radius);
	}

	TSpatialHash(const TArray<TVec3<T>>& Particles)
	: MParticles(Particles)
	{
		Init();
	}

	~TSpatialHash() = default;

	void Update(const TArray<TVec3<T>>& Particles, const T Radius);
	void Update(const TArray<TVec3<T>>& Particles);
	void Update(const T Radius);

	// Returns all the points in MaxRadius, result not sorted
	TArray<int32> GetClosestPoints(const TVec3<T>& Particle, const T MaxRadius);
	// Returns all the points in MaxRadius, no more than MaxCount, result always sorted
	TArray<int32> GetClosestPoints(const TVec3<T>& Particle, const T MaxRadius, const int32 MaxPoints);
	int32 GetClosestPoint(const TVec3<T>& Particle);
	
private:
	void Init(const T Radius);
	void Init();
	
	int32 SmallestAxis() const
	{
		TVec3<T> Extents = MBoundingBox.Extents();

		if (Extents[0] < Extents[1] && Extents[0] < Extents[2])
		{
			return 0;
		}
		else if (Extents[1] < Extents[2])
		{
			return 1;
		}
		else
		{
			return 2;
		}
	}

	int32 ComputeMaxN(const TVec3<T>& Particle, const T Radius);
	TSet<int32> GetNRing(const TVec3<T>& Particle, const int32 N);
	void ComputeGridXYZ(const TVec3<T>& Particle, int32& XIndex, int32& YIndex, int32& ZIndex);

	int32 HashFunction(const TVec3<T>& Particle);
	int32 HashFunction(const int32 XIndex, const int32 YIndex, const int32 ZIndex);

private:
	TArray<TVec3<T>> MParticles;
	T MCellSize;
	TAABB<T, 3> MBoundingBox;
	int32 MNumberOfCellsX, MNumberOfCellsY, MNumberOfCellsZ;
	TMap<int32, TArray<int32>> MHashTable;
};
}
