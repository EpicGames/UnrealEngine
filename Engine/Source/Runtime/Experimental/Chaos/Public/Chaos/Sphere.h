// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Box.h"
#include "Chaos/GJKShape.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/Raycasts.h"
#include "ChaosArchive.h"

#include "Math/VectorRegister.h"

#include "UObject/ReleaseObjectVersion.h"

namespace Chaos
{
	template<typename T, int d>
	struct TSphereSpecializeSamplingHelper
	{
		static FORCEINLINE void ComputeSamplePoints(TArray<TVec3<T>>& Points, const class TSphere<T, 3>& Sphere, const int32 NumPoints)
		{
			check(false);
		}
	};

	template<class T, int d>
	class TSphere final : public FImplicitObject
	{
	public:

		using FImplicitObject::GetTypeName;

		TSphere(const TVector<T, d>& InCenter, const T InRadius)
		    : FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Sphere)
		    , Center(InCenter)
		{
			SetRadius(FRealSingle(InRadius));
		}

		TSphere(const TSphere<T, d>& Other)
		    : FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Sphere)
		    , Center(Other.Center)
		{
			SetRadius(Other.GetRadiusf());
		}

		TSphere(TSphere<T, d>&& Other)
		    : FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Sphere)
		    , Center(MoveTemp(Other.Center))
		{
			SetRadius(Other.GetRadiusf());
		}

		TSphere& operator=(TSphere<T, d>&& InSteal)
		{
			this->Type = InSteal.Type;
			this->bIsConvex = InSteal.bIsConvex;
			this->bDoCollide = InSteal.bDoCollide;
			this->bHasBoundingBox = InSteal.bHasBoundingBox;
			Center = MoveTemp(InSteal.Center);
			SetRadius(InSteal.GetRadiusf());

			return *this;
		}

		virtual ~TSphere() = default;

		static constexpr EImplicitObjectType StaticType()
		{ 
			return ImplicitObjectType::Sphere; 
		}

		UE_DEPRECATED(5.6, "Use GetRadiusf instead.")
		virtual FReal GetRadius() const override
		{
			return Margin;
		}

		virtual FRealSingle GetRadiusf() const override
		{
			return Margin;
		}

		virtual T PhiWithNormal(const TVector<T, d>& InSamplePoint, TVector<T, d>& OutNormal) const override
		{
			OutNormal = InSamplePoint - Center;
			return OutNormal.SafeNormalize() - GetRadiusf();
		}

		bool Intersects(const TSphere<T, d>& Other) const
		{
			FRealSingle CenterDistanceSquared = FVec3f::DistSquared(Other.GetCenterf(), GetCenterf());
			FRealSingle RadialSum = Other.GetRadiusf() + GetRadiusf();
			return RadialSum * RadialSum >= CenterDistanceSquared;
		}

		TVector<T, d> FindClosestPoint(const TVector<T, d>& StartPoint, const T Thickness = (T)0) const
		{
			TVector<T, d> Result = Center + (StartPoint - Center).GetSafeNormal() * (GetRadiusf() + Thickness);
			return Result;
		}

		virtual bool Raycast(const TVector<T, d>& StartPoint, const TVector<T, d>& Dir, const T Length, const T Thickness, T& OutTime, TVector<T, d>& OutPosition, TVector<T, d>& OutNormal, int32& OutFaceIndex) const override
		{
			OutFaceIndex = INDEX_NONE;
			return Raycasts::RaySphere(StartPoint, Dir, Length, Thickness, Center, GetRadiusf(), OutTime, OutPosition, OutNormal);
		}

		virtual Pair<TVector<T, d>, bool> FindClosestIntersectionImp(const TVector<T, d>& StartPoint, const TVector<T, d>& EndPoint, const T Thickness) const override
		{
			TVector<T, d> Direction = EndPoint - StartPoint;
			T Length = Direction.Size();
			Direction = Direction.GetSafeNormal();
			TVector<T, d> SphereToStart = StartPoint - Center;
			T DistanceProjected = TVector<T, d>::DotProduct(Direction, SphereToStart);
			T EffectiveRadius = GetRadiusf() + Thickness;
			T UnderRoot = DistanceProjected * DistanceProjected - SphereToStart.SizeSquared() + EffectiveRadius * EffectiveRadius;
			if (UnderRoot < 0)
			{
				return MakePair(TVector<T, d>(0), false);
			}
			if (UnderRoot == 0)
			{
				if (-DistanceProjected < 0 || -DistanceProjected > Length)
				{
					return MakePair(TVector<T, d>(0), false);
				}
				return MakePair(TVector<T, d>(-DistanceProjected * Direction + StartPoint), true);
			}
			T Root1 = -DistanceProjected + sqrt(UnderRoot);
			T Root2 = -DistanceProjected - sqrt(UnderRoot);
			if (Root1 < 0 || Root1 > Length)
			{
				if (Root2 < 0 || Root2 > Length)
				{
					return MakePair(TVector<T, d>(0), false);
				}
				return MakePair(TVector<T, d>(Root2 * Direction + StartPoint), true);
			}
			if (Root2 < 0 || Root2 > Length)
			{
				return MakePair(TVector<T, d>(Root1 * Direction + StartPoint), true);
			}
			if (Root1 < Root2)
			{
				return MakePair(TVector<T, d>(Root1 * Direction + StartPoint), true);
			}
			return MakePair(TVector<T, d>(Root2 * Direction + StartPoint), true);
		}

		TVector<T, d> Support(const TVector<T, d>& Direction, const T Thickness, int32& VertexIndex) const
		{
			//We want N / ||N|| and to avoid inf
			//So we want N / ||N|| < 1 / eps => N eps < ||N||, but this is clearly true for all eps < 1 and N > 0
			T SizeSqr = Direction.SizeSquared();
			VertexIndex = 0;
			if (SizeSqr <= TNumericLimits<T>::Min())
			{
				return Center;
			}
			const TVector<T,d> Normalized = Direction / sqrt(SizeSqr);

			return Center + Normalized * (GetRadiusf() + Thickness);
		}

		FORCEINLINE const TVector<T, d> SupportCore(const TVector<T, d>& Direction, const FReal InMargin, FReal* OutSupportDelta, int32& VertexIndex) const
		{
			VertexIndex = 0;
			// Note: ignores InMargin, assumed Radius
			return Center;
		}

		FORCEINLINE VectorRegister4Float SupportCoreSimd(const VectorRegister4Float& Direction, const FReal InMargin) const
		{
			return MakeVectorRegisterFloat(Center[0], Center[1], Center[2], 0.0f);
		}
		FORCEINLINE TVector<T, d> SupportCoreScaled(const TVector<T, d>& Direction, const FReal InMargin, const TVector<T, d>& Scale, FReal* OutSupportDelta, int32& VertexIndex) const
		{
			VertexIndex = 0;
			// Note: ignores InMargin, assumed Radius
			return Center * Scale;
		}

		virtual const TAABB<T, d> BoundingBox() const override
		{
			return TAABB<T,d>(Center - FVec3f(GetRadiusf()),Center + FVec3f(GetRadiusf()));
		}

		virtual FAABB3 CalculateTransformedBounds(const FRigidTransform3& Transform) const override
		{
			const FVec3 TransformedCenter = Transform.TransformPosition(FVec3(Center));
			const FVec3 Extents = FVec3(GetRadiusf());
			return FAABB3(TransformedCenter - Extents, TransformedCenter + Extents);
		}

		T GetArea() const 
		{ 
			return GetArea(GetRadiusf());
		}
		
		static T GetArea(const T InRadius)
		{
			static const T FourPI = UE_PI * 4;
			static const T TwoPI = UE_PI * 2;
			return d == 3 ? FourPI * InRadius * InRadius : TwoPI * InRadius;
		}

		T GetVolume() const 
		{
			return GetVolume(GetRadiusf());
		}

		static T GetVolume(const T InRadius)
		{
			check(d == 3);
			static const T FourThirdsPI = 4. / 3 * UE_PI;
			return FourThirdsPI * InRadius * InRadius * InRadius;
		}

		UE_DEPRECATED(5.6, "Use GetCenterf instead")
		const TVector<T, d> GetCenter() const 
		{ 
			return Center; 
		}

		const FVec3f& GetCenterf() const
		{
			return Center;
		}

		const TVector<T, d> GetCenterOfMass() const 
		{ 
			return Center; 
		}
		
		const FVec3f& GetCenterOfMassf() const
		{
			return Center;
		}

		virtual FString ToString() const override
		{
			return FString::Printf(TEXT("Sphere: Center:%s, Radius:%f"), *Center.ToString(), GetRadiusf());
		}

		FORCEINLINE void SerializeImp(FArchive& Ar)
		{
			FImplicitObject::SerializeImp(Ar);
			Ar << Center;

			// Radius is now stored in the base class Margin
			FRealSingle ArRadius = GetRadiusf();
			Ar << ArRadius;
			SetRadius(ArRadius);
		}

		virtual void Serialize(FChaosArchive& Ar) override 
		{ 
			FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName());
			SerializeImp(Ar);
		}

		virtual void Serialize(FArchive& Ar) override
		{ 
			SerializeImp(Ar); 
		}

		/**
		 * Returns sample points centered about the origin.
		 */
		TArray<TVector<T, d>> ComputeLocalSamplePoints(const int NumPoints) const
		{
			TArray<TVector<T, d>> Points;
			
			if(GetRadiusf() <= UE_KINDA_SMALL_NUMBER)
			{
				// If we're too small (and will create NaNs) then just take the centre
				Points.Add(TVector<T, d>(0.0));
			}
			else
			{
				Points.Reserve(NumPoints);
				TSphere<T, d> LocalSphere(TVector<T, d>(0.0), GetRadiusf());
				TSphereSpecializeSamplingHelper<T, d>::ComputeSamplePoints(Points, LocalSphere, NumPoints);
			}

			return Points;
		}

		TArray<TVector<T, d>> ComputeLocalSamplePoints(const T PointsPerUnitArea, const int32 MinPoints = 0, const int32 MaxPoints = 1000) const
		{ 
			return ComputeLocalSamplePoints(FMath::Clamp(static_cast<int32>(ceil(PointsPerUnitArea * GetArea())), MinPoints, MaxPoints)); 
		}

		/**
		 * Returns sample points at the current location of the sphere.
		 */
		TArray<TVector<T, d>> ComputeSamplePoints(const int NumPoints) const
		{
			TArray<TVector<T, d>> Points;
			TSphereSpecializeSamplingHelper<T, d>::ComputeSamplePoints(Points, *this, NumPoints);
			return Points;
		}

		TArray<TVector<T, d>> ComputeSamplePoints(const T PointsPerUnitArea, const int32 MinPoints = 0, const int32 MaxPoints = 1000) const
		{ 
			return ComputeSamplePoints(FMath::Clamp(static_cast<int32>(ceil(PointsPerUnitArea * GetArea())), MinPoints, MaxPoints)); 
		}

		PMatrix<T, d, d> GetInertiaTensor(const T InMass, const bool bInThinShell = false) const 
		{ 
			return GetInertiaTensor(InMass, GetRadiusf(), bInThinShell);
		}

		static PMatrix<T, d, d> GetInertiaTensor(const T InMass, const T InRadius, const bool bInThinShell = false)
		{
			static const T TwoThirds = static_cast<T>(2.0 / 3.0);
			static const T TwoFifths = static_cast<T>(2.0 / 5.0);
			const T Diagonal = bInThinShell ? TwoThirds * InMass * InRadius * InRadius : TwoFifths * InMass * InRadius * InRadius;
			return PMatrix<T, d, d>(Diagonal, Diagonal, Diagonal);
		}

		TRotation<T, d> GetRotationOfMass() const
		{
			return TRotation<T, d>::FromIdentity();
		}

		virtual uint32 GetTypeHash() const override
		{
			const uint32 CenterHash = UE::Math::GetTypeHash(Center);
			const uint32 RadiusHash = ::GetTypeHash(GetRadiusf());
			return HashCombine(CenterHash, RadiusHash);
		}

		virtual Chaos::FImplicitObjectPtr CopyGeometry() const override
		{
			return Chaos::FImplicitObjectPtr(new FSphere(Center, GetRadiusf()));
		}

		virtual Chaos::FImplicitObjectPtr CopyGeometryWithScale(const FVec3& Scale) const override
		{
			return  Chaos::FImplicitObjectPtr(new FSphere(Center * Scale, GetRadiusf() * Scale.Min()));
		}

#if INTEL_ISPC
		// See PerParticlePBDCollisionConstraint.cpp
		// ISPC code has matching structs for interpreting FImplicitObjects.
		// This is used to verify that the structs stay the same.
		struct FISPCDataVerifier
		{
			static constexpr int32 OffsetOfCenter() { return offsetof(TSphere, Center); }
			static constexpr int32 SizeOfCenter() { return sizeof(TSphere::Center); }
		};
		friend FISPCDataVerifier;
#endif // #if INTEL_ISPC

	private:
		void SetRadius(FRealSingle InRadius) { SetMargin(InRadius); }

		FVec3f Center;

	private:

		// TImplicitObject requires ability to default construct when deserializing shapes
		friend FImplicitObject;
		TSphere()
		    : FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Sphere) 
		{}

	};

	template<typename T>
	struct TSphereSpecializeSamplingHelper<T, 2>
	{
		static FORCEINLINE void ComputeSamplePoints(TArray<TVec2<T>>& Points, const TSphere<T, 2>& Sphere, const int32 NumPoints)
		{
			if (NumPoints <= 1 || Sphere.GetRadiusf() < UE_KINDA_SMALL_NUMBER)
			{
				const int32 Offset = Points.AddUninitialized(1);
				Points[Offset] = Sphere.Center();
				return;
			}
			ComputeGoldenSpiralPoints(Points, Sphere, NumPoints);
		}

		/**
		 * Returns \c NumPoints points evenly distributed on a 2D \c Sphere (disk).
		 */
		static FORCEINLINE void ComputeGoldenSpiralPoints(TArray<TVec2<T>>& Points, const TSphere<T, 2>& Sphere, const int32 NumPoints)
		{
			ComputeGoldenSpiralPoints(Points, Sphere.Center(), Sphere.GetRadius(), NumPoints);
		}

		static FORCEINLINE void ComputeGoldenSpiralPoints(
		    TArray<TVec2<T>>& Points,
		    const TVec2<T>& Center,
		    const T Radius,
		    const int32 NumPoints,
		    const int32 SpiralSeed = 0)
		{
			const int32 Offset = Points.AddUninitialized(NumPoints);

			// Stand at the center, turn a golden ratio of whole turns, then emit
			// a point in that direction.
			//
			// Golden ratio: (1 + sqrt(5)) / 2
			// Polar sunflower increment: pi * (1 + sqrt(5))

			// Increment = 10.16640738463053...
			static const T Increment = static_cast<T>(UE_PI * (1.0 + sqrt(5)));
			for (int32 i = 0; i < NumPoints; i++)
			{
				const T Z = static_cast<T>(0.5 + i);
				// sqrt((i+0.5) / NumPoints) sampling i = [0, NumPoints) varies: (0, 1).
				// We then scale to the radius of our Sphere.
				const T R = FMath::Sqrt(Z / static_cast<T>(NumPoints)) * Radius;
				// Theta increases linearly from [Increment/2, Increment*NumPoints)
				const T Theta = Increment * (Z + static_cast<T>(SpiralSeed));

				// Convert polar coordinates to Cartesian, offset by the Sphere's location.
				const int32 Index = i + Offset;
				Points[Index] = Center +TVec2<T>(R * FMath::Cos(Theta), R * FMath::Sin(Theta));

				// Check to make sure the point is inside the sphere
				checkSlow((Points[Index] - Center).Size() - Radius < UE_KINDA_SMALL_NUMBER);
			}
		}
	};

	template<typename T>
	struct TSphereSpecializeSamplingHelper<T, 3>
	{
		static FORCEINLINE void ComputeSamplePoints(TArray<TVec3<T>>& Points, const TSphere<T, 3>& Sphere, const int32 NumPoints)
		{
			if (NumPoints <= 1 || Sphere.GetRadiusf() < UE_KINDA_SMALL_NUMBER)
			{
				const int32 Offset = Points.AddUninitialized(1);
				Points[Offset] = Sphere.GetCenterf();
				return;
			}
			ComputeGoldenSpiralPoints(Points, Sphere, NumPoints);
		}

		/**
		 * Returns \c NumPoints points evenly distributed on a 3D \c Sphere.
		 */
		static FORCEINLINE void ComputeGoldenSpiralPoints(
		    TArray<TVec3<T>>& Points, const TSphere<T, 3>& Sphere, const int32 NumPoints,
		    const bool FirstHalf = true, const bool SecondHalf = true, const int32 SpiralSeed = 0)
		{
			ComputeGoldenSpiralPoints(Points, Sphere.GetCenterf(), Sphere.GetRadiusf(), NumPoints, FirstHalf, SecondHalf, SpiralSeed);
		}

		/**
		 * Use the golden spiral method to evenly distribute points on a sphere.
		 *
		 * The "golden" part is derived from the golden ratio; stand at the center,
		 * turn a golden ratio of whole turns, then emit a point in that direction.
		 *
		 * Points are generated starting from the bottom of the sphere, ending at 
		 * the top.  Contiguous entries in \p Points generally will not be spatially
		 * adjacent.
		 *
		 * \p Points to append to.
		 * \p Center is the center of the sphere.
		 * \p Radius is the radius of the sphere.
		 * \p NumPoints is the number of points to generate.
		 * \p BottomHalf causes the bottom half of the sphere to be generated, 
		 *    starting at \p Center - (0, 0, \p Radius).
		 * \p TopHalf causes the top half of the sphere to be generated, 
		 *    starting at \p Center.
		 * \p SpiralSeed is the starting index for golden spiral generation.  When 
		 *    using this method to continue a spiral started elsewhere, \p SpiralSeed 
		 *    should equal the number of particles already created.
		 */
		static FORCEINLINE void ComputeGoldenSpiralPoints(
		    TArray<TVec3<T>>& Points,
		    const TVec3<T>& Center,
		    const T Radius,
		    const int32 NumPoints,
		    const bool BottomHalf = true,
		    const bool TopHalf = true,
		    const int32 SpiralSeed = 0)
		{
			if (!TopHalf && !BottomHalf)
			{
				return;
			}

			const int32 Offset = Points.AddUninitialized(NumPoints);

			// We use the same method in 3D as 2D, but in spherical coordinates rather than polar.
			//
			// Theta is the angle about the Z axis, relative to the positive X axis.
			// Phi is the angle between the positive Z axis and the line from the origin to the point

			// GRIncrement = 10.16640738463053...
			static const T GRIncrement = static_cast<FReal>(UE_PI * (1.0 + sqrt(5)));

			// If PhiSteps is 2X NumPoints, then we'll only generate half the sphere.
			//const int32 PhiSteps = TopHalf + BottomHalf == 1 ? NumPoints * 2 : NumPoints;
			// If PhiSeed is 0, then we'll generate the sphere from the beginning - the bottom.
			// Otherwise, we'll generate the sphere from the middle.
			//const int32 PhiSeed = !TopHalf && BottomHalf ? NumPoints : 0;

			int32 Index = Offset;
			if (BottomHalf && !TopHalf)
			{
				for (int32 i = 0; i < NumPoints; i++)
				{
					const T Sample = static_cast<T>(0.5 + i);
					// ((i + 0.5) / (NumPoints * 2)) varies: (0.0, 0.5)
					// So, (2 * (i + 0.5) / (NumPoints * 2)) varies: (0.0, 1.0)
					// So, ((2 * (i + 0.5) / (NumPoints * 2)) - 1) varies: (-1, 0.0)
					const T V = static_cast<T>((2.0 * (0.5 + i) / (2.0 * NumPoints)) - 1.0);
					const T Phi = FMath::Acos(V);
					checkSlow(Phi > UE_PI / 2 - UE_KINDA_SMALL_NUMBER);
					const T Theta = GRIncrement * (Sample + static_cast<T>(SpiralSeed));

					// Convert spherical coordinates to Cartesian, scaled by the radius of our Sphere, and offset by its location.
					const T SinPhi = FMath::Sin(Phi);
					TVec3<T>& Pt = Points[Index++];
					Pt = Center +
					    TVec3<T>(
					        Radius * FMath::Cos(Theta) * SinPhi,
					        Radius * FMath::Sin(Theta) * SinPhi,
					        Radius * FMath::Cos(Phi));

					checkSlow(FMath::Abs(TSphere<T, 3>(Center, Radius).SignedDistance(Pt)) < UE_KINDA_SMALL_NUMBER);
					checkSlow(Pt[2] < Center[2] + UE_KINDA_SMALL_NUMBER);
				}
			}
			else if (!BottomHalf && TopHalf)
			{
				for (int32 i = 0; i < NumPoints; i++)
				{
					const T Sample = static_cast<T>(0.5 + i);
					const T V = static_cast<T>((2.0 * (0.5 + i) / (2.0 * NumPoints))); // varies: (0.0, 1.0)
					const T Phi = FMath::Acos(V);
					checkSlow(Phi < UE_PI / 2 + UE_KINDA_SMALL_NUMBER);
					const T Theta = GRIncrement * (Sample + static_cast<T>(SpiralSeed));

					// Convert spherical coordinates to Cartesian, scaled by the radius of our Sphere, and offset by its location.
					const T SinPhi = FMath::Sin(Phi);
					TVec3<T>& Pt = Points[Index++];
					Pt = Center +
					    TVec3<T>(
					        Radius * FMath::Cos(Theta) * SinPhi,
					        Radius * FMath::Sin(Theta) * SinPhi,
					        Radius * FMath::Cos(Phi));

					checkSlow(FMath::Abs(TSphere<T, 3>(Center, Radius).SignedDistance(Pt)) < UE_KINDA_SMALL_NUMBER);
					checkSlow(Pt[2] > Center[2] - UE_KINDA_SMALL_NUMBER);
				}
			}
			else
			{
				for (int32 i = 0; i < NumPoints; i++)
				{
					const T Sample = static_cast<T>(0.5 + i);
					// arccos(x), where x = [-1, 1] varies: [PI, 0]
					// ((i + 0.5) / NumPoints) varies: (0.0, 1.0)
					// So, (2 * (i + 0.5) / NumPoints) varies: (0.0, 2.0)
					// So, (1 - (2 * (i + 0.5) / NumPoints) varies: (-1.0, 1.0)
					// So, Phi varies: (PI, 0) as i varies: [0, NumPoints-1].
					const T Phi = static_cast<T>(FMath::Acos(1.0 - 2.0 * Sample / NumPoints));
					// Theta varies: [5.0832036..., NumPoints*Increment)
					const T Theta = GRIncrement * (Sample + static_cast<T>(SpiralSeed));

					// Convert spherical coordinates to Cartesian, scaled by the radius of our Sphere, and offset by its location.
					const T SinPhi = FMath::Sin(Phi);
					TVec3<T>& Pt = Points[Index++];
					Pt = Center +
					    TVec3<T>(
					        Radius * FMath::Cos(Theta) * SinPhi,
					        Radius * FMath::Sin(Theta) * SinPhi,
					        Radius * FMath::Cos(Phi));

					checkSlow(FMath::Abs(TSphere<T, 3>(Center, Radius).SignedDistance(Pt)) < UE_KINDA_SMALL_NUMBER);
				}
			}
		}

		static FORCEINLINE void ComputeBottomHalfSemiSphere(
		    TArray<TVec3<T>>& Points, const TSphere<T, 3>& Sphere, const int32 NumPoints, const int32 SpiralSeed = 0)
		{
			ComputeGoldenSpiralPoints(Points, Sphere, NumPoints, true, false, SpiralSeed);
		}

		static FORCEINLINE void ComputeTopHalfSemiSphere(
		    TArray<TVec3<T>>& Points, const TSphere<T, 3>& Sphere, const int32 NumPoints, const int32 SpiralSeed = 0)
		{
			ComputeGoldenSpiralPoints(Points, Sphere, NumPoints, false, true, SpiralSeed);
		}
	};

} // namespace Chaos
