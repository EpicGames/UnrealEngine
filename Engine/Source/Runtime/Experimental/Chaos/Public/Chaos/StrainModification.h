// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "ParticleHandleFwd.h"

class IPhysicsProxyBase;
class FGeometryCollectionPhysicsProxy;

namespace Chaos
{
	class FRigidClustering;
	class FClusterUnionPhysicsProxy;
	class FStrainModifierAccessor;
	class FStrainedProxyModifier;
	class FStrainedProxyIterator;
	class FStrainedProxyRange;
	class FStrainModifierAccessor;

	enum EStrainTypes : uint8
	{
		InternalStrain = 1 << 0,
		ExternalStrain = 1 << 1,
		CollisionStrain = 1 << 2
	};

	struct FStrainedProxyAndRoot
	{
		FGeometryCollectionPhysicsProxy* Proxy = nullptr;

		// Can be a root if proxy is a GC or the directly strained particle if the proxy is a cluster union
		Chaos::FPBDRigidClusteredParticleHandle* ParticleHandle = nullptr;

		// true if the particle handle is actually a partial GC attached toa cluster union
		bool bPartialDestruction = false;

		FGeometryCollectionPhysicsProxy* CastToGeometryCollectionProxy() const;
		
		bool IsPartialDestruction() const;

		bool operator==(const  Chaos::FStrainedProxyAndRoot& Other) const;
	};

	// FStrainedProxyModifier
	//
	// User-facing api for accessing the proxy of a strained cluster. Provides const access to the
	// proxy and limited read/write access to its internal strain data.
	class FStrainedProxyModifier
	{
	public:
		FStrainedProxyModifier(FRigidClustering& InRigidClustering, FStrainedProxyAndRoot InProxyAndRoot)
			: RigidClustering(InRigidClustering)
			, ProxyAndRoot(InProxyAndRoot)
			, RestChildren(InitRestChildren(InProxyAndRoot.CastToGeometryCollectionProxy()))
		{ }

		FStrainedProxyModifier(const FStrainedProxyModifier& Other)
			: RigidClustering(Other.RigidClustering)
			, ProxyAndRoot(Other.ProxyAndRoot)
			, RestChildren(Other.RestChildren)
		{ }

		// Get the proxy that owns the strained cluster or clusters
		CHAOS_API const IPhysicsProxyBase* GetProxy() const;

		// Get the original root handle from the geometry collection proxy
		CHAOS_API const Chaos::FPBDRigidParticleHandle* GetOriginalRootHandle() const;

		// Get the physics handle for the strained parent cluster or the strained particle directly if it's part of partial destruction
		CHAOS_API const Chaos::FPBDRigidParticleHandle* GetParticleHandle() const;

		// Get the physics handle for the strained parent cluster
		UE_DEPRECATED(5.4, "This has been replaced by GetParticleHandle and GetOriginalRootHandle for finer grain access to the underlying handles")
		const Chaos::FPBDRigidParticleHandle* GetRootHandle() const { return GetParticleHandle(); }

		// Get the number of level-1 strainable entities (number of rest-children in the per-particle
		// strain model, or number of rest-connections in the edge/area model).
		CHAOS_API int32 GetNumRestBreakables() const;

		// Get the number of breaking strains amongst the level-1 strainables
		// If DoubleCount is true, then add N for each strain which is strong enough to
		// break a connection N times.
		CHAOS_API int32 GetNumBreakingStrains(bool bDoubleCount = true, const uint8 StrainTypes = EStrainTypes::ExternalStrain | EStrainTypes::CollisionStrain) const;

		// Go through the children and find the one with the largest ratio of applied strain vs internal strain and return this ratio
		// @param FatigueThreshold if the applied strains are below the fatgue threshold, they will be ignored and the returned ratio will be 0 
		// @param StrainTypes type strain to account for ( collision and/or external )
		CHAOS_API float GetMaxBreakStrainRatio(const float FatigueThresholdPercent, const float FatigueThresholdMinimum, bool bRelative, const uint8 StrainTypes) const;

		// A helper debug method to pair w/ GetMaxBreakStrainRatio, where it will return the Max Applied Strain used for the Max Break Strain. 
		CHAOS_API float GetStrainUsedForBreakRatio(const float FatigueThresholdPercent, const float FatigueThresholdMinimum, bool bRelative, const uint8 StrainTypes);

		// Clear strains for all strained cluster children
		CHAOS_API void ClearStrains();

		// Adjust strain for all the strained children above the fatigue threshold so that their strain is large enough to break
		// @param FatigueThreshold if the applied strains are below the fatgue threshold, they will be ignored 
		// @param StrainTypes type strain to account for ( collision and/or external )
		CHAOS_API void AdjustStrainForBreak(const float FatigueThresholdPercent, const float FatigueThresholdMinimum, const uint8 StrainTypes);

	private:

		static CHAOS_API const TSet<int32>* InitRestChildren(FGeometryCollectionPhysicsProxy* Proxy);

		FRigidClustering& RigidClustering;
		FStrainedProxyAndRoot ProxyAndRoot;
		const TSet<int32>* RestChildren;
	};

	// FStrainedProxyIterator
	//
	// Iterator produced by FStrainedProxyRange, for looping over proxies which are
	// associated with rigid clusters which have been strained.
	class FStrainedProxyIterator
	{
	public:
		FStrainedProxyIterator(FRigidClustering& InRigidClustering, TArray<FStrainedProxyAndRoot>& InProxyAndRoots, int32 InIndex)
			: RigidClustering(InRigidClustering)
			, ProxyAndRoots(InProxyAndRoots)
			, Index(InIndex)
		{ }

		FStrainedProxyIterator(const FStrainedProxyIterator& Other)
			: RigidClustering(Other.RigidClustering)
			, ProxyAndRoots(Other.ProxyAndRoots)
			, Index(Other.Index)
		{ }

		CHAOS_API FStrainedProxyModifier operator*();
		CHAOS_API FStrainedProxyIterator& operator++();
		CHAOS_API bool operator==(const FStrainedProxyIterator& Other) const;
		bool operator!=(const FStrainedProxyIterator& Other) const
		{
			return !operator==(Other);
		}

	private:
		FRigidClustering& RigidClustering;
		TArray<FStrainedProxyAndRoot>& ProxyAndRoots;
		int32 Index;
	};

	// FStrainedProxyRange
	//
	// Provides an interface for use with ranged-for, for iterating over strained proxies.
	// Constructor produces filtered array of proxies, and begin and end functions produce
	// iterators which can modify strain related properties of the clusters associated
	// with each proxy.
	class FStrainedProxyRange
	{
	public:
		CHAOS_API FStrainedProxyRange(Chaos::FRigidClustering& InRigidClustering, bool bRootLevelOnly, const TArray<FPBDRigidClusteredParticleHandle*>* InStrainedParticles);

		FStrainedProxyRange(const FStrainedProxyRange& Other)
			: RigidClustering(Other.RigidClustering)
			, ProxyAndRoots(Other.ProxyAndRoots)
		{ }

		FStrainedProxyIterator begin()
		{
			return FStrainedProxyIterator(RigidClustering, ProxyAndRoots, 0);
		}

		FStrainedProxyIterator end()
		{
			return FStrainedProxyIterator(RigidClustering, ProxyAndRoots, ProxyAndRoots.Num());
		}

	private:
		FRigidClustering& RigidClustering;
		TArray<FStrainedProxyAndRoot> ProxyAndRoots;
		const TArray<FPBDRigidClusteredParticleHandle*>* StrainedParticles;
	};

	// FStrainModifierAccessor
	//
	// Provides access to strained proxies and clusters
	class FStrainModifierAccessor
	{
	public:

		FStrainModifierAccessor(FRigidClustering& InRigidClustering, const TArray<FPBDRigidClusteredParticleHandle*>* InStrainedParticles = nullptr)
			: RigidClustering(InRigidClustering)
			, StrainedParticles(InStrainedParticles)
		{}

		// Get an iterable range of unique geometry collection proxies which
		// correspond to all strained clusters. Optionally, only include proxies
		// for whom the strained parent is still the original root (ie, unbroken).
		CHAOS_API FStrainedProxyRange GetStrainedProxies(bool bRootLevelOnly = true);

	private:

		FRigidClustering& RigidClustering;
		const TArray<FPBDRigidClusteredParticleHandle*>* StrainedParticles;
	};
}
