// Copyright Epic Games, Inc. All Rights Reserved.

//CHAOS_INNER_SUSP_PROPERTY(OuterProp, FuncName, InnerType)							

CHAOS_INNER_SUSP_PROPERTY(SuspensionSettings, Enabled, bool);
CHAOS_INNER_SUSP_PROPERTY(SuspensionSettings, Target, FVec3);
CHAOS_INNER_SUSP_PROPERTY(SuspensionSettings, HardstopStiffness, FReal);
CHAOS_INNER_SUSP_PROPERTY(SuspensionSettings, HardstopVelocityCompensation, FReal);
CHAOS_INNER_SUSP_PROPERTY(SuspensionSettings, SpringPreload, FReal);
CHAOS_INNER_SUSP_PROPERTY(SuspensionSettings, SpringStiffness, FReal);
CHAOS_INNER_SUSP_PROPERTY(SuspensionSettings, SpringDamping, FReal);
CHAOS_INNER_SUSP_PROPERTY(SuspensionSettings, MinLength, FReal);
CHAOS_INNER_SUSP_PROPERTY(SuspensionSettings, MaxLength, FReal);
CHAOS_INNER_SUSP_PROPERTY(SuspensionSettings, Axis, FVec3);
CHAOS_INNER_SUSP_PROPERTY(SuspensionSettings, Normal, FVec3)
CHAOS_INNER_SUSP_PROPERTY(SuspensionLocation, Location, FVec3)

#undef CHAOS_INNER_SUSP_PROPERTY