// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ImplicitObject.h"
#include "Chaos/Core.h"
#include "Chaos/TaperedCylinder.h"

namespace Chaos
{
	struct FTaperedCapsuleSpecializeSamplingHelper;

	class FTaperedCapsule: public FImplicitObject
	{
	public:
		FTaperedCapsule()
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::TaperedCapsule)
		{
			this->bIsConvex = true;
		}
		FTaperedCapsule(const FVec3& X1, const FVec3& X2, const FReal InRadius1, const FReal InRadius2, const FVec3& SplitXAxis = FVec3::ZeroVector)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::TaperedCapsule)
		    , Origin(X1)
		    , Axis((X2 - X1).GetSafeNormal())
			, OneSidedPlaneNormal((FVec3f(SplitXAxis) - FVec3f(SplitXAxis).Dot(Axis)*Axis).GetSafeNormal())
		    , Height((FRealSingle)((X2 - X1).Size()))
		    , Radius1((FRealSingle)InRadius1)
		    , Radius2((FRealSingle)InRadius2)
			, bIsOneSided(!OneSidedPlaneNormal.IsNearlyZero())
		    , LocalBoundingBox(X1, X1)
		{
			this->bIsConvex = true;
			LocalBoundingBox.GrowToInclude(X2);
			const FRealSingle MaxRadius = FMath::Max(Radius1, Radius2);
			LocalBoundingBox = FAABB3(LocalBoundingBox.Min() - FVec3(MaxRadius), LocalBoundingBox.Max() + FVec3(MaxRadius));
		}
		FTaperedCapsule(const FTaperedCapsule& Other)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::TaperedCapsule)
		    , Origin(Other.Origin)
		    , Axis(Other.Axis)
			, OneSidedPlaneNormal(Other.OneSidedPlaneNormal)
		    , Height(Other.Height)
		    , Radius1(Other.Radius1)
		    , Radius2(Other.Radius2)
			, bIsOneSided(Other.bIsOneSided)
		    , LocalBoundingBox(Other.LocalBoundingBox)
		{
			this->bIsConvex = true;
		}
		FTaperedCapsule(FTaperedCapsule&& Other)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::TaperedCapsule)
		    , Origin(MoveTemp(Other.Origin))
		    , Axis(MoveTemp(Other.Axis))
			, OneSidedPlaneNormal(MoveTemp(Other.OneSidedPlaneNormal))
		    , Height(Other.Height)
		    , Radius1(Other.Radius1)
		    , Radius2(Other.Radius2)
			, bIsOneSided(Other.bIsOneSided)
		    , LocalBoundingBox(MoveTemp(Other.LocalBoundingBox))
		{
			this->bIsConvex = true;
		}
		~FTaperedCapsule() = default;

		static constexpr EImplicitObjectType StaticType() { return ImplicitObjectType::TaperedCapsule; }

		/**
		 * Returns sample points centered about the origin.
		 *
		 * \p NumPoints specifies how many points to generate.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the capsule.
		 */
		TArray<FVec3> ComputeLocalSamplePoints(const int32 NumPoints) const;

		/** 
		 * Returns sample points centered about the origin. 
		 *
		 * \p PointsPerUnitArea specifies how many points to generate per square 
		 *    unit (cm). 0.5 would generate 1 point per 2 square cm.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the capsule.
		 */
		TArray<FVec3> ComputeLocalSamplePoints(const FReal PointsPerUnitArea, const int32 MinPoints = 0, const int32 MaxPoints = 1000) const
		{ 
			return ComputeLocalSamplePoints(FMath::Clamp(static_cast<int32>(ceil(PointsPerUnitArea * GetArea(true))), MinPoints, MaxPoints)); 
		}

		/**
		 * Returns sample points at the current location of the capsule.
		 *
		 * \p NumPoints specifies how many points to generate.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the capsule.
		 */
		TArray<FVec3> ComputeSamplePoints(const int32 NumPoints) const;

		/** 
		 * Returns sample points at the current location of the capsule.
		 *
		 * \p PointsPerUnitArea specifies how many points to generate per square 
		 *    unit (cm). 0.5 would generate 1 point per 2 square cm.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the capsule.
		 */
		TArray<FVec3> ComputeSamplePoints(const FReal PointsPerUnitArea, const int32 MinPoints, const int32 MaxPoints) const
		{ 
			const int32 NumPoints = FMath::Clamp(static_cast<int32>(ceil(PointsPerUnitArea * GetArea(true))), MinPoints, MaxPoints);
			return ComputeSamplePoints(NumPoints);
		}

		virtual const FAABB3 BoundingBox() const override { return LocalBoundingBox; }

		FReal PhiWithNormal(const FVec3& x, FVec3& OutNormal) const
		{
			const FVec3f OriginToX = x - Origin;
			const FRealSingle DistanceAlongAxis = FMath::Clamp(FVec3f::DotProduct(OriginToX, Axis), (FRealSingle)0.0, Height);
			const FVec3f ClosestPoint = Origin + Axis * DistanceAlongAxis;
			const FRealSingle Radius = (Height > UE_SMALL_NUMBER) ? FMath::Lerp(Radius1, Radius2, DistanceAlongAxis / Height) : FMath::Max(Radius1, Radius2);
			OutNormal = (x - ClosestPoint);
			return OutNormal.SafeNormalize() - Radius;
		}

		FReal GetRadius1() const { return Radius1; }
		FReal GetRadius2() const { return Radius2; }
		FReal GetHeight() const { return Height; }
		FReal GetSlantHeight() const { const FReal R1mR2 = Radius1-Radius2; return FMath::Sqrt(R1mR2*R1mR2 + Height*Height); }
		FVec3 GetX1() const { return Origin; }
		FVec3 GetX2() const { return Origin + Axis * Height; }
		/** Returns the bottommost hemisphere center of the capsule. */
		FVec3 GetOrigin() const { return GetX1(); }
		const FVec3f& GetOriginf() const { return Origin; }
		/** Returns the topmost hemisphere center of capsule . */
		FVec3 GetInsertion() const { return GetX2(); }
		FVec3 GetCenter() const { return Origin + Axis * (Height * (FRealSingle)0.5); }
		bool IsOneSided() const { return bIsOneSided; }
		FVec3 GetOneSidedPlaneNormal() const { return OneSidedPlaneNormal; }
		const FVec3f& GetOneSidedPlaneNormalf() const { return OneSidedPlaneNormal; }
		/** Returns the centroid (center of mass). */
		FVec3 GetCenterOfMass() const // centroid
		{
			const FReal R1R1 = Radius1 * Radius1;
			const FReal R2R2 = Radius2 * Radius2;
			const FReal R1R2 = Radius1 * Radius2;
			//  compute center of mass as a distance along the axis from the origin as the shape as the axis as a symmetry line 
			FReal TaperedSectionCenterOfMass = (Height * (R1R1 + (FReal)2.0 * R1R2 + (FReal)3.0 * R2R2) / (FReal)4.0 * (R1R1 + R1R2 + R2R2));
			FReal Hemisphere1CenterOfMass = -((FReal)3.0 * Radius1 / (FReal)8.0);
			FReal Hemisphere2CenterOfMass = (Height + ((FReal)3.0 * Radius2 / (FReal)8.0));

			// we need to combine all 3 using relative volume ratios
			const FReal TaperedSectionVolume = GetTaperedSectionVolume(Height, Radius1, Radius2);
			const FReal Hemisphere1Volume = GetHemisphereVolume(Radius1);
			const FReal Hemisphere2Volume = GetHemisphereVolume(Radius2);
			const FReal TotalVolume = TaperedSectionVolume + Hemisphere1Volume + Hemisphere2Volume;

			const FReal TotalCenterOfMassAlongAxis = ((TaperedSectionCenterOfMass * TaperedSectionVolume) + (Hemisphere1CenterOfMass * Hemisphere1Volume) + (Hemisphere2CenterOfMass * Hemisphere2Volume)) / TotalVolume;
			return FVec3(0,0,1) * TotalCenterOfMassAlongAxis; 
		}
		FVec3 GetAxis() const { return Axis; }

		FReal GetArea(const bool IncludeEndCaps = true) const { return GetArea(Height, Radius1, Radius2, IncludeEndCaps); }
		static FReal GetArea(const FReal Height, const FReal Radius1, const FReal Radius2, const bool IncludeEndCaps)
		{
			static const FReal TwoPI = UE_PI * 2;
			FReal AreaNoCaps = (FReal)0.0;
			if (Radius1 == Radius2)
			{
				AreaNoCaps = TwoPI * Radius1 * Height;
			}
			else
			{
				const FReal R1_R2 = Radius1 - Radius2;
				AreaNoCaps = UE_PI * (Radius1 + Radius2) * FMath::Sqrt((R1_R2 * R1_R2) + (Height * Height));
			}
			if (IncludeEndCaps)
			{
				const FReal Hemisphere1Area = TSphere<FReal, 3>::GetArea(Radius1) / (FReal)2.;
				const FReal Hemisphere2Area = TSphere<FReal, 3>::GetArea(Radius2) / (FReal)2.;
				return AreaNoCaps + Hemisphere1Area + Hemisphere2Area;
			}
			return AreaNoCaps;
		}

		FReal GetVolume() const { return GetVolume(Height, Radius1, Radius2); }
		static FReal GetVolume(const FReal Height, const FReal Radius1, const FReal Radius2)
		{
			const FReal TaperedSectionVolume = GetTaperedSectionVolume(Height, Radius1, Radius2);
			const FReal Hemisphere1Volume = GetHemisphereVolume(Radius1);
			const FReal Hemisphere2Volume = GetHemisphereVolume(Radius2);
			return TaperedSectionVolume + Hemisphere1Volume + Hemisphere2Volume;
		}

		FMatrix33 GetInertiaTensor(const FReal Mass) const { return GetInertiaTensor(Mass, Height, Radius1, Radius2); }
		static FMatrix33 GetInertiaTensor(const FReal Mass, const FReal Height, const FReal Radius1, const FReal Radius2)
		{
			// TODO(chaos) : we should actually take hemispheres in account 
			// https://www.wolframalpha.com/input/?i=conical+frustum
			const FReal R1 = FMath::Min(Radius1, Radius2);
			const FReal R2 = FMath::Max(Radius1, Radius2);
			const FReal HH = Height * Height;
			const FReal R1R1 = R1 * R1;
			const FReal R1R2 = R1 * R2;
			const FReal R2R2 = R2 * R2;

			const FReal Num1 = (FReal)2.0 * HH * (R1R1 + (FReal)3.0 * R1R2 + (FReal)6.0 * R2R2); // 2H^2 * (R1^2 + 3R1R2 + 6R2^2)
			const FReal Num2 = (FReal)3.0 * (R1R1 * R1R1 + R1R1 * R1R2 + R1R2 * R1R2 + R1R2 * R2R2 + R2R2 * R2R2); // 3 * (R1^4 + R1^3R2 + R1^2R2^2 + R1R2^3 + R2^4)
			const FReal Den1 = UE_PI * (R1R1 + R1R2 + R2R2); // PI * (R1^2 + R1R2 + R2^2)

			const FReal Diag12 = Mass * (Num1 + Num2) / ((FReal)20.0 * Den1);
			const FReal Diag3 = Mass * Num2 / ((FReal)10.0 * Den1);

			return FMatrix33(Diag12, Diag12, Diag3);
		}

		FRotation3 GetRotationOfMass() const { return GetRotationOfMass(GetAxis()); }
		static FRotation3 GetRotationOfMass(const FVec3& Axis)
		{
			// since the capsule stores an axis and the InertiaTensor is assumed to be along the ZAxis
			// we need to make sure to return the rotation of the axis from Z
			return FRotation3::FromRotatedVector(FVec3(0, 0, 1), Axis);
		}

		virtual uint32 GetTypeHash() const override
		{
			const uint32 OriginAxisHash = HashCombine(UE::Math::GetTypeHash(Origin), UE::Math::GetTypeHash(Axis));
			const uint32 PropertyHash = HashCombine(::GetTypeHash(Height), HashCombine(::GetTypeHash(Radius1), ::GetTypeHash(Radius2)));

			return HashCombine(OriginAxisHash, PropertyHash);
		}

#if INTEL_ISPC
		// See PerParticlePBDCollisionConstraint.cpp
		// ISPC code has matching structs for interpreting FImplicitObjects.
		// This is used to verify that the structs stay the same.
		struct FISPCDataVerifier
		{
			static constexpr int32 OffsetOfOrigin() { return offsetof(FTaperedCapsule, Origin); }
			static constexpr int32 SizeOfOrigin() { return sizeof(FTaperedCapsule::Origin); }
			static constexpr int32 OffsetOfAxis() { return offsetof(FTaperedCapsule, Axis); }
			static constexpr int32 SizeOfAxis() { return sizeof(FTaperedCapsule::Axis); }
			static constexpr int32 OffsetOfOneSidedPlaneNormal() { return offsetof(FTaperedCapsule, OneSidedPlaneNormal); }
			static constexpr int32 SizeOfOneSidedPlaneNormal() { return sizeof(FTaperedCapsule::OneSidedPlaneNormal); }
			static constexpr int32 OffsetOfHeight() { return offsetof(FTaperedCapsule, Height); }
			static constexpr int32 SizeOfHeight() { return sizeof(FTaperedCapsule::Height); }
			static constexpr int32 OffsetOfRadius1() { return offsetof(FTaperedCapsule, Radius1); }
			static constexpr int32 SizeOfRadius1() { return sizeof(FTaperedCapsule::Radius1); }
			static constexpr int32 OffsetOfRadius2() { return offsetof(FTaperedCapsule, Radius2); }
			static constexpr int32 SizeOfRadius2() { return sizeof(FTaperedCapsule::Radius2); }
			static constexpr int32 OffsetOfIsOneSided() { return offsetof(FTaperedCapsule, bIsOneSided); }
			static constexpr int32 SizeOfIsOneSided() { return sizeof(FTaperedCapsule::bIsOneSided); }
		};
		friend FISPCDataVerifier;
#endif // #if INTEL_ISPC

	private:
		//Phi is distance from closest point on plane1
		FReal GetRadiusAtDistance(const FReal& Phi) const
		{
			const FReal Alpha = Phi / Height;
			return FMath::Lerp(Radius1, Radius2, Alpha);
		}

		static FReal GetHemisphereVolume(const FReal Radius)
		{
			return (FReal)2.0 * UE_PI * (Radius * Radius * Radius) / (FReal)3.0;
		}

		static FReal GetTaperedSectionVolume(const FReal Height, const FReal Radius1, const FReal Radius2)
		{
			static const FReal PI_OVER_3 = UE_PI / (FReal)3.0;
			return PI_OVER_3 * Height * (Radius1 * Radius1 + Radius1 * Radius2 + Radius2 * Radius2);
		}

		FVec3f Origin, Axis;
		FVec3f OneSidedPlaneNormal = FVec3f::ZeroVector;
		FRealSingle Height, Radius1, Radius2;
		bool bIsOneSided = false;
		FAABB3 LocalBoundingBox;
	};

	struct FTaperedCapsuleSpecializeSamplingHelper
	{
		static FORCEINLINE void ComputeSamplePoints(
		    TArray<FVec3>& Points, const FTaperedCapsule& Capsule,
		    const int32 NumPoints)
		{
			if (NumPoints <= 1 ||
			    (Capsule.GetRadius1() <= UE_KINDA_SMALL_NUMBER &&
				 Capsule.GetRadius2() <= UE_KINDA_SMALL_NUMBER))
			{
				const int32 Offset = Points.Num();
				if (Capsule.GetHeight() <= UE_KINDA_SMALL_NUMBER)
				{
					Points.SetNumUninitialized(Offset + 1);
					Points[Offset] = Capsule.GetCenter();
				}
				else
				{
					Points.SetNumUninitialized(Offset + 3);
					Points[Offset + 0] = Capsule.GetOrigin();
					Points[Offset + 1] = Capsule.GetCenter();
					Points[Offset + 2] = Capsule.GetInsertion();
				}
				return;
			}
			ComputeGoldenSpiralPoints(Points, Capsule, NumPoints);
		}

		static FORCEINLINE void ComputeGoldenSpiralPoints(TArray<FVec3>& Points, const FTaperedCapsule& Capsule, const int32 NumPoints)
		{
			ComputeGoldenSpiralPoints(Points, Capsule.GetOrigin(), Capsule.GetAxis(), Capsule.GetRadius1(), Capsule.GetRadius2(), Capsule.GetHeight(), NumPoints);
		}

		/**
		 * Use the golden spiral method to generate evenly spaced points on a tapered 
		 * capsule (truncated cone with two hemispherical ends).
		 *
		 * The "golden" part is derived from the golden ratio; stand at the center,
		 * turn a golden ratio of whole turns, then emit a point in that direction.
		 *
		 * Points are generated starting from the bottom of the tapered capsule part, ending at 
		 * the top.  Contiguous entries in \p Points generally will not be spatially
		 * adjacent.
		 *
		 * \p Points to append to.
		 * \p Origin is the bottom-most point of the tapered capsule.
		 * \p Axis is the orientation of the tapered capsule.
		 * \p Radius1 is the first radius of the tapered capsule.
		 * \p Radius2 is the second radius of the tapered capsule.
		 * \p Height is the height of the tapered capsule.
		 * \p NumPoints is the number of points to generate.
		 * \p SpiralSeed is the starting index for golden spiral generation.  When 
		 *    using this method to continue a spiral started elsewhere, \p SpiralSeed 
		 *    should equal the number of particles already created.
		 */
		static /*FORCEINLINE*/ void ComputeGoldenSpiralPoints(
		    TArray<FVec3>& Points,
		    const FVec3& Origin,
		    const FVec3& Axis,
		    const FReal Radius1,
		    const FReal Radius2,
		    const FReal Height,
		    const int32 NumPoints,
			const int32 SpiralSeed = 0)
		{
			// Axis should be normalized.
			checkSlow(FMath::Abs(Axis.Size() - 1.0) < UE_KINDA_SMALL_NUMBER);

			const int32 Offset = Points.Num();
			ComputeGoldenSpiralPointsUnoriented(Points, Radius1, Radius2, Height, NumPoints, SpiralSeed);

			// At this point, Points are centered about the origin (0,0,0), built
			// along the Z axis.  Transform them to where they should be.
			const FReal HalfHeight = Height / 2;
			const FRotation3 Rotation = FRotation3::FromRotatedVector(FVec3(0, 0, 1), Axis);
			checkSlow(((Origin + Axis * Height) - (Rotation.RotateVector(FVec3(0, 0, Height)) + Origin)).Size() < UE_KINDA_SMALL_NUMBER);
			for (int32 i = Offset; i < Points.Num(); i++)
			{
				FVec3& Point = Points[i];
				const FVec3 PointNew = Rotation.RotateVector(Point + FVec3(0, 0, HalfHeight)) + Origin;
//				checkSlow(FMath::Abs(FTaperedCapsule(Origin, Origin + Axis * Height, Radius1, Radius2).SignedDistance(PointNew)) < KINDA_SMALL_NUMBER);
				Point = PointNew;
			}
		}

		/**
		 * Generates points on a tapered capsule (truncated cone), oriented about 
		 * the Z axis, varying from [-Height/2, Height/2].
		 *
		 * TODO: Note that this method does not produce evenly spaced points!  It'll 
		 * bunch points together on the side of the capsule with the smaller radius, 
		 * and spread them apart on the larger.  We need a routine that operates in 
		 * conical space, rather than cylindrical.  That said, points are distributed 
		 * evenly between the two end caps, proportional to their respective areas.
		 *
		 * The "golden" part is derived from the golden ratio; stand at the center,
		 * turn a golden ratio of whole turns, then emit a point in that direction.
		 *
		 * Points are generated starting from the bottom of the capsule, ending at 
		 * the top.  Contiguous entries in \p Points generally will not be spatially
		 * adjacent.
		 *
		 * \p Points to append to.
		 * \p Radius1 is the first radius of the tapered capsule.
		 * \p Radius2 is the second radius of the tapered capsule.
		 * \p Height is the height of the capsule.
		 * \p NumPoints is the number of points to generate.
		 * \p SpiralSeed is the starting index for golden spiral generation.  When 
		 *    using this method to continue a spiral started elsewhere, \p SpiralSeed 
		 *    should equal the number of particles already created.
		 */
		static /*FORCEINLINE*/ void ComputeGoldenSpiralPointsUnoriented(
		    TArray<FVec3>& Points,
		    const FReal Radius1,
		    const FReal Radius2,
		    const FReal Height,
		    const int32 NumPoints,
			int32 SpiralSeed = 0
		)
		{
			// Evenly distribute points between the capsule body and the end caps.
			int32 NumPointsEndCap1;
			int32 NumPointsEndCap2;
			int32 NumPointsTaperedSection;

			const FReal Cap1Area = FSphere::GetArea(Radius1) / (FReal)2.;
			const FReal Cap2Area = FSphere::GetArea(Radius2) / (FReal)2.;
			const FReal TaperedSectionArea = FTaperedCapsule::GetArea(Height, Radius1, Radius2, /*IncludeEndCaps*/ false);
			const FReal AllArea = TaperedSectionArea + Cap1Area + Cap2Area;
			if (AllArea > UE_KINDA_SMALL_NUMBER)
			{
				NumPointsEndCap1 = static_cast<int32>(round(Cap1Area / AllArea * static_cast<FReal>(NumPoints)));
				NumPointsEndCap2 = static_cast<int32>(round(Cap2Area / AllArea * static_cast<FReal>(NumPoints)));
				NumPointsTaperedSection = NumPoints - NumPointsEndCap1 - NumPointsEndCap2;
			}
			else
			{
				NumPointsTaperedSection = 0;
				NumPointsEndCap1 = NumPointsEndCap2 = (NumPoints - (NumPoints % 2)) / 2;
			}
			
			const int32 NumPointsToAdd = NumPointsTaperedSection + NumPointsEndCap1 + NumPointsEndCap2;
			Points.Reserve(Points.Num() + NumPointsToAdd);

			int32 Offset = Points.Num();
			const FReal HalfHeight = Height / 2;
			{
				// Points vary in Z: [-Radius1-HalfHeight, -HalfHeight]
				TSphereSpecializeSamplingHelper<FReal, 3>::ComputeBottomHalfSemiSphere(
					Points, FSphere(FVec3(0, 0, -HalfHeight), Radius1), NumPointsEndCap1, SpiralSeed);
				SpiralSeed += Points.Num();

				// Points vary in Z: [-HalfHeight, HalfHeight], about the Z axis.
				FTaperedCylinderSpecializeSamplingHelper::ComputeGoldenSpiralPointsUnoriented(
					Points, Radius1, Radius2, Height, NumPointsTaperedSection, false, SpiralSeed);
				SpiralSeed += Points.Num();

				// Points vary in Z: [HalfHeight, HalfHeight+Radius2]
				TSphereSpecializeSamplingHelper<FReal, 3>::ComputeTopHalfSemiSphere(
					Points, FSphere(FVec3(0, 0, HalfHeight), Radius2), NumPointsEndCap2, SpiralSeed);
				SpiralSeed += Points.Num();
			}
		}
	};

	FORCEINLINE TArray<FVec3> FTaperedCapsule::ComputeLocalSamplePoints(const int32 NumPoints) const
	{
		TArray<FVec3> Points;
		FTaperedCapsuleSpecializeSamplingHelper::ComputeSamplePoints(
			Points,
			FTaperedCapsule(Origin, Origin + Axis * Height, GetRadius1(), GetRadius2()),
			NumPoints);
		return Points;
	}

	FORCEINLINE TArray<FVec3> FTaperedCapsule::ComputeSamplePoints(const int32 NumPoints) const
	{
		TArray<FVec3> Points;
		FTaperedCapsuleSpecializeSamplingHelper::ComputeSamplePoints(Points, *this, NumPoints);
		return Points;
	}

	template<class T>
	using TTaperedCapsule UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FTaperedCapsule instead") = FTaperedCapsule;

	template<class T>
	using TTaperedCapsuleSpecializeSamplingHelper UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FTaperedCapsuleSpecializeSamplingHelper instead") = FTaperedCapsuleSpecializeSamplingHelper;

} // namespace Chaos
