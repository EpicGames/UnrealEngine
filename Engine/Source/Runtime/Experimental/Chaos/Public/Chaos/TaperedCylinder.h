// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ImplicitObject.h"
#include "Chaos/Cylinder.h"
#include "Chaos/Plane.h"

namespace Chaos
{
	struct FTaperedCylinderSpecializeSamplingHelper;

	class FTaperedCylinder : public FImplicitObject
	{
	public:
		FTaperedCylinder()
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::TaperedCylinder)
		{
			this->bIsConvex = true;
		}
		FTaperedCylinder(const FVec3& x1, const FVec3& x2, const FReal Radius1, const FReal Radius2)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::TaperedCylinder)
		    , MPlane1(x1, (x2 - x1).GetSafeNormal())
		    , MPlane2(x2, -MPlane1.Normal())
		    , MHeight((FRealSingle)((x2 - x1).Size()))
		    , MRadius1((FRealSingle)Radius1)
		    , MRadius2((FRealSingle)Radius2)
		    , MLocalBoundingBox(x1, x1)
		{
			this->bIsConvex = true;
			MLocalBoundingBox.GrowToInclude(x2);
			FRealSingle MaxRadius = MRadius1;
			if (MaxRadius < MRadius2)
				MaxRadius = MRadius2;
			MLocalBoundingBox = FAABB3(MLocalBoundingBox.Min() - FVec3(MaxRadius), MLocalBoundingBox.Max() + FVec3(MaxRadius));
		}
		FTaperedCylinder(const FTaperedCylinder& Other)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::TaperedCylinder)
		    , MPlane1(Other.MPlane1)
		    , MPlane2(Other.MPlane2)
		    , MHeight(Other.MHeight)
		    , MRadius1(Other.MRadius1)
		    , MRadius2(Other.MRadius2)
		    , MLocalBoundingBox(Other.MLocalBoundingBox)
		{
			this->bIsConvex = true;
		}
		FTaperedCylinder(FTaperedCylinder&& Other)
		    : FImplicitObject(EImplicitObject::FiniteConvex, ImplicitObjectType::TaperedCylinder)
		    , MPlane1(MoveTemp(Other.MPlane1))
		    , MPlane2(MoveTemp(Other.MPlane2))
		    , MHeight(Other.MHeight)
		    , MRadius1(Other.MRadius1)
		    , MRadius2(Other.MRadius2)
		    , MLocalBoundingBox(MoveTemp(Other.MLocalBoundingBox))
		{
			this->bIsConvex = true;
		}
		~FTaperedCylinder() = default;

		static constexpr EImplicitObjectType StaticType() { return ImplicitObjectType::TaperedCylinder; }

		/**
		 * Returns sample points centered about the origin.
		 *
		 * \p NumPoints specifies how many points to generate.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the cylinder.
		 */
		TArray<FVec3> ComputeLocalSamplePoints(const int32 NumPoints, const bool IncludeEndCaps = true) const;

		/** 
		 * Returns sample points centered about the origin. 
		 *
		 * \p PointsPerUnitArea specifies how many points to generate per square 
		 *    unit (cm). 0.5 would generate 1 point per 2 square cm.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the cylinder.
		 */
		TArray<FVec3> ComputeLocalSamplePoints(const FReal PointsPerUnitArea, const bool IncludeEndCaps = true, const int32 MinPoints = 0, const int32 MaxPoints = 1000) const
		{ return ComputeLocalSamplePoints(FMath::Clamp(static_cast<int32>(ceil(PointsPerUnitArea * GetArea(IncludeEndCaps))), MinPoints, MaxPoints), IncludeEndCaps); }

		/**
		 * Returns sample points at the current location of the cylinder.
		 *
		 * \p NumPoints specifies how many points to generate.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the cylinder.
		 */
		TArray<FVec3> ComputeSamplePoints(const int32 NumPoints, const bool IncludeEndCaps = true) const;

		/** 
		 * Returns sample points at the current location of the cylinder.
		 *
		 * \p PointsPerUnitArea specifies how many points to generate per square 
		 *    unit (cm). 0.5 would generate 1 point per 2 square cm.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the cylinder.
		 */
		TArray<FVec3> ComputeSamplePoints(const FReal PointsPerUnitArea, const bool IncludeEndCaps = true, const int32 MinPoints = 0, const int32 MaxPoints = 1000) const
		{ return ComputeSamplePoints(FMath::Clamp(static_cast<int32>(ceil(PointsPerUnitArea * GetArea(IncludeEndCaps))), MinPoints, MaxPoints), IncludeEndCaps); }

		virtual const FAABB3 BoundingBox() const override { return MLocalBoundingBox; }

		FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const
		{
			const FVec3f Xf(x);
			const FVec3f& Normal1 = MPlane1.Normal();
			const FRealSingle Distance1 = MPlane1.SignedDistance(Xf);
			if (Distance1 < UE_SMALL_NUMBER)
			{
				ensure(MPlane2.SignedDistance(Xf) > (FRealSingle)0.);
				const FVec3f v = x - FVec3f(Normal1 * Distance1 + MPlane1.X());
				if (v.Size() > MRadius1)
				{
					const FVec3f Corner = FVec3f(v.GetSafeNormal()) * MRadius1 + MPlane1.X();
					const FVec3f CornerVector = x - Corner;
					Normal = CornerVector.GetSafeNormal();
					return CornerVector.Size();
				}
				else
				{
					Normal = -Normal1;
					return -Distance1;
				}
			}
			const FVec3f& Normal2 = MPlane2.Normal();  // Used to be Distance2 = MPlane2.PhiWithNormal(x, Normal2); but that would trigger 
			const FRealSingle Distance2 = MHeight - Distance1;          // the ensure on Distance2 being slightly larger than MHeight in some border cases
			if (Distance2 < UE_SMALL_NUMBER)
			{
				const FVec3f v = Xf - FVec3f(Normal2 * Distance2 + MPlane2.X());
				if (v.Size() > MRadius2)
				{
					const FVec3f Corner = FVec3f(v.GetSafeNormal()) * MRadius2 + MPlane2.X();
					const FVec3f CornerVector = Xf - Corner;
					Normal = CornerVector.GetSafeNormal();
					return CornerVector.Size();
				}
				else
				{
					Normal = -Normal2;
					return -Distance2;
				}
			}
			ensure(Distance1 <= MHeight && Distance2 <= MHeight);
			const FVec3f SideVector = (Xf - FVec3f(Normal1 * Distance1 + MPlane1.X()));
			const FRealSingle SideDistance = SideVector.Size() - GetRadiusAtDistance(Distance1);
			if (SideDistance < 0.)
			{
				const FRealSingle TopDistance = Distance1 < Distance2 ? Distance1 : Distance2;
				if (TopDistance < -SideDistance)
				{
					Normal = Distance1 < Distance2 ? -Normal1 : -Normal2;
					return -TopDistance;
				}
			}
			Normal = SideVector.GetSafeNormal();
			return SideDistance;
		}

		Pair<FVec3, bool> FindClosestIntersection(const FVec3& StartPoint, const FVec3& EndPoint, const FReal Thickness)
		{
			TArray<Pair<FReal, FVec3>> Intersections;
			FReal DeltaRadius = FGenericPlatformMath::Abs(MRadius2 - MRadius1);
			if (DeltaRadius == 0)
				return FCylinder(MPlane1.X(), MPlane2.X(), MRadius1).FindClosestIntersection(StartPoint, EndPoint, Thickness);
			FVec3 BaseNormal;
			FReal BaseRadius;
			FVec3 BaseCenter;
			if (MRadius2 > MRadius1)
			{
				BaseNormal = MPlane2.Normal();
				BaseRadius = MRadius2 + Thickness;
				BaseCenter = MPlane2.X();
			}
			else
			{
				BaseNormal = MPlane1.Normal();
				BaseRadius = MRadius1 + Thickness;
				BaseCenter = MPlane1.X();
			}
			FVec3 Top = BaseRadius / DeltaRadius * MHeight * BaseNormal + BaseCenter;
			FReal theta = atan2(BaseRadius, (Top - BaseCenter).Size());
			FReal costheta = cos(theta);
			FReal cossqtheta = costheta * costheta;
			check(theta > 0 && theta < UE_PI / 2);
			FVec3 Direction = EndPoint - StartPoint;
			FReal Length = Direction.Size();
			Direction = Direction.GetSafeNormal();
			auto DDotN = FVec3::DotProduct(Direction, -BaseNormal);
			auto SMT = StartPoint - Top;
			auto SMTDotN = FVec3::DotProduct(SMT, -BaseNormal);
			FReal a = DDotN * DDotN - cossqtheta;
			FReal b = 2 * (DDotN * SMTDotN - FVec3::DotProduct(Direction, SMT) * cossqtheta);
			FReal c = SMTDotN * SMTDotN - SMT.SizeSquared() * cossqtheta;
			FReal Determinant = b * b - 4 * a * c;
			if (Determinant == 0)
			{
				FReal Root = -b / (2 * a);
				FVec3 RootPoint = Root * Direction + StartPoint;
				if (Root >= 0 && Root <= Length && FVec3::DotProduct(RootPoint - Top, -BaseNormal) >= 0)
				{
					Intersections.Add(MakePair(Root, RootPoint));
				}
			}
			if (Determinant > 0)
			{
				FReal Root1 = (-b - sqrt(Determinant)) / (2 * a);
				FReal Root2 = (-b + sqrt(Determinant)) / (2 * a);
				FVec3 Root1Point = Root1 * Direction + StartPoint;
				FVec3 Root2Point = Root2 * Direction + StartPoint;
				if (Root1 < 0 || Root1 > Length || FVec3::DotProduct(Root1Point - Top, -BaseNormal) < 0)
				{
					if (Root2 >= 0 && Root2 <= Length && FVec3::DotProduct(Root2Point - Top, -BaseNormal) >= 0)
					{
						Intersections.Add(MakePair(Root2, Root2Point));
					}
				}
				else if (Root2 < 0 || Root2 > Length || FVec3::DotProduct(Root2Point - Top, -BaseNormal) < 0)
				{
					Intersections.Add(MakePair(Root1, Root1Point));
				}
				else if (Root1 < Root2 && FVec3::DotProduct(Root1Point - Top, -BaseNormal) >= 0)
				{
					Intersections.Add(MakePair(Root1, Root1Point));
				}
				else if (FVec3::DotProduct(Root2Point - Top, -BaseNormal) >= 0)
				{
					Intersections.Add(MakePair(Root2, Root2Point));
				}
			}
			auto Plane1Intersection = MPlane1.FindClosestIntersection(StartPoint, EndPoint, Thickness);
			if (Plane1Intersection.Second)
				Intersections.Add(MakePair((FReal)(Plane1Intersection.First - StartPoint).Size(), Plane1Intersection.First));
			auto Plane2Intersection = MPlane2.FindClosestIntersection(StartPoint, EndPoint, Thickness);
			if (Plane2Intersection.Second)
				Intersections.Add(MakePair((FReal)(Plane2Intersection.First - StartPoint).Size(), Plane2Intersection.First));
			Intersections.Sort([](const Pair<FReal, FVec3>& Elem1, const Pair<FReal, FVec3>& Elem2) { return Elem1.First < Elem2.First; });
			for (const auto& Elem : Intersections)
			{
				if (SignedDistance(Elem.Second) <= (Thickness + 1e-4))
				{
					return MakePair(Elem.Second, true);
				}
			}
			return MakePair(FVec3(0), false);
		}

		FReal GetRadius1() const { return MRadius1; }
		FReal GetRadius2() const { return MRadius2; }
		FReal GetHeight() const { return MHeight; }
		FReal GetSlantHeight() const { const FReal R1mR2 = MRadius1-MRadius2; return FMath::Sqrt(R1mR2*R1mR2 + MHeight*MHeight); }
		FVec3 GetX1() const { return MPlane1.X(); }
		const FVec3f& GetX1f() const { return MPlane1.X(); }
		FVec3 GetX2() const { return MPlane2.X(); }
		const FVec3f& GetX2f() const { return MPlane2.X(); }
		/** Returns the bottommost point on the cylinder. */
		FVec3 GetOrigin() const { return MPlane1.X(); }
		const FVec3f& GetOriginf() const { return MPlane1.X(); }
		/** Returns the topmost point on the cylinder. */
		FVec3 GetInsertion() const { return MPlane2.X(); }
		const FVec3f& GetInsertionf() const { return MPlane2.X(); }
		FVec3 GetCenter() const { return (MPlane1.X() + MPlane2.X()) * (FReal)0.5; }
		/** Returns the centroid (center of mass). */
		FVec3 GetCenterOfMass() const // centroid
		{
			const FReal R1R1 = MRadius1 * MRadius1;
			const FReal R2R2 = MRadius2 * MRadius2;
			const FReal R1R2 = MRadius1 * MRadius2;
			return FVec3(0, 0, static_cast<FReal>(MHeight*(R1R1 + 2.*R1R2 + 3.*R2R2) / 4.*(R1R1 + R1R2 + R2R2)));
		}
		FVec3 GetAxis() const { return (MPlane2.X() - MPlane1.X()).GetSafeNormal(); }

		FReal GetArea(const bool IncludeEndCaps = true) const { return GetArea(MHeight, MRadius1, MRadius2, IncludeEndCaps); }
		static FReal GetArea(const FReal Height, const FReal Radius1, const FReal Radius2, const bool IncludeEndCaps)
		{
			static const FReal TwoPI = UE_PI * 2;
			if (Radius1 == Radius2)
			{
				const FReal TwoPIR1 = TwoPI * Radius1;
				return IncludeEndCaps ?
				    TwoPIR1 * Height + TwoPIR1 * Radius1 :
				    TwoPIR1 * Height;
			}
			else
			{
				const FReal R1_R2 = Radius1 - Radius2;
				const FReal CylArea = UE_PI * (Radius1 + Radius2) * FMath::Sqrt((R1_R2 * R1_R2) + (Height * Height));
				return IncludeEndCaps ?
				    CylArea + UE_PI * Radius1 * Radius1 + UE_PI * Radius2 * Radius2 :
				    CylArea;
			}
		}

		FReal GetVolume() const { return GetVolume(MHeight, MRadius1, MRadius2); }
		static FReal GetVolume(const FReal Height, const FReal Radius1, const FReal Radius2)
		{
			static const FReal PI_3 = UE_PI / 3;
			return PI_3 * Height * (Radius1 * Radius1 + Radius1 * Radius2 + Radius2 * Radius2);
		}

		FMatrix33 GetInertiaTensor(const FReal Mass) const { return GetInertiaTensor(Mass, MHeight, MRadius1, MRadius2); }
		static FMatrix33 GetInertiaTensor(const FReal Mass, const FReal Height, const FReal Radius1, const FReal Radius2)
		{
			// https://www.wolframalpha.com/input/?i=conical+frustum
			const FReal R1 = FMath::Min(Radius1, Radius2);
			const FReal R2 = FMath::Max(Radius1, Radius2);
			const FReal HH = Height * Height;
			const FReal R1R1 = R1 * R1;
			const FReal R1R2 = R1 * R2;
			const FReal R2R2 = R2 * R2;

			const FReal Num1 = static_cast<FReal>(2. * HH * (R1R1 + 3. * R1R2 + 6. * R2R2)); // 2H^2 * (R1^2 + 3R1R2 + 6R2^2)
			const FReal Num2 = static_cast<FReal>(3. * (R1R1 * R1R1 + R1R1 * R1R2 + R1R2 * R1R2 + R1R2 * R2R2 + R2R2 * R2R2)); // 3 * (R1^4 + R1^3R2 + R1^2R2^2 + R1R2^3 + R2^4)
			const FReal Den1 = UE_PI * (R1R1 + R1R2 + R2R2); // PI * (R1^2 + R1R2 + R2^2)

			const FReal Diag12 = Mass * (Num1 + Num2) / (static_cast<FReal>(20.) * Den1);
			const FReal Diag3 = Mass * Num2 / (static_cast<FReal>(10.) * Den1);

			return FMatrix33(Diag12, Diag12, Diag3);
		}

		FRotation3 GetRotationOfMass() const { return GetRotationOfMass(GetAxis()); }
		static FRotation3 GetRotationOfMass(const FVec3& Axis)
		{ 
			// since the cylinder stores an axis and the InertiaTensor is assumed to be along the ZAxis
			// we need to make sure to return the rotation of the axis from Z
			return FRotation3::FromRotatedVector(FVec3(0, 0, 1), Axis);
		}

		virtual uint32 GetTypeHash() const override
		{
			const uint32 PlaneHashes = HashCombine(MPlane1.GetTypeHash(), MPlane2.GetTypeHash());
			const uint32 PropertyHash = HashCombine(::GetTypeHash(MHeight), HashCombine(::GetTypeHash(MRadius1), ::GetTypeHash(MRadius2)));

			return HashCombine(PlaneHashes, PropertyHash);
		}

#if INTEL_ISPC
		// See PerParticlePBDCollisionConstraint.cpp
		// ISPC code has matching structs for interpreting FImplicitObjects.
		// This is used to verify that the structs stay the same.
		struct FISPCDataVerifier
		{
			static constexpr int32 OffsetOfMPlane1() { return offsetof(FTaperedCylinder, MPlane1); }
			static constexpr int32 SizeOfMPlane1() { return sizeof(FTaperedCylinder::MPlane1); }
			static constexpr int32 OffsetOfMPlane2() { return offsetof(FTaperedCylinder, MPlane2); }
			static constexpr int32 SizeOfMPlane2() { return sizeof(FTaperedCylinder::MPlane2); }
			static constexpr int32 OffsetOfMHeight() { return offsetof(FTaperedCylinder, MHeight); }
			static constexpr int32 SizeOfMHeight() { return sizeof(FTaperedCylinder::MHeight); }
			static constexpr int32 OffsetOfMRadius1() { return offsetof(FTaperedCylinder, MRadius1); }
			static constexpr int32 SizeOfMRadius1() { return sizeof(FTaperedCylinder::MRadius1); }
			static constexpr int32 OffsetOfMRadius2() { return offsetof(FTaperedCylinder, MRadius2); }
			static constexpr int32 SizeOfMRadius2() { return sizeof(FTaperedCylinder::MRadius2); }
		};
		friend FISPCDataVerifier;
#endif // #if INTEL_ISPC

	private:
		//Phi is distance from closest point on plane1
		FRealSingle GetRadiusAtDistance(const FRealSingle& Phi) const
		{
			const FRealSingle Alpha = Phi / MHeight;
			return MRadius1 * (static_cast<FRealSingle>(1.) - Alpha) + MRadius2 * Alpha;
		}

		TPlaneConcrete<FRealSingle, 3> MPlane1, MPlane2;
		FRealSingle MHeight, MRadius1, MRadius2;
		FAABB3 MLocalBoundingBox;
	};

	struct FTaperedCylinderSpecializeSamplingHelper
	{
		static FORCEINLINE void ComputeSamplePoints(
		    TArray<FVec3>& Points, const FTaperedCylinder& Cylinder,
		    const int32 NumPoints, const bool IncludeEndCaps = true)
		{
			if (NumPoints <= 1 ||
			    (Cylinder.GetRadius1() <= UE_KINDA_SMALL_NUMBER &&
			        Cylinder.GetRadius2() <= UE_KINDA_SMALL_NUMBER))
			{
				const int32 Offset = Points.Num();
				if (Cylinder.GetHeight() <= UE_KINDA_SMALL_NUMBER)
				{
					Points.SetNumUninitialized(Offset + 1);
					Points[Offset] = Cylinder.GetCenter();
				}
				else
				{
					Points.SetNumUninitialized(Offset + 3);
					Points[Offset + 0] = Cylinder.GetOrigin();
					Points[Offset + 1] = Cylinder.GetCenter();
					Points[Offset + 2] = Cylinder.GetInsertion();
				}
				return;
			}
			ComputeGoldenSpiralPoints(Points, Cylinder, NumPoints, IncludeEndCaps);
		}

		static FORCEINLINE void ComputeGoldenSpiralPoints(TArray<FVec3>& Points, const FTaperedCylinder& Cylinder, const int32 NumPoints, const bool IncludeEndCaps = true)
		{
			ComputeGoldenSpiralPoints(Points, Cylinder.GetOrigin(), Cylinder.GetAxis(), Cylinder.GetRadius1(), Cylinder.GetRadius2(), Cylinder.GetHeight(), NumPoints, IncludeEndCaps);
		}

		/**
		 * Use the golden spiral method to generate evenly spaced points on a tapered 
		 * cylinder (truncated cone).
		 *
		 * The "golden" part is derived from the golden ratio; stand at the center,
		 * turn a golden ratio of whole turns, then emit a point in that direction.
		 *
		 * Points are generated starting from the bottom of the cylinder, ending at 
		 * the top.  Contiguous entries in \p Points generally will not be spatially
		 * adjacent.
		 *
		 * \p Points to append to.
		 * \p Origin is the bottom-most point of the tapered cylinder.
		 * \p Axis is the orientation of the tapered cylinder.
		 * \p Radius1 is the first radius of the tapered cylinder.
		 * \p Radius2 is the second radius of the tapered cylinder.
		 * \p Height is the height of the tapered cylinder.
		 * \p NumPoints is the number of points to generate.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the tapered cylinder.
		 * \p SpiralSeed is the starting index for golden spiral generation.  When 
		 *    using this method to continue a spiral started elsewhere, \p SpiralSeed 
		 *    should equal the number of particles already created.
		 */
		static /*FORCEINLINE*/ void ComputeGoldenSpiralPoints(
		    TArray<FVec3>& Points,
		    const FVec3& Origin,
		    const FVec3& Axis,
		    const FReal Radius1,
		    const FReal Radius2,
		    const FReal Height,
		    const int32 NumPoints,
		    const bool IncludeEndCaps = true,
		    int32 SpiralSeed = 0)
		{
			// Axis should be normalized.
			checkSlow(FMath::Abs(Axis.Size() - 1.0) < UE_KINDA_SMALL_NUMBER);

			const int32 Offset = Points.Num();
			ComputeGoldenSpiralPointsUnoriented(Points, Radius1, Radius2, Height, NumPoints, IncludeEndCaps, SpiralSeed);

			// At this point, Points are centered about the origin (0,0,0), built
			// along the Z axis.  Transform them to where they should be.
			const FReal HalfHeight = Height / 2;
			const FRotation3 Rotation = FRotation3::FromRotatedVector(FVec3(0, 0, 1), Axis);
			checkSlow(((Origin + Axis * Height) - (Rotation.RotateVector(FVec3(0, 0, Height)) + Origin)).Size() < UE_KINDA_SMALL_NUMBER);
			for (int32 i = Offset; i < Points.Num(); i++)
			{
				FVec3& Point = Points[i];
				const FVec3 PointNew = Rotation.RotateVector(Point + FVec3(0, 0, HalfHeight)) + Origin;
//				checkSlow(FMath::Abs(FTaperedCylinder(Origin, Origin + Axis * Height, Radius1, Radius2).SignedDistance(PointNew)) < KINDA_SMALL_NUMBER);
				Point = PointNew;
			}
		}

		/**
		 * Generates points on a tapered cylinder (truncated cone), oriented about 
		 * the Z axis, varying from [-Height/2, Height/2].
		 *
		 * TODO: Note that this method does not produce evenly spaced points!  It'll 
		 * bunch points together on the side of the cylinder with the smaller radius, 
		 * and spread them apart on the larger.  We need a routine that operates in 
		 * conical space, rather than cylindrical.  That said, points are distributed 
		 * evenly between the two end caps, proportional to their respective areas.
		 *
		 * The "golden" part is derived from the golden ratio; stand at the center,
		 * turn a golden ratio of whole turns, then emit a point in that direction.
		 *
		 * Points are generated starting from the bottom of the cylinder, ending at 
		 * the top.  Contiguous entries in \p Points generally will not be spatially
		 * adjacent.
		 *
		 * \p Points to append to.
		 * \p Radius1 is the first radius of the tapered cylinder.
		 * \p Radius2 is the second radius of the tapered cylinder.
		 * \p Height is the height of the cylinder.
		 * \p NumPoints is the number of points to generate.
		 * \p IncludeEndCaps determines whether or not points are generated on the 
		 *    end caps of the cylinder.
		 * \p SpiralSeed is the starting index for golden spiral generation.  When 
		 *    using this method to continue a spiral started elsewhere, \p SpiralSeed 
		 *    should equal the number of particles already created.
		 */
		static /*FORCEINLINE*/ void ComputeGoldenSpiralPointsUnoriented(
		    TArray<FVec3>& Points,
		    const FReal Radius1,
		    const FReal Radius2,
		    const FReal Height,
		    const int32 NumPoints,
		    const bool IncludeEndCaps = true,
		    int32 SpiralSeed = 0)
		{
			// Evenly distribute points between the cylinder body and the end caps.
			int32 NumPointsEndCap1;
			int32 NumPointsEndCap2;
			int32 NumPointsCylinder;
			if (IncludeEndCaps)
			{
				const FReal Cap1Area = UE_PI * Radius1 * Radius1;
				const FReal Cap2Area = UE_PI * Radius2 * Radius2;
				const FReal CylArea =
				    UE_PI * Radius2 * (Radius2 + FMath::Sqrt(Height * Height + Radius2 * Radius2)) -
				    UE_PI * Radius1 * (Radius1 + FMath::Sqrt(Height * Height + Radius1 * Radius1));
				const FReal AllArea = CylArea + Cap1Area + Cap2Area;
				if (AllArea > UE_KINDA_SMALL_NUMBER)
				{
					NumPointsEndCap1 = static_cast<int32>(round(Cap1Area / AllArea * static_cast<FReal>(NumPoints)));
					NumPointsEndCap2 = static_cast<int32>(round(Cap2Area / AllArea * static_cast<FReal>(NumPoints)));
					NumPointsCylinder = NumPoints - NumPointsEndCap1 - NumPointsEndCap2;
				}
				else
				{
					NumPointsCylinder = 0;
					NumPointsEndCap1 = NumPointsEndCap2 = (NumPoints - (NumPoints % 2)) / 2;
				}
			}
			else
			{
				NumPointsCylinder = NumPoints;
				NumPointsEndCap1 = 0;
				NumPointsEndCap2 = 0;
			}
			const int32 NumPointsToAdd = NumPointsCylinder + NumPointsEndCap1 + NumPointsEndCap2;
			Points.Reserve(Points.Num() + NumPointsToAdd);

			int32 Offset = Points.Num();
			const FReal HalfHeight = Height / 2;
			TArray<FVec2> Points2D;
			Points2D.Reserve(NumPointsEndCap1);
			if (IncludeEndCaps)
			{
				TSphereSpecializeSamplingHelper<FReal, 2>::ComputeGoldenSpiralPoints(
				    Points2D, FVec2((FReal)0.0), Radius1, NumPointsEndCap1, SpiralSeed);
				Offset = Points.AddUninitialized(Points2D.Num());
				for (int32 i = 0; i < Points2D.Num(); i++)
				{
					const FVec2& Pt = Points2D[i];
					checkSlow(Pt.Size() < Radius1 + UE_KINDA_SMALL_NUMBER);
					Points[i + Offset] = FVec3(Pt[0], Pt[1], -HalfHeight);
				}
				// Advance the SpiralSeed by the number of points generated.
				SpiralSeed += Points2D.Num();
			}

			Offset = Points.AddUninitialized(NumPointsCylinder);
			if (NumPointsCylinder == 1)
			{
				Points[Offset] = FVec3(0, 0, HalfHeight);
			}
			else
			{
				static const FRealSingle Increment = UE_PI * (1.0f + FMath::Sqrt(5.0f));
				for (int32 i = 0; i < NumPointsCylinder; i++)
				{
					// In the 2D sphere (disc) case, we vary R so it increases monotonically,
					// which spreads points out across the disc:
					//     const FReal R = FMath::Sqrt((0.5 + Index) / NumPoints) * Radius;
					// But we're mapping to a cylinder, which means we want to keep R constant.
					const FReal R = FMath::Lerp(Radius1, Radius2, static_cast<FReal>(i) / static_cast<FReal>(NumPointsCylinder - 1));
					const FReal Theta = Increment * (0.5f + static_cast<FReal>(i + SpiralSeed));

					// Map polar coordinates to Cartesian, and vary Z by [-HalfHeight, HalfHeight].
					const FReal Z = FMath::LerpStable(-HalfHeight, HalfHeight, static_cast<FReal>(i) / static_cast<FReal>(NumPointsCylinder - 1));
					Points[i + Offset] =
					    FVec3(
					        R * FMath::Cos(Theta),
					        R * FMath::Sin(Theta),
					        Z);

					//checkSlow(FMath::Abs(FVec2(Points[i + Offset][0], Points[i + Offset][1]).Size() - Radius) < KINDA_SMALL_NUMBER);
				}
			}
			// Advance the SpiralSeed by the number of points generated.
			SpiralSeed += NumPointsCylinder;

			if (IncludeEndCaps)
			{
				Points2D.Reset();
				TSphereSpecializeSamplingHelper<FReal, 2>::ComputeGoldenSpiralPoints(
				    Points2D, FVec2((FReal)0.0), Radius2, NumPointsEndCap2, SpiralSeed);
				Offset = Points.AddUninitialized(Points2D.Num());
				for (int32 i = 0; i < Points2D.Num(); i++)
				{
					const FVec2& Pt = Points2D[i];
					checkSlow(Pt.Size() < Radius2 + UE_KINDA_SMALL_NUMBER);
					Points[i + Offset] = FVec3(Pt[0], Pt[1], HalfHeight);
				}
			}
		}
	};

	FORCEINLINE TArray<FVec3> FTaperedCylinder::ComputeLocalSamplePoints(const int32 NumPoints, const bool IncludeEndCaps) const
	{
		TArray<FVec3> Points;
		const FVec3 Mid = GetCenter();
		FTaperedCylinderSpecializeSamplingHelper::ComputeSamplePoints(
			Points,
			FTaperedCylinder(MPlane1.X() - Mid, MPlane2.X() - Mid, GetRadius1(), GetRadius2()),
			NumPoints, IncludeEndCaps);
		return Points;
	}

	FORCEINLINE TArray<FVec3> FTaperedCylinder::ComputeSamplePoints(const int32 NumPoints, const bool IncludeEndCaps) const
	{
		TArray<FVec3> Points;
		FTaperedCylinderSpecializeSamplingHelper::ComputeSamplePoints(Points, *this, NumPoints, IncludeEndCaps);
		return Points;
	}

	template<class T>
	using TTaperedCylinder UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FTaperedCylinder instead") = FTaperedCylinder;

	template<typename T>
	using TTaperedCylinderSpecializeSamplingHelper UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FTaperedCylinderSpecializeSamplingHelper instead") = FTaperedCylinderSpecializeSamplingHelper;


} // namespace Chaos
