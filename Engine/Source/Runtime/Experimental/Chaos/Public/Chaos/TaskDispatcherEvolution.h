// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDRigidsSOAs.h"
#include "Chaos/PendingSpatialData.h"
#include "HAL/Event.h"

namespace Chaos::Private
{

// This class is responsible to dispatch the evolution tasks
// It orchestrates the task dependencies according to data flow.
// This class shouldn't do any simulation logic and code.
class FTaskDispatcherEvolution
{
private:
	constexpr static int32 MinParticlePerTask = 40;

	UE::Tasks::FTask AsyncQueueDynTask;
	UE::Tasks::FTask AsyncQueueKinTask;
	UE::Tasks::FTask FlushAsyncQueueTask;
	UE::Tasks::FTask UpdateViewTask;

	TArray<UE::Tasks::FTask> IntegrationPendingTasks;
	TArray<UE::Tasks::FTask> KinematicPendingTasks;

	FPBDRigidsSOAs& Particles;
	FPendingSpatialInternalDataQueue& InternalAccelerationQueue;

	int32 NumTasks;
	int32 NumDynParticles;
	int32 NumKinBatches;

public:
	FTaskDispatcherEvolution(FPBDRigidsSOAs& ParticlesIn, FPendingSpatialInternalDataQueue& InternalAccelerationQueueIn)
		: Particles(ParticlesIn)
		, InternalAccelerationQueue(InternalAccelerationQueueIn)
		, NumTasks(FMath::Max(FMath::Min(FTaskGraphInterface::Get().GetNumWorkerThreads(), Chaos::MaxNumWorkers), 1))
		, NumDynParticles()
		, NumKinBatches()
	{
	}

	void ComputeKinematicBatch()
	{
		// Compute the number of kinematic particles to allocate the InternalAccelerationQueue pending data
		const TParticleView<FPBDRigidParticles>& KienamticParticles = Particles.GetActiveMovingKinematicParticlesView();
		const int32 NumView = KienamticParticles.SOAViews.Num();
		NumKinBatches = 0;
		for (int32 ViewIndex = 0; ViewIndex < NumView; ++ViewIndex)
		{
			const int32 NumKinParticles = KienamticParticles.SOAViews[ViewIndex].Size();
			const int32 ParticleByTask = FMath::Max(FMath::DivideAndRoundUp(NumKinParticles, FMath::Max(NumTasks, 1)), MinParticlePerTask);
			NumKinBatches += FMath::DivideAndRoundUp(NumKinParticles, ParticleByTask);
		}
	}

	template<typename Lambda>
	void DispatchIntegrate(Lambda IntegrateWork)
	{
		NumDynParticles = Particles.GetActiveParticlesArray().Num();

		int32 NumDynTask = FMath::Max(FMath::Min(NumTasks, NumDynParticles), 1);

		const int32 ParticleByTask = FMath::Max(FMath::DivideAndRoundUp(NumDynParticles, NumDynTask), MinParticlePerTask);
		int32 NumDynBatches = FMath::DivideAndRoundUp(NumDynParticles, ParticleByTask);

		IntegrationPendingTasks.Reset(NumDynBatches);
		InternalAccelerationQueue.KinematicBatchStartIndex = NumDynBatches + 1;
		ComputeKinematicBatch();
		InternalAccelerationQueue.PendingDataArrays.SetNum(InternalAccelerationQueue.KinematicBatchStartIndex + NumKinBatches); // Keep 0 for the dirty element from PushData
		for (int32 BatchIndex = 0; BatchIndex < NumDynBatches; BatchIndex++)
		{
			const int32 StartIndex = BatchIndex * ParticleByTask;
			int32 EndIndex = (BatchIndex + 1) * ParticleByTask;
			EndIndex = FMath::Min(NumDynParticles, EndIndex);

			UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, IntegrateWork, BatchIndex, StartIndex, EndIndex]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_IntegrateTask);
				IntegrateWork(BatchIndex, StartIndex, EndIndex);
			}, LowLevelTasks::ETaskPriority::High);
			IntegrationPendingTasks.Add(PendingTask);
		}
	}

	template<typename Lambda>
	void DispatchDynAsyncDirty(Lambda AsyncDirtyWork)
	{
		// Why do we add the dynamic particle to the async queue which is suppose to be for static tree ? 
		AsyncQueueDynTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [AsyncDirtyWork]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_DirtyAsyncDynamicParticleTask);
			AsyncDirtyWork();
		}, LowLevelTasks::ETaskPriority::High);
	}

	template<typename Lambda>
	void DispatchKinAsyncDirtyAndUpdateKinematic(Lambda AsyncDirty, bool bIsLastStep)
	{
		// To start dirtying Kinematics particles, the Dynamics particles must be finished
		// Is it necessary to update the kinematic particles for the async data ? 
		AsyncQueueKinTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [AsyncDirty]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_DirtyMovingKinematicsTask);
			AsyncDirty();
		}, AsyncQueueDynTask, LowLevelTasks::ETaskPriority::High);

		TArray<UE::Tasks::FTask> PendingTasks;
		PendingTasks.Add(AsyncQueueKinTask);

		UE::Tasks::FTask MovingKinematicsTask;
		// done with update, let's clear the tracking structures
		if (bIsLastStep)
		{
			PendingTasks.Append(KinematicPendingTasks);
			MovingKinematicsTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateAllMovingKinematicTask);
				constexpr bool bUpdateView = false;
				Particles.UpdateAllMovingKinematic(bUpdateView);
			}, PendingTasks, LowLevelTasks::ETaskPriority::High);

		}
		else
		{
			MovingKinematicsTask = AsyncQueueKinTask;
		}

		// Updating the views can be done only when all kinematics and dynamics update are finished.
		UpdateViewTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_UpdateViewTask);
			// If we changed any particle state, the views need to be refreshed
			Particles.UpdateDirtyViews();

		}, MovingKinematicsTask, LowLevelTasks::ETaskPriority::High);
	}

	template<typename Lambda>
	void DispatchKinematicsTarget(Lambda KinematicTargetsWork, int32 NumParticles, int32 DispatchBatchIndex)
	{
		int32 NumKinTask = FMath::Max(FMath::Min(NumTasks, NumParticles), 1);
		const int32 ParticleByTask = FMath::Max(FMath::DivideAndRoundUp(NumParticles, NumKinTask), MinParticlePerTask);
		int32 NumBatches = FMath::DivideAndRoundUp(NumParticles, ParticleByTask);

		check(NumBatches < InternalAccelerationQueue.PendingDataArrays.Num() + 1);
		check(NumBatches > 0)
		// We only write to particle state
		for (int32 BatchIndex = 0; BatchIndex < NumBatches; BatchIndex++)
		{
			const int32 StartIndex = BatchIndex * ParticleByTask;
			int32 EndIndex = (BatchIndex + 1) * ParticleByTask;
			EndIndex = FMath::Min(NumParticles, EndIndex);

			UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [KinematicTargetsWork, StartIndex, EndIndex, DispatchBatchIndex, BatchIndex]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_KinematicTargetsTask);
				KinematicTargetsWork(StartIndex, EndIndex, DispatchBatchIndex + BatchIndex);
			}, LowLevelTasks::ETaskPriority::High);
			KinematicPendingTasks.Add(PendingTask);
		}
	}

	template <typename Lambda>
	void PruneInternalPendingData(Lambda HasToBeUpdated)
	{
		const int32 NumBatchDyn = FMath::Min(InternalAccelerationQueue.KinematicBatchStartIndex, InternalAccelerationQueue.PendingDataArrays.Num()) - 1;
		check(IntegrationPendingTasks.Num() == 0 || NumBatchDyn == IntegrationPendingTasks.Num());

		TArray<UE::Tasks::FTask> CurrentPendingTasks;
		CurrentPendingTasks.Reserve(InternalAccelerationQueue.PendingDataArrays.Num());

		TArray<TArray<FPendingSpatialData>> NewDynPendingDataArray;
		TArray<TArray<FPendingSpatialData>> NewKinPendingDataArray;

		UE::Tasks::FTask CleanUpTask;
		int32 NumActiveParticleTask = IntegrationPendingTasks.Num() + KinematicPendingTasks.Num();
		if (NumActiveParticleTask > 0)
		{
			// Moving particles could be potentially duplicated in batch 0 (inside InternalAccelerationQueue.PendingDataArrays), 
			// before processing batch 0, lets remove all duplicated particles in that batch.
			TArray<UE::Tasks::FTask> MovingParticlesTasks;
			MovingParticlesTasks.Reserve(NumActiveParticleTask);
			MovingParticlesTasks.Append(IntegrationPendingTasks);
			MovingParticlesTasks.Append(KinematicPendingTasks);
			CleanUpTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_CleanUpDuplicated);
				InternalAccelerationQueue.CleanUpDuplicated();
			}, MovingParticlesTasks, LowLevelTasks::ETaskPriority::High);

			CurrentPendingTasks.Add(CleanUpTask);
		}

		// Prune Dynamics particles
		// If IntegrationPendingTasks.Num() == 0, it means a post integrate callback was called, then we waited for the tasks to finish and then cleared them.
		// In this case we still do the pruning but without waiting for the integrate tasks, because it is already done.
		if (NumBatchDyn > 0 && (IntegrationPendingTasks.Num() == NumBatchDyn || IntegrationPendingTasks.IsEmpty()))
		{
			check(IntegrationPendingTasks.Num() == NumBatchDyn || IntegrationPendingTasks.IsEmpty());
			check(NumBatchDyn <= InternalAccelerationQueue.PendingDataArrays.Num());

			const bool bLinkIntegrateTask = !IntegrationPendingTasks.IsEmpty();

			NewDynPendingDataArray.SetNum(NumBatchDyn);
			TArray<UE::Tasks::FTask> DynamicTasks;
			for (int32 BatchIndex = 0; BatchIndex < NumBatchDyn; BatchIndex++)
			{
				// Dynamic particles start at index 1
				TArray<FPendingSpatialData>& OldPendingData = InternalAccelerationQueue.PendingDataArrays[BatchIndex + 1];
				TArray<FPendingSpatialData>& NewPendingData = NewDynPendingDataArray[BatchIndex];
				UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [HasToBeUpdated, &OldPendingData, &NewPendingData]()
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_PruneDynamicsPendingDataTask);
					for (const FPendingSpatialData& PendingData : OldPendingData)
					{
						if (HasToBeUpdated(PendingData))
						{
							NewPendingData.Add(PendingData);
						}
					}
					}, bLinkIntegrateTask ? IntegrationPendingTasks[BatchIndex] : UE::Tasks::FTask{}, LowLevelTasks::ETaskPriority::High);
				DynamicTasks.Add(PendingTask);
			}
			// In order to safely copy all all pruned data, we need to be sure 
			DynamicTasks.Add(CleanUpTask);
			UE::Tasks::FTask CopyDynamicTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &NewDynPendingDataArray, NumBatchDyn]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_CopyPrunedDynamicPendingDataTask);
				for (int32 BatchIndex = 0; BatchIndex < NumBatchDyn; BatchIndex++)
				{
					// Dynamic particles start at index 1
					TArray<FPendingSpatialData>& OldPendingData = InternalAccelerationQueue.PendingDataArrays[BatchIndex + 1];
					TArray<FPendingSpatialData>& NewPendingData = NewDynPendingDataArray[BatchIndex];
					OldPendingData = MoveTemp(NewPendingData);
				}
			}, DynamicTasks, LowLevelTasks::ETaskPriority::High);
			CurrentPendingTasks.Add(CopyDynamicTask);
		}
		else if (NumBatchDyn > 0)
		{
			UE_LOGF(LogChaos, Verbose, "No pruning happened in the Spatial Acceleration structure, a slight perfromance hit could occured. IntegrationPendingTasks.Num(): %d !=  NumBatchDyn:  %d", IntegrationPendingTasks.Num(), NumBatchDyn);
			CurrentPendingTasks.Append(IntegrationPendingTasks);
		}

		int32 NumBatchKin = InternalAccelerationQueue.PendingDataArrays.Num() - InternalAccelerationQueue.KinematicBatchStartIndex;
		// Prune Kinematics particles
		// If KinematicPendingTasks.Num() == 0, it means a post integrate callback was called, then we waited for the tasks to finish and then cleared them.
		// In this case we still do the pruning but without waiting for the kinematic update tasks
		if (NumBatchKin > 0 && InternalAccelerationQueue.KinematicBatchStartIndex > 0 && (KinematicPendingTasks.Num() == NumBatchKin || KinematicPendingTasks.IsEmpty()))
		{
			check(KinematicPendingTasks.Num() == NumBatchKin || KinematicPendingTasks.Num() == 0);
			
			const bool bHasToLinkTask = !KinematicPendingTasks.IsEmpty();
			
			NewKinPendingDataArray.SetNum(NumBatchKin);
			TArray<UE::Tasks::FTask> KinematicTasks;
			for (int32 BatchIndex = 0; BatchIndex < NumBatchKin; BatchIndex++)
			{
				TArray<FPendingSpatialData>& OldPendingData = InternalAccelerationQueue.PendingDataArrays[InternalAccelerationQueue.KinematicBatchStartIndex + BatchIndex];
				TArray<FPendingSpatialData>& NewPendingData = NewKinPendingDataArray[BatchIndex];
				UE::Tasks::FTask PendingTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &OldPendingData, &NewPendingData, HasToBeUpdated]()
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_PruneKinematicPendingDataTask);
					for (const FPendingSpatialData& PendingData : OldPendingData)
					{
						if (HasToBeUpdated(PendingData))
						{
							NewPendingData.Add(PendingData);
						}
					}
				}, bHasToLinkTask? KinematicPendingTasks[BatchIndex]: UE::Tasks::FTask{}, LowLevelTasks::ETaskPriority::High);
				KinematicTasks.Add(PendingTask);
			}
			
			KinematicTasks.Add(CleanUpTask);
			UE::Tasks::FTask CopyKinematicTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, &NewKinPendingDataArray, NumBatchKin]()
			{
				QUICK_SCOPE_CYCLE_COUNTER(STAT_CopyPrunedKinematicPendingDataTask);
				for (int32 BatchIndex = 0; BatchIndex < NumBatchKin; BatchIndex++)
				{
					TArray<FPendingSpatialData>& OldPendingData = InternalAccelerationQueue.PendingDataArrays[InternalAccelerationQueue.KinematicBatchStartIndex + BatchIndex];
					TArray<FPendingSpatialData>& NewPendingData = NewKinPendingDataArray[BatchIndex];
					OldPendingData = MoveTemp(NewPendingData);
				}
			}, KinematicTasks, LowLevelTasks::ETaskPriority::High);
			CurrentPendingTasks.Add(CopyKinematicTask);
		}
		else if (NumBatchKin > 0 && InternalAccelerationQueue.KinematicBatchStartIndex > 0)
		{
			// In this case, no post-integrate was called, but somehow NumBatchKin doesn't equal KinematicPendingTasks.Num()
			UE_LOGF(LogChaos, Verbose, "No pruning happened in the Spatial Acceleration structure, a slight perfromance hit could occured. KinematicPendingTasks.Num(): %d !=  NumBatchKin:  %d", KinematicPendingTasks.Num(), NumBatchKin);
			CurrentPendingTasks.Append(KinematicPendingTasks);
		}
		UE::Tasks::Wait(CurrentPendingTasks);
		IntegrationPendingTasks.Reset();
		KinematicPendingTasks.Reset();
	}

	template<typename Lambda>
	void FlushAccelerationQueue(Lambda FlushAccelerationQueueLambda)
	{
		TArray<UE::Tasks::FTask> PendingTasks{AsyncQueueDynTask, AsyncQueueKinTask};
		PendingTasks.Append(KinematicPendingTasks);
		PendingTasks.Append(IntegrationPendingTasks);
		FlushAsyncQueueTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [FlushAccelerationQueueLambda]()
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FlushAsyncAccelerationQueueTask);
			FlushAccelerationQueueLambda();
		}, PendingTasks, LowLevelTasks::ETaskPriority::High);
	}

	// This will be called if it is required to wait tasks result to invoke a callback.
	// We never know what a callback is going to use.
	void WaitIntegrationComplete()
	{
		TArray<UE::Tasks::FTask> Tasks = { UpdateViewTask };
		Tasks.Reserve(IntegrationPendingTasks.Num() + KinematicPendingTasks.Num() + 1);
		Tasks.Append(IntegrationPendingTasks);
		Tasks.Append(KinematicPendingTasks);
		UE::Tasks::Wait(Tasks);
		UpdateViewTask = UE::Tasks::FTask();
		IntegrationPendingTasks.Reset();
		KinematicPendingTasks.Reset();
	}

	void WaitAsyncQueueTask()
	{
		UE::Tasks::Wait(FlushAsyncQueueTask);
	}

	void WaitTaskEndSpatial()
	{
		UE::Tasks::Wait(UpdateViewTask);
	}
};
}
