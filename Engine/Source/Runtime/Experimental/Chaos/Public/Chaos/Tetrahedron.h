// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/AABB.h"
#include "Chaos/Core.h"
#include "Chaos/Triangle.h"
#include "Chaos/Plane.h"
#include "Chaos/Math/Poisson.h" // 3x3 row major matrix functions
#include "GenericPlatform/GenericPlatformMath.h"
#include <limits>


namespace Chaos {

	template <typename T>
	class TTetrahedron
	{
	public:
		TTetrahedron()
		{
		}

		TTetrahedron(const TVec3<T>& In1, const TVec3<T>& In2, const TVec3<T>& In3, const TVec3<T>& In4)
			: X{ In1, In2, In3, In4 }
		{
		}

		FORCEINLINE TVec3<T>& operator[](uint32 InIndex)
		{
			checkSlow(InIndex < 4);
			return X[InIndex];
		}

		FORCEINLINE const TVec3<T>& operator[](uint32 InIndex) const
		{
			checkSlow(InIndex < 4);
			return X[InIndex];
		}

		void Invert()
		{
			Swap(X[0], X[1]); 
		}

		TVec3<T> GetCenter() const
		{
			return static_cast<T>(.25) * (X[0] + X[1] + X[2] + X[3]);
		}

		bool HasBoundingBox() const { return true; }
		TAABB<T, 3> BoundingBox() const { return GetBoundingBox(); }
		TAABB<T, 3> GetBoundingBox() const
		{
			TAABB<T, 3> Box = TAABB<T, 3>::EmptyAABB();
			for (int32 i = 0; i < 4; i++)
			{
				Box.GrowToInclude(X[i]);
			}
			return Box;
		}

		T GetMinEdgeLengthSquared() const
		{
			return FMath::Min(
				FMath::Min3((X[0] - X[1]).SizeSquared(), (X[1] - X[2]).SizeSquared(), (X[2] - X[0]).SizeSquared()), 
				FMath::Min3((X[0] - X[3]).SizeSquared(), (X[1] - X[3]).SizeSquared(), (X[2] - X[3]).SizeSquared()));
		}

		T GetMinEdgeLength() const
		{
			return FMath::Sqrt(GetMinEdgeLengthSquared());
		}

		T GetMaxEdgeLengthSquared() const
		{
			return FMath::Max(
				FMath::Max3((X[0] - X[1]).SizeSquared(), (X[1] - X[2]).SizeSquared(), (X[2] - X[0]).SizeSquared()),
				FMath::Max3((X[0] - X[3]).SizeSquared(), (X[1] - X[3]).SizeSquared(), (X[2] - X[3]).SizeSquared()));
		}

		T GetMaxEdgeLength() const
		{
			return FMath::Sqrt(GetMaxEdgeLengthSquared());
		}

		T GetVolume() const
		{
			return fabs(GetSignedVolume());
		}

		T GetSignedVolume() const
		{
			TVec3<T> U = X[1] - X[0];
			TVec3<T> V = X[2] - X[0];
			TVec3<T> W = X[3] - X[0];
			return TripleProduct(U, V, W) / 6;
		}

		T GetMinimumAltitude(int32* MinAltitudeVertex=nullptr) const
		{
			TArray<TTriangle<T>> Tris = GetTriangles();
			TVec4<T> Distance(0, 0, 0, 0);
			for (int32 i = 0; i < 4; i++)
			{
				const TTriangle<T>& Tri = Tris[i];
				TVec3<T> Norm;
				Distance[i] = -Tri.PhiWithNormal(X[3 - i], Norm);
			}
			if (MinAltitudeVertex)
			{
				(*MinAltitudeVertex) = 0;
			}
			T MinAltitude = Distance[0];
			for (int32 i = 0; i < 4; i++)
			{
				if (Distance[i] < MinAltitude)
				{
					MinAltitude = Distance[i];
					if (MinAltitudeVertex)
					{
						(*MinAltitudeVertex) = i;
					}
				}
			}
			if (MinAltitudeVertex)
			{
				(*MinAltitudeVertex) = 4 - (*MinAltitudeVertex);
			}
			return MinAltitude;
		}

		T GetAspectRatio() const
		{
			return GetMaxEdgeLength() / GetMinimumAltitude();
		}

		TVec3<T> GetFirstThreeBarycentricCoordinates(const TVec3<T>& Location) const 
		{
			TVec3<T> L1 = X[0] - X[3];
			TVec3<T> L2 = X[1] - X[3];
			TVec3<T> L3 = X[2] - X[3];
			TVec3<T> RHS = Location - X[3];
			T Matrix[9] = 
			{ 
				L1[0], L2[0], L3[0], 
				L1[1], L2[1], L3[1],
				L1[2], L2[2], L3[2]
			};
			return RowMaj3x3RobustSolveLinearSystem(&Matrix[0], RHS);
		}

		TVec4<T> GetBarycentricCoordinates(const TVec3<T>& Location) const 
		{
			TVec3<T> W = GetFirstThreeBarycentricCoordinates(Location);
			return TVec4<T>(W[0], W[1], W[2], static_cast<T>(1.0) - W[0] - W[1] - W[2]);
		}

		TVec3<T> GetPointFromBarycentricCoordinates(const TVec3<T>& Weights) const
		{
			return Weights[0] * X[0] + Weights[1] * X[1] + Weights[2] * X[2] + Weights[3] * (static_cast<T>(1.0) - Weights[0] - Weights[1] - Weights[2]);
		}

		TVec3<T> GetPointFromBarycentricCoordinates(const TVec4<T>& Weights) const
		{
			return Weights[0] * X[0] + Weights[1] * X[1] + Weights[2] * X[2] + Weights[3] * X[3];
		}

		bool BarycentricInside(const TVec3<T>& Location, const T Tolerance=0) const
		{
			TVec3<T> Weights = GetFirstThreeBarycentricCoordinates(Location);
			return Weights[0] >= Tolerance && Weights[1] >= Tolerance && Weights[2] >= Tolerance && Weights[0]+Weights[1]+Weights[2] <= 1+Tolerance;
		}

		FString ToString() const
		{
			return FString::Printf(TEXT("Tetrahedron: A: [%f, %f, %f], B: [%f, %f, %f], C: [%f, %f, %f], D: [%f, %f, %f]"),
				X[0][0], X[0][1], X[0][2],
				X[1][0], X[1][1], X[1][2],
				X[2][0], X[2][1], X[2][2]);
		}

		//! Initialize outward facing triangles, regardless of the orientation of the tetrahedron.
		TArray<TTriangle<T>> GetTriangles() const
		{
			TArray<TTriangle<T>> Triangles;
			Triangles.SetNumUninitialized(4);
			if (GetSignedVolume() <= 0)
			{
				Triangles[0] = TTriangle<T>(X[0], X[1], X[2]);
				Triangles[1] = TTriangle<T>(X[0], X[3], X[1]);
				Triangles[2] = TTriangle<T>(X[0], X[2], X[3]);
				Triangles[3] = TTriangle<T>(X[1], X[3], X[2]);
			}
			else
			{
				Triangles[0] = TTriangle<T>(X[0], X[2], X[1]);
				Triangles[1] = TTriangle<T>(X[0], X[1], X[3]);
				Triangles[2] = TTriangle<T>(X[0], X[3], X[2]);
				Triangles[3] = TTriangle<T>(X[1], X[2], X[3]);
			}
			return Triangles;
		}

		bool Inside(const TVec3<T>& Location, const T HalfThickness=0) const
		{
			TArray<TTriangle<T>> Tris = GetTriangles();
			return Inside(Tris, Location, HalfThickness);
		}

		static bool Inside(const TArray<TTriangle<T>>& Tris, const TVec3<T>& Location, const T HalfThickness=0)
		{
			checkSlow(Tris.Num() == 4);

			for(int32 i=0; i < Tris.Num(); i++)
				if (!Inside(Tris[i].GetPlane(), Location, -HalfThickness))
				{
					return false;
				}
			return true;
		}

		bool Outside(const TVec3<T>& Location, const T HalfThickness = 0) const
		{
			TArray<TTriangle<T>> Tris = GetTriangles();
			return Outside(Tris, Location, HalfThickness);
		}

		static bool Outside(const TArray<TTriangle<T>>& Tris, const TVec3<T>& Location, const T HalfThickness = 0)
		{
			checkSlow(Tris.Num() == 4);
			for (int32 i = 0; i < Tris.Num(); i++)
				if (Outside(Tris[i].GetPlane(), Location, HalfThickness))
				{
					return true;
				}
			return false;
		}

		//! \p Tolerance should be a small negative number to include boundary.
		bool RobustInside(const TVec3<T>& Location, const T Tolerance=0) const
		{
			T V1 = TripleProduct(Location - X[0], X[1] - X[0], X[2] - X[0]);
			T V2 = TripleProduct(X[3] - X[0], X[1] - X[0], Location - X[0]);
			T V3 = TripleProduct(X[3] - X[0], Location - X[0], X[2] - X[0]);
			T V4 = TripleProduct(X[3] - Location, X[1] - Location, X[2] - Location);
			return !(fabs(V1) <= Tolerance || fabs(V2) <= Tolerance || fabs(V3) <= Tolerance || fabs(V4) <= Tolerance ||
				(Sign(V1) != Sign(V2) || Sign(V2) != Sign(V3) || Sign(V3) != Sign(V4)));
		}

		// Note: this method will fail to project to the actual surface if Location is outside and the closest point is
		// on an edge with an acute angle such that Location is outside both connected faces. FindClosestPointAndBary will work in this case.
		TVec3<T> ProjectToSurface(const TArray<TTriangle<T>>& Tris, const TVec3<T>& Location) const
		{
			if (Inside(Tris, Location)) 
			{
				int32 Idx = 0;
				T Dist = TVec3<T>::DotProduct(Tris[0][0] - Location, Tris[0].GetNormal());
				T DistTemp = TVec3<T>::DotProduct(Tris[1][0] - Location, Tris[1].GetNormal());
				if (DistTemp < Dist) 
				{ 
					Idx = 1;
					Dist = DistTemp;
				}
				DistTemp = TVec3<T>::DotProduct(Tris[2][0] - Location, Tris[2].GetNormal());
				if (DistTemp < Dist)
				{ 
					Idx = 2;
					Dist = DistTemp;
				}
				DistTemp = TVec3<T>::DotProduct(Tris[3][0] - Location, Tris[3].GetNormal());
				if (DistTemp < Dist) 
				{ 
					Idx = 3;
					Dist = DistTemp;
				}
				return Location + Dist * Tris[Idx].GetNormal();
			}
			else 
			{
				TVec3<T> SurfacePoint(Location);
				for (int32 i = 0; i < 4; i++)
				{
					if (Tris[i].GetPlane().SignedDistance(SurfacePoint) > static_cast<T>(0.0))
					{
						const TVec3<T> Norm = Tris[i].GetNormal();
						SurfacePoint -= TVec3<T>::DotProduct(Location - Tris[i][0], Norm) * Norm;
					}
				}
				return SurfacePoint;
			}
		}

		// Tolerance should be small negative number or zero. It is used to compare barycentric coordinate values
		TVec3<T> FindClosestPointAndBary(const TVec3<T>& Location, TVec4<T>& OutBary, const T Tolerance = 0) const
		{
			const TVec3<T> TetWeights = GetFirstThreeBarycentricCoordinates(Location);
			if (TetWeights[0] >= Tolerance && TetWeights[1] >= Tolerance && TetWeights[2] >= Tolerance && TetWeights[0] + TetWeights[1] + TetWeights[2] <= 1 + Tolerance)
			{
				// Inside tetrahedron
				OutBary = TVec4<T>(TetWeights[0], TetWeights[1], TetWeights[2], static_cast<T>(1.0) - TetWeights[0] - TetWeights[1] - TetWeights[2]);
				checkSlow(TVec3<T>::DistSquared(Location, GetPointFromBarycentricCoordinates(OutBary)) < static_cast<T>(UE_SMALL_NUMBER));
				return Location;
			}
			
			// Test closest point is inside a face
			const bool IsInverted = GetSignedVolume() <= 0;
			static const TVec3<int32> InvertedTriangleIndices[4] =
			{
				TVec3<int32>(0,1,2),
				TVec3<int32>(0,3,1),
				TVec3<int32>(0,2,3),
				TVec3<int32>(1,3,2)
			};
			static const TVec3<int32> NonInvertedTriangleIndices[4] =
			{
				TVec3<int32>(0,2,1),
				TVec3<int32>(0,1,3),
				TVec3<int32>(0,3,2),
				TVec3<int32>(1,2,3)
			};
			const TVec3<int32>* FaceIndices = IsInverted ? InvertedTriangleIndices : NonInvertedTriangleIndices;
			for (int32 FaceId = 0; FaceId < 4; ++FaceId)
			{
				const TVec3<int32>& TriIndices = FaceIndices[FaceId];
				const TTriangle<T> Tri(X[TriIndices[0]], X[TriIndices[1]], X[TriIndices[2]]);
				const TPlaneConcrete<T, 3> Plane = Tri.GetPlane(0);
				const T Dist = TVec3<T>::DotProduct(Location - Plane.X(), Plane.Normal());
				if (Dist >= 0)
				{
					// Point is on front-face side of tri.
					const TVec3<T> ClosestPointOnPlane = Location - Dist * Plane.Normal();
					const TVec2<T> TriBary = ComputeBarycentricInPlane(Tri[0], Tri[1], Tri[2], ClosestPointOnPlane);
					if (TriBary[0] >= Tolerance && TriBary[1] >= Tolerance && TriBary[0] + TriBary[1] <= 1 + Tolerance)
					{
						OutBary = TVec4<T>(static_cast<T>(0.));
						OutBary[TriIndices[1]] = TriBary[0];
						OutBary[TriIndices[2]] = TriBary[1];
						OutBary[TriIndices[0]] = static_cast<T>(1.) - TriBary[0] - TriBary[1];
						checkSlow(TVec3<T>::DistSquared(ClosestPointOnPlane, GetPointFromBarycentricCoordinates(OutBary)) < static_cast<T>(UE_SMALL_NUMBER));
						return ClosestPointOnPlane;
					}
				}
			}

			// Closest point is on an edge/vertex
			static const TVec2<int32> EdgeIndices[6] =
			{
				TVec2<int32>(0,1),
				TVec2<int32>(0,2),
				TVec2<int32>(0,3),
				TVec2<int32>(1,2),
				TVec2<int32>(1,3),
				TVec2<int32>(2,3)
			};
			TVec3<T> ClosestEdgePoint = TVec3<T>::ZeroVector;
			T ClosestEdgeAlpha(0);
			int32 ClosestEdgeIndex = INDEX_NONE;
			T ClosestEdgeDistSq = std::numeric_limits<T>::max();
			for (int32 EdgeIndex = 0; EdgeIndex < 6; ++EdgeIndex)
			{
				T Alpha;
				const TVec3<T> ClosestPoint = FindClosestPointAndAlphaOnLineSegment(X[EdgeIndices[EdgeIndex][0]], X[EdgeIndices[EdgeIndex][1]], Location, Alpha);
				const T DistSq = TVec3<T>::DistSquared(Location, ClosestPoint);
				if (DistSq < ClosestEdgeDistSq)
				{
					ClosestEdgeDistSq = DistSq;
					ClosestEdgePoint = ClosestPoint;
					ClosestEdgeAlpha = Alpha;
					ClosestEdgeIndex = EdgeIndex;
				}
			}
			checkSlow(ClosestEdgeIndex != INDEX_NONE);

			OutBary = TVec4<T>(static_cast<T>(0.));
			OutBary[EdgeIndices[ClosestEdgeIndex][0]] = static_cast<T>(1.) - ClosestEdgeAlpha;
			OutBary[EdgeIndices[ClosestEdgeIndex][1]] = ClosestEdgeAlpha;
			checkSlow(TVec3<T>::DistSquared(ClosestEdgePoint, GetPointFromBarycentricCoordinates(OutBary)) < static_cast<T>(UE_SMALL_NUMBER));
			return ClosestEdgePoint;
		}

	private:
		static T TripleProduct(const TVec3<T>& U, const TVec3<T>& V, const TVec3<T>& W)
		{
			return TVec3<T>::DotProduct(U, TVec3<T>::CrossProduct(V, W));
		}

		static int32 Sign(const T Value)
		{
			return Value < static_cast<T>(0) ? -1 : 1;
		}

		static bool Inside(const TPlane<T, 3>& Plane, const TVec3<T>& Location, const T HalfThickness = 0)
		{
			return Plane.SignedDistance(Location) <= -HalfThickness;
		}

		static bool Outside(const TPlane<T, 3>& Plane, const TVec3<T>& Location, const T HalfThickness = 0)
		{
			return !Inside(Plane, Location, -HalfThickness);
		}

		friend FChaosArchive& operator<<(FChaosArchive& Ar, TTetrahedron);

		TVec3<T> X[4];
	};

	using FTetrahedron = TTetrahedron<FReal>;

	template <typename T>
	inline FChaosArchive& operator<<(FChaosArchive& Ar, TTetrahedron<T>& Value)
	{
		Ar << Value.X[0] << Value.X[1] << Value.X[2] << Value.X[3];
		return Ar;
	}

} // namespace Chaos
