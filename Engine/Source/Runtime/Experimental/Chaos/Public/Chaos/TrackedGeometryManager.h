// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Serializable.h"
#include "Chaos/ImplicitObject.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#include "Serialization/MemoryWriter.h"

class FOutputDevice;

namespace Chaos
{
	class FTrackedGeometryManager
	{
	public:
		static CHAOS_API FTrackedGeometryManager& Get();
		
		UE_DEPRECATED(5.8, "Use the version taking the remainder of the command")
		CHAOS_API void DumpMemoryUsage(FOutputDevice* Ar) const;
		CHAOS_API void DumpMemoryUsage(const TCHAR* Cmd, FOutputDevice* Ar) const;
		
	private:
		TMap<TSerializablePtr<FImplicitObject>, FString> SharedGeometry;
		FTransactionallySafeCriticalSection CriticalSection;
		
		friend FImplicitObject;
		
		//These are private because of various threading considerations. ImplicitObject does the cleanup because it needs extra information
		CHAOS_API void AddGeometry(TSerializablePtr<FImplicitObject> Geometry, const FString& DebugInfo);
		
		CHAOS_API void RemoveGeometry(const FImplicitObject* Geometry);
	};
}
