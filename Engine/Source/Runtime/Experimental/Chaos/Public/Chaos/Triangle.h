// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Chaos/GJK.h"
#include "ImplicitObject.h"
#include "Plane.h"
#include "Chaos/Plane.h"

namespace Chaos
{
	/**
	 * @brief Convert the cartesian coordinate into a barycentric corrdinate.
	 * Compute barycentric coordinates/weights of Point inside 3D triangle (V0,V1,V2).
	 * If point is in triangle plane and inside triangle, coords will be positive and sum to 1.
	 * ie if result is a, then vPoint = a.x*V0 + a.y*V1 + a.z*V2.
	 * 
	 * @note For points outside the triangle it will return negative barycentric coordinates that should still sum to 1. However,
	 * if the triangle is degenerate and the point is not on the line/vertex, it will return (1,0,0) with no way to detect this condition.
	 * 
	 * @see FromBarycentric()
	 */
	template <typename RealType>
	inline TVec3<RealType> ToBarycentric(const TVec3<RealType>& Point, const TVec3<RealType>& V0, const TVec3<RealType>& V1, const TVec3<RealType>& V2)
	{
		const TVec3<RealType> V02 = V0 - V2;
		const TVec3<RealType> V12 = V1 - V2;
		const TVec3<RealType> PV2 = Point - V2;
		const RealType M00 = V02.Dot(V02);
		const RealType M01 = V02.Dot(V12);
		const RealType M11 = V12.Dot(V12);
		const RealType R0 = V02.Dot(PV2);
		const RealType R1 = V12.Dot(PV2);
		const RealType Det = M00 * M11 - M01 * M01;
		RealType Bary1, Bary2, Bary3;
		if (Det > UE_SMALL_NUMBER)
		{
			// Non-degenerate triangle
			const RealType InvDet = RealType(1) / Det;
			Bary1 = (M11 * R0 - M01 * R1) * InvDet;
			Bary2 = (M00 * R1 - M01 * R0) * InvDet;
			Bary3 = RealType(1) - Bary1 - Bary2;
		}
		else
		{
			const bool bHaveLine02 = (M00 > UE_SMALL_NUMBER);
			const bool bHaveLine12 = (M11 > UE_SMALL_NUMBER);
			const bool bInsideLine02 = ((M00 > UE_SMALL_NUMBER) && (R0 >= 0) && (R0 <= M00));
			const bool bInsideLine12 = ((M11 > UE_SMALL_NUMBER) && (R1 >= 0) && (R1 <= M11));
			if (bInsideLine02 || (bHaveLine02 && !bInsideLine12))
			{
				// Line-degenerate and point on line segment V02
				const RealType Alpha02 = R0 / M00;
				Bary1 = Alpha02;
				Bary2 = RealType(0);
				Bary3 = RealType(1) - Alpha02;
			}
			else if (bHaveLine12)
			{
				// Line-degenerate and point on line segment V12
				const RealType Alpha12 = R1 / M11;
				Bary1 = RealType(0);
				Bary2 = Alpha12;
				Bary3 = RealType(1) - Alpha12;
			}
			else
			{
				// Point-degenerate, or line-degenerate and point not on line
				Bary1 = RealType(1);
				Bary2 = RealType(0);
				Bary3 = RealType(0);
			}
		}
		return TVec3<RealType>(Bary1, Bary2, Bary3);
	}

	/**
	 * @brief Convert the barycentric coordinate into a cartesian corrdinate
	 * @see ToBarycentric()
	*/
	template <typename RealType>
	inline TVec3<RealType> FromBarycentric(const TVec3<RealType>& Barycentric, const TVec3<RealType>& V0, const TVec3<RealType>& V1, const TVec3<RealType>& V2)
	{
		return Barycentric.X * V0 + Barycentric.Y * V1 + Barycentric.Z * V2;
	}


	template<typename T>
	class TTriangle
	{
	public:
		TTriangle()
		{
		}

		TTriangle(const TVec3<T>& InA, const TVec3<T>& InB, const TVec3<T>& InC)
			: ABC{ InA, InB, InC }
		{
		}

		FORCEINLINE TVec3<T>& operator[](uint32 InIndex)
		{
			checkSlow(InIndex < 3);
			return ABC[InIndex];
		}

		FORCEINLINE const TVec3<T>& operator[](uint32 InIndex) const
		{
			checkSlow(InIndex < 3);
			return ABC[InIndex];
		}

		FORCEINLINE const TVec3<T>& GetVertex(const int32 InIndex) const
		{
			checkSlow(InIndex < 3);
			return ABC[InIndex];
		}

		FORCEINLINE TVec3<T> GetNormal() const
		{
			return TVec3<T>::CrossProduct(ABC[1] - ABC[0], ABC[2] - ABC[0]).GetSafeNormal();
		}

		FORCEINLINE TPlane<T, 3> GetPlane() const
		{
			return TPlane<T, 3>(ABC[0], GetNormal());
		}

		// Face index is ignored since we only have one face
		// Used for manifold generation
		FORCEINLINE TPlaneConcrete<T, 3> GetPlane(int32 FaceIndex) const
		{
			return TPlaneConcrete <T, 3> (ABC[0], GetNormal());
		}

		FORCEINLINE void GetPlaneNX(const int32 FaceIndex, TVec3<T>& OutN, TVec3<T>& OutX) const
		{
			OutN = GetNormal();
			OutX = ABC[0];
		}

		FORCEINLINE TVec3<T> GetCentroid() const
		{
			return (GetVertex(0) + GetVertex(1) + GetVertex(2)) / T(3.0);
		}

		// Get the nearest point on an edge and the edge vertices
		// Used for manifold generation
		TVec3<T> GetClosestEdge(int32 PlaneIndexHint, const TVec3<T>& Position, TVec3<T>& OutEdgePos0, TVec3<T>& OutEdgePos1) const
		{
			TVec3<T> ClosestEdgePosition = TVec3<T>(0);
			T ClosestDistanceSq = TNumericLimits<T>::Max();

			int32 PlaneVerticesNum = 3;
			
			TVec3<T> P0 = ABC[2];
			for (int32 PlaneVertexIndex = 0; PlaneVertexIndex < PlaneVerticesNum; ++PlaneVertexIndex)
			{
				const TVector<T, 3>& P1 = GetVertex(PlaneVertexIndex);
				
				const TVec3<T> EdgePosition = FMath::ClosestPointOnLine(P0, P1, Position);
				const T EdgeDistanceSq = (EdgePosition - Position).SizeSquared();

				if (EdgeDistanceSq < ClosestDistanceSq)
				{
					ClosestDistanceSq = EdgeDistanceSq;
					ClosestEdgePosition = EdgePosition;
					OutEdgePos0 = P0;
					OutEdgePos1 = P1;
				}

				P0 = P1;
			}

			return ClosestEdgePosition;
		}

		// Get the nearest point on an edge
		// Used for manifold generation
		TVec3<T> GetClosestEdgePosition(int32 PlaneIndexHint, const TVec3<T>& Position) const
		{
			TVec3<T> Unused0, Unused1;
			return GetClosestEdge(PlaneIndexHint, Position, Unused0, Unused1);
		}


		// The number of vertices that make up the corners of the specified face
		// Used for manifold generation
		int32 NumPlaneVertices(int32 PlaneIndex) const
		{
			return 3;
		}

		// Change the winding order of this triangle by flipping the last two vertices
		FTriangle& ReverseWinding()
		{
			Swap(ABC[1], ABC[2]);
			return *this;
		}

		// Triangle winding order is always 1.0. When we reverse the winding of a triangle we flip the verts. @see ReverseWinding
		// Used for manifold generation
		FORCEINLINE T GetWindingOrder() const
		{
			return 1.0f;
		}

		// Get an array of all the plane indices that belong to a vertex (up to MaxVertexPlanes).
		// Returns the number of planes found.
		FORCEINLINE int32 FindVertexPlanes(int32 VertexIndex, int32* OutVertexPlanes, int32 MaxVertexPlanes) const
		{
			if(MaxVertexPlanes > 0)
			{
				OutVertexPlanes[0] = 0;
			}
			return 1; 
		}
		
		// Get up to the 3  plane indices that belong to a vertex
		// Returns the number of planes found.
		int32 GetVertexPlanes3(int32 VertexIndex, int32& PlaneIndex0, int32& PlaneIndex1, int32& PlaneIndex2) const
		{
			PlaneIndex0 = 0;
			return 1;
		}
		
		// Get the index of the plane that most opposes the normal
		int32 GetMostOpposingPlane(const TVec3<T>& Normal) const
		{
			return 0; // Only have one plane
		}

		// Get the vertex index of one of the vertices making up the corners of the specified face
		// Used for manifold generation
		int32 GetPlaneVertex(int32 PlaneIndex, int32 PlaneVertexIndex) const
		{
			return PlaneVertexIndex;
		}

		// Triangle is just one plane
		// Used for manifold generation
		int32 NumPlanes() const { return 1; }

		FORCEINLINE T PhiWithNormal(const TVec3<T>& InSamplePoint, TVec3<T>& OutNormal) const
		{
			OutNormal = GetNormal();
			TVec3<T> ClosestPoint = FindClosestPointOnTriangle(GetPlane(), ABC[0], ABC[1], ABC[2], InSamplePoint);
			return TVec3<T>::DotProduct((InSamplePoint - ClosestPoint), OutNormal);
		}

		FORCEINLINE TVec3<T> Support(const TVec3<T>& Direction, const T Thickness, int32& VertexIndex) const
		{
			const T DotA = TVec3<T>::DotProduct(ABC[0], Direction);
			const T DotB = TVec3<T>::DotProduct(ABC[1], Direction);
			const T DotC = TVec3<T>::DotProduct(ABC[2], Direction);

			if(DotA >= DotB && DotA >= DotC)
			{
				VertexIndex = 0;
				if(Thickness != 0)
				{
					return ABC[0] + Direction.GetUnsafeNormal() * Thickness;
				}
				return ABC[0];
			}
			else if(DotB >= DotA && DotB >= DotC)
			{
				VertexIndex = 1;
				if(Thickness != 0)
				{
					return ABC[1] + Direction.GetUnsafeNormal() * Thickness;
				}
				return ABC[1];
			}
			VertexIndex = 2;
			if(Thickness != 0)
			{
				return ABC[2] + Direction.GetUnsafeNormal() * Thickness;
			}
			return ABC[2];
		}

		FORCEINLINE_DEBUGGABLE TVec3<T> SupportCore(const TVec3<T>& Direction, const T InMargin, T* OutSupportDelta,int32& VertexIndex) const
		{
			// Note: assumes margin == 0
			const T DotA = TVec3<T>::DotProduct(ABC[0], Direction);
			const T DotB = TVec3<T>::DotProduct(ABC[1], Direction);
			const T DotC = TVec3<T>::DotProduct(ABC[2], Direction);

			if (DotA >= DotB && DotA >= DotC)
			{
				VertexIndex = 0;
				return ABC[0];
			}
			else if (DotB >= DotA && DotB >= DotC)
			{
				VertexIndex = 1;
				return ABC[1];
			}
			VertexIndex = 2;
			return ABC[2];
		}

		FORCEINLINE TVec3<T> SupportCoreScaled(const TVec3<T>& Direction, T InMargin, const TVec3<T>& Scale, T* OutSupportDelta, int32& VertexIndex) const
		{
			// Note: ignores InMargin, assumed 0 (triangles cannot have a margin as they are zero thickness)
			return SupportCore(Direction * Scale, 0.0f, OutSupportDelta, VertexIndex) * Scale;
		}

		FORCEINLINE T GetMargin() const { return 0; }
		FORCEINLINE T GetRadius() const { return 0; }
		FORCEINLINE FRealSingle GetMarginf() const { return 0.0f; }
		FORCEINLINE FRealSingle GetRadiusf() const { return 0.0f; }

		FORCEINLINE bool Raycast(const TVec3<T>& StartPoint, const TVec3<T>& Dir, const T Length, const T Thickness, T& OutTime, TVec3<T>& OutPosition, TVec3<T>& OutNormal, int32& OutFaceIndex) const
		{
			// No face as this is only one triangle
			OutFaceIndex = INDEX_NONE;

			// Pass through GJK #BGTODO Maybe specialise if it's possible to be faster
			const FRigidTransform3 StartTM(StartPoint, FRotation3::FromIdentity());
			const TSphere<T, 3> Sphere(TVec3<T>(0), Thickness);
			return GJKRaycast(*this, Sphere, StartTM, Dir, Length, OutTime, OutPosition, OutNormal);
		}

		FORCEINLINE bool Overlap(const TVec3<T>& Point, const T Thickness) const
		{
			const TVec3<T> ClosestPoint = FindClosestPointOnTriangle(GetPlane(), ABC[0], ABC[1], ABC[2], Point);
			const T AdjustedThickness = FMath::Max(Thickness, UE_KINDA_SMALL_NUMBER);
			return (Point - ClosestPoint).SizeSquared() <= (AdjustedThickness * AdjustedThickness);
		}

		FORCEINLINE bool IsConvex() const
		{
			return true;
		}

		FString ToString() const
		{
			return FString::Printf(TEXT("Triangle: A: [%f, %f, %f], B: [%f, %f, %f], C: [%f, %f, %f]"), GetVertex(0).X, GetVertex(0).Y, GetVertex(0).Z, GetVertex(1).X, GetVertex(1).Y, GetVertex(1).Z, GetVertex(2).X, GetVertex(2).Y, GetVertex(2).Z);
		}

		struct FLineIntersectionToleranceProvider
		{
			static constexpr T ParallelTolerance = (T)UE_SMALL_NUMBER;
			static constexpr T BaryTolerance = (T)0;
		};
		FORCEINLINE bool LineIntersection(const TVec3<T>& StartPoint, const TVec3<T>& EndPoint, TVector<T, 2>& OutBary, T& OutTime) const;

	private:

		friend FChaosArchive& operator<<(FChaosArchive& Ar, TTriangle& Value);

		TVec3<T> ABC[3];
	};

	using FTriangle = TTriangle<FReal>;

	template<typename T>
	inline FChaosArchive& operator<<(FChaosArchive& Ar, TTriangle<T>& Value)
	{
		Ar << Value.ABC[0] << Value.ABC[1] << Value.ABC[2];
		return Ar;
	}

	template<typename T>
	struct TRayTriangleIntersectionDefaultToleranceProvider
	{
		static constexpr T ParallelTolerance = (T)UE_SMALL_NUMBER;
		static constexpr T BaryTolerance = (T)UE_SMALL_NUMBER;
	};

	template<typename T, typename ToleranceProvider = TRayTriangleIntersectionDefaultToleranceProvider<T> >
	FORCEINLINE_DEBUGGABLE bool RayTriangleIntersectionAndBary(const TVec3<T>& RayStart, const TVec3<T>& RayDir, T RayLength,
		const TVec3<T>& A, const TVec3<T>& B, const TVec3<T>& C, T& OutT, TVec2<T>& OutBary, TVec3<T>& OutN)
	{
		const TVec3<T> AB = B - A; // edge 1
		const TVec3<T> AC = C - A; // edge 2
		const TVec3<T> Normal = TVec3<T>::CrossProduct(AB, AC);
		const TVec3<T> NegRayDir = -RayDir;

		const T Den = TVec3<T>::DotProduct(NegRayDir, Normal);
		if (FMath::Abs(Den) < ToleranceProvider::ParallelTolerance)
		{
			// ray is parallel or away to the triangle plane it is a miss
			return false;
		}

		const T InvDen = (T)1 / Den;

		// let's compute the time to intersection
		const TVec3<T> RayToA = RayStart - A;
		const T Time = TVec3<T>::DotProduct(RayToA, Normal) * InvDen;
		if (Time < (T)0 || Time > RayLength)
		{
			return false;
		}

		// now compute barycentric coordinates
		const TVec3<T> RayToACrossNegDir = FVec3::CrossProduct(NegRayDir, RayToA);
		const T UU = TVec3<T>::DotProduct(AC, RayToACrossNegDir) * InvDen;
		if (UU < -ToleranceProvider::BaryTolerance || UU >(1 + ToleranceProvider::BaryTolerance))
		{
			return false; // outside of the triangle
		}
		const T VV = -TVec3<T>::DotProduct(AB, RayToACrossNegDir) * InvDen;
		if (VV < -ToleranceProvider::BaryTolerance || (VV + UU) >(1 + ToleranceProvider::BaryTolerance))
		{
			return false; // outside of the triangle
		}

		// point is within the triangle, let's compute 
		OutT = Time;
		OutBary = { UU, VV };
		OutN = Normal.GetSafeNormal();
		OutN *= FMath::Sign(Den);
		return true;
	}

	/**
	* Ray / triangle  intersection
	* this provides a double sided test
	* note : this method assumes that the triangle formed by A,B and C is well formed
	*/
	template<typename T>
	FORCEINLINE bool RayTriangleIntersection(
		const TVec3<T>& RayStart, const TVec3<T>& RayDir, T RayLength,
		const TVec3<T>& A, const TVec3<T>& B, const TVec3<T>& C,
		T& OutT, TVec3<T>& OutN
	)
	{
		TVec2<T> BaryUnused;
		return RayTriangleIntersectionAndBary(RayStart, RayDir, RayLength, A, B, C, OutT, BaryUnused, OutN);
	}

	template<typename T>
	FORCEINLINE bool TTriangle<T>::LineIntersection(const TVec3<T>& StartPoint, const TVec3<T>& EndPoint, TVector<T, 2>& OutBary, T& OutTime) const
	{
		const TVec3<T> StartToEnd = EndPoint - StartPoint;
		const T SegmentLenSq = StartToEnd.SizeSquared();
		if (SegmentLenSq < UE_SMALL_NUMBER)
		{
			return false;
		}
		const T SegmentLen = FMath::Sqrt(SegmentLenSq);
		const T OneOverSegmentLen = (T)1 / SegmentLen;
		const TVec3<T> Ray = StartToEnd * OneOverSegmentLen;

		TVec3<T> NormalUnused;
		if (RayTriangleIntersectionAndBary<T, FLineIntersectionToleranceProvider>(StartPoint, Ray, SegmentLen, ABC[0], ABC[1], ABC[2], OutTime, OutBary, NormalUnused))
		{
			// OutTime is between 0 and SegmentLen. Convert to 0 to 1
			OutTime *= OneOverSegmentLen;
			return true;
		}

		return false;
	}
	
	template<typename T>
	class UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use other triangle based ImplicitObjects") TImplicitTriangle final : public FImplicitObject
	{
	public:

		TImplicitTriangle()
			: FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Triangle)
		{}

		TImplicitTriangle(const TImplicitTriangle&) = delete;

		TImplicitTriangle(TImplicitTriangle&& InToSteal)
			: FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Triangle)
		{}

		TImplicitTriangle(const TVec3<T>& InA, const TVec3<T>& InB, const TVec3<T>& InC)
			: FImplicitObject(EImplicitObject::IsConvex | EImplicitObject::HasBoundingBox, ImplicitObjectType::Triangle)
			, Tri(InA, InB, InC)
		{
		}

		TVec3<T>& operator[](uint32 InIndex)
		{
			return Tri[InIndex];
		}

		const TVec3<T>& operator[](uint32 InIndex) const
		{
			return Tri[InIndex];
		}

		TVec3<T> GetNormal() const
		{
			return Tri.GetNormal();
		}

		TPlane<T, 3> GetPlane() const
		{
			return Tri.GetPlane();
		}

		static constexpr EImplicitObjectType StaticType()
		{
			return ImplicitObjectType::Triangle;
		}

		virtual T PhiWithNormal(const TVec3<T>& InSamplePoint, TVec3<T>& OutNormal) const override
		{
			return Tri.PhiWithNormal(InSamplePoint, OutNormal);
		}

		virtual const class TAABB<T, 3> BoundingBox() const override
		{
			TAABB<T, 3> Bounds(Tri[0], Tri[0]);
			Bounds.GrowToInclude(Tri[1]);
			Bounds.GrowToInclude(Tri[2]);

			return Bounds;
		}

		virtual TVec3<T> Support(const TVec3<T>& Direction, const T Thickness, int32& VertexIndex) const override
		{
			return Tri.Support(Direction, Thickness, VertexIndex);
		}

		virtual bool Raycast(const TVec3<T>& StartPoint, const TVec3<T>& Dir, const T Length, const T Thickness, T& OutTime, TVec3<T>& OutPosition, TVec3<T>& OutNormal, int32& OutFaceIndex) const override
		{
			return Tri.Raycast(StartPoint, Dir, Length, Thickness, OutTime, OutPosition, OutNormal, OutFaceIndex);
		}

		virtual TVec3<T> FindGeometryOpposingNormal(const TVec3<T>& DenormDir, int32 FaceIndex, const TVec3<T>& OriginalNormal) const override
		{
			return Tri.GetNormal();
		}

		virtual bool Overlap(const TVec3<T>& Point, const T Thickness) const override
		{
			return Tri.Overlap(Point, Thickness);
		}

		virtual FString ToString() const override
		{
			return FString::Printf(TEXT("Triangle: A: [%f, %f, %f], B: [%f, %f, %f], C: [%f, %f, %f]"), Tri[0].X, Tri[0].Y, Tri[0].Z, Tri[1].X, Tri[1].Y, Tri[1].Z, Tri[2].X, Tri[2].Y, Tri[2].Z);
		}

		virtual void Serialize(FChaosArchive& Ar) override
		{
			Ar << Tri;
		}

		virtual uint32 GetTypeHash() const override
		{
			return HashCombine(UE::Math::GetTypeHash(Tri[0]), HashCombine(UE::Math::GetTypeHash(Tri[1]), UE::Math::GetTypeHash(Tri[2])));
		}

		virtual FName GetTypeName() const override
		{
			return FName("Triangle");
		}

	private:

		TTriangle<T> Tri;
	};
}