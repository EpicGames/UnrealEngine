// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Templates/UnrealTemplate.h"

#include "Chaos/Core.h"


namespace Chaos
{
	/**
	 * @brief Data returned by TriangleMesh point-triangle and edge-edge queries.
	 */
	template<typename T>
	struct TTriangleCollisionPoint
	{
		// Type of contact (determines meaning of some properties below such as Indices and BarycentricCoordinates)
		enum struct EContactType : uint8
		{
			Invalid = 0,
			PointFace,
			EdgeEdge,
			EdgeFace
		} ContactType;

		// Index into Point and Face arrays or Edge arrays
		int32 Indices[2];

		// Barycentric coordinates for contacts
		// Point-Face: Bary[0] = 1.0 (Point), Bary[1:3] correspond with coordinates on Face
		// Edge-Edge: Bary[0:1] = coordinates on first edge, [2:3] coordinates on second edge
		// Edge-Face: Bary[0] = second coordinate on edge (first can be calculated as 1-Bary[0], Bary[1:3] correspond with coordinates on Face
		TVec4<T> Bary;

		// World space location (which point depends on query type)
		TVec3<T> Location;

		// World space normal (which normal depends on query type)
		TVec3<T> Normal;

		// Contact separation (negative for overlap)
		T Phi;

		TTriangleCollisionPoint()
			: ContactType(EContactType::Invalid)
			, Phi(TNumericLimits<T>::Max())
		{
		}

		// Whether the contact point has been set up with contact data
		bool IsSet() const { return ContactType != EContactType::Invalid; }
	};
}
