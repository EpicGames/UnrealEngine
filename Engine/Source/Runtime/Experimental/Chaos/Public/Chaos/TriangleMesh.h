// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/Map.h"
#include "Chaos/Particles.h"
#include "Chaos/SegmentMesh.h"
#include "Containers/ContainersFwd.h"

#include "AABBTree.h"

namespace Chaos
{
	template <typename TPayloadType, typename T> class THierarchicalSpatialHash;

	template<typename T> struct TTriangleCollisionPoint;

	namespace Softs
	{
	class FPBDFlatWeightMap;
	}

	class FTriangleMesh
	{
	public:
		CHAOS_API FTriangleMesh();
		CHAOS_API FTriangleMesh(TArray<TVec3<int32>>&& Elements, const int32 StartIdx = 0, const int32 EndIdx = -1, const bool CullDegenerateElements=true);
		FTriangleMesh(const FTriangleMesh& Other) = delete;
		CHAOS_API FTriangleMesh(FTriangleMesh&& Other);
		CHAOS_API ~FTriangleMesh();

		/**
		 * Initialize the \c FTriangleMesh.
		 *
		 *	\p CullDegenerateElements removes faces with degenerate indices, and 
		 *	will change the order of \c MElements.
		 */
		CHAOS_API void Init(TArray<TVec3<int32>>&& Elements, const int32 StartIdx = 0, const int32 EndIdx = -1, const bool CullDegenerateElements=true);
		CHAOS_API void Init(const TConstArrayView<TVec3<int32>>& Elements, const int32 StartIdx = 0, const int32 EndIdx = -1, const bool CullDegenerateElements = true);

		CHAOS_API void ResetAuxiliaryStructures();

		CHAOS_API void Serialize(FArchive& Ar);

		/**
		 * Returns the closed interval of the smallest vertex index used by 
		 * this class, to the largest.
		 *
		 * If this mesh is empty, the second index of the range will be negative.
		 */
		CHAOS_API TVec2<int32> GetVertexRange() const;
		int32 GetStartIndex() const 
		{
			return MStartIdx;
		}
		int32 GetNumIndices() const
		{
			return MNumIndices;
		}

		/** Returns the set of vertices used by triangles. */
		CHAOS_API TSet<int32> GetVertices() const;
		/** Returns the unique set of vertices used by this triangle mesh. */
		CHAOS_API void GetVertexSet(TSet<int32>& VertexSet) const;

		/** Returns the unique set of vertices used by this triangle mesh. */
		CHAOS_API void GetVertexSetAsArray(TArray<int32>& VertexSet) const;

		/**
		 * Extends the vertex range.
		 *
		 * Since the vertex range is built from connectivity, it won't include any 
		 * free vertices that either precede the first vertex, or follow the last.
		 */
		FORCEINLINE void ExpandVertexRange(const int32 StartIdx, const int32 EndIdx)
		{
			const TVec2<int32> CurrRange = GetVertexRange();
			if (StartIdx <= CurrRange[0] && EndIdx >= CurrRange[1])
			{
				MStartIdx = StartIdx;
				MNumIndices = EndIdx - StartIdx + 1;
			}
		}

		FORCEINLINE const TArray<TVec3<int32>>& GetElements() const& { return MElements; }
		/**
		 * Move accessor for topology array.
		 *
		 * Use via:
		 * \code
		 * TArray<TVec3<int32>> Triangles;
		 * FTriangleMesh TriMesh(Triangles); // steals Triangles to TriMesh::MElements
		 * Triangles = MoveTemp(TriMesh).GetElements(); // steals TriMesh::MElements back to Triangles
		 * \endcode
		 */
		FORCEINLINE TArray<TVec3<int32>> GetElements() && { return MoveTemp(MElements); }

		FORCEINLINE const TArray<TVec3<int32>>& GetSurfaceElements() const& { return MElements; }
		/**
		 * Move accessor for topology array.
		 *
		 * Use via:
		 * \code
		 * TArray<TVec3<int32>> Triangles;
		 * FTriangleMesh TriMesh(Triangles); // steals Triangles to TriMesh::MElements
		 * Triangles = MoveTemp(TriMesh).GetSurfaceElements(); // steals TriMesh::MElements back to Triangles
		 * \endcode
		 */
		FORCEINLINE TArray<TVec3<int32>> GetSurfaceElements() && { return MoveTemp(MElements); }

		FORCEINLINE int32 GetNumElements() const { return MElements.Num(); }

		CHAOS_API const TMap<int32, TSet<int32>>& GetPointToNeighborsMap() const;
		FORCEINLINE const TSet<int32>& GetNeighbors(const int32 Element) const { return GetPointToNeighborsMap()[Element]; }

		CHAOS_API TConstArrayView<TArray<int32>> GetPointToTriangleMap() const;  // Return an array view using global indexation. Only elements starting at MStartIdx will be valid!
		FORCEINLINE const TArray<int32>& GetCoincidentTriangles(const int32 Element) const { return GetPointToTriangleMap()[Element]; }

		FORCEINLINE TSet<int32> GetNRing(const int32 Element, const int32 N) const
		{
			TSet<int32> Neighbors;
			TSet<int32> LevelNeighbors, PrevLevelNeighbors;
			PrevLevelNeighbors = GetNeighbors(Element);
			for (auto SubElement : PrevLevelNeighbors)
			{
				check(SubElement != Element);
				Neighbors.Add(SubElement);
			}
			for (int32 i = 1; i < N; ++i)
			{
				for (auto SubElement : PrevLevelNeighbors)
				{
					const auto& SubNeighbors = GetNeighbors(SubElement);
					for (auto SubSubElement : SubNeighbors)
					{
						if (!Neighbors.Contains(SubSubElement) && SubSubElement != Element)
						{
							LevelNeighbors.Add(SubSubElement);
						}
					}
				}
				PrevLevelNeighbors = LevelNeighbors;
				LevelNeighbors.Reset();
				for (auto SubElement : PrevLevelNeighbors)
				{
					if (!Neighbors.Contains(SubElement))
					{
						check(SubElement != Element);
						Neighbors.Add(SubElement);
					}
				}
			}
			return Neighbors;
		}

		/** Return the array of all cross segment indices for all pairs of adjacent triangles. */
		CHAOS_API TArray<Chaos::TVec2<int32>> GetUniqueAdjacentPoints() const;
		/** Return the array of bending element indices {i0, i1, i2, i3}, with {i0, i1} the segment indices and {i2, i3} the cross segment indices. */
		CHAOS_API TArray<Chaos::TVec4<int32>> GetUniqueAdjacentElements() const;

		/** The GetFaceNormals functions assume Counter Clockwise triangle windings in a Left Handed coordinate system
			If this is not the case the returned face normals may be inverted
		*/
		template <typename T>
		CHAOS_API TArray<TVec3<T>> GetFaceNormals(const TConstArrayView<TVec3<T>>& Points, const bool ReturnEmptyOnError = true) const;
		template <typename T>
		CHAOS_API void GetFaceNormals(TArray<TVec3<T>>& Normals, const TConstArrayView<TVec3<T>>& Points, const bool ReturnEmptyOnError = true) const;
		FORCEINLINE TArray<FVec3> GetFaceNormals(const FParticles& InParticles, const bool ReturnEmptyOnError = true) const
		{ return GetFaceNormals(TConstArrayView<FVec3>(InParticles.X()), ReturnEmptyOnError); }

		CHAOS_API TArray<FVec3> GetPointNormals(const TConstArrayView<FVec3>& points, const bool ReturnEmptyOnError = true, const bool bUseGlobalArray=false);
		FORCEINLINE TArray<FVec3> GetPointNormals(const FParticles& InParticles, const bool ReturnEmptyOnError = true)
		{ return GetPointNormals(TConstArrayView<FVec3>(InParticles.X()), ReturnEmptyOnError); }
		
		CHAOS_API void GetPointNormals(TArrayView<FVec3> PointNormals, const TConstArrayView<FVec3>& FaceNormals, const bool bUseGlobalArray);
		/** \brief Get per-point normals. 
		 * This const version of this function requires \c GetPointToTriangleMap() 
		 * to be called prior to invoking this function. 
		 * @param bUseGlobalArray When true, fill the array from the StartIdx to StartIdx + NumIndices - 1 positions, otherwise fill the array from the 0 to NumIndices - 1 positions.
		 */
		template <typename T>
		CHAOS_API void GetPointNormals(TArrayView<TVec3<T>> PointNormals, const TConstArrayView<TVec3<T>>& FaceNormals, const bool bUseGlobalArray) const;

		CHAOS_API void GetPointNormals(TArrayView<TVec3<FRealSingle>> PointNormals, const TConstArrayView<TVec3<FRealSingle>>& FaceNormals, const bool bUseGlobalArray) const;

		static CHAOS_API FTriangleMesh GetConvexHullFromParticles(const TConstArrayView<FVec3>& points);
		/** Deprecated. Use TArrayView version. */
		static FORCEINLINE FTriangleMesh GetConvexHullFromParticles(const FParticles& InParticles)
		{ return GetConvexHullFromParticles(InParticles.X()); }

		/**
		 * @brief Note that the SegmentMesh is lazily calculated (this method is not threadsafe unless it is known that the SegmentMesh is already up to date)
		 * @ret The connectivity of this mesh represented as a collection of unique segments.
		 */
		CHAOS_API const FSegmentMesh& GetSegmentMesh() const;
		/**
		 * @brief Note that this data is lazily calculated with the SegmentMesh (this method is not threadsafe unless it is known that the SegmentMesh is already up to date)
		 * @ret A map from all face indices, to the indices of their associated edges.
		 */
		CHAOS_API const TArray<TVec3<int32>>& GetFaceToEdges() const;
		/**
		 * @brief Note that this data is lazily calculated with the SegmentMesh (this method is not threadsafe unless it is known that the SegmentMesh is already up to date)
		 * @ret A map from all edge indices, to the indices of their containing faces. 
		 */
		CHAOS_API const TArray<TVec2<int32>>& GetEdgeToFaces() const;

		/**
		 * @ret Curvature between adjacent faces, specified on edges in radians.
		 * @param faceNormals - a normal per face.
		 * Curvature between adjacent faces is measured by the angle between face normals,
		 * where a curvature of 0 means they're coplanar.
		 */
		CHAOS_API TArray<FReal> GetCurvatureOnEdges(const TArray<FVec3>& faceNormals);
		/** @brief Helper that generates face normals on the fly. */
		CHAOS_API TArray<FReal> GetCurvatureOnEdges(const TConstArrayView<FVec3>& points);

		/**
		 * @ret The maximum curvature at points from connected edges, specified in radians.
		 * @param edgeCurvatures - a curvature per edge.
		 * The greater the number, the sharper the crease. -FLT_MAX denotes free particles.
		 */
		CHAOS_API TArray<FReal> GetCurvatureOnPoints(const TArray<FReal>& edgeCurvatures);
		/** @brief Helper that generates edge curvatures on the fly. */
		CHAOS_API TArray<FReal> GetCurvatureOnPoints(const TConstArrayView<FVec3>& points);

		/**
		 * Get the set of point indices that live on the boundary (an edge with only 1 
		 * coincident face).
		 */
		CHAOS_API TSet<int32> GetBoundaryPoints();

		/**
		 * Find vertices that are coincident within the subset @param TestIndices 
		 * of given coordinates @param Points, and return a correspondence mapping
		 * from redundant vertex index to consolidated vertex index.
		 */
		CHAOS_API TMap<int32, int32> FindCoincidentVertexRemappings(
			const TArray<int32>& TestIndices,
			const TConstArrayView<FVec3>& Points);

		/**
		 * @ret An array of vertex indices ordered from most important to least.
		 * @param Points - point positions.
		 * @param PointCurvatures - a per-point measure of curvature.
		 * @param CoincidentVertices - indices of points that are coincident to another point.
		 * @param RestrictToLocalIndexRange - ignores points outside of the index range used by this mesh.
		 */
		CHAOS_API TArray<int32> GetVertexImportanceOrdering(
		    const TConstArrayView<FVec3>& Points,
		    const TArray<FReal>& PointCurvatures,
		    TArray<int32>* CoincidentVertices = nullptr,
		    const bool RestrictToLocalIndexRange = false);
		/** @brief Helper that generates point curvatures on the fly. */
		CHAOS_API TArray<int32> GetVertexImportanceOrdering(
		    const TConstArrayView<FVec3>& Points,
		    TArray<int32>* CoincidentVertices = nullptr,
		    const bool RestrictToLocalIndexRange = false);

		/** @brief Reorder vertices according to @param Order. */
		CHAOS_API void RemapVertices(const TArray<int32>& Order);
		CHAOS_API void RemapVertices(const TMap<int32, int32>& Remapping);

		CHAOS_API void RemoveDuplicateElements();
		CHAOS_API void RemoveDegenerateElements();

		template <typename T>
		static FORCEINLINE void InitEquilateralTriangleXY(FTriangleMesh& TriMesh, TParticles<T, 3>& Particles)
		{
			const int32 Idx = Particles.Size();
			Particles.AddParticles(3);
			// Left handed
			Particles.X(Idx + 0) = FVec3((T)0., (T)0.8083, (T)0.);
			Particles.X(Idx + 1) = FVec3((T)0.7, (T)-0.4041, (T)0.);
			Particles.X(Idx + 2) = FVec3((T)-0.7, (T)-0.4041, (T)0.);

			TArray<TVec3<int32>> Elements;
			Elements.SetNum(1);
			Elements[0] = TVec3<int32>(Idx + 0, Idx + 1, Idx + 2);

			TriMesh.Init(MoveTemp(Elements));
		}
		template <typename T>
		static FORCEINLINE void InitEquilateralTriangleYZ(FTriangleMesh& TriMesh, TParticles<T, 3>& Particles)
		{
			const int32 Idx = Particles.Size();
			Particles.AddParticles(3);
			// Left handed
			Particles.SetX(Idx + 0, FVec3((T)0., (T)0., (T)0.8083));
			Particles.SetX(Idx + 1, FVec3((T)0., (T)0.7, (T)-0.4041));
			Particles.SetX(Idx + 2, FVec3((T)0., (T)-0.7, (T)-0.4041));

			TArray<TVec3<int32>> Elements;
			Elements.SetNum(1);
			Elements[0] = TVec3<int32>(Idx + 0, Idx + 1, Idx + 2);

			TriMesh.Init(MoveTemp(Elements));
		}

		// BVH-based collision queries
		template<typename T>
		using TBVHType = TAABBTree<int32, TAABBTreeLeafArray<int32, /*bComputeBounds=*/false, T>, /*bMutable=*/true, T>;

		template<typename T>
		void BuildBVH(const TConstArrayView<TVec3<T>>& Points, TBVHType<T>& BVH) const;

		// NOTE: This method assumes the BVH has already been built/fitted to Points.
		template<typename T>
		bool PointProximityQuery(const TBVHType<T>& BVH, const TConstArrayView<TVec3<T>>& Points, const int32 PointIndex, const TVec3<T>& PointPosition, const T PointThickness, const T ThisThickness, 
			TFunctionRef<bool (const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<T>>& Result) const;

		template<typename T>
		bool EdgeIntersectionQuery(const TBVHType<T>& BVH, const TConstArrayView<TVec3<T>>& Points, const int32 EdgeIndex, const TVec3<T>& EdgePosition1, const TVec3<T>& EdgePosition2,
			TFunctionRef<bool(const int32 EdgeIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<T>>& Result) const;

		//! Returns \c false if \p Point is outside of the smooth normal cone, where a smooth projection doesn't exist.
		template<typename T>
		bool SmoothProject(const TBVHType<T>& BVH, const TConstArrayView<FVec3>& Points, const TArray<FVec3>& PointNormals,
			const FVec3& Point, int32& TriangleIndex, FVec3& Weights, const int32 MaxIters=10) const;

		template<typename T>
		using TSpatialHashType = THierarchicalSpatialHash<int32, T>;

		// Hierarchy will only go down to lods as small as MinSpatialLodSize.
		template<typename T>
		CHAOS_API void BuildSpatialHash(const TConstArrayView<TVec3<T>>& Points, TSpatialHashType<T>& SpatialHash, const T MinSpatialLodSize = (T)0.) const;
		CHAOS_API void BuildSpatialHash(const TConstArrayView<TVec3<FRealSingle>>& Points, TSpatialHashType<FRealSingle>& SpatialHash, const Softs::FPBDFlatWeightMap& PointThicknesses, int32 ThicknessMapIndexOffset, const FRealSingle MinSpatialLodSize = 0.f) const;

		template<typename T>
		CHAOS_API bool PointProximityQuery(const TSpatialHashType<T>& SpatialHash, const TConstArrayView<TVec3<T>>& Points, const int32 PointIndex, const TVec3<T>& PointPosition, const T PointThickness, const T ThisThickness,
			TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<T>>& Result) const;
		CHAOS_API bool PointProximityQuery(const TSpatialHashType<FRealSingle>& SpatialHash, const TConstArrayView<TVec3<FRealSingle>>& Points, const int32 PointIndex, const TVec3<FRealSingle>& PointPosition, const FRealSingle PointThickness, const Softs::FPBDFlatWeightMap& ThisThicknesses,
			const FRealSingle ThisThicknessExtraMultiplier, int32 ThicknessMapIndexOffset, TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<FRealSingle>>& Result) const;

		template<typename T>
		CHAOS_API bool PointClosestTriangleQuery(const TSpatialHashType<T>& SpatialHash, const TConstArrayView<TVec3<T>>& Points, const int32 PointIndex, const TVec3<T>& PointPosition, const T PointThickness, const T ThisThickness,
			TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<T>>& Result) const;

		// uses GetPosition lambda function to query point position instead of Points array
		template<typename T>
		CHAOS_API bool PointClosestTriangleQuery(const TSpatialHashType<T>& SpatialHash, TFunctionRef<const TVec3<T>&(int32 PointIndex)> GetPosition, const int32 PointIndex, const TVec3<T>& PointPosition, const T PointThickness, const T ThisThickness,
			TFunctionRef<bool(const int32 PointIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<T>>& Result) const;

		template<typename T>
		bool EdgeIntersectionQuery(const TSpatialHashType<T>& SpatialHash, const TConstArrayView<TVec3<T>>& Points, const int32 EdgeIndex, const TVec3<T>& EdgePosition1, const TVec3<T>& EdgePosition2,
			TFunctionRef<bool(const int32 EdgeIndex, const int32 TriangleIndex)> BroadphaseTest, TArray<TTriangleCollisionPoint<T>>& Result) const;
		
	private:
		CHAOS_API void InitHelper(const int32 StartIdx, const int32 EndIdx, const bool CullDegenerateElements=true);

		FORCEINLINE int32 GlobalToLocal(int32 GlobalIdx) const
		{
			const int32 LocalIdx = GlobalIdx - MStartIdx;
			check(LocalIdx >= 0 && LocalIdx < MNumIndices);
			return LocalIdx;
		}

		FORCEINLINE int32 LocalToGlobal(int32 LocalIdx) const
		{
			const int32 GlobalIdx = LocalIdx + MStartIdx;
			check(GlobalIdx >= MStartIdx && GlobalIdx < MStartIdx + MNumIndices);
			return GlobalIdx;
		}

		TArray<TVec3<int32>> MElements;

		mutable TArray<TArray<int32>> MPointToTriangleMap;  // !! Unlike the TArrayView returned by GetPointToTriangleMap, this array starts at 0 for the point of index MStartIdx. Use GlobalToLocal to access with a global index. Note that this array's content is always indexed in global index.
		mutable TMap<int32, TSet<int32>> MPointToNeighborsMap;

		mutable FSegmentMesh MSegmentMesh;
		mutable TArray<TVec3<int32>> MFaceToEdges;
		mutable TArray<TVec2<int32>> MEdgeToFaces;

		int32 MStartIdx;
		int32 MNumIndices;
	};

	template <typename T>
	using TTriangleMesh = FTriangleMesh;
}
