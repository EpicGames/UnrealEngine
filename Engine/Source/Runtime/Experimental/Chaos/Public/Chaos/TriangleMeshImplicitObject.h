// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/GeometryParticles.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/OBBVectorized.h"
#include "Chaos/SegmentMesh.h"
#include "Chaos/Triangle.h"

#include "AABBTree.h"
#include "BoundingVolume.h"
#include "BoundingVolumeHierarchy.h"
#include "Box.h"
#include "ChaosArchive.h"
#include "ImplicitObject.h"
#include "UObject/ExternalPhysicsCustomObjectVersion.h"
#include "UObject/PhysicsObjectVersion.h"

#include <type_traits>

namespace Chaos::Private
{
	class FMeshContactGenerator;
}

namespace UE::Physics
{
	class FTriangleMeshGeometry;
}

namespace Chaos
{
	extern CHAOS_API bool TriMeshPerPolySupport;

	struct FMTDInfo;
	template <typename QueryGeomType>
	struct FTriangleMeshOverlapVisitorNoMTD;
	class FTriangleMeshImplicitObject;
	FArchive& operator<<(FArchive& Ar, FAABBVectorized& Bounds);

	class FTrimeshIndexBuffer;

	struct FTrimeshBVH
	{
		using FAABBType = TAABB<FRealSingle, 3>;

		enum class EPerFaceBoundsRepresentation : uint8
		{
			VectorizedFloat = 0,
			Byte
		};

		enum class EVisitorResult
		{
			Stop = 0,
			Continue,
		};

		enum class EFilterResult
		{
			Skip = 0,
			Keep,
		};
		
		struct alignas(16) FChildData
		{
			FChildData()
				: ChildOrFaceIndex{ INDEX_NONE, INDEX_NONE }
				, FaceCount{ 0, 0 }
			{
			}

			FORCEINLINE int32 GetChildOrFaceIndex(int ChildIndex) const
			{
				return ChildOrFaceIndex[ChildIndex];
			}

			FORCEINLINE int32 GetFaceCount(int ChildIndex) const
			{
				return FaceCount[ChildIndex];
			}

			FORCEINLINE void SetChildOrFaceIndex(int ChildIndex, int32 InChildOrFaceIndex)
			{
				ChildOrFaceIndex[ChildIndex] = InChildOrFaceIndex;
			}

			FORCEINLINE void SetFaceCount(int ChildIndex, int32 InFaceCount)
			{
				FaceCount[ChildIndex] = InFaceCount;
			}

			FORCEINLINE void SetBounds(int ChildIndex, const FAABB3& AABB)
			{
				Bounds[ChildIndex] = FAABBVectorized(AABB);
			}
			FORCEINLINE void SetBounds(int ChildIndex, const TAABB<FRealSingle, 3>& AABB)
			{
				Bounds[ChildIndex] = FAABBVectorized(AABB);
			}

			FORCEINLINE const FAABBVectorized& GetBounds(int ChildIndex) const { return Bounds[ChildIndex]; }

			void Serialize(FArchive& Ar)
			{
				Ar << Bounds[0];
				Ar << ChildOrFaceIndex[0];
				Ar << FaceCount[0];
				Ar << Bounds[1];
				Ar << ChildOrFaceIndex[1];
				Ar << FaceCount[1];
			}

		private:
			FAABBVectorized Bounds[2];
			int32 ChildOrFaceIndex[2];
			int32 FaceCount[2];
		};
		
		struct FNode
		{
			FNode() {}

			void Serialize(FArchive& Ar)
			{
				Children.Serialize(Ar);
			}
			FChildData Children;
		};

		template <typename SQVisitor>
		FORCEINLINE_DEBUGGABLE void Raycast(const FVec3& Start, const FVec3& Dir, const FReal Length, SQVisitor& Visitor) const
		{
			FRealSingle CurrentLength = static_cast<FRealSingle>(Length);
			const VectorRegister4Float StartSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegisterDouble(Start.X, Start.Y, Start.Z, 0.0));
			const VectorRegister4Float DirSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegisterDouble(Dir.X, Dir.Y, Dir.Z, 0.0));
			const VectorRegister4Float Parallel = VectorCompareGT(GlobalVectorConstants::SmallNumber, VectorAbs(DirSimd));
			const VectorRegister4Float InvDirSimd = VectorBitwiseNotAnd(Parallel, VectorDivide(VectorOne(), DirSimd));

			const auto BoundsFilter = [&StartSimd, &InvDirSimd, &Parallel, &CurrentLength](const FAABBVectorized& Bounds) -> EFilterResult
			{
				const VectorRegister4Float CurDataLength = VectorLoadFloat1(&CurrentLength);
				const bool bHit = Bounds.RaycastFast(StartSimd, InvDirSimd, Parallel, CurDataLength);

				return bHit ? EFilterResult::Keep : EFilterResult::Skip;
			};

			const auto FaceVisitor = [&Visitor, &CurrentLength](int32 FaceIndex)
			{
				const bool bContinueVisiting = Visitor.VisitRaycast(FaceIndex, CurrentLength);
				return bContinueVisiting? EVisitorResult::Continue: EVisitorResult::Stop;
			};

			VisitTree(BoundsFilter, FaceVisitor);
		}

		template <typename SQVisitor>
		FORCEINLINE_DEBUGGABLE void Sweep(const FVec3& Start, const FVec3& Dir, const FReal Length, const FVec3& QueryHalfExtents, SQVisitor& Visitor) const
		{
			FRealSingle CurrentLength = static_cast<FRealSingle>(Length);
			const VectorRegister4Float StartSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegisterDouble(Start.X, Start.Y, Start.Z, 0.0));
			const VectorRegister4Float DirSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegisterDouble(Dir.X, Dir.Y, Dir.Z, 0.0));
			const VectorRegister4Float Parallel = VectorCompareGT(GlobalVectorConstants::SmallNumber, VectorAbs(DirSimd));
			const VectorRegister4Float InvDirSimd = VectorBitwiseNotAnd(Parallel, VectorDivide(VectorOne(), DirSimd));

			const VectorRegister4Float QueryHalfExtentsSimd = MakeVectorRegisterFloatFromDouble(MakeVectorRegisterDouble(QueryHalfExtents.X, QueryHalfExtents.Y, QueryHalfExtents.Z, 0.0));
			const auto BoundsFilter = [&StartSimd, &InvDirSimd, &Parallel, &CurrentLength, &QueryHalfExtentsSimd](const FAABBVectorized& Bounds) -> EFilterResult
			{
				const VectorRegister4Float CurDataLength = VectorLoadFloat1(&CurrentLength);
				const FAABBVectorized SweepBounds(VectorSubtract(Bounds.GetMin(), QueryHalfExtentsSimd), VectorAdd(Bounds.GetMax(), QueryHalfExtentsSimd));
				const bool bHit = SweepBounds.RaycastFast(StartSimd, InvDirSimd, Parallel, CurDataLength);

				return bHit ? EFilterResult::Keep : EFilterResult::Skip;
			};

			const auto FaceVisitor = [&Visitor, &CurrentLength](int32 FaceIndex)
			{
				const bool bContinueVisiting = Visitor.VisitSweep(FaceIndex, CurrentLength);
				return bContinueVisiting ? EVisitorResult::Continue : EVisitorResult::Stop;
			};

			VisitTree(BoundsFilter, FaceVisitor);
		}

		template <typename SQVisitor>
		FORCEINLINE_DEBUGGABLE void Overlap(const FAABBVectorized& AABB, SQVisitor& Visitor) const
		{
			const auto BoundsFilter = [&AABB](const FAABBVectorized& Bounds) -> EFilterResult
			{
				const bool bHit = Bounds.Intersects(AABB);
				return bHit ? EFilterResult::Keep : EFilterResult::Skip;
			};

			const auto FaceVisitor = [&Visitor](int32 FaceIndex)
			{
				const bool bContinueVisiting = Visitor.VisitOverlap(FaceIndex);
				return bContinueVisiting ? EVisitorResult::Continue : EVisitorResult::Stop;
			};

			VisitTree(BoundsFilter, FaceVisitor);
		}

		template <typename SQVisitor>
		FORCEINLINE_DEBUGGABLE void OverlapOBB(const Private::FOBBVectorized& Obb, SQVisitor& Visitor) const
		{
			const auto BoundsFilter = [&Obb](const FAABBVectorized& Bounds) -> EFilterResult
			{
				const bool bHit = Obb.IntersectAABB(Bounds);
				return bHit ? EFilterResult::Keep : EFilterResult::Skip;
			};

			const auto FaceVisitor = [&Visitor](int32 FaceIndex)
			{
				const bool bContinueVisiting = Visitor.VisitOverlap(FaceIndex);
				return bContinueVisiting ? EVisitorResult::Continue : EVisitorResult::Stop;
			};

			VisitTree(BoundsFilter, FaceVisitor);
		}

		template <typename QueryGeomType>
		bool FindAllIntersectionsNoMTD(const Private::FOBBVectorized& Intersection, const TRigidTransform<FReal, 3>& Transform, const QueryGeomType& QueryGeom, FReal Thickness, const FVec3& TriMeshScale, const FTriangleMeshImplicitObject* TriMesh) const;
		template <typename QueryGeomType>
		bool FindAllIntersectionsNoMTD(const FAABB3& Intersection, const TRigidTransform<FReal, 3>& Transform, const QueryGeomType& QueryGeom, FReal Thickness, const FVec3& TriMeshScale, const FTriangleMeshImplicitObject* TriMesh) const;
		CHAOS_API TArray<int32> FindAllIntersections(const FAABB3& Intersection) const;
		
		// original visit faces method, using original/full-float face bounds
		template <typename BoundsFilterType, typename FaceVisitorType>
		FORCEINLINE_DEBUGGABLE EVisitorResult VisitFaces(int32 StartIndex, int32 IndexCount, BoundsFilterType& BoundsFilter, FaceVisitorType& FaceVisitor) const
		{
			const int32 EndIndex = (StartIndex + IndexCount);  
			for (int32 FaceIndex = StartIndex; FaceIndex < EndIndex; ++FaceIndex)
			{
				if (BoundsFilter(FaceBounds[FaceIndex]) == EFilterResult::Keep)
				{
					if (FaceVisitor(FaceIndex) == EVisitorResult::Stop)
					{
						return EVisitorResult::Stop;
					}
				}
			}
			return EVisitorResult::Continue;
		}

		template <typename BoundsFilterType, typename FaceVisitorType>
		FORCEINLINE_DEBUGGABLE void VisitTree(BoundsFilterType& BoundsFilter, FaceVisitorType& FaceVisitor) const
		{
			if (Nodes.Num() == 0)
			{
				return;
			}
			if (!FaceBounds.IsEmpty())
			{
				VisitTreeImpl<BoundsFilterType, FaceVisitorType, false>(BoundsFilter, FaceVisitor);
			}
			else
			{
				VisitTreeImpl<BoundsFilterType, FaceVisitorType, true>(BoundsFilter, FaceVisitor);
			}
		}
private:
		template <typename BoundsFilterType, typename FaceVisitorType, bool bUseQuantizedBounds>
		FORCEINLINE_DEBUGGABLE void VisitTreeImpl(BoundsFilterType& BoundsFilter, FaceVisitorType& FaceVisitor) const
		{
			TArray<int32, TInlineAllocator<16>> NodeIndexStack;
			NodeIndexStack.Push(0);
			while (NodeIndexStack.Num())
			{
				const int32 NodeIndex = NodeIndexStack.Pop(EAllowShrinking::No);
				check(Nodes.IsValidIndex(NodeIndex));
				const FNode& Node = Nodes[NodeIndex];

				for (int32 ChildIndex = 0; ChildIndex < 2; ++ChildIndex)
				{
					const FChildData& ChildData = Node.Children; 
					const int32 FaceIndex = ChildData.GetChildOrFaceIndex(ChildIndex);
					if (FaceIndex != INDEX_NONE)
					{
						if (BoundsFilter(ChildData.GetBounds(ChildIndex)) == EFilterResult::Keep)
						{
							const uint32 FaceCount = ChildData.GetFaceCount(ChildIndex);
							if (FaceCount > 0)
							{
								if constexpr (!bUseQuantizedBounds)
								{
									if (EVisitorResult::Stop == VisitFaces(FaceIndex, FaceCount, BoundsFilter, FaceVisitor))
									{
										return;
									}
								}
								else
								{
									if (EVisitorResult::Stop == VisitFacesQuantized(ChildData.GetBounds(ChildIndex), FaceIndex, FaceCount, BoundsFilter, FaceVisitor))
									{
										return;
									}
								}
							}
							else
							{
								NodeIndexStack.Push(FaceIndex);
							}
						}
					}
				}
			}
		}
public:

		CHAOS_API void Serialize(FChaosArchive& Ar);
		
		TArray<FNode> Nodes;
		TArray<FAABBVectorized> FaceBounds;

		/// Quantized bounds data/methods:
		/// Optionally stores the FaceBounds as uint8 fractions of their parent node bounds
		/// to save storage costs, instead of using the FaceBounds array

private:
		// Array of quantized face bound data; with 6 elements per triangle + one buffer element at the end to allow safe use of vectorized loads
		TArray<uint8> QuantizedFaceBoundData;

		// @return true if the BVH should use quantized bounds -- i.e., if it is not empty and has no FaceBounds
		bool NeedsQuantizedBounds();
		
		// Set quantized bounds from mesh data
		void InitializeQuantizedBounds(const TParticles<FRealSingle, 3>& Particles, const FTrimeshIndexBuffer& IndexBuffer);
		
		// Reset face bounds data
		void ResetFaceBounds(int32 NumTris, bool bUseQuantized);
		
		// Add a new quantized bound
		void AddQuantizedFaceBound(const TAABB<FRealSingle, 3>& NodeBounds, const TAABB<FRealSingle, 3>& TriBounds);	

		// alternate visit faces method, using quantized face bounds
		template <typename BoundsFilterType, typename FaceVisitorType>
		FORCEINLINE_DEBUGGABLE EVisitorResult VisitFacesQuantized(const FAABBVectorized& NodeBounds, int32 StartIndex, int32 IndexCount, BoundsFilterType& BoundsFilter, FaceVisitorType& FaceVisitor) const
		{
			// compute the scale factor to convert quantized uint8 values to relative positions within the node
			constexpr VectorRegister4Float DeQuantFracScale = MakeVectorRegisterFloatConstant(1.f / 255.f, 1.f / 255.f, 1.f / 255.f, 1.f);
			const VectorRegister4Float DeQuantScale = VectorMultiply(VectorSubtract(NodeBounds.GetMax(), NodeBounds.GetMin()), DeQuantFracScale);

			const int32 EndIndex = (StartIndex + IndexCount);
			int32 FaceIndex = StartIndex;
			// Note EndIndex*6 is 1 element *past* the actual quantized bounds data, but should still be valid because we store an extra buffer byte to allow VectorLoadByte4 to be safely used
			checkSlow(IndexCount > 0 && (QuantizedFaceBoundData.IsValidIndex(FaceIndex * 6) && QuantizedFaceBoundData.IsValidIndex(EndIndex * 6)));
			for (const uint8* QBoundPtr = QuantizedFaceBoundData.GetData() + StartIndex * 6; FaceIndex < EndIndex; ++FaceIndex, QBoundPtr += 6)
			{
				if (BoundsFilter(FAABBVectorized(
					VectorMultiplyAdd(DeQuantScale, VectorLoadByte4(&QBoundPtr[0]), NodeBounds.GetMin()),
					VectorMultiplyAdd(DeQuantScale, VectorLoadByte4(&QBoundPtr[3]), NodeBounds.GetMin()))) == EFilterResult::Keep)
				{
					if (FaceVisitor(FaceIndex) == EVisitorResult::Stop)
					{
						return EVisitorResult::Stop;
					}
				}
			}
			return EVisitorResult::Continue;
		}
		
		// Private helper to set quantized bound data
		void SetQuantizedBounds(uint8* QuantizedData, const TAABB<FRealSingle, 3>& NodeBounds, const TAABB<FRealSingle, 3>& TriBounds);
		
		// FTriangleMeshImplicitObject has friend access to build the quantized bounds data
		friend class FTriangleMeshImplicitObject;
	};

	FORCEINLINE_DEBUGGABLE FChaosArchive& operator<<(FChaosArchive& Ar, FTrimeshBVH::FChildData& ChildData)
	{
		ChildData.Serialize(Ar);
		return Ar;
	}
	
	FORCEINLINE_DEBUGGABLE FChaosArchive& operator<<(FChaosArchive& Ar, FTrimeshBVH::FNode& Node)
	{
		Node.Serialize(Ar);
		return Ar;
	}

	FORCEINLINE_DEBUGGABLE FChaosArchive& operator<<(FChaosArchive& Ar, FTrimeshBVH::FAABBType& Bounds)
	{
		TBox<FRealSingle, 3>::SerializeAsAABB(Ar, Bounds);
		return Ar;
	}

	FORCEINLINE_DEBUGGABLE FChaosArchive& operator<<(FChaosArchive& Ar, FAABBVectorized& Bounds)
	{
		alignas(16) FRealSingle Floats[4];
		VectorStoreAligned(Bounds.GetMin(), Floats);
		TVector<FRealSingle, 3>  Min(Floats[0], Floats[1], Floats[2]);
		VectorStoreAligned(Bounds.GetMax(), Floats);
		TVector<FRealSingle, 3>  Max(Floats[0], Floats[1], Floats[2]);
		FTrimeshBVH::FAABBType BoundsToSerialize(Min, Max);
		TBox<FRealSingle, 3>::SerializeAsAABB(Ar, BoundsToSerialize);
		Bounds = FAABBVectorized(BoundsToSerialize);
		return Ar;
	}

	FORCEINLINE_DEBUGGABLE FArchive& operator<<(FArchive& Ar, FAABBVectorized& Bounds)
	{
		alignas(16) FRealSingle Floats[4];
		VectorStoreAligned(Bounds.GetMin(), Floats);
		TVector<FRealSingle, 3>  Min(Floats[0], Floats[1], Floats[2]);
		VectorStoreAligned(Bounds.GetMax(), Floats);
		TVector<FRealSingle, 3>  Max(Floats[0], Floats[1], Floats[2]);
		FTrimeshBVH::FAABBType BoundsToSerialize(Min, Max);
		TBox<FRealSingle, 3>::SerializeAsAABB(Ar, BoundsToSerialize);
		Bounds = FAABBVectorized(BoundsToSerialize);
		return Ar;
	}

	FORCEINLINE_DEBUGGABLE FChaosArchive& operator<<(FChaosArchive& Ar, FTrimeshBVH& TrimeshBVH)
	{
		TrimeshBVH.Serialize(Ar);
		return Ar;
	}
	
	class FTrimeshIndexBuffer
	{
	public:
		using LargeIdxType = int32;
		using SmallIdxType = uint16;

		FTrimeshIndexBuffer() = default;
		FTrimeshIndexBuffer(TArray<TVec3<LargeIdxType>>&& Elements)
		    : LargeIdxBuffer(MoveTemp(Elements))
		    , bRequiresLargeIndices(true)
		{
		}

		FTrimeshIndexBuffer(TArray<TVec3<SmallIdxType>>&& Elements)
		    : SmallIdxBuffer(MoveTemp(Elements))
		    , bRequiresLargeIndices(false)
		{
		}

		FTrimeshIndexBuffer(const FTrimeshIndexBuffer& Other) = delete;
		FTrimeshIndexBuffer& operator=(const FTrimeshIndexBuffer& Other) = delete;


		void Reinitialize(TArray<TVec3<LargeIdxType>>&& Elements)
		{
			LargeIdxBuffer.Empty();
			SmallIdxBuffer.Empty();
			bRequiresLargeIndices = true;
			LargeIdxBuffer = MoveTemp(Elements);
		}

		void Reinitialize(TArray<TVec3<SmallIdxType>>&& Elements)
		{
			LargeIdxBuffer.Empty();
			SmallIdxBuffer.Empty();
			bRequiresLargeIndices = false;
			SmallIdxBuffer = MoveTemp(Elements);
		}

		void Serialize(FArchive& Ar)
		{
			Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);

			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::TrimeshCanUseSmallIndices)
			{
				Ar << LargeIdxBuffer;
				bRequiresLargeIndices = true;
			}
			else
			{
				Ar << bRequiresLargeIndices;
				if (bRequiresLargeIndices)
				{
					Ar << LargeIdxBuffer;
				}
				else
				{
					Ar << SmallIdxBuffer;
				}
			}
		}

		bool RequiresLargeIndices() const
		{
			return bRequiresLargeIndices;
		}

		const TArray<TVec3<LargeIdxType>>& GetLargeIndexBuffer() const
		{
			check(bRequiresLargeIndices);
			return LargeIdxBuffer;
		}

		const TArray<TVec3<SmallIdxType>>& GetSmallIndexBuffer() const
		{
			check(!bRequiresLargeIndices);
			return SmallIdxBuffer;
		}

		int32 GetNumTriangles() const
		{
			if(bRequiresLargeIndices)
			{
				return LargeIdxBuffer.Num();
			}

			return SmallIdxBuffer.Num();
		}

		template<typename ExpectedType>
		const TArray<TVec3<ExpectedType>>& GetIndexBuffer() const
		{
			if constexpr(std::is_same_v<ExpectedType, LargeIdxType>)
			{
				check(bRequiresLargeIndices);
				return LargeIdxBuffer;
			}
			else if constexpr(std::is_same_v<ExpectedType, SmallIdxType>)
			{
				check(!bRequiresLargeIndices);
				return SmallIdxBuffer;
			}
			else
			{
				static_assert(sizeof(ExpectedType) == 0, "Unsupported index buffer type");
				return SmallIdxBuffer; // Appease the static analyzer by having a pointless return value
			}
		}

	private:
		TArray<TVec3<LargeIdxType>> LargeIdxBuffer;
		TArray<TVec3<SmallIdxType>> SmallIdxBuffer;
		bool bRequiresLargeIndices;
	};

	FORCEINLINE_DEBUGGABLE FArchive& operator<<(FArchive& Ar, FTrimeshIndexBuffer& Buffer)
	{
		Buffer.Serialize(Ar);
		return Ar;
	}

	template <typename IdxType, typename ParticlesType>
	inline void TriangleMeshTransformVertsHelper(const FVec3& TriMeshScale, int32 TriIdx, const ParticlesType& Particles,
		const TArray<TVector<IdxType, 3>>& Elements, FVec3& OutA, FVec3& OutB, FVec3& OutC)
	{
		OutA = Particles.GetX(Elements[TriIdx][0]) * TriMeshScale;
		OutB = Particles.GetX(Elements[TriIdx][1]) * TriMeshScale;
		OutC = Particles.GetX(Elements[TriIdx][2]) * TriMeshScale;
	}

	template <typename IdxType, typename ParticlesType>
	inline void TriangleMeshTransformVertsHelper(const FRigidTransform3& Transform, int32 TriIdx, const ParticlesType& Particles,
		const TArray<TVector<IdxType, 3>>& Elements, FVec3& OutA, FVec3& OutB, FVec3& OutC,
		int32& OutVertexIndexA, int32& OutVertexIndexB, int32& OutVertexIndexC)
	{
		const int32 VertexIndexA = Elements[TriIdx][0];
		const int32 VertexIndexB = Elements[TriIdx][1];
		const int32 VertexIndexC = Elements[TriIdx][2];
		// Note: deliberately using scaled transform here. See VisitTriangles
		OutA = Transform.TransformPosition(FVector(Particles.GetX(VertexIndexA)));
		OutB = Transform.TransformPosition(FVector(Particles.GetX(VertexIndexB)));
		OutC = Transform.TransformPosition(FVector(Particles.GetX(VertexIndexC)));
		OutVertexIndexA = VertexIndexA;
		OutVertexIndexB = VertexIndexB;
		OutVertexIndexC = VertexIndexC;
	}


	class FTriangleMeshImplicitObject final : public FImplicitObject
	{
	public:
		using FImplicitObject::GetTypeName;

		using ParticlesType = TParticles<FRealSingle, 3>;
		using ParticleVecType = TVec3<FRealSingle>;

		struct FBuildSettings
		{
			// Whether to store "quantized" bounds per triangle, instead of full vectorized bounds
			// This can save roughly 40% memory for the in-RAM representation of the triangle mesh, for some additional access cost (depending on the platform)
			FTrimeshBVH::EPerFaceBoundsRepresentation PerFaceBounds = FTrimeshBVH::EPerFaceBoundsRepresentation::VectorizedFloat;
			
			FBuildSettings() {}
		};

		// Bitflag representation to pack triangle mesh settings
		enum class ETriangleMeshSettingsFlags : uint8
		{
			CullsBackFaceRaycast = 1 << 0,
			QuantizePerFaceBounds = 1 << 1
		};
		FRIEND_ENUM_CLASS_FLAGS(ETriangleMeshSettingsFlags);

		template <typename IdxType>
		FTriangleMeshImplicitObject(ParticlesType&& Particles, TArray<TVec3<IdxType>>&& Elements, TArray<uint16>&& InMaterialIndices,
			TUniquePtr<TArray<int32>>&& InExternalFaceIndexMap = nullptr,
			TUniquePtr<TArray<int32>>&& InExternalVertexIndexMap = nullptr,
			const bool bInCullsBackFaceRaycast = false,
			const FBuildSettings InBuildSettings = FBuildSettings())
		: FImplicitObject(EImplicitObject::HasBoundingBox | EImplicitObject::DisableCollisions, ImplicitObjectType::TriangleMesh)
		, MParticles(MoveTemp(Particles))
		, MElements(MoveTemp(Elements))
		, MLocalBoundingBox(FVec3(0), FVec3(0))
		, MaterialIndices(MoveTemp(InMaterialIndices))
		, ExternalFaceIndexMap(MoveTemp(InExternalFaceIndexMap))
		, ExternalVertexIndexMap(MoveTemp(InExternalVertexIndexMap))
		, MeshSettingsFlags(PackSettingsFlags(bInCullsBackFaceRaycast, InBuildSettings))
		{
			const int32 NumTriangles = MElements.GetNumTriangles();
			if(NumTriangles > 0)
			{

				const TArray<TVec3<IdxType>>& Tris = MElements.GetIndexBuffer<IdxType>();
				const TVec3<IdxType>& FirstTri = Tris[0];

				MLocalBoundingBox = FAABB3(MParticles.GetX(FirstTri[0]), MParticles.GetX(FirstTri[0]));
				MLocalBoundingBox.GrowToInclude(MParticles.GetX(FirstTri[1]));
				MLocalBoundingBox.GrowToInclude(MParticles.GetX(FirstTri[2]));

				for(int32 TriangleIndex = 1; TriangleIndex < NumTriangles; ++TriangleIndex)
				{
					const TVec3<IdxType>& Tri = Tris[TriangleIndex];
					MLocalBoundingBox.GrowToInclude(MParticles.GetX(Tri[0]));
					MLocalBoundingBox.GrowToInclude(MParticles.GetX(Tri[1]));
					MLocalBoundingBox.GrowToInclude(MParticles.GetX(Tri[2]));
				}
			}
			
			RebuildFastBVH();
		}

		FTriangleMeshImplicitObject(const FTriangleMeshImplicitObject& Other) = delete;
		FTriangleMeshImplicitObject(FTriangleMeshImplicitObject&& Other) = delete;
		CHAOS_API virtual ~FTriangleMeshImplicitObject();

		virtual FReal GetRadius() const override
		{
			return 0.0f;
		}

		CHAOS_API virtual FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const override;

		CHAOS_API virtual bool Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const override;
		CHAOS_API virtual bool Overlap(const FVec3& Point, const FReal Thickness) const override;

		CHAOS_API bool OverlapGeom(const FSphere& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
		CHAOS_API bool OverlapGeom(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
		CHAOS_API bool OverlapGeom(const FCapsule& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;
		CHAOS_API bool OverlapGeom(const FConvex& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr) const;

		CHAOS_API bool OverlapGeom(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr, FVec3 TriMeshScale = FVec3(1.0f)) const;
		CHAOS_API bool OverlapGeom(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr, FVec3 TriMeshScale = FVec3(1.0f)) const;
		CHAOS_API bool OverlapGeom(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr, FVec3 TriMeshScale = FVec3(1.0f)) const;
		CHAOS_API bool OverlapGeom(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr, FVec3 TriMeshScale = FVec3(1.0f)) const;

		CHAOS_API bool SweepGeom(const FSphere& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, const bool bComputeMTD = false, FVec3 TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;
		CHAOS_API bool SweepGeom(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, const bool bComputeMTD = false, FVec3 TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;
		CHAOS_API bool SweepGeom(const FCapsule& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, const bool bComputeMTD = false, FVec3 TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;
		CHAOS_API bool SweepGeom(const FConvex& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, const bool bComputeMTD = false, FVec3 TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;

		CHAOS_API bool SweepGeom(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, const bool bComputeMTD = false, FVec3 TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;
		CHAOS_API bool SweepGeom(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, const bool bComputeMTD = false, FVec3 TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;
		CHAOS_API bool SweepGeom(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, const bool bComputeMTD = false, FVec3 TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;
		CHAOS_API bool SweepGeom(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness = 0, const bool bComputeMTD = false, FVec3 TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;

		// Sweep used for CCD. Ignores triangles we penetrate by less than IgnorePenetration, and calculate the TOI for a depth of TargetPenetration. If both are zero, this is equivalent to SweepGeom
		CHAOS_API bool SweepGeomCCD(const FSphere& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;
		CHAOS_API bool SweepGeomCCD(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;
		CHAOS_API bool SweepGeomCCD(const FCapsule& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;
		CHAOS_API bool SweepGeomCCD(const FConvex& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;
		CHAOS_API bool SweepGeomCCD(const TImplicitObjectScaled<FSphere>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;
		CHAOS_API bool SweepGeomCCD(const TImplicitObjectScaled<TBox<FReal, 3>>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;
		CHAOS_API bool SweepGeomCCD(const TImplicitObjectScaled<FCapsule>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;
		CHAOS_API bool SweepGeomCCD(const TImplicitObjectScaled<FConvex>& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;

		CHAOS_API bool GJKContactPoint(const FSphere& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& Location, FVec3& Normal, FReal& Penetration, int32& FaceIndex) const;
		CHAOS_API bool GJKContactPoint(const TBox<FReal, 3>& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& Location, FVec3& Normal, FReal& Penetration, int32& FaceIndex) const;
		CHAOS_API bool GJKContactPoint(const FCapsule& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& Location, FVec3& Normal, FReal& Penetration, int32& FaceIndex) const;
		CHAOS_API bool GJKContactPoint(const FConvex& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& Location, FVec3& Normal, FReal& Penetration, int32& FaceIndex) const;

		CHAOS_API bool GJKContactPoint(const TImplicitObjectScaled < FSphere >& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& Location, FVec3& Normal, FReal& Penetration, int32& FaceIndex, FVec3 TriMeshScale = FVec3(1.0f)) const;
		CHAOS_API bool GJKContactPoint(const TImplicitObjectScaled < TBox<FReal, 3> >& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& Location, FVec3& Normal, FReal& Penetration, int32& FaceIndex, FVec3 TriMeshScale = FVec3(1.0f)) const;
		CHAOS_API bool GJKContactPoint(const TImplicitObjectScaled < FCapsule >& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& Location, FVec3& Normal, FReal& Penetration, int32& FaceIndex, FVec3 TriMeshScale = FVec3(1.0f)) const;
		CHAOS_API bool GJKContactPoint(const TImplicitObjectScaled < FConvex >& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& Location, FVec3& Normal, FReal& Penetration, int32& FaceIndex, FVec3 TriMeshScale = FVec3(1.0f)) const;

		// Returns -1 if InternalFaceIndex is not in map, or map is invalid.
		CHAOS_API int32 GetExternalFaceIndexFromInternal(int32 InternalFaceIndex) const;

		// Does Trimesh cull backfaces in raycast.
		CHAOS_API bool GetCullsBackFaceRaycast() const;
		CHAOS_API void SetCullsBackFaceRaycast(const bool bInCullsBackFace);

		CHAOS_API static ETriangleMeshSettingsFlags PackSettingsFlags(bool bInCullsBackFaceRaycast, FBuildSettings BuildSettings);

		CHAOS_API virtual int32 FindMostOpposingFace(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDistance) const override;
		CHAOS_API virtual int32 FindMostOpposingFaceScaled(const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist, const FVec3& Scale) const override;
		CHAOS_API virtual FVec3 FindGeometryOpposingNormal(const FVec3& DenormDir, int32 FaceIndex, const FVec3& OriginalNormal) const override;

		virtual const FAABB3 BoundingBox() const override
		{
			return MLocalBoundingBox;
		}

		const FAABB3f BoundingBoxf() const
		{
			return FAABB3f(MLocalBoundingBox);
		}

		static constexpr EImplicitObjectType StaticType()
		{
			return ImplicitObjectType::TriangleMesh;
		}

		CHAOS_API virtual Chaos::FImplicitObjectPtr CopyGeometry() const;
		CHAOS_API virtual Chaos::FImplicitObjectPtr DeepCopyGeometry() const;

		UE_DEPRECATED(5.4, "Use DeepCopyGeometry instead")
		CHAOS_API TUniquePtr<FTriangleMeshImplicitObject> CopySlow() const;

		void SerializeImp(FChaosArchive& Ar)
		{
			Ar.UsingCustomVersion(FExternalPhysicsCustomObjectVersion::GUID);
			Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

			FImplicitObject::SerializeImp(Ar);
			Ar << MParticles;
			Ar << MElements;
			TBox<FReal, 3>::SerializeAsAABB(Ar, MLocalBoundingBox);

			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::RemovedConvexHullsFromTriangleMeshImplicitObject)
			{
				TUniquePtr<TGeometryParticles<FReal, 3>> ConvexHulls;
				Ar << ConvexHulls;
			}

			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::TrimeshSerializesBV)
			{
				// Should now only hit when loading older trimeshes
				RebuildFastBVH();
			}
			else if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) < FExternalPhysicsCustomObjectVersion::TrimeshSerializesAABBTree)
			{
				TBoundingVolume<int32> Dummy;
				Ar << Dummy;
				RebuildFastBVH();
			}
			else if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::UseTriangleMeshBVH)
			{
				// Serialize acceleration
				BVHType BVH;
				Ar << BVH;
				if (BVH.GetNodes().IsEmpty())
				{
					RebuildFastBVH();
				}
				else
				{
					RebuildFastBVHFromTree(BVH);
				}
			}
			else
			{
				Ar << FastBVH;
			}

			if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::RemoveTriangleMeshBVHFaces)
			{
				// Force to rebuild the BVH
				RebuildFastBVH();
			}

			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::AddTrimeshMaterialIndices)
			{
				Ar << MaterialIndices;
			}

			if (Ar.CustomVer(FExternalPhysicsCustomObjectVersion::GUID) >= FExternalPhysicsCustomObjectVersion::TriangleMeshHasFaceIndexMap)
			{
				// TODO: This data is only needed in editor unless project configuration requests this for gameplay. We should not serialize this when cooking
				// unless it is required for gameplay, as we are wasting disk space.
				if (Ar.IsLoading())
				{
					TUniquePtr<TArray<int32>> ExternalFaceIndexMapTemp = MakeUnique<TArray<int32>>(TArray<int32>());
					Ar << *ExternalFaceIndexMapTemp;
					if (!ExternalFaceIndexMapTemp->IsEmpty())
					{
						ExternalFaceIndexMap = MoveTemp(ExternalFaceIndexMapTemp);
					}
				}
				else
				{
					if (ExternalFaceIndexMap == nullptr)
					{
						TArray<int32> EmptyArray;
						Ar << EmptyArray;
					}
					else
					{
						Ar << *ExternalFaceIndexMap;
					}
				}
			}

			Ar.UsingCustomVersion(FPhysicsObjectVersion::GUID);
			if (Ar.CustomVer(FPhysicsObjectVersion::GUID) >= FPhysicsObjectVersion::TriangleMeshHasVertexIndexMap)
			{
				if (Ar.IsLoading())
				{
					ExternalVertexIndexMap = MakeUnique<TArray<int32>>(TArray<int32>());
					Ar << *ExternalVertexIndexMap;
				}
				else
				{
					if (ExternalVertexIndexMap == nullptr)
					{
						TArray<int32> EmptyArray;
						Ar << EmptyArray;
					}
					else
					{
						Ar << *ExternalVertexIndexMap;
					}
				}
			}

			// Quantized bounds are not serialized; we recompute them on load
			if (Ar.IsLoading() && FastBVH.NeedsQuantizedBounds())
			{
				MeshSettingsFlags |= ETriangleMeshSettingsFlags::QuantizePerFaceBounds;
				FastBVH.InitializeQuantizedBounds(MParticles, MElements);
			}
		}

		CHAOS_API virtual void Serialize(FChaosArchive& Ar) override;

		virtual FString ToString() const
		{
			return FString::Printf(TEXT("TriangleMesh"));
		}

		CHAOS_API virtual uint32 GetTypeHash() const override;

		CHAOS_API FVec3 GetFaceNormal(const int32 FaceIdx) const;

		CHAOS_API virtual uint16 GetMaterialIndex(uint32 HintIndex) const override;

		CHAOS_API const ParticlesType& Particles() const;
		CHAOS_API const FTrimeshIndexBuffer& Elements() const;

		CHAOS_API void UpdateVertices(const TArray<FVector>& Positions);

		/**
		 * @brief Generate the triangle at the specified index with the specified transform (including scale)
		 * @note does not correct winding for negative scales
		*/
		void GetTransformedTriangle(const int32 TriangleIndex, const FRigidTransform3& Transform, FTriangle& OutTriangle, int32& OutVertexIndex0, int32& OutVertexIndex1, int32& OutVertexIndex2) const
		{
			if (MElements.RequiresLargeIndices())
			{
				TriangleMeshTransformVertsHelper(Transform, TriangleIndex, MParticles, MElements.GetLargeIndexBuffer(), OutTriangle[0], OutTriangle[1], OutTriangle[2], OutVertexIndex0, OutVertexIndex1, OutVertexIndex2);
			}
			else
			{
				TriangleMeshTransformVertsHelper(Transform, TriangleIndex, MParticles, MElements.GetSmallIndexBuffer(), OutTriangle[0], OutTriangle[1], OutTriangle[2], OutVertexIndex0, OutVertexIndex1, OutVertexIndex2);
			}
		}

		/**
		 * @brief Get a list of triangle indices that overlap the query bounds
		 * @param QueryBounds query bounds in trimesh space
		*/
		void FindOverlappingTriangles(const FAABB3& QueryBounds, TArray<int32>& OutTriangleIndices) const
		{
			OutTriangleIndices = FastBVH.FindAllIntersections(QueryBounds);
		}

		/**
		 * @param QueryBounds Bounding box in which we want to produce triangles, in unscaled TriMesh space
		 * @param QueryTransform Transforms from TriMesh space to desired space (e.g., could be trimesh world transform including scale to get world-space triangles)
		 * @param CullDistance The distance at which we can ignore triangles
		 * @param Visitor void(const FTriangle& Triangle, const int32 TriangleIndex, const int32 VertexIndex0, const int32 VertexIndex1, const int32 OutVertexIndex2)
		*/
		template<typename TriangleVisitor>
		void VisitTriangles(const FAABB3& QueryBounds, const FRigidTransform3& QueryTransform, const TriangleVisitor& Visitor) const
		{
			TArray<int32> OverlapIndices;
			FindOverlappingTriangles(QueryBounds, OverlapIndices);

			const bool bStandardWinding = ((QueryTransform.GetScale3D().X * QueryTransform.GetScale3D().Y * QueryTransform.GetScale3D().Z) >= FReal(0));

			for (int32 OverlapIndex = 0; OverlapIndex < OverlapIndices.Num(); ++OverlapIndex)
			{
				const int32 TriangleIndex = OverlapIndices[OverlapIndex];
				FTriangle Triangle;
				int32 VertexIndex0, VertexIndex1, VertexIndex2;
				GetTransformedTriangle(TriangleIndex, QueryTransform, Triangle, VertexIndex0, VertexIndex1, VertexIndex2);

				if (!bStandardWinding)
				{
					Triangle.ReverseWinding();
					Swap(VertexIndex1, VertexIndex2);
				}

				Visitor(Triangle, TriangleIndex, VertexIndex0, VertexIndex1, VertexIndex2);
			}
		}

		// Internal: do not use - this API will change as we optimize mesh collision
		void CollectTriangles(const FAABB3& MeshQueryBounds, const FRigidTransform3& MeshToObjectTransform, const FAABB3& ObjectBounds, Private::FMeshContactGenerator& Collector) const;

		// Internal: Only for CVD
		UE_INTERNAL CHAOS_API const FTrimeshBVH& GetBVH() const;

	private:
		using BVHType = TAABBTree<int32, TAABBTreeLeafArray<int32, /*bComputeBounds=*/ false, FRealSingle>, /*bMutable=*/false, FRealSingle>;
		CHAOS_API void RebuildFastBVHFromTree(const BVHType& BVH);
		CHAOS_API void RebuildFastBVH();

		ParticlesType MParticles;
		FTrimeshIndexBuffer MElements;
		FAABB3 MLocalBoundingBox;
		TArray<uint16> MaterialIndices;
		TUniquePtr<TArray<int32>> ExternalFaceIndexMap;
		TUniquePtr<TArray<int32>> ExternalVertexIndexMap;
		ETriangleMeshSettingsFlags MeshSettingsFlags;

		// Initialising constructor privately declared for use in CopySlow to copy the underlying BVH
		template <typename IdxType>
		FTriangleMeshImplicitObject(ParticlesType&& Particles, TArray<TVec3<IdxType>>&& Elements, TArray<uint16>&& InMaterialIndices, const BVHType& InBvhToCopy,
			TUniquePtr<TArray<int32>>&& InExternalFaceIndexMap = nullptr, TUniquePtr<TArray<int32>>&& InExternalVertexIndexMap = nullptr, 
			const bool bInCullsBackFaceRaycast = false,
			const FBuildSettings InBuildSettings = FBuildSettings())
			: FImplicitObject(EImplicitObject::HasBoundingBox | EImplicitObject::DisableCollisions, ImplicitObjectType::TriangleMesh)
			, MParticles(MoveTemp(Particles))
			, MElements(MoveTemp(Elements))
			, MLocalBoundingBox(MParticles.GetX(0), MParticles.GetX(0))
			, MaterialIndices(MoveTemp(InMaterialIndices))
			, ExternalFaceIndexMap(MoveTemp(InExternalFaceIndexMap))
			, ExternalVertexIndexMap(MoveTemp(InExternalVertexIndexMap))
			, MeshSettingsFlags(PackSettingsFlags(bInCullsBackFaceRaycast, InBuildSettings))
		{
			const int32 NumTriangles = MElements.GetNumTriangles();
			if(NumTriangles > 0)
			{
				const TArray<TVec3<IdxType>>& Tris = MElements.GetIndexBuffer<IdxType>();
				const TVec3<IdxType>& FirstTri = Tris[0];

				MLocalBoundingBox = FAABB3(MParticles.GetX(FirstTri[0]), MParticles.GetX(FirstTri[0]));
				MLocalBoundingBox.GrowToInclude(MParticles.GetX(FirstTri[1]));
				MLocalBoundingBox.GrowToInclude(MParticles.GetX(FirstTri[2]));

				for(int32 TriangleIndex = 1; TriangleIndex < NumTriangles; ++TriangleIndex)
				{
					const TVec3<IdxType>& Tri = Tris[TriangleIndex];
					MLocalBoundingBox.GrowToInclude(MParticles.GetX(Tri[0]));
					MLocalBoundingBox.GrowToInclude(MParticles.GetX(Tri[1]));
					MLocalBoundingBox.GrowToInclude(MParticles.GetX(Tri[2]));
				}
			}
			
			RebuildFastBVHFromTree(InBvhToCopy);
		}

		template<typename InStorageType, typename InRealType>
		friend struct FBvEntry;
		template <typename QueryGeomType> 
		friend struct FTriangleMeshOverlapVisitorNoMTD;

		template<bool bRequiresLargeIndex>
		struct FBvEntry
		{
			FTriangleMeshImplicitObject* TmData;
			int32 Index;

			bool HasBoundingBox() const { return true; }

			TAABB<FRealSingle, 3> BoundingBox() const
			{
				auto LambdaHelper = [&](const auto& Elements)
				{
					TAABB<FRealSingle,3> Bounds(TmData->MParticles.GetX(Elements[Index][0]), TmData->MParticles.GetX(Elements[Index][0]));

					Bounds.GrowToInclude(TmData->MParticles.GetX(Elements[Index][1]));
					Bounds.GrowToInclude(TmData->MParticles.GetX(Elements[Index][2]));

					return Bounds;
				};

				if(bRequiresLargeIndex)
				{
					return LambdaHelper(TmData->MElements.GetLargeIndexBuffer());
				}
				else
				{
					return LambdaHelper(TmData->MElements.GetSmallIndexBuffer());
				}
			}

			template<typename TPayloadType>
			int32 GetPayload(int32 Idx) const
			{
				return Idx;
			}

			FUniqueIdx UniqueIdx() const
			{
				return FUniqueIdx(Index);
			}
		};

		FTrimeshBVH FastBVH;

		template<typename Geom, typename IdxType>
		friend struct FTriangleMeshSweepVisitor;

		template<typename Geom, typename IdxType>
		friend struct FTriangleMeshSweepVisitorCCD;

		// Required by implicit object serialization, disabled for general use.
		friend class FImplicitObject;
		friend class UE::Physics::FTriangleMeshGeometry;

		FTriangleMeshImplicitObject()
		    : FImplicitObject(EImplicitObject::HasBoundingBox, ImplicitObjectType::TriangleMesh){};

		template <typename QueryGeomType>
		bool GJKContactPointImp(const QueryGeomType& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FVec3& Location, FVec3& Normal, FReal& Penetration, int32& FaceIndex, FVec3 TriMeshScale = FVec3(1.0)) const;

		template<bool IsSpherical, typename QueryGeomType>
		bool OverlapGeomImp(const QueryGeomType& QueryGeom, const FRigidTransform3& QueryTM, const FReal Thickness, FMTDInfo* OutMTD = nullptr, FVec3 TriMeshScale = FVec3(1.0f)) const;

		template<typename QueryGeomType>
		bool SweepGeomImp(const QueryGeomType& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FReal Thickness, const bool bComputeMTD, FVec3 TriMeshScale = FVec3(1.0f), bool bOutputIncludesScale = false) const;

		template <typename QueryGeomType>
		bool SweepGeomCCDImp(const QueryGeomType& QueryGeom, const FRigidTransform3& StartTM, const FVec3& Dir, const FReal Length, const FReal IgnorePenetration, const FReal TargetPenetration, FReal& OutTOI, FReal& OutPhi, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex, FVec3& OutFaceNormal, const FVec3& TriMeshScale, bool bOutputIncludesScale) const;

		template <typename IdxType>
		bool RaycastImp(const TArray<TVec3<IdxType>>& Elements, const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const;

		template <typename IdxType>
		bool OverlapImp(const TArray<TVec3<IdxType>>& Elements, const FVec3& Point, const FReal Thickness) const;

		template<typename IdxType>
		int32 FindMostOpposingFace(const TArray<TVec3<IdxType>>& Elements, const FVec3& Position, const FVec3& UnitDir, int32 HintFaceIndex, FReal SearchDist, const FVec3& Scale) const;

		template <typename IdxType>
		void RebuildBVImp(const TArray<TVec3<IdxType>>& Elements, BVHType& BVH);

		template <typename IdxType>
		FImplicitObjectPtr CopySlowImpl(const TArray < TVector<IdxType, 3>>& InElements) const;
	};

	ENUM_CLASS_FLAGS(FTriangleMeshImplicitObject::ETriangleMeshSettingsFlags);

}
