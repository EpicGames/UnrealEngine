// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "Chaos/Core.h"
#include "Math/VectorRegister.h"
#include "Chaos/AABB.h"

namespace Chaos
{
	class FTriangleRegister
	{
	public:
		FTriangleRegister()
		{
		}

		FTriangleRegister(const VectorRegister4Float& InA, const VectorRegister4Float& InB, const VectorRegister4Float& InC)
			: A(InA)
			, B(InB)
			, C(InC)
		{
		}

		FORCEINLINE VectorRegister4Float SupportCoreSimd(const VectorRegister4Float& Direction, const FReal InMargin) const
		{
			// Note: assumes margin == 0
			VectorRegister4Float DotA = VectorDot4(A, Direction);
			VectorRegister4Float DotB = VectorDot4(B, Direction);
			VectorRegister4Float DotC = VectorDot4(C, Direction);

			VectorRegister4Float DotAGEDotB = VectorCompareGE(DotA, DotB);
			VectorRegister4Float DotAGEDotC = VectorCompareGE(DotA, DotC);
			VectorRegister4Float DotAG = VectorBitwiseAnd(DotAGEDotB, DotAGEDotC);

			VectorRegister4Float DotBGEDotA = VectorCompareGE(DotB, DotA);
			VectorRegister4Float DotBGEDotC = VectorCompareGE(DotB, DotC);
			VectorRegister4Float DotBG = VectorBitwiseAnd(DotBGEDotA, DotBGEDotC);

			VectorRegister4Float Result = VectorSelect(DotAG, A, C);
			Result = VectorSelect(DotBG, B, Result);

			return Result;
		}

		FORCEINLINE FReal GetMargin() const { return 0; }
		FORCEINLINE FReal GetRadius() const { return 0; }
		FORCEINLINE FRealSingle GetMarginf() const { return 0.0f; }
		FORCEINLINE FRealSingle GetRadiusf() const { return 0.0f; }

		FORCEINLINE const FAABB3 BoundingBox() const
		{
			VectorRegister4Float MinSimd = VectorMin(VectorMin(A, B), C);
			VectorRegister4Float MaxSimd = VectorMax(VectorMax(A, B), C);
			
			alignas(32) FReal AlignedArray[4];
			VectorStoreAligned(MinSimd, AlignedArray);
			const FVec3  Min(static_cast<FReal>(AlignedArray[0]), static_cast<FReal>(AlignedArray[1]), static_cast<FReal>(AlignedArray[2]));
			VectorStoreAligned(MaxSimd, AlignedArray);
			const FVec3  Max(static_cast<FReal>(AlignedArray[0]), static_cast<FReal>(AlignedArray[1]), static_cast<FReal>(AlignedArray[2]));
			return FAABB3(Min, Max);
		}


		FORCEINLINE bool IsTooBigForSinglePrecision() const
		{
			constexpr FRealSingle SqrSideThreshold = 100000.0f*100000.0f;
			// If two sides are quite small, the third one cannot be huge
			const VectorRegister4Float BigSide = VectorMax(VectorAbs(VectorSubtract(A, B)), VectorAbs(VectorSubtract(B, C)));
			// Warning this could 
			return VectorDot3Scalar(BigSide, BigSide) > SqrSideThreshold;
		}


	private:

		VectorRegister4Float A;
		VectorRegister4Float B;
		VectorRegister4Float C;

	};

}