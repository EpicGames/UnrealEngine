// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Chaos/Convex.h"

namespace Chaos
{
namespace Private
{
		
/**
 * @brief Tribox object that represents a k-DOP18 convex
 * 
 * @note This implementation provides a fast way of constructing Tribox in order to be used at runtime to build
 * approximate convex implicit object
 * 
 */

class FTribox
{
public :

	using FRealType = FRealSingle;
	using FVec3Type = TVec3<FRealType>;
	using FRigidTransform3Type = TRigidTransform<FRealType, 3>;
	using FMatrix33Type = PMatrix<FRealType, 3, 3>;
	using FPlaneType = TPlaneConcrete<FRealType, 3>;

	// Number of planes that will be used to define the Tribox 
	static constexpr int32 NumPlanes = 18;

	// Number of principal planes
	static constexpr int32 NumPrincipalPlanes = 6;

	// Number of chamfer planes
	static constexpr int32 NumChamferPlanes = 12;

	// Distance used to inflate the Tribox to avoid degenerate case
	static constexpr FRealType InflateDistance = 0.5;

	// Base Constructor
	FORCEINLINE FTribox() : MaxDists(), bIsValid(false), bHasDatas(false)
	{
		for(int32 DistsIndex = 0; DistsIndex< NumPlanes; ++DistsIndex)
		{
			MaxDists[DistsIndex] = TNumericLimits<FRealType>::Lowest();
		}
	};

	// Base Constructor
	FORCEINLINE FTribox(const FTribox& OtherTribox) : MaxDists(), bIsValid(false), bHasDatas(false)
	{
		for(int32 DistsIndex = 0; DistsIndex < NumPlanes; ++DistsIndex)
		{
			MaxDists[DistsIndex] = OtherTribox.MaxDists[DistsIndex];
		}
		bIsValid = OtherTribox.bIsValid;
		bHasDatas = OtherTribox.bHasDatas;
	};
	
	// Get the tribox center
	FVec3Type GetCenter() const;

	// Get the bounding box
	FAABB3 GetBounds() const;
	
	// Get the closest plane along the +X,-X,+Y,-Y,+Z,-Z directions
	FRealType GetClosestPlane(const FVec3Type& PointPosition, int32& PlaneAxis, FRealType& PlaneProjection) const;

	// Add a point position to the Tribox
	void AddPoint(const FVec3Type& PointPosition);

	// Add convex vertices to a tribox
	void AddConvex(const FConvex* Convex, const FRigidTransform3Type& RelativeTransform);

	// Inflate + Scale the Max distances
	bool BuildTribox();

	// Find the overlapping tribox 
	bool OverlapTribox(const FTribox& OtherTribox, FTribox& OverlapTribox) const;

	// Check if the tribox is overlapping or not
	bool IsTriboxOverlapping(const FTribox& OtherTribox) const;

	// Split the tribox in 2 along a defined cuttng plane
	bool SplitTriboxSlab(const int32 PlaneAxis, const FRealType& PlaneDistance,
					FTribox& LeftTribox, FTribox& RightTribox) const;

	// Get the thickest tribox slab
	int32 GetThickestSlab() const;

	// Sample a point along the plane direction in betwen min and max
	FRealType SampleSlabPoint(const int32 PlaneAxis, const FRealType& LocalDistance) const;

	// Compute the tribox volume
	FRealType ComputeVolume() const;

	// Create a convex from the Tribox
	FImplicitObjectPtr MakeConvex() const;

	// Add a tribox to this and return this
	FTribox& operator+=(const FTribox& OtherTribox);

	// Add a tribox to this and return a new one
	FTribox operator+( const FTribox& OtherTribox) const;

	// Check ihe tribox is valid
	bool IsValid() const {return bIsValid;}

	// Set the valid flag
	void SetValid(const bool bValid) {bIsValid = bValid;}

	// Check ihe tribox have been built with datas
	bool HasDatas() const {return bHasDatas;}

	// Reset the tribox max distances
	void ResetDists()
	{
		for(int32 DistsIndex = 0; DistsIndex< NumPlanes; ++DistsIndex)
		{
			MaxDists[DistsIndex] = TNumericLimits<FRealType>::Lowest();
		}
	}

private : 
	// Solve the intersection point position
	FVec3 SolveIntersection(const int32 FaceIndex[3], const FMatrix33Type& A) const;

	// Add the intersection point to the list of faces/vertices
	void AddIntersection(const int32 FaceIndex[3], const int32 FaceOrder[3], const FVec3Type& VertexPosition, TArray<TArray<int32>>& FaceIndices, TArray<FConvex::FVec3Type>& ConvexVertices) const;

	// Solve and add the intersection point in between the 3 planes
	void ComputeIntersection(const int32 CornerIndex, const int32 PrincipalIndex, const int32 ChamferIndex, TArray<TArray<int32>>& FaceIndices, TArray<FConvex::FVec3Type>& ConvexVertices) const;

	// Build the max dist along the principal axis
	void BuildPrincipalDist(const FVec3Type& P, const int32 CoordIndexA, const int32 DistsIndexA, const int32 DistsIndexB);

	// Build the max dist along the chamfer axis
	void BuildChamferDist(const FVec3Type& P, const int32 CoordIndexA, const int32 CoordIndexB,
			const int32 DistsIndexA, const int32 DistsIndexB, const int32 DistsIndexC, const int32 DistsIndexD);

	// Create a FConvex from a list of convex planes, face indices and convex vertices
	FImplicitObjectPtr CreateConvexFromTopology(TArray<FConvex::FPlaneType>&& ConvexPlanes,
		TArray<TArray<int32>>&& FaceIndices, TArray<FConvex::FVec3Type>&& ConvexVertices) const;

	// Max distance along eaxh tribox axis (principal + chamfer)
	FRealType MaxDists[NumPlanes] = {TNumericLimits<FRealType>::Lowest(), TNumericLimits<FRealType>::Lowest(),
									 TNumericLimits<FRealType>::Lowest(), TNumericLimits<FRealType>::Lowest(),
									 TNumericLimits<FRealType>::Lowest(), TNumericLimits<FRealType>::Lowest(),
									 TNumericLimits<FRealType>::Lowest(), TNumericLimits<FRealType>::Lowest(),
									 TNumericLimits<FRealType>::Lowest()};

	// Boolean to specify if the tribox is valid or not 
	bool bIsValid = false;

	// Boolean to specify if the tribox has been built with datas or not
	bool bHasDatas = false;
};

}
}