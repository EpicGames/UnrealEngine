// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Vector.h"
#include "ChaosCheck.h"
#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/DynamicParticles.h"

namespace Chaos
{
template<class T, int d>
class TArrayND;
template<class T, int d>
class TArrayFaceND;

template<typename T>
struct TGridPrecisionLimit
{
	static_assert(sizeof(T) == 0, "Unsupported grid floating point type");
};

// Precision limits where our floating point precision breaks down to 1 (each value above this adds more than 1 to the number)
// Note the F32 limit is actually higher than this point (should be 8.388e6) but kept at 1e7 for legacy reasons
template<> struct TGridPrecisionLimit<FRealSingle> { static constexpr FRealSingle value = 1e7; };
template<> struct TGridPrecisionLimit<FRealDouble> { static constexpr FRealDouble value = 4.5035e15; };

template<class T, int d>
class TUniformGridBase
{
  protected:
	TUniformGridBase() {}
	TUniformGridBase(const TVector<T, d>& MinCorner, const TVector<T, d>& MaxCorner, const TVector<int32, d>& Cells, const uint32 GhostCells)
	    : MMinCorner(MinCorner), MMaxCorner(MaxCorner), MCells(Cells)
	{
		for (int32 Axis = 0; Axis < d; ++Axis)
		{
			check(MCells[Axis] != 0);
		}

		// Are corners valid?
		bool bValidBounds = true;
		if (MMaxCorner == TVector<T, d>(-TNumericLimits<T>::Max()) && MMinCorner == TVector<T, d>(TNumericLimits<T>::Max()))
		{
			// This is an Empty AABB
			bValidBounds = false;
		}
		else
		{
			for (int32 i = 0; i < d; ++i)
			{
				// This is invalid
				if (!CHAOS_ENSURE(MMaxCorner[i] >= MMinCorner[i])) //TODO convert back to normal ensure once we find out why we hit this.
				{
					bValidBounds = false;
					break;
				}
			}
		}

		if(bValidBounds)
		{
			MDx = TVector<T, d>(MMaxCorner - MMinCorner) / MCells;
		}
		else
		{
			MDx = TVector<T,d>(0);
			MMaxCorner = TVector<T, d>(0);
			MMinCorner = TVector<T, d>(0);
		}

		if (GhostCells > 0)
		{
			MMinCorner -= MDx * static_cast<T>(GhostCells);
			MMaxCorner += MDx * static_cast<T>(GhostCells);
			MCells += TVector<T, d>(2 * static_cast<T>(GhostCells));
		}

		if (MDx.Min() >= UE_SMALL_NUMBER)
		{
			const TVector<T, d> MinToDXRatio = MMinCorner / MDx;
			for (int32 Axis = 0; Axis < d; ++Axis)
			{
				ensure(FMath::Abs(MinToDXRatio[Axis]) < TGridPrecisionLimit<T>::value); //make sure we have the precision we need
			}
		}
	}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
	TUniformGridBase(std::istream& Stream)
	    : MMinCorner(Stream), MMaxCorner(Stream), MCells(Stream)
	{
		MDx = TVector<T, d>(MMaxCorner - MMinCorner) / MCells;
	}
#endif
	~TUniformGridBase() = default;

  public:
#if COMPILE_WITHOUT_UNREAL_SUPPORT
	void Write(std::ostream& Stream) const
	{
		MMinCorner.Write(Stream);
		MMaxCorner.Write(Stream);
		MCells.Write(Stream);
	}
#endif
	void Serialize(FArchive& Ar)
	{
		Ar << MMinCorner;
		Ar << MMaxCorner;
		Ar << MCells;
		Ar << MDx;
	}

	TVector<T, d> Location(const TVector<int32, d>& Cell) const
	{
		return MDx * Cell + MMinCorner + (MDx / 2);
	}
	TVector<T, d> Location(const Pair<int32, TVector<int32, 3>>& Face) const
	{
		return MDx * Face.Second + MMinCorner + (TVector<T, d>(1) - TVector<T, d>::AxisVector(Face.First)) * (MDx / 2);
	}

	void Reset()
	{
		MMinCorner = TVector<T, d>(0);
		MMaxCorner = TVector<T, d>(0);
		MCells = TVector<int32, d>(0);
		MDx = TVector<T, d>(0);
	}

#ifdef PLATFORM_COMPILER_CLANG
	// Disable optimization (-ffast-math) since its currently causing regressions.
	//		freciprocal-math:
	//		x / y = x * rccps(y) 
	//		rcpps is faster but less accurate (12 bits of precision), this can causes incorrect CellIdx
	DISABLE_FUNCTION_OPTIMIZATION 
#endif
	TVector<int32, d> CellUnsafe(const TVector<T, d>& X) const
	{
		const TVector<T, d> Delta = X - MMinCorner;
		TVector<int32, d> Result = Delta / MDx;
		for (int Axis = 0; Axis < d; ++Axis)
		{
			if (Delta[Axis] < 0)
			{
				Result[Axis] -= 1;	//negative snaps to the right which is wrong. Consider -50 x for DX of 100: -50 / 100 = 0 but we actually want -1
			}
		}
		return Result;
	} 
#ifdef PLATFORM_COMPILER_CLANG
	// Disable optimization (-ffast-math) since its currently causing regressions.
	//		freciprocal-math:
	//		x / y = x * rccps(y) 
	//		rcpps is faster but less accurate (12 bits of precision), this can causes incorrect CellIdx
	DISABLE_FUNCTION_OPTIMIZATION
#endif
	TVector<int32, d> Cell(const TVector<T, d>& X) const
	{
		const TVector<T, d> Delta = X - MMinCorner;
		TVector<int32, d> Result = Delta / MDx;
		for (int Axis = 0; Axis < d; ++Axis)
		{
			Result[Axis] = Result[Axis] >= MCells[Axis] ? MCells[Axis] - 1 : (Result[Axis] < 0 ? 0 : Result[Axis]);
		}
		return Result;
	}

	TVector<int32, d> Face(const TVector<T, d>& X, const int32 Component) const
	{
		return Cell(X + (MDx / 2) * TVector<T, d>::AxisVector(Component));
	}
	TVector<T, d> DomainSize() const
	{
		return (MMaxCorner - MMinCorner);
	}

	int32 GetNumCells() const
	{
		return MCells.Product();
	}

	int32 GetNumNodes() const
	{
		return (MCells + TVector<int32, 3>(1, 1, 1)).Product();
	}

	TVector<T, d> Node(const TVector<int32, d>& Index) const
	{
		TVector<T, d> Tmp = MMinCorner;
		for (int32 i = 0; i < d; i++)
			Tmp[i] += MDx[i] * Index[i];
		return Tmp;
	}

	TVector<T, d> Node(const int32 FlatIndex) const
	{
		TVector<int32, d> Index; 
		FlatToMultiIndex(FlatIndex, Index, true);
		return Node(Index);
	}

	bool InteriorNode(const TVector<int32, d>& Index) const
	{
		bool Interior = true;
		for (int32 i = 0; i < d && Interior; i++)
		{
			if (Index[i] <= 0 || Index[i] >= (MCells[i] - 1))
				Interior = false;
		}
		return Interior;
	}

	int32 FlatIndex(const TVector<int32, 2>& MIndex, const bool NodeIndex=false) const
	{
		return MIndex[0] * (NodeIndex ? MCells[1]+1 : MCells[1]) + MIndex[1];
	}

	int32 FlatIndex(const TVector<int32, 3>& MIndex, const bool NodeIndex=false) const
	{
		return NodeIndex ?
			MIndex[0] * (MCells[1]+1) * (MCells[2]+1) + MIndex[1] * (MCells[2]+1) + MIndex[2] :
			MIndex[0] * MCells[1] * MCells[2] + MIndex[1] * MCells[2] + MIndex[2];
	}

	void FlatToMultiIndex(const int32 FlatIndex, TVector<int32, 2>& MIndex, const bool NodeIndex=false) const
	{
		MIndex[0] = FlatIndex / (NodeIndex?MCells[1]+1:MCells[1]);
		MIndex[1] = FlatIndex % (NodeIndex?MCells[1]+1: MCells[1]);
	}

	void FlatToMultiIndex(const int32 FlatIndex, TVector<int32, 3>& MIndex, const bool NodeIndex=false) const
	{
		if (NodeIndex)
		{
			MIndex[0] = FlatIndex / ((MCells[1]+1) * (MCells[2]+1));
			MIndex[1] = (FlatIndex / (MCells[2]+1)) % (MCells[1]+1);
			MIndex[2] = FlatIndex % (MCells[2]+1);
		}
		else
		{
			MIndex[0] = FlatIndex / (MCells[1] * MCells[2]);
			MIndex[1] = (FlatIndex / MCells[2]) % MCells[1];
			MIndex[2] = FlatIndex % MCells[2];
		}
	}

	template<class T_SCALAR>
	T_SCALAR LinearlyInterpolate(const TArrayND<T_SCALAR, d>& ScalarN, const TVector<T, d>& X) const;
	T LinearlyInterpolateComponent(const TArrayND<T, d>& ScalarNComponent, const TVector<T, d>& X, const int32 Axis) const;
	TVector<T, d> LinearlyInterpolate(const TArrayFaceND<T, d>& ScalarN, const TVector<T, d>& X) const;
	TVector<T, d> LinearlyInterpolate(const TArrayFaceND<T, d>& ScalarN, const TVector<T, d>& X, const Pair<int32, TVector<int32, d>> Index) const;
	const TVector<int32, d>& Counts() const { return MCells; }
	const TVector<int32, d> NodeCounts() const { return MCells + TVector<int32, d>(1); }
	const TVector<T, d>& Dx() const { return MDx; }
	const TVector<T, d>& MinCorner() const { return MMinCorner; }
	const TVector<T, d>& MaxCorner() const { return MMaxCorner; }

  protected:
	TVector<T, d> MMinCorner;
	TVector<T, d> MMaxCorner;
	TVector<int32, d> MCells;
	TVector<T, d> MDx;
};

template<class T, int d>
class TUniformGrid : public TUniformGridBase<T, d>
{
	using TUniformGridBase<T, d>::MCells;
	using TUniformGridBase<T, d>::MMinCorner;
	using TUniformGridBase<T, d>::MMaxCorner;
	using TUniformGridBase<T, d>::MDx;

  public:
	using TUniformGridBase<T, d>::Location;

	TUniformGrid() {}
	TUniformGrid(const TVector<T, d>& MinCorner, const TVector<T, d>& MaxCorner, const TVector<int32, d>& Cells, const uint32 GhostCells = 0)
	    : TUniformGridBase<T, d>(MinCorner, MaxCorner, Cells, GhostCells) {}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
	TUniformGrid(std::istream& Stream)
	    : TUniformGridBase<T, d>(Stream) {}
#endif
	~TUniformGrid() = default;
	TVector<int32, d> GetIndex(const int32 Index) const;
	TVector<T, d> Center(const int32 Index) const
	{
		return TUniformGridBase<T, d>::Location(GetIndex(Index));
	}
	TVector<int32, d> ClampIndex(const TVector<int32, d>& Index) const
	{
		TVector<int32, d> Result;
		for (int32 i = 0; i < d; ++i)
		{
			if (Index[i] >= MCells[i])
				Result[i] = MCells[i] - 1;
			else if (Index[i] < 0)
				Result[i] = 0;
			else
				Result[i] = Index[i];
		}
		return Result;
	}

	TVector<T, d> Clamp(const TVector<T, d>& X) const;
	TVector<T, d> ClampMinusHalf(const TVector<T, d>& X) const;
	
	bool IsValid(const TVector<int32, d>& X) const
	{
		return X == ClampIndex(X);
	}
};

template<class T>
class TMPMGrid : public TUniformGridBase<T, 3>
{
	using TUniformGridBase<T, 3>::MCells;
	using TUniformGridBase<T, 3>::MMinCorner;
	using TUniformGridBase<T, 3>::MMaxCorner;
	using TUniformGridBase<T, 3>::MDx;

public:
	using TUniformGridBase<T, 3>::GetNumCells;
	using TUniformGridBase<T, 3>::Location;

	TMPMGrid() {}
	TMPMGrid(const TVector<T, 3>& MinCorner, const TVector<T, 3>& MaxCorner, const TVector<int32, 3>& Cells, const uint32 GhostCells = 0)
		: TUniformGridBase<T, 3>(MinCorner, MaxCorner, Cells, GhostCells) {}
	TMPMGrid(const int32 GridN) { for (int32 i = 0; i < 3; i++) { MDx[i] = (T)1. / (T)GridN; } }
	TMPMGrid(const T GridDx) { for (int32 i = 0; i < 3; i++) { MDx[i] = GridDx; } }
#if COMPILE_WITHOUT_UNREAL_SUPPORT
	TMPMGrid(std::istream& Stream)
		: TUniformGridBase<T, 3>(Stream) {}
#endif
	~TMPMGrid() = default;

	CHAOS_API void BaseNodeIndex(const TVector<T, 3>& X, TVector<int32, 3>& Index, TVector<T, 3>& weights) const;
	
	inline T Nijk(T w, int32 ii) const {
		if (interp == linear)
			return T(1) - w + T(ii) * (T(2) * w - T(1));
		else
			return ((T(1.5) * (w * w - w) - T(0.25)) * (T(ii) - 1) + (T(0.5) * w - T(0.25))) * (T(ii) - 1) - w * w + w + T(0.5);
	}

	inline void GradNi(const TVector<T, 3>& Ni, const TVector<T, 3>& dNi, TVector<T, 3>& result) const 
	{
		result[0] = dNi[0] * Ni[1] * Ni[2];
		result[1] = Ni[0] * dNi[1] * Ni[2];
		result[2] = Ni[0] * Ni[1] * dNi[2];
	}

	inline T dNijk(T w, int32 ii, T dx) const {
		if (interp == linear)
			return (T(2) * T(ii) - T(1)) / dx;
		else
			return ((w - 1) * (T(ii) - 1) * (T(ii) - 2) * T(0.5) + (2 * w - 1) * T(ii) * (T(ii) - 2) + w * T(ii) * (T(ii) - 1) * T(0.5)) / dx;
	}

	inline TVector<int32, 3> Loc2GlobIndex(const TVector<int32, 3>& IndexIn, const TVector<int32, 3>& LocalIndexIn) const {
		TVector<int32, 3> Result;
		if (interp == linear) {
			for (uint32 i = 0; i < 3; ++i)
				Result[i] = IndexIn[i] + LocalIndexIn[i];
		}
		else {
			for (uint32 i = 0; i < 3; ++i)
				Result[i] = IndexIn[i] + LocalIndexIn[i] - 1;
		}
		return Result;
	}

	inline int32 Size() const {
		if (interp == linear)
			return (MCells[0] * MCells[1] * MCells[2]);
		else
			return ((MCells[0] + 1) * (MCells[1] + 1) * (MCells[2] + 1));
	}

	int32 Loc2GlobIndex(const int32 IndexIn, const TVector<int32, 3>& LocalIndexIn) const;

	inline TVector<T, 3> Node(const TVector<int32, 3>& IndexIn) const {
		TVector<T, 3> Result((T)0.);

		if (interp == linear) {
			for (int32 i = 0; i < 3; ++i) {
				Result[i] = T(IndexIn[i]) * MDx[i] + MMinCorner[i];
			}
		}
		else {
			for (size_t i = 0; i < 3; ++i) {
				Result[i] = (T(IndexIn[i]) + T(0.5)) * MDx[i] + MMinCorner[i];
			}
		}

		return Result;
	}

	CHAOS_API TVector<T, 3> Node(int32 FlatIndexIn) const;

	CHAOS_API int32 FlatIndex(const TVector<int32, 3>& Index) const;
	TVector<int32, 3> Lin2MultiIndex(const int32 IndexIn) const;
	const TVector<int32, 3> GetCells() const { return MCells; }
	const TVector<T, 3> GetMinCorner() const { return MMinCorner; } 
	const TVector<T, 3>& GetDx() const { return MDx; }
	void SetDx(const TVector<T, 3> DxIn) { MDx = DxIn; }
	CHAOS_API void UpdateGridFromPositions(const Chaos::TDynamicParticles<T, 3>& InParticles);
	
	enum InterpType { linear = 1, quadratic = 2 };
	void SetInterp(InterpType InterpIn);

	InterpType interp = linear;
	uint32 NPerDir = 2;

};


template<class T>
class TUniformGrid<T, 3> : public TUniformGridBase<T, 3>
{
	using TUniformGridBase<T, 3>::MCells;
	using TUniformGridBase<T, 3>::MMinCorner;
	using TUniformGridBase<T, 3>::MMaxCorner;
	using TUniformGridBase<T, 3>::MDx;

  public:
	using TUniformGridBase<T, 3>::GetNumCells;
	using TUniformGridBase<T, 3>::Location;
	using TUniformGridBase<T, 3>::Node;

	TUniformGrid() {}
	TUniformGrid(const TVector<T, 3>& MinCorner, const TVector<T, 3>& MaxCorner, const TVector<int32, 3>& Cells, const uint32 GhostCells = 0)
	    : TUniformGridBase<T, 3>(MinCorner, MaxCorner, Cells, GhostCells) {}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
	TUniformGrid(std::istream& Stream)
	    : TUniformGridBase<T, 3>(Stream) {}
#endif
	~TUniformGrid() = default;
	CHAOS_API TVector<int32, 3> GetIndex(const int32 Index) const;
	CHAOS_API Pair<int32, TVector<int32, 3>> GetFaceIndex(int32 Index) const;
	int32 GetNumFaces() const
	{
		return GetNumCells() * 3 + MCells[0] * MCells[1] + MCells[1] * MCells[2] + MCells[0] * MCells[3];
	}
	TVector<T, 3> Center(const int32 Index) const
	{
		return TUniformGridBase<T, 3>::Location(GetIndex(Index));
	}
	CHAOS_API TVector<int32, 3> ClampIndex(const TVector<int32, 3>& Index) const;
	CHAOS_API TVector<T, 3> Clamp(const TVector<T, 3>& X) const;
	CHAOS_API TVector<T, 3> ClampMinusHalf(const TVector<T, 3>& X) const;
	CHAOS_API bool IsValid(const TVector<int32, 3>& X) const;

	CHAOS_API TUniformGrid<T, 3> SubGrid(const TVector<int32, 3>& MinCell, const TVector<int32, 3>& MaxCell) const;
};


template <typename T, int d>
FArchive& operator<<(FArchive& Ar, TUniformGridBase<T, d>& Value)
{
	Value.Serialize(Ar);
	return Ar;
}

#if PLATFORM_MAC || PLATFORM_LINUX
extern template class CHAOS_API Chaos::TUniformGridBase<Chaos::FReal, 3>;
extern template class CHAOS_API Chaos::TUniformGrid<Chaos::FReal, 3>;
extern template class CHAOS_API Chaos::TMPMGrid<Chaos::FReal>;
extern template class CHAOS_API Chaos::TMPMGrid<Chaos::FRealSingle>;
extern template class CHAOS_API Chaos::TUniformGrid<Chaos::FReal, 2>;
#endif
}
