// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Chaos/Matrix.h"
#include "Chaos/Transform.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Vector.h"

#include "Math/NumericLimits.h"
#include "Templates/IsIntegral.h"

namespace Chaos
{
	namespace Utilities
	{
		// Get the signed square of the input, i.e. (A * Abs(A)) or (Sign(A) * Square(A))
		// SignedSquare(2) -> 4
		// SignedSquare(-2) -> -4
		template<typename TRealType>
		inline TRealType SignedSquare(TRealType A)
		{
			return A * FMath::Abs(A);
		}

		//! Take the factorial of \p Num, which should be of integral type.
		template<class TINT = uint64>
		TINT Factorial(TINT Num)
		{
			static_assert(TIsIntegral<TINT>::Value, "Templated type must be integral.");
			TINT Result = Num;
			while (Num > 2)
			{
				Result *= --Num;
			}
			return Result;
		}

		//! Number of ways to choose of \p R elements from a set of size \p N, with no repetitions.
		template<class TINT = uint64>
		TINT NChooseR(const TINT N, const TINT R)
		{
			static_assert(TIsIntegral<TINT>::Value, "Templated type must be integral.");
			return Factorial(N) / (Factorial(R) * Factorial(N - R));
		}

		//! Number of ways to choose of \p R elements from a set of size \p N, with repetitions.
		template<class TINT = uint64>
		TINT NPermuteR(const TINT N, const TINT R)
		{
			static_assert(TIsIntegral<TINT>::Value, "Templated type must be integral.");
			return Factorial(N) / Factorial(N - R);
		}

		//! Compute the minimum, average, and maximum values of \p Values.
		template<class T, class TARRAY = TArray<T>>
		void GetMinAvgMax(const TARRAY& Values, T& MinV, double& AvgV, T& MaxV)
		{
			MinV = TNumericLimits<T>::Max();
			MaxV = TNumericLimits<T>::Lowest();
			AvgV = 0.0;
			for (const T& V : Values)
			{
				MinV = V < MinV ? V : MinV;
				MaxV = V > MaxV ? V : MaxV;
				AvgV += V;
			}
			if (Values.Num())
			{
				AvgV /= Values.Num();
			}
			else
			{
				MinV = MaxV = 0;
			}
		}

		//! Compute the average value.
		template<class T, class TARRAY = TArray<T>>
		T GetAverage(const TARRAY& Values)
		{
			double AvgV = 0.0;
			for (const T& V : Values)
			{
				AvgV += V;
			}
			if (Values.Num())
			{
				AvgV /= Values.Num();
			}
			return static_cast<T>(AvgV);
		}

		//! Compute the variance of \p Values, given the average value of \p Avg.
		template<class T, class TARRAY = TArray<T>>
		T GetVariance(const TARRAY& Values, const T Avg)
		{
			double Variance = 0.0;
			for (const T& V : Values)
			{
				const T Deviation = V - Avg;
				Variance += Deviation * Deviation;
			}
			if (Values.Num())
			{
				Variance /= Values.Num();
			}
			return Variance;
		}

		//! Compute the variance of \p Values (computes their average on the fly).
		template<class T, class TARRAY = TArray<T>>
		T GetVariance(const TARRAY& Values)
		{
			return GetVariance(Values, GetAverage(Values));
		}

		//! Compute the standard deviation of \p Values, given the average value of \p Avg.
		template<class T, class TARRAY = TArray<T>>
		T GetStandardDeviation(const TARRAY& Values, const T Avg)
		{
			const T Variance = GetVariance(Values, Avg);
			return FMath::Sqrt(Variance);
		}

		//! Compute the standard deviation of \p Values (computes their average on the fly).
		template<class T, class TARRAY = TArray<T>>
		T GetStandardDeviation(const TARRAY& Values)
		{
			const T Variance = GetVariance(Values);
			return FMath::Sqrt(Variance);
		}

		//! Compute the standard deviation from \p Variance.
		template<class T>
		T GetStandardDeviation(const T Variance)
		{
			return FMath::Sqrt(Variance);
		}

		inline static FMatrix33 CrossProductMatrix(const FVec3& V)
		{
			return FMatrix33(
				0, -V.Z, V.Y,
				V.Z, 0, -V.X,
				-V.Y, V.X, 0);
		}

		/**
		 * Multiple two matrices: C = L.R
		 * @note This is the mathematically expected operator. FMatrix operator* calculates C = R.Transpose(L), so this is not equivalent to that.
		 */
		inline FMatrix33 Multiply(const FMatrix33& L, const FMatrix33& R)
		{
			// @todo(ccaulfield): optimize: simd

			// We want L.R (FMatrix operator* actually calculates R.(L)T; i.e., Right is on the left, and the Left is transposed on the right.)
			// NOTE: PMatrix constructor takes values in column order
			return FMatrix33(
				L.M[0][0] * R.M[0][0] + L.M[1][0] * R.M[0][1] + L.M[2][0] * R.M[0][2],	// x00
				L.M[0][0] * R.M[1][0] + L.M[1][0] * R.M[1][1] + L.M[2][0] * R.M[1][2],	// x01
				L.M[0][0] * R.M[2][0] + L.M[1][0] * R.M[2][1] + L.M[2][0] * R.M[2][2],	// x02

				L.M[0][1] * R.M[0][0] + L.M[1][1] * R.M[0][1] + L.M[2][1] * R.M[0][2],	// x10
				L.M[0][1] * R.M[1][0] + L.M[1][1] * R.M[1][1] + L.M[2][1] * R.M[1][2],	// x11
				L.M[0][1] * R.M[2][0] + L.M[1][1] * R.M[2][1] + L.M[2][1] * R.M[2][2],	// x12

				L.M[0][2] * R.M[0][0] + L.M[1][2] * R.M[0][1] + L.M[2][2] * R.M[0][2],	// x20
				L.M[0][2] * R.M[1][0] + L.M[1][2] * R.M[1][1] + L.M[2][2] * R.M[1][2],	// x21
				L.M[0][2] * R.M[2][0] + L.M[1][2] * R.M[2][1] + L.M[2][2] * R.M[2][2]	// x22
			);
		}

		inline FMatrix44 Multiply(const FMatrix44& L, const FMatrix44& R)
		{
			// @todo(ccaulfield): optimize: simd

			// We want L.R (FMatrix operator* actually calculates R.(L)T; i.e., Right is on the left, and the Left is transposed on the right.)
			// NOTE: PMatrix constructor takes values in column order
			return FMatrix44(
				L.M[0][0] * R.M[0][0] + L.M[1][0] * R.M[0][1] + L.M[2][0] * R.M[0][2] + L.M[3][0] * R.M[0][3],	// x00
				L.M[0][0] * R.M[1][0] + L.M[1][0] * R.M[1][1] + L.M[2][0] * R.M[1][2] + L.M[3][0] * R.M[1][3],	// x01
				L.M[0][0] * R.M[2][0] + L.M[1][0] * R.M[2][1] + L.M[2][0] * R.M[2][2] + L.M[3][0] * R.M[2][3],	// x02
				L.M[0][0] * R.M[3][0] + L.M[1][0] * R.M[3][1] + L.M[2][0] * R.M[3][2] + L.M[3][0] * R.M[3][3],	// x03

				L.M[0][1] * R.M[0][0] + L.M[1][1] * R.M[0][1] + L.M[2][1] * R.M[0][2] + L.M[3][1] * R.M[0][3],	// x10
				L.M[0][1] * R.M[1][0] + L.M[1][1] * R.M[1][1] + L.M[2][1] * R.M[1][2] + L.M[3][1] * R.M[1][3],	// x11
				L.M[0][1] * R.M[2][0] + L.M[1][1] * R.M[2][1] + L.M[2][1] * R.M[2][2] + L.M[3][1] * R.M[2][3],	// x12
				L.M[0][1] * R.M[3][0] + L.M[1][1] * R.M[3][1] + L.M[2][1] * R.M[3][2] + L.M[3][1] * R.M[3][3],	// x13

				L.M[0][2] * R.M[0][0] + L.M[1][2] * R.M[0][1] + L.M[2][2] * R.M[0][2] + L.M[3][2] * R.M[0][3],	// x20
				L.M[0][2] * R.M[1][0] + L.M[1][2] * R.M[1][1] + L.M[2][2] * R.M[1][2] + L.M[3][2] * R.M[1][3],	// x21
				L.M[0][2] * R.M[2][0] + L.M[1][2] * R.M[2][1] + L.M[2][2] * R.M[2][2] + L.M[3][2] * R.M[2][3],	// x22
				L.M[0][2] * R.M[3][0] + L.M[1][2] * R.M[3][1] + L.M[2][2] * R.M[3][2] + L.M[3][2] * R.M[3][3],	// x23

				L.M[0][3] * R.M[0][0] + L.M[1][3] * R.M[0][1] + L.M[2][3] * R.M[0][2] + L.M[3][3] * R.M[0][3],	// x30
				L.M[0][3] * R.M[1][0] + L.M[1][3] * R.M[1][1] + L.M[2][3] * R.M[1][2] + L.M[3][3] * R.M[1][3],	// x31
				L.M[0][3] * R.M[2][0] + L.M[1][3] * R.M[2][1] + L.M[2][3] * R.M[2][2] + L.M[3][3] * R.M[2][3],	// x32
				L.M[0][3] * R.M[3][0] + L.M[1][3] * R.M[3][1] + L.M[2][3] * R.M[3][2] + L.M[3][3] * R.M[3][3]	// x33
			);
		}

		inline FMatrix33 MultiplyAB(const FMatrix33& LIn, const FMatrix33& RIn)
		{
			return Multiply(LIn, RIn);
		}

		inline FMatrix33 MultiplyABt(const FMatrix33& L, const FMatrix33& R)
		{
			return FMatrix33(
				L.M[0][0] * R.M[0][0] + L.M[1][0] * R.M[1][0] + L.M[2][0] * R.M[2][0],	// x00
				L.M[0][0] * R.M[0][1] + L.M[1][0] * R.M[1][1] + L.M[2][0] * R.M[2][1],	// x01
				L.M[0][0] * R.M[0][2] + L.M[1][0] * R.M[1][2] + L.M[2][0] * R.M[2][2],	// x02

				L.M[0][1] * R.M[0][0] + L.M[1][1] * R.M[1][0] + L.M[2][1] * R.M[2][0],	// x10
				L.M[0][1] * R.M[0][1] + L.M[1][1] * R.M[1][1] + L.M[2][1] * R.M[2][1],	// x11
				L.M[0][1] * R.M[0][2] + L.M[1][1] * R.M[1][2] + L.M[2][1] * R.M[2][2],	// x12

				L.M[0][2] * R.M[0][0] + L.M[1][2] * R.M[1][0] + L.M[2][2] * R.M[2][0],	// x20
				L.M[0][2] * R.M[0][1] + L.M[1][2] * R.M[1][1] + L.M[2][2] * R.M[2][1],	// x21
				L.M[0][2] * R.M[0][2] + L.M[1][2] * R.M[1][2] + L.M[2][2] * R.M[2][2]	// x22
			);
		}

		inline FMatrix33 MultiplyAtB(const FMatrix33& L, const FMatrix33& R)
		{
			return FMatrix33(
				L.M[0][0] * R.M[0][0] + L.M[0][1] * R.M[0][1] + L.M[0][2] * R.M[0][2],	// x00
				L.M[0][0] * R.M[1][0] + L.M[0][1] * R.M[1][1] + L.M[0][2] * R.M[1][2],	// x01
				L.M[0][0] * R.M[2][0] + L.M[0][1] * R.M[2][1] + L.M[0][2] * R.M[2][2],	// x02

				L.M[1][0] * R.M[0][0] + L.M[1][1] * R.M[0][1] + L.M[1][2] * R.M[0][2],	// x10
				L.M[1][0] * R.M[1][0] + L.M[1][1] * R.M[1][1] + L.M[1][2] * R.M[1][2],	// x11
				L.M[1][0] * R.M[2][0] + L.M[1][1] * R.M[2][1] + L.M[1][2] * R.M[2][2],	// x12

				L.M[2][0] * R.M[0][0] + L.M[2][1] * R.M[0][1] + L.M[2][2] * R.M[0][2],	// x20
				L.M[2][0] * R.M[1][0] + L.M[2][1] * R.M[1][1] + L.M[2][2] * R.M[1][2],	// x21
				L.M[2][0] * R.M[2][0] + L.M[2][1] * R.M[2][1] + L.M[2][2] * R.M[2][2]	// x22
			);

		}

		/**
		 * Multiple a vector by a matrix: C = L.R
		 * If L is a rotation matrix, then this will return R rotated by that rotation.
		 */
		inline FVec3 Multiply(const FMatrix33& L, const FVec3& R)
		{
			// @todo(chaos): optimize: use simd
			return FVec3(
				L.M[0][0] * R.X + L.M[1][0] * R.Y + L.M[2][0] * R.Z,
				L.M[0][1] * R.X + L.M[1][1] * R.Y + L.M[2][1] * R.Z,
				L.M[0][2] * R.X + L.M[1][2] * R.Y + L.M[2][2] * R.Z);
		}

		inline TVec3<FRealSingle> Multiply(const TMatrix33<FRealSingle>& L, const TVec3<FRealSingle>& R)
		{
			// @todo(chaos): optimize: use simd
			return TVec3<FRealSingle>(
				L.M[0][0] * R.X + L.M[1][0] * R.Y + L.M[2][0] * R.Z,
				L.M[0][1] * R.X + L.M[1][1] * R.Y + L.M[2][1] * R.Z,
				L.M[0][2] * R.X + L.M[1][2] * R.Y + L.M[2][2] * R.Z);
		}

		inline FVec4 Multiply(const FMatrix44& L, const FVec4& R)
		{
			// @todo(chaos): optimize: use simd
			return FVec4(
				L.M[0][0] * R.X + L.M[1][0] * R.Y + L.M[2][0] * R.Z + L.M[3][0] * R.W,
				L.M[0][1] * R.X + L.M[1][1] * R.Y + L.M[2][1] * R.Z + L.M[3][1] * R.W,
				L.M[0][2] * R.X + L.M[1][2] * R.Y + L.M[2][2] * R.Z + L.M[3][2] * R.W,
				L.M[0][3] * R.X + L.M[1][3] * R.Y + L.M[2][3] * R.Z + L.M[3][3] * R.W
			);
		}

		// Solves X * A = B (or A.Multiply(X) = B) using Gaussian elimination, writing the result
		// into X. Returns true if there is a solution, false otherwise.
		inline bool Solve(FVec3& X, const FMatrix33& A, const FVec3& B)
		{
			const FReal Determinant = A.Determinant();
			if (FMath::IsNearlyZero(Determinant))
			{
				return false;
			}

			// Matrix33 stores in columns...
			const FVec3 C0 = A.GetColumn(0);
			const FVec3 C1 = A.GetColumn(1);
			const FVec3 C2 = A.GetColumn(2);

			// All reference material represents the problem in row form. 
			// Form the 3x4 augmented matrix
			FVec4 R0(C0.X, C1.X, C2.X, B.X);
			FVec4 R1(C0.Y, C1.Y, C2.Y, B.Y);
			FVec4 R2(C0.Z, C1.Z, C2.Z, B.Z);

			// Now apply row operations to reduce it to echelon form: Ones along the diagonal, and
			// zeros in the bottom left.
			R0 *= 1 / R0[0];  // Make (0, 0) = 1
			R1 -= R0 * R1[0]; // Make (1, 0) = 0
			R2 -= R0 * R2[0]; // Make (2, 0) = 0

			R1 *= 1 / R1[1];  // Make (1, 1) = 1
			R2 -= R1 * R2[1]; // Make (2, 1) = 0

			R2 *= 1 / R2[2]; // Make (2, 2) = 1

			// R is now in echelon form, so we can work up from the bottom to calculate result
			X[2] = R2[3];
			X[1] = R1[3] - R1[2] * X[2];
			X[0] = R0[3] - (R0[1] * X[1] + R0[2] * X[2]);
			return true;
		}

		/**
		 * Concatenate two transforms. This returns a transform that logically applies R then L.
		 */
		inline FRigidTransform3 Multiply(const FRigidTransform3 L, const FRigidTransform3& R)
		{
			return FRigidTransform3(L.GetTranslation() + L.GetRotation().RotateVector(R.GetTranslation()), L.GetRotation() * R.GetRotation());
		}

		/**
		 * Calculate the world-space inertia (or inverse inertia) for a body with center-of-mass rotation "CoMRotation" and local-space inertia/inverse-inertia "I".
		 */
		inline FMatrix33 ComputeWorldSpaceInertia(const FRotation3& CoMRotation, const FMatrix33& I)
		{
			FMatrix33 QM = CoMRotation.ToMatrix();
			return MultiplyAB(QM, MultiplyABt(I, QM));
		}

		inline FMatrix33 ComputeWorldSpaceInertia(const FRotation3& CoMRotation, const FVec3& I)
		{
			// @todo(ccaulfield): optimize ComputeWorldSpaceInertia
			return ComputeWorldSpaceInertia(CoMRotation, FMatrix33(I.X, I.Y, I.Z));
		}

		// Given a body with world orientation Q, world angular velocity W, local moments of inertia
		// I, this calculates the new angular velocity it should have after integrating forwards by
		// Dt, considering the gyroscopic torques.
		FVec3 GetAngularVelocityAdjustedForGyroscopicTorques(
			const FRotation3& Q, const FVec3& I, const FVec3& W, const FReal Dt);

		/**
		 * Calculate the matrix that maps a constraint position error to constraint position and rotation corrections.
		 */
		template<class T>
		PMatrix<T, 3, 3> ComputeJointFactorMatrix(const TVec3<T>& V, const PMatrix<T, 3, 3>& M, const T& Im)
		{
			// Rigid objects rotational contribution to the impulse.
			// Vx*M*VxT+Im
			check(Im > FLT_MIN);
			return PMatrix<T, 3, 3>(
				-V[2] * (-V[2] * M.M[1][1] + V[1] * M.M[2][1]) + V[1] * (-V[2] * M.M[2][1] + V[1] * M.M[2][2]) + Im,
				V[2] * (-V[2] * M.M[1][0] + V[1] * M.M[2][0]) - V[0] * (-V[2] * M.M[2][1] + V[1] * M.M[2][2]),
				-V[1] * (-V[2] * M.M[1][0] + V[1] * M.M[2][0]) + V[0] * (-V[2] * M.M[1][1] + V[1] * M.M[2][1]),
				V[2] * (V[2] * M.M[0][0] - V[0] * M.M[2][0]) - V[0] * (V[2] * M.M[2][0] - V[0] * M.M[2][2]) + Im,
				-V[1] * (V[2] * M.M[0][0] - V[0] * M.M[2][0]) + V[0] * (V[2] * M.M[1][0] - V[0] * M.M[2][1]),
				-V[1] * (-V[1] * M.M[0][0] + V[0] * M.M[1][0]) + V[0] * (-V[1] * M.M[1][0] + V[0] * M.M[1][1]) + Im);
		}
		
		/**
		 * Calculate the matrix diagonal that maps a constraint position error to constraint position and rotation corrections.
		*/
		template<class T>
		TVec3<T> ComputeDiagonalJointFactorMatrix(const TVec3<T>& V, const PMatrix<T, 3, 3>& M, const T& Im)
		{
			// Rigid objects rotational contribution to the impulse.
			// Vx*M*VxT+Im
			check(Im > FLT_MIN);
			return TVec3<T>(
				-V[2] * (-V[2] * M.M[1][1] + V[1] * M.M[2][1]) + V[1] * (-V[2] * M.M[2][1] + V[1] * M.M[2][2]) + Im,
				 V[2] * (V[2] * M.M[0][0] - V[0] * M.M[2][0]) - V[0] * (V[2] * M.M[2][0] - V[0] * M.M[2][2]) + Im,
				-V[1] * (-V[1] * M.M[0][0] + V[0] * M.M[1][0]) + V[0] * (-V[1] * M.M[1][0] + V[0] * M.M[1][1]) + Im);
		}

		/**
		 * Detects intersections between 2D line segments, returns intersection results as times along each line segment - these times can be
		 * used to calculate exact intersection locations
		 */
		template<typename T>
		bool IntersectLineSegments2D(const TVec2<T>& InStartA, const TVec2<T>& InEndA, const TVec2<T>& InStartB, const TVec2<T>& InEndB, T& OutTA, T& OutTB)
		{
			// Each line can be described as p0 + t(p1 - p0) = P. Set equal to each other and solve for t0 and t1
			OutTA = OutTB = 0;

			T Divisor = (InEndB[0] - InStartB[0]) * (InStartA[1] - InEndA[1]) - (InStartA[0] - InEndA[0]) * (InEndB[1] - InStartB[1]);

			if (FMath::IsNearlyZero(Divisor))
			{
				// The line segments are parallel and will never meet for any values of Ta/Tb
				return false;
			}

			OutTA = ((InStartB[1] - InEndB[1]) * (InStartA[0] - InStartB[0]) + (InEndB[0] - InStartB[0]) * (InStartA[1] - InStartB[1])) / Divisor;
			OutTB = ((InStartA[1] - InEndA[1]) * (InStartA[0] - InStartB[0]) + (InEndA[0] - InStartA[0]) * (InStartA[1] - InStartB[1])) / Divisor;

			return OutTA >= 0 && OutTA <= 1 && OutTB > 0 && OutTB < 1;
		}

		/**
		 * Clip a line segment to inside a plane (plane normal pointing outwards).
		 * @return false if the line is completely outside the plane, true otherwise.
		 */
		inline bool ClipLineSegmentToPlane(FVec3& V0, FVec3& V1, const FVec3& PlaneNormal, const FVec3& PlanePos)
		{
			FReal Dist0 = FVec3::DotProduct(V0 - PlanePos, PlaneNormal);
			FReal Dist1 = FVec3::DotProduct(V1 - PlanePos, PlaneNormal);
			if ((Dist0 > 0.0f) && (Dist1 > 0.0f))
			{
				// Whole line segment is outside of face - reject it
				return false;
			}

			if ((Dist0 > 0.0f) && (Dist1 < 0.0f))
			{
				// We must move vert 0 to the plane
				FReal ClippedT = -Dist1 / (Dist0 - Dist1);
				V0 = FMath::Lerp(V1, V0, ClippedT);
			}
			else if ((Dist1 > 0.0f) && (Dist0 < 0.0f))
			{
				// We must move vert 1 to the plane
				FReal ClippedT = -Dist0 / (Dist1 - Dist0);
				V1 = FMath::Lerp(V0, V1, ClippedT);
			}

			return true;
		}

		/**
		 * Clip a line segment to the inside of an axis aligned plane (normal pointing outwards).
		 */
		inline bool ClipLineSegmentToAxisAlignedPlane(FVec3& V0, FVec3& V1, const int32 AxisIndex, const FReal PlaneDir, const FReal PlanePos)
		{
			FReal Dist0 = (V0[AxisIndex] - PlanePos) * PlaneDir;
			FReal Dist1 = (V1[AxisIndex] - PlanePos) * PlaneDir;
			if ((Dist0 > 0.0f) && (Dist1 > 0.0f))
			{
				// Whole line segment is outside of face - reject it
				return false;
			}

			if ((Dist0 > 0.0f) && (Dist1 < 0.0f))
			{
				// We must move vert 0 to the plane
				FReal ClippedT = -Dist1 / (Dist0 - Dist1);
				V0 = FMath::Lerp(V1, V0, ClippedT);
			}
			else if ((Dist1 > 0.0f) && (Dist0 < 0.0f))
			{
				// We must move vert 1 to the plane
				FReal ClippedT = -Dist0 / (Dist1 - Dist0);
				V1 = FMath::Lerp(V0, V1, ClippedT);
			}

			return true;
		}

		/**
		 * Project a point V along direction Dir onto an axis aligned plane.
		 * /note Does not check for division by zero (Dir parallel to plane).
		 */
		inline void ProjectPointOntoAxisAlignedPlane(FVec3& V, const FVec3& Dir, int32 AxisIndex, FReal PlaneDir, FReal PlanePos)
		{
			// V -> V + ((PlanePos - V) | PlaneNormal) / (Dir | PlaneNormal)
			FReal Denominator = Dir[AxisIndex] * PlaneDir;
			FReal Numerator = (PlanePos - V[AxisIndex]) * PlaneDir;
			FReal F = Numerator / Denominator;
			V = V + F * Dir;
		}

		/**
		 * Project a point V along direction Dir onto an axis aligned plane.
		 * /return true if the point was successfully projected onto the plane; false if the direction is parallel to the plane.
		 */
		inline bool ProjectPointOntoAxisAlignedPlaneSafe(FVec3& V, const FVec3& Dir, int32 AxisIndex, FReal PlaneDir, FReal PlanePos, FReal Epsilon)
		{
			// V -> V + ((PlanePos - V) | PlaneNormal) / (Dir | PlaneNormal)
			FReal Denominator = Dir[AxisIndex] * PlaneDir;
			if (Denominator > Epsilon)
			{
				FReal Numerator = (PlanePos - V[AxisIndex]) * PlaneDir;
				FReal F = Numerator / Denominator;
				V = V + F * Dir;
				return true;
			}
			return false;
		}

		inline bool NormalizeSafe(FVec3& V, FReal EpsilonSq = UE_SMALL_NUMBER)
		{
			FReal VLenSq = V.SizeSquared();
			if (VLenSq > EpsilonSq)
			{
				V = V * FMath::InvSqrt(VLenSq);
				return true;
			}
			return false;
		}

		template <class T>
		inline T DotProduct(const TArray<T>& X, const TArray<T>& Y)
		{
			T Result = T(0);
			for (int32 i = 0; i < X.Num(); i++)
			{
				Result += X[i] * Y[i];
			}
			return Result;
		}

		/**
		 * Given the local-space diagonal inertia for an unscaled object, return an inertia as if generated from a non-uniformly scaled shape with the specified scale.
		 * If bScaleMass is true, it also takes into account the fact that the mass would have changed by the increase in volume.
		 */
		template<typename T>
		inline TVec3<T> ScaleInertia(const TVec3<T>& Inertia, const TVec3<T>& Scale, const bool bScaleMass)
		{
			// support for negative scale 
			const TVec3<T> AbsScale = Scale.GetAbs();

			TVec3<T> XYZSq = (TVec3<T>(0.5f * (Inertia.X + Inertia.Y + Inertia.Z)) - Inertia) * AbsScale * AbsScale;
			T XX = XYZSq.Y + XYZSq.Z;
			T YY = XYZSq.X + XYZSq.Z;
			T ZZ = XYZSq.X + XYZSq.Y;
			TVec3<T> ScaledInertia = TVec3<T>(XX, YY, ZZ);
			T MassScale = (bScaleMass) ? AbsScale.X * AbsScale.Y * AbsScale.Z : 1.0f;
			return MassScale * ScaledInertia;
		}

		/**
		 * Given the local-space inertia for an unscaled object, return an inertia as if generated from a non-uniformly scaled shape with the specified scale.
		 * If bScaleMass is true, it also takes into account the fact that the mass would have changed by the increase in volume.
		 */
		inline FMatrix33 ScaleInertia(const FMatrix33& Inertia, const FVec3& Scale, const bool bScaleMass)
		{
			// support for negative scale 
			const FVec3 AbsScale = Scale.GetAbs();

			// @todo(chaos): do we need to support a rotation of the scale axes?
			FVec3 D = Inertia.GetDiagonal();
			FVec3 XYZSq = (FVec3(0.5f * (D.X + D.Y + D.Z)) - D) * AbsScale * AbsScale;
			FReal XX = XYZSq.Y + XYZSq.Z;
			FReal YY = XYZSq.X + XYZSq.Z;
			FReal ZZ = XYZSq.X + XYZSq.Y;
			FReal XY = Inertia.M[0][1] * AbsScale.X * AbsScale.Y;
			FReal XZ = Inertia.M[0][2] * AbsScale.X * AbsScale.Z;
			FReal YZ = Inertia.M[1][2] * AbsScale.Y * AbsScale.Z;
			FReal MassScale = (bScaleMass) ? AbsScale.X * AbsScale.Y * AbsScale.Z : 1.0f;
			FMatrix33 ScaledInertia = FMatrix33(
				MassScale * XX, MassScale * XY, MassScale * XZ,
				MassScale * XY, MassScale * YY, MassScale * YZ,
				MassScale * XZ, MassScale * YZ, MassScale * ZZ);
			return ScaledInertia;
		}

		// Compute the box size that would generate the given (diagonal) inertia
		inline bool BoxFromInertia(const FVec3& InInertia, const FReal Mass, FVec3& OutCenter, FVec3& OutSize)
		{
			OutSize = FVec3(0);
			OutCenter = FVec3(0);

			// System of 3 equations in X^2, Y^2, Z^2
			//		Inertia.X = 1/12 M (Size.Y^2 + Size.Z^2)
			//		Inertia.Y = 1/12 M (Size.Z^2 + Size.X^2)
			//		Inertia.Z = 1/12 M (Size.X^2 + Size.Y^2)
			// Unless the center of mass has been modified, in which case we have
			//		Inertia.X = 1/12 M (Size.Y^2 + Size.Z^2) + M D.X^2
			//		Inertia.Y = 1/12 M (Size.Z^2 + Size.X^2) + M D.Y^2
			//		Inertia.Z = 1/12 M (Size.X^2 + Size.Y^2) + M D.Z^2
			// Which will not have a unique solution (3 equations in 6 unknowns).
			// There's no way to know here that the center of mass was modified so we assume it wasn't unless we cannot
			// solve the equations and then we must make some guesses to recover an equivalent box.
			if (Mass > 0)
			{
				// RInv is the inverse of the coefficient matrix (0,1,1)(1,0,1)(0,1,1)
				const FMatrix33 RInv = FMatrix33(-0.5f, 0.5f, 0.5f, 0.5f, -0.5f, 0.5f, 0.5f, 0.5f, -0.5f);
				FVec3 Inertia = InInertia / Mass;
				FVec3 XYZSq = RInv * (Inertia * 12.0f);

				// If we have a shape with a modified center of mass that is outside the equivalent box, we will end up with negative
				// coefficients here and cannot calculate a box equivalent. To do this properly we need to know what center of mass offset was applied.
				// But lets try to do something anyway so that debug draw shows something...we'll pretend that the shifted inertia component would be equal to the
				// smallest component in the absense of the shift. This works "correctly" for a uniform shape (e.g., a box), but will be wrong for everything else!
				// Also, there's a sign problem since shifting the center of mass in the opposite direction would have altered the inertia the same way.
				// Net result: I'm not sure how useful this is - see if we can do something better one day (maybe store the ComNudge)
				if (XYZSq.X < 0)
				{
					FReal DXSq = (Inertia.X - FMath::Min(Inertia.Y, Inertia.Z));
					if (DXSq > 0)
					{
						OutCenter.X = FMath::Sqrt(DXSq);
						Inertia.X -= DXSq;
						XYZSq = RInv * (Inertia * 12.0f);
					}
				}
				if (XYZSq.Y < 0)
				{
					FReal DYSq = (Inertia.Y - FMath::Min(Inertia.X, Inertia.Z));
					if (DYSq > 0)
					{
						OutCenter.Y = FMath::Sqrt(DYSq);
						Inertia.Y -= DYSq;
						XYZSq = RInv * (Inertia * 12.0f);
					}
				}
				if (XYZSq.Z < 0)
				{
					FReal DZSq = (Inertia.Z - FMath::Min(Inertia.X, Inertia.Y));
					if (DZSq > 0)
					{
						OutCenter.Z = FMath::Sqrt(DZSq);
						Inertia.Z -= DZSq;
						XYZSq = RInv * (Inertia * 12.0f);
					}
				}

				OutSize = FVec3(
					FMath::Sqrt(FMath::Max(XYZSq.X, FReal(0))),
					FMath::Sqrt(FMath::Max(XYZSq.Y, FReal(0))),
					FMath::Sqrt(FMath::Max(XYZSq.Z, FReal(0))));

				return true;
			}
			return false;
		}

		// Replacement for FMath::Wrap that works for integers and returns a value in [Begin, End).
		// Note: this implementation uses a loop to bring the value into range - it should not be used if the value is much larger than the range.
		inline int32 WrapIndex(int32 V, int32 Begin, int32 End)
		{
			int32 Range = End - Begin;
			while (V < Begin)
			{
				V += Range;
			}
			while (V >= End)
			{
				V -= Range;
			}
			return V;
		}

		// Get the closest point on a line segment between P1 and Q1 and a 
		// second line segment between P2 and Q2
		// For implementation notes, see "Realtime Collision Detection", Christer Ericson, 2005
		inline void NearestPointsOnLineSegments(
			const FVec3& P1, const FVec3& Q1,
			const FVec3& P2, const FVec3& Q2,
			FReal& S, FReal& T,
			FVec3& C1, FVec3& C2,
			const FReal Epsilon = 1.e-4f)
		{
			const FReal EpsilonSq = Epsilon * Epsilon;
			const FVec3 D1 = Q1 - P1;
			const FVec3 D2 = Q2 - P2;
			const FVec3 R = P1 - P2;
			const FReal A = FVec3::DotProduct(D1, D1);
			const FReal B = FVec3::DotProduct(D1, D2);
			const FReal C = FVec3::DotProduct(D1, R);
			const FReal E = FVec3::DotProduct(D2, D2);
			const FReal F = FVec3::DotProduct(D2, R);
			constexpr FReal Min = 0, Max = 1;

			S = 0.0f;
			T = 0.0f;

			if ((A <= EpsilonSq) && (B <= EpsilonSq))
			{
				// Both segments are points
			}
			else if (A <= EpsilonSq)
			{
				// First segment (only) is a point
				T = FMath::Clamp<FReal>(F / E, Min, Max);
			}
			else if (E <= EpsilonSq)
			{
				// Second segment (only) is a point
				S = FMath::Clamp<FReal>(-C / A, Min, Max);
			}
			else
			{
				// Non-degenrate case - we have two lines
				const FReal Denom = A * E - B * B;
				if (Denom != 0.0f)
				{
					S = FMath::Clamp<FReal>((B * F - C * E) / Denom, Min, Max);
				}
				T = (B * S + F) / E;

				if (T < 0.0f)
				{
					S = FMath::Clamp<FReal>(-C / A, Min, Max);
					T = 0.0f;
				}
				else if (T > 1.0f)
				{
					S = FMath::Clamp<FReal>((B - C) / A, Min, Max);
					T = 1.0f;
				}
			}

			C1 = P1 + S * D1;
			C2 = P2 + T * D2;
		}

		// Get the closest point on a line segment between SegmentBegin and SegmentEnd to an infinite
		// line through LinePos along LineVector in both direction.
		// For implementation notes, see "Realtime Collision Detection", Christer Ericson, 2005
		inline void NearestPointsOnLineSegmentToLine(
			const FVec3& SegmentBegin, const FVec3& SegmentEnd,
			const FVec3& LinePos, const FVec3& LineVector,
			FReal& S, FReal& T,
			FVec3& C1, FVec3& C2,
			const FReal Epsilon = 1.e-4f)
		{
			const FReal EpsilonSq = Epsilon * Epsilon;
			const FVec3& P1 = SegmentBegin;
			const FVec3& Q1 = SegmentEnd;
			const FVec3& P2 = LinePos;
			const FVec3 D1 = Q1 - P1;
			const FVec3& D2 = LineVector;
			const FVec3 R = P1 - P2;
			const FReal A = FVec3::DotProduct(D1, D1);
			const FReal B = FVec3::DotProduct(D1, D2);
			const FReal C = FVec3::DotProduct(D1, R);
			const FReal E = FVec3::DotProduct(D2, D2);
			const FReal F = FVec3::DotProduct(D2, R);
			constexpr FReal Min = 0, Max = 1;

			S = 0.0f;
			T = 0.0f;

			if ((A <= EpsilonSq) && (B <= EpsilonSq))
			{
				// Both segments are points
			}
			else if (A <= EpsilonSq)
			{
				// First segment (only) is a point
				T = F / E;
			}
			else if (E <= EpsilonSq)
			{
				// Second line (only) is a point (i.e., LineVector is zero)
				S = FMath::Clamp<FReal>(-C / A, Min, Max);
			}
			else
			{
				// Non-degenrate case - we have two lines
				const FReal Denom = A * E - B * B;
				if (Denom != 0.0f)
				{
					S = FMath::Clamp<FReal>((B * F - C * E) / Denom, Min, Max);
				}
				T = (B * S + F) / E;
			}

			C1 = P1 + S * D1;
			C2 = P2 + T * D2;
		}

		namespace
		{
			// Use dot product with a barrier to avoid fma to generate inaccuracy results
			template<typename T>
			T DotWithoutFMA(const TVec3<T>& A, const TVec3<T>& B)
			{
				T X = A.X * B.X;
				T Y = A.Y * B.Y;
				T Z = A.Z * B.Z;
#if defined(_MSC_VER) && !defined(__clang__)
				_ReadWriteBarrier();
#endif
				return X + Y + Z;
			}
		}

		template<typename T>
		inline T ClosestTimeOnLineSegment(const TVec3<T>& Point, const TVec3<T>& StartPoint, const TVec3<T>& EndPoint)
		{
			const TVec3<T> Segment = EndPoint - StartPoint;
			const TVec3<T> VectToPoint = Point - StartPoint;

			// See if closet point is before StartPoint
			const T Dot1 = DotWithoutFMA<T>(VectToPoint, Segment);
			if (Dot1 <= T(0))
			{
				return T(0);
			}

			// See if closest point is beyond EndPoint
			const T Dot2 = DotWithoutFMA<T>(Segment, Segment);
			if (Dot2 <= Dot1)
			{
				return T(1);
			}

			// Closest Point is within segment
			return Dot1 / Dot2;
		}

		/**
		 * @brief The distance from Start to the sphere along the vector Dir. Returns numeric max if no intersection.
		*/
		template<typename T>
		inline T RaySphereIntersectionDistance(const TVec3<T>& RayStart, const TVec3<T>& RayDir, const TVec3<T>& SpherePos, const T SphereRadius)
		{
			const TVec3<T> EO = SpherePos - RayStart;
			const T V = TVec3<T>::DotProduct(RayDir, EO);
			const T Disc = SphereRadius * SphereRadius - (TVec3<T>::DotProduct(EO, EO) - V * V);
			if (Disc >= 0)
			{
				return (V - FMath::Sqrt(Disc));
			}
			else
			{
				return TNumericLimits<T>::Max();
			}
		}

		/**
		 * Approximate Asin(X) to 1st order ~= X + ...
		 * Returns an approximation for all valid input range [-1, 1] with an error that increases as the input magnitude approaches 1.
		 */
		template<typename T>
		inline T AsinEst1(const T X)
		{
			return X /*+...*/;
		}


		/**
		 * Approximate Asin(X) to 3rd order ~= X + (1/6)X^3 + ...
		 * Returns an approximation for all valid input range [-1, 1] with an error that increases as the input magnitude approaches 1.
		 */
		template<typename T>
		inline T AsinEst3(const T X)
		{
			constexpr T C0 = T(1.0);
			constexpr T C1 = T(1.0 / 6.0);
			const T X2 = X * X;
			return X * (C0 + X2 * (C1 /*+...*/));
		}

		/**
		 * Approximate Asin(X) to 5th order ~= X + (1/6)X^3 + (3/40)X^5 + ...
		 * Returns an approximation for all valid input range [-1, 1] with an error that increases as the input magnitude approaches 1.
		 */
		template<typename T>
		inline T AsinEst5(const T X)
		{
			constexpr T C0 = T(1.0);
			constexpr T C1 = T(1.0 / 6.0);
			constexpr T C2 = T(3.0 / 40.0);
			const T X2 = X * X;
			return X * (C0 + X2 * (C1 + X2 * (C2 /*+...*/)));

		}

		/**
		 * Approximate Asin(X) to 7th order ~= X + (1/6)X^3 + (3/40)X^5 + (15/336)X^7 + ...
		 * Returns an approximation for all valid input range [-1, 1] with an error that increases as the input magnitude approaches 1.
		 */
		template<typename T>
		inline T AsinEst7(const T X)
		{
			constexpr T C0 = T(1.0);
			constexpr T C1 = T(1.0 / 6.0);
			constexpr T C2 = T(3.0 / 40.0);
			constexpr T C3 = T(15.0 / 336.0);
			const T X2 = X * X;
			return X * (C0 + X2 * (C1 + X2 * (C2 + X2 * (C3 /*+...*/))));
		}

		/**
		 * Approximate Asin using expansion to the specified Order (must be 1, 3, 5, or 7). Defaults to 5th order.
		*/
		template<typename T, int Order = 5>
		inline T AsinEst(const T X)
		{
			static_assert((Order == 1) || (Order == 3) || (Order == 5) || (Order == 7), "AsinEst: Only 1, 3, 5, or 7 is supported for the Order");
			if (Order == 1)
			{
				return AsinEst1(X);
			}
			else if (Order == 3)
			{
				return AsinEst3(X);
			}
			else if (Order == 5)
			{
				return AsinEst5(X);
			}
			else
			{
				return AsinEst7(X);
			}
		}

		/**
		 * Approximate Asin(X). Like AsinApprox but with a crossover to FMath::Asin for larger input magnitudes.
		 */
		template<typename T, int Order = 5>
		inline T AsinEstCrossover(const T X, const T Crossover = T(0.7))
		{
			if (FMath::Abs(X) < Crossover)
			{
				return AsinEst<T, Order>(X);
			}
			else
			{
				return FMath::Asin(X);
			}
		}

		/**
		 * @brief Generate a tetrahedral mesh from a Freudenthal lattice defined on a 3 dimensional grid.
		 */
		template <class T, class TV, class TV_INT4>
		void TetMeshFromGrid(const TUniformGrid<T, 3>& Grid, TArray<TV_INT4>& Mesh, TArray<TV>& X)
		{
			Mesh.SetNum(20 * Grid.GetNumCells() / 4);
			int32* MeshPtr = &Mesh[0][0];

			const int32 NumNodes = Grid.GetNumNodes();
			X.SetNum(NumNodes);
			for(int32 ii=0; ii < NumNodes; ii++)
			{
				X[ii] = Grid.Node(ii);
			}

			int32 Count = 0;
			for (int32 i = 0; i < Grid.Counts()[0]; i++) 
			{
				for (int32 j = 0; j < Grid.Counts()[1]; j++) 
				{
					for (int32 k = 0; k < Grid.Counts()[2]; k++) 
					{
						int32 ijk000 = Grid.FlatIndex(TVector<int32, 3>(i, j, k), true);
						int32 ijk010 = Grid.FlatIndex(TVector<int32, 3>(i, j + 1, k), true);
						int32 ijk001 = Grid.FlatIndex(TVector<int32, 3>(i, j, k + 1), true);
						int32 ijk011 = Grid.FlatIndex(TVector<int32, 3>(i, j + 1, k + 1), true);
						int32 ijk100 = Grid.FlatIndex(TVector<int32, 3>(i + 1, j, k), true);
						int32 ijk110 = Grid.FlatIndex(TVector<int32, 3>(i + 1, j + 1, k), true);
						int32 ijk101 = Grid.FlatIndex(TVector<int32, 3>(i + 1, j, k + 1), true);
						int32 ijk111 = Grid.FlatIndex(TVector<int32, 3>(i + 1, j + 1, k + 1), true);
						int32 ijk_index = i + j + k;
						if (ijk_index % 2 == 0) 
						{
							MeshPtr[20 * Count] = ijk010;
							MeshPtr[20 * Count + 1] = ijk000;
							MeshPtr[20 * Count + 2] = ijk110;
							MeshPtr[20 * Count + 3] = ijk011;

							MeshPtr[20 * Count + 4] = ijk111;
							MeshPtr[20 * Count + 5] = ijk110;
							MeshPtr[20 * Count + 6] = ijk101;
							MeshPtr[20 * Count + 7] = ijk011;

							MeshPtr[20 * Count + 8] = ijk100;
							MeshPtr[20 * Count + 9] = ijk101;
							MeshPtr[20 * Count + 10] = ijk110;
							MeshPtr[20 * Count + 11] = ijk000;

							MeshPtr[20 * Count + 12] = ijk001;
							MeshPtr[20 * Count + 13] = ijk000;
							MeshPtr[20 * Count + 14] = ijk011;
							MeshPtr[20 * Count + 15] = ijk101;

							MeshPtr[20 * Count + 16] = ijk110;
							MeshPtr[20 * Count + 17] = ijk011;
							MeshPtr[20 * Count + 18] = ijk000;
							MeshPtr[20 * Count + 19] = ijk101;
						}
						else 
						{
							MeshPtr[20 * Count] = ijk000;
							MeshPtr[20 * Count + 1] = ijk100;
							MeshPtr[20 * Count + 2] = ijk010;
							MeshPtr[20 * Count + 3] = ijk001;

							MeshPtr[20 * Count + 4] = ijk011;
							MeshPtr[20 * Count + 5] = ijk010;
							MeshPtr[20 * Count + 6] = ijk111;
							MeshPtr[20 * Count + 7] = ijk001;

							MeshPtr[20 * Count + 8] = ijk100;
							MeshPtr[20 * Count + 9] = ijk111;
							MeshPtr[20 * Count + 10] = ijk010;
							MeshPtr[20 * Count + 11] = ijk001;

							MeshPtr[20 * Count + 12] = ijk101;
							MeshPtr[20 * Count + 13] = ijk111;
							MeshPtr[20 * Count + 14] = ijk100;
							MeshPtr[20 * Count + 15] = ijk001;

							MeshPtr[20 * Count + 16] = ijk111;
							MeshPtr[20 * Count + 17] = ijk010;
							MeshPtr[20 * Count + 18] = ijk110;
							MeshPtr[20 * Count + 19] = ijk100;
						}
						Count++;
					}
				}
			}
		}

		template <int d>
		TArray<TArray<int>> ComputeIncidentElements(const TArray<TVector<int32, d>>& Mesh, TArray<TArray<int32>>* LocalIndex=nullptr)
		{
			int32 MaxIdx = 0;
			for(int32 i=0; i < Mesh.Num(); i++)
			{
				for (int32 j = 0; j < d; j++)
				{
					const int32 NodeIdx = Mesh[i][j];
					MaxIdx = MaxIdx > NodeIdx ? MaxIdx : NodeIdx;
				}
			}

			TArray<TArray<int>> IncidentElements;
			IncidentElements.SetNum(MaxIdx + 1);
			if (LocalIndex)
				LocalIndex->SetNum(MaxIdx + 1);

			for (int32 i = 0; i < Mesh.Num(); i++)
			{
				for (int32 j = 0; j < d; j++)
				{
					const int32 NodeIdx = Mesh[i][j];
					if (NodeIdx >= 0)
					{
						IncidentElements[NodeIdx].Add(i);
						if (LocalIndex)
							(*LocalIndex)[NodeIdx].Add(j);
					}
				}
			}

			return IncidentElements;
		}

		inline void DFS_iterative(const TArray<TArray<int32>>& L, int32 v, TArray<bool>& visited, TArray<int32>& component) {
			TArray<int32> Stack({ v });
			Stack.Heapify();
			while (!Stack.IsEmpty()) {
				Stack.HeapPop(v);
				if (!visited[v]) {
					visited[v] = true;
					component.Emplace(v);
					for (int32 i : L[v]) {
						if (!visited[i]) {
							Stack.HeapPush(i);
						}
					}
				}
			}
		}

		inline void ConnectedComponentsDFSIterative(const TArray<TArray<int32>>& L, TArray<TArray<int32>>& C) {
			//Input: L, the given adjacency list
			//Output: C, connected components
			TArray<bool> Visited;
			Visited.Init(false, L.Num());
			for (int32 v = 0; v < L.Num(); ++v) {
				if (!Visited[v]) {
					TArray<int32> component;
					DFS_iterative(L, v, Visited, component);
					C.Emplace(component);
				}
			}
		}

		inline void FindConnectedRegions(const TArray<FIntVector4>& Elements, TArray<TArray<int32>>& ConnectedComponents) 
		{
			TArray<int32> AllEntries, ElementIndices, Ordering, Ranges;
			int32 count = 0;
			AllEntries.Init(-1, Elements.Num() * 4);
			ElementIndices.Init(-1, Elements.Num()*4);
			Ordering.Init(-1, Elements.Num()*4);
			Ranges.Init(-1, Elements.Num()*4 + 1);
			for (int32 i = 0; i < Elements.Num(); i++) 
			{
				for (int32 j = 0; j < 4; j++) 
				{
					AllEntries[4 * i + j] = Elements[i][j];
					ElementIndices[4*i+j] = i;
					Ordering[4*i+j] = count;
					Ranges[4*i+j] = count++;
				}
			}
			Ranges[Elements.Num() * 4] = count;
			
			if (AllEntries.Num() == 0)
				return;
			
			Ordering.Sort([&AllEntries](int32 a, int32 b) { return AllEntries[a] < AllEntries[b]; });
			TArray<int32> UniqueRanges({ 0 });
			
			for (int32 i = 0; i < Ranges.Num() - 2; ++i)
			{
				if (AllEntries[Ordering[i]] != AllEntries[Ordering[i+1]])
				{
					UniqueRanges.Emplace(i + 1);
				}
			}
			
			UniqueRanges.Emplace(count);

			TArray<TArray<int32>> AdjacencyList;
			AdjacencyList.Init(TArray<int32>(), Elements.Num());
			for (int32 r = 0; r < UniqueRanges.Num() - 1; r++) 
			{
				if (UniqueRanges[r + 1] - UniqueRanges[r] > 1) 
				{
					for (int32 i = UniqueRanges[r]; i < UniqueRanges[r + 1]; i++) 
					{
						for (int32 j = i + 1; j < UniqueRanges[r + 1]; j++) 
						{
							AdjacencyList[ElementIndices[Ordering[i]]].Emplace(ElementIndices[Ordering[j]]);
							AdjacencyList[ElementIndices[Ordering[j]]].Emplace(ElementIndices[Ordering[i]]);
						}
					}
				}
			}
			ConnectedComponentsDFSIterative(AdjacencyList, ConnectedComponents);
		}

		inline void FindConnectedRegions(const TArray<FIntVector3>& Elements, TArray<TArray<int32>>& ConnectedComponents)
		{
			TArray<int32> AllEntries, ElementIndices, Ordering, Ranges;
			int32 count = 0;
			AllEntries.Init(-1, Elements.Num() * 3);
			ElementIndices.Init(-1, Elements.Num() * 3);
			Ordering.Init(-1, Elements.Num() * 3);
			Ranges.Init(-1, Elements.Num() * 3 + 1);
			for (int32 i = 0; i < Elements.Num(); i++)
			{
				for (int32 j = 0; j < 3; j++)
				{
					AllEntries[3 * i + j] = Elements[i][j];
					ElementIndices[3 * i + j] = i;
					Ordering[3 * i + j] = count;
					Ranges[3 * i + j] = count++;
				}
			}
			Ranges[Elements.Num() * 3] = count;

			if (AllEntries.Num() == 0)
				return;

			Ordering.Sort([&AllEntries](int32 a, int32 b) { return AllEntries[a] < AllEntries[b]; });
			TArray<int32> UniqueRanges({ 0 });

			for (int32 i = 0; i < Ranges.Num() - 2; ++i)
			{
				if (AllEntries[Ordering[i]] != AllEntries[Ordering[i + 1]])
				{
					UniqueRanges.Emplace(i + 1);
				}
			}

			UniqueRanges.Emplace(count);

			TArray<TArray<int32>> AdjacencyList;
			AdjacencyList.Init(TArray<int32>(), Elements.Num());
			for (int32 r = 0; r < UniqueRanges.Num() - 1; r++)
			{
				if (UniqueRanges[r + 1] - UniqueRanges[r] > 1)
				{
					for (int32 i = UniqueRanges[r]; i < UniqueRanges[r + 1]; i++)
					{
						for (int32 j = i + 1; j < UniqueRanges[r + 1]; j++)
						{
							AdjacencyList[ElementIndices[Ordering[i]]].Emplace(ElementIndices[Ordering[j]]);
							AdjacencyList[ElementIndices[Ordering[j]]].Emplace(ElementIndices[Ordering[i]]);
						}
					}
				}
			}
			ConnectedComponentsDFSIterative(AdjacencyList, ConnectedComponents);
		}

		inline TArray<int32> ComputeBoundaryNodes(const TArray<TVec3<int32>>& TriangleMesh)
		{
			TMap<TPair<int32, int32>, int32> FaceCountPerEdge;
		
			auto AddTriToFaceCountPerEdge = [&FaceCountPerEdge](const TVec3<int32> &Face)
			{
				auto AddEdge = [&FaceCountPerEdge](int32 A, int32 B)
				{
					FaceCountPerEdge.FindOrAdd(TPair<int32, int32>(FMath::Min(A, B), FMath::Max(A, B)))++;
				};
				AddEdge(Face[0], Face[1]);
				AddEdge(Face[0], Face[2]);
				AddEdge(Face[1], Face[2]);
			};

			auto CandidateTriWouldMakeNonManifoldEdge = [&FaceCountPerEdge](int32 A, int32 B, int32 C)
			{
				auto GetCount = [&FaceCountPerEdge](int32 InnerA, int32 InnerB)
				{
					int32 *Count = FaceCountPerEdge.Find(TPair<int32, int32>(FMath::Min(InnerA, InnerB), FMath::Max(InnerA, InnerB)));
					return Count ? *Count : 0;
				};
				return GetCount(A, B) > 1 || GetCount(B, C) > 1 || GetCount(C, A) > 1;
			};

			for (const TVec3<int32> &Face : TriangleMesh)
			{
				AddTriToFaceCountPerEdge(Face);
			}

			TArray<TPair<int32, int32>> AllEdges;
			TSet<int32> BoundaryVerts;
			int32 NumKeys = FaceCountPerEdge.GetKeys(AllEdges);

			for (const TPair<int32, int32>& Edge: AllEdges)
			{
				if (FaceCountPerEdge[Edge] == 1)
				{
					BoundaryVerts.Emplace(Edge.Key);
					BoundaryVerts.Emplace(Edge.Value);
				}
			}

			return BoundaryVerts.Array();
		}

		inline TArray<TArray<int32>> ComputeIncidentElements(const TArray<TArray<int32>>& Constraints, TArray<TArray<int32>>* LocalIndex = nullptr)
		{
			int32 MaxIdx = 0;
			for (int32 i = 0; i < Constraints.Num(); i++)
			{
				for (int32 j = 0; j < Constraints[i].Num(); j++)
				{
					const int32 NodeIdx = Constraints[i][j];
					MaxIdx = MaxIdx > NodeIdx ? MaxIdx : NodeIdx;
				}
			}

			TArray<TArray<int>> IncidentElements;
			IncidentElements.SetNum(MaxIdx + 1);
			if (LocalIndex)
			{
				LocalIndex->SetNum(MaxIdx + 1);
			}
			for (int32 i = 0; i < Constraints.Num(); i++)
			{
				for (int32 j = 0; j < Constraints[i].Num(); j++)
				{
					const int32 NodeIdx = Constraints[i][j];
					if (NodeIdx >= 0)
					{
						IncidentElements[NodeIdx].Add(i);
						if (LocalIndex)
							(*LocalIndex)[NodeIdx].Add(j);
					}
				}
			}

			return IncidentElements;
		}

		inline void MergeIncidentElements(const TArray<TArray<int32>>& ExtraIncidentElements, const TArray<TArray<int32>>& ExtraIncidentElementsLocal, TArray<TArray<int32>>& IncidentElements, TArray<TArray<int32>>& IncidentElementsLocal)
		{
			if (ensureMsgf(IncidentElements.Num() == ExtraIncidentElements.Num() && IncidentElementsLocal.Num() == ExtraIncidentElementsLocal.Num(), TEXT("Input incident elements are of different size")))
			{
				for (int32 i = 0; i < IncidentElements.Num(); i++)
				{
					IncidentElements[i] += ExtraIncidentElements[i];
					IncidentElementsLocal[i] += ExtraIncidentElementsLocal[i];
				}
			}
		}

		inline FIntVector3 SortFIntVector3(FIntVector3& V)
		{
			if (V[1] < V[0])
			{
				int32 V0 = V[0];
				V[0] = V[1];
				V[1] = V0;
			}
			if (V[2] < V[0])
			{
				int32 V2 = V[2];
				V[2] = V[1];
				V[1] = V[0];
				V[0] = V2;
			}
			else if (V[2] < V[1])
			{
				int32 V1 = V[1];
				V[1] = V[2];
				V[2] = V1;
			}
			return V;
		}

		// For each of the 4 triangle faces in a tet, return local vertex indices
		// right hand rule, triangle normal pointing outwards
		inline FIntVector3 TetFace(int32 f) {
			switch (f) {
			case 0:
				return { 1, 2, 3 };
			case 1:
				return { 0, 3, 2 };
			case 2:
				return { 0, 1, 3 };
			case 3:
				return { 0, 2, 1 };
			default:
				return { -1, -1, -1 };
			}
		}

		template <typename Func1, typename Func2>
		TArray<FIntVector2> ComputeMeshFacePairs(const TArray<FIntVector4>& Mesh, Func1 GreaterThan, Func2 Equal) {
			int32 NumFacesPerElement = 4;
			int32 face_number = Mesh.Num() * NumFacesPerElement;
			TArray<int32> AllFaces, ranges;
			AllFaces.SetNum(face_number);
			for (int32 f = 0; f < face_number; f++) {
				AllFaces[f] = f;
			}

			AllFaces.Sort([&GreaterThan, &Mesh](int32 face_a, int32 face_b) { return GreaterThan(Mesh, face_a, face_b); });
			TArray<FIntVector2> CommonFacePairs;
			int32 f1 = 0, f2 = 1;
			while (f2 < AllFaces.Num())
			{
				if (Equal(Mesh, AllFaces[f1], AllFaces[f2])) //common face from two tetrahedra
				{
					CommonFacePairs.Add(FIntVector2(AllFaces[f1], AllFaces[f2]));
					f1 += 2;
					f2 += 2;
				}
				else //boundary face
				{
					CommonFacePairs.Add(FIntVector2(AllFaces[f1], -1));
					f1 += 1;
					f2 += 1;
				}
			}
			if (f1 < AllFaces.Num())
			{
				CommonFacePairs.Add(FIntVector2(AllFaces[f1], -1));
			}
			return CommonFacePairs;
		}

		//returns pairs of face indices {FaceA, FaceB} that are shared by two adjacent tetrahedra or boundary face {FaceA, -1}
		inline TArray<FIntVector2> ComputeTetMeshFacePairs(const TArray<FIntVector4>& TetMesh) {
			auto FaceGreaterThan = [](FIntVector3& i1, FIntVector3& i2)
			{
				SortFIntVector3(i1);
				SortFIntVector3(i2);
				if (i1[0] > i2[0])
					return true;
				else if (i1[0] == i2[0] && i1[1] > i2[1])
					return true;
				return (i1[0] == i2[0] && i1[1] == i2[1] && i1[2] > i2[2]);
			};
			auto FaceEqual = [](FIntVector3& i1, FIntVector3& i2)
			{
				SortFIntVector3(i1);
				SortFIntVector3(i2);
				return i1 == i2;
			};
			auto GreaterThan = [&FaceGreaterThan](const TArray<FIntVector4>& Mesh, int32 e1, int32 e2)
			{
				int32 Local1 = e1 % 4;
				int32 Element1 = e1 / 4;
				FIntVector3 LocalFace1 = TetFace(Local1);
				FIntVector3 Edge1 = FIntVector3(Mesh[Element1][LocalFace1[0]], Mesh[Element1][LocalFace1[1]], Mesh[Element1][LocalFace1[2]]);
				int32 Local2 = e2 % 4;
				int32 Element2 = e2 / 4;
				FIntVector3 LocalFace2 = TetFace(Local2);
				FIntVector3 Edge2 = FIntVector3(Mesh[Element2][LocalFace2[0]], Mesh[Element2][LocalFace2[1]], Mesh[Element2][LocalFace2[2]]);
				return FaceGreaterThan(Edge1, Edge2);
			};
			auto Equal = [&FaceEqual](const TArray<FIntVector4>& Mesh, int32 e1, int32 e2)
			{
				int32 Local1 = e1 % 4;
				int32 Element1 = e1 / 4;
				FIntVector3 LocalFace1 = TetFace(Local1);
				FIntVector3 Edge1 = FIntVector3(Mesh[Element1][LocalFace1[0]], Mesh[Element1][LocalFace1[1]], Mesh[Element1][LocalFace1[2]]);
				int32 Local2 = e2 % 4;
				int32 Element2 = e2 / 4;
				FIntVector3 LocalFace2 = TetFace(Local2);
				FIntVector3 Edge2 = FIntVector3(Mesh[Element2][LocalFace2[0]], Mesh[Element2][LocalFace2[1]], Mesh[Element2][LocalFace2[2]]);
				return FaceEqual(Edge1, Edge2);
			};
			return ComputeMeshFacePairs(TetMesh, GreaterThan, Equal);
		}

		//Return a randomly sampled point in selected tetrahedra following uniform distribution
		inline TArray<TArray<FVector3f>> RandomPointsInTet(const TArray<FVector3f>& x, const TArray<FIntVector4>& TetMesh, const TArray<int32>& SampleElements, int32 NumRandomPointsPerElement = 1) {
			srand(1);
			TArray<TArray<FVector3f>> SampledPoints;
			SampledPoints.SetNum(SampleElements.Num());
			for (int32 ElemIdx = 0; ElemIdx < SampleElements.Num(); ++ElemIdx)
			{
				int32 e = SampleElements[ElemIdx];
				for (int32 i = 0; i < NumRandomPointsPerElement; ++i) {
					FVector4f Weights(0);
					FVector3f Point(0);
					float TotalWeight = 0;
					for (int32 j = 0; j < 4; ++j) {
						Weights[j] = float(rand()) / float(RAND_MAX);
						TotalWeight += Weights[j];
					}
					for (int32 j = 0; j < 4; ++j) {
						Weights[j] /= TotalWeight;
						Point += Weights[j] * x[TetMesh[e][j]];
					}
					SampledPoints[ElemIdx].Add(Point);
				}
			}
			return SampledPoints;
		}
	} // namespace Utilities
} // namespace Chaos