// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Chaos/SoftsSimulationSpace.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/PBDFlatWeightMap.h"

namespace Chaos::Softs
{

// Velocity field used solely for aerodynamics effects, use Chaos Fields for other types of fields.
class FVelocityAndPressureField
{
public:
	static constexpr FSolverReal DefaultDragCoefficient = (FSolverReal)0.5;
	static constexpr FSolverReal DefaultLiftCoefficient = (FSolverReal)0.1;
	static constexpr FSolverReal DefaultFluidDensity = (FSolverReal)1.225;
	static constexpr FSolverReal MinCoefficient = (FSolverReal)0.;   // Applies to both drag and lift
	static constexpr FSolverReal MaxCoefficient = (FSolverReal)10.;  //
	static constexpr EChaosSoftsSimulationSpace DefaultWindVelocitySpace = EChaosSoftsSimulationSpace::WorldSpace;
	static constexpr FSolverReal DefaultTurbulenceRatio = (FSolverReal)1.f;

	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return
			IsDragEnabled(PropertyCollection, false) ||
			IsLiftEnabled(PropertyCollection, false) ||
			IsPressureEnabled(PropertyCollection, false);
	}

	explicit FVelocityAndPressureField(const FCollectionPropertyConstFacade& PropertyCollection)
		: Offset(INDEX_NONE)
		, NumParticles(0)
		, Lift(GetWeightedFloatLift(PropertyCollection, (FSolverReal)0.).ClampAxes(MinCoefficient, MaxCoefficient))
		, OuterLift(GetWeightedFloatOuterLift(PropertyCollection, FSolverVec2(Lift.GetLow(), Lift.GetHigh())).ClampAxes(MinCoefficient, MaxCoefficient))
		, Drag(GetWeightedFloatDrag(PropertyCollection, (FSolverReal)0.).ClampAxes(MinCoefficient, MaxCoefficient))
		, OuterDrag(GetWeightedFloatOuterDrag(PropertyCollection, FSolverVec2(Drag.GetLow(), Drag.GetHigh())).ClampAxes(MinCoefficient, MaxCoefficient))
		, Pressure(GetWeightedFloatPressure(PropertyCollection, (FSolverReal)0.))
		, Rho(FMath::Max(GetFluidDensity(PropertyCollection, (FSolverReal)0.), (FSolverReal)0.))
		, QuarterRho(Rho * (FSolverReal)0.25f)
		, TurbulenceRatio(FMath::Clamp(GetTurbulenceRatio(PropertyCollection, DefaultTurbulenceRatio), 0.f, 1.f))
		, DragIndex(PropertyCollection)
		, OuterDragIndex(PropertyCollection)
		, LiftIndex(PropertyCollection)
		, OuterLiftIndex(PropertyCollection)
		, FluidDensityIndex(PropertyCollection)
		, PressureIndex(PropertyCollection)
		, TurbulenceRatioIndex(PropertyCollection)
		, WindVelocityIndex(PropertyCollection)
		, WindVelocitySpaceIndex(PropertyCollection)
	{
	}

	FVelocityAndPressureField(
		const FSolverParticlesRange& Particles,
		const FTriangleMesh* TriangleMesh,
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& Weightmaps,
		FSolverReal WorldScale)
		: Lift(GetWeightedFloatLift(PropertyCollection, (FSolverReal)0.).ClampAxes(MinCoefficient, MaxCoefficient))
		, OuterLift(GetWeightedFloatOuterLift(PropertyCollection, FSolverVec2(Lift.GetLow(), Lift.GetHigh())).ClampAxes(MinCoefficient, MaxCoefficient))
		, Drag(GetWeightedFloatDrag(PropertyCollection, (FSolverReal)0.).ClampAxes(MinCoefficient, MaxCoefficient))
		, OuterDrag(GetWeightedFloatOuterDrag(PropertyCollection, FSolverVec2(Drag.GetLow(), Drag.GetHigh())).ClampAxes(MinCoefficient, MaxCoefficient))
		, Pressure(GetWeightedFloatPressure(PropertyCollection, (FSolverReal)0.)/WorldScale)
		, Rho(FMath::Max(GetFluidDensity(PropertyCollection, (FSolverReal)0.)/FMath::Cube(WorldScale), (FSolverReal)0.))
		, QuarterRho(Rho* (FSolverReal)0.25f)
		, TurbulenceRatio(FMath::Clamp(GetTurbulenceRatio(PropertyCollection, DefaultTurbulenceRatio), 0.f, 1.f))
		, DragIndex(PropertyCollection)
		, OuterDragIndex(PropertyCollection)
		, LiftIndex(PropertyCollection)
		, OuterLiftIndex(PropertyCollection)
		, FluidDensityIndex(PropertyCollection)
		, PressureIndex(PropertyCollection)
		, TurbulenceRatioIndex(PropertyCollection)
		, WindVelocityIndex(PropertyCollection)
		, WindVelocitySpaceIndex(PropertyCollection)
	{
		SetGeometry(Particles, TriangleMesh);
		SetMultipliers(PropertyCollection, Weightmaps);
		InitColor(Particles);
	}

	// Construct an uninitialized field. Mesh, properties, and velocity will have to be set for this field to be valid.
	FVelocityAndPressureField()
		: Offset(INDEX_NONE)
		, NumParticles(0)
		, Lift(FSolverVec2(0.))
		, OuterLift(FSolverVec2(0.))
		, Drag(FSolverVec2(0.))
		, OuterDrag(FSolverVec2(0.))
		, Pressure(FSolverVec2(0.))
		, Rho((FSolverReal)0.)
		, QuarterRho(Rho* (FSolverReal)0.25f)
		, TurbulenceRatio(DefaultTurbulenceRatio)
		, DragIndex(ForceInit)
		, OuterDragIndex(ForceInit)
		, LiftIndex(ForceInit)
		, OuterLiftIndex(ForceInit)
		, FluidDensityIndex(ForceInit)
		, PressureIndex(ForceInit)
		, TurbulenceRatioIndex(ForceInit)
		, WindVelocityIndex(ForceInit)
		, WindVelocitySpaceIndex(ForceInit)
	{
	}

	virtual ~FVelocityAndPressureField() = default;

	CHAOS_API void UpdateForces(const FSolverParticles& InParticles, const FSolverReal /*Dt*/);

	inline void Apply(FSolverParticles& InParticles, const FSolverReal Dt, const int32 Index) const
	{
		checkSlow(Index >= Offset && Index < Offset + NumParticles);  // The index should always match the original triangle mesh range

		const TArray<int32>& ElementIndices = PointToTriangleMap[Index];
		for (const int32 ElementIndex : ElementIndices)
		{
			InParticles.Acceleration(Index) += InParticles.InvM(Index) * Forces[ElementIndex];
		}
	}

	CHAOS_API void Apply(FSolverParticlesRange& InParticles, const FSolverReal Dt) const;

	// This version will not load WindVelocity from the config. Call SetVelocity to set it explicitly.
	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& Weightmaps,
		FSolverReal WorldScale,
		bool bEnableAerodynamics);

	// This version will load WindVelocity from the config
	// Provide LocalSpaceRotation and/or ReferenceSpaceRotation to convert WindVelocity to solver space based on WindVelocitySpace
	CHAOS_API void SetPropertiesAndWind(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& Weightmaps,
		FSolverReal WorldScale,
		bool bEnableAerodynamics,
		const FSolverVec3& SolverWind,
		const FRotation3& LocalSpaceRotation = FRotation3::Identity,
		const FRotation3& ReferenceSpaceRotation = FRotation3::Identity);

	CHAOS_API void SetProperties(
		const FSolverVec2& Drag,
		const FSolverVec2& OuterDrag,
		const FSolverVec2& Lift,
		const FSolverVec2& OuterLift,
		const FSolverReal FluidDensity,
		const FSolverVec2& Pressure = FSolverVec2::ZeroVector,
		FSolverReal WorldScale = 1.f);

	bool IsActive() const 
	{ 
		return Pressure.GetLow() != (FSolverReal)0. || Pressure.GetHigh() != (FSolverReal)0. ||
			(AreAerodynamicsEnabled() && (
				Drag.GetLow() > (FSolverReal)0. || Drag.GetOffsetRange()[1] != (FSolverReal)0. ||  // Note: range can be a negative value (although not when Lift or Drag base is zero)
				OuterDrag.GetLow() > (FSolverReal)0. || OuterDrag.GetOffsetRange()[1] != (FSolverReal)0. ||
				Lift.GetLow() > (FSolverReal)0. || Lift.GetOffsetRange()[1] != (FSolverReal)0. ||
				OuterLift.GetLow() > (FSolverReal)0. || OuterLift.GetOffsetRange()[1] != (FSolverReal)0.));
	}

	CHAOS_API void SetGeometry(
		const FTriangleMesh* TriangleMesh,
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& Weightmaps,
		FSolverReal WorldScale);

	CHAOS_API void SetGeometry(
		const FTriangleMesh* TriangleMesh,
		const TConstArrayView<FRealSingle>& DragMultipliers,
		const TConstArrayView<FRealSingle>& OuterDragMultipliers,
		const TConstArrayView<FRealSingle>& LiftMultipliers,
		const TConstArrayView<FRealSingle>& OuterLiftMultipliers,
		const TConstArrayView<FRealSingle>& PressureMultipliers);

	UE_DEPRECATED(5.5, "Use SetGeometry with OuterDrag and OuterLift multipliers")
	void SetGeometry(
		const FTriangleMesh* TriangleMesh,
		const TConstArrayView<FRealSingle>& DragMultipliers,
		const TConstArrayView<FRealSingle>& LiftMultipliers,
		const TConstArrayView<FRealSingle>& PressureMultipliers = TConstArrayView<FRealSingle>())
	{
		SetGeometry(TriangleMesh, DragMultipliers, DragMultipliers, LiftMultipliers, LiftMultipliers, PressureMultipliers);
	}

	void SetVelocity(const FSolverVec3& InVelocity) { Velocity = InVelocity; }

	const FSolverVec3& GetVelocity() const { return Velocity; }

	TConstArrayView<TVector<int32, 3>> GetElements() const { return TConstArrayView<TVector<int32, 3>>(Elements); }
	TConstArrayView<FSolverVec3> GetForces() const { return TConstArrayView<FSolverVec3>(Forces); }

	// This method is currently used for debug drawing.
	CHAOS_API FSolverVec3 CalculateForce(const TConstArrayView<FSolverVec3>& Xs, const TConstArrayView<FSolverVec3>& Vs, int32 ElementIndex) const;

	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(Drag, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(OuterDrag, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(Lift, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(OuterLift, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(FluidDensity, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(Pressure, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(TurbulenceRatio, float);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(WindVelocity, FVector3f);
	UE_CHAOS_DECLARE_INDEXLESS_PROPERTYCOLLECTION_NAME(WindVelocitySpace, int32);

protected:
	bool AreAerodynamicsEnabled() const { return QuarterRho > (FSolverReal)0.; }

	CHAOS_API void InitColor(const FSolverParticlesRange& InParticles);
	CHAOS_API void ResetColor(); // Used when setting geometry without Particles

	CHAOS_API void SetGeometry(const FSolverParticlesRange& Particles, const FTriangleMesh* TriangleMesh);
	CHAOS_API void SetGeometry(const FTriangleMesh* TriangleMesh);

	CHAOS_API void SetMultipliers(const FCollectionPropertyConstFacade& PropertyCollection,const TMap<FString, TConstArrayView<FRealSingle>>& Weightmaps);

	CHAOS_API void SetMultipliers(
		const TConstArrayView<FRealSingle>& DragMultipliers,
		const TConstArrayView<FRealSingle>& OuterDragMultipliers,
		const TConstArrayView<FRealSingle>& LiftMultipliers,
		const TConstArrayView<FRealSingle>& OuterLiftMultipliers,
		const TConstArrayView<FRealSingle>& PressureMultipliers);


	static FSolverVec3 CalculateForceLowLevel(const FSolverVec3 &InV, const FSolverVec3 &InN, const FSolverReal InQuarterRho, const FSolverReal InDoubleArea, 
		FSolverReal InCdI, FSolverReal InCdO, const FSolverReal InClI, const FSolverReal InClO, const FSolverReal InCp, const FSolverReal InTr)
	{		
		// Set the aerodynamic forces
		const FSolverReal VDotN = FSolverVec3::DotProduct(InV, InN);
		const FSolverReal VSquare = FSolverVec3::DotProduct(InV, InV);

		const FSolverReal TurbulenceFactor = VSquare > UE_SMALL_NUMBER ? ((1.f - InTr) / FMath::Sqrt(VSquare) + InTr) : 1.f;

		return InQuarterRho * InDoubleArea * TurbulenceFactor * (VDotN >= (FSolverReal)0. ?  // The flow can hit either side of the triangle, so the normal might need to be reversed
			(InCdI - InClI) * VDotN * InV + InClI * VSquare * InN :
			(InClO - InCdO) * VDotN * InV - InClO * VSquare * InN) + InDoubleArea * (FSolverReal)0.5 * InCp * InN;
	}

	FSolverVec3 CalculateForce(const TConstArrayView<FSolverVec3>& Xs, const TConstArrayView<FSolverVec3>& Vs, int32 ElementIndex, const FSolverVec3& InVelocity, const FSolverReal CdI, const FSolverReal CdO, const FSolverReal ClI, const FSolverReal ClO, const FSolverReal Cp) const
	{
		const TVec3<int32>& Element = Elements[ElementIndex];

		// Calculate the normal and the area of the surface exposed to the flow
		FSolverVec3 N = FSolverVec3::CrossProduct(
			Xs[Element[2]] - Xs[Element[0]],
			Xs[Element[1]] - Xs[Element[0]]);
		const FSolverReal DoubleArea = N.SafeNormalize();

		// Calculate the direction and the relative velocity of the triangle to the flow
		const FSolverVec3& SurfaceVelocity = (FSolverReal)(1. / 3.) * (
			Vs[Element[0]] +
			Vs[Element[1]] +
			Vs[Element[2]]);
		const FSolverVec3 V = InVelocity - SurfaceVelocity;

		return CalculateForceLowLevel(V, N, QuarterRho, DoubleArea, CdI, CdO, ClI, ClO, Cp, TurbulenceRatio);
	}

	void UpdateField(const FSolverParticles& InParticles, int32 ElementIndex, const FSolverVec3& InVelocity, const FSolverReal CdI, const FSolverReal CdO, const FSolverReal ClI, const FSolverReal ClO, const FSolverReal Cp)
	{
		Forces[ElementIndex] = CalculateForce(TConstArrayView<FSolverVec3>(InParticles.XArray()), TConstArrayView<FSolverVec3>(InParticles.GetV()), ElementIndex, InVelocity, CdI, CdO, ClI, ClO, Cp);
	}

	FSolverVec3 CalculateForce(const TConstArrayView<FSolverVec3>& Xs, const TConstArrayView<FSolverVec3>& Vs, int32 ElementIndex, const FSolverVec3& InVelocity, const FSolverReal CdI, const FSolverReal CdO, const FSolverReal ClI, const FSolverReal ClO, const FSolverReal Cp, const FSolverReal MaxVelocitySquared) const
	{
		checkSlow(MaxVelocitySquared > (FSolverReal)0);

		const TVec3<int32>& Element = Elements[ElementIndex];

		// Calculate the normal and the area of the surface exposed to the flow
		FSolverVec3 N = FSolverVec3::CrossProduct(
			Xs[Element[2]] - Xs[Element[0]],
			Xs[Element[1]] - Xs[Element[0]]);
		const FSolverReal DoubleArea = N.SafeNormalize();

		// Calculate the direction and the relative velocity of the triangle to the flow
		const FSolverVec3& SurfaceVelocity = (FSolverReal)(1. / 3.) * (
			Vs[Element[0]] +
			Vs[Element[1]] +
			Vs[Element[2]]);
		FSolverVec3 V = InVelocity - SurfaceVelocity;

		// Clamp the velocity
		const FSolverReal RelVelocitySquared = V.SquaredLength();
		if (RelVelocitySquared > MaxVelocitySquared)
		{
			V *= FMath::Sqrt(MaxVelocitySquared / RelVelocitySquared);
		}


		return CalculateForceLowLevel(V, N, QuarterRho, DoubleArea, CdI, CdO, ClI, ClO, Cp, TurbulenceRatio);
	}

	void UpdateField(const FSolverParticles& InParticles, int32 ElementIndex, const FSolverVec3& InVelocity, const FSolverReal CdI, const FSolverReal CdO, const FSolverReal ClI, const FSolverReal ClO, const FSolverReal Cp, const FSolverReal MaxVelocitySquared)
	{
		Forces[ElementIndex] = CalculateForce(TConstArrayView<FSolverVec3>(InParticles.XArray()), TConstArrayView<FSolverVec3>(InParticles.GetV()), ElementIndex, InVelocity, CdI, CdO, ClI, ClO, Cp, MaxVelocitySquared);
	}

protected:
	int32 Offset;
	int32 NumParticles;
	TConstArrayView<TArray<int32>> PointToTriangleMap; // Points use global indexing. May point to PointToTriangleMapLocal or data in the original FTriangleMesh
	TConstArrayView<TVec3<int32>> Elements; // May point to ElementsLocal or data in the original FTriangleMesh
	TArray<TArray<int32>> PointToTriangleMapLocal; // Points use local indexing. Only used with ElementsLocal.
	TArray<TVec3<int32>> ElementsLocal; // Local copy of the triangle mesh's elements. Kinematic faces have been removed, and may be reordered by coloring.
	TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.
	FPBDFlatWeightMap Lift;
	FPBDFlatWeightMap OuterLift;
	FPBDFlatWeightMap Drag;
	FPBDFlatWeightMap OuterDrag;
	FPBDFlatWeightMap Pressure;

	TArray<FSolverVec3> Forces;
	FSolverVec3 Velocity;
	FSolverReal Rho;
	FSolverReal QuarterRho;
	FSolverReal TurbulenceRatio = 1.f;

	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(Drag, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(OuterDrag, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(Lift, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(OuterLift, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(FluidDensity, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(Pressure, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(TurbulenceRatio, float);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(WindVelocity, FVector3f);
	UE_CHAOS_DECLARE_INDEXED_PROPERTYCOLLECTION_NAME(WindVelocitySpace, int32);
};

}  // End namespace Chaos::Softs

#if !defined(CHAOS_VELOCITY_FIELD_ISPC_ENABLED_DEFAULT)
#define CHAOS_VELOCITY_FIELD_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || (UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
static constexpr bool bChaos_VelocityField_ISPC_Enabled = INTEL_ISPC && CHAOS_VELOCITY_FIELD_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_VelocityField_ISPC_Enabled;
#endif
