// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ImplicitObject.h"
#include "Chaos/ArrayND.h"
#include "Chaos/AABB.h"
#include "Chaos/HierarchicalSpatialHash.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"

struct FKSkinnedLevelSetElem;

namespace Chaos
{
struct FWeightedLatticeImplicitObjectFactory;

struct FWeightedLatticeInfluenceData
{
	FWeightedLatticeInfluenceData()
	: NumInfluences(0)
	{
		FMemory::Memset(BoneIndices, (uint8)INDEX_NONE, sizeof(BoneIndices));
		FMemory::Memset(BoneWeights, 0, sizeof(BoneWeights));
	}

	uint32 GetTypeHash() const
	{
		uint32 Result = ::GetTypeHash(NumInfluences);
		for (uint8 Index = 0; Index < NumInfluences; ++Index)
		{
			Result = HashCombine(Result, ::GetTypeHash(BoneIndices[Index]));
			Result = HashCombine(Result, ::GetTypeHash(BoneWeights[Index]));
		}
		return Result;
	}

	static const uint8 MaxTotalInfluences = 12;
	uint8 NumInfluences;
	uint16 BoneIndices[MaxTotalInfluences]; // This is index into FWeightedLatticeImplicitObject::UsedBones (which then maps into skeletalmesh bones)
	float BoneWeights[MaxTotalInfluences];
};

inline FArchive& operator<<(FArchive& Ar, FWeightedLatticeInfluenceData& Value)
{
	Ar << Value.NumInfluences;
	for (int32 Index = 0; Index < Value.NumInfluences; ++Index)
	{
		Ar << Value.BoneIndices[Index];
		Ar << Value.BoneWeights[Index];
	}
	return Ar;
}

/**
 * Embed the contained shape in a deformable lattice.
 */
class FWeightedLatticeImplicitObject : public FImplicitObject
{
public:
	using ObjectType = FImplicitObjectPtr;
	using FImplicitObject::GetTypeName;

	CHAOS_API FWeightedLatticeImplicitObject(int32 Flags, EImplicitObjectType InType, TUniformGrid<FReal, 3>&& InGrid,
		TArrayND<FWeightedLatticeInfluenceData, 3>&& InBoneData, TArray<FName>&& InUsedBones, TArray<FTransform>&& InReferenceRelativeTransforms);

	CHAOS_API FWeightedLatticeImplicitObject(FWeightedLatticeImplicitObject&& Other);
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FWeightedLatticeImplicitObject() override = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual const FAABB3 BoundingBox() const override { return LocalBoundingBox; }
	const TArrayND<FWeightedLatticeInfluenceData, 3>& GetBoneData() const { return BoneData; }
	const TArrayND<FVec3, 3>& GetDeformedPoints() const { return DeformedPoints; }
	const TArrayND<bool, 3>& GetEmptyCells() const { return EmptyCells; }
	const TUniformGrid<FReal, 3>& GetGrid() const { return Grid; }
	const TArray<FName>& GetUsedBones() const { return UsedBones; }

	UE_DEPRECATED(5.6, "SolverBoneIndices are no longer stored on WeightedLatticeImplicitObjects")
		const TArray<int32>& GetSolverBoneIndices() const
	{
		static TArray<int32> Dummy;
		return Dummy;
	}
	UE_DEPRECATED(5.6, "SolverBoneIndices are no longer stored on WeightedLatticeImplicitObjects")
	void SetSolverBoneIndices(TArray<int32>&& InSolverBoneIndices) { }

	CHAOS_API virtual void Serialize(FChaosArchive& Ar) override;

	CHAOS_API FVec3 GetDeformedPoint(const FVec3& UndeformedPoint) const;
	CHAOS_API void UpdateSpatialHierarchy(const FReal MinLodSize = 0.);
	CHAOS_API void FinalizeConstruction();
	CHAOS_API void DeformPoints(const TArray<FTransform>& RelativeTransforms);

	struct FEmbeddingCoordinate
	{
		TVec3<int32> CellIndex = TVec3<int32>(INDEX_NONE, INDEX_NONE, INDEX_NONE);
		int32 LocalTetrahedron = INDEX_NONE;
		FVec3 BarycentricCoordinate;

		FEmbeddingCoordinate() = default;
		FEmbeddingCoordinate(const FEmbeddingCoordinate& Other) = default;
		FEmbeddingCoordinate& operator=(const FEmbeddingCoordinate& Other) = default;

		FEmbeddingCoordinate(const TVec3<int32>& InCellIndex, int32 InLocalTetrahedron, const FVec3& InBarycentric)
			:CellIndex(InCellIndex), LocalTetrahedron(InLocalTetrahedron), BarycentricCoordinate(InBarycentric)
		{}

		CHAOS_API FEmbeddingCoordinate(const TVec3<int32>& InCellIndex, const FVec3& TrilinearCoordinate);

		bool IsValid() const
		{
			checkSlow((CellIndex == TVec3<int32>(INDEX_NONE, INDEX_NONE, INDEX_NONE)) == (LocalTetrahedron == INDEX_NONE));
			return LocalTetrahedron != INDEX_NONE;
		}

		inline static const TVec4<TVec3<int32>> EvenIndexTetrahedraOffsets[5] =
		{
			TVec4<TVec3<int32>>(TVec3<int32>(0,1,0), TVec3<int32>(0,0,0), TVec3<int32>(1,1,0), TVec3<int32>(0,1,1)),
			TVec4<TVec3<int32>>(TVec3<int32>(1,1,1), TVec3<int32>(1,1,0), TVec3<int32>(1,0,1), TVec3<int32>(0,1,1)),
			TVec4<TVec3<int32>>(TVec3<int32>(1,0,0), TVec3<int32>(1,0,1), TVec3<int32>(1,1,0), TVec3<int32>(0,0,0)),
			TVec4<TVec3<int32>>(TVec3<int32>(0,0,1), TVec3<int32>(0,0,0), TVec3<int32>(0,1,1), TVec3<int32>(1,0,1)),
			TVec4<TVec3<int32>>(TVec3<int32>(1,1,0), TVec3<int32>(0,1,1), TVec3<int32>(0,0,0), TVec3<int32>(1,0,1)),
		};

		inline static const TVec4<TVec3<int32>> OddIndexTetrahedraOffsets[5] =
		{
			TVec4<TVec3<int32>>(TVec3<int32>(0,0,0), TVec3<int32>(1,0,0), TVec3<int32>(0,1,0), TVec3<int32>(0,0,1)),
			TVec4<TVec3<int32>>(TVec3<int32>(0,1,1), TVec3<int32>(0,1,0), TVec3<int32>(1,1,1), TVec3<int32>(0,0,1)),
			TVec4<TVec3<int32>>(TVec3<int32>(1,0,1), TVec3<int32>(1,1,1), TVec3<int32>(1,0,0), TVec3<int32>(0,0,1)),
			TVec4<TVec3<int32>>(TVec3<int32>(1,1,0), TVec3<int32>(0,1,0), TVec3<int32>(1,0,0), TVec3<int32>(1,1,1)),
			TVec4<TVec3<int32>>(TVec3<int32>(1,0,0), TVec3<int32>(1,1,1), TVec3<int32>(0,1,0), TVec3<int32>(0,0,1)),
		};

		inline const TVec4<TVec3<int32>>& TetrahedronOffsets() const
		{
			checkSlow(LocalTetrahedron >= 0 && LocalTetrahedron < 5);

			if (((CellIndex.X + CellIndex.Y + CellIndex.Z) & 1) == 0)
			{
				return EvenIndexTetrahedraOffsets[LocalTetrahedron];
			}
			else
			{
				return OddIndexTetrahedraOffsets[LocalTetrahedron];
			}
		}

		CHAOS_API FMatrix DeformationTransform(const TArrayND<FVec3, 3>& DeformedPoints, const TUniformGrid<FReal, 3>& Grid) const;
		inline FVec3 UndeformedPosition(const TUniformGrid<FReal, 3>& InGrid) const;
		inline FVec3 DeformedPosition(const TArrayND<FVec3, 3>& InDeformedPoints) const;
		inline int32 GreatestInfluenceBone(const TArrayND<FWeightedLatticeInfluenceData, 3>& InBoneData) const;
	};

	CHAOS_API bool GetEmbeddingCoordinates(const FVec3& DeformedPoint, TArray<FEmbeddingCoordinate>& CoordinatesOut, bool bFindClosest = false) const;

protected:
	FWeightedLatticeImplicitObject(int32 Flags, EImplicitObjectType InType)
		:FImplicitObject(Flags, InType | ImplicitObjectType::IsWeightedLattice)
	{}

	CHAOS_API FWeightedLatticeImplicitObject(const FWeightedLatticeImplicitObject& Other);

	CHAOS_API void InitializeDeformedPoints();
	CHAOS_API void SetEmptyCells();
	CHAOS_API uint32 GetTypeHashHelper(const uint32 InHash) const;


	// Serialized data. Only non-const because of serialization
	TUniformGrid<FReal, 3> Grid;
	TArrayND<FWeightedLatticeInfluenceData, 3> BoneData;
	TArray<FName> UsedBones;
	FTransform ReferenceRootTransform;
	TArray<FTransform> ReferenceRelativeTransforms; // ReferenceRootTransform * RefBaseMatrixInv(UsedBoneIdx)

	// Non-serialized cached (const) data derived from serialized data
	TArrayND<bool, 3> EmptyCells;

	// Context-specific data (e.g., data a specific solver will need to do DeformPoints)
	UE_DEPRECATED(5.6, "SolverBoneIndices are deprecated")
	TArray<int32> SolverBoneIndices; // indices set by the solver for bone transform query

	// Non-serialized deforming data
	TAABB<FReal, 3> LocalBoundingBox;
	TArrayND<FVec3, 3> DeformedPoints;
	THierarchicalSpatialHash<int32, FReal> Spatial;
	bool bSpatialDirty = true;
};

template<typename TConcrete>
class TWeightedLatticeImplicitObject : public FWeightedLatticeImplicitObject
{
public:
	using T = typename TConcrete::TType;
	using TType = T;
	using ObjectType = TRefCountPtr<TConcrete>;

	TWeightedLatticeImplicitObject(ObjectType&& InObject, TUniformGrid<FReal, 3>&& InGrid,
		TArrayND<FWeightedLatticeInfluenceData, 3>&& InBoneData, TArray<FName>&& InUsedBones, TArray<FTransform>&& InReferenceRelativeTransforms);

	virtual ~TWeightedLatticeImplicitObject() override = default;

	TWeightedLatticeImplicitObject(const TWeightedLatticeImplicitObject& Other) = delete;

	TWeightedLatticeImplicitObject(TWeightedLatticeImplicitObject&& Other);

	static constexpr EImplicitObjectType StaticType()
	{
		return TConcrete::StaticType() | ImplicitObjectType::IsWeightedLattice;
	}

	const TConcrete* GetEmbeddedObject() const
	{
		return Object.GetReference();
	}

	virtual FImplicitObjectPtr CopyGeometry() const override;
	virtual FImplicitObjectPtr DeepCopyGeometry() const override;

	virtual uint32 GetTypeHash() const override
	{
		return GetTypeHashHelper(Object->GetTypeHash());
	}

	// Note that if x is in an empty cell / outside the deformed lattice and bIncludeEmptyCells is false, this will return UE_BIG_NUMBER and Normal will be unchanged.
	// SurfaceCoord will be invalid.
	// When bIncludeEmptyCells is true, the closest non-empty lattice cell will be used to find the closest surface point.
	FReal PhiWithNormalAndSurfacePoint(const FVec3& X, FVec3& Normal, FEmbeddingCoordinate& SurfaceCoord, bool bIncludeEmptyCells = false) const;

	// Batch calls used by cloth.
	// Note that if x is in an empty cell / outside the deformed lattice, Phi will be UE_BIG_NUMBER and Normal will be unchanged (BatchNormals will be resized using SetNumUninitialized, so it may be left uninitialized)
	// GreatestInfluenceBone will not be calculated (and set to INDEX_NONE) if Phi is >= Thickness.
	// GreatestInfluenceBone is an index into UsedBones
	// All outputs will be unchanged (and possibly uninitialized) for points with InvM == 0.
	void BatchPhiWithNormalAndGreatestInfluenceBone(const TConstArrayView<Softs::FPAndInvM> PAndInvMArray,
		const Softs::FSolverRigidTransform3& SolverToThis, const Softs::FSolverReal Thickness,
		TArray<Softs::FSolverReal>& OutBatchPhis, TArray<Softs::FSolverVec3>& OutBatchNormals, TArray<int32>& OutBatchGreatestInfluenceBones) const;

	// Batch calls used by cloth.
	// Note that if x is in an empty cell / outside the deformed lattice, Phi will be UE_BIG_NUMBER and Normal will be unchanged (BatchNormals will be resized using SetNumUninitialized, so it may be left uninitialized)
	// All outputs will be unchanged (and possibly uninitialized) for points with InvM == 0.
	void BatchPhiWithNormal(const TConstArrayView<Softs::FPAndInvM> PAndInvMArray,
		const Softs::FSolverRigidTransform3& SolverToThis, TArray<Softs::FSolverReal>& OutBatchPhis, TArray<Softs::FSolverVec3>& OutBatchNormals) const;

	virtual FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const override
	{
		FEmbeddingCoordinate SurfaceCoordUnused;
		constexpr bool bIncludeEmptyCells = true;
		return PhiWithNormalAndSurfacePoint(x, Normal, SurfaceCoordUnused, bIncludeEmptyCells);
	}

	virtual void Serialize(FChaosArchive& Ar) override
	{
		FChaosArchiveScopedMemory ScopedMemory(Ar, GetTypeName(), false);
		FWeightedLatticeImplicitObject::Serialize(Ar);
		Ar << Object;
	}

private:

	// Serialized data
	ObjectType Object;

	TWeightedLatticeImplicitObject(ObjectType&& InObject, const FWeightedLatticeImplicitObject& Other)
		: FWeightedLatticeImplicitObject(Other)
		, Object(MoveTemp(InObject))
	{}

	//needed for serialization
	TWeightedLatticeImplicitObject() 
		: FWeightedLatticeImplicitObject(EImplicitObject::HasBoundingBox, StaticType()) 
		, Object(nullptr) 
	{}

	friend FImplicitObject;	//needed for serialization
	friend ::FKSkinnedLevelSetElem; //needed for serialization
};

class FWeightedLatticeImplicitObjectBuilder
{
public:

	CHAOS_API void GenerateGrid(const int32 GridResolution, const TAABB<FReal, 3>& ObjectBBox);
	CHAOS_API void AddInfluence(int32 FlatIndex, uint16 BoneIndex, float Weight, bool bIsOuterWeight);

	// BoneIndexToBoneName(int32 BoneIndex)->FName return FName of Bone for given BoneIndex (BoneIndex used by AddInfluence)
	// BoneIndexToReferenceTransform(int32 BoneIndex)->FTransform return transform of Bone (BoneIndex used by AddInfluence). Should return transform for RootBone (parent of this object) when BoneIndex == INDEX_NONE
	template<typename FBoneIndexToBoneName, typename FBoneIndexToReferenceTransform>
	inline void FinalizeInfluences(const FBoneIndexToBoneName& BoneIndexToBoneName, const FBoneIndexToReferenceTransform& BoneIndexToReferenceTransform);

	template<typename TConcrete>
	UE_DEPRECATED(5.4, "Please use Generate with TRefCountPtr instead")
	inline TUniquePtr< TWeightedLatticeImplicitObject<TConcrete> > Generate(TUniquePtr<TConcrete>&& Object);
	
	template<typename TConcrete>
	inline TRefCountPtr< TWeightedLatticeImplicitObject<TConcrete> > Generate(TRefCountPtr<TConcrete>&& Object);

	const TUniformGrid<FReal, 3>& GetGrid() const { return Grid; }
	const TArrayND<FWeightedLatticeInfluenceData, 3>& GetBoneData() const { return BoneData; }

private:

	CHAOS_API void NormalizeBoneWeights();

	template<typename FBoneIndexToBoneName, typename FBoneIndexToReferenceTransform>
	inline void CalcUsedBonesAndReIndex(const FBoneIndexToBoneName& BoneIndexToBoneName, const FBoneIndexToReferenceTransform& BoneIndexToReferenceTransform);


	TUniformGrid<FReal, 3> Grid;
	TArrayND<FWeightedLatticeInfluenceData, 3> BoneData;
	TArray<FName> UsedBones;
	TArray<FTransform> ReferenceRelativeTransforms;

	// Add on to FWeightedLatticeInfluenceData
	struct FInfluenceBuildData
	{
		bool WeightsAreOuter = true;
	};
	TArray<FInfluenceBuildData> BuildData;

	enum struct EBuildStep
	{
		None = 0,
		GridValid,
		InfluencesFinalized,
		Finished = None, // Once Finished, all data has been moved out and the builder is no longer valid
	};
	EBuildStep BuildStep = EBuildStep::None;
};

// Not actually a collision object. CollisionParticles that are used to update bones used by the FWeightedLatticeImplicitObject have this as their geometry
class UE_DEPRECATED(5.6, "FWeightedLatticeBoneProxy have been deprecated. Do not use.") FWeightedLatticeBoneProxy : public FImplicitObject
{
public:
	using FImplicitObject::GetTypeName;

	FWeightedLatticeBoneProxy()
		: FImplicitObject(EImplicitObject::DisableCollisions, ImplicitObjectType::Unknown)
	{}

	static constexpr EImplicitObjectType StaticType() { return ImplicitObjectType::Unknown; }

	virtual FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const override
	{
		return UE_BIG_NUMBER;
	}

	virtual uint32 GetTypeHash() const override
	{
		return 0;
	}
};


// inline definitions
inline FVec3 FWeightedLatticeImplicitObject::FEmbeddingCoordinate::UndeformedPosition(const TUniformGrid<FReal, 3>& InGrid) const
{
	const TVec4<TVec3<int32>>& TetOffsets = TetrahedronOffsets();
	return BarycentricCoordinate.X * InGrid.Node(CellIndex + TetOffsets[0]) + BarycentricCoordinate.Y * InGrid.Node(CellIndex + TetOffsets[1]) + BarycentricCoordinate.Z * InGrid.Node(CellIndex + TetOffsets[2])
		+ (1. - BarycentricCoordinate.X - BarycentricCoordinate.Y - BarycentricCoordinate.Z) * InGrid.Node(CellIndex + TetOffsets[3]);
}

inline FVec3 FWeightedLatticeImplicitObject::FEmbeddingCoordinate::DeformedPosition(const TArrayND<FVec3, 3>& InDeformedPoints) const
{
	const TVec4<TVec3<int32>>& TetOffsets = TetrahedronOffsets();

	return BarycentricCoordinate.X * InDeformedPoints(CellIndex + TetOffsets[0]) + BarycentricCoordinate.Y * InDeformedPoints(CellIndex + TetOffsets[1]) + BarycentricCoordinate.Z * InDeformedPoints(CellIndex + TetOffsets[2])
		+ (1. - BarycentricCoordinate.X - BarycentricCoordinate.Y - BarycentricCoordinate.Z) * InDeformedPoints(CellIndex + TetOffsets[3]);
}

inline int32 FWeightedLatticeImplicitObject::FEmbeddingCoordinate::GreatestInfluenceBone(const TArrayND<FWeightedLatticeInfluenceData, 3>& InBoneData) const
{
	if (!IsValid())
	{
		return INDEX_NONE;
	}

	const TVec4<TVec3<int32>>& TetOffsets = TetrahedronOffsets();
	int32 BoneIndex = INDEX_NONE;
	const TVec4<float> BarycentricCoordinate4((float)BarycentricCoordinate.X, (float)BarycentricCoordinate.Y, 
		(float)BarycentricCoordinate.Z, 1.f - (float)BarycentricCoordinate.X - (float)BarycentricCoordinate.Y - (float)BarycentricCoordinate.Z);
	float LargestWeight = 0.f;
	for (int32 Coord = 0; Coord < 4; ++Coord)
	{
		const FWeightedLatticeInfluenceData& Influences = InBoneData(CellIndex + TetOffsets[Coord]);
		for (uint8 InfIndex = 0; InfIndex < Influences.NumInfluences; ++InfIndex)
		{
			const float Weight = BarycentricCoordinate4[Coord] * Influences.BoneWeights[InfIndex];
			if (Weight > LargestWeight)
			{
				LargestWeight = Weight;
				BoneIndex = Influences.BoneIndices[InfIndex];
			}
		}
	}

	return BoneIndex;
}

template<typename FBoneIndexToBoneName, typename FBoneIndexToReferenceTransform>
inline void FWeightedLatticeImplicitObjectBuilder::CalcUsedBonesAndReIndex(const FBoneIndexToBoneName& BoneIndexToBoneName, const FBoneIndexToReferenceTransform& BoneIndexToReferenceTransform)
{
	TSet<uint16> UsedBoneSet;
	for (int32 Idx = 0; Idx < BoneData.Num(); ++Idx)
	{
		const FWeightedLatticeInfluenceData& Data = BoneData[Idx];
		for (uint8 InfIdx = 0; InfIdx < Data.NumInfluences; ++InfIdx)
		{
			UsedBoneSet.Add(Data.BoneIndices[InfIdx]);
		}
	}
	const TArray<uint16> UsedBoneIndices = UsedBoneSet.Array();
	UsedBones.Reset();
	UsedBones.SetNumUninitialized(UsedBoneIndices.Num());
	ReferenceRelativeTransforms.Reset();
	ReferenceRelativeTransforms.SetNumUninitialized(UsedBoneIndices.Num());

	const FTransform ReferenceRootTransform = (BoneIndexToReferenceTransform(INDEX_NONE)).Inverse();

	TMap<uint16, uint16> UsedBoneReverseLookup;
	for (uint16 ArrayIdx = 0; ArrayIdx < (uint16)UsedBoneIndices.Num(); ++ArrayIdx)
	{
		UsedBones[ArrayIdx] = BoneIndexToBoneName(UsedBoneIndices[ArrayIdx]);
		ReferenceRelativeTransforms[ArrayIdx] = ReferenceRootTransform * BoneIndexToReferenceTransform(UsedBoneIndices[ArrayIdx]);
		UsedBoneReverseLookup.Add(UsedBoneIndices[ArrayIdx], ArrayIdx);
	}

	for (int32 Idx = 0; Idx < BoneData.Num(); ++Idx)
	{
		FWeightedLatticeInfluenceData& Data = BoneData[Idx];
		for (uint8 InfIdx = 0; InfIdx < Data.NumInfluences; ++InfIdx)
		{
			Data.BoneIndices[InfIdx] = UsedBoneReverseLookup[Data.BoneIndices[InfIdx]];
		}
	}
}

template<typename FBoneIndexToBoneName, typename FBoneIndexToReferenceTransform>
inline void FWeightedLatticeImplicitObjectBuilder::FinalizeInfluences(const FBoneIndexToBoneName& BoneIndexToBoneName, const FBoneIndexToReferenceTransform& BoneIndexToReferenceTransform)
{
	check(BuildStep == EBuildStep::GridValid);
	NormalizeBoneWeights();
	CalcUsedBonesAndReIndex(BoneIndexToBoneName, BoneIndexToReferenceTransform);
	BuildStep = EBuildStep::InfluencesFinalized;
}

template<typename TConcrete>
inline TRefCountPtr< TWeightedLatticeImplicitObject<TConcrete> > FWeightedLatticeImplicitObjectBuilder::Generate(TRefCountPtr<TConcrete>&& Object)
{
	check(BuildStep == EBuildStep::InfluencesFinalized);

	TRefCountPtr<TWeightedLatticeImplicitObject<TConcrete>> Ret( new TWeightedLatticeImplicitObject<TConcrete>(
		MoveTemp(Object), MoveTemp(Grid), MoveTemp(BoneData), MoveTemp(UsedBones), MoveTemp(ReferenceRelativeTransforms)));

	BuildStep = EBuildStep::Finished;

	return Ret;
}
}
