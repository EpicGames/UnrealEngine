// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDBendingConstraintsBase.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/PBDFlatWeightMap.h"
#include "ChaosStats.h"

namespace Chaos
{
class FTriangleMesh;
}

namespace Chaos::Softs
{

class FXPBDAnisotropicBendingConstraints final : public FPBDBendingConstraintsBase
{
	typedef FPBDBendingConstraintsBase Base;

public:
	// Stiffness is in kg cm^2 / rad^2 s^2
	static constexpr FSolverReal MinStiffness = (FSolverReal)0; // We're not checking against MinStiffness (except when it's constant and == 0)
	static constexpr FSolverReal MaxStiffness = (FSolverReal)1e7;
	static constexpr FSolverReal MinDamping = (FSolverReal)0.;
	static constexpr FSolverReal MaxDamping = (FSolverReal)1000.;

	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsXPBDAnisoBendingStiffnessWarpEnabled(PropertyCollection, false);
	}

	CHAOS_API FXPBDAnisotropicBendingConstraints(const FSolverParticlesRange& InParticles,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection);

	CHAOS_API FXPBDAnisotropicBendingConstraints(const FSolverParticles& InParticles,
		int32 InParticleOffset,
		int32 InParticleCount,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection);

	CHAOS_API FXPBDAnisotropicBendingConstraints(const FSolverParticles & InParticles,
		int32 InParticleOffset,
		int32 InParticleCount,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
		const TConstArrayView<FRealSingle>& BucklingRatioMultipliers,
		const TConstArrayView<FRealSingle>& BucklingStiffnessWarpMultipliers,
		const TConstArrayView<FRealSingle>& BucklingStiffnessWeftMultipliers,
		const TConstArrayView<FRealSingle>& BucklingStiffnessBiasMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const TConstArrayView<FRealSingle>& RestAngleMap,
		const FSolverVec2& InStiffnessWarp,
		const FSolverVec2& InStiffnessWeft,
		const FSolverVec2& InStiffnessBias,
		const FSolverVec2& InBucklingRatio,
		const FSolverVec2& InBucklingStiffnessWarp,
		const FSolverVec2& InBucklingStiffnessWeft,
		const FSolverVec2& InBucklingStiffnessBias,
		const FSolverVec2& InDampingRatio,
		const FSolverVec2& RestAngleValue,
		ERestAngleConstructionType RestAngleConstructionType);

	virtual ~FXPBDAnisotropicBendingConstraints() override = default;

	template<typename SolverParticlesOrRange>
	CHAOS_API void Init(const SolverParticlesOrRange& InParticles);

	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);

	void ApplyProperties(const FSolverReal /*Dt*/, const int32 /*NumIterations*/)
	{
		BucklingRatioWeighted.ApplyValues();
	}

	template<typename SolverParticlesOrRange>
	CHAOS_API void Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const;

	const TArray<int32>& GetConstraintsPerColorStartIndex() const { return ConstraintsPerColorStartIndex; }
	const TArray<FSolverVec3>& GetWarpWeftBiasBaseMultipliers() const { return WarpWeftBiasBaseMultipliers; }

	CHAOS_API void AddAnisotropicBendingResidualAndHessian(const FSolverParticlesRange& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian);
	CHAOS_API void AddAnisotropicBendingResidualAndHessian(const FSolverParticles& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian);

	TArray<TArray<int32>> GetConstraintsArray() const
	{
		TArray<TArray<int32>> ConstraintsArray;
		ConstraintsArray.SetNum(Constraints.Num());
		for (int32 ConstraintIdx = 0; ConstraintIdx < Constraints.Num(); ConstraintIdx++)
		{
			ConstraintsArray[ConstraintIdx].SetNum(4);
			for (int32 LocalIdx = 0; LocalIdx < 4; LocalIdx++)
			{
				ConstraintsArray[ConstraintIdx][LocalIdx] = Constraints[ConstraintIdx][LocalIdx];
			}
		}
		return ConstraintsArray;
	}

	CHAOS_API void AddInternalForceDifferential(const FSolverParticlesRange& InParticles, const TArray<TVector<FSolverReal, 3>>& DeltaParticles, TArray<TVector<FSolverReal, 3>>& Ndf);
	CHAOS_API void AddInternalForceDifferential(const FSolverParticles& InParticles, const TArray<TVector<FSolverReal, 3>>& DeltaParticles, TArray<TVector<FSolverReal, 3>>& Ndf);

	void ResetRestLengths(const TConstArrayView<FSolverVec3>& Positions,
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
	{
		CalculateRestAngles(Positions, ParticleOffset, ParticleCount,
			GetRestAngleMapFromCollection(WeightMaps, PropertyCollection),
			GetRestAngleValueFromCollection(PropertyCollection),
			(ERestAngleConstructionType)GetXPBDAnisoRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles));
	}
private:
	template<typename SolverParticlesOrRange>
	void InitColor(const SolverParticlesOrRange& InParticles);
	template<bool bDampingOnly, bool bElasticOnly, typename SolverParticlesOrRange>
	void ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverVec3& ExpStiffnessValues,
		const FSolverVec3& ExpBucklingStiffnessValues, const FSolverReal DampingRatioValue) const;

	TArray<FSolverVec3> GenerateWarpWeftBiasBaseMultipliers(const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions, const FTriangleMesh& TriangleMesh) const;

	TConstArrayView<FRealSingle> GetRestAngleMapFromCollection(
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection)
	{
		const ERestAngleConstructionType ConstructionType = (ERestAngleConstructionType)GetXPBDAnisoRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles);

		switch (ConstructionType)
		{
		default:
		case ERestAngleConstructionType::Use3DRestAngles:
			return TConstArrayView<FRealSingle>(); // Unused
		case ERestAngleConstructionType::FlatnessRatio:
			return WeightMaps.FindRef(GetXPBDAnisoFlatnessRatioString(PropertyCollection, XPBDAnisoFlatnessRatioName.ToString()));
		case ERestAngleConstructionType::ExplicitRestAngles:
			return WeightMaps.FindRef(GetXPBDAnisoRestAngleString(PropertyCollection, XPBDAnisoRestAngleName.ToString()));
		}
	}

	FSolverVec2 GetRestAngleValueFromCollection(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		const ERestAngleConstructionType ConstructionType = (ERestAngleConstructionType)GetXPBDAnisoRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles);

		switch (ConstructionType)
		{
		default:
		case ERestAngleConstructionType::Use3DRestAngles:
			return FSolverVec2(0.f); // Unused
		case ERestAngleConstructionType::FlatnessRatio:
			return FSolverVec2(GetWeightedFloatXPBDAnisoFlatnessRatio(PropertyCollection, 0.f));
		case ERestAngleConstructionType::ExplicitRestAngles:
			return FSolverVec2(GetWeightedFloatXPBDAnisoRestAngle(PropertyCollection, 0.f));
		}
	}
	void ComputeGradTheta(const FSolverVec3& X0, const FSolverVec3& X1, const FSolverVec3& X2, const FSolverVec3& X3, const int32 Index, FSolverVec3& dThetadx, FSolverReal& Theta); 

	using Base::Constraints;
	using Base::ParticleOffset;
	using Base::ParticleCount;
	using Base::RestAngles;

	FPBDFlatWeightMap StiffnessWarp;
	FPBDFlatWeightMap StiffnessWeft;
	FPBDFlatWeightMap StiffnessBias;
	FPBDFlatWeightMap BucklingStiffnessWarp;
	FPBDFlatWeightMap BucklingStiffnessWeft;
	FPBDFlatWeightMap BucklingStiffnessBias;

	FPBDFlatWeightMap DampingRatio;
	
	mutable TArray<FSolverReal> Lambdas;
	mutable TArray<FSolverReal> LambdasDamping;
	TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.

#if INTEL_ISPC
	// Constraint SOA. InitColor will initialize these. Only used if using ISPC
	TArray<int32> ConstraintsIndex1;
	TArray<int32> ConstraintsIndex2;
	TArray<int32> ConstraintsIndex3;
	TArray<int32> ConstraintsIndex4;

	// Particles.X but stored per constraint.
	// These are only copied over if using ISPC.
	TArray<FSolverVec3> X1Array;
	TArray<FSolverVec3> X2Array;
	TArray<FSolverVec3> X3Array;
	TArray<FSolverVec3> X4Array;
#endif

	TArray<FSolverVec3> WarpWeftBiasBaseMultipliers;

	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoBendingStiffnessWarp, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoBendingStiffnessWeft, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoBendingStiffnessBias, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoBendingDamping, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoBucklingRatio, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoBucklingStiffnessWarp, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoBucklingStiffnessWeft, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoBucklingStiffnessBias, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoFlatnessRatio, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoRestAngle, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoRestAngleType, int32);
};

}  // End namespace Chaos::Softs
