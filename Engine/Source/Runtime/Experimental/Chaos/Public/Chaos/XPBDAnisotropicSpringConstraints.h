// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/PBDAxialSpringConstraintsBase.h"
#include "Chaos/PBDSpringConstraintsBase.h"
#include "Chaos/PBDWeightMap.h"
#include "Chaos/PBDFlatWeightMap.h"

namespace Chaos
{
class FTriangleMesh;
}

namespace Chaos::Softs
{

// Edge and axial springs. Spatially varying stiffnesses and restlengths controlled by Warp/Weft/Bias.
// Both types of springs controlled by the same input parameters.
class FXPBDAnisotropicEdgeSpringConstraints : public FPBDSpringConstraintsBase
{
	typedef FPBDSpringConstraintsBase Base;

public:
	// Stiffness is in kg /s^2
	static constexpr FSolverReal MinStiffness = (FSolverReal)0; // We're not checking against MinStiffness (except when it's constant and == 0)
	static constexpr FSolverReal MaxStiffness = (FSolverReal)1e9;
	static constexpr FSolverReal MinDampingRatio = (FSolverReal)0.;
	static constexpr FSolverReal MaxDampingRatio = (FSolverReal)1000.;
	static constexpr FSolverReal MinWarpWeftScale = (FSolverReal)0.;
	static constexpr FSolverReal MaxWarpWeftScale = (FSolverReal)1e7; // No particular reason for this number. Just can't imagine wanting something bigger?

	CHAOS_API FXPBDAnisotropicEdgeSpringConstraints(
		const FSolverParticlesRange& Particles,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		bool bUse3dRestLengths,
		bool bSavePreResizedSimPosition3D,
		const TArray<FVector3f>& PreResizedSimPosition3D,
		FSolverReal MeshScale,
		const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
		const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
		const FSolverVec2& InStiffnessWarp,
		const FSolverVec2& InStiffnessWeft,
		const FSolverVec2& InStiffnessBias,
		const FSolverVec2& InDampingRatio,
		const FSolverVec2& InWarpScale,
		const FSolverVec2& InWeftScale);

	CHAOS_API FXPBDAnisotropicEdgeSpringConstraints(
		const FSolverParticles& InParticles,
		int32 InParticleOffset,
		int32 InParticleCount,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		bool bUse3dRestLengths,
		bool bSavePreResizedSimPosition3D,
		const TArray<FVector3f>& PreResizedSimPosition3D,
		FSolverReal MeshScale,
		const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
		const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
		const FSolverVec2& InStiffnessWarp,
		const FSolverVec2& InStiffnessWeft,
		const FSolverVec2& InStiffnessBias,
		const FSolverVec2& InDampingRatio,
		const FSolverVec2& InWarpScale,
		const FSolverVec2& InWeftScale);

	UE_DEPRECATED(5.6, "No longer supported, use new constructors instead")
	CHAOS_API FXPBDAnisotropicEdgeSpringConstraints(
		const FSolverParticlesRange& Particles,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		bool bUse3dRestLengths,
		const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
		const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
		const FSolverVec2& InStiffnessWarp,
		const FSolverVec2& InStiffnessWeft,
		const FSolverVec2& InStiffnessBias,
		const FSolverVec2& InDampingRatio,
		const FSolverVec2& InWarpScale,
		const FSolverVec2& InWeftScale);

	UE_DEPRECATED(5.6, "No longer supported, use new constructors instead")
	CHAOS_API FXPBDAnisotropicEdgeSpringConstraints(
		const FSolverParticles& InParticles,
		int32 InParticleOffset,
		int32 InParticleCount,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		bool bUse3dRestLengths,
		const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
		const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
		const FSolverVec2& InStiffnessWarp,
		const FSolverVec2& InStiffnessWeft,
		const FSolverVec2& InStiffnessBias,
		const FSolverVec2& InDampingRatio,
		const FSolverVec2& InWarpScale,
		const FSolverVec2& InWeftScale);

	virtual ~FXPBDAnisotropicEdgeSpringConstraints() override = default;

	void Init()
	{
		Lambdas.Reset();
		Lambdas.AddZeroed(Constraints.Num());
		LambdasDamping.Reset();
		LambdasDamping.AddZeroed(Constraints.Num());
	}

	// Update rest lengths from warp/weft scale
	void ApplyProperties(const FSolverReal /*Dt*/, const int32 /*NumIterations*/)
	{
		bool bWarpScaleChanged = false;
		WarpScale.ApplyValues(&bWarpScaleChanged);
		bool bWeftScaleChanged = false;
		WeftScale.ApplyValues(&bWeftScaleChanged);
		if (bWarpScaleChanged || bWeftScaleChanged)
		{
			// Need to update distances
			UpdateDists();
		}
	}

	template<typename SolverParticlesOrRange>
	CHAOS_API void Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const;

	CHAOS_API void UpdateLinearSystem(const FSolverParticlesRange& Particles, const FSolverReal Dt, FEvolutionLinearSystem& LinearSystem) const;

	const TArray<int32>& GetConstraintsPerColorStartIndex() const { return ConstraintsPerColorStartIndex; }
	const TArray<FSolverVec3>& GetWarpWeftBiasBaseMultipliers() const { return WarpWeftBiasBaseMultipliers; }

	void ResetRestLengths(const TConstArrayView<FSolverVec3>& Positions)
	{
		CalculateRestLengths(Positions, TArrayView<FSolverReal>(BaseDists));
		UpdateDists();
	}

private:
	template<typename SolverParticlesOrRange>
	void InitColor(const SolverParticlesOrRange& InParticles);

	void InitFromPatternData(int32 ParticlesSize, bool bUse3dRestLengths, const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions, const FTriangleMesh& TriangleMesh, bool bSavePreResizedSimPosition3D, const TArray<FVector3f>& PreResizedSimPosition3D, FSolverReal MeshScale);

	CHAOS_API void UpdateDists();
	
	template<bool bDampingBefore, bool bSingleLambda, bool bSeparateStretch, bool bDampingAfter, typename SolverParticlesOrRange>
	void ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverVec3& StiffnessValue, const FSolverReal DampingRatioValue) const;

	using Base::Constraints;
	using Base::ParticleOffset;
	using Base::ParticleCount;
	using Base::Dists; // These will be updated when Warp/Weft Scales change.

	FPBDFlatWeightMap StiffnessWarp;
	FPBDFlatWeightMap StiffnessWeft;
	FPBDFlatWeightMap StiffnessBias;
	FPBDFlatWeightMap DampingRatio;
	FPBDWeightMap WarpScale;
	FPBDWeightMap WeftScale;

	mutable TArray<FSolverReal> Lambdas;
	mutable TArray<FSolverReal> LambdasDamping;
	TArray<FSolverReal> BaseDists; // Without Warp/Weft Scale applied
	TArray<FSolverVec3> WarpWeftBiasBaseMultipliers;
	TArray<FSolverVec2> WarpWeftScaleBaseMultipliers;
	TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.

	friend class FXPBDAnisotropicSpringConstraints;
};

class FXPBDAnisotropicAxialSpringConstraints : public FPBDAxialSpringConstraintsBase
{
	typedef FPBDAxialSpringConstraintsBase Base;

public:
	// Stiffness is in kg /s^2
	static constexpr FSolverReal MinStiffness = (FSolverReal)0; // We're not checking against MinStiffness (except when it's constant and == 0)
	static constexpr FSolverReal MaxStiffness = (FSolverReal)1e9;
	static constexpr FSolverReal MinDampingRatio = (FSolverReal)0.;
	static constexpr FSolverReal MaxDampingRatio = (FSolverReal)1000.;
	static constexpr FSolverReal MinWarpWeftScale = (FSolverReal)0.;
	static constexpr FSolverReal MaxWarpWeftScale = (FSolverReal)1e7; // No particular reason for this number. Just can't imagine wanting something bigger?	
	
	CHAOS_API FXPBDAnisotropicAxialSpringConstraints(
	const FSolverParticlesRange& Particles,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		bool bUse3dRestLengths,
		bool bSavePreResizedSimPosition3D,
		const TArray<FVector3f>& PreResizedSimPosition3D,
		FSolverReal MeshScale,
		const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
		const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
		const FSolverVec2& InStiffnessWarp,
		const FSolverVec2& InStiffnessWeft,
		const FSolverVec2& InStiffnessBias,
		const FSolverVec2& InDampingRatio,
		const FSolverVec2& InWarpScale,
		const FSolverVec2& InWeftScale);

	CHAOS_API FXPBDAnisotropicAxialSpringConstraints(
		const FSolverParticles& InParticles,
		int32 InParticleOffset,
		int32 InParticleCount,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		bool bUse3dRestLengths,
		bool bSavePreResizedSimPosition3D,
		const TArray<FVector3f>& PreResizedSimPosition3D,
		FSolverReal MeshScale,
		const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
		const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
		const FSolverVec2& InStiffnessWarp,
		const FSolverVec2& InStiffnessWeft,
		const FSolverVec2& InStiffnessBias,
		const FSolverVec2& InDampingRatio,
		const FSolverVec2& InWarpScale,
		const FSolverVec2& InWeftScale);

	UE_DEPRECATED(5.6, "No longer supported, use new constructors instead")
	CHAOS_API FXPBDAnisotropicAxialSpringConstraints(
		const FSolverParticlesRange& Particles,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		bool bUse3dRestLengths,
		const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
		const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
		const FSolverVec2& InStiffnessWarp,
		const FSolverVec2& InStiffnessWeft,
		const FSolverVec2& InStiffnessBias,
		const FSolverVec2& InDampingRatio,
		const FSolverVec2& InWarpScale,
		const FSolverVec2& InWeftScale);

	UE_DEPRECATED(5.6, "No longer supported, use new constructors instead")
	CHAOS_API FXPBDAnisotropicAxialSpringConstraints(
		const FSolverParticles& InParticles,
		int32 InParticleOffset,
		int32 InParticleCount,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		bool bUse3dRestLengths,
		const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
		const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
		const FSolverVec2& InStiffnessWarp,
		const FSolverVec2& InStiffnessWeft,
		const FSolverVec2& InStiffnessBias,
		const FSolverVec2& InDampingRatio,
		const FSolverVec2& InWarpScale,
		const FSolverVec2& InWeftScale);

	virtual ~FXPBDAnisotropicAxialSpringConstraints() override = default;

	void Init()
	{
		Lambdas.Reset();
		Lambdas.AddZeroed(Constraints.Num());
		LambdasDamping.Reset();
		LambdasDamping.AddZeroed(Constraints.Num());
	}

	// Update rest lengths from warp/weft scale
	void ApplyProperties(const FSolverReal /*Dt*/, const int32 /*NumIterations*/)
	{
		bool bWarpScaleChanged = false;
		WarpScale.ApplyValues(&bWarpScaleChanged);
		bool bWeftScaleChanged = false;
		WeftScale.ApplyValues(&bWeftScaleChanged);
		if (bWarpScaleChanged || bWeftScaleChanged)
		{
			// Need to update distances
			UpdateDists();
		}
	}

	template<typename SolverParticlesOrRange>
	CHAOS_API void Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const;

	CHAOS_API void UpdateLinearSystem(const FSolverParticlesRange& Particles, const FSolverReal Dt, FEvolutionLinearSystem& LinearSystem) const;

	const TArray<int32>& GetConstraintsPerColorStartIndex() const { return ConstraintsPerColorStartIndex; }
	const TArray<FSolverVec3>& GetWarpWeftBiasBaseMultipliers() const { return WarpWeftBiasBaseMultipliers; }

	void ResetRestLengths(const TConstArrayView<FSolverVec3>& Positions)
	{
		CalculateRestLengths(Positions, TArrayView<FSolverReal>(BaseDists));
		UpdateDists();
	}

private:
	template<typename SolverParticlesOrRange>
	void InitColor(const SolverParticlesOrRange& InParticles);

	void InitFromPatternData(int32 ParticlesSize, bool bUse3dRestLengths, const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions, const FTriangleMesh& TriangleMesh, bool bSavePreResizedSimPosition3D, const TArray<FVector3f>& PreResizedSimPosition3D, FSolverReal MeshScale);

	CHAOS_API void UpdateDists();

	template<bool bDampingBefore, bool bSingleLambda, bool bSeparateStretch, bool bDampingAfter, typename SolverParticlesOrRange>
	void ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverVec3& StiffnessValue, const FSolverReal DampingRatioValue) const;

	using Base::Constraints;
	using Base::ParticleOffset;
	using Base::ParticleCount;
	using Base::Barys;
	using Base::Dists; // These will be updated when Warp/Weft Scales change.

	FPBDFlatWeightMap StiffnessWarp;
	FPBDFlatWeightMap StiffnessWeft;
	FPBDFlatWeightMap StiffnessBias;
	FPBDFlatWeightMap DampingRatio;
	FPBDWeightMap WarpScale;
	FPBDWeightMap WeftScale;

	mutable TArray<FSolverReal> Lambdas;
	mutable TArray<FSolverReal> LambdasDamping;
	TArray<FSolverReal> BaseDists; // Without Warp/Weft Scale applied
	TArray<FSolverVec3> WarpWeftBiasBaseMultipliers;
	TArray<FSolverVec2> WarpWeftScaleBaseMultipliers;
	TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.

	friend class FXPBDAnisotropicSpringConstraints;

};

class FXPBDAnisotropicSpringConstraints final
{
public:
	static constexpr FSolverReal DefaultStiffness = (FSolverReal)100.;
	static constexpr FSolverReal DefaultDamping = (FSolverReal)1.;
	static constexpr bool bDefaultUse3dRestLengths = true;
	static constexpr FSolverReal DefaultWarpWeftScale = (FSolverReal)1.;
	static constexpr bool bDefaultSavePreResizedSimPosition3D = false;

	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsXPBDAnisoSpringStiffnessWarpEnabled(PropertyCollection, false);
	}

	FXPBDAnisotropicSpringConstraints(
		const FSolverParticlesRange& Particles,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TArray<FVector3f>& PreResizedSimPosition3D,
		Softs::FSolverReal MeshScale)
		: EdgeConstraints(
			Particles,
			TriangleMesh,
			FaceVertexPatternPositions,
			GetXPBDAnisoSpringUse3dRestLengths(PropertyCollection, bDefaultUse3dRestLengths),
			GetSavePreResizedSimPosition3D(PropertyCollection, bDefaultSavePreResizedSimPosition3D),
			PreResizedSimPosition3D,
			MeshScale,
			WeightMaps.FindRef(GetXPBDAnisoSpringStiffnessWarpString(PropertyCollection, XPBDAnisoSpringStiffnessWarpName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringStiffnessWeftString(PropertyCollection, XPBDAnisoSpringStiffnessWeftName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringStiffnessBiasString(PropertyCollection, XPBDAnisoSpringStiffnessBiasName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringDampingString(PropertyCollection, XPBDAnisoSpringDampingName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringWarpScaleString(PropertyCollection, XPBDAnisoSpringWarpScaleName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringWeftScaleString(PropertyCollection, XPBDAnisoSpringWeftScaleName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringStiffnessWarp(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringStiffnessWeft(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringStiffnessBias(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringDamping(PropertyCollection, DefaultDamping)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringWarpScale(PropertyCollection, DefaultWarpWeftScale)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringWeftScale(PropertyCollection, DefaultWarpWeftScale)))
		, AxialConstraints(
			Particles,
			TriangleMesh,
			FaceVertexPatternPositions,
			GetXPBDAnisoSpringUse3dRestLengths(PropertyCollection, bDefaultUse3dRestLengths),
			GetSavePreResizedSimPosition3D(PropertyCollection, bDefaultSavePreResizedSimPosition3D),
			PreResizedSimPosition3D,
			MeshScale,
			WeightMaps.FindRef(GetXPBDAnisoSpringStiffnessWarpString(PropertyCollection, XPBDAnisoSpringStiffnessWarpName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringStiffnessWeftString(PropertyCollection, XPBDAnisoSpringStiffnessWeftName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringStiffnessBiasString(PropertyCollection, XPBDAnisoSpringStiffnessBiasName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringDampingString(PropertyCollection, XPBDAnisoSpringDampingName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringWarpScaleString(PropertyCollection, XPBDAnisoSpringWarpScaleName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringWeftScaleString(PropertyCollection, XPBDAnisoSpringWeftScaleName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringStiffnessWarp(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringStiffnessWeft(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringStiffnessBias(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringDamping(PropertyCollection, DefaultDamping)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringWarpScale(PropertyCollection, DefaultWarpWeftScale)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringWeftScale(PropertyCollection, DefaultWarpWeftScale)))
		, XPBDAnisoSpringUse3dRestLengthsIndex(PropertyCollection)
		, XPBDAnisoSpringStiffnessWarpIndex(PropertyCollection)
		, XPBDAnisoSpringStiffnessWeftIndex(PropertyCollection)
		, XPBDAnisoSpringStiffnessBiasIndex(PropertyCollection)
		, XPBDAnisoSpringDampingIndex(PropertyCollection)
		, XPBDAnisoSpringWarpScaleIndex(PropertyCollection)
		, XPBDAnisoSpringWeftScaleIndex(PropertyCollection)
		, SavePreResizedSimPosition3DIndex(PropertyCollection)
	{}

	FXPBDAnisotropicSpringConstraints(
		const FSolverParticles& InParticles,
		int32 InParticleOffset,
		int32 InParticleCount,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexPatternPositions,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TArray<FVector3f>& PreResizedSimPosition3D,
		Softs::FSolverReal MeshScale)
		: EdgeConstraints(
			InParticles,
			InParticleOffset,
			InParticleCount,
			TriangleMesh,
			FaceVertexPatternPositions,
			GetXPBDAnisoSpringUse3dRestLengths(PropertyCollection, bDefaultUse3dRestLengths),
			GetSavePreResizedSimPosition3D(PropertyCollection, bDefaultSavePreResizedSimPosition3D),
			PreResizedSimPosition3D,
			MeshScale,
			WeightMaps.FindRef(GetXPBDAnisoSpringStiffnessWarpString(PropertyCollection, XPBDAnisoSpringStiffnessWarpName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringStiffnessWeftString(PropertyCollection, XPBDAnisoSpringStiffnessWeftName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringStiffnessBiasString(PropertyCollection, XPBDAnisoSpringStiffnessBiasName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringDampingString(PropertyCollection, XPBDAnisoSpringDampingName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringWarpScaleString(PropertyCollection, XPBDAnisoSpringWarpScaleName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringWeftScaleString(PropertyCollection, XPBDAnisoSpringWeftScaleName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringStiffnessWarp(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringStiffnessWeft(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringStiffnessBias(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringDamping(PropertyCollection, DefaultDamping)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringWarpScale(PropertyCollection, DefaultWarpWeftScale)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringWeftScale(PropertyCollection, DefaultWarpWeftScale)))
		, AxialConstraints(
			InParticles,
			InParticleOffset,
			InParticleCount,
			TriangleMesh,
			FaceVertexPatternPositions,
			GetXPBDAnisoSpringUse3dRestLengths(PropertyCollection, bDefaultUse3dRestLengths),
			GetSavePreResizedSimPosition3D(PropertyCollection, bDefaultSavePreResizedSimPosition3D),
			PreResizedSimPosition3D,
			MeshScale,
			WeightMaps.FindRef(GetXPBDAnisoSpringStiffnessWarpString(PropertyCollection, XPBDAnisoSpringStiffnessWarpName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringStiffnessWeftString(PropertyCollection, XPBDAnisoSpringStiffnessWeftName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringStiffnessBiasString(PropertyCollection, XPBDAnisoSpringStiffnessBiasName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringDampingString(PropertyCollection, XPBDAnisoSpringDampingName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringWarpScaleString(PropertyCollection, XPBDAnisoSpringWarpScaleName.ToString())),
			WeightMaps.FindRef(GetXPBDAnisoSpringWeftScaleString(PropertyCollection, XPBDAnisoSpringWeftScaleName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringStiffnessWarp(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringStiffnessWeft(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringStiffnessBias(PropertyCollection, DefaultStiffness)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringDamping(PropertyCollection, DefaultDamping)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringWarpScale(PropertyCollection, DefaultWarpWeftScale)),
			FSolverVec2(GetWeightedFloatXPBDAnisoSpringWeftScale(PropertyCollection, DefaultWarpWeftScale)))
		, XPBDAnisoSpringUse3dRestLengthsIndex(PropertyCollection)
		, XPBDAnisoSpringStiffnessWarpIndex(PropertyCollection)
		, XPBDAnisoSpringStiffnessWeftIndex(PropertyCollection)
		, XPBDAnisoSpringStiffnessBiasIndex(PropertyCollection)
		, XPBDAnisoSpringDampingIndex(PropertyCollection)
		, XPBDAnisoSpringWarpScaleIndex(PropertyCollection)
		, XPBDAnisoSpringWeftScaleIndex(PropertyCollection)
		, SavePreResizedSimPosition3DIndex(PropertyCollection)
	{}

	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);

	// Update rest lengths from warp/weft scale
	void ApplyProperties(const FSolverReal Dt, const int32 NumIterations)
	{
		EdgeConstraints.ApplyProperties(Dt, NumIterations);
		AxialConstraints.ApplyProperties(Dt, NumIterations);
	}

	void Init()
	{
		EdgeConstraints.Init();
		AxialConstraints.Init();
	}

	const FXPBDAnisotropicEdgeSpringConstraints& GetEdgeConstraints() const { return EdgeConstraints; }
	const FXPBDAnisotropicAxialSpringConstraints& GetAxialConstraints() const { return AxialConstraints; }

	void ResetRestLengths(const TConstArrayView<FSolverVec3>& Positions)
	{
		EdgeConstraints.ResetRestLengths(Positions);
		AxialConstraints.ResetRestLengths(Positions);
	}

private:
	FXPBDAnisotropicEdgeSpringConstraints EdgeConstraints;
	FXPBDAnisotropicAxialSpringConstraints AxialConstraints;

	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoSpringUse3dRestLengths, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoSpringStiffnessWarp, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoSpringStiffnessWeft, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoSpringStiffnessBias, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoSpringDamping, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoSpringWarpScale, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoSpringWeftScale, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(SavePreResizedSimPosition3D, bool);
};

}  // End namespace Chaos::Softs

