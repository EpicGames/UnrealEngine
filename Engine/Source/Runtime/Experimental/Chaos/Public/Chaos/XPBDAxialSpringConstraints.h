// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDAxialSpringConstraintsBase.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosStats.h"

DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Axial Spring Constraint"), STAT_XPBD_AxialSpring, STATGROUP_Chaos);

namespace Chaos::Softs
{

class FXPBDAxialSpringConstraints : public FPBDAxialSpringConstraintsBase
{
	typedef FPBDAxialSpringConstraintsBase Base;

public:
	// Stiffness is in kg/s^2
	static constexpr FSolverReal MinStiffness = (FSolverReal)1e-4; // Stiffness below this will be considered 0 since all of our calculations are actually based on 1 / stiffness.
	static constexpr FSolverReal MaxStiffness = (FSolverReal)1e7;

	FXPBDAxialSpringConstraints(
		const FSolverParticlesRange& Particles,
		const TArray<TVec3<int32>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const FSolverVec2& InStiffness,
		bool bTrimKinematicConstraints)
		: Base(
			Particles,
			InConstraints,
			StiffnessMultipliers,
			InStiffness,
			bTrimKinematicConstraints,
			MaxStiffness)
	{
		Lambdas.Init(0.f, Constraints.Num());
	}

	FXPBDAxialSpringConstraints(
		const FSolverParticles& Particles,
		int32 ParticleOffset,
		int32 ParticleCount,
		const TArray<TVec3<int32>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const FSolverVec2& InStiffness,
		bool bTrimKinematicConstraints)
		: Base(
			Particles,
			ParticleOffset,
			ParticleCount,
			InConstraints,
			StiffnessMultipliers,
			InStiffness,
			bTrimKinematicConstraints,
			MaxStiffness)
	{
		Lambdas.Init(0.f, Constraints.Num());
	}

	virtual ~FXPBDAxialSpringConstraints() override = default;

	void SetProperties(const FSolverVec2& InStiffness) { Stiffness.SetWeightedValue(InStiffness, MaxStiffness); }

	void ApplyProperties(const FSolverReal /*Dt*/, const int32 /*NumIterations*/) { Stiffness.ApplyXPBDValues(MaxStiffness); }

	void Init() const { for (FSolverReal& Lambda : Lambdas) { Lambda = (FSolverReal)0.; } }

	template<typename SolverParticlesOrRange>
	void Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const
	{
		SCOPE_CYCLE_COUNTER(STAT_XPBD_AxialSpring);
		if (!Stiffness.HasWeightMap())
		{
			const FSolverReal ExpStiffnessValue = (FSolverReal)Stiffness;
			if (ExpStiffnessValue < MinStiffness)
			{
				return;
			}
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const TVector<int32, 3>& constraint = Constraints[ConstraintIndex];
				const int32 i1 = constraint[0];
				const int32 i2 = constraint[1];
				const int32 i3 = constraint[2];
				const FSolverVec3 Delta = GetDelta(Particles, Dt, ConstraintIndex, ExpStiffnessValue);
				const FSolverReal Multiplier = (FSolverReal)2. / (FMath::Max(Barys[ConstraintIndex], (FSolverReal)1. - Barys[ConstraintIndex]) + (FSolverReal)1.);
				if (Particles.InvM(i1) > 0)
				{
					Particles.P(i1) -= Multiplier * Particles.InvM(i1) * Delta;
				}
				if (Particles.InvM(i2) != 0)
				{
					Particles.P(i2) += Multiplier * Particles.InvM(i2) * Barys[ConstraintIndex] * Delta;
				}
				if (Particles.InvM(i3) != 0)
				{
					Particles.P(i3) += Multiplier * Particles.InvM(i3) * ((FSolverReal)1. - Barys[ConstraintIndex]) * Delta;
				}
			}
		}
		else
		{
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const FSolverReal ExpStiffnessValue = Stiffness[ConstraintIndex];
				const TVector<int32, 3>& constraint = Constraints[ConstraintIndex];
				const int32 i1 = constraint[0];
				const int32 i2 = constraint[1];
				const int32 i3 = constraint[2];
				const FSolverVec3 Delta = GetDelta(Particles, Dt, ConstraintIndex, ExpStiffnessValue);
				const FSolverReal Multiplier = (FSolverReal)2. / (FMath::Max(Barys[ConstraintIndex], (FSolverReal)1. - Barys[ConstraintIndex]) + (FSolverReal)1.);
				if (Particles.InvM(i1) > 0)
				{
					Particles.P(i1) -= Multiplier * Particles.InvM(i1) * Delta;
				}
				if (Particles.InvM(i2) != 0)
				{
					Particles.P(i2) += Multiplier * Particles.InvM(i2) * Barys[ConstraintIndex] * Delta;
				}
				if (Particles.InvM(i3) != 0)
				{
					Particles.P(i3) += Multiplier * Particles.InvM(i3) * ((FSolverReal)1. - Barys[ConstraintIndex]) * Delta;
				}
			}
		}
	}

private:
	template<typename SolverParticlesOrRange>
	FSolverVec3 GetDelta(const SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 InConstraintIndex, const FSolverReal StiffnessValue) const
	{
		const TVector<int32, 3>& Constraint = Constraints[InConstraintIndex];
		const int32 i1 = Constraint[0];
		const int32 i2 = Constraint[1];
		const int32 i3 = Constraint[2];

		const FSolverReal Bary = Barys[InConstraintIndex];
		const FSolverReal PInvMass = Particles.InvM(i3) * ((FSolverReal)1. - Bary) + Particles.InvM(i2) * Bary;
		if (StiffnessValue < MinStiffness || ( Particles.InvM(i1) == (FSolverReal)0. && PInvMass == (FSolverReal)0.))
		{
			return FSolverVec3((FSolverReal)0.);
		}
		const FSolverReal CombinedInvMass = PInvMass + Particles.InvM(i1);
		ensure(CombinedInvMass > (FSolverReal)SMALL_NUMBER);

		const FSolverVec3& P1 = Particles.P(i1);
		const FSolverVec3& P2 = Particles.P(i2);
		const FSolverVec3& P3 = Particles.P(i3);
		const FSolverVec3 P = (P2 - P3) * Bary + P3;

		const FSolverVec3 Difference = P1 - P;
		const FSolverReal Distance = Difference.Size();
		if (UNLIKELY(Distance <= SMALL_NUMBER))
		{
			return FSolverVec3((FSolverReal)0.);
		}
		const FSolverVec3 Direction = Difference / Distance;
		const FSolverReal Offset = (Distance - Dists[InConstraintIndex]);

		FSolverReal& Lambda = Lambdas[InConstraintIndex];
		const FSolverReal Alpha = (FSolverReal)1 / (StiffnessValue * Dt * Dt);

		const FSolverReal DLambda = (Offset - Alpha * Lambda) / (CombinedInvMass + Alpha);
		const FSolverVec3 Delta = DLambda * Direction;
		Lambda += DLambda;

		return Delta;
	}

protected:
	using Base::Constraints;
	using Base::ParticleOffset;
	using Base::ParticleCount;
	using Base::Stiffness;

private:
	using Base::Barys;
	using Base::Dists;

	mutable TArray<FSolverReal> Lambdas;
};

class FXPBDAreaSpringConstraints final : public FXPBDAxialSpringConstraints
{
public:
	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsXPBDAreaSpringStiffnessEnabled(PropertyCollection, false);
	}

	FXPBDAreaSpringConstraints(
		const FSolverParticlesRange& Particles,
		const TArray<TVec3<int32>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints)
		: FXPBDAxialSpringConstraints(
			Particles,
			InConstraints,
			WeightMaps.FindRef(GetXPBDAreaSpringStiffnessString(PropertyCollection, XPBDAreaSpringStiffnessName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDAreaSpringStiffness(PropertyCollection, MaxStiffness)),
			bTrimKinematicConstraints)
		, XPBDAreaSpringStiffnessIndex(PropertyCollection)
	{}

	FXPBDAreaSpringConstraints(
		const FSolverParticles& Particles,
		int32 ParticleOffset,
		int32 ParticleCount,
		const TArray<TVec3<int32>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints)
		: FXPBDAxialSpringConstraints(
			Particles,
			ParticleOffset,
			ParticleCount,
			InConstraints,
			WeightMaps.FindRef(GetXPBDAreaSpringStiffnessString(PropertyCollection, XPBDAreaSpringStiffnessName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDAreaSpringStiffness(PropertyCollection, MaxStiffness)),
			bTrimKinematicConstraints)
		, XPBDAreaSpringStiffnessIndex(PropertyCollection)
	{}

	virtual ~FXPBDAreaSpringConstraints() override = default;

	void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
	{
		if (IsXPBDAreaSpringStiffnessMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatXPBDAreaSpringStiffness(PropertyCollection));
			if (IsXPBDAreaSpringStiffnessStringDirty(PropertyCollection))
			{
				const FString& WeightMapName = GetXPBDAreaSpringStiffnessString(PropertyCollection);
				Stiffness = FPBDStiffness(
					WeightedValue,
					WeightMaps.FindRef(WeightMapName),
					TConstArrayView<TVec3<int32>>(Constraints),
					ParticleOffset,
					ParticleCount,
					FPBDStiffness::DefaultTableSize,
					FPBDStiffness::DefaultParameterFitBase,
					MaxStiffness);
			}
			else
			{
				Stiffness.SetWeightedValue(WeightedValue, MaxStiffness);
			}
		}
	}

private:
	using FXPBDAxialSpringConstraints::Constraints;
	using FXPBDAxialSpringConstraints::ParticleOffset;
	using FXPBDAxialSpringConstraints::ParticleCount;
	using FXPBDAxialSpringConstraints::Stiffness;

	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAreaSpringStiffness, float);
};

}  // End namespace Chaos::Softs
