// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDBendingConstraintsBase.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosStats.h"
#include "Chaos/PBDFlatWeightMap.h"

namespace Chaos::Softs
{

class FXPBDBendingConstraints final : public FPBDBendingConstraintsBase
{
	typedef FPBDBendingConstraintsBase Base;

public:
	// Stiffness is in kg cm^2 / rad^2 s^2
	static constexpr FSolverReal MinStiffness = (FSolverReal)0;  // We're not checking against MinStiffness (except when it's constant and == 0)
	static constexpr FSolverReal MaxStiffness = (FSolverReal)1e7;
	static constexpr FSolverReal MinDamping = (FSolverReal)0.;
	static constexpr FSolverReal MaxDamping = (FSolverReal)1000.;

	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsXPBDBendingElementStiffnessEnabled(PropertyCollection, false);
	}

	FXPBDBendingConstraints(const FSolverParticlesRange& InParticles,
		TArray<TVec4<int32>>&& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection)
		: Base(
			InParticles,
			MoveTemp(InConstraints),
			TConstArrayView<FRealSingle>(), // We don't use base stiffness weight maps
			WeightMaps.FindRef(GetXPBDBucklingRatioString(PropertyCollection, XPBDBucklingRatioName.ToString())),
			TConstArrayView<FRealSingle>(), // We don't use base stiffness weight maps
			GetRestAngleMapFromCollection(WeightMaps, PropertyCollection),
			FSolverVec2(GetWeightedFloatXPBDBendingElementStiffness(PropertyCollection, MaxStiffness)),
			FSolverVec2(GetWeightedFloatXPBDBucklingRatio(PropertyCollection, 0.f)),
			FSolverVec2(GetWeightedFloatXPBDBucklingStiffness(PropertyCollection, MaxStiffness)),
			GetRestAngleValueFromCollection(PropertyCollection),
			(ERestAngleConstructionType)GetXPBDRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles),
			true /*bTrimKinematicConstraints*/,
			MaxStiffness)
		, XPBDStiffness(
			FSolverVec2(GetWeightedFloatXPBDBendingElementStiffness(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
			WeightMaps.FindRef(GetXPBDBendingElementStiffnessString(PropertyCollection, XPBDBendingElementStiffnessName.ToString())),
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount)
		, XPBDBucklingStiffness(
			FSolverVec2(GetWeightedFloatXPBDBucklingStiffness(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
			WeightMaps.FindRef(GetXPBDBucklingStiffnessString(PropertyCollection, XPBDBucklingStiffnessName.ToString())),
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount)
		, DampingRatio(
			FSolverVec2(GetWeightedFloatXPBDBendingElementDamping(PropertyCollection, MinDamping)).ClampAxes(MinDamping, MaxDamping),
			WeightMaps.FindRef(GetXPBDBendingElementDampingString(PropertyCollection, XPBDBendingElementDampingName.ToString())),
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount)
		, XPBDBendingElementStiffnessIndex(PropertyCollection)
		, XPBDBendingElementDampingIndex(PropertyCollection)
		, XPBDBucklingRatioIndex(PropertyCollection)
		, XPBDBucklingStiffnessIndex(PropertyCollection)
		, XPBDFlatnessRatioIndex(PropertyCollection)
		, XPBDRestAngleIndex(PropertyCollection)
		, XPBDRestAngleTypeIndex(PropertyCollection)
	{
		Lambdas.Init((FSolverReal)0., Constraints.Num());
		LambdasDamping.Init((FSolverReal)0., Constraints.Num());
		InitColor(InParticles);
	}

	UE_DEPRECATED(5.4, "XPBD Constraints must always trim kinematic constraints")
	FXPBDBendingConstraints(const FSolverParticlesRange& InParticles,
		TArray<TVec4<int32>>&& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints)
		: FXPBDBendingConstraints(InParticles, MoveTemp(InConstraints), WeightMaps, PropertyCollection)
	{}

	FXPBDBendingConstraints(const FSolverParticles& InParticles,
		int32 InParticleOffset,
		int32 InParticleCount,
		TArray<TVec4<int32>>&& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection)
		: Base(
			InParticles,
			InParticleOffset,
			InParticleCount,
			MoveTemp(InConstraints),
			TConstArrayView<FRealSingle>(), // We don't use base stiffness weight maps
			WeightMaps.FindRef(GetXPBDBucklingRatioString(PropertyCollection, XPBDBucklingRatioName.ToString())),
			TConstArrayView<FRealSingle>(), // We don't use base stiffness weight maps
			GetRestAngleMapFromCollection(WeightMaps, PropertyCollection),
			FSolverVec2(GetWeightedFloatXPBDBendingElementStiffness(PropertyCollection, MaxStiffness)),
			FSolverVec2(GetWeightedFloatXPBDBucklingRatio(PropertyCollection, 0.f)),
			FSolverVec2(GetWeightedFloatXPBDBucklingStiffness(PropertyCollection, MaxStiffness)),
			GetRestAngleValueFromCollection(PropertyCollection),
			(ERestAngleConstructionType)GetXPBDRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles),
			true /*bTrimKinematicConstraints*/,
			MaxStiffness)
		, XPBDStiffness(
			FSolverVec2(GetWeightedFloatXPBDBendingElementStiffness(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
			WeightMaps.FindRef(GetXPBDBendingElementStiffnessString(PropertyCollection, XPBDBendingElementStiffnessName.ToString())),
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount)
		, XPBDBucklingStiffness(
			FSolverVec2(GetWeightedFloatXPBDBucklingStiffness(PropertyCollection, MaxStiffness)).ClampAxes(0, MaxStiffness),
			WeightMaps.FindRef(GetXPBDBucklingStiffnessString(PropertyCollection, XPBDBucklingStiffnessName.ToString())),
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount)
		, DampingRatio(
			FSolverVec2(GetWeightedFloatXPBDBendingElementDamping(PropertyCollection, MinDamping)).ClampAxes(MinDamping, MaxDamping),
			WeightMaps.FindRef(GetXPBDBendingElementDampingString(PropertyCollection, XPBDBendingElementDampingName.ToString())),
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount)
		, XPBDBendingElementStiffnessIndex(PropertyCollection)
		, XPBDBendingElementDampingIndex(PropertyCollection)
		, XPBDBucklingRatioIndex(PropertyCollection)
		, XPBDBucklingStiffnessIndex(PropertyCollection)
		, XPBDFlatnessRatioIndex(PropertyCollection)
		, XPBDRestAngleIndex(PropertyCollection)
		, XPBDRestAngleTypeIndex(PropertyCollection)
	{
		Lambdas.Init((FSolverReal)0., Constraints.Num());
		LambdasDamping.Init((FSolverReal)0., Constraints.Num());
		InitColor(InParticles);
	}

	UE_DEPRECATED(5.4, "XPBD Constraints must always trim kinematic constraints")
	FXPBDBendingConstraints(const FSolverParticles& InParticles,
		int32 InParticleOffset,
		int32 InParticleCount,
		TArray<TVec4<int32>>&& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints)
		: FXPBDBendingConstraints(InParticles, InParticleOffset, InParticleCount, MoveTemp(InConstraints), WeightMaps, PropertyCollection)
	{}

	FXPBDBendingConstraints(const FSolverParticles& InParticles,
		int32 ParticleOffset,
		int32 ParticleCount,
		TArray<TVec4<int32>>&& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& BucklingRatioMultipliers,
		const TConstArrayView<FRealSingle>& BucklingStiffnessMultipliers,
		const TConstArrayView<FRealSingle>& InDampingMultipliers,
		const TConstArrayView<FRealSingle>& RestAngleMap,
		const FSolverVec2& InStiffness,
		const FSolverVec2& InBucklingRatio,
		const FSolverVec2& InBucklingStiffness,
		const FSolverVec2& InDampingRatio,
		const FSolverVec2& RestAngleValue,
		ERestAngleConstructionType RestAngleConstructionType)
		: Base(
			InParticles,
			ParticleOffset,
			ParticleCount,
			MoveTemp(InConstraints),
			TConstArrayView<FRealSingle>(), // We don't use base stiffness weight maps
			BucklingRatioMultipliers,
			TConstArrayView<FRealSingle>(), // We don't use base stiffness weight maps
			RestAngleMap,
			InStiffness,
			InBucklingRatio,
			InBucklingStiffness,
			RestAngleValue,
			RestAngleConstructionType,
			true /*bTrimKinematicConstraints*/,
			MaxStiffness)
		, XPBDStiffness(
			InStiffness.ClampAxes(0, MaxStiffness),
			StiffnessMultipliers,
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount)
		, XPBDBucklingStiffness(
			InBucklingStiffness.ClampAxes(0, MaxStiffness),
			BucklingStiffnessMultipliers,
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount)
		, DampingRatio(
			InDampingRatio.ClampAxes((FSolverReal)0., (FSolverReal)1.),
			InDampingMultipliers,
			TConstArrayView<TVec2<int32>>(ConstraintSharedEdges),
			ParticleOffset,
			ParticleCount)
		, XPBDBendingElementStiffnessIndex(ForceInit)
		, XPBDBendingElementDampingIndex(ForceInit)
		, XPBDBucklingRatioIndex(ForceInit)
		, XPBDBucklingStiffnessIndex(ForceInit)
		, XPBDFlatnessRatioIndex(ForceInit)
		, XPBDRestAngleIndex(ForceInit)
		, XPBDRestAngleTypeIndex(ForceInit)
	{
		Lambdas.Init((FSolverReal)0., Constraints.Num());
		LambdasDamping.Init((FSolverReal)0., Constraints.Num());
		InitColor(InParticles);
	}
	

	UE_DEPRECATED(5.5, "Use constructor with BucklingRatioMultipliers")
	FXPBDBendingConstraints(const FSolverParticles& InParticles,
		int32 ParticleOffset,
		int32 ParticleCount,
		TArray<TVec4<int32>>&& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& BucklingStiffnessMultipliers,
		const TConstArrayView<FRealSingle>& InDampingMultipliers,
		const FSolverVec2& InStiffness,
		const FSolverReal InBucklingRatio,
		const FSolverVec2& InBucklingStiffness,
		const FSolverVec2& InDampingRatio)
		: FXPBDBendingConstraints(
			InParticles, 
			ParticleOffset, 
			ParticleCount, 
			MoveTemp(InConstraints), 
			StiffnessMultipliers,
			TConstArrayView<FRealSingle>(),
			BucklingStiffnessMultipliers, 
			InDampingMultipliers,
			TConstArrayView<FRealSingle>(),
			InStiffness, 
			FSolverVec2(InBucklingRatio),
			InBucklingStiffness, 
			InDampingRatio,
			FSolverVec2((FSolverReal)0.f),
			ERestAngleConstructionType::Use3DRestAngles)
	{}

	UE_DEPRECATED(5.4, "XPBD Constraints must always trim kinematic constraints")
	FXPBDBendingConstraints(const FSolverParticles& InParticles,
		int32 ParticleOffset,
		int32 ParticleCount,
		TArray<TVec4<int32>>&& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& BucklingStiffnessMultipliers,
		const TConstArrayView<FRealSingle>& InDampingMultipliers,
		const FSolverVec2& InStiffness,
		const FSolverReal InBucklingRatio,
		const FSolverVec2& InBucklingStiffness,
		const FSolverVec2& InDampingRatio,
		bool bTrimKinematicConstraints)
		: FXPBDBendingConstraints(
			InParticles, 
			ParticleOffset, 
			ParticleCount, 
			MoveTemp(InConstraints), 
			StiffnessMultipliers,
			TConstArrayView<FRealSingle>(),
			BucklingStiffnessMultipliers, 
			InDampingMultipliers,
			TConstArrayView<FRealSingle>(),
			InStiffness, 
			FSolverVec2(InBucklingRatio),
			InBucklingStiffness, 
			InDampingRatio,
			FSolverVec2((FSolverReal)0.f),
			ERestAngleConstructionType::Use3DRestAngles)
	{}

	virtual ~FXPBDBendingConstraints() override = default;

	template<typename SolverParticlesOrRange>
	void CHAOS_API Init(const SolverParticlesOrRange& InParticles);

	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);

	void SetProperties(const FSolverVec2& InStiffness, const FSolverVec2& InBucklingRatio, const FSolverVec2& InBucklingStiffness, const FSolverVec2& InDampingRatio)
	{
		XPBDStiffness.SetWeightedValue(InStiffness.ClampAxes(0, MaxStiffness));
		BucklingRatioWeighted.SetWeightedValue(InBucklingRatio.ClampAxes((FSolverReal)0., (FSolverReal)1.));
		XPBDBucklingStiffness.SetWeightedValue(InBucklingStiffness.ClampAxes(0, MaxStiffness));
		DampingRatio.SetWeightedValue(InDampingRatio.ClampAxes(MinDamping, MaxDamping));
	}

	UE_DEPRECATED(5.5, "Use SetProperties with Weighted Buckling Ratio")
	void SetProperties(const FSolverVec2& InStiffness, const FSolverReal InBucklingRatio, const FSolverVec2& InBucklingStiffness, const FSolverVec2& InDampingRatio = FSolverVec2::ZeroVector)
	{
		SetProperties(InStiffness, FSolverVec2(InBucklingRatio), InBucklingStiffness, InDampingRatio);
	}

	void ApplyProperties(const FSolverReal /*Dt*/, const int32 /*NumIterations*/)
	{
		BucklingRatioWeighted.ApplyValues();
	}

	template<typename SolverParticlesOrRange>
	CHAOS_API void Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const;

	const TArray<int32>& GetConstraintsPerColorStartIndex() const { return ConstraintsPerColorStartIndex; }

	CHAOS_API void AddBendingResidualAndHessian(const FSolverParticlesRange& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian);
	CHAOS_API void AddBendingResidualAndHessian(const FSolverParticles& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian);

	TArray<TArray<int32>> GetConstraintsArray() const
	{
		TArray<TArray<int32>> ConstraintsArray;
		ConstraintsArray.SetNum(Constraints.Num());
		for (int32 ConstraintIdx = 0; ConstraintIdx < Constraints.Num(); ConstraintIdx++)
		{
			ConstraintsArray[ConstraintIdx].SetNum(4);
			for (int32 LocalIdx = 0; LocalIdx < 4; LocalIdx++)
			{
				ConstraintsArray[ConstraintIdx][LocalIdx] = Constraints[ConstraintIdx][LocalIdx];
			}
		}
		return ConstraintsArray;
	}

	CHAOS_API void AddInternalForceDifferential(const FSolverParticlesRange& InParticles, const TArray<TVector<FSolverReal, 3>>& DeltaParticles, TArray<TVector<FSolverReal, 3>>& Ndf);
	CHAOS_API void AddInternalForceDifferential(const FSolverParticles& InParticles, const TArray<TVector<FSolverReal, 3>>& DeltaParticles, TArray<TVector<FSolverReal, 3>>& Ndf);

	CHAOS_API FSolverReal ComputeTotalEnergy(const FSolverParticles& InParticles, const FSolverReal ExplicitStiffness = -1.f);

	void ResetRestLengths(const TConstArrayView<FSolverVec3>& Positions,
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps)
	{
		CalculateRestAngles(Positions, ParticleOffset, ParticleCount,
			GetRestAngleMapFromCollection(WeightMaps, PropertyCollection),
			GetRestAngleValueFromCollection(PropertyCollection),
			(ERestAngleConstructionType)GetXPBDRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles));
	}
private:
	template<typename SolverParticlesOrRange>
	CHAOS_API void InitColor(const SolverParticlesOrRange& InParticles);
	template<bool bDampingOnly, bool bElasticOnly, typename SolverParticlesOrRange>
	void ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal ExpStiffnessValue, const FSolverReal ExpBucklingValue, const FSolverReal DampingRatioValue) const;

	TConstArrayView<FRealSingle> GetRestAngleMapFromCollection(
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection)
	{
		const ERestAngleConstructionType ConstructionType = (ERestAngleConstructionType)GetXPBDRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles);

		switch (ConstructionType)
		{
		default:
		case ERestAngleConstructionType::Use3DRestAngles:
			return TConstArrayView<FRealSingle>(); // Unused
		case ERestAngleConstructionType::FlatnessRatio:
			return WeightMaps.FindRef(GetXPBDFlatnessRatioString(PropertyCollection, XPBDFlatnessRatioName.ToString()));
		case ERestAngleConstructionType::ExplicitRestAngles:
			return WeightMaps.FindRef(GetXPBDRestAngleString(PropertyCollection, XPBDRestAngleName.ToString()));
		}
	}

	FSolverVec2 GetRestAngleValueFromCollection(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		const ERestAngleConstructionType ConstructionType = (ERestAngleConstructionType)GetXPBDRestAngleType(PropertyCollection, (int32)ERestAngleConstructionType::Use3DRestAngles);

		switch (ConstructionType)
		{
		default:
		case ERestAngleConstructionType::Use3DRestAngles:
			return FSolverVec2(0.f); // Unused
		case ERestAngleConstructionType::FlatnessRatio:
			return FSolverVec2(GetWeightedFloatXPBDFlatnessRatio(PropertyCollection, 0.f));
		case ERestAngleConstructionType::ExplicitRestAngles:
			return FSolverVec2(GetWeightedFloatXPBDRestAngle(PropertyCollection, 0.f));
		}
	}

	using Base::Constraints;
	using Base::ParticleOffset;
	using Base::ParticleCount;
	using Base::RestAngles;

	FPBDFlatWeightMap XPBDStiffness;
	FPBDFlatWeightMap XPBDBucklingStiffness;
	FPBDFlatWeightMap DampingRatio;
	mutable TArray<FSolverReal> Lambdas;
	mutable TArray<FSolverReal> LambdasDamping;
	TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.

#if INTEL_ISPC
	// Constraint SOA. InitColor will initialize these. Only used by ISPC
	TArray<int32> ConstraintsIndex1;
	TArray<int32> ConstraintsIndex2;
	TArray<int32> ConstraintsIndex3;
	TArray<int32> ConstraintsIndex4;

	// Particles.X but stored per constraint.
	// These are only copied over if using ISPC.
	TArray<FSolverVec3> X1Array;
	TArray<FSolverVec3> X2Array;
	TArray<FSolverVec3> X3Array;
	TArray<FSolverVec3> X4Array;
#endif 

	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDBendingElementStiffness, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDBendingElementDamping, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDBucklingRatio, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDBucklingStiffness, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDFlatnessRatio, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDRestAngle, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDRestAngleType, int32);
};

// Support split vs shared damping models in non-shipping builds
#if UE_BUILD_SHIPPING
const bool bChaos_XPBDBending_SplitLambdaDamping = true;
#else
extern bool bChaos_XPBDBending_SplitLambdaDamping;
#endif

}  // End namespace Chaos::Softs

#if !defined(CHAOS_XPBD_BENDING_ISPC_ENABLED)
#define CHAOS_XPBD_BENDING_ISPC_ENABLED 1
#endif

// Support ISPC enable/disable in non-shipping builds
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bChaos_XPBDBending_ISPC_Enabled = INTEL_ISPC && CHAOS_XPBD_BENDING_ISPC_ENABLED;
#else
extern CHAOS_API bool bChaos_XPBDBending_ISPC_Enabled;
#endif
