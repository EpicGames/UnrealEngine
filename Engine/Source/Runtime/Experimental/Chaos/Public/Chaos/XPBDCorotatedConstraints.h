// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/Deformable/ChaosDeformableSolverTypes.h"

DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Corotated Constraint"), STAT_ChaosXPBDCorotated, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Corotated Constraint Polar Compute"), STAT_ChaosXPBDCorotatedPolar, STATGROUP_Chaos);
DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Corotated Constraint Det Compute"), STAT_ChaosXPBDCorotatedDet, STATGROUP_Chaos);

namespace Chaos::Softs
{
	// Tetrahedral constraint using fixed corotated constitutive model [Stomakhin et al., 2012]
	class FXPBDCorotatedTetrahedralConstraints
	{

	public:
		CHAOS_API FXPBDCorotatedTetrahedralConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 4>>& InMesh,
			const bool bRecordMetricIn = true,
			const FSolverReal& EMesh = (FSolverReal)10.0,
			const FSolverReal& NuMesh = (FSolverReal).3);

		CHAOS_API FXPBDCorotatedTetrahedralConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 4>>& InMesh,
			const TArray<FSolverReal>& EMeshArray,
			const FSolverReal& NuMesh = (FSolverReal).3,
			const bool bRecordMetricIn = false);

		CHAOS_API FXPBDCorotatedTetrahedralConstraints(
			const FSolverParticlesRange& InParticles,
			const TArray<TVector<int32, 4>>& InMesh,
			const TArray<FSolverReal>& EMeshArray,
			const TArray<FSolverReal>& NuMeshArray,
			TArray<FSolverReal>&& AlphaJMeshArray,
			const FDeformableXPBDCorotatedParams& InParams,
			const FSolverReal& NuMesh = (FSolverReal).3,
			const bool bRecordMetricIn = false,
			const bool bDoColoring = true);

		FXPBDCorotatedTetrahedralConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 4>>& InMesh,
			const TArray<FSolverReal>& EMeshArray,
			const TArray<FSolverReal>& NuMeshArray,
			TArray<FSolverReal>&& AlphaJMeshArray,
			const FDeformableXPBDCorotatedParams& InParams,
			const FSolverReal& NuMesh = (FSolverReal).3,
			const bool bRecordMetricIn = false,
			const bool bDoColoring = true)
			: FXPBDCorotatedTetrahedralConstraints(
				FSolverParticlesRange(InParticles),
				InMesh, EMeshArray, NuMeshArray, MoveTemp(AlphaJMeshArray),
				InParams, NuMesh, bRecordMetricIn, bDoColoring)
		{}

		CHAOS_API FXPBDCorotatedTetrahedralConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 4>>& InMesh,
			const FSolverReal GridN,
			const FSolverReal& EMesh = (FSolverReal)10.0,
			const FSolverReal& NuMesh = (FSolverReal).3);

		virtual ~FXPBDCorotatedTetrahedralConstraints() = default;

		virtual void Init() const
		{
			for (FSolverReal& Lambdas : LambdaArray) { Lambdas = (FSolverReal)0.; }
		}

		CHAOS_API virtual void ApplyInSerial(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ElementIndex) const;

		CHAOS_API void ApplyInSerial(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

		CHAOS_API void ApplyInParallel(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

		CHAOS_API void ApplyInParallel(FSolverParticles& Particles, const FSolverReal Dt) const;

		void Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const
		{
			ApplyInParallel(Particles, Dt);
		}

		CHAOS_API void ModifyDmInverseFromMuscleLength(const int32 ElemIdx, const FSolverReal FiberLengthRatio, const PMatrix<FSolverReal, 3, 3>& FiberDir, const FSolverReal ContractionVolumeScale) const;

		CHAOS_API void ModifyDmInverseSaveFromInflationVolumeScale(const int32 ElemIdx, const FSolverReal InflationVolumeScale, const PMatrix<FSolverReal, 3, 3>& FiberDir);

	protected:

		PMatrix<FSolverReal, 3, 3> ElementDmInv(const int32 ElemIdx) const
		{
			PMatrix<FSolverReal, 3, 3> DmInv((FSolverReal)0.);
			for (int32 Row = 0; Row < 3; Row++) {
				for (int32 Col = 0; Col < 3; Col++) {
					DmInv.SetAt(Row, Col, DmInverse[(3 * 3) * ElemIdx + 3 * Row + Col]);
				}
			}
			return DmInv;
		}

		PMatrix<FSolverReal, 3, 3> ElementDmInvSave(const int32 ElemIdx) const
		{
			PMatrix<FSolverReal, 3, 3> DmInv((FSolverReal)0.);
			for (int32 Row = 0; Row < 3; Row++) {
				for (int32 Col = 0; Col < 3; Col++) {
					DmInv.SetAt(Row, Col, DmInverseSave[(3 * 3) * ElemIdx + 3 * Row + Col]);
				}
			}
			return DmInv;
		}

		PMatrix<FSolverReal, 3, 3> DsInit(const int32 ElemIdx, const FSolverParticlesRange& InParticles) const;

		PMatrix<FSolverReal, 3, 3> DsInit(const int32 ElemIdx, const FSolverParticles& InParticles) const
		{
			return DsInit(ElemIdx, FSolverParticlesRange(InParticles));
		}

		PMatrix<FSolverReal, 3, 3> Ds(const int32 ElemIdx, const FSolverParticlesRange& InParticles) const;

		PMatrix<FSolverReal, 3, 3> F(const int32 ElemIdx, const FSolverParticlesRange& InParticles) const
		{
			return ElementDmInv(ElemIdx) * Ds(ElemIdx, InParticles);
		}

		TVec4<TVector<FSolverReal, 3>> GetPolarGradient(const PMatrix<FSolverReal, 3, 3>& Fe, const PMatrix<FSolverReal, 3, 3>& Re, const PMatrix<FSolverReal, 3, 3>& DmInvT, const FSolverReal C1) const;

		TVec4<TVector<FSolverReal, 3>> GetDeterminantGradient(const PMatrix<FSolverReal, 3, 3>& Fe, const PMatrix<FSolverReal, 3, 3>& DmInvT) const;

		CHAOS_API void InitColor(const FSolverParticlesRange& Particles);

		void InitColor(const FSolverParticles& Particles)
		{
			return InitColor(FSolverParticlesRange(Particles));
		}

		CHAOS_API virtual TVec4<TVector<FSolverReal, 3>> GetDeterminantDelta(const FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ElementIndex, const FSolverReal Tol = (FSolverReal)1e-3) const;

		CHAOS_API virtual TVec4<TVector<FSolverReal, 3>> GetPolarDelta(const FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ElementIndex, const FSolverReal Tol = (FSolverReal)1e-3) const;

	protected:
		mutable TArray<FSolverReal> LambdaArray;
		mutable TArray<FSolverReal> DmInverse;
		TArray<FSolverReal> DmInverseSave;
		//parallel data:
		FDeformableXPBDCorotatedParams CorotatedParams;

		//material constants calculated from E:
		FSolverReal Mu;
		FSolverReal Lambda;
		TArray<FSolverReal> MuElementArray;
		TArray<FSolverReal> LambdaElementArray;
		TArray<FSolverReal> AlphaJArray;
		mutable FSolverReal HError;
		mutable TArray<FSolverReal> HErrorArray;
		bool bRecordMetric;
		bool VariableStiffness = false;

		TArray<TVector<int32, 4>> MeshConstraints;
		mutable TArray<FSolverReal> Measure;
		FSolverParticles RestParticles;
		TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.
		mutable TArray<FSolverReal> GError;
	};

	template <typename T, typename ParticleType>
	using FXPBDCorotatedConstraints UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FXPBDCorotatedTetrahedralConstraints instead") = FXPBDCorotatedTetrahedralConstraints;

}  // End namespace Chaos::Softs

