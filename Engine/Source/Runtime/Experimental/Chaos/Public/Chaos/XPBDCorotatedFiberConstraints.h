// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"
#include "Chaos/XPBDCorotatedConstraints.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"

//DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Corotated Constraint"), STAT_XPBD_Spring, STATGROUP_Chaos);

namespace Chaos::Softs
{
	// Tetrahedral constraint using fixed corotated + muscle fiber constitutive model
	// Fixed corotated: [Stomakhin et al., 2012]
	// Muscle fiber: [Blemker et al., 2005]
	class FXPBDCorotatedFiberTetrahedralConstraints : public FXPBDCorotatedTetrahedralConstraints
	{

		typedef FXPBDCorotatedTetrahedralConstraints Base;
		using Base::MeshConstraints;
		using Base::LambdaArray;
		using Base::Measure;
		using Base::DmInverse;

	public:
		//this one only accepts tetmesh input and mesh
		FXPBDCorotatedFiberTetrahedralConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 4>>& InMesh,
			const bool bRecordMetricIn = true,
			const FSolverReal& EMesh = (FSolverReal)10.0,
			const FSolverReal& NuMesh = (FSolverReal).3,
			const TVector<FSolverReal, 3> InFiberDir = TVector<FSolverReal, 3>((FSolverReal)1., (FSolverReal)0., (FSolverReal)0.),
			const FSolverReal InSigmaMax = (FSolverReal)3e5
		)
			: Base(InParticles, InMesh, bRecordMetricIn, EMesh, NuMesh), SigmaMax(InSigmaMax), FiberDir(InFiberDir)
		{
			LambdaArray.Init((FSolverReal)0., 3 * MeshConstraints.Num());
		}

		virtual ~FXPBDCorotatedFiberTetrahedralConstraints() = default;
		
		TVector<FSolverReal, 3> GetFiberDir() { return FiberDir; }

		void SetActivation(const FSolverReal AlphaIn) { AlphaActivation = AlphaIn; }

		void SetTime(const float Time) const {
			float FinalTime = 4.0f;
			float CurrentTime = Time;
			while (CurrentTime > FinalTime)
			{
				CurrentTime -= FinalTime;
			}
			AlphaActivation = (FSolverReal)1. - (FSolverReal)4. / FinalTime * FMath::Abs(CurrentTime - FinalTime / (FSolverReal)2.);
		}

		virtual void ApplyInSerial(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ElementIndex) const override
		{
			TVec4<TVector<FSolverReal, 3>> PolarDelta = Base::GetPolarDelta(Particles, Dt, ElementIndex);

			for (int i = 0; i < 4; i++)
			{
				Particles.P(MeshConstraints[ElementIndex][i]) += PolarDelta[i];
			}

			TVec4<TVector<FSolverReal, 3>> DetDelta = Base::GetDeterminantDelta(Particles, Dt, ElementIndex);

			for (int i = 0; i < 4; i++)
			{
				Particles.P(MeshConstraints[ElementIndex][i]) += DetDelta[i];
			}

			TVec4<TVector<FSolverReal, 3>> FiberDelta = GetFiberDelta(Particles, Dt, ElementIndex);

			for (int i = 0; i < 4; i++)
			{
				Particles.P(MeshConstraints[ElementIndex][i]) += FiberDelta[i];
			}
			

		}

		TVec4<TVector<FSolverReal, 3>> GetFiberGradient(const FSolverReal dFpdL, const FSolverReal dFadL,const FSolverReal C3, const TVec4<TVector<FSolverReal, 3>>& dLdX) const 
		{
			FSolverReal dC3dL = (FSolverReal)0.5 * (dFpdL + AlphaActivation * dFadL);
			if (C3 == 0)
			{
				return TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.));
			}
			else
			{
				dC3dL /= C3;
			}

			TVec4<TVector<FSolverReal, 3>> dC3(TVector<FSolverReal, 3>((FSolverReal)0.));
			for (int32 i = 0; i < 4; i++)
			{
				for (int32 j = 0; j < 3; j++)
				{
					dC3[i][j] = dC3dL * dLdX[i][j];
				}
			}

			return dC3;
		
		}


	private:

		TVec4<TVector<FSolverReal, 3>> GetFiberDelta(const FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ElementIndex, const FSolverReal Tol = FSolverReal(1e-3)) const
		{
			const PMatrix<FSolverReal, 3, 3> Fe = Base::F(ElementIndex, Particles);

			PMatrix<FSolverReal, 3, 3> Re((FSolverReal)0.), Se((FSolverReal)0.);

			Chaos::PolarDecomposition(Fe, Re, Se);

			// l: fiber stretch, f = (vTCv)^(1/2)
			TVector<FSolverReal, 3> FeV = Fe.GetTransposed() * FiberDir;
			TVector<FSolverReal, 3> DmInverseV = Base::ElementDmInv(ElementIndex).GetTransposed() * FiberDir;
			FSolverReal L = FeV.Size();
			TVec4<TVector<FSolverReal, 3>> dLdX(TVector<FSolverReal, 3>((FSolverReal)0.));
			for (int32 alpha = 0; alpha < 3; alpha++)
			{
				for (int32 s = 0; s < 3; s++)
				{
					dLdX[0][alpha] -= FeV[alpha] * DmInverseV[s] / L;
				}

			}
			for (int32 ie = 1; ie < 4; ie++)
			{
				for (int32 alpha = 0; alpha < 3; alpha++)
				{
					dLdX[ie][alpha] = FeV[alpha] * DmInverseV[ie - 1] / L;
				}
			}

			FSolverReal LambdaOFL = (FSolverReal)1.4;
			FSolverReal P1 = (FSolverReal)0.05;
			FSolverReal P2 = (FSolverReal)6.6;
			FSolverReal FpIntegral = (FSolverReal)0.;
			FSolverReal FaIntegral = (FSolverReal)0.;

			FSolverReal C3 = (FSolverReal)0.;
			FSolverReal AlphaTilde = LambdaOFL / (SigmaMax * Dt * Dt * Measure[ElementIndex]);
			FSolverReal dFpdL = (FSolverReal)0.;
			FSolverReal dFadL = (FSolverReal)0.;

			if (L > LambdaOFL)
			{
				FpIntegral = P1 * LambdaOFL / P2 * FMath::Exp(P2 * (L / LambdaOFL - (FSolverReal)1.)) - P1 * (L - LambdaOFL);
				dFpdL = P1 * FMath::Exp(P2 * (L / LambdaOFL - (FSolverReal)1.)) - P1;
			}
			if (L > (FSolverReal)0.4 * LambdaOFL && L < (FSolverReal)0.6 * LambdaOFL)
			{
				FaIntegral = (FSolverReal)3. * LambdaOFL * FMath::Pow((L / LambdaOFL - (FSolverReal)0.4), 3);
				dFadL = (FSolverReal)9. * FMath::Pow((L / LambdaOFL - (FSolverReal)0.4), 2);
			}
			else if (L >= (FSolverReal)0.6 * LambdaOFL && L <= (FSolverReal)1.4 * LambdaOFL)
			{
				FaIntegral = (FSolverReal)3. * LambdaOFL * (FSolverReal)0.008 + L - (FSolverReal)4. / (FSolverReal)3. * LambdaOFL * FMath::Pow(L / LambdaOFL - (FSolverReal)1., 3) - (FSolverReal)0.6 * LambdaOFL - (FSolverReal)4. / (FSolverReal)3. * LambdaOFL * FMath::Pow((FSolverReal)0.4, 3);
				dFadL = (FSolverReal)1. - (FSolverReal)4. * FMath::Pow(L / LambdaOFL - (FSolverReal)1., 2);
			}
			else if (L > (FSolverReal)1.4 * LambdaOFL && L <= (FSolverReal)1.6 * LambdaOFL)
			{
				FaIntegral = (FSolverReal)3. * LambdaOFL * (FSolverReal)0.008 + (FSolverReal)0.8 * LambdaOFL - (FSolverReal)8. / (FSolverReal)3. * LambdaOFL * FMath::Pow((FSolverReal)0.4, 3) + (FSolverReal)3. * LambdaOFL * FMath::Pow(L / LambdaOFL - (FSolverReal)1.6, 3) + (FSolverReal)3. * LambdaOFL * (FSolverReal)0.008;
				dFadL = (FSolverReal)9. * FMath::Pow((L / LambdaOFL - (FSolverReal)1.6), 2);

			}
			else if (L > (FSolverReal)1.6 * LambdaOFL)
			{
				FaIntegral = (FSolverReal)3. * LambdaOFL * (FSolverReal)0.008 + (FSolverReal)0.8 * LambdaOFL - (FSolverReal)8. / (FSolverReal)3. * LambdaOFL * FMath::Pow((FSolverReal)0.4, 3) + (FSolverReal)3. * LambdaOFL * (FSolverReal)0.008;
				dFadL = (FSolverReal)0.;
			}


			C3 = FMath::Sqrt(FpIntegral + AlphaActivation * FaIntegral);


			//FSolverReal dC3dL = (FSolverReal)0.5 * (dFpdL + AlphaActivation * dFadL);
			//if (C3 == 0)
			//{
			//	return TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.));
			//}
			//else
			//{
			//	dC3dL /= C3;
			//}

			TVec4<TVector<FSolverReal, 3>> dC3 = GetFiberGradient(dFpdL,dFadL, C3, dLdX);

			//TVec4<TVector<FSolverReal, 3>> dC3(TVector<FSolverReal, 3>((FSolverReal)0.));
			//for (int32 i = 0; i < 4; i++)
			//{
			//	for (int32 j = 0; j < 3; j++)
			//	{
			//		dC3[i][j] = dC3dL * dLdX[i][j];
			//	}
			//}

			FSolverReal DLambda = -C3 - AlphaTilde * LambdaArray[2 * ElementIndex + 2];

			FSolverReal Denom = AlphaTilde;
			for (int i = 0; i < 4; i++)
			{
				for (int j = 0; j < 3; j++)
				{
					Denom += dC3[i][j] * Particles.InvM(MeshConstraints[ElementIndex][i]) * dC3[i][j];
				}
			}
			DLambda /= Denom;
			LambdaArray[2 * ElementIndex + 2] += DLambda;
			TVec4<TVector<FSolverReal, 3>> Delta(TVector<FSolverReal, 3>((FSolverReal)0.));
			for (int i = 0; i < 4; i++)
			{
				for (int j = 0; j < 3; j++)
				{
					Delta[i][j] = Particles.InvM(MeshConstraints[ElementIndex][i]) * dC3[i][j] * DLambda;
				}
			}
			return Delta;



		}

	private:

		//material constants calculated from E:
		FSolverReal SigmaMax;
		mutable FSolverReal AlphaActivation;
		TVector<FSolverReal, 3> FiberDir;

	};

	template <typename T, typename ParticleType>
	using FXPBDCorotatedFiberConstraints UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FXPBDCorotatedFiberTetrahedralConstraints instead") = FXPBDCorotatedFiberTetrahedralConstraints;

}  // End namespace Chaos::Softs

