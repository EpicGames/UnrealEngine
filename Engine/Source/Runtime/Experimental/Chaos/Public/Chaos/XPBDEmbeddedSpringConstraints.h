// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/CollectionEmbeddedSpringConstraintFacade.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/XPBDSpringConstraints.h"
#include "Chaos/PBDFlatWeightMap.h"
#include "Chaos/SoftsSpring.h"
#include "Containers/Array.h"

namespace Chaos::Softs
{
	template<int32 Source, int32 Target>
	struct TIsXPBDEmbeddedSpringSizePrecompiled
	{
		static constexpr bool Value = false;
	};
	template<>
	struct TIsXPBDEmbeddedSpringSizePrecompiled<1, 1>
	{
		static constexpr bool Value = true;
	};
	template<>
	struct TIsXPBDEmbeddedSpringSizePrecompiled<1, 3>
	{
		static constexpr bool Value = true;
	};
	template<>
	struct TIsXPBDEmbeddedSpringSizePrecompiled<3, 3>
	{
		static constexpr bool Value = true;
	};

	template <int32 Source, int32 Target>
	class TXPBDEmbeddedSpringConstraints
	{
	public:
		static constexpr FSolverReal MinStiffness = (FSolverReal)0;
		static constexpr FSolverReal MaxStiffness = (FSolverReal)UE_BIG_NUMBER;
		static constexpr FSolverReal SoftMaxStiffness = (FSolverReal)1e14; // Stiffnesses greater than this will be treated as "hard" PBD constraints
		static constexpr FSolverReal MinDampingRatio = (FSolverReal)0.;
		static constexpr FSolverReal MaxDampingRatio = (FSolverReal)1000.;
		static constexpr int32 N = Source + Target;
		static constexpr bool bCanUseISPC = INTEL_ISPC && TIsXPBDEmbeddedSpringSizePrecompiled<Source, Target>::Value;

		TXPBDEmbeddedSpringConstraints(
			const FSolverParticlesRange& Particles,
			const TConstArrayView<TArray<int32>>& InSourceIndices,
			const TConstArrayView<TArray<FRealSingle>>& InSourceWeights,
			const TConstArrayView<TArray<int32>>& InTargetIndices,
			const TConstArrayView<TArray<FRealSingle>>& InTargetWeights,
			const TConstArrayView<FRealSingle>& InSpringLengths,
			const TConstArrayView<FRealSingle>& InExtensionStiffnessMultipliers,
			const TConstArrayView<FRealSingle>& InCompressionStiffnessMultipliers,
			const TConstArrayView<FRealSingle>& InDampingMultipliers,
			const FSolverVec2& InExtensionStiffness,
			const FSolverVec2& InCompressionStiffness,
			const FSolverVec2& InDampingRatio);

		virtual ~TXPBDEmbeddedSpringConstraints() = default;

		void Init()
		{
			Lambdas.Reset();
			Lambdas.SetNumZeroed(Constraints.Num());
			LambdasDamping.Reset();
			LambdasDamping.SetNumZeroed(Constraints.Num());
		}

		void ApplyProperties(const FSolverReal /*Dt*/, const int32 /*NumIterations*/)
		{
		}

		void Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

		const TArray<TVector<int32, N>>& GetConstraints() const
		{
			return Constraints;
		}
		const TArray<TVector<FSolverReal, N>>& GetWeights() const
		{
			return Weights;
		}
		const TArray<FSolverReal>& GetSpringLengths() const
		{
			return SpringLengths;
		}
		FSolverReal GetExtensionStiffness(const int32 ConstraintIndex) const
		{
			return ExtensionStiffness.GetValue(ConstraintIndex);
		}
		FSolverReal GetCompressionStiffness(const int32 ConstraintIndex) const
		{
			return CompressionStiffness.GetValue(ConstraintIndex);
		}

	private:

		void InitColorAndRemap(const FSolverParticlesRange& Particles);

#if INTEL_ISPC
		void InitColor(const FSolverParticlesRange& Particles);
		CHAOS_API void ApplyISPC(FSolverParticlesRange& Particles, const FSolverReal Dt) const;
#endif
		void ApplyInternal(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

		TArray<TVector<int32, N>> Constraints;
		TArray<TVector<FSolverReal, N>> Weights; // Weights for Targets are -TargetWeight
		TArray<FSolverReal> SpringLengths;

		TArray<int32> OrigMapToReordered; // Constraints can be trimmed and reordered for coloring.

		mutable TArray<FSolverReal> Lambdas;
		mutable TArray<FSolverReal> LambdasDamping;
		TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.

	protected:
		FPBDFlatWeightMap ExtensionStiffness;
		FPBDFlatWeightMap CompressionStiffness;
		FPBDFlatWeightMap DampingRatio;
	};

	template<int32 Source, int32 Target>
	TXPBDEmbeddedSpringConstraints<Source, Target>::TXPBDEmbeddedSpringConstraints(
		const FSolverParticlesRange& Particles,
		const TConstArrayView<TArray<int32>>& InSourceIndices,
		const TConstArrayView<TArray<FRealSingle>>& InSourceWeights,
		const TConstArrayView<TArray<int32>>& InTargetIndices,
		const TConstArrayView<TArray<FRealSingle>>& InTargetWeights,
		const TConstArrayView<FRealSingle>& InSpringLengths,
		const TConstArrayView<FRealSingle>& InExtensionStiffnessMultipliers,
		const TConstArrayView<FRealSingle>& InCompressionStiffnessMultipliers,
		const TConstArrayView<FRealSingle>& InDampingMultipliers,
		const FSolverVec2& InExtensionStiffness,
		const FSolverVec2& InCompressionStiffness,
		const FSolverVec2& InDampingRatio)
		: ExtensionStiffness(InExtensionStiffness.ClampAxes(MinStiffness, MaxStiffness), InExtensionStiffnessMultipliers, InSourceIndices.Num(), FSolverVec2(MinStiffness, MaxStiffness))
		, CompressionStiffness(InCompressionStiffness.ClampAxes(MinStiffness, MaxStiffness), InCompressionStiffnessMultipliers, InSourceIndices.Num(), FSolverVec2(MinStiffness, MaxStiffness))
		, DampingRatio(InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio), InDampingMultipliers, InSourceIndices.Num(), FSolverVec2(MinDampingRatio, MaxDampingRatio))
	{
		// Validate data
		const int32 NumConstraints = InSourceIndices.Num();
		check(InSourceWeights.Num() == NumConstraints);
		check(InTargetIndices.Num() == NumConstraints);
		check(InTargetWeights.Num() == NumConstraints);
		check(InSpringLengths.Num() == NumConstraints);
		check(InExtensionStiffnessMultipliers.IsEmpty() || InExtensionStiffnessMultipliers.Num() == NumConstraints);
		check(InCompressionStiffnessMultipliers.IsEmpty() || InCompressionStiffnessMultipliers.Num() == NumConstraints);
		check(InDampingMultipliers.IsEmpty() || InDampingMultipliers.Num() == NumConstraints);

		Constraints.Reserve(NumConstraints);
		Weights.Reserve(NumConstraints);
		SpringLengths.Reserve(NumConstraints);

		OrigMapToReordered.SetNumUninitialized(NumConstraints);

		auto IsKinematic = [&Particles](const TArray<int32>& Indices, const TArray<FRealSingle>& InWeights, int32 ExpectedNum)
			{
				for (int32 Idx = 0; Idx < ExpectedNum; ++Idx)
				{
					if (InWeights[Idx] != 0.f && Particles.InvM(Indices[Idx]) != 0.f)
					{
						return false;
					}
				}
				return true;
			};

		for (int32 ConstraintIdx = 0; ConstraintIdx < NumConstraints; ++ConstraintIdx)
		{
			if (IsKinematic(InSourceIndices[ConstraintIdx], InSourceWeights[ConstraintIdx], Source)
				&& IsKinematic(InTargetIndices[ConstraintIdx], InTargetWeights[ConstraintIdx], Target))
			{
				// Strip this constraint
				OrigMapToReordered[ConstraintIdx] = INDEX_NONE;
				continue;
			}

			TVector<int32, N> Constraint;
			TVector<FSolverReal, N> Weight;
			for (int32 Idx = 0; Idx < Source; ++Idx)
			{
				Constraint[Idx] = InSourceIndices[ConstraintIdx][Idx];
				Weight[Idx] = InSourceWeights[ConstraintIdx][Idx];
			}

			for (int32 Idx = 0; Idx < Target; ++Idx)
			{
				Weight[Source + Idx] = -InTargetWeights[ConstraintIdx][Idx];
				Constraint[Source + Idx] = InTargetIndices[ConstraintIdx][Idx];
			}

			const int32 NewConstraintIndex = Constraints.Emplace(Constraint);
			Weights.Emplace(Weight);
			SpringLengths.Emplace(InSpringLengths[ConstraintIdx]);

			OrigMapToReordered[ConstraintIdx] = NewConstraintIndex;
		}

		// You must call InitColorAndRemap and at least update the Stiffness maps with the OrigMapToReordered data.
		InitColorAndRemap(Particles);
	}

	template<int32 Source, int32 Target>
	void TXPBDEmbeddedSpringConstraints<Source, Target>::InitColorAndRemap(const FSolverParticlesRange& Particles)
	{
#if INTEL_ISPC
		if constexpr (bCanUseISPC)
		{
			InitColor(Particles);
		}
#endif
		// Need to reorder and shrink even if not using ISPC since this also handles trimmed constraints.
		const int32 NumConstraints = Constraints.Num();
		ExtensionStiffness.ReorderIndicesAndShrink(OrigMapToReordered, NumConstraints);
		CompressionStiffness.ReorderIndicesAndShrink(OrigMapToReordered, NumConstraints);
		DampingRatio.ReorderIndicesAndShrink(OrigMapToReordered, NumConstraints);
	}

	template<int32 Source, int32 Target>
	void TXPBDEmbeddedSpringConstraints<Source, Target>::Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TXPBDEmbeddedSpringConstraints_Apply);
#if INTEL_ISPC
		if (bChaos_XPBDSpring_ISPC_Enabled && bCanUseISPC)
		{
			ApplyISPC(Particles, Dt);
		}
		else
#endif
		{
			ApplyInternal(Particles, Dt);
		}
	}

	template<int32 Source, int32 Target>
	void TXPBDEmbeddedSpringConstraints<Source, Target>::ApplyInternal(FSolverParticlesRange& Particles, const FSolverReal Dt) const
	{
		const bool bDampingHasWeightMap = DampingRatio.HasWeightMap();
		const FSolverReal DampingNoMap = (FSolverReal)DampingRatio;
		if (DampingNoMap > 0 || bDampingHasWeightMap)
		{
			// Damping
			for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
			{
				const FSolverVec3 Delta = Spring::GetXPBDEmbeddedSpringDampingDelta(Particles, Dt, Constraints[ConstraintIndex], Weights[ConstraintIndex], SpringLengths[ConstraintIndex], LambdasDamping[ConstraintIndex], ExtensionStiffness.GetValue(ConstraintIndex), CompressionStiffness.GetValue(ConstraintIndex), DampingRatio.GetValue(ConstraintIndex));

				for (int32 NIndex = 0; NIndex < N; ++NIndex)
				{
					const int32 NodeIndex = Constraints[ConstraintIndex][NIndex];
					Particles.P(NodeIndex) += Particles.InvM(NodeIndex) * Weights[ConstraintIndex][NIndex] * Delta;
				}
			}
		}

		// Stretch
		for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints.Num(); ++ConstraintIndex)
		{
			const FSolverVec3 Delta = Spring::GetXPBDEmbeddedSpringDelta(Particles, Dt, Constraints[ConstraintIndex], Weights[ConstraintIndex], SpringLengths[ConstraintIndex], Lambdas[ConstraintIndex], ExtensionStiffness.GetValue(ConstraintIndex), CompressionStiffness.GetValue(ConstraintIndex));

			for (int32 NIndex = 0; NIndex < N; ++NIndex)
			{
				const int32 NodeIndex = Constraints[ConstraintIndex][NIndex];
				Particles.P(NodeIndex) += Particles.InvM(NodeIndex) * Weights[ConstraintIndex][NIndex] * Delta;
			}
		}
	}

	class FXPBDVertexConstraints : public TXPBDEmbeddedSpringConstraints<1, 1>
	{
	public:
		static constexpr FSolverReal DefaultStiffness = (FSolverReal)100.;
		static constexpr FSolverReal DefaultDamping = (FSolverReal)1.;

		static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
		{
			return IsVertexSpringExtensionStiffnessEnabled(PropertyCollection, false);
		}

		CHAOS_API FXPBDVertexConstraints(
			const FSolverParticlesRange& Particles,
			const FCollectionPropertyConstFacade& PropertyCollection,
			const FEmbeddedSpringConstraintFacade& SpringConstraintFacade
		);

		CHAOS_API void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection);
	
	private:
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(VertexSpringExtensionStiffness, float);
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(VertexSpringCompressionStiffness, float);
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(VertexSpringDamping, float);
	};

	class FXPBDVertexFaceConstraints : public TXPBDEmbeddedSpringConstraints<1, 3>
	{
	public:
		static constexpr FSolverReal DefaultStiffness = (FSolverReal)100.;
		static constexpr FSolverReal DefaultDamping = (FSolverReal)1.;

		static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
		{
			return IsVertexFaceSpringExtensionStiffnessEnabled(PropertyCollection, false);
		}

		CHAOS_API FXPBDVertexFaceConstraints(
			const FSolverParticlesRange& Particles,
			const FCollectionPropertyConstFacade& PropertyCollection,
			const FEmbeddedSpringConstraintFacade& SpringConstraintFacade
		);

		CHAOS_API void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection);

	private:
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(VertexFaceSpringExtensionStiffness, float);
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(VertexFaceSpringCompressionStiffness, float);
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(VertexFaceSpringDamping, float);
	};

	class FXPBDFaceConstraints : public TXPBDEmbeddedSpringConstraints<3, 3>
	{
	public:
		static constexpr FSolverReal DefaultStiffness = (FSolverReal)100.;
		static constexpr FSolverReal DefaultDamping = (FSolverReal)1.;

		static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
		{
			return IsFaceSpringExtensionStiffnessEnabled(PropertyCollection, false);
		}

		CHAOS_API FXPBDFaceConstraints(
			const FSolverParticlesRange& Particles,
			const FCollectionPropertyConstFacade& PropertyCollection,
			const FEmbeddedSpringConstraintFacade& SpringConstraintFacade
		);

		CHAOS_API void SetProperties(const FCollectionPropertyConstFacade& PropertyCollection);

	private:
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(FaceSpringExtensionStiffness, float);
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(FaceSpringCompressionStiffness, float);
		UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(FaceSpringDamping, float);
	};
}  // End namespace Chaos::Softs

