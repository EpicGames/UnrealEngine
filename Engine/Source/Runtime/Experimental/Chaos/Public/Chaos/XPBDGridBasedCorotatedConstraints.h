// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "Chaos/PBDSpringConstraintsBase.h"
#include "Chaos/XPBDCorotatedConstraints.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"
#include "Chaos/MPMTransfer.h"
#include "Misc/FileHelper.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/Paths.h"

//DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Corotated Constraint"), STAT_XPBD_Spring, STATGROUP_Chaos);

namespace Chaos::Softs
{
	class FXPBDGridBasedCorotatedTetrahedralConstraints : public FXPBDCorotatedTetrahedralConstraints
	{

		typedef FXPBDCorotatedTetrahedralConstraints Base;
		using Base::MeshConstraints;
		using Base::LambdaArray;
		using Base::Measure;
		using Base::DmInverse;
		using Base::Mu;
		using Base::Init;
		using Base::Lambda;

	public:
		//this one only accepts tetmesh input and mesh
		FXPBDGridBasedCorotatedTetrahedralConstraints(
			const FSolverParticles& InParticles,
			const TArray<TVector<int32, 4>>& InMesh,
			const FSolverReal GridDx = (FSolverReal).1,
			const bool bRecordMetricIn = true,
			const FSolverReal MaxDtIn = (FSolverReal).1,
			const FSolverReal MinDtIn = (FSolverReal).01,
			const FSolverReal CFLCoeffIn = (FSolverReal).4,
			const FSolverReal& EMesh = (FSolverReal)1000.0,
			const FSolverReal& NuMesh = (FSolverReal).3
		)
			: Base(InParticles, InMesh, GridDx, EMesh, NuMesh), MaxDt(MaxDtIn), MinDt(MinDtIn), CFLCoeff(CFLCoeffIn), MPMGrid(GridDx), MPMTransfer(MPMGrid, InParticles.Size()), PreviousColoringPtr(nullptr)
		{
			//TMPMGrid<FSolverReal> MPMGrid(GridN);
			MPMGrid.UpdateGridFromPositions(InParticles);
			//TMPMTransfer<FSolverReal> MPMTrasnfer(MPMGrid);
			LambdaArray.Init((FSolverReal)0., 2 * MeshConstraints.Num());
			DmInverse.Init((FSolverReal)0., 9 * MeshConstraints.Num());
			Measure.Init((FSolverReal)0., MeshConstraints.Num());
			Lambda = EMesh * NuMesh / (((FSolverReal)1. + NuMesh) * ((FSolverReal)1. - (FSolverReal)2. * NuMesh));
			Mu = EMesh / ((FSolverReal)2. * ((FSolverReal)1. + NuMesh));
			LambdaElementArray.Init(Lambda, MeshConstraints.Num());
			MuElementArray.Init(Mu, MeshConstraints.Num());
			for (int e = 0; e < InMesh.Num(); e++)
			{
				PMatrix<FSolverReal, 3, 3> Dm = Base::DsInit(e, InParticles);
				PMatrix<FSolverReal, 3, 3> DmInv = Dm.Inverse();
				for (int r = 0; r < 3; r++) {
					for (int c = 0; c < 3; c++) {
						DmInverse[(3 * 3) * e + 3 * r + c] = DmInv.GetAt(r, c);
					}
				}

				Measure[e] = Dm.Determinant() / (FSolverReal)6.;

				if (Measure[e] < (FSolverReal)0.)
				{
					Measure[e] = -Measure[e];
				}

			}

			ElementsPerColor = FGraphColoring::ComputeGraphColoringAllDynamic(MeshConstraints, InParticles);
			Init(InParticles, (FSolverReal)0.);
			WriteGridNodes();
		}

		void WriteGridNodes()
		{
			FString file = FPaths::ProjectDir();
			//file.Append(TEXT("\HoudiniOuput\Test.geo"));
			file.Append(TEXT("/HoudiniOutput/GridData.geo"));

			TArray<TVector<FSolverReal, 3>> ActivatedGridNodes;

			for (int32 i = 0; i < MPMGrid.Size(); i++)
			{
				if (GridData[i * (MPMTransfer.NTransfer+1)] > FSolverReal(0.))
				{
					TVector<FSolverReal, 3> GridNode = MPMGrid.Node(i);
					ActivatedGridNodes.Emplace(GridNode);
				}
			}

			int32 Np = ActivatedGridNodes.Num();

			// We will use this FileManager to deal with the file.
			IPlatformFile& FileManager = FPlatformFileManager::Get().GetPlatformFile();
			FFileHelper::SaveStringToFile(FString("PGEOMETRY V5\r\n"), *file);
			FString HeaderInfo = FString("NPoints ") + FString::FromInt(Np) + FString(" NPrims ") + FString::FromInt(0) + FString("\r\n");
			FString MoreHeader = FString("NPointGroups 0 NPrimGroups 0\r\nNPointAttrib 0 NVertexAttrib 0 NPrimAttrib 0 NAttrib 0\r\n");

			FFileHelper::SaveStringToFile(HeaderInfo, *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
			FFileHelper::SaveStringToFile(MoreHeader, *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);

			for (int32 i = 0; i < Np; i++) {

				FString ParticleInfo = FString::SanitizeFloat(ActivatedGridNodes[i][0]) + FString(" ") + FString::SanitizeFloat(ActivatedGridNodes[i][1]) + FString(" ") + FString::SanitizeFloat(ActivatedGridNodes[i][2]) + FString(" ") + FString::FromInt(1) + FString("\r\n");
				FFileHelper::SaveStringToFile(ParticleInfo, *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);

			}


			FFileHelper::SaveStringToFile(FString("beginExtra\n"), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);
			FFileHelper::SaveStringToFile(FString("endExtra\n"), *file, FFileHelper::EEncodingOptions::AutoDetect, &IFileManager::Get(), EFileWrite::FILEWRITE_Append);

		}

		void Init(const FSolverParticles& InParticles, const FSolverReal Dt) 
		{
			for (FSolverReal& Lambdas : LambdaArray) { Lambdas = (FSolverReal)0.; }
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDGridBasedCorotatedUpdateGrid);
				MPMTransfer.Grid.UpdateGridFromPositions(InParticles);
			}
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDGridBasedCorotatedInitialP2G);
				MPMTransfer.InitialP2G(InParticles, GridData);
			}
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDGridBasedCorotatedMetaDataComputation);
				MPMTransfer.ComputeElementMetaData(MeshConstraints);
			}
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDGridBasedCorotatedSubcoloring);
				ComputeGridBasedGraphSubColoringPointer(ElementsPerColor, MPMTransfer.Grid, MPMTransfer.Grid.Size(), PreviousColoringPtr, MPMTransfer.ElementGridNodesSet, ElementsPerSubColors);
			}
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDGridBasedCorotatedComputeGridPositions);
				MPMTransfer.ComputeGridPositions(GridData, Dt, GridPositions);
			}
		}

		void TimeStepPostprocessing(FSolverParticles& InParticles, const FSolverReal Dt)
		{
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDGridBasedCorotatedGrid::FinalG2P);
				FinalG2P(InParticles);
			}
			
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDGridBasedCorotated::GridPositions2GridData);
				MPMTransfer.GridPositionsToGridData(GridPositions, Dt, GridData);
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDGridBasedCorotated::ComputeAArray);
				MPMTransfer.ComputeAArray(GridData, InParticles);
			}
		
		}

		FSolverReal ComputeCFLDt(const FSolverParticles& InParticles)
		{
			FSolverReal VMax = (FSolverReal)-1.;
			for (int32 p = 0; p < int32(InParticles.Size()); p++)
			{
				FSolverReal VMag = InParticles.V(p).Size();
				if (VMag > VMax)
				{
					VMax = VMag;
				}
			}

			if (VMax > 1e5)
			{
				return MinDt;
			}
			if (VMax < 1e-5)
			{
				return MaxDt;
			}

			FSolverReal CFLDt = MPMGrid.Dx()[0] * CFLCoeff / VMax;

			if (CFLDt > MaxDt)
			{
				return MaxDt;
			}

			if (CFLDt < MinDt)
			{
				return MinDt;
			}

			return CFLDt;

		}

		const TArray<TArray<int32>> GetElementsPerColor () const { return ElementsPerColor; }
		const TArray<TArray<int32>> GetPreviousColoring () const { return *PreviousColoringPtr; }
		const TArray<TArray<TArray<int32>>> GetElementsPerSubColors() const { return ElementsPerSubColors; }
		const TMPMTransfer<FSolverReal> GetMPMTransfer() const { return MPMTransfer; }

		void ApplyPolar(FSolverParticles& Particles, const FSolverReal Dt, const int32 ElementIndex)  
		{
			//step 1: sparse g2p:
			TVec4<TVector<FSolverReal, 3>> XPosFromGrid = MPMTransfer.SparseG2P(GridPositions, ElementIndex);

			//step 2: gradient and constraint computation:
			FSolverReal C1 = (FSolverReal)0.;
			TVec4<TVector<FSolverReal, 3>> PolarGradient = GetPolarGradient(XPosFromGrid, ElementIndex, C1);
			
			if (C1 == (FSolverReal)0.)
			{
				return;
			}

			//step 3 : sparse p2g:
			TArray<TVector<FSolverReal, 3>> GridGradient = MPMTransfer.SparseP2G(PolarGradient, ElementIndex);

			//step 4: computation for delta:
			FSolverReal AlphaTilde = (FSolverReal)1. / (Dt * Dt * Mu * Measure[ElementIndex]);

			FSolverReal DeltaLambda = -C1 - AlphaTilde * LambdaArray[2 * ElementIndex + 0];
			FSolverReal Denom = AlphaTilde;

			for (int32 k = 0; k < MPMTransfer.ElementGridNodeIncidentElements[ElementIndex].Num(); k++) {
				FSolverReal Mass = GridData[(MPMTransfer.NTransfer + 1) * MPMTransfer.ElementGridNodes[ElementIndex][MPMTransfer.ElementGridNodeIncidentElements[ElementIndex][k][0]]];
				if (Mass != 0) {
					for (int32 alpha = 0; alpha < 3; alpha++) {
						Denom += GridGradient[k][alpha] * GridGradient[k][alpha] / Mass;
					}
				}
			}

			DeltaLambda /= Denom;
			LambdaArray[2 * ElementIndex + 0] += DeltaLambda;

			for (int32 k = 0; k < MPMTransfer.ElementGridNodeIncidentElements[ElementIndex].Num(); k++) {
				int32 CurrentNode = MPMTransfer.ElementGridNodes[ElementIndex][MPMTransfer.ElementGridNodeIncidentElements[ElementIndex][k][0]];
				FSolverReal Mass = GridData[(MPMTransfer.NTransfer + 1) * CurrentNode];
				if (Mass != 0) {
					for (int32 alpha = 0; alpha < 3; alpha++) {
						GridPositions[CurrentNode][alpha] += DeltaLambda * GridGradient[k][alpha] / Mass;
					}
				}
			}
		}

		void ApplyDet(FSolverParticles& Particles, const FSolverReal Dt, const int32 ElementIndex) 
		{
			//TArray<TVector<FSolverReal, 3>> XPosFromGrid;
			//XPosFromGrid.SetNum(4);
			//step 1: sparse g2p:
			TVec4<TVector<FSolverReal, 3>> XPosFromGrid = MPMTransfer.SparseG2P(GridPositions, ElementIndex);

			//step 2: gradient and constraint computation:
			FSolverReal C2 = (FSolverReal)0.;
			TVec4<TVector<FSolverReal, 3>> DetGradient = GetDetGradient(XPosFromGrid, ElementIndex, C2);

			if (C2 == (FSolverReal)0.)
			{
				return;
			}

			//step 3 : sparse p2g:
			TArray<TVector<FSolverReal, 3>> GridGradient = MPMTransfer.SparseP2G(DetGradient, ElementIndex);

			//step 4: computation for delta:
			FSolverReal AlphaTilde = (FSolverReal)2. / (Dt * Dt * Lambda * Measure[ElementIndex]);

			FSolverReal DeltaLambda = -C2 - AlphaTilde * LambdaArray[2 * ElementIndex + 1];
			FSolverReal Denom = AlphaTilde;

			for (int32 k = 0; k < MPMTransfer.ElementGridNodeIncidentElements[ElementIndex].Num(); k++) {
				FSolverReal Mass = GridData[(MPMTransfer.NTransfer + 1) * MPMTransfer.ElementGridNodes[ElementIndex][MPMTransfer.ElementGridNodeIncidentElements[ElementIndex][k][0]]];
				if (Mass != 0) {
					for (int32 alpha = 0; alpha < 3; alpha++) {
						Denom += GridGradient[k][alpha] * GridGradient[k][alpha] / Mass;
					}
				}
			}

			DeltaLambda /= Denom;
			LambdaArray[2 * ElementIndex + 1] += DeltaLambda;

			for (int32 k = 0; k < MPMTransfer.ElementGridNodeIncidentElements[ElementIndex].Num(); k++) {
				int32 CurrentNode = MPMTransfer.ElementGridNodes[ElementIndex][MPMTransfer.ElementGridNodeIncidentElements[ElementIndex][k][0]];
				FSolverReal Mass = GridData[(MPMTransfer.NTransfer + 1) * CurrentNode];
				if (Mass != 0) {
					for (int32 alpha = 0; alpha < 3; alpha++) {
						GridPositions[CurrentNode][alpha] += DeltaLambda * GridGradient[k][alpha] / Mass;
					}
				}
			}
		}

		void ApplyInParallel(FSolverParticles& Particles, const FSolverReal Dt) 
		{
			for (int32 color = 0; color < ElementsPerSubColors.Num(); color++)
			{
				for (int32 subcolor = 0; subcolor < ElementsPerSubColors[color].Num(); subcolor++)
				{
					PhysicsParallelFor(ElementsPerSubColors[color][subcolor].Num(), [&](const int32 ColorIndex)
						{
							const int32 ElementIndex = ElementsPerSubColors[color][subcolor][ColorIndex];
							ApplyPolar(Particles, Dt, ElementIndex);
							ApplyDet(Particles, Dt, ElementIndex);
						}, ElementsPerSubColors[color][subcolor].Num() < 30);
				}
			}
			//final G2P to compute particle positions:
			//FinalG2P(Particles);
		}

	private:

		PMatrix<FSolverReal, 3, 3> Ds(const TVec4<TVector<FSolverReal, 3>>& LocalPositions) const {
			PMatrix<FSolverReal, 3, 3> Result((FSolverReal)0.);
			for (int i = 0; i < 3; i++) {
				for (int c = 0; c < 3; c++) {
					Result.SetAt(c, i, LocalPositions[i+1][c] - LocalPositions[0][c]);
				}
			}
			return Result;
		}


		PMatrix<FSolverReal, 3, 3> F(const TVec4<TVector<FSolverReal, 3>>& LocalPositions, const int32 ElementIndex) const 
		{
			return Base::ElementDmInv(ElementIndex) * Ds(LocalPositions);
		}

		TVec4<TVector<FSolverReal, 3>> GetPolarGradient(const TVec4<TVector<FSolverReal, 3>>& LocalPositions, const int32 ElementIndex, FSolverReal& Constraint, const FSolverReal Tol = FSolverReal(1e-4)) const
		{
			PMatrix<FSolverReal, 3, 3> Fe = F(LocalPositions, ElementIndex);
			PMatrix<FSolverReal, 3, 3> Re((FSolverReal)0.), Se((FSolverReal)0.);

			Chaos::PolarDecomposition(Fe, Re, Se);

			FSolverReal C1 = (FSolverReal)0.;
			for (int i = 0; i < 3; i++)
			{
				for (int j = 0; j < 3; j++)
				{
					C1 += FMath::Square((Fe - Re).GetAt(i, j));
				}
			}
			C1 = FMath::Sqrt(C1);

			
			if (C1 < Tol)
			{
				Constraint = (FSolverReal)0.;
				return TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.));
			}
			Constraint = C1;

			PMatrix<FSolverReal, 3, 3> DmInvT = Base::ElementDmInv(ElementIndex).GetTransposed();
			TVector<FSolverReal, 81> dRdF((FSolverReal)0.);
			Chaos::dRdFCorotated(Fe, dRdF);
			TVec4<TVector<FSolverReal, 3>> dC1(TVector<FSolverReal, 3>((FSolverReal)0.));
			//dC1 = dC1dF * dFdX
			for (int alpha = 0; alpha < 3; alpha++) {
				for (int l = 0; l < 3; l++) {

					dC1[0][alpha] += (DmInvT * Re).GetAt(alpha, l) - (DmInvT * Fe).GetAt(alpha, l);

				}
			}
			for (int ie = 0; ie < 3; ie++) {
				for (int alpha = 0; alpha < 3; alpha++) {
					dC1[ie + 1][alpha] = (DmInvT * Fe).GetAt(alpha, ie) - (DmInvT * Re).GetAt(alpha, ie);
				}
			}
			//it's really ie-1 here
			for (int ie = 0; ie < 3; ie++) {
				for (int alpha = 0; alpha < 3; alpha++) {
					for (int m = 0; m < 3; m++) {
						for (int n = 0; n < 3; n++) {
							for (int j = 0; j < 3; j++) {
								dC1[ie + 1][alpha] -= (Fe.GetAt(m, n) - Re.GetAt(m, n)) * dRdF[9 * (alpha * 3 + j) + m * 3 + n] * DmInvT.GetAt(j, ie);
							}
						}
					}
				}
			}
			for (int alpha = 0; alpha < 3; alpha++) {
				for (int m = 0; m < 3; m++) {
					for (int n = 0; n < 3; n++) {
						for (int l = 0; l < 3; l++) {
							for (int j = 0; j < 3; j++) {
								dC1[0][alpha] += (Fe.GetAt(m, n) - Re.GetAt(m, n)) * dRdF[9 * (alpha * 3 + j) + m * 3 + n] * DmInvT.GetAt(j, l);
							}
						}
					}
				}
			}

			if (C1 != 0)
			{
				for (int i = 0; i < 4; i++)
				{
					for (int j = 0; j < 3; j++)
					{
						dC1[i][j] /= C1;
					}
				}
			}
			return dC1;
		}

		TVec4<TVector<FSolverReal, 3>> GetDetGradient(const TVec4<TVector<FSolverReal, 3>>& LocalPositions, const int32 ElementIndex, FSolverReal& Constraint, const FSolverReal Tol = FSolverReal(1e-4)) const
		{
			const PMatrix<FSolverReal, 3, 3> Fe = F(LocalPositions, ElementIndex);
			PMatrix<FSolverReal, 3, 3> DmInvT = Base::ElementDmInv(ElementIndex).GetTransposed();

			FSolverReal J = Fe.Determinant();
			if (J - 1 < Tol)
			{
				Constraint = (FSolverReal)0.;
				return TVec4<TVector<FSolverReal, 3>>(TVector<FSolverReal, 3>((FSolverReal)0.));
			}
			Constraint = J - 1;

			TVec4<TVector<FSolverReal, 3>> dC2(TVector<FSolverReal, 3>((FSolverReal)0.));

			PMatrix<FSolverReal, 3, 3> JFinvT((FSolverReal)0.);
			JFinvT.SetAt(0, 0, Fe.GetAt(1, 1) * Fe.GetAt(2, 2) - Fe.GetAt(2, 1) * Fe.GetAt(1, 2));
			JFinvT.SetAt(0, 1, Fe.GetAt(2, 0) * Fe.GetAt(1, 2) - Fe.GetAt(1, 0) * Fe.GetAt(2, 2));
			JFinvT.SetAt(0, 2, Fe.GetAt(1, 0) * Fe.GetAt(2, 1) - Fe.GetAt(2, 0) * Fe.GetAt(1, 1));
			JFinvT.SetAt(1, 0, Fe.GetAt(2, 1) * Fe.GetAt(0, 2) - Fe.GetAt(0, 1) * Fe.GetAt(2, 2));
			JFinvT.SetAt(1, 1, Fe.GetAt(0, 0) * Fe.GetAt(2, 2) - Fe.GetAt(2, 0) * Fe.GetAt(0, 2));
			JFinvT.SetAt(1, 2, Fe.GetAt(2, 0) * Fe.GetAt(0, 1) - Fe.GetAt(0, 0) * Fe.GetAt(2, 1));
			JFinvT.SetAt(2, 0, Fe.GetAt(0, 1) * Fe.GetAt(1, 2) - Fe.GetAt(1, 1) * Fe.GetAt(0, 2));
			JFinvT.SetAt(2, 1, Fe.GetAt(1, 0) * Fe.GetAt(0, 2) - Fe.GetAt(0, 0) * Fe.GetAt(1, 2));
			JFinvT.SetAt(2, 2, Fe.GetAt(0, 0) * Fe.GetAt(1, 1) - Fe.GetAt(1, 0) * Fe.GetAt(0, 1));

			PMatrix<FSolverReal, 3, 3> JinvTDmInvT = DmInvT * JFinvT;

			for (int32 ie = 0; ie < 3; ie++) {
				for (int32 alpha = 0; alpha < 3; alpha++) {
					dC2[ie + 1][alpha] = JinvTDmInvT.GetAt(alpha, ie);
				}
			}
			for (int32 alpha = 0; alpha < 3; alpha++) {
				for (int32 l = 0; l < 3; l++) {
					dC2[0][alpha] -= JinvTDmInvT.GetAt(alpha, l);
				}
			}
			return dC2;
		}

		//interpolate particle positions from grid positions
		void FinalG2P(FSolverParticles& Particles)
		{
			PhysicsParallelFor(Particles.Size(), [&](const int32 p)
				{
					Particles.P(p) = TVec3<FSolverReal>((FSolverReal)0.);
					for (int32 ii = 0; ii < int32(MPMGrid.NPerDir); ii++)
					{
						FSolverReal Nii = MPMGrid.Nijk(MPMTransfer.Weights[p][0], ii);
						for (int32 jj = 0; jj < int32(MPMGrid.NPerDir); jj++)
						{
							FSolverReal Njj = MPMGrid.Nijk(MPMTransfer.Weights[p][1], jj);
							for (int32 kk = 0; kk < int32(MPMGrid.NPerDir); kk++)
							{
								FSolverReal Nkk = MPMGrid.Nijk(MPMTransfer.Weights[p][2], kk);
								TVector<int32, 3> LocIndex = { ii, jj, kk };
								TVector<int32, 3> GlobIndex = MPMGrid.Loc2GlobIndex(MPMTransfer.Indices[p], LocIndex);
								int32 GlobIndexFlat = MPMGrid.FlatIndex(GlobIndex);
								//G2PHelper(Nii * Njj * Nkk, GridPositions[GlobIndexFlat], p, Particles);
								Particles.P(p) += Nii * Njj * Nkk * GridPositions[GlobIndexFlat];
							}
						}
					}
				}, Particles.Size() < 50);
		}

	protected:
		FSolverReal MaxDt;
		FSolverReal MinDt;
		FSolverReal CFLCoeff;
		TMPMGrid<FSolverReal> MPMGrid;
		TMPMTransfer<FSolverReal> MPMTransfer;
		TArray<TArray<int32>> ElementsPerColor;
		UE_DEPRECATED(5.6, "Use PreviousColoringPtr instead")
		TArray<TArray<int32>>* PreviousColoring = nullptr;
		TUniquePtr<TArray<TArray<int32>>> PreviousColoringPtr;
		TArray<TArray<TArray<int32>>> ElementsPerSubColors;
		TArray<FSolverReal> GridData;
		TArray<TVector<FSolverReal, 3>> GridPositions;
		bool InitialGridDataWritten = false;
	};

	template <typename T, typename ParticleType>
	using FXPBDGridBasedCorotatedConstraints UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FXPBDGridBasedCorotatedTetrahedralConstraints instead") = FXPBDGridBasedCorotatedTetrahedralConstraints;

}