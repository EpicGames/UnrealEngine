// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDLongRangeConstraintsBase.h"
#include "Chaos/PBDParticles.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "ChaosStats.h"

DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Long Range Constraint"), STAT_XPBD_LongRange, STATGROUP_Chaos);

namespace Chaos::Softs
{

class FXPBDLongRangeConstraints final : public FPBDLongRangeConstraintsBase
{
public:
	typedef FPBDLongRangeConstraintsBase Base;
	typedef typename Base::FTether FTether;

	static constexpr FSolverReal MinStiffness = (FSolverReal)1e-1;
	static constexpr FSolverReal MaxStiffness = (FSolverReal)1e7;

	FXPBDLongRangeConstraints(
		const FSolverParticles& Particles,
		const int32 InParticleOffset,
		const int32 InParticleCount,
		const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& InTethers,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		FSolverReal MeshScale)
		: FPBDLongRangeConstraintsBase(
			Particles,
			InParticleOffset,
			InParticleCount,
			InTethers,
			WeightMaps.FindRef(GetXPBDTetherStiffnessString(PropertyCollection, XPBDTetherStiffnessName.ToString())),
			WeightMaps.FindRef(GetXPBDTetherScaleString(PropertyCollection, XPBDTetherScaleName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDTetherStiffness(PropertyCollection, MaxStiffness)),
			FSolverVec2(GetWeightedFloatXPBDTetherScale(PropertyCollection, 1.f)),  // Scale clamping done in constructor
			MaxStiffness,
			MeshScale)
		, XPBDTetherStiffnessIndex(PropertyCollection)
		, XPBDTetherScaleIndex(PropertyCollection)
	{
		NumTethers = 0;
		for (const TConstArrayView<FTether>& TetherBatch : Tethers)
		{
			NumTethers += TetherBatch.Num();
		}
		Lambdas.Reserve(NumTethers);
	}

	FXPBDLongRangeConstraints(
		const FSolverParticles& Particles,
		const int32 InParticleOffset,
		const int32 InParticleCount,
		const TArray<TConstArrayView<TTuple<int32, int32, FRealSingle>>>& InTethers,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& ScaleMultipliers,
		const FSolverVec2& InStiffness = FSolverVec2::UnitVector,
		const FSolverVec2& InScale = FSolverVec2::UnitVector)
		: FPBDLongRangeConstraintsBase(
			Particles,
			InParticleOffset,
			InParticleCount,
			InTethers,
			StiffnessMultipliers,
			ScaleMultipliers,
			InStiffness,
			InScale,  // Scale clamping done in constructor
			MaxStiffness)
		, XPBDTetherStiffnessIndex(ForceInit)
		, XPBDTetherScaleIndex(ForceInit)
	{
		NumTethers = 0;
		for (const TConstArrayView<FTether>& TetherBatch : Tethers)
		{
			NumTethers += TetherBatch.Num();
		}
		Lambdas.Reserve(NumTethers);
	}

	virtual ~FXPBDLongRangeConstraints() override = default;

	void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		FSolverReal MeshScale)
	{
		if (IsXPBDTetherStiffnessMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue(GetWeightedFloatXPBDTetherStiffness(PropertyCollection));
			if (IsXPBDTetherStiffnessStringDirty(PropertyCollection))
			{
				const FString& WeightMapName = GetXPBDTetherStiffnessString(PropertyCollection);
				Stiffness = FPBDStiffness(
					WeightedValue,
					WeightMaps.FindRef(WeightMapName),
					ParticleCount,
					FPBDStiffness::DefaultTableSize,
					FPBDStiffness::DefaultParameterFitBase,
					MaxStiffness);
			}
			else
			{
				Stiffness.SetWeightedValue(WeightedValue, MaxStiffness);
			}
		}
		if (IsXPBDTetherScaleMutable(PropertyCollection))
		{
			const FSolverVec2 WeightedValue = FSolverVec2(GetWeightedFloatXPBDTetherScale(PropertyCollection)).ClampAxes(MinTetherScale, MaxTetherScale) * MeshScale;
			if (IsXPBDTetherScaleStringDirty(PropertyCollection))
			{
				const FString& WeightMapName = GetXPBDTetherScaleString(PropertyCollection);
				TetherScale = FPBDWeightMap(WeightedValue, WeightMaps.FindRef(WeightMapName), ParticleCount);
			}
			else
			{
				TetherScale.SetWeightedValue(WeightedValue);
			}
		}
	}

	// Set the stiffness and scale values used by the constraint
	void SetProperties(const FSolverVec2& InStiffness, const FSolverVec2& InTetherScale, FSolverReal MeshScale = (FSolverReal)1)
	{
		Stiffness.SetWeightedValue(InStiffness, MaxStiffness);
		TetherScale.SetWeightedValue(InTetherScale.ClampAxes(MinTetherScale, MaxTetherScale) * MeshScale);
	}

	// Set stiffness offset and range, as well as the simulation stiffness exponent
	void ApplyProperties(const FSolverReal /*Dt*/, const int32 /*NumIterations*/)
	{
		Stiffness.ApplyXPBDValues(MaxStiffness);
		TetherScale.ApplyValues();
	}

	void Init() const
	{
		Lambdas.Reset();
		Lambdas.AddZeroed(NumTethers);
	}

	void Apply(FSolverParticles& Particles, const FSolverReal Dt) const 
	{
		SCOPE_CYCLE_COUNTER(STAT_XPBD_LongRange);
		// Run particles in parallel, and ranges in sequence to avoid a race condition when updating the same particle from different tethers
		const int32 MinParallelSize = GetMinParallelBatchSize();

		if (Stiffness.HasWeightMap())
		{
			if (TetherScale.HasWeightMap())
			{
				int32 ConstraintOffset = 0;
				for (const TConstArrayView<FTether>& TetherBatch : Tethers)
				{
					PhysicsParallelFor(TetherBatch.Num(), [this, &Particles, Dt, &TetherBatch, ConstraintOffset](int32 Index)
						{
							const FTether& Tether = TetherBatch[Index];
							const int32 LocalParticleIndex = GetEndIndex(Tether);
							const FSolverReal Scale = TetherScale[LocalParticleIndex];
							const FSolverReal ExpStiffnessValue = Stiffness[LocalParticleIndex];
							Apply(Particles, Dt, Tether, ConstraintOffset + Index, ExpStiffnessValue, Scale);
						}, TetherBatch.Num() < MinParallelSize);
					ConstraintOffset += TetherBatch.Num();
				}
			}
			else
			{
				const FSolverReal ScaleValue = (FSolverReal)TetherScale;
				int32 ConstraintOffset = 0;
				for (const TConstArrayView<FTether>& TetherBatch : Tethers)
				{
					PhysicsParallelFor(TetherBatch.Num(), [this, &Particles, Dt, &TetherBatch, ScaleValue, ConstraintOffset](int32 Index)
						{
							const FTether& Tether = TetherBatch[Index];
							const int32 LocalParticleIndex = GetEndIndex(Tether);
							const FSolverReal ExpStiffnessValue = Stiffness[LocalParticleIndex];
							Apply(Particles, Dt, Tether, ConstraintOffset + Index, ExpStiffnessValue, ScaleValue);
						}, TetherBatch.Num() < MinParallelSize);
					ConstraintOffset += TetherBatch.Num();
				}
			}
		}
		else
		{
			const FSolverReal ExpStiffnessValue = (FSolverReal)Stiffness;

			if (TetherScale.HasWeightMap())
			{
				int32 ConstraintOffset = 0;
				for (const TConstArrayView<FTether>& TetherBatch : Tethers)
				{
					PhysicsParallelFor(TetherBatch.Num(), [this, &Particles, Dt, &TetherBatch, ExpStiffnessValue, ConstraintOffset](int32 Index)
						{
							const FTether& Tether = TetherBatch[Index];
							const int32 LocalParticleIndex = GetEndIndex(Tether);
							const FSolverReal Scale = TetherScale[LocalParticleIndex];
							Apply(Particles, Dt, Tether, ConstraintOffset + Index, ExpStiffnessValue, Scale);
						}, TetherBatch.Num() < MinParallelSize);
					ConstraintOffset += TetherBatch.Num();
				}
			}
			else
			{
				const FSolverReal ScaleValue = (FSolverReal)TetherScale;
				int32 ConstraintOffset = 0;
				for (const TConstArrayView<FTether>& TetherBatch : Tethers)
				{
					PhysicsParallelFor(TetherBatch.Num(), [this, &Particles, Dt, &TetherBatch, ExpStiffnessValue, ScaleValue, ConstraintOffset](int32 Index)
						{
							const FTether& Tether = TetherBatch[Index];
							Apply(Particles, Dt, Tether, ConstraintOffset + Index, ExpStiffnessValue, ScaleValue);
						}, TetherBatch.Num() < MinParallelSize);
					ConstraintOffset += TetherBatch.Num();
				}
			}
		}
	}

private:
	void Apply(FSolverParticles& Particles, const FSolverReal Dt, const FTether& Tether, int32 ConstraintIndex, const FSolverReal InStiffness, const FSolverReal InScale) const
	{
		if (InStiffness < MinStiffness)
		{
			return;
		}
		FSolverVec3 Direction;
		FSolverReal Offset;
		GetDelta(Particles, Tether, InScale, Direction, Offset);

		FSolverReal& Lambda = Lambdas[ConstraintIndex];
		const FSolverReal Alpha = (FSolverReal)1 / (InStiffness * Dt * Dt);

		const FSolverReal DLambda = (Offset - Alpha * Lambda) / ((FSolverReal)1. + Alpha);
		Particles.P(GetEndParticle(Tether)) += DLambda * Direction;
		Lambda += DLambda;
	}

	using Base::MinTetherScale;
	using Base::MaxTetherScale;
	using Base::Tethers;
	using Base::ParticleOffset;
	using Base::ParticleCount;
	using Base::Stiffness;
	using Base::TetherScale;

	mutable TArray<FSolverReal> Lambdas;
	int32 NumTethers;

	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDTetherStiffness, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDTetherScale, float);
};

}  // End namespace Chaos::Softs
