// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"

namespace Chaos::Softs
{
	template <typename T>
	class TXPBDPlanarConstraints
	{
	public:
		TXPBDPlanarConstraints() { LambdaArray.Init((T)0., 0); }

		virtual ~TXPBDPlanarConstraints() = default;

		void Apply(FSolverParticlesRange& Particles, const T Dt, const TArray<int32>& CollisionIndices, const TArray<TVec3<T>>& CollisionTargets, const TArray<TVec3<T>>& CollisionNormals) 
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDPlanarConstraintApply);
			PhysicsParallelFor(CollisionIndices.Num(), [this, &Particles, Dt, &CollisionIndices, &CollisionTargets, &CollisionNormals](const int32 ConstraintIndex)
				{
					ApplySingleConstraint(Particles, Dt, ConstraintIndex, CollisionIndices[ConstraintIndex], CollisionTargets[ConstraintIndex], CollisionNormals[ConstraintIndex]);
				});
		}

		void Init(const FSolverParticlesRange& InParticles, const T Dt, const int32 CollisionSize) { LambdaArray.Init((T)0., CollisionSize); }

		void SetTolerance(const T TolIn) { Tol = TolIn; }

		void SetStiffness(const T StiffnessIn) { Stiffness = StiffnessIn; }

	private:

		void ApplySingleConstraint(FSolverParticlesRange& Particles, const T Dt, const int32 ConstraintIndex, const int32 ParticleIndex, const TVec3<T>& CollisionTarget, const TVec3<T>& CollisionNormal) 
		{
			const TVec3<T> Diff = Particles.P(ParticleIndex) - CollisionTarget;
			const T DiffDotNormal = Chaos::TVec3<T>::DotProduct(Diff, CollisionNormal);
			if (DiffDotNormal < Tol && Particles.InvM(ParticleIndex) != T(0.) && Stiffness>.0)
			{
				T Constraint = Tol - DiffDotNormal;
				T AlphaTilde = T(1) / (Dt * Dt * Stiffness);
				if (Stiffness > StiffnessThreshold)
				{
					AlphaTilde = T(0);
				}
				T DeltaLambda = (-Constraint - AlphaTilde * LambdaArray[ConstraintIndex]) / (Particles.InvM(ParticleIndex) + AlphaTilde);
				LambdaArray[ConstraintIndex] += DeltaLambda;
				Particles.P(ParticleIndex) += (-DeltaLambda) * Particles.InvM(ParticleIndex) * CollisionNormal;
			}
		}

	protected:
		T Tol = T(.1);
		T Stiffness = 1e10;
		T StiffnessThreshold = 1e9;
		TArray<T> LambdaArray;
	};
}  // End namespace Chaos::Softs
