// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"
#include "ChaosStats.h"
#include "PBDWeightMap.h"
#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/SoftsSpring.h"

namespace Chaos::Softs
{

class FXPBDSpringConstraints : public FPBDSpringConstraintsBase
{
	typedef FPBDSpringConstraintsBase Base;

public:
	static constexpr FSolverReal MinStiffness = (FSolverReal)0; // We're not checking against MinStiffness (except when it's constant and == 0)
	static constexpr FSolverReal MaxStiffness = (FSolverReal)1e9;
	static constexpr FSolverReal MinDampingRatio = (FSolverReal)0.;
	static constexpr FSolverReal MaxDampingRatio = (FSolverReal)1000.;

	template<int32 Valence UE_REQUIRES(Valence >= 2 && Valence <= 4)>
	FXPBDSpringConstraints(
		const FSolverParticlesRange& Particles,
		const TArray<TVector<int32, Valence>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const FSolverVec2& InStiffness,
		const FSolverVec2& InDampingRatio)
		: Base(
			Particles,
			InConstraints,
			StiffnessMultipliers,
			InStiffness,
			true /*bTrimKinematicConstraints*/,
			MaxStiffness)
		, DampingRatio(
			InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio),
			DampingMultipliers,
			TConstArrayView<TVec2<int32>>(Constraints),
			ParticleOffset,
			ParticleCount)
	{
		Lambdas.Init((FSolverReal)0., Constraints.Num());
		InitColor(Particles);
	}

	template<int32 Valence UE_REQUIRES(Valence >= 2 && Valence <= 4)>
	UE_DEPRECATED(5.4, "XPBD Constraints must always trim kinematic constraints")
	FXPBDSpringConstraints(
		const FSolverParticlesRange& Particles,
		const TArray<TVector<int32, Valence>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const FSolverVec2& InStiffness,
		const FSolverVec2& InDampingRatio,
		bool bTrimKinematicConstraints)
		: Base(
			Particles,
			InConstraints,
			StiffnessMultipliers,
			InStiffness,
			true /*bTrimKinematicConstraints*/,
			MaxStiffness)
		, DampingRatio(
			InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio),
			DampingMultipliers,
			TConstArrayView<TVec2<int32>>(Constraints),
			ParticleOffset,
			ParticleCount)
	{
		Lambdas.Init((FSolverReal)0., Constraints.Num());
		InitColor(Particles);
	}

	template<int32 Valence UE_REQUIRES(Valence >= 2 && Valence <= 4)>
	FXPBDSpringConstraints(
		const FSolverParticles& Particles,
		int32 ParticleOffset,
		int32 ParticleCount,
		const TArray<TVector<int32, Valence>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const FSolverVec2& InStiffness,
		const FSolverVec2& InDampingRatio)
		: Base(
			Particles,
			ParticleOffset,
			ParticleCount,
			InConstraints,
			StiffnessMultipliers,
			InStiffness,
			true /*bTrimKinematicConstraints*/,
			MaxStiffness)
		, DampingRatio(
			InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio),
			DampingMultipliers,
			TConstArrayView<TVec2<int32>>(Constraints),
			ParticleOffset,
			ParticleCount)
	{
		Lambdas.Reset();
		Lambdas.SetNumZeroed(Constraints.Num());
		LambdasDamping.Reset();
		LambdasDamping.SetNumZeroed(Constraints.Num());
		InitColor(Particles);
	}

	template<int32 Valence UE_REQUIRES(Valence >= 2 && Valence <= 4)>
	UE_DEPRECATED(5.4, "XPBD Constraints must always trim kinematic constraints")
	FXPBDSpringConstraints(
		const FSolverParticles& Particles,
		int32 ParticleOffset,
		int32 ParticleCount,
		const TArray<TVector<int32, Valence>>& InConstraints,
		const TConstArrayView<FRealSingle>& StiffnessMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const FSolverVec2& InStiffness,
		const FSolverVec2& InDampingRatio,
		bool bTrimKinematicConstraints)
	: Base(
		Particles,
		ParticleOffset,
		ParticleCount,
		InConstraints,
		StiffnessMultipliers,
		InStiffness,
		true /*bTrimKinematicConstraints*/,
		MaxStiffness)
	, DampingRatio(
		InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio),
		DampingMultipliers,
		TConstArrayView<TVec2<int32>>(Constraints),
		ParticleOffset,
		ParticleCount)
	{
		Lambdas.Reset();
		Lambdas.SetNumZeroed(Constraints.Num());
		LambdasDamping.Reset();
		LambdasDamping.SetNumZeroed(Constraints.Num());
		InitColor(Particles);
	}

	virtual ~FXPBDSpringConstraints() override = default;

	void Init() const 
	{
		Lambdas.Reset();
		Lambdas.SetNumZeroed(Constraints.Num());
		LambdasDamping.Reset();
		LambdasDamping.SetNumZeroed(Constraints.Num());
	}

	// Update stiffness values
	void SetProperties(const FSolverVec2& InStiffness, const FSolverVec2& InDampingRatio = FSolverVec2::ZeroVector)
	{ 
		Stiffness.SetWeightedValue(InStiffness, MaxStiffness);
		DampingRatio.SetWeightedValue(InDampingRatio.ClampAxes(MinDampingRatio, MaxDampingRatio));
	}
	
	// Update stiffness table, as well as the simulation stiffness exponent
	void ApplyProperties(const FSolverReal Dt, const int32 NumIterations)
	{
		Stiffness.ApplyXPBDValues(MaxStiffness);
		DampingRatio.ApplyValues();
	}

	template<typename SolverParticlesOrRange>
	CHAOS_API void Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const;

	CHAOS_API void UpdateLinearSystem(const FSolverParticlesRange& Particles, const FSolverReal Dt, FEvolutionLinearSystem& LinearSystem) const;

	const TArray<int32>& GetConstraintsPerColorStartIndex() const { return ConstraintsPerColorStartIndex; }

private:
	template<typename SolverParticlesOrRange>
	CHAOS_API void InitColor(const SolverParticlesOrRange& InParticles);

	template<bool bDampingBefore, bool bSingleLambda, bool bSeparateStretch, bool bDampingAfter, typename SolverParticlesOrRange >
	void ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal StiffnessValue, const FSolverReal DampingRatioValue) const; 

protected:
	using Base::Constraints;
	using Base::ParticleOffset;
	using Base::ParticleCount;
	using Base::Stiffness;
	FPBDWeightMap DampingRatio;

private:
	using Base::Dists;
	mutable TArray<FSolverReal> Lambdas;
	mutable TArray<FSolverReal> LambdasDamping;
	TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.
};

class FXPBDEdgeSpringConstraints final : public FXPBDSpringConstraints
{
public:
	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsXPBDEdgeSpringStiffnessEnabled(PropertyCollection, false);
	}

	FXPBDEdgeSpringConstraints(
		const FSolverParticlesRange& Particles,
		const TArray<TVec3<int32>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection)
		: FXPBDSpringConstraints(
			Particles,
			InConstraints,
			WeightMaps.FindRef(GetXPBDEdgeSpringStiffnessString(PropertyCollection, XPBDEdgeSpringStiffnessName.ToString())),
			WeightMaps.FindRef(GetXPBDEdgeSpringDampingString(PropertyCollection, XPBDEdgeSpringDampingName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDEdgeSpringStiffness(PropertyCollection, MaxStiffness)),
			FSolverVec2(GetWeightedFloatXPBDEdgeSpringDamping(PropertyCollection, MinDampingRatio)))
		, XPBDEdgeSpringStiffnessIndex(PropertyCollection)
		, XPBDEdgeSpringDampingIndex(PropertyCollection)
	{}

	UE_DEPRECATED(5.4, "XPBD Constraints must always trim kinematic constraints")
	FXPBDEdgeSpringConstraints(
		const FSolverParticlesRange & Particles,
		const TArray<TVec3<int32>>&InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>&WeightMaps,
		const FCollectionPropertyConstFacade & PropertyCollection,
		bool bTrimKinematicConstraints)
		: FXPBDSpringConstraints(
			Particles,
			InConstraints,
			WeightMaps.FindRef(GetXPBDEdgeSpringStiffnessString(PropertyCollection, XPBDEdgeSpringStiffnessName.ToString())),
			WeightMaps.FindRef(GetXPBDEdgeSpringDampingString(PropertyCollection, XPBDEdgeSpringDampingName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDEdgeSpringStiffness(PropertyCollection, MaxStiffness)),
			FSolverVec2(GetWeightedFloatXPBDEdgeSpringDamping(PropertyCollection, MinDampingRatio)))
		, XPBDEdgeSpringStiffnessIndex(PropertyCollection)
		, XPBDEdgeSpringDampingIndex(PropertyCollection)
	{}

	FXPBDEdgeSpringConstraints(
		const FSolverParticles& Particles,
		int32 ParticleOffset,
		int32 ParticleCount,
		const TArray<TVec3<int32>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection)
		: FXPBDSpringConstraints(
			Particles,
			ParticleOffset,
			ParticleCount,
			InConstraints,
			WeightMaps.FindRef(GetXPBDEdgeSpringStiffnessString(PropertyCollection, XPBDEdgeSpringStiffnessName.ToString())),
			WeightMaps.FindRef(GetXPBDEdgeSpringDampingString(PropertyCollection, XPBDEdgeSpringDampingName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDEdgeSpringStiffness(PropertyCollection, MaxStiffness)),
			FSolverVec2(GetWeightedFloatXPBDEdgeSpringDamping(PropertyCollection, MinDampingRatio)))
		, XPBDEdgeSpringStiffnessIndex(PropertyCollection)
		, XPBDEdgeSpringDampingIndex(PropertyCollection)
	{}

	UE_DEPRECATED(5.4, "XPBD Constraints must always trim kinematic constraints")
	FXPBDEdgeSpringConstraints(
		const FSolverParticles & Particles,
		int32 ParticleOffset,
		int32 ParticleCount,
		const TArray<TVec3<int32>>&InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>&WeightMaps,
		const FCollectionPropertyConstFacade & PropertyCollection,
		bool bTrimKinematicConstraints)
		: FXPBDSpringConstraints(
			Particles,
			ParticleOffset,
			ParticleCount,
			InConstraints,
			WeightMaps.FindRef(GetXPBDEdgeSpringStiffnessString(PropertyCollection, XPBDEdgeSpringStiffnessName.ToString())),
			WeightMaps.FindRef(GetXPBDEdgeSpringDampingString(PropertyCollection, XPBDEdgeSpringDampingName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDEdgeSpringStiffness(PropertyCollection, MaxStiffness)),
			FSolverVec2(GetWeightedFloatXPBDEdgeSpringDamping(PropertyCollection, MinDampingRatio)))
		, XPBDEdgeSpringStiffnessIndex(PropertyCollection)
		, XPBDEdgeSpringDampingIndex(PropertyCollection)
	{}

	virtual ~FXPBDEdgeSpringConstraints() override = default;

	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);

private:
	using FXPBDSpringConstraints::Constraints;
	using FXPBDSpringConstraints::ParticleOffset;
	using FXPBDSpringConstraints::ParticleCount;
	using FXPBDSpringConstraints::Stiffness;
	using FXPBDSpringConstraints::DampingRatio;

	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDEdgeSpringStiffness, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDEdgeSpringDamping, float);
};

class FXPBDBendingSpringConstraints : public FXPBDSpringConstraints
{
public:
	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsXPBDBendingSpringStiffnessEnabled(PropertyCollection, false);
	}

	FXPBDBendingSpringConstraints(
		const FSolverParticlesRange& Particles,
		const TArray<TVec2<int32>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection)
		: FXPBDSpringConstraints(
			Particles,
			InConstraints,
			WeightMaps.FindRef(GetXPBDBendingSpringStiffnessString(PropertyCollection, XPBDBendingSpringStiffnessName.ToString())),
			WeightMaps.FindRef(GetXPBDBendingSpringDampingString(PropertyCollection, XPBDBendingSpringDampingName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDBendingSpringStiffness(PropertyCollection, MaxStiffness)),
			FSolverVec2(GetWeightedFloatXPBDBendingSpringDamping(PropertyCollection, MinDampingRatio)))
		, XPBDBendingSpringStiffnessIndex(PropertyCollection)
		, XPBDBendingSpringDampingIndex(PropertyCollection)
	{}

	UE_DEPRECATED(5.4, "XPBD Constraints must always trim kinematic constraints")
	FXPBDBendingSpringConstraints(
		const FSolverParticlesRange & Particles,
		const TArray<TVec2<int32>>&InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>&WeightMaps,
		const FCollectionPropertyConstFacade & PropertyCollection,
		bool bTrimKinematicConstraints)
		: FXPBDSpringConstraints(
			Particles,
			InConstraints,
			WeightMaps.FindRef(GetXPBDBendingSpringStiffnessString(PropertyCollection, XPBDBendingSpringStiffnessName.ToString())),
			WeightMaps.FindRef(GetXPBDBendingSpringDampingString(PropertyCollection, XPBDBendingSpringDampingName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDBendingSpringStiffness(PropertyCollection, MaxStiffness)),
			FSolverVec2(GetWeightedFloatXPBDBendingSpringDamping(PropertyCollection, MinDampingRatio)))
		, XPBDBendingSpringStiffnessIndex(PropertyCollection)
		, XPBDBendingSpringDampingIndex(PropertyCollection)
	{}

	FXPBDBendingSpringConstraints(
		const FSolverParticles & Particles,
		int32 ParticleOffset,
		int32 ParticleCount,
		const TArray<TVec2<int32>>&InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>&WeightMaps,
		const FCollectionPropertyConstFacade & PropertyCollection)
		: FXPBDSpringConstraints(
			Particles,
			ParticleOffset,
			ParticleCount,
			InConstraints,
			WeightMaps.FindRef(GetXPBDBendingSpringStiffnessString(PropertyCollection, XPBDBendingSpringStiffnessName.ToString())),
			WeightMaps.FindRef(GetXPBDBendingSpringDampingString(PropertyCollection, XPBDBendingSpringDampingName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDBendingSpringStiffness(PropertyCollection, MaxStiffness)),
			FSolverVec2(GetWeightedFloatXPBDBendingSpringDamping(PropertyCollection, MinDampingRatio)))
		, XPBDBendingSpringStiffnessIndex(PropertyCollection)
		, XPBDBendingSpringDampingIndex(PropertyCollection)
	{}

	UE_DEPRECATED(5.4, "XPBD Constraints must always trim kinematic constraints")
	FXPBDBendingSpringConstraints(
		const FSolverParticles& Particles,
		int32 ParticleOffset,
		int32 ParticleCount,
		const TArray<TVec2<int32>>& InConstraints,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints)
		: FXPBDSpringConstraints(
			Particles,
			ParticleOffset,
			ParticleCount,
			InConstraints,
			WeightMaps.FindRef(GetXPBDBendingSpringStiffnessString(PropertyCollection, XPBDBendingSpringStiffnessName.ToString())),
			WeightMaps.FindRef(GetXPBDBendingSpringDampingString(PropertyCollection, XPBDBendingSpringDampingName.ToString())),
			FSolverVec2(GetWeightedFloatXPBDBendingSpringStiffness(PropertyCollection, MaxStiffness)),
			FSolverVec2(GetWeightedFloatXPBDBendingSpringDamping(PropertyCollection, MinDampingRatio)))
		, XPBDBendingSpringStiffnessIndex(PropertyCollection)
		, XPBDBendingSpringDampingIndex(PropertyCollection)
	{}

	virtual ~FXPBDBendingSpringConstraints() override = default;

	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);

private:
	using FXPBDSpringConstraints::Constraints;
	using FXPBDSpringConstraints::ParticleOffset;
	using FXPBDSpringConstraints::ParticleCount;
	using FXPBDSpringConstraints::Stiffness;
	using FXPBDSpringConstraints::DampingRatio;

	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDBendingSpringStiffness, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDBendingSpringDamping, float);
};

}  // End namespace Chaos::Softs

#if !defined(CHAOS_XPBD_SPRING_ISPC_ENABLED_DEFAULT)
#define CHAOS_XPBD_SPRING_ISPC_ENABLED_DEFAULT 1
#endif

// Support ISPC enable/disable in non-shipping builds
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bChaos_XPBDSpring_ISPC_Enabled = INTEL_ISPC && CHAOS_XPBD_SPRING_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_XPBDSpring_ISPC_Enabled;
#endif
