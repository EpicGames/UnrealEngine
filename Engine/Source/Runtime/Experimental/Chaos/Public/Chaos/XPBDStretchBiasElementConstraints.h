// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/CollectionPropertyFacade.h"
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Chaos/PBDSoftsSolverParticles.h"
#include "Chaos/SoftsSolverParticlesRange.h"
#include "Chaos/PBDStiffness.h"
#include "Chaos/ParticleRule.h"
#include "ChaosStats.h"

namespace Chaos
{
class FTriangleMesh;
}

namespace Chaos::Softs
{

class FXPBDStretchBiasElementConstraints
{
public:
	// Stiffness is in kg cm / s^2 for stretch, kg cm^2 / s^2 for Bias
	static constexpr FSolverReal MinStiffness = (FSolverReal)1e-4; // Stiffness below this will be considered 0 since all of our calculations are actually based on 1 / stiffness.
	static constexpr FSolverReal MaxStiffness = (FSolverReal)1e7;
	static constexpr FSolverReal MinDamping = (FSolverReal)0.;
	static constexpr FSolverReal MaxDamping = (FSolverReal)1000.;
	static constexpr bool bDefaultUse3dRestLengths = true;
	static constexpr FSolverReal MinWarpWeftScale = (FSolverReal)0.; 
	static constexpr FSolverReal MaxWarpWeftScale = (FSolverReal)1e7; // No particular reason for this number. Just can't imagine wanting something bigger?
	static constexpr FSolverReal DefaultWarpWeftScale = (FSolverReal)1.;

	static bool IsEnabled(const FCollectionPropertyConstFacade& PropertyCollection)
	{
		return IsXPBDAnisoStretchStiffnessWarpEnabled(PropertyCollection, false);
	}

	CHAOS_API FXPBDStretchBiasElementConstraints(const FSolverParticlesRange& InParticles,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexUVs,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints = false);

	CHAOS_API FXPBDStretchBiasElementConstraints(const FSolverParticles& InParticles,
		int32 ParticleOffset,
		int32 ParticleCount,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexUVs,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps,
		const FCollectionPropertyConstFacade& PropertyCollection,
		bool bTrimKinematicConstraints = false);

	CHAOS_API FXPBDStretchBiasElementConstraints(const FSolverParticles& InParticles,
		int32 ParticleOffset,
		int32 ParticleCount,
		const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexUVs,
		const TConstArrayView<FRealSingle>& StiffnessWarpMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessWeftMultipliers,
		const TConstArrayView<FRealSingle>& StiffnessBiasMultipliers,
		const TConstArrayView<FRealSingle>& DampingMultipliers,
		const TConstArrayView<FRealSingle>& WarpScaleMultipliers,
		const TConstArrayView<FRealSingle>& WeftScaleMultipliers,
		const FSolverVec2& InStiffnessWarp,
		const FSolverVec2& InStiffnessWeft,
		const FSolverVec2& InStiffnessBias,
		const FSolverVec2& InDampingRatio,
		const FSolverVec2& InWarpScale,
		const FSolverVec2& InWeftScale,
		bool bUse3dRestLengths,
		bool bTrimKinematicConstraints = false);

	virtual ~FXPBDStretchBiasElementConstraints() = default;

	void Init()
	{ 
		Lambdas.Reset();
		Lambdas.AddZeroed(Constraints.Num());
	}

	CHAOS_API void SetProperties(
		const FCollectionPropertyConstFacade& PropertyCollection,
		const TMap<FString, TConstArrayView<FRealSingle>>& WeightMaps);

	void SetProperties(const FSolverVec2& InStiffnessWarp, const FSolverVec2& InStiffnessWeft, const FSolverVec2& InStiffnessBias, const FSolverVec2& InDampingRatio, const FSolverVec2& InWarpScale, const FSolverVec2& InWeftScale)
	{
		StiffnessWarp.SetWeightedValue(InStiffnessWarp, MaxStiffness);
		StiffnessWeft.SetWeightedValue(InStiffnessWeft, MaxStiffness);
		StiffnessBias.SetWeightedValue(InStiffnessBias, MaxStiffness);
		DampingRatio.SetWeightedValue(InDampingRatio.ClampAxes(MinDamping, MaxDamping));
		WarpScale.SetWeightedValue(InWarpScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale));
		WeftScale.SetWeightedValue(InWeftScale.ClampAxes(MinWarpWeftScale, MaxWarpWeftScale));
	}

	// Update stiffness table, as well as the simulation stiffness exponent
	void ApplyProperties(const FSolverReal /*Dt*/, const int32 /*NumIterations*/)
	{
		StiffnessWarp.ApplyXPBDValues(MaxStiffness);
		StiffnessWeft.ApplyXPBDValues(MaxStiffness);
		StiffnessBias.ApplyXPBDValues(MaxStiffness);
		DampingRatio.ApplyValues();
		WarpScale.ApplyValues();
		WeftScale.ApplyValues();
	}

	template<typename SolverParticlesOrRange>
	CHAOS_API void Apply(SolverParticlesOrRange& Particles, const FSolverReal Dt) const;
	CHAOS_API void CalculateUVStretch(const int32 ConstraintIndex, const FSolverVec3& P0, const FSolverVec3& P1, const FSolverVec3& P2, FSolverVec3& DXDu, FSolverVec3& DXDv) const;
	
	const TArray<TVec3<int32>>& GetConstraints() const { return Constraints; }

	TArray<TArray<int32>> GetConstraintsArray() const //TODO: avoid converting
	{
		TArray<TArray<int32>> ConstraintsArray;
		ConstraintsArray.SetNum(Constraints.Num());
		for (int32 ConstraintIdx = 0; ConstraintIdx < Constraints.Num(); ConstraintIdx++)
		{
			ConstraintsArray[ConstraintIdx].SetNum(3);
			for (int32 TriLocalIdx = 0; TriLocalIdx < 3; TriLocalIdx++)
			{
				ConstraintsArray[ConstraintIdx][TriLocalIdx] = Constraints[ConstraintIdx][TriLocalIdx];
			}
		}
		return ConstraintsArray;
	}
	const TArray<FSolverVec3> GetRestStretchLengths() const { return RestStretchLengths; }
	FSolverVec2 GetWarpWeftScale(const int32 ConstraintIndex) const 
	{ 
		return FSolverVec2(WarpScale.HasWeightMap() ? WarpScale[ConstraintIndex] : (FSolverReal)WarpScale,
			WeftScale.HasWeightMap() ? WeftScale[ConstraintIndex] : (FSolverReal)WeftScale);
	}
	const TArray<int32>& GetConstraintsPerColorStartIndex() const { return ConstraintsPerColorStartIndex; }

	CHAOS_API void AddStretchBiasElementResidualAndHessian(const FSolverParticlesRange& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian);
	CHAOS_API void AddStretchBiasElementResidualAndHessian(const FSolverParticles& Particles, const int32 ConstraintIndex, const int32 ConstraintIndexLocal, const FSolverReal Dt, TVec3<FSolverReal>& ParticleResidual, Chaos::PMatrix<FSolverReal, 3, 3>& ParticleHessian);

	CHAOS_API void InitializeDmInvAndMeasures(const FSolverParticlesRange& Particles);

	CHAOS_API void InitializeDmInvAndMeasures(const FSolverParticles& Particles);

	CHAOS_API void AddInternalForceDifferential(const FSolverParticlesRange& InParticles, const TArray<TVector<FSolverReal, 3>>& DeltaParticles, TArray<TVector<FSolverReal, 3>>& Ndf);
	CHAOS_API void AddInternalForceDifferential(const FSolverParticles& InParticles, const TArray<TVector<FSolverReal, 3>>& DeltaParticles, TArray<TVector<FSolverReal, 3>>& Ndf);

private:
	template<typename SolverParticlesOrRange>
	void InitConstraintsAndRestData(const SolverParticlesOrRange& InParticles, const FTriangleMesh& TriangleMesh,
		const TArray<TVec3<FVec2f>>& FaceVertexUVs, const bool bUse3dRestLengths, const bool bTrimKinematicConstraints);
	template<typename SolverParticlesOrRange>
	void InitColor(const SolverParticlesOrRange& InParticles);
	template<typename SolverParticlesOrRange>
	void ApplyHelper(SolverParticlesOrRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverVec3& ExpStiffnessValue, const FSolverReal DampingRatioValue, const FSolverReal WarpScaleValue, const FSolverReal WeftScaleValue) const;

	TArray<TVec3<int32>> Constraints;
	const int32 ParticleOffset;
	const int32 ParticleCount;
	FPBDStiffness StiffnessWarp;
	FPBDStiffness StiffnessWeft;
	FPBDStiffness StiffnessBias;
	FPBDWeightMap DampingRatio;
	FPBDWeightMap WarpScale;
	FPBDWeightMap WeftScale;
	mutable TArray<FSolverVec3> Lambdas; // separate for stretchU, stretchV, Bias
	TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.

	TArray<FSolverMatrix22> DeltaUVInverse; // Used to convert from DeltaX to dX/dU and dX/dV
	TArray<FSolverVec3> RestStretchLengths;
	TArray<FSolverVec3> StiffnessScales; // Used to make everything resolution independent.

	TArray<FSolverReal> Measure;
	TArray<FSolverMatrix22> DmInverse;
	TArray<FSolverMatrix22> DmArray;
	bool bDmInitialized = false;

	TArray<PMatrix<FSolverReal, 3, 2>> RestDmArray;

	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoStretchUse3dRestLengths, bool);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoStretchStiffnessWarp, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoStretchStiffnessWeft, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoStretchStiffnessBias, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoStretchDamping, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoStretchWarpScale, float);
	UE_CHAOS_DECLARE_PROPERTYCOLLECTION_NAME(XPBDAnisoStretchWeftScale, float);
};

}  // End namespace Chaos::Softs

#if !defined(CHAOS_XPBD_STRETCH_BIAS_ELEMENT_ISPC_ENABLED_DEFAULT)
#define CHAOS_XPBD_STRETCH_BIAS_ELEMENT_ISPC_ENABLED_DEFAULT 1
#endif

// Support ISPC enable/disable in non-shipping builds
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bChaos_XPBDStretchBiasElement_ISPC_Enabled = INTEL_ISPC && CHAOS_XPBD_STRETCH_BIAS_ELEMENT_ISPC_ENABLED_DEFAULT;
#else
extern CHAOS_API bool bChaos_XPBDStretchBiasElement_ISPC_Enabled;
#endif
