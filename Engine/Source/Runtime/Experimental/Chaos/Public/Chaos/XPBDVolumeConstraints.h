// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "Chaos/PBDSpringConstraintsBase.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/Framework/Parallel.h"


namespace Chaos::Softs
{

	class FXPBDVolumeConstraints
	{

	public:
		//this one only accepts tetmesh input and mesh
		FXPBDVolumeConstraints(
			const Chaos::Softs::FSolverParticles& InParticles,
			int32 ParticleOffset,
			int32 ParticleCount,
			const TArray<TVector<int32, 4>>& InMesh,
			const FSolverReal InStiffness
		)
			: MeshConstraints(InMesh), Stiffness(InStiffness)
		{

			LambdaArray.Init((FSolverReal)0., MeshConstraints.Num());
			Volumes.Init((FSolverReal)0., MeshConstraints.Num());
			for (int32 ElementIndex = 0; ElementIndex < InMesh.Num(); ElementIndex++)
			{
				TVec4<int32> Constraint = InMesh[ElementIndex];
				const FSolverVec3& P1 = InParticles.GetX(Constraint[0]);
				const FSolverVec3& P2 = InParticles.GetX(Constraint[1]);
				const FSolverVec3& P3 = InParticles.GetX(Constraint[2]);
				const FSolverVec3& P4 = InParticles.GetX(Constraint[3]);
				Volumes[ElementIndex] = FSolverVec3::DotProduct(FSolverVec3::CrossProduct(P2 - P1, P3 - P1), P4 - P1) / (FSolverReal)6.;
			}

			InitColor(InParticles);
		}

		virtual ~FXPBDVolumeConstraints() = default;


		void Init() const
		{
			for (FSolverReal& Lambdas : LambdaArray) { Lambdas = (FSolverReal)0.; }
		}

		virtual void ApplyOneElement(FSolverParticles& Particles, const FSolverReal Dt, const int32 ElementIndex) const
		{
			//TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDCorotatedApplySingle);
			
			TVec4<FSolverVec3> VolumeDelta = GetVolumeDelta(Particles, Dt, ElementIndex);
			for (int i = 0; i < 4; i++)
			{
				Particles.P(MeshConstraints[ElementIndex][i]) += VolumeDelta[i];
			}
			

		}

		void ApplyInSerial(FSolverParticles& Particles, const FSolverReal Dt) const
		{
			//SCOPE_CYCLE_COUNTER(STAT_ChaosXPBDCorotated);
			TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDVolumeApplySerial);
			for (int32 ElementIndex = 0; ElementIndex < MeshConstraints.Num(); ++ElementIndex)
			{
				ApplyOneElement(Particles, Dt, ElementIndex);
			}


		}

		void ApplyInParallel(FSolverParticles& Particles, const FSolverReal Dt) const
		{
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(STAT_ChaosXPBDVolumeApply);
				if ((ConstraintsPerColorStartIndex.Num() > 1))//&& (MeshConstraints.Num() > Chaos_Spring_ParallelConstraintCount))
				{
					const int32 ConstraintColorNum = ConstraintsPerColorStartIndex.Num() - 1;

					for (int32 ConstraintColorIndex = 0; ConstraintColorIndex < ConstraintColorNum; ++ConstraintColorIndex)
					{
						const int32 ColorStart = ConstraintsPerColorStartIndex[ConstraintColorIndex];
						const int32 ColorSize = ConstraintsPerColorStartIndex[ConstraintColorIndex + 1] - ColorStart;
						PhysicsParallelFor(ColorSize, [&](const int32 Index)
							{
								const int32 ConstraintIndex = ColorStart + Index;
								ApplyOneElement(Particles, Dt, ConstraintIndex);
							});
					}
				}
			}
		}

	private:

		void InitColor(const FSolverParticles& Particles)
		{

			const TArray<TArray<int32>> ConstraintsPerColor = FGraphColoring::ComputeGraphColoring(MeshConstraints, Particles);

			// Reorder constraints based on color so each array in ConstraintsPerColor contains contiguous elements.
			TArray<TVec4<int32>> ReorderedConstraints;
			TArray<FSolverReal> ReorderedRestLength;
			TArray<FSolverReal> ReorderedVolumes;
			TArray<int32> OrigToReorderedIndices; // used to reorder stiffness indices
			ReorderedConstraints.SetNumUninitialized(MeshConstraints.Num());
			ReorderedVolumes.SetNumUninitialized(Volumes.Num());
			OrigToReorderedIndices.SetNumUninitialized(MeshConstraints.Num());

			ConstraintsPerColorStartIndex.Reset(ConstraintsPerColor.Num() + 1);

			int32 ReorderedIndex = 0;
			for (const TArray<int32>& ConstraintsBatch : ConstraintsPerColor)
			{
				ConstraintsPerColorStartIndex.Add(ReorderedIndex);
				for (const int32& BatchConstraint : ConstraintsBatch)
				{
					const int32 OrigIndex = BatchConstraint;
					ReorderedConstraints[ReorderedIndex] = MeshConstraints[OrigIndex];
					//ReorderedMeasure[ReorderedIndex] = Measure[OrigIndex];
					ReorderedVolumes[ReorderedIndex] = Volumes[OrigIndex];
					OrigToReorderedIndices[OrigIndex] = ReorderedIndex;

					++ReorderedIndex;
				}
			}
			ConstraintsPerColorStartIndex.Add(ReorderedIndex);

			MeshConstraints = MoveTemp(ReorderedConstraints);
			Volumes = MoveTemp(ReorderedVolumes);
		}

	protected:


		TVec4<FSolverVec3> GetVolumeDelta(const FSolverParticles& Particles, const FSolverReal Dt, const int32 ElementIndex) const
		{
			TVec4<FSolverVec3> Grads;
			const TVec4<int32>& Constraint = MeshConstraints[ElementIndex];
			const FSolverVec3& P1 = Particles.P(Constraint[0]);
			const FSolverVec3& P2 = Particles.P(Constraint[1]);
			const FSolverVec3& P3 = Particles.P(Constraint[2]);
			const FSolverVec3& P4 = Particles.P(Constraint[3]);
			const FSolverVec3 P2P1 = P2 - P1;
			const FSolverVec3 P4P1 = P4 - P1;
			const FSolverVec3 P3P1 = P3 - P1;
			Grads[1] = FSolverVec3::CrossProduct(P3P1, P4P1) / (FSolverReal)6.;
			Grads[2] = FSolverVec3::CrossProduct(P4P1, P2P1) / (FSolverReal)6.;
			Grads[3] = FSolverVec3::CrossProduct(P2P1, P3P1) / (FSolverReal)6.;
			Grads[0] = -(Grads[1] + Grads[2] + Grads[3]);
			const int32 i1 = Constraint[0];
			const int32 i2 = Constraint[1];
			const int32 i3 = Constraint[2];
			const int32 i4 = Constraint[3];

			const FSolverReal Volume = FSolverVec3::DotProduct(FSolverVec3::CrossProduct(P2 - P1, P3 - P1), P4 - P1) / (FSolverReal)6.;

			FSolverReal AlphaTilde = (FSolverReal)1. / (Stiffness * Dt * Dt);

			FSolverReal DLambda = Volumes[ElementIndex] - Volume - AlphaTilde * LambdaArray[ElementIndex];

			FSolverReal Denom = AlphaTilde;
			for (int i = 0; i < 4; i++)
			{
				for (int j = 0; j < 3; j++)
				{
					Denom += Grads[i][j] * Particles.InvM(MeshConstraints[ElementIndex][i]) * Grads[i][j];
				}
			}
			DLambda /= Denom;
			LambdaArray[ElementIndex] += DLambda;
			TVec4<FSolverVec3> Delta(FSolverVec3((FSolverReal)0.));
			for (int i = 0; i < 4; i++)
			{
				for (int j = 0; j < 3; j++)
				{
					Delta[i][j] = Particles.InvM(MeshConstraints[ElementIndex][i]) * Grads[i][j] * DLambda;
				}
			}
			return Delta;
		}



	protected:
		mutable TArray<FSolverReal> LambdaArray;

		//material constants calculated from E:

		TArray<TVector<int32, 4>> MeshConstraints;
		TArray<FSolverReal> Volumes;
		FSolverReal Stiffness;
		TArray<int32> ConstraintsPerColorStartIndex; // Constraints are ordered so each batch is contiguous. This is ColorNum + 1 length so it can be used as start and end.
	};

}  // End namespace Chaos::Softs

#pragma once
#pragma once
