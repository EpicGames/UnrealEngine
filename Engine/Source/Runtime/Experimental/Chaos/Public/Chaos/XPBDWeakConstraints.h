// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/PBDSpringConstraintsBase.h"
#include "ChaosStats.h"
#include "Chaos/ImplicitQRSVD.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/MPMTransfer.h"
#include "Chaos/Framework/Parallel.h"
#include "Misc/FileHelper.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Paths.h"
#include "Chaos/GraphColoring.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/SoftsSolverParticlesRange.h"

//DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Corotated Constraint"), STAT_ChaosXPBDCorotated, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Corotated Constraint Polar Compute"), STAT_ChaosXPBDCorotatedPolar, STATGROUP_Chaos);
//DECLARE_CYCLE_STAT(TEXT("Chaos XPBD Corotated Constraint Det Compute"), STAT_ChaosXPBDCorotatedDet, STATGROUP_Chaos);

namespace Chaos::Softs
{

	struct FDeformableXPBDWeightedSpringConstraintParams
	{
		float DebugLineWidth = 5.f;
		float DebugParticleWidth = 20.f;
		bool bVisualizeBindings = false;
	};

	using Chaos::TVec3;

	// Builds weighted spring constraints between two arbitrary groups of vertices
	class FXPBDWeightedSpringConstraints
	{

	public:
		static constexpr FSolverReal SoftMaxStiffness = (FSolverReal)1e14; // Stiffnesses greater than this will be treated as "hard" PBD constraints

		// FSolverParticles constructors (delegate to Range versions)
		CHAOS_API FXPBDWeightedSpringConstraints(
			const FSolverParticles& InParticles,
			const TArray<TArray<int32>>& InIndices,
			const TArray<TArray<FSolverReal>>& InWeights,
			const TArray<FSolverReal>& InStiffness
		);

		CHAOS_API FXPBDWeightedSpringConstraints(
			const FSolverParticles& InParticles,
			const TArray<TArray<int32>>& InIndices,
			const TArray<TArray<FSolverReal>>& InWeights,
			const TArray<FSolverReal>& InStiffness,
			const TArray<TArray<int32>>& InSecondIndices,
			const TArray<TArray<FSolverReal>>& InSecondWeights,
			const FDeformableXPBDWeightedSpringConstraintParams& InParams
		);

		CHAOS_API FXPBDWeightedSpringConstraints(
			const FSolverParticles& InParticles,
			const FSolverParticles& InSecondParticles,
			const TArray<TArray<int32>>& InIndices,
			const TArray<TArray<FSolverReal>>& InWeights,
			const TArray<FSolverReal>& InStiffness,
			const TArray<TArray<int32>>& InSecondIndices,
			const TArray<TArray<FSolverReal>>& InSecondWeights,
			const FDeformableXPBDWeightedSpringConstraintParams& InParams
		);

		// Range-based constructors
		CHAOS_API FXPBDWeightedSpringConstraints(
			const FSolverParticlesRange& InParticlesRange,
			const TArray<TArray<int32>>& InIndices,
			const TArray<TArray<FSolverReal>>& InWeights,
			const TArray<FSolverReal>& InStiffness
		);

		CHAOS_API FXPBDWeightedSpringConstraints(
			const FSolverParticlesRange& InParticlesRange,
			const TArray<TArray<int32>>& InIndices,
			const TArray<TArray<FSolverReal>>& InWeights,
			const TArray<FSolverReal>& InStiffness,
			const TArray<TArray<int32>>& InSecondIndices,
			const TArray<TArray<FSolverReal>>& InSecondWeights,
			const FDeformableXPBDWeightedSpringConstraintParams& InParams
		);

		CHAOS_API FXPBDWeightedSpringConstraints(
			const FSolverParticlesRange& InParticlesRange,
			const FSolverParticlesRange& InSecondParticlesRange,
			const TArray<TArray<int32>>& InIndices,
			const TArray<TArray<FSolverReal>>& InWeights,
			const TArray<FSolverReal>& InStiffness,
			const TArray<TArray<int32>>& InSecondIndices,
			const TArray<TArray<FSolverReal>>& InSecondWeights,
			const FDeformableXPBDWeightedSpringConstraintParams& InParams
		);

		virtual ~FXPBDWeightedSpringConstraints() = default;

		CHAOS_API void ApplyInParallel(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

		CHAOS_API void ApplyInParallel(FSolverParticles& Particles, const FSolverReal Dt) const;

		CHAOS_API void Apply(FSolverParticlesRange& Particles, const FSolverReal Dt) const;

		CHAOS_API void ApplyInSerial(FSolverParticlesRange& Particles, FSolverParticlesRange& SecondParticles, const FSolverReal Dt) const;

		CHAOS_API void ApplyInParallel(FSolverParticlesRange& Particles, FSolverParticlesRange& SecondParticles, const FSolverReal Dt) const;

		const TArray<TArray<int32>>& GetIndices()
		{
			return Indices;
		}

		CHAOS_API void Init(const FSolverParticlesRange& InParticles, const FSolverReal Dt) const;

		CHAOS_API void Init(const FSolverParticles& InParticles, const FSolverReal Dt) const;

		CHAOS_API void Init(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles, const FSolverReal Dt) const;

		void UpdateTargets(TArray<TVector<FSolverReal, 3>>&& InTargets)
		{
			Constraints = MoveTemp(InTargets);
		}


		CHAOS_API void VisualizeAllBindings(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles,
			const TFunctionRef<void(const FVector& Pos0, const FVector& Pos1, const FLinearColor& Color)>& DrawLine,
			const TFunctionRef<void(const FVector& Pos, const FLinearColor& Color)>& DrawPoint) const;

	protected:
		CHAOS_API virtual bool ComputeSpringEdge(const FSolverParticlesRange& InParticles, TVec3<FSolverReal>& OutSpringEdge, int32 ConstraintIndex, bool bUseParticleX, bool bUseConstraintTargetPosition) const;

		CHAOS_API virtual bool ComputeSpringEdge(const FSolverParticlesRange& InParticles, const FSolverParticlesRange& InSecondParticles,
			TVec3<FSolverReal>& OutSpringEdge, int32 ConstraintIndex, bool bUseParticleX, bool bUseConstraintTargetPosition) const;

	protected:
		TArray<TArray<int32>> Indices;
		TArray<TArray<FSolverReal>> Weights;
		TArray<TVector<FSolverReal, 3>> Constraints;
		TArray<TArray<int32>> SecondIndices;
		TArray<TArray<FSolverReal>> SecondWeights;
		TArray<FSolverReal> Stiffness;

	private:
		void InitColor(const FSolverParticlesRange& ParticlesRange);
		void InitColor(const FSolverParticlesRange& ParticlesRange, const FSolverParticlesRange& SecondParticlesRange);

		void ApplySingleConstraintWithoutSelfTarget(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal Tol = UE_KINDA_SMALL_NUMBER) const;

		void ApplySingleConstraintWithSelfTarget(FSolverParticlesRange& Particles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal Tol = UE_KINDA_SMALL_NUMBER) const;

		//Deprecating with ApplySingleConstraintBetweenParticles
		void ApplySingleConstraintWithSelfTarget(FSolverParticlesRange& Particles, FSolverParticlesRange& SecondParticles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal Tol = UE_KINDA_SMALL_NUMBER) const;

		void ApplySingleConstraintBetweenParticles(FSolverParticlesRange& Particles, FSolverParticlesRange& SecondParticles, const FSolverReal Dt, const int32 ConstraintIndex, const FSolverReal Tol = UE_KINDA_SMALL_NUMBER) const;

		void VisualizeAllBindings(const FSolverParticlesRange& InParticles, const FSolverReal Dt) const;

		void VisualizeAllBindings(const FSolverParticlesRange& InParticles,
			const TFunctionRef<void(const FVector& Pos0, const FVector& Pos1, const FLinearColor& Color)>& DrawLine,
			const TFunctionRef<void(const FVector& Pos, const FLinearColor& Color)>& DrawPoint) const;

		TArray<TArray<int32>> ConstraintsPerColor;
		mutable TArray<FSolverReal> LambdaArray;
		bool VisualizeBindings = false;
		FDeformableXPBDWeightedSpringConstraintParams DebugDrawParams;
	};

	using FDeformableXPBDWeakConstraintParams = FDeformableXPBDWeightedSpringConstraintParams;

	template <typename T, typename ParticleType>
	using FXPBDWeakConstraints UE_DEPRECATED(5.8, "Deprecated. this class is to be deleted, use FXPBDWeightedSpringConstraints instead") = FXPBDWeightedSpringConstraints;


}  // End namespace Chaos::Softs

