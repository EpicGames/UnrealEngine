// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosDebugDraw/ChaosDDFrame.h"
#include "ChaosDebugDraw/ChaosDDTypes.h"
#include "HAL/CriticalSection.h"
#include "HAL/ThreadSingleton.h"
#include "Misc/ScopeLock.h"

#if CHAOS_DEBUG_DRAW

namespace ChaosDD::Private
{
	class FChaosDDContext;
}

UE_DECLARE_THREAD_SINGLETON_TLS(ChaosDD::Private::FChaosDDContext, CHAOS_API)

namespace ChaosDD::Private
{
	//
	// A thread-local debug draw context used to access the queue to draw to
	// for any thread on which debug draw has been set up.
	//
	class FChaosDDContext : public TThreadSingleton<FChaosDDContext>
	{
	public:
		CHAOS_API FChaosDDContext();

		// Whether the debug draw system is enabled
		static bool IsDebugDrawEnabled()
		{
			return bDebugDrawEnabled;
		}

		// Enable/Disable the debug draw system
		static void SetIsDebugDrawEnabled(bool bInEnabled)
		{
			bDebugDrawEnabled = bInEnabled;
		}

		// Write access to the debug draw frame for the current thread
		static FChaosDDFrameWriter GetWriter()
		{
			if (bDebugDrawEnabled)
			{
				return FChaosDDFrameWriter(Get().GetFrame());
			}
			return FChaosDDFrameWriter({});
		}

		// For internal use - collect all out-of-frame debug draw commands for rendering
		static CHAOS_API FChaosDDFramePtr ExtractGlobalFrame();

	private:
		friend class FChaosDDScene;
		friend class FChaosDDTaskContext;
		friend class FChaosDDTaskParentContext;
		friend class FChaosDDTimelineContext;

		const FChaosDDFramePtr& GetFrame() const
		{
			// The frame we should be drawing into on this thread
			if (!Frame.IsValid())
			{
				// If there is no Context set up on this thread we fall back to a global frame that 
				// is tied to the game thread. If debug draw commands are queued while the game thread
				// is rendering the DDScene, the commands will be split across frames resulting in flicker
				// @todo(chaos): ideally we don't have a global frame - try to get rid of it
				return GetGlobalFrame();
			}

			return Frame;
		}

		// Global frame management
		static CHAOS_API const FChaosDDFramePtr& GetGlobalFrame();
		static void CreateGlobalFrame();
		static void SetGlobalDrawRegion(const FSphere3d& InDrawRegion);
		static void SetGlobalCommandBudget(int32 InCommandBudget);

		// The frame to draw to on this thread (or null)
		FChaosDDFramePtr Frame;

		// Whether the system is enabled
		static CHAOS_API bool bDebugDrawEnabled;

		// Global frame: fallback for out-of-context debug draw
		static FCriticalSection GlobalFrameCS;
		static FChaosDDFramePtr GlobalFrame;
		static int32 GlobalCommandBudget;
	};

	//
	// Initializes the FChaosDDContext for a thread that owns a timeline
	// 
	// This starts a new frame (debug draw buffer) and sets up the FChaosDDContext for this thread.
	// The active context should be accessed via FChaosDDContext::GetWriter(). FChaosDDTimelineContext
	// is not directly used other than to instantiate.
	//
	class FChaosDDTimelineContext
	{
	public:
		CHAOS_API void BeginFrame(const FChaosDDTimelinePtr& InTimeline, double InTime, double InDt);
		CHAOS_API void EndFrame();

	private:
		FChaosDDTimelinePtr Timeline;
		FChaosDDFramePtr PreviousFrame;
		bool bInContext = false;
	};

	//
	// A scoped wrapper for FChaosDDTimelineContext
	//
	class FChaosDDScopeTimelineContext
	{
	public:
		CHAOS_API FChaosDDScopeTimelineContext(const FChaosDDTimelinePtr& InTimeline, double InTime, double InDt);
		CHAOS_API ~FChaosDDScopeTimelineContext();

	private:
		FChaosDDTimelineContext Context;
	};

	//
	// Used to propagate a debug draw context to a child thread.
	// To use: 
	//		- put a FChaosDDTaskParentContext on the stack on the parent thread
	//		- pass the FChaosDDTaskParentContext to the child thread
	//		- put FChaosDDScopeTaskContext(ParentContext) on the child thread
	// (Search for FChaosDDScopeTaskContext for examples.)
	//
	class FChaosDDTaskParentContext
	{
	public:
		CHAOS_API FChaosDDTaskParentContext();
	private:
		friend class FChaosDDTaskContext;
		FChaosDDFramePtr Frame;
	};

	//
	// Initializes the FChaosDDContext for a task thread.
	// Assumes that the task is kicked off from a thread that has an active debug draw context,
	// which should be passed into this context. Any debug draws from the task will go to the 
	// same frame as the parent context.
	// 
	// The active context is accessed via FChaosDDContext::GetWriter() (and not this object).
	// 
	// NOTE: This is only intended to be used for tasks which will be awaited before the end
	// of the frame (truly asynchronous tasks would need their own timeline, or just set up
	// a context that writes to the global frame)
	// 
	// @todo(chaos): ChaosDDFrame should track how many contexts it is referenced by and
	// assert that it is not active when we end the frame.
	//
	class FChaosDDTaskContext
	{
	public:
		CHAOS_API void BeginThread(const FChaosDDTaskParentContext& InParentDDContext);
		CHAOS_API void EndThread();

	private:
		FChaosDDFramePtr PreviousFrame;
		bool bInContext = false;
	};

	//
	// A scoped wrapper for FChaosDDTaskContext
	//
	class FChaosDDScopeTaskContext
	{
	public:
		CHAOS_API FChaosDDScopeTaskContext(const FChaosDDTaskParentContext& InParentDDContext);
		CHAOS_API ~FChaosDDScopeTaskContext();

	private:
		FChaosDDTaskContext Context;
	};
}

#endif