// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosDebugDraw/ChaosDDTypes.h"
#include "Chaos/DebugDrawCommand.h"
#include "Containers/Map.h"
#include "HAL/PlatformTLS.h"
#include "HAL/CriticalSection.h"
#include "Math/Box.h"
#include "Math/Sphere.h"


#if CHAOS_DEBUG_DRAW

namespace ChaosDD::Private
{
	//
	// A frame is a sequence of debug draw commands.
	// A command is just a functor that uses a DD Renderer and can be a simple as drawing
	// a line, or as complex as drawing a set of rigid bodies, constraints, etc.
	// 
	// E.g., See FChaosDDLine, FChaosDDSphere, FChaosDDParticle
	//
	using FChaosDDCommand = TFunction<void(IChaosDDRenderer&)>;

	//
	// A single frame of debug draw data
	//
	// @todo(chaos): move commands to a per-thread buffer and eliminate write locks
	//
	class FChaosDDFrame : public TSharedFromThis<FChaosDDFrame>
	{
	public:
		FChaosDDFrame(const FChaosDDTimelineWeakPtr& InTimeline, int64 InFrameIndex, double InTime, double InDt, const FSphere3d& InDrawRegion, int32 InCommandBudget, int32 InCommandQueueLength)
			: Timeline(InTimeline)
			, FrameIndex(InFrameIndex)
			, Time(InTime)
			, Dt(InDt)
			, DrawRegion(InDrawRegion)
			, CommandBudget(InCommandBudget)
			, CommandCost(0)
		{
			if (InCommandQueueLength > 0)
			{
				Commands.Reserve(InCommandQueueLength);
				LatentCommands.Reserve(InCommandQueueLength);
			}
			BuildName();
		}

		virtual ~FChaosDDFrame() = default;

		const FString& GetName() const
		{
			return Name;
		}

		FChaosDDTimelinePtr GetTimeline() const
		{
			return Timeline.Pin();
		}

		int64 GetFrameIndex() const
		{
			return FrameIndex;
		}

		double GetTime() const
		{
			return Time;
		}

		double GetDt() const
		{
			return Dt;
		}

		void SetDrawRegion(const FSphere3d& InRegion)
		{
			DrawRegion = InRegion;
		}

		const FSphere3d& GetDrawRegion() const
		{
			return DrawRegion;
		}

		bool IsInDrawRegion(const FVector& InPos) const
		{
			if (DrawRegion.W > 0.0)
			{
				return DrawRegion.IsInside(InPos);
			}
			return true;
		}

		bool IsInDrawRegion(const FSphere3d& InSphere) const
		{
			if (DrawRegion.W > 0.0)
			{
				return DrawRegion.Intersects(InSphere);
			}
			return true;
		}

		bool IsInDrawRegion(const FBox3d& InBox) const
		{
			if (DrawRegion.W > 0.0)
			{
				const double BoxDistanceSq = InBox.ComputeSquaredDistanceToPoint(DrawRegion.Center);
				return BoxDistanceSq < FMath::Square(DrawRegion.W);
			}
			return true;
		}

		void SetCommandBudget(int32 InCommandBudget)
		{
			CommandBudget = InCommandBudget;
		}

		int32 GetCommandBudget() const
		{
			return CommandBudget;
		}

		int32 GetCommandCost() const
		{
			return CommandCost;
		}

		bool WasCommandBudgetExceeded() const
		{
			return (CommandCost > CommandBudget) && (CommandBudget > 0);
		}

		bool AddToCost(int32 InCost)
		{
			FScopeLock Lock(&CommandsCS);

			CommandCost += InCost;

			// A budget of zero means infinite
			if ((CommandCost <= CommandBudget) || (CommandBudget == 0))
			{
				return true;
			}

			return false;
		}

		void EnqueueCommand(FChaosDDCommand&& InCommand)
		{
			FScopeLock Lock(&CommandsCS);

			Commands.Emplace(MoveTemp(InCommand));
		}

		void EnqueueLatentCommand(const Chaos::FLatentDrawCommand& InCommand)
		{
			FScopeLock Lock(&CommandsCS);

			LatentCommands.Add(InCommand);
		}

		int32 GetNumCommands() const
		{
			FScopeLock Lock(&CommandsCS);

			return Commands.Num();
		}

		int32 GetNumLatentCommands() const
		{
			FScopeLock Lock(&CommandsCS);

			return LatentCommands.Num();
		}

		// VisitorType = void(const FChaosDDCommand& Command)
		template<typename VisitorType>
		void VisitCommands(const VisitorType& Visitor)
		{
			FScopeLock Lock(&CommandsCS);

			for (const FChaosDDCommand& Command : Commands)
			{
				Visitor(Command);
			}
		}

		// VisitorType = void(const Chaos::FLatentDrawCommand& Command)
		template<typename VisitorType>
		void VisitLatentCommands(const VisitorType& Visitor)
		{
			FScopeLock Lock(&CommandsCS);

			for (const Chaos::FLatentDrawCommand& Command : LatentCommands)
			{
				Visitor(Command);
			}
		}

		// Used by the global frame to prevent render while queuing commands
		virtual void BeginWrite()
		{
		}

		// Used by the global frame to prevent render while queuing commands
		virtual void EndWrite()
		{
		}

		// Used by the global frame to extract all debug draw commands so far into a new frame for rendering
		virtual FChaosDDFramePtr ExtractFrame()
		{
			const int32 CommandQueueLength = Commands.Max();
			const int32 LatentCommandQueueLength = LatentCommands.Max();

			FChaosDDFrame* ExtractedFrame = new FChaosDDFrame(MoveTemp(*this));

			Commands.Reset(CommandQueueLength);
			LatentCommands.Reset(LatentCommandQueueLength);
			CommandCost = 0;

			return FChaosDDFramePtr(ExtractedFrame);
		}

	protected:
		friend class FChaosDDGlobalFrame;

		FChaosDDFrame(FChaosDDFrame&& Other)
			: Timeline(Other.Timeline)
			, FrameIndex(Other.FrameIndex)
			, Time(Other.Time)
			, Dt(Other.Dt)
			, DrawRegion(Other.DrawRegion)
			, CommandBudget(Other.CommandBudget)
			, CommandCost(Other.CommandCost)
			, Commands(MoveTemp(Other.Commands))
			, LatentCommands(MoveTemp(Other.LatentCommands))
		{
			BuildName();
		}

		// Implemented in ChaosDDTimeline.h
		void BuildName();

		FChaosDDTimelineWeakPtr Timeline;
		int64 FrameIndex;
		double Time;
		double Dt;
		FSphere3d DrawRegion;
		int32 CommandBudget;
		int32 CommandCost;

		FString Name;

		// Debug draw commands
		TArray<FChaosDDCommand> Commands;

		// Legacy debug draw commands (see FDebugDrawQueue)
		TArray<Chaos::FLatentDrawCommand> LatentCommands;

		mutable FCriticalSection CommandsCS;
	};

	//
	// A special frame used for out-of-frame debug draw. All debug draw commands from a thread that does not
	// have a context set up will use the global frame. This global frame suffers will be flickery because
	// the render may occur while enqueueing a set of related debug draw commands.
	// 
	// @todo(chaos): eventually all threads that want debug draw should have a valid frame and this will be redundant.
	//
	class FChaosDDGlobalFrame : public FChaosDDFrame
	{
	public:

		FChaosDDGlobalFrame(int32 InCommandBudget)
			: FChaosDDFrame(FChaosDDTimelinePtr(), 0, 0.0f, 0.0f, FSphere3d(FVector::Zero(), 0.0), InCommandBudget, 0)
		{
		}

		virtual void BeginWrite() override
		{
			FrameWriteCS.Lock();
		}

		virtual void EndWrite() override
		{
			FrameWriteCS.Unlock();
		}

		// Create a new frame containing the accumulated commands and reset this frame
		virtual FChaosDDFramePtr ExtractFrame() override
		{
			FScopeLock Lock(&FrameWriteCS);

			return FChaosDDFrame::ExtractFrame();
		}

	protected:

		mutable FCriticalSection FrameWriteCS;
	};

	//
	// Used to write to a debug draw frame.
	// Currently this writes to the Frame's draw buffer and holds a lock
	// preventing the frame from being Ended. Eventually this will be a
	// per-thread buffer to avoid the need for locks.
	//
	class FChaosDDFrameWriter
	{
	public:
		FChaosDDFrameWriter(const FChaosDDFramePtr& InFrame)
			: Frame(InFrame)
		{
			if (Frame.IsValid())
			{
				Frame->BeginWrite();
			}
		}

		~FChaosDDFrameWriter()
		{
			if (Frame.IsValid())
			{
				Frame->EndWrite();
			}
		}

		FSphere3d GetDrawRegion() const
		{
			if (Frame.IsValid())
			{
				return Frame->GetDrawRegion();
			}
			return FSphere3d(FVector3d(0), 0);
		}

		bool IsInDrawRegion(const FVector& InPos) const
		{
			if (Frame.IsValid())
			{
				return Frame->IsInDrawRegion(InPos);
			}
			return false;
		}

		bool IsInDrawRegion(const FSphere3d& InSphere) const
		{
			if (Frame.IsValid())
			{
				return Frame->IsInDrawRegion(InSphere);
			}
			return false;
		}

		bool IsInDrawRegion(const FBox3d& InBox) const
		{
			if (Frame.IsValid())
			{
				return Frame->IsInDrawRegion(InBox);
			}
			return false;
		}

		bool AddToCost(int32 InCost)
		{
			if (Frame.IsValid())
			{
				return Frame->AddToCost(InCost);
			}
			return false;
		}

		bool TryEnqueueCommand(int32 InCost, const FBox3d& InBox, FChaosDDCommand&& InCommand)
		{
			if (Frame.IsValid())
			{
				if (Frame->IsInDrawRegion(InBox) && Frame->AddToCost(InCost))
				{
					Frame->EnqueueCommand(MoveTemp(InCommand));
					return true;
				}
			}
			return false;
		}

		void EnqueueCommand(FChaosDDCommand&& InCommand)
		{
			if (Frame.IsValid())
			{
				Frame->EnqueueCommand(MoveTemp(InCommand));
			}
		}

		void EnqueueLatentCommand(const Chaos::FLatentDrawCommand& InCommand)
		{
			if (Frame.IsValid())
			{
				Frame->EnqueueLatentCommand(InCommand);
			}
		}

	private:
		FChaosDDFramePtr Frame;
	};
}

#endif
