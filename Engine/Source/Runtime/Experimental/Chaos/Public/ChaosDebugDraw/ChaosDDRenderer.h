// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "ChaosDebugDraw/ChaosDDTypes.h"

#if CHAOS_DEBUG_DRAW

namespace Chaos
{
	struct FLatentDrawCommand;
}

namespace ChaosDD::Private
{
	//
	// Primitive rendering API for use by debug draw objects
	//
	class IChaosDDRenderer
	{
	public:
		IChaosDDRenderer() {}
		virtual ~IChaosDDRenderer() = default;

		// Are we rendering a Server scene?
		virtual bool IsServer() const = 0;

		// The region of interest
		virtual FSphere3d GetDrawRegion() const = 0;

		// Utility functions for use by Debug Draw commands (e.g., FChaosDDLine)
		virtual void RenderPoint(const FVector3d& Position, const FColor& Color, float PointSize, float Lifetime) = 0;
		virtual void RenderLine(const FVector3d& A, const FVector3d& B, const FColor& Color, float LineThickness, float Lifetime) = 0;
		virtual void RenderArrow(const FVector3d& A, const FVector3d& B, float ArrowSize, const FColor& Color, float LineThickness, float Lifetime) = 0;
		virtual void RenderCircle(const FVector3d& Center, const FMatrix& Axes, float Radius, const FColor& Color, float LineThickness, float Lifetime) = 0;
		virtual void RenderSphere(const FVector3d& Center, float Radius, const FColor& Color, float LineThickness, float Lifetime) = 0;
		virtual void RenderCapsule(const FVector3d& Center, const FQuat4d& Rotation, float HalfHeight, float Radius, const FColor& Color, float LineThickness, float Lifetime) = 0;
		virtual void RenderBox(const FVector3d& Position, const FQuat4d& Rotation, const FVector3d& Size, const FColor& Color, float LineThickness, float Lifetime) = 0;
		virtual void RenderTriangle(const FVector3d& A, const FVector3d& B, const FVector3d& C, const FColor& Color, float LineThickness, float Lifetime) = 0;
		virtual void RenderString(const FVector3d& TextLocation, const FString& Text, const FColor& Color, float FontScale, bool bDrawShadow, float Lifetime) = 0;

		// Render legacy debug draw command (See FChaosDDScene::RenderLatestFrames)
		virtual void RenderLatentCommand(const Chaos::FLatentDrawCommand& Command) = 0;
	};
}

#endif
