// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosDebugDraw/ChaosDDTypes.h"
#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "Math/Sphere.h"

#define UE_API CHAOS_API

#if CHAOS_DEBUG_DRAW

namespace ChaosDD::Private
{
	class IChaosDDRenderer;
	class FChaosDDTimeline;

	//
	// Debug draw system for a world. In PIE there will be one of these for the server and each client.
	//
	// @todo(chaos): enable retention of debug draw frames and debug draw from a specific time
	class FChaosDDScene : public TSharedFromThis<FChaosDDScene>
	{
	public:
		UE_API FChaosDDScene(const FString& InName, bool bIsServer);
		UE_API ~FChaosDDScene();

		const FString& GetName() const
		{
			return Name;
		}

		UE_API bool IsServer() const;
		UE_API void SetRenderEnabled(bool bInRenderEnabled);
		UE_API bool IsRenderEnabled() const;

		// Specify the region of in which we wish to enable debug draw. A radius of zero means everywhere.
		UE_API void SetDrawRegion(const FSphere3d& InDrawRegion);

		// The region of interest
		UE_API const FSphere3d& GetDrawRegion() const;

		// Set the line budget for debug draw
		UE_API void SetCommandBudget(int32 InCommandBudget);

		// The number of commands we can draw (also max number of lines for now)
		UE_API int32 GetCommandBudget() const;

		// Create a new timeline. E.g., PT, GT, RBAN
		// The caller must hold a shared pointer to the timeline to keep it alive.
		UE_API FChaosDDTimelinePtr CreateTimeline(const FString& Name);

		// Collect all the latest complete frames for rendering
		UE_API TArray<FChaosDDFramePtr> GetLatestFrames();

	private:
		UE_API TArray<FChaosDDFramePtr> GetFrames();
		UE_API void PruneTimelines();

		mutable FCriticalSection TimelinesCS;

		FString Name;
		TArray<FChaosDDTimelineWeakPtr> Timelines;
		FSphere3d DrawRegion;
		int32 CommandBudget;
		bool bIsServer;
		bool bRenderEnabled;
	};
}

#endif

#undef UE_API
