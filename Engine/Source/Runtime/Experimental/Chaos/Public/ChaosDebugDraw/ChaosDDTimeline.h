// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosDebugDraw/ChaosDDTypes.h"
#include "ChaosDebugDraw/ChaosDDFrame.h"
#include "ChaosDebugDraw/ChaosDDLog.h"
#include "Containers/Map.h"
#include "HAL/CriticalSection.h"
#include "Math/Sphere.h"
#include "Misc/ScopeLock.h"

#if CHAOS_DEBUG_DRAW

namespace ChaosDD::Private
{
	//
	// Holds a sequence of debug draw frames. Every system that runs its own loop 
	// will need a timeline. E.g., Physics Thread, Game Thread, RBANs
	// 
	// @todo(chaos): Add per-thread buffers to avoid locks
	//
	class FChaosDDTimeline : public TSharedFromThis<FChaosDDTimeline>
	{
	public:
		FChaosDDTimeline(const FString& InName, int32 InCommandBudget)
			: Name(InName)
			, MaxFrames(1)
			, NextFrameIndex(0)
			, DrawRegion(FSphere3d(FVector::Zero(), 0.0))
			, CommandBudget(InCommandBudget)
			, CommandQueueLength(0)
		{
		}

		const FString& GetName() const
		{
			return Name;
		}

		void SetDrawRegion(const FSphere3d& InDrawRegion)
		{
			FScopeLock Lock(&FramesCS);

			DrawRegion = InDrawRegion;
		}

		void SetCommandBudget(int32 InCommandBudget)
		{
			FScopeLock Lock(&FramesCS);

			CommandBudget = InCommandBudget;
		}

		void BeginFrame(double InTime, double InDt)
		{
			FScopeLock Lock(&FramesCS);

			// Duplicate BeginFrame?
			check(!ActiveFrame.IsValid());

			ActiveFrame = MakeShared<FChaosDDFrame>(this->AsShared(), NextFrameIndex++, InTime, InDt, DrawRegion, CommandBudget, CommandQueueLength);

			//UE_LOGF(LogChaosDD, VeryVerbose, "BeginFrame %ls %lld", *GetName(), ActiveFrame->GetFrameIndex());
		}

		void EndFrame()
		{
			FScopeLock Lock(&FramesCS);

			// Missing BeginFrame?
			check(ActiveFrame.IsValid());

			//UE_LOGF(LogChaosDD, VeryVerbose, "EndFrame %ls %lld (%d Commands)", *GetName(), ActiveFrame->GetFrameIndex(), ActiveFrame->GetNumCommands());

			// Remeber the queue size to prevent array growth every frame
			CommandQueueLength = FMath::RoundUpToPowerOfTwo(FMath::Max(CommandQueueLength, ActiveFrame->GetNumCommands()));

			Frames.Add(ActiveFrame);
			ActiveFrame.Reset();

			PruneFrames();
		}

		const FChaosDDFramePtr& GetActiveFrame() const
		{
			return ActiveFrame;
		}

		void GetFrames(TArray<FChaosDDFramePtr>& InOutFrames)
		{
			FScopeLock Lock(&FramesCS);

			for (const FChaosDDFramePtr& Frame : Frames)
			{
				InOutFrames.Add(Frame);
			}
		}

	private:
		void PruneFrames()
		{
			const int32 NumToRemove = Frames.Num() - MaxFrames;
			if (NumToRemove > 0)
			{
				Frames.RemoveAt(0, NumToRemove, EAllowShrinking::No);
			}
		}

		FString Name;
		TArray<FChaosDDFramePtr> Frames;
		FChaosDDFramePtr ActiveFrame;
		int32 MaxFrames;
		int64 NextFrameIndex;
		FSphere3d DrawRegion;
		int32 CommandBudget;
		int32 CommandQueueLength;
		FCriticalSection FramesCS;
	};


	// Here to avoid circular dependency with ChaosDDFrame.h
	inline void FChaosDDFrame::BuildName()
	{
		if (FChaosDDTimelinePtr PinnedTimeline = GetTimeline())
		{
			Name = FString::Format(TEXT("{0}: {1}"), { PinnedTimeline->GetName(), GetFrameIndex() });
		}
		else
		{
			Name = FString::Format(TEXT("<Global>: {0}"), { GetFrameIndex()});
		}
	}

}

#endif
