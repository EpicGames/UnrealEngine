// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Declares.h"
#include "Math/MathFwd.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"

#if CHAOS_DEBUG_DRAW

namespace Chaos
{
	struct FLatentDrawCommand;
}


// @todo(chaos): Move to ChaosDD when API is decided
namespace ChaosDD::Private
{
	class FChaosDDContext;
	class FChaosDDFrame;
	class FChaosDDFrameWriter;
	class FChaosDDGlobalFrame;
	class FChaosDDScene;
	class FChaosDDTaskContext;
	class FChaosDDTaskParentContext;
	class FChaosDDTimeline;
	class FChaosDDTimelineContext;

	class IChaosDDRenderer;

	using FChaosDDFramePtr = TSharedPtr<FChaosDDFrame, ESPMode::ThreadSafe>;
	using FChaosDDGlobalFramePtr = TSharedPtr<FChaosDDGlobalFrame, ESPMode::ThreadSafe>;
	using FChaosDDScenePtr = TSharedPtr<FChaosDDScene, ESPMode::ThreadSafe>;
	using FChaosDDSceneWeakPtr = TWeakPtr<FChaosDDScene, ESPMode::ThreadSafe>;
	using FChaosDDTimelinePtr = TSharedPtr<FChaosDDTimeline, ESPMode::ThreadSafe>;
	using FChaosDDTimelineWeakPtr = TWeakPtr<FChaosDDTimeline, ESPMode::ThreadSafe>;
}

#endif