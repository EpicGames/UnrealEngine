// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_TRACE_ENABLED

#include "CoreTypes.h"

namespace Chaos::Insights
{
	enum class ELockEventType
	{
		Mutex,
		RWLockReadLock,
		RWLockWriteLock
	};

	CHAOS_API uint64 BeginLockAcquireEvent(ELockEventType Type);
	CHAOS_API void AcquiredLock();
	CHAOS_API void EndLockAcquireEvent();
}

#define TRACE_CHAOS_BEGIN_LOCK(Type)	Chaos::Insights::BeginLockAcquireEvent(Type);
#define TRACE_CHAOS_ACQUIRE_LOCK()		Chaos::Insights::AcquiredLock();
#define TRACE_CHAOS_END_LOCK()			Chaos::Insights::EndLockAcquireEvent();

#else

#define TRACE_CHAOS_BEGIN_LOCK(...)
#define TRACE_CHAOS_ACQUIRE_LOCK(...)
#define TRACE_CHAOS_END_LOCK(...)

#endif // UE_TRACE_ENABLED