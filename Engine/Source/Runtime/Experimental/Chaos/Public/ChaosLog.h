// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ProfilingDebugging/CsvProfiler.h"

CHAOS_API DECLARE_LOG_CATEGORY_EXTERN(LogChaosGeneral, Log, All);
CHAOS_API DECLARE_LOG_CATEGORY_EXTERN(LogChaosThread, Log, All);
CHAOS_API DECLARE_LOG_CATEGORY_EXTERN(LogChaosSimulation, Log, All);
CHAOS_API DECLARE_LOG_CATEGORY_EXTERN(LogChaosDebug, Log, All);
CHAOS_API DECLARE_LOG_CATEGORY_EXTERN(LogChaosDataflow, Log, All);

CHAOS_API DECLARE_LOG_CATEGORY_EXTERN(LogChaos, Log, All);

UE_DEPRECATED(5.7, "Use LogChaos instead")
CHAOS_API DECLARE_LOG_CATEGORY_EXTERN(UManagedArrayLogging, NoLogging, All);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CHAOS_API, Chaos);

CSV_DECLARE_CATEGORY_MODULE_EXTERN(CHAOS_API, PhysicsVerbose);
CSV_DECLARE_CATEGORY_MODULE_EXTERN(CHAOS_API, PhysicsCounters);
