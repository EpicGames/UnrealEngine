// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class FChaosEngineModule : public IModuleInterface
{
public:

	CHAOS_API virtual void StartupModule() override;
	CHAOS_API virtual void ShutdownModule() override;
};
