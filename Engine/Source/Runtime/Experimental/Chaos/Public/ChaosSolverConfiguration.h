// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SolverEventFilters.h"

#include "ChaosSolverConfiguration.generated.h"

UENUM()
enum class EClusterUnionMethod : uint8
{
	PointImplicit,
	DelaunayTriangulation,
	MinimalSpanningSubsetDelaunayTriangulation,
	PointImplicitAugmentedWithMinimalDelaunay,
	BoundsOverlapFilteredDelaunayTriangulation,
	None
};

USTRUCT()
struct FChaosSolverDestructionSettings
{
	GENERATED_BODY();

	/** Number of breaks allowed to occur for each invocation of AdvanceClustering */
	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Destruction", meta = (ClampMin = "0"))
	int32 PerAdvanceBreaksAllowed = TNumericLimits<int32>::Max();

	/** Number of breaks allowed to be rescheduled for next frame if any */
	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Destruction", meta = (ClampMin = "0"))
	int32 PerAdvanceBreaksRescheduleLimit = TNumericLimits<int32>::Max();

	/** Minimum number of active geometry collection to reach before clustering start to disable a percentage of the released particle per cluster */
	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Destruction", meta = (ClampMin = "-1"))
	int32 ClusteringParticleReleaseThrottlingMinCount = INDEX_NONE;

	/** Maximum number of active geometry collection to reach before all released clustering disable all released particle instantly */
	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Destruction", meta = (ClampMin = "-1"))
	int32 ClusteringParticleReleaseThrottlingMaxCount = INDEX_NONE;

	/** 
	* When enabled, this will avoid creating non-necessary geometry collection physics data until the root breaks 
	* This will save runtime memory for unbroken geometry collection at the cost of allocating those particle when the first break happens
	* It is false by default 
	*/
	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Destruction")
	bool bOptimizeForRuntimeMemory = false;
};

USTRUCT(BlueprintType)
struct FChaosSolverConfiguration
{
	GENERATED_BODY();

	CHAOS_API FChaosSolverConfiguration();

	// Handle renamed properties
	CHAOS_API void MoveRenamedPropertyValues();

	// The number of position iterations to run during the constraint solver step
	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Iterations", meta = (ClampMin = "0"))
	int32 PositionIterations;
	
	// The number of velocity iterations to run during the constraint solver step
	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Iterations", meta = (ClampMin = "0"))
	int32 VelocityIterations;

	// The number of projection iterations to run during the constraint solver step
	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Iterations", meta = (ClampMin = "0"))
	int32 ProjectionIterations;


	// A collision margin as a fraction of size used by some boxes and convex shapes to improve collision detection results.
	// The core geometry of shapes that support a margin are reduced in size by the margin, and the margin
	// is added back on during collision detection. The net result is a shape of the same size but with rounded corners.
	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Collision", meta = (ClampMin = "0.0"))
	float CollisionMarginFraction;

	// An upper limit on the collision margin that will be subtracted from boxes and convex shapes. See CollisionMarginFraction
	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Collision", meta = (ClampMin = "0.0"))
	float CollisionMarginMax;

	// During collision detection, if tweo shapes are at least this far apart we do not calculate their nearest features
	// during the collision detection step.
	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Collision", meta = (ClampMin = "0.0"))
	float CollisionCullDistance;

	// The maximum speed at which two bodies can be extracted from each other when they start a frame inter-penetrating. This can
	// happen because they spawned on top of each other, or the solver failed to fully reolve collisions last frame. A value of
	// zero means "no limit". A non-zero value can be used to prevent explosive behaviour when bodies start deeply penetrating. 
	// An alternative to using this approach is to increase the number of Velocity Iterations, which is more expensive but will 
	// ensure the bodies are depenetrated in a single frame without explosive behaviour.
	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Collision", meta = (ClampMin = "0.0"))
	float CollisionMaxPushOutVelocity;

	// If two bodies start off in overlapping each other, they will depentrate at this speed when they wake.
	// If set to a large value, initially-overlapping objects will tend to "explode" apart at a speed that depends on the
	// overlap amount and the timestep (this is the original, previously untunable behaviour). If set to zero, 
	// initially-overlapping objects will remain stationary and go to sleep until acted on by some other object or force.
	// A negative value (-1) disables the feature and is equivalent to infinity.
	// This property can be overridden per Body (see FBodyInstance::MaxDepenetrationVelocity)
	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Collision")
	float CollisionInitialOverlapDepenetrationVelocity;

	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Clustering")
	float ClusterConnectionFactor;

	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Clustering")
	EClusterUnionMethod ClusterUnionConnectionType;

	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|Destruction")
	FChaosSolverDestructionSettings DestructionSettings;

	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|GeneratedData")
	bool bGenerateCollisionData;

	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|GeneratedData", meta=(EditCondition=bGenerateCollisionData))
	FSolverCollisionFilterSettings CollisionFilterSettings;

	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|GeneratedData")
	bool bGenerateBreakData;

	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|GeneratedData", meta = (EditCondition = bGenerateBreakData))
	FSolverBreakingFilterSettings BreakingFilterSettings;

	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|GeneratedData")
	bool bGenerateTrailingData;

	UPROPERTY(EditAnywhere, Category = "SolverConfiguration|GeneratedData", meta = (EditCondition = bGenerateTrailingData))
	FSolverTrailingFilterSettings TrailingFilterSettings;

private:

	// Renamed to PositionIterations
	UPROPERTY()
	int32 Iterations_DEPRECATED;

	// Renamed to VelocityIterations
	UPROPERTY()
	int32 PushOutIterations_DEPRECATED;

	// No longer used
	UPROPERTY()
	bool bGenerateContactGraph_DEPRECATED;
};
