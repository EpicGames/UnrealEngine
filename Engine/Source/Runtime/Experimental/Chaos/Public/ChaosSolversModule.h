// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Event.h"
#include "HAL/IConsoleManager.h"
#include "HAL/ThreadSafeBool.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Async/AsyncWork.h"
#include "UObject/ObjectMacros.h"
#include "Framework/Threading.h"
#include "PhysicsCoreTypes.h"
#include "Chaos/Framework/MultiBufferResource.h"
#include "Chaos/Declares.h"
#include "Chaos/PhysicalMaterials.h"
#include "Chaos/Defines.h"
#include "Async/TaskGraphInterfaces.h"

/** Classes that want to set the solver actor class can implement this. */
class IChaosSolverActorClassProvider
{
public:
	virtual UClass* GetSolverActorClass() const = 0;
};

/** Class that external users can implement and set on the module to provide various settings to the system (see FChaosSolversModule::SetSettingsProvider) */
class IChaosSettingsProvider
{
public:
	virtual ~IChaosSettingsProvider() = default;

	virtual float GetMinDeltaVelocityForHitEvents() const
	{
		return 0.f;
	}

	virtual bool GetPhysicsPredictionEnabled() const
	{
		return false;
	}

	UE_DEPRECATED(5.5, "GetResimulationErrorThreshold has been renamed, please use GetResimulationErrorPositionThreshold.")
	virtual float GetResimulationErrorThreshold() const
	{
		return GetResimulationErrorPositionThreshold();
	}

	virtual bool GetResimulationErrorPositionThresholdEnabled() const
	{
		return false;
	}

	virtual float GetResimulationErrorPositionThreshold() const
	{
		return 0;
	}

	virtual bool GetResimulationErrorRotationThresholdEnabled() const
	{
		return false;
	}

	virtual float GetResimulationErrorRotationThreshold() const
	{
		return 0;
	}

	virtual bool GetResimulationErrorLinearVelocityThresholdEnabled() const
	{
		return false;
	}

	virtual float GetResimulationErrorLinearVelocityThreshold() const
	{
		return 0;
	}

	virtual bool GetResimulationErrorAngularVelocityThresholdEnabled() const
	{
		return false;
	}

	virtual float GetResimulationErrorAngularVelocityThreshold() const
	{
		return 0;
	}

	virtual float GetPhysicsHistoryTimeLength() const
	{
		return 0;
	}

	virtual int32 GetPhysicsHistoryCount() const
	{
		return 0;
	}
}; 

namespace Chaos
{
	class FPersistentPhysicsTask;
	class FPhysicsProxy;
	class FPhysicsSolverBase;
}

// Default settings implementation
namespace Chaos
{
	class FInternalDefaultSettings : public IChaosSettingsProvider
	{
	public:
	};

	extern CHAOS_API FInternalDefaultSettings GDefaultChaosSettings;
}

struct FSolverStateStorage
{
	friend class FChaosSolversModule;

	Chaos::FPhysicsSolver* Solver;
	TArray<Chaos::FPhysicsProxy*> ActiveProxies;
	TArray<Chaos::FPhysicsProxy*> ActiveProxies_GameThread;

private:

	// Private so only the module can actually make these so they can be tracked
	FSolverStateStorage();
	FSolverStateStorage(const FSolverStateStorage& InCopy) = default;
	FSolverStateStorage(FSolverStateStorage&& InSteal) = default;
	FSolverStateStorage& operator =(const FSolverStateStorage& InCopy) = default;
	FSolverStateStorage& operator =(FSolverStateStorage&& InSteal) = default;

};

enum class ESolverFlags : uint8
{
	None = 0,
	Standalone = 1 << 0
};
ENUM_CLASS_FLAGS(ESolverFlags);

class FChaosSolversModule
{
public:

	static CHAOS_API FChaosSolversModule* GetModule();

	CHAOS_API FChaosSolversModule();

	CHAOS_API void StartupModule();
	CHAOS_API void ShutdownModule();

	CHAOS_API void Initialize();
	CHAOS_API void Shutdown();

	/**
	 * Queries for multithreaded configurations
	 */
	CHAOS_API bool IsPersistentTaskEnabled() const;
	CHAOS_API bool IsPersistentTaskRunning() const;


	/**
	 * Gets the inner physics thread task if it has been spawned. Care must be taken when
	 * using methods and members that the calling context can safely access those fields
	 * as the task will be running on its own thread.
	 */
	CHAOS_API Chaos::FPersistentPhysicsTask* GetDedicatedTask() const;

	/**
	 * Called to request a sync between the game thread and the currently running physics task
	 * @param bForceBlockingSync forces this 
	 */
	CHAOS_API void SyncTask(bool bForceBlockingSync = false);

	/**
	 * Create a new solver state storage object to contain a solver and proxy storage object. Intended
	 * to be used by the physics scene to create a common storage object that can be passed to a dedicated
	 * thread when it is enabled without having to link Engine from Chaos.
	 *
	 * Should be called from the game thread to create a new solver. After creation, non-standalone solvers
	 * are dispatched to the physics thread automatically if it is available
	 *
	 * @param InOwner Ptr to some owning UObject. The module doesn't use this but allows calling code to organize solver ownership
	 * @param ThreadingMode The desired threading mode the solver will use
	 * @param bStandalone Whether the solver is standalone (not sent to physics thread - updating left to caller)
	 */
	CHAOS_API Chaos::FPBDRigidsSolver* CreateSolver(UObject* InOwner, Chaos::FReal InAsyncDt, Chaos::EThreadingMode ThreadingMode = Chaos::EThreadingMode::SingleThread, const FName& DebugName = NAME_None);

	CHAOS_API void MigrateSolver(Chaos::FPhysicsSolverBase* InSolver, const UObject* InNewOwner);

	/**
	 * Sets the actor type which should be AChaosSolverActor::StaticClass() but that is not accessible from engine.
	 */
	void SetSolverActorClass(UClass* InActorClass, UClass* InActorRequiredBaseClass)
	{
		SolverActorClass = InActorClass;
		SolverActorRequiredBaseClass = InActorRequiredBaseClass;
		check(IsValidSolverActorClass(SolverActorClass));
	}

	CHAOS_API UClass* GetSolverActorClass() const;

	CHAOS_API bool IsValidSolverActorClass(UClass* Class) const;

	/**
	 * Shuts down and destroys a solver state
	 *
	 * Should be called on whichever thread currently owns the solver state
	 */
	CHAOS_API void DestroySolver(Chaos::FPhysicsSolverBase* InState);

	/** Retrieve the list of all extant solvers. This contains all owned, unowned and standalone solvers */
	CHAOS_API const TArray<Chaos::FPhysicsSolverBase*>& GetAllSolvers() const;

	/**
	 * Read access to the current solver-state objects, be aware which thread owns this data when
	 * attempting to use this. Physics thread will query when spinning up to get current world state
	 */
	CHAOS_API TArray<const Chaos::FPhysicsSolverBase*> GetSolvers(const UObject* InOwner) const;
	CHAOS_API void GetSolvers(const UObject* InOwner, TArray<const Chaos::FPhysicsSolverBase*>& OutSolvers) const;

	/** Non-const access to the solver array is private - only the module should ever modify the storage lists */
	CHAOS_API TArray<Chaos::FPhysicsSolverBase*> GetSolversMutable(const UObject* InOwner);
	CHAOS_API void GetSolversMutable(const UObject* InOwner, TArray<Chaos::FPhysicsSolverBase*>& OutSolvers);

	/**
	 * Outputs statistics for the solver hierarchies. Currently engine calls into this
	 * from a console command on demand.
	 */
	CHAOS_API void DumpHierarchyStats(int32* OutOptMaxCellElements = nullptr);

#if WITH_EDITOR
	/**
	 * Pause solvers. Thread safe.
	 * This is typically called from a playing editor to pause all solvers.
	 * @note Game pause must use a different per solver mechanics.
	 */
	CHAOS_API void PauseSolvers();

	/**
	 * Resume solvers. Thread safe.
	 * This is typically called from a paused editor to resume all solvers.
	 * @note Game resume must use a different per solver mechanics.
	 */
	CHAOS_API void ResumeSolvers();

	/**
	 * Single-step advance solvers. Thread safe.
	 * This is typically called from a paused editor to single step all solvers.
	 */
	CHAOS_API void SingleStepSolvers();

	/**
	 * Query whether a particular solver should advance and update its single-step counter. Thread safe.
	 */
	CHAOS_API bool ShouldStepSolver(int32& InOutSingleStepCounter) const;
#endif  // #if WITH_EDITOR

	void RegisterSolverActorClassProvider(IChaosSolverActorClassProvider* Provider)
	{
		SolverActorClassProvider = Provider;
	};


	/** Safe method for always getting a settings provider (from the external caller or an internal default) */
	CHAOS_API const IChaosSettingsProvider& GetSettingsProvider() const;
	void SetSettingsProvider(IChaosSettingsProvider* InProvider)
	{
		SettingsProvider = InProvider;
	}

	void LockSolvers() { SolverLock.Lock(); }
	void UnlockSolvers() { SolverLock.Unlock(); }

	/** Events that the material manager will emit for us to update our threaded data */
	CHAOS_API void OnUpdateMaterial(Chaos::FMaterialHandle InHandle);
	CHAOS_API void OnCreateMaterial(Chaos::FMaterialHandle InHandle);
	CHAOS_API void OnDestroyMaterial(Chaos::FMaterialHandle InHandle);

	CHAOS_API void OnUpdateMaterialMask(Chaos::FMaterialMaskHandle InHandle);
	CHAOS_API void OnCreateMaterialMask(Chaos::FMaterialMaskHandle InHandle);
	CHAOS_API void OnDestroyMaterialMask(Chaos::FMaterialMaskHandle InHandle);

	/** Gets a list of pending prerequisites required before proceeding with a solver update */
	CHAOS_API void GetSolverUpdatePrerequisites(FGraphEventArray& InPrerequisiteContainer);

private:

	/** Object that contains implementation of GetSolverActorClass() */
	IChaosSolverActorClassProvider* SolverActorClassProvider;

	/** Settings provider from external user */
	IChaosSettingsProvider* SettingsProvider;

	// Whether we actually spawned a physics task (distinct from whether we _should_ spawn it)
	bool bPersistentTaskSpawned;

	// The actually running tasks if running in a multi threaded configuration.
	FAsyncTask<Chaos::FPersistentPhysicsTask>* PhysicsAsyncTask;
	Chaos::FPersistentPhysicsTask* PhysicsInnerTask;

	// Core delegate signaling app shutdown, clean up and spin down threads before exit.
	FDelegateHandle PreExitHandle;

	// Flat list of every solver the module has created.
	TArray<Chaos::FPhysicsSolverBase*> AllSolvers;

	// Map of solver owners to the solvers they own. Nullptr is a valid 'owner' in that it's a map
	// key for unowned, standalone solvers
	TMap<const UObject*, TArray<Chaos::FPhysicsSolverBase*>> SolverMap;

	// Lock for the above list to ensure we don't delete solvers out from underneath other threads
	// or mess up the solvers array during use.
	mutable FCriticalSection SolverLock;

	/** Store the ChaosSolverActor type */
	UClass* SolverActorClass;

	/** SolverActorClass is require to be this class or a child thereof */
	UClass* SolverActorRequiredBaseClass;

	/** Reference to an in progress or completed task to run global and thread commands (a prerequisite for any solver ticking) */
	FGraphEventRef GlobalCommandTaskEventRef;

#if STATS
	// Stored stats from the physics thread
	float AverageUpdateTime;
	float TotalAverageUpdateTime;
	float Fps;
	float EffectiveFps;
#endif

#if WITH_EDITOR
	// Pause/Resume/Single-step thread safe booleans.
	FThreadSafeBool bPauseSolvers;
	FThreadSafeCounter SingleStepCounter;  // Counter that increments its value each time a single step is instructed.
#endif  // #if WITH_EDITOR

	// Whether we're initialized, gates work in Initialize() and Shutdown()
	bool bModuleInitialized;

	/** Event binding handles */
	FDelegateHandle OnCreateMaterialHandle;
	FDelegateHandle OnDestroyMaterialHandle;
	FDelegateHandle OnUpdateMaterialHandle;

	FDelegateHandle OnCreateMaterialMaskHandle;
	FDelegateHandle OnDestroyMaterialMaskHandle;
	FDelegateHandle OnUpdateMaterialMaskHandle;
};

struct FChaosScopeSolverLock
{
	FChaosScopeSolverLock()
	{
		FChaosSolversModule::GetModule()->LockSolvers();
	}

	~FChaosScopeSolverLock()
	{
		FChaosSolversModule::GetModule()->UnlockSolvers();
	}
};
