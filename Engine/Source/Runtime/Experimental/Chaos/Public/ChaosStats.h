// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/IConsoleManager.h"
#include "Stats/Stats.h"

DECLARE_STATS_GROUP(TEXT("Chaos"), STATGROUP_Chaos, STATCAT_Advanced);
DECLARE_STATS_GROUP_VERBOSE(TEXT("ChaosWide"), STATGROUP_ChaosWide, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("ChaosThread"), STATGROUP_ChaosThread, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("ChaosDedicated"), STATGROUP_ChaosDedicated, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("ChaosEngine"), STATGROUP_ChaosEngine, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("ChaosCollision"), STATGROUP_ChaosCollision, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("ChaosConstraintSolver"), STATGROUP_ChaosConstraintSolver, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("ChaosJoint"), STATGROUP_ChaosJoint, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("ChaosMinEvolution"), STATGROUP_ChaosMinEvolution, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("ChaosCounters"), STATGROUP_ChaosCounters, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("ChaosIterations"), STATGROUP_ChaosIterations, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("ChaosCollisionCounters"), STATGROUP_ChaosCollisionCounters, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("ChaosConstraintDetails"), STATGROUP_ChaosConstraintDetails, STATCAT_Advanced);
DECLARE_STATS_GROUP(TEXT("ChaosIslands"), STATGROUP_ChaosIslands, STATCAT_Advanced);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Physics Tick"), STAT_ChaosTick, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Physics Advance"), STAT_PhysicsAdvance, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Solver Advance"), STAT_SolverAdvance, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Handle Solver Commands"), STAT_HandleSolverCommands, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Integrate Solver"), STAT_IntegrateSolver, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sync Physics Proxies"), STAT_SyncProxies, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Handle Physics Commands"), STAT_PhysCommands, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Handle Task Commands"), STAT_TaskCommands, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Wait for previous global commands"), STAT_WaitGlobalCommands, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Kinematic Particle Update"), STAT_KinematicUpdate, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Begin Frame"), STAT_BeginFrame, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("End Frame"), STAT_EndFrame, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update Reverse Mapping"), STAT_UpdateReverseMapping, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collision Data Generation"), STAT_CollisionContactsCallback, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Breaking Data Generation"), STAT_BreakingCallback, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Trailing Data Generation"), STAT_TrailingCallback, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Geometry Collection Raycast"), STAT_GCRaycast, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Geometry Collection Overlap"), STAT_GCOverlap, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Geometry Collection Sweep"), STAT_GCSweep, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Geoemtry Collection Update Filter Data"), STAT_GCUpdateFilterData, STATGROUP_Chaos, CHAOS_API)
DECLARE_CYCLE_STAT_EXTERN(TEXT("Geometry Collection Component UpdateBounds"), STAT_GCCUpdateBounds, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Geometry Collection Component CalculateGlobalMatrices"), STAT_GCCUGlobalMatrices, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Geometry Collection Component OnPostPhysicsSync"), STAT_GCPostPhysicsSync, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Geometry Collection Component FullyDecayed event Broadcast"), STAT_GCFullyDecayedBroadcast, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Geometry Collection InitDynamicData"), STAT_GCInitDynamicData, STATGROUP_Chaos, CHAOS_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Geometry Collection Total Transforms"), STAT_GCTotalTransforms, STATGROUP_Chaos, CHAOS_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Geometry Collection Changed Transforms"), STAT_GCChangedTransforms, STATGROUP_Chaos, CHAOS_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Geometry Collection Replicated Clusters"), STAT_GCReplicatedClusters, STATGROUP_ChaosCounters, CHAOS_API);
DECLARE_DWORD_COUNTER_STAT_EXTERN(TEXT("Geometry Collection Replicated Fractures"), STAT_GCReplicatedFractures, STATGROUP_ChaosCounters, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Physics Lock Waits"), STAT_LockWaits, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Geometry Collection Begin Frame"), STAT_GeomBeginFrame, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Skeletal Mesh Update Anim"), STAT_SkelMeshUpdateAnim, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Dispatch Event Notifies"), STAT_DispatchEventNotifies, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Dispatch Collision Events"), STAT_DispatchCollisionEvents, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Dispatch Break Events"), STAT_DispatchBreakEvents, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Dispatch Crumbling Events"), STAT_DispatchCrumblingEvents, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("BufferPhysicsResults"), STAT_BufferPhysicsResults, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Flip Results"), STAT_FlipResults, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("ProcessDeferredCreatePhysicsState"), STAT_ProcessDeferredCreatePhysicsState, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("SQ - Update Materials"), STAT_SqUpdateMaterials, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[BufferPhysicsResults] - Geometry Collection"), STAT_CacheResultGeomCollection, STATGROUP_ChaosWide, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Update GC Views"), STAT_UpdateGeometryCollectionViews, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Particle Loop"), STAT_BufferPhysicsResultsParticleLoop, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Capture Solver Data"), STAT_CaptureSolverData, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[BufferPhysicsResults] - StaticMesh"), STAT_CacheResultStaticMesh, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Capture Disabled State"), STAT_CaptureDisabledState, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Calc Global Matrices"), STAT_CalcGlobalGCMatrices, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Calc Global Bounds"), STAT_CalcGlobalGCBounds, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Calc ParticleToWorld"), STAT_CalcParticleToWorld, STATGROUP_Chaos, CHAOS_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Create bodies"), STAT_CreateBodies, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Parameter Update"), STAT_UpdateParams, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Disable collisions"), STAT_DisableCollisions, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Evolution/Kinematic update and forces"), STAT_EvolutionAndKinematicUpdate, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("AdvanceOneTimestep Event Waits"), STAT_AdvanceEventWaits, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Gathering Event Data"), STAT_EventDataGathering, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Fill Event Producer Data"), STAT_FillProducerData, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Flip Event Buffer"), STAT_FlipBuffersIfRequired, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Gathering Collision Event Data"), STAT_GatherCollisionEvent, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Gathering Breaking Event Data"), STAT_GatherBreakingEvent, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Gathering Trailing Event Data"), STAT_GatherTrailingEvent, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Gathering Sleeping Event Data"), STAT_GatherSleepingEvent, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Acceleration Structure Reset"), STAT_AccelerationStructureReset, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Gathering Crumbling Event Data"), STAT_GatherCrumblingEvent, STATGROUP_Chaos, CHAOS_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Finalize Sim Callbacks"), STAT_FinalizeCallbacks, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Reset Clustering Events"), STAT_ResetClusteringEvents, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Rewind Finish Frame"), STAT_RewindFinishFrame, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Reset Marshalling Data"), STAT_ResetMarshallingData, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Conditional Apply Rewind"), STAT_ConditionalApplyRewind, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Finalize Pull Data"), STAT_FinalizePullData, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Destroy Pending Proxies"), STAT_DestroyPendingProxies, STATGROUP_Chaos, CHAOS_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Reset Collision Rule"), STAT_ResetCollisionRule, STATGROUP_Chaos, CHAOS_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Object Parameter Update"), STAT_ParamUpdateObject, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Field Parameter Update"), STAT_ParamUpdateField, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Sync Events - Game Thread"), STAT_SyncEvents_GameThread, STATGROUP_Chaos, CHAOS_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Proxies Set Compaction"), STAT_ProxiesSetCompaction, STATGROUP_Chaos, CHAOS_API);

DECLARE_CYCLE_STAT_EXTERN(TEXT("Physics Thread Stat Update"), STAT_PhysicsStatUpdate, STATGROUP_ChaosThread, CHAOS_API);
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Physics Thread Time Actual (ms)"), STAT_PhysicsThreadTime, STATGROUP_ChaosThread, CHAOS_API);
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Physics Thread Time Effective (ms)"), STAT_PhysicsThreadTimeEff, STATGROUP_ChaosThread, CHAOS_API);
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Physics Thread FPS Actual"), STAT_PhysicsThreadFps, STATGROUP_ChaosThread, CHAOS_API);
DECLARE_FLOAT_COUNTER_STAT_EXTERN(TEXT("Physics Thread FPS Effective"), STAT_PhysicsThreadFpsEff, STATGROUP_ChaosThread, CHAOS_API);

// Interface / scene stats
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Scene] - StartFrame"), STAT_Scene_StartFrame, STATGROUP_ChaosEngine, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Scene] - EndFrame"), STAT_Scene_EndFrame, STATGROUP_ChaosEngine, CHAOS_API);

// Field update stats
DECLARE_CYCLE_STAT_EXTERN(TEXT("Field System Object Parameter Update"), STAT_ParamUpdateField_Object, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Field System Object Force Update"), STAT_ForceUpdateField_Object, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Field System Object Niagara Update"), STAT_NiagaraUpdateField_Object, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] DynamicState"), STAT_ParamUpdateField_DynamicState, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] ActivateDisabled"), STAT_ParamUpdateField_ActivateDisabled, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] ExternalClusterStrain"), STAT_ParamUpdateField_ExternalClusterStrain, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] Kill"), STAT_ParamUpdateField_Kill, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] LinearVelocity"), STAT_ParamUpdateField_LinearVelocity, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] AngularVelocity"), STAT_ParamUpdateField_AngularVelocity, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] SleepingThreshold"), STAT_ParamUpdateField_SleepingThreshold, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] DisableThreshold"), STAT_ParamUpdateField_DisableThreshold, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] InternalClusterStrain"), STAT_ParamUpdateField_InternalClusterStrain, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] CollisionGroup"), STAT_ParamUpdateField_CollisionGroup, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] PositionStatic"), STAT_ParamUpdateField_PositionStatic, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] PositionTarget"), STAT_ParamUpdateField_PositionTarget, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] PositionAnimated"), STAT_ParamUpdateField_PositionAnimated, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] DynamicConstraint"), STAT_ParamUpdateField_DynamicConstraint, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] LinearForce"), STAT_ForceUpdateField_LinearForce, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] AngularTorque"), STAT_ForceUpdateField_AngularTorque, STATGROUP_Chaos, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("[Field Update] Impulse"), STAT_ForceUpdateField_LinearImpulse, STATGROUP_Chaos, CHAOS_API);

// Collision Detection
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collisions::Detect"), STAT_Collisions_Detect, STATGROUP_ChaosCollision, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collisions::BroadPhase::ParticlePair"), STAT_Collisions_ParticlePairBroadPhase, STATGROUP_ChaosCollision, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collisions::BroadPhase::Spatial"), STAT_Collisions_SpatialBroadPhase, STATGROUP_ChaosCollision, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collisions::MidPhase"), STAT_Collisions_BroadPhase, STATGROUP_ChaosCollision, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collisions::AssignMidPhases"), STAT_Collisions_AssignMidPhases, STATGROUP_ChaosCollision, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collisions::NarrowPhase"), STAT_Collisions_NarrowPhase, STATGROUP_ChaosCollision, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collisions::GenerateCollisions"), STAT_Collisions_GenerateCollisions, STATGROUP_ChaosCollision, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collisions::Gather"), STAT_Collisions_Gather, STATGROUP_ChaosCollision, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collisions::Scatter"), STAT_Collisions_Scatter, STATGROUP_ChaosCollision, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collisions::Apply"), STAT_Collisions_Apply, STATGROUP_ChaosCollision, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collisions::ApplyPushOut"), STAT_Collisions_ApplyPushOut, STATGROUP_ChaosCollision, CHAOS_API);
DECLARE_CYCLE_STAT_EXTERN(TEXT("Collisions::SimplifyConvexes"), STAT_Collisions_SimplifyConvexes, STATGROUP_ChaosCollision, CHAOS_API);


#if 0
#define PHYSICS_CSV_SCOPED_EXPENSIVE(Category, Name) CSV_SCOPED_TIMING_STAT(Category, Name)
#define PHYSICS_CSV_CUSTOM_EXPENSIVE(Category, Name, Value, Op) CSV_CUSTOM_STAT(Category, Name, Value, Op)
#else
#define PHYSICS_CSV_SCOPED_EXPENSIVE(Category, Name) 
#define PHYSICS_CSV_CUSTOM_EXPENSIVE(Category, Name, Value, Op) 
#endif

#if 0
#define PHYSICS_CSV_SCOPED_VERY_EXPENSIVE(Category, Name) CSV_SCOPED_TIMING_STAT(Category, Name)
#define PHYSICS_CSV_CUSTOM_VERY_EXPENSIVE(Category, Name, Value, Op) CSV_CUSTOM_STAT(Category, Name, Value, Op)
#else
#define PHYSICS_CSV_SCOPED_VERY_EXPENSIVE(Category, Name) 
#define PHYSICS_CSV_CUSTOM_VERY_EXPENSIVE(Category, Name, Value, Op) 
#endif