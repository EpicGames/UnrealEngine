// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDOptionalDataChannel.h"
#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "HAL/ThreadSingleton.h"
#include "Templates/SharedPointer.h"

#if WITH_CHAOS_VISUAL_DEBUGGER

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "ChaosVDRuntimeModule.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

enum class EChaosVDContextType : int32
{
	Any,
	Solver,
	Query,
	SubTraceQuery,
};

enum class EChaosVDContextAttributes : int32
{
	None = 0,
	Resimulated = 1 << 0
};
ENUM_CLASS_FLAGS(EChaosVDContextAttributes)

/** Chaos Visual Debugger data used to context for logging or debugging purposes */
struct FChaosVDContext
{
	int32 OwnerID = INDEX_NONE;
	int32 Id = INDEX_NONE;
	int32 Type = INDEX_NONE;
	int32 Attributes = 0;

	FORCEINLINE void SetDataChannel(const TWeakPtr<Chaos::VisualDebugger::FChaosVDOptionalDataChannel>& NewDataChannel)
	{
		CurrentDataChannel = NewDataChannel;
	}

	FORCEINLINE bool IsDataChannelEnabled() const
	{
		if (TSharedPtr<Chaos::VisualDebugger::FChaosVDOptionalDataChannel> LockedCurrentDataChannel = CurrentDataChannel.Pin())
		{
			return LockedCurrentDataChannel->IsChannelEnabled();
		}
		else
		{
			return false;
		}
	}

private:
	TWeakPtr<Chaos::VisualDebugger::FChaosVDOptionalDataChannel> CurrentDataChannel = CVDDC_Default;
};

UE_DECLARE_THREAD_SINGLETON_TLS(class FChaosVDThreadContext, CHAOS_API)

/** Singleton class that manages the thread local storage used to store CVD Context data */
class FChaosVDThreadContext : public TThreadSingleton<FChaosVDThreadContext>
{
public:

	CHAOS_API virtual ~FChaosVDThreadContext() override;

	/** Copies the Current CVD context data into the provided struct
	 * @return true if the copy was successful
	 */
	CHAOS_API bool GetCurrentContext(FChaosVDContext& OutContext);
	
	/** Gets the current CVD context data -
	 * Don't use of a function that will recursively push new context data as it might invalidate the pointer
	 * @param Type type of the context we want to get
	 * @return Ptr to the Current CVD context data if it matches the provided type
	 */
	CHAOS_API const FChaosVDContext* GetCurrentContext(EChaosVDContextType Type);
	
	/** Gets the current CVD context data -
	 * Don't use of a function that will recursively push new context data as it might invalidate the pointer
	 * @return Ptr to the Current CVD context data
	 */
	CHAOS_API const FChaosVDContext* GetCurrentContext();
	
	/** Pushed a new CVD Context data to the local cvd context stack */
	CHAOS_API void PushContext(const FChaosVDContext& InContext);
	
	/** Removes the CVD Context data at the top of the local cvd context stack */
	CHAOS_API void PopContext();

protected:
	TArray<FChaosVDContext, TInlineAllocator<16>> LocalContextStack;

	CHAOS_API TArray<uint8>& GetTLSDataBufferRef();

	FDelegateHandle RecordingStoppedHandle;

	int32 BufferAccessCounter = 0;
	TArray<uint8> CVDDataBuffer;

	friend struct FChaosVDScopedTLSBufferAccessor;
};

namespace Chaos::VisualDebugger::Utils
{
	FORCEINLINE bool IsContextEnabledAndValid(const FChaosVDContext* Context)
	{
		if (!ensure(Context))
		{
			return false;
		}

		return Context->IsDataChannelEnabled();
	}
}

/** Utility Class that will push the provided CVD Context Data to the local thread storage
 * and remove it when it goes out of scope
 */
struct FChaosVDScopeContext
{
	FChaosVDScopeContext(const FChaosVDContext& InCVDContext)
	{
		FChaosVDThreadContext::Get().PushContext(InCVDContext);
	}

	~FChaosVDScopeContext()
	{
		FChaosVDThreadContext::Get().PopContext();
	}
};

struct FChaosCVDScopedDataChannelOverride
{
	FChaosCVDScopedDataChannelOverride(const TWeakPtr<Chaos::VisualDebugger::FChaosVDOptionalDataChannel>& NewDataChannel)
	{
		if (const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext())
		{
			FChaosVDContext NewContext = *CVDContextData;
			NewContext.SetDataChannel(NewDataChannel);

			FChaosVDThreadContext::Get().PushContext(MoveTemp(NewContext));
		}
	}

	~FChaosCVDScopedDataChannelOverride()
	{
		FChaosVDThreadContext::Get().PopContext();
	}
};

/** Wrapper class that provides access to the data buffer that should be used to trace CVD (Chaos Visual Debugger) data.
 * It ensures the buffer is cleared on scope exit but keeping a minimum amount of memory allocated
 */
struct FChaosVDScopedTLSBufferAccessor
{
	FChaosVDScopedTLSBufferAccessor() : BufferRef(FChaosVDThreadContext::Get().GetTLSDataBufferRef())
	{
		FChaosVDThreadContext::Get().BufferAccessCounter++;

		// This is close to the max amount of data we can Trace as a single trace event
		// Some times we will go over and some times this be more than we need
		// but keeping this amount allocated on average reduces the allocation cost when tracing.
		// TODO: There might be a better value for this, we need to do more testing
		constexpr uint32 MaxTraceChunkSize = TNumericLimits<uint16>::Max();
		BufferRef.Reserve(MaxTraceChunkSize);
	}

	~FChaosVDScopedTLSBufferAccessor()
	{
		// Clear the buffer on exit but maintain allocated a minimum amount of memory
		constexpr uint32 MaxTraceChunkSize = TNumericLimits<uint16>::Max();
		BufferRef.Reset(MaxTraceChunkSize);

		FChaosVDThreadContext::Get().BufferAccessCounter--;
	}
	
	TArray<uint8>& BufferRef;
};

#ifndef CVD_GET_CURRENT_CONTEXT
	#define CVD_GET_CURRENT_CONTEXT(OutContext) \
		FChaosVDThreadContext::Get().GetCurrentContext(OutContext);
#endif

#ifndef CVD_GET_WRAPPED_CURRENT_CONTEXT
	#define CVD_GET_WRAPPED_CURRENT_CONTEXT(OutWrappedContext) \
		FChaosVDContext CurrentContext; \
		FChaosVDThreadContext::Get().GetCurrentContext(CurrentContext); \
		OutWrappedContext = FChaosVDContextWrapper(CurrentContext);
#endif

#ifndef CVD_SCOPE_CONTEXT
	#define CVD_SCOPE_CONTEXT(InContext) \
		FChaosVDScopeContext CVDScope(InContext);
#endif

#ifndef CVD_SCOPED_DATA_CHANNEL_OVERRIDE
	#define CVD_SCOPED_DATA_CHANNEL_OVERRIDE(DataChannel) \
	FChaosCVDScopedDataChannelOverride CVDDC_Scope_Override##DataChannel(DataChannel);
#endif

#else // WITH_CHAOS_VISUAL_DEBUGGER

#ifndef CVD_GET_CURRENT_CONTEXT
	#define CVD_GET_CURRENT_CONTEXT(OutContext)
#endif

#ifndef CVD_SCOPE_CONTEXT
	#define CVD_SCOPE_CONTEXT(InContext)
#endif

#ifndef CVD_GET_WRAPPED_CURRENT_CONTEXT
	#define CVD_GET_WRAPPED_CURRENT_CONTEXT(OutWrappedContext)
#endif

#ifndef CVD_SCOPED_CONTEXT_DATA_CHANNEL_OVERRIDE
	#define CVD_SCOPED_CONTEXT_DATA_CHANNEL_OVERRIDE(DataChannel)
#endif

#endif // WITH_CHAOS_VISUAL_DEBUGGER

struct FChaosVDContextWrapper
{
	FChaosVDContextWrapper()
	{
	}
	
#if WITH_CHAOS_VISUAL_DEBUGGER
	FChaosVDContextWrapper(const FChaosVDContext& InContext)
	: Context(InContext)
	{
	}

	FChaosVDContext Context;
#endif
};
