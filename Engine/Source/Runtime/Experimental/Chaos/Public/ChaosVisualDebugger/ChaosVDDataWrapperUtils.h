// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/AABBTree.h"
#include "Chaos/Collision/ContactPoint.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/ShapeInstanceFwd.h"
#include "DataWrappers/ChaosVDCollisionDataWrappers.h"
#include "DataWrappers/ChaosVDAccelerationStructureDataWrappers.h"
#include "Chaos/ISpatialAccelerationCollection.h"

struct FChaosVDJointSettingsDataWrapper;

namespace Chaos
{
class FJointConstraint;
class FSolverSerializer;
}

struct FChaosVDAABBTreeDataWrapper;
struct FChaosVDCharacterGroundConstraint;
struct FChaosVDJointConstraint;

namespace Chaos
{
	class FCharacterGroundConstraintHandle;
	class FPBDJointConstraintHandle;

	namespace VisualDebugger
	{
		class FChaosVDSerializableNameTable;
	}

	class FParticlePairMidPhase;
	class FPBDCollisionConstraint;
	class FPBDCollisionConstraintMaterial;
}

class FChaosVisualDebuggerTrace;
struct FChaosVDCollisionMaterial;
struct FChaosVDConstraint;
struct FChaosVDParticlePairMidPhase;
struct FChaosVDParticleDataWrapper;

namespace Chaos::VisualDebugger::Utils
{
	template<typename InType,typename OutType, int32 Size, typename TransformT>
	void TransformStaticArray(const InType (&In)[Size], OutType (&Out)[Size], TransformT Trans)
	{
		for (int32 Index = 0; Index < Size; Index++)
		{
			Out[Index] = Invoke(Trans, In[Index]);
		}
	}

	template<typename InType,typename OutType, int32 Size>
	void CopyStaticArray(const InType (&In)[Size], OutType (&Out)[Size])
	{
		FMemory::Memcpy(Out, In, Size * sizeof(InType));
	}

	inline FTransform ConvertToFTransform(const FRigidTransform3& InChaosTransform)
	{
		return InChaosTransform;
	}
}

/**
 * Helper class used to build Chaos Visual Debugger data wrappers, without directly referencing chaos' types in them.
 *
 * @note: This is needed for now because we want to keep the data wrapper structs/classes on a separate module where possible, but if we reference Chaos's types
 * directly we will end up with a circular dependency issue because the ChaosVDRuntime module will need the Chaos module but the Chaos module will need the ChaosVDRuntime module to use the structs
 * Once development is done and can we commit to backward compatibility, this helper class might go away (trough the proper deprecation process)
 */
class FChaosVDDataWrapperUtils
{
private:

	/** Takes a FManifoldPoint and copies the relevant data to the CVD counterpart */
	static void CopyManifoldPointsToDataWrapper(const Chaos::FManifoldPoint& InCopyFrom, FChaosVDManifoldPoint& OutCopyTo);

	/** Takes a FManifoldPointResult and copies the relevant data to the CVD counterpart */
	static void CopyManifoldPointResultsToDataWrapper(const Chaos::FManifoldPointResult& InCopyFrom, FChaosVDManifoldPoint& OutCopyTo);

	/** Takes a FPBDCollisionConstraintMaterial and copies the relevant data to the CVD counterpart */
	static void CopyCollisionMaterialToDataWrapper(const Chaos::FPBDCollisionConstraintMaterial& InCopyFrom, FChaosVDCollisionMaterial& OutCopyTo);
public:
	/** Creates and populates a FChaosVDParticleDataWrapper with the data of the provided FGeometryParticleHandle */
	UE_INTERNAL static FChaosVDParticleDataWrapper BuildParticleDataWrapperFromParticle(const Chaos::FGeometryParticleHandle* ParticleHandlePtr);
	UE_INTERNAL static FChaosVDParticleDataWrapper BuildParticleDataWrapperFromGTParticle(const Chaos::FGeometryParticle* GTParticle);

private:
	/** Creates and populates a FChaosVDConstraint with the data of the provided FPBDCollisionConstraint */
	static FChaosVDConstraint BuildConstraintDataWrapperFromConstraint(const Chaos::FPBDCollisionConstraint& InConstraint);

	/** Creates and populates a FChaosVDParticlePairMidPhase with the data of the provided FParticlePairMidPhase */
	static FChaosVDParticlePairMidPhase BuildMidPhaseDataWrapperFromMidPhase(const Chaos::FParticlePairMidPhase& InMidPhase);
	static void CopyJointConstraintSettingsToWrapper(const Chaos::FPBDJointSettings& InSettings, FChaosVDJointSettingsDataWrapper& InOutWrappedJointSettingsData);

public:
	/** Creates and populates a FChaosVDJointConstraint with the data of the provided FPBDJointConstraintHandle */
	UE_INTERNAL static FChaosVDJointConstraint BuildJointDataWrapper(const Chaos::FPBDJointConstraintHandle* ConstaintHanlde);
	UE_INTERNAL static FChaosVDJointConstraint BuildGTJointDataWrapper(Chaos::FJointConstraint* GTConstraintPtr);

	UE_INTERNAL static void ApplyJointDataWrapperToHandle(Chaos::FPBDJointConstraintHandle* ConstaintHanlde, const FChaosVDJointConstraint& WrappedJointData);
	UE_INTERNAL static void ApplyJointDataWrapperGTConstraint(Chaos::FJointConstraint* ConstraintPtr, const FChaosVDJointConstraint& InWrappedJointData);

	/** Creates and populates a FChaosVDChartacterGroundConstraint with the data of the provided FCharacterGroundConstraintHandle */
	UE_INTERNAL static FChaosVDCharacterGroundConstraint BuildCharacterGroundConstraintDataWrapper(const Chaos::FCharacterGroundConstraintHandle* ConstaintHanlde);
private:

	static void CopyJointSettingsFromWrapper(const FChaosVDJointSettingsDataWrapper& InWrappedJointSettingsData, Chaos::FPBDJointSettings& NewSettings);

	/** Converts a Chaos::FVec3 to a FVector. It is worth notice that FVector is double precision and FVec3 is single */
	static FVector ConvertToFVector(const Chaos::FVec3f& VectorRef) { return FVector(VectorRef); }

	static void CopyShapeDataToWrapper(const Chaos::FShapeInstancePtr& ShapeDataPtr, FChaosVDShapeCollisionData& OutCopyTo);
	
	template<typename LeafType>
	static void BuildDataWrapperFromAABBStructure(const Chaos::TAABBTree<Chaos::FAccelerationStructureHandle, LeafType>& AABBTree, FChaosVDAABBTreeDataWrapper& OutAABBTreeWrapper);

	/** Converts a Chaos::FVec3 to a FVector. It is worth notice that FVector is double precision and FVec3 is single */
	static FBox ConvertToFBox(const Chaos::TAABB<Chaos::FReal, 3>& Bounds)
	{
		FBox ConvertedBox(Bounds.Min(), Bounds.Max());
		return ConvertedBox;
	}

	static void BuildDataWrapperFromAABBStructure(const Chaos::ISpatialAccelerationCollection<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>*, int32 OwnerSolverID, TArray<FChaosVDAABBTreeDataWrapper>& OutAABBTrees);

	static void AddTreeLeaves(const TConstArrayView<Chaos::TAABBTreeLeafArray<Chaos::FAccelerationStructureHandle>>& LeavesContainer, FChaosVDAABBTreeDataWrapper& InOutAABBTreeWrapper);
	static void AddTreeLeaves(const TConstArrayView<Chaos::TBoundingVolume<Chaos::FAccelerationStructureHandle>>& LeavesContainer, FChaosVDAABBTreeDataWrapper& InOutAABBTreeWrapper);

	friend FChaosVisualDebuggerTrace;
};

template<typename LeafType>
void FChaosVDDataWrapperUtils::BuildDataWrapperFromAABBStructure(const Chaos::TAABBTree<Chaos::FAccelerationStructureHandle, LeafType>& AABBTree, FChaosVDAABBTreeDataWrapper& OutAABBTreeWrapper)
{
	using namespace Chaos;

	OutAABBTreeWrapper.MaxTreeDepth = AABBTree.MaxTreeDepth;
	OutAABBTreeWrapper.MaxChildrenInLeaf = AABBTree.MaxTreeDepth;
	OutAABBTreeWrapper.MaxPayloadBounds = AABBTree.MaxPayloadBounds;
	OutAABBTreeWrapper.RootNodeIndex = AABBTree.RootNode;
	OutAABBTreeWrapper.bDynamicTree = AABBTree.bDynamicTree;

	const int32 NodesToCopyNum = AABBTree.Nodes.Num();
	OutAABBTreeWrapper.Nodes.Reserve(NodesToCopyNum);
	OutAABBTreeWrapper.NodesNum = NodesToCopyNum;

	OutAABBTreeWrapper.Type = static_cast<EChaosVDAccelerationStructureType>(AABBTree.StaticType);
		
	for (const typename TAABBTree<FAccelerationStructureHandle, LeafType>::FNode& Node : AABBTree.Nodes)
	{
		FChaosVDAABBTreeNodeDataWrapper CVDNode;
		CVDNode.bLeaf = Node.bLeaf;
		CVDNode.bDirtyNode = Node.bDirtyNode;

		VisualDebugger::Utils::TransformStaticArray(Node.ChildrenBounds, CVDNode.ChildrenBounds, &FChaosVDDataWrapperUtils::ConvertToFBox);
		VisualDebugger::Utils::CopyStaticArray(Node.ChildrenNodes, CVDNode.ChildrenNodes);

		CVDNode.ParentNode =  Node.ParentNode;
		CVDNode.MarkAsValid();

		OutAABBTreeWrapper.Nodes.Emplace(CVDNode);
	}
	
	AddTreeLeaves(MakeArrayView(AABBTree.Leaves.GetData(), AABBTree.Leaves.Num()), OutAABBTreeWrapper);
	
	OutAABBTreeWrapper.MarkAsValid();
}
