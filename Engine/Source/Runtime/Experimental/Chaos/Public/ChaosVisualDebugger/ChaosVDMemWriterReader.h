// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDSerializedNameTable.h"
#include "Chaos/ChaosArchive.h"
#include "HAL/Platform.h"
#include "Misc/EngineVersion.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/SharedPointer.h"

class FName;
class FArchive;

namespace Chaos::VisualDebugger
{
	struct FChaosVDArchiveHeader
	{
		CHAOS_API static FStringView WrapperTypeName;

		FChaosVDArchiveHeader()
		{
		}

		/** Contains version and branch info from where it was saved */
		FEngineVersion EngineVersion;

		/** Custom versions */
		FCustomVersionContainer CustomVersionContainer;

		/** Serialization function, returns false if the archive ends up in an error state */
		CHAOS_API bool Serialize(FArchive& Ar);

		/** Returns a header that matches current version */
		CHAOS_API static FChaosVDArchiveHeader Current();
	};

	FORCEINLINE void NameTableFNameSerializer(FArchive& Ar ,FName& Name, const TSharedRef<FChaosVDSerializableNameTable>& InNameTable)
	{
		// Serialize the FName as a CVD Name ID
		if (Ar.IsLoading())
		{
			uint64 NameID = 0;
			Ar << NameID;
				
			Name = InNameTable->GetNameFromTable(NameID);
		}
		else
		{
			uint64 NameID = InNameTable->AddNameToTable(Name);
			Ar << NameID;
		}
	}

	/** Preferred memory writer for CVD recorded structs. It has support for FName serialization and de-duplication */
	class FChaosVDMemoryWriter final : public FMemoryWriter
	{
	public:
		FChaosVDMemoryWriter(TArray<uint8>& InBytes, const TSharedRef<FChaosVDSerializableNameTable>& InNameTableInstance)
			: FMemoryWriter(InBytes), NameTableInstance(InNameTableInstance)
		{
		}

		virtual FArchive& operator<<(FName& Name) override
		{
			NameTableFNameSerializer(*this, Name, NameTableInstance);
			return *this;
		}

		TSharedRef<FChaosVDSerializableNameTable> NameTableInstance;
	};

	/** Preferred memory reader for CVD recorded structs. It has support for FName serialization and de-duplication */
	class FChaosVDMemoryReader final : public FMemoryReader
	{
	public:
		FChaosVDMemoryReader(const TArray<uint8>& InBytes, const TSharedRef<FChaosVDSerializableNameTable>& InNameTableInstance)
			: FMemoryReader(InBytes), NameTableInstance(InNameTableInstance)
		{
		}

		virtual FArchive& operator<<(FName& Name) override
		{
			NameTableFNameSerializer(*this, Name, NameTableInstance);
			return *this;
		}

		TSharedRef<FChaosVDSerializableNameTable> NameTableInstance;
	};

	/** Version of Chaos Archive that allows CVD to provide its own shared context instance */
	class FChaosVDSharedContextArchive : public FChaosArchive
	{
	public:
		explicit FChaosVDSharedContextArchive(FArchive& ArIn, const TSharedRef<FChaosArchiveContext>& InExternalSharedContext)
			: FChaosArchive(ArIn, InExternalSharedContext)
		{
		}
	};
}
