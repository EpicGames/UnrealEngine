// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_CHAOS_VISUAL_DEBUGGER

#include "Templates/SharedPointer.h"
#include "Delegates/Delegate.h"

class FText;
class FName;

/** Set of flags to control how a CVD Data Channel gets initialized*/
enum class EChaosVDDataChannelInitializationFlags : uint8
{
	/** If Set, the data channel will start in an enabled state */
	StartEnabled = 1 << 0,
	/** If Set, the data channel can be toggled on and off as desired */
	CanChangeEnabledState = 1 << 1
};
ENUM_CLASS_FLAGS(EChaosVDDataChannelInitializationFlags)

namespace Chaos::VisualDebugger
{
	/** Structure holding the state of a CVD data channel. Used to enabled and disable recording of specific categories of data */
	struct FChaosVDOptionalDataChannel : TSharedFromThis<FChaosVDOptionalDataChannel>
	{
		explicit FChaosVDOptionalDataChannel(const TSharedRef<FName>& InChannelID, const TSharedRef<FText>& InDisplayName, EChaosVDDataChannelInitializationFlags InitializationFlags)
				: LocalizableChannelName(InDisplayName), ChannelId(InChannelID)
		{
			bIsEnabled = EnumHasAnyFlags(InitializationFlags, EChaosVDDataChannelInitializationFlags::StartEnabled);
			bCanChangeEnabledState = EnumHasAnyFlags(InitializationFlags, EChaosVDDataChannelInitializationFlags::CanChangeEnabledState);
		}

		virtual ~FChaosVDOptionalDataChannel();

		/** Registers itself with the optional channel data manager */
		void Initialize();

		/** The localizable display name that will be used for this channel in UI elements */
		CHAOS_API const FText& GetDisplayName() const;

		/** A name used as ID to find this changed using the CVD Channel Manager */
		FName GetId() const
		{
			return *ChannelId;
		}

		/** Returns true if this data channel is enabled */
		FORCEINLINE bool IsChannelEnabled() const 
		{ 
			return bIsEnabled;
		}

		/** Enables or disabled this Data Channel */
		CHAOS_API void SetChannelEnabled(bool bNewEnabled);

		/** Returns true if the enabled state of this channel can be changed */
		bool CanChangeEnabledState() const 
		{ 
			return bCanChangeEnabledState;
		}

	private:
		TSharedRef<FText> LocalizableChannelName;
		TSharedRef<FName> ChannelId;
		std::atomic<bool> bIsEnabled = true;
		bool bCanChangeEnabledState = false;
	};

	/** Manager that provides a way to iterate through all existing CVD Data channels or get a specific one */
	class FChaosVDDataChannelsManager
	{
	public:

		static CHAOS_API FChaosVDDataChannelsManager& Get();

		/** Iterates through all the available data channels and executes the provided callback passing each channel as an argument.
		 * If the callback returns false, the iteration will be stopped.
		 */
		template<typename TCallback>
		UE_DEPRECATED(5.9, "please use EnumerateDataChannels instead")
		void EnumerateChannels(TCallback Callback)
		{
			for (const TPair<FName, TWeakPtr<FChaosVDOptionalDataChannel>>& ChannelWithName : AvailableChannels)
			{
				if (TSharedPtr<FChaosVDOptionalDataChannel> LockedChannel = ChannelWithName.Value.Pin())
				{
					if (!Callback(LockedChannel))
					{
						return;
					}
				}
			}
		}

		/** Iterates through all the available data channels and executes the provided callback passing each channel as an argument.
		 * If the callback returns false, the iteration will be stopped.
		 */
		void EnumerateDataChannels(TFunction<bool(const TWeakPtr<FChaosVDOptionalDataChannel>&)> Callback)
		{
			for (const TPair<FName, TWeakPtr<FChaosVDOptionalDataChannel>>& ChannelWithName : AvailableChannels)
			{
				if (!Callback(ChannelWithName.Value))
				{
					return;
				}
			}
		}

		/** Returns the CVD Data Channel instance for the provided ID*/
		TWeakPtr<FChaosVDOptionalDataChannel> GetChannelById(FName ChannelId)
		{
			if (TWeakPtr<FChaosVDOptionalDataChannel>* ChannelFound = AvailableChannels.Find(ChannelId))
			{
				return *ChannelFound;
			}

			return nullptr;
		}

	private:

		DECLARE_MULTICAST_DELEGATE_OneParam(FChannelStateChangedDelegate, TWeakPtr<FChaosVDOptionalDataChannel> DataChannelChanged);
		FChannelStateChangedDelegate ChannelStateChangedDelegate;

	public:
		FChannelStateChangedDelegate& OnChannelStateChanged()
		{
			return ChannelStateChangedDelegate;
		}
	private:
		
		/** Registers a CVD Data channel instance - Do not call directly */
		void RegisterChannel(const TWeakPtr<FChaosVDOptionalDataChannel>& NewChannel)
		{
			if (TSharedPtr<FChaosVDOptionalDataChannel> LockedNewChannel = NewChannel.Pin())
			{
				AvailableChannels.Add(LockedNewChannel->GetId(), NewChannel);
			}
		}
		
		/** Remove a CVD Data channel instance - Do not call directly */
		void UnregisterChannel(FName ChannelId)
		{
			AvailableChannels.Remove(ChannelId);
		}

		TMap<FName, TWeakPtr<FChaosVDOptionalDataChannel>> AvailableChannels;

		friend FChaosVDOptionalDataChannel;
	};

	void ParseChannelListFromCommandArgument(TArray<FString>& OutParsedChannelList, const FString& InCommandArgument);

	/** Creates a CVD Data Channel Instance - Only to be used by the CVD Macros */
	UE_DEPRECATED(5.9, "please use CreateDataChannelShared instead to keep the lifetime of the channel explicit");
	CHAOS_API TSharedRef<FChaosVDOptionalDataChannel> CreateDataChannel(FName InChannelID, const TSharedRef<FText>& InDisplayName, EChaosVDDataChannelInitializationFlags InitializationFlags);

	/** Creates a CVD Data Channel Instance - Only to be used by the CVD Macros */
	CHAOS_API TSharedPtr<FChaosVDOptionalDataChannel> CreateDataChannelShared(FName InChannelID, const TSharedRef<FText>& InDisplayName, EChaosVDDataChannelInitializationFlags InitializationFlags);
}

#endif //WITH_CHAOS_VISUAL_DEBUGGER

#ifndef CVD_CONCAT_NX
	#define CVD_CONCAT_NX(A, B) A ## B
#endif

#ifndef CVD_CONCAT
	#define CVD_CONCAT(A, B) CVD_CONCAT_NX(A,B)
#endif

#ifndef CVD_STRINGIZE_NX
	#define CVD_STRINGIZE_NX(A) #A
#endif

#ifndef CVD_STRINGIZE
	#define CVD_STRINGIZE(A) CVD_STRINGIZE_NX(A)
#endif

#if WITH_CHAOS_VISUAL_DEBUGGER

/** Declares an Optional CVD Data channel to be available globally. The data channel can be accessed by using CVDDC_TheNameOfTheChannelUsedWithThisMacro */
#ifndef CVD_DECLARE_OPTIONAL_DATA_CHANNEL
	#define CVD_DECLARE_OPTIONAL_DATA_CHANNEL(DataChannelName) \
			extern CHAOS_API TSharedPtr<Chaos::VisualDebugger::FChaosVDOptionalDataChannel> CVDDC_##DataChannelName;
#endif

/** Declares an Optional CVD Data channel outside the Chaos Module to be available globally. The data channel can be accessed by using CVDDC_TheNameOfTheChannelUsedWithThisMacro */
#ifndef CVD_DECLARE_OPTIONAL_DATA_CHANNEL_EXTERN
	#define CVD_DECLARE_OPTIONAL_DATA_CHANNEL_EXTERN(DataChannelName, ModuleAPI) \
		extern ModuleAPI TSharedPtr<Chaos::VisualDebugger::FChaosVDOptionalDataChannel> CVDDC_##DataChannelName;
#endif

/** Defines and initializes an Optional CVD Data Channel */
#ifndef CVD_DEFINE_OPTIONAL_DATA_CHANNEL
	#define CVD_DEFINE_OPTIONAL_DATA_CHANNEL(DataChannelName, InitializationFlags) \
			TSharedPtr<Chaos::VisualDebugger::FChaosVDOptionalDataChannel> CVDDC_##DataChannelName = Chaos::VisualDebugger::CreateDataChannel(#DataChannelName, MakeShared<FText>(NSLOCTEXT(CVD_STRINGIZE(ChaosVisualDebugger), CVD_STRINGIZE(CVD_CONCAT(DataChannelName,_ChannelName)), #DataChannelName)), InitializationFlags);
#endif


// Declare CVD's Default set of channels
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(Default);
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(EvolutionStart);							// Initial particle positions for this tick. Same as end of previous EvolutionEnd but with any external changes applied (set position, velocity, etc).
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(PostIntegrate);							// Predicted particle positions before user callbacks and constraint solver. Includes this tick's movement from external forces, velocity, kinematic targets, etc.
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(PreConstraintSolve);						// Predicted particle positions as used by the constraint solver. After user callbacks, collision modifiers, etc., but before collisions and joints are resolved.
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(CollisionDetectionBroadPhase);
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(CollisionDetectionNarrowPhase);
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(EndOfEvolutionCollisionConstraints);
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(PostConstraintSolve);						// Particle positions corrected by the constraint solver (collisions, joints, etc.), but before user callbacks or destruction handling.
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(EvolutionEnd);							// Final particle positions, including destruction and user callbacks.
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(SceneQueries);
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(JointLinearConstraints);
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(JointNonLinearConstraints);
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(CharacterGroundConstraints);
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(AccelerationStructures);

// When a particle is added to a cluster will get disabled and therefore not updated by CVD's delta frames implementation.
// If this channel is enabled, we will record all child particles of any active dirty cluster particle
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(ClusterParticlesChildData);

CVD_DECLARE_OPTIONAL_DATA_CHANNEL(CameraPaths);

// Channel for logging more detailed steps related to resim
CVD_DECLARE_OPTIONAL_DATA_CHANNEL(ResimDetails)

#else  //WITH_CHAOS_VISUAL_DEBUGGER

/** Declares an Optional CVD Data channel to be available globally. The data channel can be accessed by using CVDDC_TheNameOfTheChannelUsedWithThisMacro */
#ifndef CVD_DECLARE_OPTIONAL_DATA_CHANNEL
	#define CVD_DECLARE_OPTIONAL_DATA_CHANNEL(DataChannelName)
#endif

/** Declares an Optional CVD Data channel outside the Chaos Module to be available globally. The data channel can be accessed by using CVDDC_TheNameOfTheChannelUsedWithThisMacro */
#ifndef CVD_DECLARE_OPTIONAL_DATA_CHANNEL_EXTERN
	#define CVD_DECLARE_OPTIONAL_DATA_CHANNEL_EXTERN(DataChannelName, ModuleAPI)
#endif

/** Defines and initializes an Optional CVD Data Channel */
#ifndef CVD_DEFINE_OPTIONAL_DATA_CHANNEL
	#define CVD_DEFINE_OPTIONAL_DATA_CHANNEL(DataChannelName, InitializationFlags)
#endif

#endif  //WITH_CHAOS_VISUAL_DEBUGGER
