// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class FArchive;
class UScriptStruct;

namespace Chaos::VisualDebugger
{
	class FChaosVDSerializableNameTable;

	/** Controls how the struct content bytes inside FChaosVDExtraDataStructEntry were written.
	 *  Recorded in the trace so the reader can apply the symmetric deserialization. */
	enum class EChaosVDExtraDataSerializationMode : uint8
	{
		/** Default. Uses UStruct::SerializeBin — property-tagged, handles field additions/removals gracefully. */
		SerializeBin = 0,
		/** Opt-in. Uses UScriptStruct::SerializeItem — invokes native Serialize if present.
		 *  More expressive but potentially binary-incompatible across builds. */
		NativeSerialization = 1,
	};

	/** A single serialized struct entry: the reflected type path and the struct's serialized bytes. */
	struct FChaosVDExtraDataStructEntry
	{
		/** Full path of the UScriptStruct, e.g. "/Script/Mover.FMoverDefaultSyncState".
		 *  Stored as FName so it deduplicates via the CVD name table when serialized
		 *  with FChaosVDMemoryWriter/Reader. */
		FName StructTypePath;

		/** Struct content serialized with FChaosVDStructCollectionMemoryWriter. */
		TArray<uint8> Bytes;

		/** How the bytes were written. Read back at load time to apply symmetric deserialization. */
		EChaosVDExtraDataSerializationMode SerializationMode = EChaosVDExtraDataSerializationMode::SerializeBin;

		CHAOS_API bool Serialize(FArchive& Ar);

		/** Serialize the struct into Bytes using the given mode and record the mode.
		 *  Callers should use this instead of writing Bytes manually. */
		CHAOS_API void SerializeFrom(UScriptStruct* Struct, const void* Data,
			const TSharedRef<FChaosVDSerializableNameTable>& NameTable,
			EChaosVDExtraDataSerializationMode Mode = EChaosVDExtraDataSerializationMode::SerializeBin);
	};

	/** A named group of struct entries.  Entries sharing a CategoryName are displayed together
	 *  in the CVD particle details panel under that category. */
	struct FChaosVDExtraDataCategory
	{
		FName CategoryName;
		/** Optional: ID of the CVD optional data channel this category was traced through.
		 *  NAME_None when the caller did not specify a channel. */
		FName SourceChannelId;
		TArray<FChaosVDExtraDataStructEntry> Entries;

		CHAOS_API bool Serialize(FArchive& Ar);

		/** Serialize a struct instance into a new entry. Fetches the CVD name table internally.
		 *  This is a no-op when WITH_CHAOS_VISUAL_DEBUGGER is not set. */
		CHAOS_API void AddEntry(UScriptStruct* Struct, const void* Data,
			EChaosVDExtraDataSerializationMode Mode = EChaosVDExtraDataSerializationMode::SerializeBin);
	};

	/**
	 * Per-particle extra data payload.  Contains an ordered list of named categories,
	 * each holding zero or more serialized struct entries.
	 *
	 * Written at runtime via TraceChaosVDParticleExtraData() and stored in
	 * FChaosVDParticleExtraDataContainer inside FChaosVDSolverFrameData::CustomData.
	 * On the editor side, FChaosVDParticleExtraDataComponent reads this container and
	 * injects the reconstructed structs into the particle details panel via the
	 * FChaosVDScene::OnParticleDetailsEnrichmentRequested delegate.
	 */
	struct FChaosVDParticleExtraData
	{
		CHAOS_API static FStringView WrapperTypeName;

		int32 SolverID = INDEX_NONE;
		int32 ParticleID = INDEX_NONE;

		TArray<FChaosVDExtraDataCategory> Categories;

		CHAOS_API bool Serialize(FArchive& Ar);

		bool HasValidData() const
		{
			return SolverID != INDEX_NONE && ParticleID != INDEX_NONE;
		}
	};

	/**
	 * Traces per-particle extra struct data for the current solver frame.
	 *
	 * Call this from any module (after the particle has been processed) to associate
	 * arbitrary named categories of UStructs with a specific particle. The data is
	 * keyed by SolverID + ParticleID and made available in the CVD particle details
	 * panel without requiring changes to the particle trace call site.
	 *
	 * This is a no-op when CVD tracing is disabled (WITH_CHAOS_VISUAL_DEBUGGER=0).
	 *
	 * @param SolverID       ID of the solver owning the particle.
	 * @param ParticleID     Index of the particle within the solver.
	 * @param InExtraData    Populated extra data payload to trace.
	 */
	CHAOS_API void TraceChaosVDParticleExtraData(int32 SolverID, int32 ParticleID, FChaosVDParticleExtraData& InExtraData);
}
