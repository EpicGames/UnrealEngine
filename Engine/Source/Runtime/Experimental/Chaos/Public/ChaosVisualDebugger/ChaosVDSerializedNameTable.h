// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/StringView.h"
#include "HAL/Platform.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Templates/SharedPointer.h"

#define UE_API CHAOS_API

namespace Chaos::VisualDebugger
{
	class FChaosVDMemoryReader;
}

namespace Chaos::VisualDebugger
{
	/** Serialized Name entry that can be loaded by its recorded ID */
	struct FChaosVDSerializedNameEntry
	{
		static UE_API FStringView WrapperTypeName;

		uint64 NameID = 0;
		int32 NameNumber = NAME_NO_NUMBER_INTERNAL;
		FString Name;
	};

	/** Simple Name table that keeps track of FNames by their current ID, which can be rebuilt later on with these IDs*/
	class FChaosVDSerializableNameTable : public TSharedFromThis<FChaosVDSerializableNameTable>
	{
	public:
		/** Adds a FName to the name table */
		UE_API uint64 AddNameToTable(FName Name);

		/** Adds a Serialized Name entry to the name table. Used by the CVD Data processor to rebuild the name table on load */
		UE_API uint64 AddNameToTable(const FChaosVDSerializedNameEntry& InNameEntry);

		/** Returns the FName associated with the provided ID in this table */
		UE_API FName GetNameFromTable(uint64 NameID);

		/** Clears all the FNames tracked by this table */
		UE_API void ResetTable();

	private:
			
		TMap<uint64, FName> NamesByID;
		
		FTransactionallySafeRWLock NamesByIDLock;
	};

	inline FArchive& operator<<(FArchive& Ar, FChaosVDSerializedNameEntry& NameEntry)
	{
		Ar << NameEntry.NameID;
		Ar << NameEntry.NameNumber;
		Ar << NameEntry.Name;

		return Ar;
	}
}

#undef UE_API
