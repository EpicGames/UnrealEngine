// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDMemWriterReader.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/Class.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"

namespace Chaos::VisualDebugger
{
	/**
	 * Memory writer that combines CVD name-table FName serialization with object-reference skipping.
	 *
	 * - FNames are written as uint64 name table IDs (same as FChaosVDMemoryWriter).
	 * - UScriptStruct object references are preserved as their full path string so they can
	 *   be resolved on the reading side via FindObject. This allows FInstancedStruct and
	 *   similar struct-typed UPROPERTY members to round-trip correctly.
	 * - All other UObject and FWeakObjectPtr references are silently dropped (left null on read).
	 *
	 * Intended for serializing arbitrary UStruct instances into CVD particle extra data.
	 */
	class FChaosVDStructCollectionMemoryWriter : public FMemoryWriter
	{
	public:
		FChaosVDStructCollectionMemoryWriter(TArray<uint8>& InBytes, const TSharedRef<FChaosVDSerializableNameTable>& InNameTable)
			: FMemoryWriter(InBytes), NameTableInstance(InNameTable)
		{
		}

		virtual FArchive& operator<<(FName& Name) override
		{
			NameTableFNameSerializer(*this, Name, NameTableInstance);
			return *this;
		}

		virtual FArchive& operator<<(FObjectPtr& Value) override
		{
			UObject* Obj = Value.Get();
			SerializeObjectRef(Obj);
			return *this;
		}

		virtual FArchive& operator<<(UObject*& Res) override
		{
			SerializeObjectRef(Res);
			return *this;
		}

		// Weak object refs are always dropped — nothing written to the buffer.
		virtual FArchive& operator<<(FWeakObjectPtr& Value) override
		{
			return *this;
		}

		TSharedRef<FChaosVDSerializableNameTable> NameTableInstance;

	private:
		void SerializeObjectRef(UObject* Obj)
		{
			UScriptStruct* ScriptStruct = (Obj && Obj->IsA<UScriptStruct>()) ? static_cast<UScriptStruct*>(Obj) : nullptr;
			bool bIsScriptStruct = (ScriptStruct != nullptr);
			*this << bIsScriptStruct;
			if (ScriptStruct)
			{
				// Strip the "ScriptStruct " prefix (13 chars) - same convention as FSkipObjectRefsMemoryWriter.
				FString StructName = ScriptStruct->GetFullName(nullptr).RightChop(13);
				*this << StructName;
			}
		}
	};

	/**
	 * Memory reader counterpart of FChaosVDStructCollectionMemoryWriter.
	 *
	 * - FNames are read back via the name table.
	 * - UScriptStruct refs are resolved via FindObject using the stored path string.
	 * - All other object refs are set to null (they were not written).
	 * - FWeakObjectPtr refs are left default-constructed (nothing was written).
	 */
	class FChaosVDStructCollectionMemoryReader : public FMemoryReader
	{
	public:
		FChaosVDStructCollectionMemoryReader(const TArray<uint8>& InBytes, const TSharedRef<FChaosVDSerializableNameTable>& InNameTable)
			: FMemoryReader(InBytes), NameTableInstance(InNameTable)
		{
		}

		virtual FArchive& operator<<(FName& Name) override
		{
			NameTableFNameSerializer(*this, Name, NameTableInstance);
			return *this;
		}

		virtual FArchive& operator<<(FObjectPtr& Value) override
		{
			UObject* Obj = nullptr;
			*this << Obj;
			Value = Obj;
			return *this;
		}

		virtual FArchive& operator<<(UObject*& Res) override
		{
			bool bIsScriptStruct = false;
			*this << bIsScriptStruct;
			if (bIsScriptStruct)
			{
				FString StructName;
				*this << StructName;
				Res = FindObject<UScriptStruct>(nullptr, *StructName);
			}
			else
			{
				Res = nullptr;
			}
			return *this;
		}

		// Nothing was written for weak refs; leave value default-constructed.
		virtual FArchive& operator<<(FWeakObjectPtr& Value) override
		{
			return *this;
		}

		TSharedRef<FChaosVDSerializableNameTable> NameTableInstance;
	};
}
