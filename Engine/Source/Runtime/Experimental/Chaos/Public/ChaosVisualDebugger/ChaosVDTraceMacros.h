// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_CHAOS_VISUAL_DEBUGGER

	#ifndef CVD_DEFINE_TRACE_VECTOR
		#define CVD_DEFINE_TRACE_VECTOR(Type, Name) \
		UE_TRACE_MINIMAL_EVENT_FIELD(Type, Name##X) \
		UE_TRACE_MINIMAL_EVENT_FIELD(Type, Name##Y) \
		UE_TRACE_MINIMAL_EVENT_FIELD(Type, Name##Z)
	#endif

	#ifndef CVD_DEFINE_TRACE_ROTATOR
		#define CVD_DEFINE_TRACE_ROTATOR(Type, Name) \
		UE_TRACE_MINIMAL_EVENT_FIELD(Type, Name##X) \
		UE_TRACE_MINIMAL_EVENT_FIELD(Type, Name##Y) \
		UE_TRACE_MINIMAL_EVENT_FIELD(Type, Name##Z) \
		UE_TRACE_MINIMAL_EVENT_FIELD(Type, Name##W)
	#endif

	#ifndef CVD_TRACE_VECTOR_ON_EVENT
		#define CVD_TRACE_VECTOR_ON_EVENT(EventName, Name, Vector) \
		EventName.Name##X(Vector.X) \
		<< EventName.Name##Y(Vector.Y) \
		<< EventName.Name##Z(Vector.Z)
	#endif

	#ifndef CVD_TRACE_ROTATOR_ON_EVENT
		#define CVD_TRACE_ROTATOR_ON_EVENT(EventName, Name, Rotator) \
		EventName.Name##X(Rotator.X) \
		<< EventName.Name##Y(Rotator.Y) \
		<< EventName.Name##Z(Rotator.Z) \
		<< EventName.Name##W(Rotator.W)
	#endif

	#ifndef CVD_TRACE_PARTICLE
		#define CVD_TRACE_PARTICLE(ParticleHandle) \
			FChaosVisualDebuggerTrace::TraceParticle(ParticleHandle);
	#endif

	#ifndef CVD_TRACE_PARTICLES
		#define CVD_TRACE_PARTICLES(ParticleHandles) \
			FChaosVisualDebuggerTrace::TraceParticles(ParticleHandles);
	#endif

	#ifndef CVD_TRACE_PARTICLES_VIEW
		#define CVD_TRACE_PARTICLES_VIEW(ParticleHandlesView) \
		FChaosVisualDebuggerTrace::TraceParticlesView(ParticleHandlesView);
	#endif

	#ifndef CVD_TRACE_PARTICLES_SOA
		#define CVD_TRACE_PARTICLES_SOA(ParticleSoA, ...) \
		FChaosVisualDebuggerTrace::TraceParticlesSoA(ParticleSoA, ##__VA_ARGS__);
	#endif

	#ifndef CVD_TRACE_CAMERA
		#define CVD_TRACE_CAMERA(Object, World, CameraName, Position, Rotation, FOV) \
		FChaosVisualDebuggerTrace::TraceCamera(Object, World, CameraName, Position, Rotation, FOV);
	#endif

	#ifndef CVD_TRACE_SOLVER_START_FRAME
		#define CVD_TRACE_SOLVER_START_FRAME(SolverType, SolverRef) \
			FChaosVDContext StartContextData; \
			FChaosVisualDebuggerTrace::GetCVDContext<SolverType>(SolverRef, StartContextData); \
			FChaosVisualDebuggerTrace::TraceSolverFrameStart(StartContextData, FChaosVisualDebuggerTrace::GetDebugName<SolverType>(SolverRef), (SolverRef).GetCVDFrameNumber());
	#endif

	#ifndef CVD_TRACE_SOLVER_END_FRAME
		#define CVD_TRACE_SOLVER_END_FRAME(SolverType, SolverRef) \
			FChaosVDContext EndContextData; \
			FChaosVisualDebuggerTrace::GetCVDContext<SolverType>(SolverRef, EndContextData); \
			FChaosVisualDebuggerTrace::TraceSolverFrameEnd(EndContextData);
	#endif

	#ifndef CVD_SCOPE_TRACE_SOLVER_FRAME
		#define CVD_SCOPE_TRACE_SOLVER_FRAME(SolverType, SolverRef) \
			FChaosVDScopeSolverFrame<SolverType> ScopeSolverFrame(SolverRef);
	#endif

	#ifndef CVD_TRACE_SOLVER_STEP_START
		#define CVD_TRACE_SOLVER_STEP_START(StepName) \
			FChaosVisualDebuggerTrace::TraceSolverStepStart(StepName);
	#endif

	#ifndef CVD_TRACE_SOLVER_STEP_END
		#define CVD_TRACE_SOLVER_STEP_END() \
			FChaosVisualDebuggerTrace::TraceSolverStepEnd();
	#endif

	#ifndef CVD_SCOPE_TRACE_SOLVER_STEP
		#define CVD_SCOPE_TRACE_SOLVER_STEP(DataChannel, StepName) \
			CVD_SCOPED_DATA_CHANNEL_OVERRIDE(DataChannel) \
			FChaosVDScopeSolverStep ScopeSolverStep(StepName)
	#endif

	#ifndef CVD_TRACE_BINARY_DATA
		#define CVD_TRACE_BINARY_DATA(InData, TypeName, ...) \
		FChaosVisualDebuggerTrace::TraceBinaryData(InData, TypeName, ##__VA_ARGS__);
	#endif

	#ifndef CVD_TRACE_SOLVER_SIMULATION_SPACE
		#define CVD_TRACE_SOLVER_SIMULATION_SPACE(InSimulationSpace) \
		FChaosVisualDebuggerTrace::TraceSolverSimulationSpace(InSimulationSpace);
	#endif

	#ifndef CVD_TRACE_PARTICLE_DESTROYED
		#define CVD_TRACE_PARTICLE_DESTROYED(DestroyedParticleHandle) \
		FChaosVisualDebuggerTrace::TraceParticleDestroyed(DestroyedParticleHandle);
	#endif

	#ifndef CVD_TRACE_MID_PHASE
		#define CVD_TRACE_MID_PHASE(MidPhase) \
		FChaosVisualDebuggerTrace::TraceMidPhase(MidPhase);
	#endif

	#ifndef CVD_TRACE_COLLISION_CONSTRAINT
		#define CVD_TRACE_COLLISION_CONSTRAINT(Constraint) \
		FChaosVisualDebuggerTrace::TraceCollisionConstraint(Constraint);
	#endif

	#ifndef CVD_TRACE_COLLISION_CONSTRAINT_VIEW
		#define CVD_TRACE_COLLISION_CONSTRAINT_VIEW(ConstraintView) \
		FChaosVisualDebuggerTrace::TraceCollisionConstraintView(ConstraintView);
	#endif

	#ifndef CVD_TRACE_STEP_MID_PHASES_FROM_COLLISION_CONSTRAINTS
		#define CVD_TRACE_STEP_MID_PHASES_FROM_COLLISION_CONSTRAINTS(DataChannel, CollisionConstraints) \
		CVD_SCOPED_DATA_CHANNEL_OVERRIDE(DataChannel) \
		FChaosVisualDebuggerTrace::TraceMidPhasesFromCollisionConstraints(CollisionConstraints);
	#endif

	#ifndef CVD_TRACE_INVALIDATE_CACHED_GEOMETRY
		#define CVD_TRACE_INVALIDATE_CACHED_GEOMETRY(ImplicitObjectPtr) \
		FChaosVisualDebuggerTrace::InvalidateGeometryFromCache(ImplicitObjectPtr);
	#endif

	#ifndef CVD_TRACE_INVALIDATE_CACHED_PARTICLE_METADATA
		#define CVD_TRACE_INVALIDATE_CACHED_PARTICLE_METADATA(ParticleHandlePtr) \
		FChaosVisualDebuggerTrace::InvalidateParticleMetadataFromCache(ParticleHandlePtr);
	#endif

	#ifndef CVD_TRACE_SCOPED_SCENE_QUERY
		#define CVD_TRACE_SCENE_QUERY_START(InputGeometry, GeometryOrientation, Start, End, TraceChannel, Params, ResponseParams, ObjectParams, QueryType, QueryMode, SolverID, bIsRetry) \
		FChaosVisualDebuggerTrace::TraceSceneQueryStart(InputGeometry, GeometryOrientation, Start, End, TraceChannel, Params, ResponseParams, ObjectParams, QueryType, QueryMode, SolverID, bIsRetry);
	#endif

	#ifndef CVD_TRACE_SCOPED_SCENE_QUERY_VISIT
		#define CVD_TRACE_SCOPED_SCENE_QUERY_VISIT(InQueryVisitData) \
		FChaosVDScopeSceneQueryVisit CVDQueryVisit(InQueryVisitData);
	#endif

	#ifndef CVD_TRACE_SCENE_QUERY_VISIT
		#define CVD_TRACE_SCENE_QUERY_VISIT(InQueryVisitData) \
		FChaosVisualDebuggerTrace::TraceSceneQueryVisit(InQueryVisitData);
	#endif

	#ifndef CVD_TRACE_JOINT_CONSTRAINTS
		#define CVD_TRACE_JOINT_CONSTRAINTS(DataChannel, InJointConstraints) \
			{ \
				CVD_SCOPED_DATA_CHANNEL_OVERRIDE(DataChannel) \
				FChaosVisualDebuggerTrace::TraceJointsConstraints(InJointConstraints); \
			}
	#endif

	#ifndef CVD_TRACE_CHARACTER_GROUND_CONSTRAINTS
		#define CVD_TRACE_CHARACTER_GROUND_CONSTRAINTS(DataChannel, InConstraints) \
			{ \
				CVD_SCOPED_DATA_CHANNEL_OVERRIDE(DataChannel) \
				FChaosVisualDebuggerTrace::TraceCharacterGroundConstraints(InConstraints); \
			}
	#endif

	#ifndef CVD_TRACE_CONSTRAINTS_CONTAINER
		#define CVD_TRACE_CONSTRAINTS_CONTAINER(ContainerView) \
			FChaosVisualDebuggerTrace::TraceConstraintsContainer(ContainerView);
	#endif

	#ifndef CVD_TRACE_ACCELERATION_STRUCTURES
		#define CVD_TRACE_ACCELERATION_STRUCTURES(AccelerationStructuresCollections, SolverType, SolverRef, DataChannel) \
			{ \
				FChaosVDContext StartAccelerationStrutureCVDContextData; \
				FChaosVisualDebuggerTrace::GetCVDContext<SolverType>(SolverRef, StartAccelerationStrutureCVDContextData); \
				CVD_SCOPE_CONTEXT(StartAccelerationStrutureCVDContextData); \
				CVD_SCOPED_DATA_CHANNEL_OVERRIDE(DataChannel) \
				FChaosVisualDebuggerTrace::TraceSceneAccelerationStructures(AccelerationStructuresCollections); \
			}
	#endif

	#ifndef CVD_TRACE_NETWORK_TICK_OFFSET
		#define CVD_TRACE_NETWORK_TICK_OFFSET(TickOffset, SolverID) \
		{ \
			FChaosVisualDebuggerTrace::TraceNetworkTickOffset(TickOffset, SolverID); \
		}
	#endif

	#ifndef CVD_TRACE_GET_SOLVER_ID_FROM_WORLD
		#define CVD_TRACE_GET_SOLVER_ID_FROM_WORLD(World) \
				FChaosVisualDebuggerTrace::GetSolverIDFromWorld(World)
	#endif

	#ifndef CVD_TRACE_DEBUG_DRAW_BOX
      		#define CVD_TRACE_DEBUG_DRAW_BOX(Box, ...) \
      		FChaosVisualDebuggerTrace::TraceDebugDrawBox(Box, ##__VA_ARGS__)
	#endif

	#ifndef CVD_TRACE_DEBUG_DRAW_LINE
		#define CVD_TRACE_DEBUG_DRAW_LINE(StartLocation, EndLocation, ...) \
		FChaosVisualDebuggerTrace::TraceDebugDrawLine(StartLocation, EndLocation, ##__VA_ARGS__)
	#endif

	#ifndef CVD_TRACE_DEBUG_DRAW_VECTOR
		#define CVD_TRACE_DEBUG_DRAW_VECTOR(StartLocation, Vector, ...) \
		FChaosVisualDebuggerTrace::TraceDebugDrawVector(StartLocation, Vector, ##__VA_ARGS__)
	#endif

	#ifndef CVD_TRACE_DEBUG_DRAW_SPHERE
		#define CVD_TRACE_DEBUG_DRAW_SPHERE(Center, Radius, ...) \
		FChaosVisualDebuggerTrace::TraceDebugDrawSphere(Center, Radius, ##__VA_ARGS__)
	#endif

	#ifndef CVD_TRACE_DEBUG_DRAW_IMPLICIT_OBJECT
		#define CVD_TRACE_DEBUG_DRAW_IMPLICIT_OBJECT(ImplicitObject, ParentTransform, ...) \
		FChaosVisualDebuggerTrace::TraceDebugDrawImplicitObject(ImplicitObject, ParentTransform,  ##__VA_ARGS__)
	#endif

	#ifndef CVD_SET_RELEVANCY_VOLUME
		#define CVD_SET_RELEVANCY_VOLUME(VolumeBox) \
		FChaosVisualDebuggerTrace::SetTraceRelevancyVolume(VolumeBox)
	#endif

	/**
	 * Trace a single UStruct instance as particle extra data in one line.
	 * StructType must be a USTRUCT (provides ::StaticStruct()). Mode defaults to SerializeBin.
	 *
	 * Unlike the rest of the particle trace data, extra data is NOT guaranteed to be backwards
	 * compatible. This is by design: it trades version safety for convenience — callers can pass
	 * any USTRUCT directly without writing a dedicated data wrapper.
	 *
	 * The default mode (SerializeBin) uses property-tagged serialization and handles field additions
	 * and removals gracefully. Passing EChaosVDExtraDataSerializationMode::NativeSerialization invokes
	 * the struct's native Serialize function, which may be binary-incompatible across builds and can
	 * cause a crash if the struct layout has changed. Native serialization generally requires the CVD
	 * viewer binary to match the one that produced the recording; it is safe to use when CVD runs
	 * from inside the editor against a session recorded by the same editor build.
	 *
	 * If this causes issues when loading older recordings, enable Safe Loading Mode in CVD General
	 * Settings (Project > Plugins > Chaos Visual Debugger > General). When active, CVD skips all
	 * non-backwards-compatible data rather than risking a crash.
	 *
	 * Usage:
	 *   CVD_TRACE_PARTICLE_EXTRA_DATA(SolverID, ParticleID, "MyCategory", FMyStruct, MyData);
	 *   CVD_TRACE_PARTICLE_EXTRA_DATA(SolverID, ParticleID, "MyCategory", FMyStruct, MyData, EChaosVDExtraDataSerializationMode::NativeSerialization);
	 */
	#ifndef CVD_TRACE_PARTICLE_EXTRA_DATA
		#define CVD_TRACE_PARTICLE_EXTRA_DATA(SolverID, ParticleID, CategoryName, StructType, Data, ...) \
		{ \
			if (FChaosVisualDebuggerTrace::IsTracing()) \
			{ \
				Chaos::VisualDebugger::FChaosVDParticleExtraData CVDParticleExtraData__; \
				Chaos::VisualDebugger::FChaosVDExtraDataCategory& CVDCategory__ = CVDParticleExtraData__.Categories.AddDefaulted_GetRef(); \
				CVDCategory__.CategoryName = (CategoryName); \
				CVDCategory__.AddEntry(StructType::StaticStruct(), &(Data), ##__VA_ARGS__); \
				Chaos::VisualDebugger::TraceChaosVDParticleExtraData((SolverID), (ParticleID), CVDParticleExtraData__); \
			} \
		}
	#endif

	/** Channel-aware variant of CVD_TRACE_PARTICLE_EXTRA_DATA.
	 *  ChannelId should be passed as FName(TEXT("ChannelName")) or NAME_None.
	 *  Records which CVD optional data channel this entry was traced from so the
	 *  post-load native-serialization dialog can name the source channel. */
	#ifndef CVD_TRACE_PARTICLE_EXTRA_DATA_IN_CHANNEL
		#define CVD_TRACE_PARTICLE_EXTRA_DATA_IN_CHANNEL(SolverID, ParticleID, ChannelId, CategoryName, StructType, Data, ...) \
		{ \
			if (FChaosVisualDebuggerTrace::IsTracing()) \
			{ \
				Chaos::VisualDebugger::FChaosVDParticleExtraData CVDParticleExtraData__; \
				Chaos::VisualDebugger::FChaosVDExtraDataCategory& CVDCategory__ = CVDParticleExtraData__.Categories.AddDefaulted_GetRef(); \
				CVDCategory__.CategoryName = (CategoryName); \
				CVDCategory__.SourceChannelId = (ChannelId); \
				CVDCategory__.AddEntry(StructType::StaticStruct(), &(Data), ##__VA_ARGS__); \
				Chaos::VisualDebugger::TraceChaosVDParticleExtraData((SolverID), (ParticleID), CVDParticleExtraData__); \
			} \
		}
	#endif

#else // WITH_CHAOS_VISUAL_DEBUGGER

	#ifndef CVD_TRACE_PARTICLE
		#define CVD_TRACE_PARTICLE(ParticleHandle)
	#endif

	#ifndef CVD_TRACE_PARTICLES
		#define CVD_TRACE_PARTICLES(ParticleHandles)
	#endif

	#ifndef CVD_TRACE_CAMERA
		#define CVD_TRACE_CAMERA(Object, World, CameraName, Position, Rotation, FOV)
	#endif

	#ifndef CVD_TRACE_SOLVER_START_FRAME
		#define CVD_TRACE_SOLVER_START_FRAME(SolverType, SolverRef)
	#endif

	#ifndef CVD_TRACE_SOLVER_END_FRAME
		#define CVD_TRACE_SOLVER_END_FRAME(SolverType, SolverRef)
	#endif

	#ifndef CVD_SCOPE_TRACE_SOLVER_FRAME
		#define CVD_SCOPE_TRACE_SOLVER_FRAME(SolverType, SolverRef)
	#endif

	#ifndef CVD_TRACE_SOLVER_STEP_START
		#define CVD_TRACE_SOLVER_STEP_START(StepName)
	#endif

	#ifndef CVD_TRACE_SOLVER_STEP_END
		#define CVD_TRACE_SOLVER_STEP_END()
	#endif

	#ifndef CVD_SCOPE_TRACE_SOLVER_STEP
		#define CVD_SCOPE_TRACE_SOLVER_STEP(DataChannel, StepName)
	#endif

	#ifndef CVD_TRACE_BINARY_DATA
		#define CVD_TRACE_BINARY_DATA(InData, TypeName, ...)
	#endif

	#ifndef CVD_TRACE_SOLVER_SIMULATION_SPACE
			#define CVD_TRACE_SOLVER_SIMULATION_SPACE(InSimulationSpace)
	#endif

	#ifndef CVD_TRACE_PARTICLES_SOA
		#define CVD_TRACE_PARTICLES_SOA(ParticleSoA, ...)
	#endif

	#ifndef CVD_TRACE_PARTICLE_DESTROYED
		#define CVD_TRACE_PARTICLE_DESTROYED(DestroyedParticleHandle)
	#endif

	#ifndef CVD_TRACE_MID_PHASE
		#define CVD_TRACE_MID_PHASE(MidPhase)
	#endif
	#ifndef CVD_TRACE_COLLISION_CONSTRAINT
		#define CVD_TRACE_COLLISION_CONSTRAINT(Constraint)
	#endif
	#ifndef CVD_TRACE_COLLISION_CONSTRAINT_VIEW
		#define CVD_TRACE_COLLISION_CONSTRAINT_VIEW(ConstraintView)
	#endif

	#ifndef CVD_TRACE_PARTICLES_VIEW
		#define CVD_TRACE_PARTICLES_VIEW(ParticleHandlesView)
	#endif

	#ifndef CVD_TRACE_STEP_MID_PHASES_FROM_COLLISION_CONSTRAINTS
		#define CVD_TRACE_STEP_MID_PHASES_FROM_COLLISION_CONSTRAINTS(DataChannel, CollisionConstraints)
	#endif

	#ifndef CVD_TRACE_INVALIDATE_CACHED_GEOMETRY
		#define CVD_TRACE_INVALIDATE_CACHED_GEOMETRY(ImplicitObjectPtr)
	#endif

	#ifndef CVD_TRACE_INVALIDATE_CACHED_PARTICLE_METADATA
		#define CVD_TRACE_INVALIDATE_CACHED_PARTICLE_METADATA(ParticleHandlePtr)
	#endif

	#ifndef CVD_TRACE_SCOPED_SCENE_QUERY
		#define CVD_TRACE_SCENE_QUERY_START(InputGeometry, GeometryOrientation, Start, End, TraceChannel, Params, ResponseParams, ObjectParams, QueryType, QueryMode, SolverID)
	#endif

	#ifndef CVD_TRACE_SCOPED_SCENE_QUERY_VISIT
		#define CVD_TRACE_SCOPED_SCENE_QUERY_VISIT(InQueryVisitData)
	#endif

	#ifndef CVD_TRACE_SCENE_QUERY_VISIT
		#define CVD_TRACE_SCENE_QUERY_VISIT(InQueryVisitData)
	#endif

	#ifndef CVD_TRACE_JOINT_CONSTRAINTS
		#define CVD_TRACE_JOINT_CONSTRAINTS(DataChannel, InJointConstraints)
	#endif

	#ifndef CVD_TRACE_CHARACTER_GROUND_CONSTRAINTS
		#define CVD_TRACE_CHARACTER_GROUND_CONSTRAINTS(DataChannel, InConstraints)
	#endif

	#ifndef CVD_TRACE_CONSTRAINTS_CONTAINER
		#define CVD_TRACE_CONSTRAINTS_CONTAINER(ContainerView)
	#endif

	#ifndef CVD_TRACE_ACCELERATION_STRUCTURES
			#define CVD_TRACE_ACCELERATION_STRUCTURES(AccelerationStructuresCollections, SolverType, SolverRef, DataChannel)
	#endif

	#ifndef CVD_TRACE_NETWORK_TICK_OFFSET
		#define CVD_TRACE_NETWORK_TICK_OFFSET(TickOffset, SolverID)
	#endif

	#ifndef CVD_TRACE_GET_SOLVER_ID_FROM_WORLD
		#define CVD_TRACE_GET_SOLVER_ID_FROM_WORLD(World)
	#endif

	#ifndef CVD_TRACE_DEBUG_DRAW_BOX
			  #define CVD_TRACE_DEBUG_DRAW_BOX(Box, ...)
	#endif

	#ifndef CVD_TRACE_DEBUG_DRAW_LINE
		#define CVD_TRACE_DEBUG_DRAW_LINE(StartLocation, EndLocation, ...)
	#endif

	#ifndef CVD_TRACE_DEBUG_DRAW_VECTOR
		#define CVD_TRACE_DEBUG_DRAW_VECTOR(StartLocation, Vector, ...)
	#endif

	#ifndef CVD_TRACE_DEBUG_DRAW_SPHERE
		#define CVD_TRACE_DEBUG_DRAW_SPHERE(...)
	#endif

	#ifndef CVD_TRACE_DEBUG_DRAW_IMPLICIT_OBJECT
		#define CVD_TRACE_DEBUG_DRAW_IMPLICIT_OBJECT(ImplicitObject, ParentTransform, ...)
	#endif

	#ifndef CVD_SET_RELEVANCY_VOLUME
		#define CVD_SET_RELEVANCY_VOLUME(VolumeBox)
	#endif

	#ifndef CVD_TRACE_PARTICLE_EXTRA_DATA
		#define CVD_TRACE_PARTICLE_EXTRA_DATA(SolverID, ParticleID, CategoryName, StructType, Data, ...)
	#endif

	#ifndef CVD_TRACE_PARTICLE_EXTRA_DATA_IN_CHANNEL
		#define CVD_TRACE_PARTICLE_EXTRA_DATA_IN_CHANNEL(SolverID, ParticleID, ChannelId, CategoryName, StructType, Data, ...)
	#endif

#endif // WITH_CHAOS_VISUAL_DEBUGGER