// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Chaos/Serializable.h"
#include "ChaosVDContextProvider.h"
#include "HAL/ThreadSafeBool.h"

#if WITH_CHAOS_VISUAL_DEBUGGER
#include "ChaosVDRuntimeModule.h"
#endif

#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "Chaos/ChaosArchive.h"
#include "Chaos/Core.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/ParticleIterator.h"
#include "ChaosVDOptionalDataChannel.h"
#include "Containers/Set.h"

#ifndef CHAOS_VISUAL_DEBUGGER_ENABLED
	#define CHAOS_VISUAL_DEBUGGER_ENABLED WITH_CHAOS_VISUAL_DEBUGGER
#endif

#ifndef TRACE_BASED_DEBUGGERS_WITHOUT_TRACE
	#define TRACE_BASED_DEBUGGERS_WITHOUT_TRACE 0
#endif

#include "Chaos/Framework/PhysicsSolverBase.h"
#include "ChaosVisualDebugger/ChaosVDTraceMacros.h"

#if WITH_CHAOS_VISUAL_DEBUGGER

#include "Chaos/ChaosDebugNameDefines.h"
#include "ChaosVDMemWriterReader.h"
#include "Containers/StripedMap.h"
#include "DataWrappers/ChaosVDImplicitObjectDataWrapper.h"
#include "Trace/Trace.h"
#include "Trace/Trace.inl"

enum class EChaosVDFullCaptureFlags;

namespace Chaos
{
class FRigidClustering;
class FAccelerationStructureHandle;
class FCharacterGroundConstraintContainer;
class FPBDConstraintContainer;
class FPBDJointConstraints;
}

UE_TRACE_MINIMAL_CHANNEL_EXTERN(ChaosVDChannel, CHAOS_API)

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDSolverFrameStart)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, SolverID)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, DebugName)
	UE_TRACE_MINIMAL_EVENT_FIELD(bool, IsKeyFrame)
	UE_TRACE_MINIMAL_EVENT_FIELD(bool, IsReSimulated)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, CurrentFrameNumber)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDSolverFrameEnd)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, SolverID)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDParticleCreated)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, SolverID)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, ParticleID)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDParticleDestroyed)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, SolverID)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, ParticleID)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDSolverStepStart)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, SolverID)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, StepName)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDSolverStepEnd)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, SolverID)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, StepNumber)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDBinaryDataStart)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, TypeName)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, DataID)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, DataSize)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint32, OriginalSize)
	UE_TRACE_MINIMAL_EVENT_FIELD(bool, IsCompressed)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDBinaryDataContent)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, DataID)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint8[], RawData)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDBinaryDataEnd)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, DataID)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDSolverSimulationSpace)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, SolverID)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	CVD_DEFINE_TRACE_VECTOR(Chaos::FReal, Position)
	CVD_DEFINE_TRACE_ROTATOR(Chaos::FReal, Rotation)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDNonSolverLocation)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	CVD_DEFINE_TRACE_VECTOR(Chaos::FReal, Position)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, DebugName)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDNonSolverTransform)
	UE_TRACE_MINIMAL_EVENT_FIELD(uint64, Cycle)
	CVD_DEFINE_TRACE_VECTOR(Chaos::FReal, Position)
	CVD_DEFINE_TRACE_VECTOR(Chaos::FReal, Scale)
	CVD_DEFINE_TRACE_ROTATOR(Chaos::FReal, Rotation)
	UE_TRACE_MINIMAL_EVENT_FIELD(UE::Trace::WideString, DebugName)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDNetworkTickOffset)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, Offset)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, SolverID)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDRolledBackDataID)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, DataID)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDUsesAutoRTFM)
	UE_TRACE_MINIMAL_EVENT_FIELD(bool, bUsingAutoRTFM)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDMiscHeaderData)
	UE_TRACE_MINIMAL_EVENT_FIELD(bool, bUsesSharedGeometryContextArchive)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDTraceRelevancyVolume)
	CVD_DEFINE_TRACE_VECTOR(Chaos::FReal, BoxMin)
	CVD_DEFINE_TRACE_VECTOR(Chaos::FReal, BoxMax)
UE_TRACE_MINIMAL_EVENT_END()

UE_TRACE_MINIMAL_EVENT_BEGIN_EXTERN(ChaosVDLogger, ChaosVDDummyEvent)
	UE_TRACE_MINIMAL_EVENT_FIELD(int32, SolverID)
UE_TRACE_MINIMAL_EVENT_END()

struct FChaosVDContext;
struct FChaosVDQueryVisitStep;
struct FChaosVDCollisionResponseParams;
struct FChaosVDCollisionObjectQueryParams;
struct FChaosVDCollisionQueryParams;
enum class EChaosVDSceneQueryMode;
enum class EChaosVDSceneQueryType;
struct FCollisionObjectQueryParams;
struct FCollisionResponseParams;
struct FCollisionQueryParams;
enum ECollisionChannel : int;
class AActor;
class UWorld;

namespace Chaos
{
	namespace VisualDebugger
	{
		class FChaosVDSerializableNameTable;
	}

	class FPBDCollisionConstraints;
	class FPBDRigidsSOAs;
	class FImplicitObject;
	class FPhysicsSolverBase;
	template <typename T, int d>
	class TGeometryParticleHandles;

	class FPBDCollisionConstraint;
	class FParticlePairMidPhase;

	template <typename PayloadType, typename T, int d>
	class ISpatialAccelerationCollection;
}

using FChaosVDImplicitObjectWrapper = FChaosVDImplicitObjectDataWrapper<Chaos::FImplicitObjectPtr, Chaos::FChaosArchive>;
using FChaosVDSerializableNameTable = Chaos::VisualDebugger::FChaosVDSerializableNameTable;

enum class EChaosVDTraceBinaryDataOptions
{
	None = 0,
	/** Will trace the provided data buffer into CVD's trace channel, regardless id CVD's trace system is fully initialized.
	 * Data traced in this way, will not be backwards compatible as the required header data will not be ready yet */
	ForceTrace = 1 << 0
};
ENUM_CLASS_FLAGS(EChaosVDTraceBinaryDataOptions)

struct FChaosVDParticleMetadata;

namespace Chaos::VD
{
	template<typename TObjectID>
	class FCachedObjectTraceIDByPtr
	{
	public:

		template<typename ObjectType, typename HashProducer>
		TObjectID FindOrProduceTraceObjectID(const ObjectType* ObjectPtr, const HashProducer& InHashProducer, bool& OutWasAlreadyCached)
		{
			if (ObjectPtr == nullptr)
			{
				return 0;
			}

			bool bHasCachedID = true;
			TObjectID FoundID = CachedIDsByPtr.FindOrProduce(reinterpret_cast<const void*>(ObjectPtr), [&InHashProducer, &bHasCachedID]()
			{
				bHasCachedID = false;
				return InHashProducer();
			});

			if (bHasCachedID)
			{
				OutWasAlreadyCached = true;
				return FoundID;
			}

			return FoundID;
		}

		template<typename ObjectType>
		void RemoveCachedTraceObjectID(const ObjectType* Implicit)
		{
			if (Implicit == nullptr)
			{
				return;
			}

			CachedIDsByPtr.Remove(reinterpret_cast<const void*>(Implicit));
		}

		void Reset()
		{
			CachedIDsByPtr.Reset();
		}

	private:

		static constexpr int32 StripCount = 32;
		TStripedMap<StripCount, const void*, TObjectID> CachedIDsByPtr;
	};

	struct FRecordingSessionState
	{
		FRecordingSessionState();

		DECLARE_DELEGATE_RetVal_TwoParams(FChaosVDParticleMetadata, FParticleMetaDataGeneratorDelegate, const IPhysicsProxyBase*, const Chaos::FGeometryParticleHandle* ParticleHandle)
		FParticleMetaDataGeneratorDelegate ExternalParticleMetadataGenerator;

		FDelegateHandle RecordingStartedDelegateHandle;
		FDelegateHandle RecordingStoppedDelegateHandle;
		FDelegateHandle RecordingFullCaptureRequestedHandle;

		TSet<int32> SolverIDsForDeltaRecording;
		TSet<int32> RequestedFullCaptureSolverIDs;
		TSharedRef<FChaosVDSerializableNameTable> CVDNameTable;

		FBox TraceRelevancyVolume;

		std::atomic<bool> bIsTracing;

		FCachedObjectTraceIDByPtr<uint32> TracedGeometryHashCache;

		FCachedObjectTraceIDByPtr<uint64> ParticleMetadataIDsCache;

		FRWLock DeltaRecordingStatesLock;

		uint64 GenerateUniqueID()
		{
			uint64 NewID = 0;
			UE_AUTORTFM_OPEN
			{
				NewID = InternalGenericIDCounter++;
			};

			return NewID;
		}

		static constexpr uint64 InvalidUniqueID = 0;

		TSharedRef<FChaosArchiveContext> ArchiveContext;
		FTransactionallySafeMutex ArchiveContextLock;

		bool UsingSharedGeometryArchiveContext = true;

		private:
		std::atomic<uint64> InternalGenericIDCounter;
	};
}

/** Class containing  all the Tracing logic to record data for the Chaos Visual Debugger tool */
class FChaosVisualDebuggerTrace
{
public:
	/**
	 * Traces data from a Particle Handle. The CVD context currently pushed into will be used to tie this particle data to a specific solver frame and step
	 * @param ParticleHandle Handle to process and Trace
	 */
	static CHAOS_API void TraceParticle(const Chaos::FGeometryParticleHandle* ParticleHandle);

	/**
	 * Traces data from a collection of Particle Handles using. The CVD context currently pushed into will be used to tie this particle data to a specific solver frame and step.
	 * It does not handle Full and Delta Recording automatically
	 * @param ParticleHandles Handles collection to process and Trace
	 */
	static CHAOS_API void TraceParticles(const Chaos::TGeometryParticleHandles<Chaos::FReal, 3>& ParticleHandles);
	
	/**
	 * Traces the destruction event for the provided particle handled so it can be reproduces in the CVD tool
	 * @param ParticleHandle Handle that is being destroyed
	 */
	static CHAOS_API void TraceParticleDestroyed(const Chaos::FGeometryParticleHandle* ParticleHandle);

	/**
	 * Traces data from particles on the provided FPBDRigidsSOAs. It traces only the DirtyParticles view unless a full capture was requested
	 * @param ParticlesSoA Particles SoA to evaluate and trace
	 */
	static CHAOS_API void TraceParticlesSoA(const Chaos::FPBDRigidsSOAs& ParticlesSoA, Chaos::FRigidClustering* ClusteringData = nullptr);


	/** Traces data from cameras */
	static CHAOS_API void TraceCamera(AActor* Object, UWorld* World, FName CameraName, const FVector& Position, const FQuat& Rotation, float FOV);

	/** Traces the provided particle view in parallel */
	template<typename ParticleType>
	static void TraceParticlesView(const Chaos::TParticleView<ParticleType>& ParticlesView);

	/** Traces a Particle pair MidPhase as binary data */
	static CHAOS_API void TraceMidPhase(const Chaos::FParticlePairMidPhase* MidPhase);

	/** Traces a Particle pair MidPhase as binary data from a provided CollisionConstraints object */
	static CHAOS_API void TraceMidPhasesFromCollisionConstraints(Chaos::FPBDCollisionConstraints& InCollisionConstraints);

	/** Traces all joint constraints in the provided container */
	static CHAOS_API void TraceJointsConstraints(Chaos::FPBDJointConstraints& InJointConstraints);

	/** Traces all character ground constraints in the provided container */
	static CHAOS_API void TraceCharacterGroundConstraints(Chaos::FCharacterGroundConstraintContainer& InConstraints);

	/** Traces a Particle pair MidPhase as binary data */
	static CHAOS_API void TraceCollisionConstraint(const Chaos::FPBDCollisionConstraint* CollisionConstraint);

	/** Traces a Particle pair MidPhase as binary data in parallel */
	static CHAOS_API void TraceCollisionConstraintView(TArrayView<Chaos::FPBDCollisionConstraint* const> CollisionConstraintView);

	/** Traces all supported constraints in the provided containers view */
	static CHAOS_API void TraceConstraintsContainer(TConstArrayView<Chaos::FPBDConstraintContainer*> ConstraintContainersView);

	/** Traces the start of a solver frame and it pushes its context data to the CVD TLS context stack */
	static CHAOS_API void TraceSolverFrameStart(const FChaosVDContext& ContextData, const FString& InDebugName, int32 FrameNumber = INDEX_NONE);
	
	/** Traces the end of a solver frame and removes its context data to the CVD TLS context stack */
	static CHAOS_API void TraceSolverFrameEnd(const FChaosVDContext& ContextData);

	/** Traces the start of a solver step
	 * @param StepName Name of the step. It will be used in the CVD Tool's UI
	 */
	static CHAOS_API void TraceSolverStepStart(FStringView StepName);
	/** Traces the end of a solver step */
	static CHAOS_API void TraceSolverStepEnd();

	/** Traces the provider transform as simulation space of the solver that is currently on the CVD Context Stack
	 * @param Transform Simulation space Transform
	 */
	static CHAOS_API void TraceSolverSimulationSpace(const Chaos::FRigidTransform3& Transform);

	/**
	 * Traces a binary blob of data outside of any solver frame solver step scope.
	 * @param InData Data to trace
	 * @param TypeName Type name the data represents. It is used during Trace Analysis serialize it back (this is not automatic)
	 */
	static CHAOS_API void TraceBinaryData(TConstArrayView<uint8> InData, FStringView TypeName, EChaosVDTraceBinaryDataOptions Options = EChaosVDTraceBinaryDataOptions::None);

	/**
	 * Serializes the implicit object contained in the wrapper and trace its it as binary data
	 * The trace event is not tied to a particular Solver Frame/Step
	 *  @param WrappedGeometryData Wrapper containing a ptr to the implicit and its ID
	 */
	UE_DEPRECATED(5.7, "Use the TraceImplicitObject version that takes a ptr to the implicit object itself")
	static CHAOS_API void TraceImplicitObject(FChaosVDImplicitObjectWrapper WrappedGeometryData);

	/**
	 * Serializes a implicit object
	 * The trace event is not tied to a particular Solver Frame/Step
	 * @param GeometryPtr Geometry instance to trace
	 * @return Hash of the traced geometry that will be used when the recording is loaded to match it to the correct piece of data
	 */
	static CHAOS_API uint32 TraceImplicitObject(Chaos::FImplicitObject* GeometryPtr);

	/**
	 * Gathers and serializes the debug name and other metadata for the provider particle once. Any subsequent calls will only return the existing hash for the debug name
	 * The trace event is not tied to a particular Solver Frame/Step
	 * @param InParticleHandle Handle to the particle from which we want to trace its debug name
	 * @return Id of the traced particle metadata that will be used when the recording is loaded to match it to the correct piece of data
	 */
	static CHAOS_API uint64 TraceParticleMetadata(Chaos::FGeometryParticleHandle* InParticleHandle);

	/**
	 * Removes an implicit object from the serialized geometry IDs cache, to ensure we re-serialize it with any new changes
	 *  @param CachedGeometryToInvalidate Ptr to the Geometry we want to invalidate from the cache
	 */
	static CHAOS_API void InvalidateGeometryFromCache(const Chaos::FImplicitObject* CachedGeometryToInvalidate);

	/**
	 * Removes the associated metadata id for a particle from the traced data IDs cache, to ensure we re-serialize it with any new changes
	 *  @param InParticleToInvalidate Ptr to the particle we want to invalidate from the cache
	 */
	static CHAOS_API void InvalidateParticleMetadataFromCache(Chaos::FGeometryParticleHandle* InParticleToInvalidate);

	/** Records a visit step of a scene query. This needs to me called within the scope of an inflight scene query
	 * @param InputGeometry : Geometry used as input to perform the query we are recording
	 * @param GeometryOrientation : Orientation of the input geometry
	 * @param Start : Start location of the Query.
	 * @param End : End location of the Query
	 * @param TraceChannel : Trace channel used for the query we are recording.
	 * @param Params : Collision query params used for the query we are recording.
	 * @param ResponseParams :  Collision response params used for the query we are recording.
	 * @param ObjectParams :  Collision object query params used for the query we are recording.
	 * @param SolverID : ID of the solver this where query is being performed.
	 * @param bIsRetry : Set to true if the query que are recording it is from a retry attempt.
	 * */
	static CHAOS_API void TraceSceneQueryStart(const Chaos::FImplicitObject* InputGeometry, const FQuat& GeometryOrientation, const FVector& Start, const FVector& End, ECollisionChannel TraceChannel, FChaosVDCollisionQueryParams&& Params, FChaosVDCollisionResponseParams&& ResponseParams, FChaosVDCollisionObjectQueryParams&& ObjectParams, EChaosVDSceneQueryType QueryType, EChaosVDSceneQueryMode QueryMode, int32 SolverID, bool bIsRetry);

	/** Records a visit step of a scene query. This needs to me called within the scope of an inflight scene query
	 * @param InQueryVisitData : Processed Scene Query step data.
	 * */
	static CHAOS_API void TraceSceneQueryVisit(FChaosVDQueryVisitStep&& InQueryVisitData);

	/** Records all the suppported accelerations structures contained by the provided acceleration structure collection
	 * @param InAccelerationCollection : Ptr to the collection we want to evaluate and trace.
	 * */
	static CHAOS_API void TraceSceneAccelerationStructures(const Chaos::ISpatialAccelerationCollection<Chaos::FAccelerationStructureHandle, Chaos::FReal, 3>* InAccelerationCollection);

	/** Records the current tick offset for any given solver. This is used in the CVD editor to sync solver tracks based on their network tick
	 * @param TickOffset : Offset relative to the server's solver current tick number.
	 * @param SolverID : Solver ID of the solver that has this offset.
	 * */
	static CHAOS_API void TraceNetworkTickOffset(int32 TickOffset, int32 SolverID);

	/** Records the provided box and the rest of the arguments, so then it can be visualized via debug draw during playback in the CVD editor
	 * @param InBox : Shape to record.
	 * @param Tag : FName that will be used as a tag for filtering & search, and debug draw as a text tag in CVD's viewport.
	 * @param Color : Color to apply to this shape when it is debug drawn in CVD.
	 * @param SolverID : ID of the solver this shape should be associated with. if no ID is provided, this shape will be added as part of the current game frame data bucket.
	 */
	static CHAOS_API void TraceDebugDrawBox(const FBox& InBox, FName Tag = NAME_None, FColor Color = FColor::Blue, int32 SolverID = INDEX_NONE);
	
	/** Records the provided Line and the rest of the arguments, so then it can be visualized via debug draw during playback in the CVD editor
	 * @param InStartLocation : Start point of the line.
	 * @param InEndLocation : End point of the line.
	 * @param Tag : FName that will be used as a tag for filtering & search, and debug draw as a text tag in CVD's viewport.
	 * @param Color : Color to apply to this shape when it is debug drawn in CVD.
	 * @param SolverID : ID of the solver this shape should be associated with. if no ID is provided, this shape will be added as part of the current game frame data bucket.
	 */
	static CHAOS_API void TraceDebugDrawLine(const FVector& InStartLocation, const FVector& InEndLocation, FName Tag = NAME_None, FColor Color = FColor::Blue, int32 SolverID = INDEX_NONE);
	
	/** Records the provided Vector and the rest of the arguments, so then it can be visualized via debug draw during playback in the CVD editor
	 * @param InStartLocation : Start point of the line.
	 * @param InVector : Vector we want to record.
	 * @param Tag : FName that will be used as a tag for filtering & search, and debug draw as a text tag in CVD's viewport.
	 * @param Color : Color to apply to this shape when it is debug drawn in CVD.
	 * @param SolverID : ID of the solver this shape should be associated with. if no ID is provided, this shape will be added as part of the current game frame data bucket.
	 */
	static CHAOS_API void TraceDebugDrawVector(const FVector& InStartLocation, const FVector& InVector, FName Tag = NAME_None, FColor Color = FColor::Blue, int32 SolverID = INDEX_NONE);

	/** Records the provided Sphere and the rest of the arguments, so then it can be visualized via debug draw during playback in the CVD editor
	 * @param Center : Origin point of the Sphere.
	 * @param Radius : Radius of the Sphere.
	 * @param Tag : FName that will be used as a tag for filtering & search, and debug draw as a text tag in CVD's viewport.
	 * @param Color : Color to apply to this shape when it is debug drawn in CVD.
	 * @param SolverID : ID of the solver this shape should be associated with. if no ID is provided, this shape will be added as part of the current game frame data bucket.
	 */
	static CHAOS_API void TraceDebugDrawSphere(const FVector& Center, float Radius, FName Tag = NAME_None, FColor Color = FColor::Blue, int32 SolverID = INDEX_NONE);

	/** Records the provided Implicit Object and the rest of the arguments, so then it can be visualized via debug draw during playback in the CVD editor
	 * @param Implicit : Ptr to the implicit object to record.
	 * @param InParentTransform : Root transform of the object that owns this geometry.
	 * @param Tag : FName that will be used as a tag for filtering & search, and debug draw as a text tag in CVD's viewport.
	 * @param Color : Color to apply to this shape when it is debug drawn in CVD.
	 * @param SolverID : ID of the solver this shape should be associated with. if no ID is provided, this shape will be added as part of the current game frame data bucket.
	 */
	static CHAOS_API void TraceDebugDrawImplicitObject(const Chaos::FImplicitObject* Implicit, const FTransform& InParentTransform, FName Tag = NAME_None, FColor Color = FColor::Blue, int32 SolverID = INDEX_NONE);

	/**
	 * Sets a bounding box to be used to filter out particles and other data outside of it.
	 * @param InTraceRelevancyVolume Volume within which object's data should be traced
	 */
	static CHAOS_API void SetTraceRelevancyVolume(const FBox& InTraceRelevancyVolume);

	/*
	 * Returns the currently bounding box used to determine if we should trace data or not, for objects that can be applied to.
	 */
	static FBox GetTraceRelevancyVolume()
	{
		return SessionState.TraceRelevancyVolume;
	}

	/**
	 * Evaluates the provided bounds to see if it is withing the current relevancy volume, if any.
	 * @param InChaosBounds Bounds to evaluate
	 * @return Returns true if the bounds are relevant or if there is no relevancy volume configured
	 */
	static bool IsInRelevancyVolume(const Chaos::FAABB3& InChaosBounds)
	{
		if (!SessionState.TraceRelevancyVolume.IsValid)
		{
			// An invalid volume means we want to trace everything
			return true;
		}

		return SessionState.TraceRelevancyVolume.Intersect(FBox(InChaosBounds.Min(), InChaosBounds.Max()));
	}

	/**
	 * Evaluates the provided bounds to see if it is withing the current relevancy volume, if any.
	 * @param InBounds Bounds to evaluate
	 * @return Returns true if the bounds are relevant or if there is no relevancy volume configured
	 */
	static bool IsInRelevancyVolume(const FBox& InBounds)
	{
		if (!SessionState.TraceRelevancyVolume.IsValid)
		{
			// An invalid volume means we want to trace everything
			return true;
		}

		return SessionState.TraceRelevancyVolume.Intersect(InBounds);
	}

	/**
	 * Evaluates the provided sphere to see if it is withing the current relevancy volume, if any.
	 * @param Center Center of the sphere
	 * @param Radius Radius of the Sphere
	 * @return Returns true if the sphere is relevant or if there is no relevancy volume configured
	 */
	static bool IsInRelevancyVolume(const FVector& Center, double Radius)
	{
		if (!SessionState.TraceRelevancyVolume.IsValid)
		{
			// An invalid volume means we want to trace everything
			return true;
		}
	
		return FMath::SphereAABBIntersection(Center, Radius * Radius, SessionState.TraceRelevancyVolume);
	}

	/**
	 * Evaluates the provided point to see if it is withing the current relevancy volume, if any.
	 * @param InPosition point to evaluate
	 * @return Returns true if the point is relevant or if there is no relevancy volume configured
	 */
	static bool IsInRelevancyVolume(const Chaos::FVec3& InPosition)
	{
		if (!SessionState.TraceRelevancyVolume.IsValid)
		{
			// An invalid volume means we want to trace everything
			return true;
		}

		return FMath::PointBoxIntersection(InPosition, SessionState.TraceRelevancyVolume);
	}

	/**
	 * Evaluates the provided line to see if it is withing the current relevancy volume, if any.
	 * @param InStartPoint Start point of the line to evaluate
	 * @param InEndPoint End point of the line to evaluate
	 * @return Returns true if the line is relevant or if there is no relevancy volume configured
	 */
	static bool IsInRelevancyVolume(const Chaos::FVec3& InStartPoint, const Chaos::FVec3& InEndPoint)
	{
		if (!SessionState.TraceRelevancyVolume.IsValid)
		{
			// An invalid volume means we want to trace everything
			return true;
		}

		return SessionState.TraceRelevancyVolume.Intersect(FBox(FVector::Min(InStartPoint, InEndPoint), FVector::Max(InStartPoint, InEndPoint)));
	}

	static bool IsInRelevancyVolume(const Chaos::FGeometryParticleHandle* InParticleHandle);
	static bool IsInRelevancyVolume(const Chaos::FImplicitObject* Geometry, const Chaos::FRigidTransform3& InTransform);

	/** Returns the ID of the main solver of the provided world
	 * @param World Ptr to the world we want to get the solverID from
	 */
	template<typename WorldType>
	static int32 GetSolverIDFromWorld(WorldType* World);

	/** Returns the CVD solver ID of the provided solver
	 * @param Solver Reference to the solver instance we want to get the id from
	 */
	static int32 CHAOS_API GetSolverID(Chaos::FPhysicsSolverBase& Solver);

	/** Returns true if the provided solver ID needs a Full Capture */
	static CHAOS_API bool ShouldPerformFullCapture(int32 SolverID);

	/**
	 * Gets the CVD Context data form an object that has such data. Usually Solvers
	 * @tparam T type of the object with CVD context
	 * @param ObjectWithContext A reference to where to get the CVD Context
	 * @param OutCVDContext A reference to the CVD contexts in the provided object
	 */
	template<typename T>
	static void GetCVDContext(T& ObjectWithContext, FChaosVDContext& OutCVDContext);

	/**
	 * Returns the debug name string of the provided object
	 */
	template<typename T>
	static FString GetDebugName(T& ObjectWithDebugName);

	/** Returns true if a CVD trace is running */
	static CHAOS_API bool IsTracing();

	/** Binds to the static events triggered by the ChaosVD Runtime module */
	static void RegisterEventHandlers();
	
	/** Unbinds to the static events triggered by the ChaosVD Runtime module */
	static void UnregisterEventHandlers();

	static CHAOS_API TSharedRef<FChaosVDSerializableNameTable>& GetNameTableInstance();

	/**
	 * If the provided array is not empty, all data channels that are not in the list and can be disabled, will be disabled
	 * @param EnabledDataChannelsOverrideList List of the only channels that should be enabled by default
	 */
	static CHAOS_API void OverrideDefaultEnabledDataChannels(TConstArrayView<FString> EnabledDataChannelsOverrideList);

	static CHAOS_API void RegisterExternalParticleDebugNameGenerator(const Chaos::VD::FRecordingSessionState::FParticleMetaDataGeneratorDelegate& InCallback);

private:

	/**
	 * Traces data from all Child Particles from any Cluster Particle inside the provided view array, using the provided CVD Context and Clustering Data
	 * @param ParticlesView Particles Array view to process and Trace
	 * @param ClusteringData Object containing the mappings required to find all child particles for a given cluster particle
	 * @param CVDContextData Context to be used to tie this Trace event with specific solver frame and step
	 */
	static CHAOS_API void TraceParticleClusterChildData(const Chaos::TParticleView<Chaos::TPBDRigidParticles<Chaos::FReal, 3>>& ParticlesView, Chaos::FRigidClustering* ClusteringData, const FChaosVDContext& CVDContextData);

	/**
	 * Traces data from a Particle Handle using the provided CVD Context
	 * @param ParticleHandle Handle to process and Trace
	 * @param ContextData Context to be used to tie this Trace event with specific solver frame and step
	 */
	static CHAOS_API void TraceParticle(Chaos::FGeometryParticleHandle* ParticleHandle, const FChaosVDContext& ContextData);

	/**
	 * Traces the provided location using with the provided ID -
	 * These are not tied to any solver step so they will be recorded as part of the current game frame data
	 * @param InLocation Location to Trace
	 * @param DebugNameID Name to use as ID
	 */
	static void TraceNonSolverLocation(const FVector& InLocation, FStringView DebugNameID);

	/**
	 * Traces the provided transform using with the provided ID -
	 * These are not tied to any solver step so they will be recorded as part of the current game frame data
	 * @param InTransform Transform to Trace
	 * @param DebugNameID Name to use as ID
	 */
	static void TraceNonSolverTransform(const FTransform& InTransform, FStringView DebugNameID);

	/**
	 * Evaluated the current thread context to determine if we can trace the requested shape. It also updates the solver ID passed as reference
	 * if it was invalid, but we have a valid cvd context
	 * @param OutSolverID Solver ID to update if needed using the CVD context data
	 * @return True if we can trace shapes
	 */
	static bool CanTraceDebugDrawShape(int32& OutSolverID);
	
	/** Resets the state of the CVD Tracer */
	static void Reset();

	static void HandleRecordingStop();
	static void TraceArchiveHeader();
	static void HandleRecordingStart();

	/** Sets up the tracer to perform a full capture in the next solver frame */
	static void PerformFullCapture(EChaosVDFullCaptureFlags CaptureOptions);

	/** Setups the current Solver frame for a full capture if needed */
	static void SetupForFullCaptureIfNeeded(int32 SolverID, bool& bOutFullCaptureRequested);

	static Chaos::VD::FRecordingSessionState SessionState;

	friend class FChaosEngineModule;
};

template <typename ParticleType>
void FChaosVisualDebuggerTrace::TraceParticlesView(const Chaos::TParticleView<ParticleType>& ParticlesView)
{
	if (!IsTracing())
	{
		return;
	}

	const FChaosVDContext* CVDContextData = FChaosVDThreadContext::Get().GetCurrentContext(EChaosVDContextType::Solver);
	if (!ensure(CVDContextData))
	{
		return;
	}

	FChaosVDContext CopyContext = *CVDContextData;
	
	ParticlesView.ParallelFor([CopyContext](auto& Particle, int32 Index)
	{
		CVD_SCOPE_CONTEXT(CopyContext);
		TraceParticle(Particle.Handle());
	});
}

template <typename WorldType>
int32 FChaosVisualDebuggerTrace::GetSolverIDFromWorld(WorldType* World)
{
	int32 SolverID = INDEX_NONE;
	if (World)
	{
		if (Chaos::FPhysicsSolverBase* Solver = World->GetPhysicsScene() ? World->GetPhysicsScene()->GetSolver() : nullptr)
		{
			SolverID = GetSolverID(*Solver);
		}
	}

	return SolverID;
}

template <typename T>
void FChaosVisualDebuggerTrace::GetCVDContext(T& ObjectWithContext, FChaosVDContext& OutCVDContext)
{
	OutCVDContext = ObjectWithContext.GetChaosVDContextData();	
}

template <typename T>
FString FChaosVisualDebuggerTrace::GetDebugName(T& ObjectWithDebugName)
{
#if CHAOS_SOLVER_DEBUG_NAME
	return ObjectWithDebugName.GetDebugName().ToString();
#else
	return FString("COMPILED OUT");
#endif
}

struct FChaosVDScopeSolverStep
{
	FChaosVDScopeSolverStep(FStringView StepName)
	{
		CVD_TRACE_SOLVER_STEP_START(StepName);
	}

	~FChaosVDScopeSolverStep()
	{
		CVD_TRACE_SOLVER_STEP_END();
	}
};

template<typename T>
struct FChaosVDScopeSolverFrame
{
	FChaosVDScopeSolverFrame(T& InSolverRef) : SolverRef(InSolverRef)
	{
		CVD_TRACE_SOLVER_START_FRAME(T, SolverRef);
	}

	~FChaosVDScopeSolverFrame()
	{
		CVD_TRACE_SOLVER_END_FRAME(T, SolverRef);
	}

	T& SolverRef;
};

struct FChaosVDScopeSceneQueryVisit
{
	FChaosVDScopeSceneQueryVisit(FChaosVDQueryVisitStep& InVisitData) : VisitData(InVisitData)
	{
	}

	~FChaosVDScopeSceneQueryVisit()
	{
		CVD_TRACE_SCENE_QUERY_VISIT(MoveTemp(VisitData));
	}

	FChaosVDQueryVisitStep& VisitData;
};

namespace Chaos::VisualDebugger
{
	template<typename TDataToSerialize>
	void WriteDataToBuffer(TArray<uint8>& InOutDataBuffer, TDataToSerialize& Data)
	{
		FChaosVDMemoryWriter MemWriterAr(InOutDataBuffer, FChaosVisualDebuggerTrace::GetNameTableInstance());
		MemWriterAr.SetShouldSkipUpdateCustomVersion(true);

		Data.Serialize(MemWriterAr);
	}

	template<typename TDataToSerialize, typename TArchive>
	void WriteDataToBuffer(TArray<uint8>& InOutDataBuffer, TDataToSerialize& Data)
	{
		FChaosVDMemoryWriter MemWriterAr(InOutDataBuffer, FChaosVisualDebuggerTrace::GetNameTableInstance());
		TArchive Ar(MemWriterAr);
		Ar.SetShouldSkipUpdateCustomVersion(true);

		Data.Serialize(Ar);
	}

	template<typename TDataToSerialize>
	void WriteDataToBuffer(TArray<uint8>& InOutDataBuffer, TDataToSerialize& Data, const TSharedRef<FChaosArchiveContext>& InSharedArchiveContext)
	{
		FChaosVDMemoryWriter MemWriterAr(InOutDataBuffer, FChaosVisualDebuggerTrace::GetNameTableInstance());
		FChaosVDSharedContextArchive Ar(MemWriterAr, InSharedArchiveContext);
		Ar.SetShouldSkipUpdateCustomVersion(true);

		Data.Serialize(Ar);
	}
}
#endif // WITH_CHAOS_VISUAL_DEBUGGER
