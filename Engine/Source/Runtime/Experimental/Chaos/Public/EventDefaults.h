// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EventManager.h"

namespace Chaos
{
	class FEventDefaults
	{
	public:

		/**
		 * Register default event types
		 */
		static CHAOS_API void RegisterSystemEvents(FEventManager& EventManager);

	private:

		/**
		 * Register collision event gathering function & data type
		 */
		static void RegisterCollisionEvent(FEventManager& EventManager);

		/**
		 * Register breaking event gathering function & data type
		 */
		static void RegisterBreakingEvent(FEventManager& EventManager);

		/**
		 * Register trailing event gathering function & data type
		 */
		static void RegisterTrailingEvent(FEventManager& EventManager);

		/**
		 * Register sleeping event gathering function & data type
		 */
		static void RegisterSleepingEvent(FEventManager& EventManager);

		/**
		 * Register removal event gathering function & data type
		 */
		static void RegisterRemovalEvent(FEventManager& EventManager);

		/**
		* Register crumbling event gathering function & data type
		*/
		static void RegisterCrumblingEvent(FEventManager& EventManager);

	};
}
