// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Async/TransactionallySafeSharedRecursiveMutex.h"
#include "Async/SharedLock.h"
#include "Async/UniqueLock.h"
#include "Chaos/Framework/MultiBufferResource.h"
#include "Chaos/Framework/PhysicsProxyBase.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "Misc/ScopeLock.h"
#include "Chaos/Defines.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TVariant.h"


namespace Chaos
{
	namespace Private
	{
		extern bool bEventManagerDispatchEmptyEvents;
	}

	//#todo : need to enable disable events, or not register them if disabled.
	//#todo : add timers
	//#todo : warning if trying to add same event ID twice
	//#todo : need sparse array for EventID -> EventContainer?

	/**
	 * Predefined System Event Types
	 */
	enum class EEventType : int32
	{
		Collision = 0,
		Breaking = 1,
		Trailing = 2,
		Sleeping = 3,
		Removal = 4,
		Crumbling = 5,
	};

	typedef int32 FEventID;


	/**
	 * Interface for event handler 
	 */
	class IEventHandler
	{
		template<typename PayloadType>
		friend class TEventContainer;

	public:
		virtual ~IEventHandler() = default;
		virtual void HandleEvent(const void* EventData) const = 0;
		virtual bool GetInterestedProxyOwners(TArray<UObject*>& Output) const = 0;
		virtual void* GetHandler() const = 0;
	};

	/** Instance event handler */
	template<typename PayloadType, typename HandlerType>
	class TRawEventHandler : public IEventHandler
	{
	public:
		typedef void (HandlerType::*FHandlerFunction)(const PayloadType&);
		typedef TArray<UObject*> (HandlerType::*FInterestedProxyOwnerFunction)();

		TRawEventHandler(HandlerType* InHandler, FHandlerFunction InFunction, FInterestedProxyOwnerFunction InFunctionProxyOwners = nullptr)
			: Handler(InHandler)
			, HandlerFunction(InFunction)
			, InterestedProxyOwnersFunction(InFunctionProxyOwners)
		{
			check(Handler);
			check(HandlerFunction);
			// This can be a nullptr
			//check(InterestedProxyOwnersFunction);
		}

		virtual void HandleEvent(const void* EventData) const override
		{
			(Handler->*HandlerFunction)(*(const PayloadType*)EventData);
		}

		virtual bool GetInterestedProxyOwners(TArray<UObject*>& Output) const override
		{
			if (!InterestedProxyOwnersFunction)
			{
				return false;
			}
			
			Output = (Handler->*InterestedProxyOwnersFunction)();
			return true;
		}

		void* GetHandler() const override
		{
			return Handler;
		}

	private:
		HandlerType* Handler;
		FHandlerFunction HandlerFunction;
		// This function is used to get the proxies we are interested in, used for optimization
		// Will be a nullptr if the handler is interested in all proxies
		FInterestedProxyOwnerFunction InterestedProxyOwnersFunction;
	};

	/**
	 * Pointer to the event handler
	 */
	typedef IEventHandler* FEventHandlerPtr;

	/**
	 * Interface for the injected producer function and associated data buffer
	 */
	class FEventContainerBase
	{
	public:
		virtual ~FEventContainerBase() = default;
		/**
		 * Register the delegate function that will handle the events on the game thread
		 */
		virtual void RegisterHandler(const FEventHandlerPtr& Handler) = 0;

		/**
		 * Unregister the delegate function that handles the events on the game thread
		 */
		virtual void UnregisterHandler(const void* Handler) = 0;

		/*
		 * Inject data from the physics solver into the producer side of the buffer
		 */
		virtual void InjectProducerData(const FPBDRigidsSolver* Solver, bool bResetData) = 0;

		/**
		 * Flips the buffer if the buffer type is double or triple
		 */
		virtual void FlipBufferIfRequired() = 0;

		/**
		 * Reset the consumer buffer, can be used just before flipping buffers to start with a clean buffer
		 */
		virtual void ResetConsumerBuffer() = 0;

		/**
		 * Dispatch events to the registered handlers
		 */
		virtual void DispatchConsumerData() = 0;
	};

	/**
	 * Class that owns the injected producer function and its associated data buffer
	 */
	template<typename PayloadType>
	class TEventContainer : public FEventContainerBase
	{

	public:
		/**
		 * Regular constructor
		 */
		TEventContainer(const Chaos::EMultiBufferMode& BufferMode, TFunction<void(const FPBDRigidsSolver* Solver, PayloadType& EventDataInOut, bool bResetData)> InFunction)
			: InjectedFunction(InFunction)
			, EventBuffer(Chaos::FMultiBufferFactory<PayloadType>::CreateBuffer(BufferMode))
		{
		}

		/**
		 * Copy constructor
		 */
		TEventContainer(TEventContainer<PayloadType>& Other)
		{
			InjectedFunction = Other.InjectedFunction;
			EventBuffer = MoveTemp(Other.EventBuffer);
			bDispatchInProgress = Other.bDispatchInProgress;
		}

		/**
		 * Destructor cleans up memory
		 */
		~TEventContainer()
		{
			UE::TWriteScopeLock ScopeLock(HandlerLock);

			for (TPair<void*, FEventHandlerPtr>& Pair : HandlerMap)
			{
				delete Pair.Value;
				Pair.Value = nullptr;
			}
		}
#
		/**
		 * Register the delegate function that will handle the events on the game thread
		 */
		virtual void RegisterHandler(const FEventHandlerPtr& Handler)
		{
			check(IsInGameThread());

			if(bDispatchInProgress)
			{
				UE::TWriteScopeLock ScopeLock(DeferredHandlerLock);
				DeferredHandlers.AddUnique(Handler);

				return;
			}

			UE::TRWScopeLock ScopeLock(HandlerLock, SLT_Write);

			HandlerMap.Add(Handler->GetHandler(), Handler);
			TArray<UObject*> ProxyOwners;
			bool bValidProxyFilter = Handler->GetInterestedProxyOwners(ProxyOwners);

			if(bValidProxyFilter)
			{
				for(UObject* ProxyOwner : ProxyOwners)
				{
					ProxyOwnerToHandlerMap.Add(ProxyOwner, Handler);
				}
			}
			else
			{
				if(GetProxyToIndexMap(EventBuffer.Get()->GetConsumerBuffer()) != nullptr) // Only if our type supports getting the ProxyToIndexMap do we bother adding this
				{
					HandlersNotInProxyOwnerMap.Add(Handler->GetHandler(), Handler);
				}
			}
		}

		/**
		 * Unregister the delegate function that handles the events on the game thread
		 */
		virtual void UnregisterHandler(const void* InHandler)
		{
			check(IsInGameThread());

			if(bDispatchInProgress)
			{
				UE::TWriteScopeLock ScopeLock(DeferredHandlerLock);
				DeferredUnregisterHandlers.AddUnique(InHandler);

				return;
			}

			UE::TWriteScopeLock ScopeLock(HandlerLock);

			TArray<TPair<UObject*, FEventHandlerPtr>> KeysAndValuesToRemove;
			for(const TPair<UObject*, FEventHandlerPtr>& KeyValue : ProxyOwnerToHandlerMap)
			{
				const FEventHandlerPtr Value = KeyValue.Value;
				if(Value && Value->GetHandler() == InHandler)
				{
					KeysAndValuesToRemove.Add(KeyValue);
				}
			}

			for(const TPair<UObject*, FEventHandlerPtr>& KeyAndValue : KeysAndValuesToRemove)
			{
				ProxyOwnerToHandlerMap.Remove(KeyAndValue.Get<0>(), KeyAndValue.Get<1>());
			}

			HandlersNotInProxyOwnerMap.Remove(InHandler);

			if(FEventHandlerPtr* HandlerPtr = HandlerMap.Find(InHandler))
			{
				DeleteHandler(*HandlerPtr);
				HandlerMap.Remove(InHandler);
			}
		}

		/*
		 * Inject data from the physics solver into the producer side of the buffer
		 */
		virtual void InjectProducerData(const FPBDRigidsSolver* Solver, bool bResetData)
		{
			InjectedFunction(Solver, *EventBuffer->AccessProducerBuffer(), bResetData);
		}

		
		virtual void DestroyStaleEvents(TFunction<void(PayloadType & EventDataInOut)> InFunction)
		{
			InFunction(*EventBuffer->AccessProducerBuffer());
		}

		virtual void AddEvent(TFunction<void(PayloadType& EventDataInOut)> InFunction)
		{
			InFunction(*EventBuffer->AccessProducerBuffer());
		}

		/**
		 * Flips the buffer if the buffer type is double or triple
		 */
		virtual void FlipBufferIfRequired()
		{
			EventBuffer->FlipProducer();
		}

		/**
		 * Reset the consumer buffer, can be used just before flipping buffers to start with a clean buffer
		 */
		virtual void ResetConsumerBuffer()
		{
			EventBuffer->GetConsumerBufferMutable()->Reset();
		}

		/**
		 * Dispatch events to the registered handlers
		 */
		virtual void DispatchConsumerData()
		{
			check(IsInGameThread());

			const PayloadType* Buffer = EventBuffer.Get()->GetConsumerBuffer();

			// We dispatch events even when the event buffers are empty because the absence
			// of collisions/breaks/etc is required information for some use-cases like
			// tracking continuous collisions.
			const bool bDispatchEmptyEvents = Private::bEventManagerDispatchEmptyEvents;
			if (IsEventDataEmpty(Buffer) && !bDispatchEmptyEvents)
			{
				return;
			}

			// Unregister any handlers that should no longer be invoked
			UnregisterDeferedHandler();
			
			{
				// We're going to iterate the handlers and invoke them, read lock the list to block any changes
				UE::TReadScopeLock ScopeLock(HandlerLock);

				// Dispatch process beginning, we can no longer modify the handlers until the dispatch
				// completes, register and unregister will defer while this guard is set
				TGuardValue<bool> ReentrancyGuard(bDispatchInProgress, true);

				const TMap<IPhysicsProxyBase*, TArray<int32>>* Map = GetProxyToIndexMap(Buffer); // Use t his map to get all proxies used in the event buffer
				// Only take this path if we have fewer Events than Handlers
				if(Map && Map->Num() + HandlersNotInProxyOwnerMap.Num() < HandlerMap.Num())
				{
					if(!ProxyOwnerToHandlerMap.IsEmpty())
					{
						TSet<FEventHandlerPtr> UniqueHandlers;
						UniqueHandlers.Reserve(Map->Num());
						for(auto& KeyValue : *Map) // Only iterating over objects that are associated with events here
						{
							const IPhysicsProxyBase* Proxy = KeyValue.Get<0>();
							const UObject* Owner = Proxy->GetOwner();
							for(TMultiMap<UObject*, FEventHandlerPtr>::TConstKeyIterator It = ProxyOwnerToHandlerMap.CreateConstKeyIterator(Owner); It; ++It)
							{
								UniqueHandlers.Add(It.Value());
							}
						}

						for(const FEventHandlerPtr Handler : UniqueHandlers)
						{
							Handler->HandleEvent(Buffer);
						}
					}

					for(const TPair<void*, FEventHandlerPtr>& Pair : HandlersNotInProxyOwnerMap)
					{
						Pair.Value->HandleEvent(Buffer);
					}
				}
				else
					// This path is taken if there are fewer Handlers than events or the handler does not support GetProxyToIndexMap
				{
					for(const TPair<void*, FEventHandlerPtr>& Pair : HandlerMap)
					{
						Pair.Value->HandleEvent(Buffer);
					}
				}
			}

			// Register any new handlers enqueued during the dispatch process
			RegisterDeferedHandler();
		}

private:

	void UnregisterDeferedHandler()
	{
		TArray<const void*> DeferredRequests;

		{
			UE::TWriteScopeLock ScopeLock(DeferredHandlerLock);
			
			DeferredRequests = MoveTemp(DeferredUnregisterHandlers);
			check(DeferredUnregisterHandlers.Num() == 0);
		}

		for (const void* HandlerPtr : DeferredRequests)
		{
			UnregisterHandler(HandlerPtr);
		}
	}

	void RegisterDeferedHandler()
	{
		TArray<FEventHandlerPtr> DeferredRequests;

		{
			UE::TWriteScopeLock ScopeLock(DeferredHandlerLock);
			
			DeferredRequests = MoveTemp(DeferredHandlers);
			check(DeferredHandlers.Num() == 0);
		}

		for (const FEventHandlerPtr& HandlerPtr : DeferredRequests)
		{
			RegisterHandler(HandlerPtr);
		}
	}

		void DeleteHandler(FEventHandlerPtr& HandlerPtr)
		{
			delete HandlerPtr;
			HandlerPtr = nullptr;
		}

		/**
		 * The function that handles filling the event data buffer
		 */
		TFunction<void(const FPBDRigidsSolver* Solver, PayloadType& EventData, bool bResetData)> InjectedFunction;

		/**
		 * The data buffer that is filled by the producer and read by the consumer
		 */
		TUniquePtr<IBufferResource<PayloadType>> EventBuffer;

		TMultiMap<UObject*, FEventHandlerPtr> ProxyOwnerToHandlerMap; // Used to prevent us from iterating through the whole HandlerMap
		TMap<void*, FEventHandlerPtr> HandlersNotInProxyOwnerMap; // Handlers not added to ProxyOwnerToHandlerMap since they do not support it

		/**
		 * Delegate function registered to handle this event when it is dispatched
		 */
		TMap<void*, FEventHandlerPtr> HandlerMap;

		FTransactionallySafeRWLock HandlerLock; // protect access ProxyOwnerToHandlerMap, HandlersNotInProxyOwnerMap, HandlerMap
		TArray<FEventHandlerPtr> DeferredHandlers; // Store handler to register, they couldn't registered to avoid reentrant lock
		TArray<const void*> DeferredUnregisterHandlers; // Store handler to unregister, they couldn't unregistered to avoid reentrant lock
		FTransactionallySafeRWLock DeferredHandlerLock; // protect access to DeferredHandlers and DeferredUnregisterHandlers

		/** Guard flag for the dispatch process - used to defer requests to register or unregister new handlers during dispatch */
		bool bDispatchInProgress = false;
	};

	/**
	 * Pointer to event data buffer & injector functionality
	 */
	using FEventContainerBasePtr = FEventContainerBase*;

	class FEventManager
	{
		friend class FPBDRigidsSolver;

		struct FLockWriteScope
		{
			FLockWriteScope(const FLockWriteScope&) = delete;
			FLockWriteScope& operator=(const FLockWriteScope&) = delete;
			explicit FLockWriteScope(TVariant<FTransactionallySafeRWLock, UE::FTransactionallySafeSharedRecursiveMutex>& Mutex)
			{
				if (UE::FTransactionallySafeSharedRecursiveMutex* M = Mutex.TryGet<UE::FTransactionallySafeSharedRecursiveMutex>())
				{
					Lock.Emplace<UE::TUniqueLock<UE::FTransactionallySafeSharedRecursiveMutex>>(*M);
				}
				else
				{
					Lock.Emplace<UE::TWriteScopeLock<FTransactionallySafeRWLock>>(Mutex.Get<FTransactionallySafeRWLock>());
				}
			}
			TVariant<TYPE_OF_NULLPTR, UE::TWriteScopeLock<FTransactionallySafeRWLock>, UE::TUniqueLock<UE::FTransactionallySafeSharedRecursiveMutex>> Lock;
		};

		struct FLockReadScope
		{
			FLockReadScope(const FLockReadScope&) = delete;
			FLockReadScope& operator=(const FLockReadScope&) = delete;
			explicit FLockReadScope(TVariant<FTransactionallySafeRWLock, UE::FTransactionallySafeSharedRecursiveMutex>& Mutex)
			{
				if (UE::FTransactionallySafeSharedRecursiveMutex* M = Mutex.TryGet<UE::FTransactionallySafeSharedRecursiveMutex>())
				{
					Lock.Emplace<UE::TSharedLock<UE::FTransactionallySafeSharedRecursiveMutex>>(*M);
				}
				else
				{
					Lock.Emplace<UE::TReadScopeLock<FTransactionallySafeRWLock>>(Mutex.Get<FTransactionallySafeRWLock>());
				}
			}
			TVariant<TYPE_OF_NULLPTR, UE::TReadScopeLock<FTransactionallySafeRWLock>, UE::TSharedLock<UE::FTransactionallySafeSharedRecursiveMutex>> Lock;
		};

	public:

		CHAOS_API FEventManager(const Chaos::EMultiBufferMode& BufferModeIn);


		~FEventManager()
		{
			Reset();
		}

		/**
		 * Clears out every handler and container calling destructors on held items
		 */
		CHAOS_API void Reset();

		/**
		 * Set the buffer mode to be used within the event containers
		 */
		void SetBufferMode(const Chaos::EMultiBufferMode& BufferModeIn)
		{
			BufferMode = BufferModeIn;
		}

		/**
		 * Register a new event into the system, providing the function that will fill the producer side of the event buffer
		 */
		template<typename PayloadType>
		void RegisterEvent(const EEventType& EventType, TFunction<void(const Chaos::FPBDRigidsSolver* Solver, PayloadType& EventData, bool bResetData)> InFunction)
		{
			FLockWriteScope ContainerWriteScope(ContainerLock);
			InternalRegisterInjector(FEventID(EventType), new TEventContainer<PayloadType>(BufferMode, InFunction));
		}

		/**
		 * Modify the producer side of the event buffer
		 */
		template<typename PayloadType>
		void ClearEvents(const EEventType& EventType, TFunction<void(PayloadType & EventData)> InFunction)
		{
			FLockWriteScope ContainerWriteScope(ContainerLock);

			if (TEventContainer<PayloadType>* EventContainer = StaticCast<TEventContainer<PayloadType>*>(EventContainers[FEventID(EventType)]))
			{
				EventContainer->DestroyStaleEvents(InFunction);
			}
		}

		/**
		 * Unregister specified event from system
		 */
		CHAOS_API void UnregisterEvent(const EEventType& EventType);

		/**
		 * Register a handler that will receive the dispatched events
		 */
		template<typename PayloadType, typename HandlerType>
		void RegisterHandler(const EEventType& EventType, HandlerType* Handler, typename TRawEventHandler<PayloadType, HandlerType>::FHandlerFunction HandlerFunction, typename TRawEventHandler<PayloadType, HandlerType>::FInterestedProxyOwnerFunction InterestedProxyOwnerFunction = nullptr)
		{
			const FEventID EventID = FEventID(EventType);
			FLockReadScope ContainerReadScope(ContainerLock);
			checkf(EventID < EventContainers.Num(), TEXT("Registering event Handler for an event ID that does not exist"));
			EventContainers[EventID]->RegisterHandler(new TRawEventHandler<PayloadType, HandlerType>(Handler, HandlerFunction, InterestedProxyOwnerFunction));
		}

		/**
		 * Unregister the specified event handler
		 */
		CHAOS_API void UnregisterHandler(const EEventType& EventType, const void* InHandler);

		/**
		 * Called by the solver to invoke the functions that fill the producer side of all the event data buffers
		 */
		CHAOS_API void FillProducerData(const Chaos::FPBDRigidsSolver* Solver, bool bResetData = true);

		/**
		 * Flips the event data buffer if it is of double or triple buffer type
		 */
		CHAOS_API void FlipBuffersIfRequired();

		/**
		 * // Dispatch events to the registered handlers
		 */
		CHAOS_API void DispatchEvents();

		/** Returns encoded collision index. */
		static CHAOS_API int32 EncodeCollisionIndex(int32 ActualCollisionIndex, bool bSwapOrder);
		/** Returns decoded collision index. */
		static CHAOS_API int32 DecodeCollisionIndex(int32 EncodedCollisionIdx, bool& bSwapOrder);

		template<typename PayloadType>
		void AddEvent(const EEventType& EventType, TFunction<void(PayloadType& EventData)> InFunction)
		{
			auto ContainerLambda = [this, EventType, InFunction]()
				{
					FLockReadScope ContainerReadScope(ContainerLock);
					if (TEventContainer<PayloadType>* EventContainer = StaticCast<TEventContainer<PayloadType>*>(EventContainers[FEventID(EventType)]))
					{
						EventContainer->AddEvent(InFunction);
					}
				};

			if (BufferMode == EMultiBufferMode::Double)
			{
				FLockReadScope ResourceReadScope(ResourceLock);
				ContainerLambda();
			}
			else
			{
				ContainerLambda();
			}
		}

	private:

		CHAOS_API void InternalRegisterInjector(const FEventID& EventID, const FEventContainerBasePtr& Container);

		Chaos::EMultiBufferMode BufferMode;			// specifies the buffer type to be constructed, single, double, triple
		TArray<FEventContainerBasePtr> EventContainers;	// Array of event types
		
		TVariant<FTransactionallySafeRWLock, UE::FTransactionallySafeSharedRecursiveMutex> ResourceLock;

		TVariant<FTransactionallySafeRWLock, UE::FTransactionallySafeSharedRecursiveMutex> ContainerLock;

	};

}
