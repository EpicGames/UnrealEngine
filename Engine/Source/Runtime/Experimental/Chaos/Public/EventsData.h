// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ExternalCollisionData.h"
#include "Chaos/CollisionResolutionTypes.h"
#include "Chaos/Framework/PhysicsProxy.h"

class UPrimitiveComponent;

namespace Chaos
{
	// base class for data that requires time of creation to be recorded
	struct FTimeResource
	{
		FTimeResource() : TimeCreated(-TNumericLimits<FReal>::Max()) {}
		FReal TimeCreated;
	};

	typedef TArray<FCollidingData> FCollisionDataArray;
	typedef TArray<FBreakingData> FBreakingDataArray;
	typedef TArray<FTrailingData> FTrailingDataArray;
	typedef TArray<FRemovalData> FRemovalDataArray;
	typedef TArray<FSleepingData> FSleepingDataArray;
	typedef TArray<FCrumblingData> FCrumblingDataArray;

	/* Common */

	/* Maps PhysicsProxy to list of indices in events arrays 
	 * - for looking up say all collisions a particular physics object had this frame
	 */
	struct FIndicesByPhysicsProxy : public FTimeResource
	{
		FIndicesByPhysicsProxy()
			: PhysicsProxyToIndicesMap(TMap<IPhysicsProxyBase*, TArray<int32>>())
		{}

		void Reset()
		{
			PhysicsProxyToIndicesMap.Reset();
		}

		TMap<IPhysicsProxyBase*, TArray<int32>> PhysicsProxyToIndicesMap; // PhysicsProxy -> Indices in Events arrays
	};

	/* Collision */

	/*   
	 * All the collision events for one frame time stamped with the time for that frame
	 */
	struct FAllCollisionData : public FTimeResource
	{
		FAllCollisionData() : AllCollisionsArray(FCollisionDataArray()) {}

		void Reset()
		{
			AllCollisionsArray.Reset();
		}

		FCollisionDataArray AllCollisionsArray;
	};

	struct FCollisionEventData
	{
		FCollisionEventData() {}

		void Reset()
		{
			PhysicsProxyToCollisionIndices.PhysicsProxyToIndicesMap.Reset();
			CollisionData.Reset();
		}

		FAllCollisionData CollisionData;
		FIndicesByPhysicsProxy PhysicsProxyToCollisionIndices;
	};

	/* Breaking */

	/*
	 * All the breaking events for one frame time stamped with the time for that frame
	 */
	struct FAllBreakingData : public FTimeResource
	{
		FAllBreakingData() : AllBreakingsArray(FBreakingDataArray()), bHasGlobalEvent(false) {}

		void Reset()
		{
			AllBreakingsArray.Reset();
			bHasGlobalEvent = false;
		}

		FBreakingDataArray AllBreakingsArray;
		bool bHasGlobalEvent;
	};

	struct FBreakingEventData
	{
		FBreakingEventData() {}

		void Reset()
		{
			BreakingData.Reset();
			PhysicsProxyToBreakingIndices.PhysicsProxyToIndicesMap.Reset();
		}

		FAllBreakingData BreakingData;
		FIndicesByPhysicsProxy PhysicsProxyToBreakingIndices;
	};

	/* Trailing */

	/*
	 * All the trailing events for one frame time stamped with the time for that frame  
	 */
	struct FAllTrailingData : FTimeResource
	{
		FAllTrailingData() : AllTrailingsArray(FTrailingDataArray()), bHasGlobalEvent(false) {}

		void Reset()
		{
			AllTrailingsArray.Reset();
			bHasGlobalEvent = false;
		}

		FTrailingDataArray AllTrailingsArray;
		bool bHasGlobalEvent;
	};


	struct FTrailingEventData
	{
		FTrailingEventData() {}

		void Reset()
		{
			TrailingData.Reset();
			PhysicsProxyToTrailingIndices.Reset();
		}

		FAllTrailingData TrailingData;
		FIndicesByPhysicsProxy PhysicsProxyToTrailingIndices;
	};

	/* Removal */

	/*
	 * All the removal events for one frame time stamped with the time for that frame
	 */
	struct FAllRemovalData : FTimeResource
	{
		FAllRemovalData() : AllRemovalArray(FRemovalDataArray()) {}

		void Reset()
		{
			AllRemovalArray.Reset();
		}

		FRemovalDataArray AllRemovalArray;
	};


	struct FRemovalEventData
	{
		FRemovalEventData() {}

		void Reset()
		{
			RemovalData.Reset();
			PhysicsProxyToRemovalIndices.PhysicsProxyToIndicesMap.Reset();
		}

		FAllRemovalData RemovalData;
		FIndicesByPhysicsProxy PhysicsProxyToRemovalIndices;
	};

	struct FSleepingEventData
	{
		FSleepingEventData() {}

		void Reset()
		{
			SleepingData.Reset();
		}

		FSleepingDataArray SleepingData;
	};

	/*
	* All the crumbling events for one frame time stamped with the time for that frame
	*/
	struct FAllCrumblingData : public FTimeResource
	{
		FAllCrumblingData() : AllCrumblingsArray(FCrumblingDataArray()), bHasGlobalEvent(false) {}

		void Reset()
		{
			AllCrumblingsArray.Reset();
			bHasGlobalEvent = false;
		}

		FCrumblingDataArray AllCrumblingsArray;
		bool bHasGlobalEvent;
	};

	struct FCrumblingEventData
	{
		FCrumblingEventData() {}

		void Reset()
		{
			CrumblingData.Reset();
			PhysicsProxyToCrumblingIndices.Reset();
		}

		FORCEINLINE_DEBUGGABLE void Reserve(int32 Num)
		{
			CrumblingData.AllCrumblingsArray.Reserve(Num);
		}
		
		FORCEINLINE_DEBUGGABLE void SetTimeCreated(FReal TimeCreatedIn)
		{
			CrumblingData.TimeCreated = TimeCreatedIn;
		}

		FORCEINLINE_DEBUGGABLE void AddCrumbling(const FCrumblingData& CrumblingToAdd)
		{
			const int32 NewIndex = CrumblingData.AllCrumblingsArray.Emplace(CrumblingToAdd);
			TArray<int32>& Indices = PhysicsProxyToCrumblingIndices.PhysicsProxyToIndicesMap.FindOrAdd(CrumblingToAdd.Proxy);
			Indices.Add(NewIndex);
		}
		
		FAllCrumblingData CrumblingData;
		FIndicesByPhysicsProxy PhysicsProxyToCrumblingIndices;
	};

	template<typename PayloadType>
	bool IsEventDataEmpty(const PayloadType* Buffer)
	{
		if (!Buffer)
		{
			return false;
		}

		if constexpr (std::is_same_v<PayloadType, FCollisionEventData>)
		{
			return Buffer->CollisionData.AllCollisionsArray.IsEmpty();
		}
		else if constexpr (std::is_same_v<PayloadType, FBreakingEventData>)
		{
			return Buffer->BreakingData.AllBreakingsArray.IsEmpty();
		}
		else if constexpr (std::is_same_v<PayloadType, FTrailingEventData>)
		{
			return Buffer->TrailingData.AllTrailingsArray.IsEmpty();
		}
		else if constexpr (std::is_same_v<PayloadType, FRemovalEventData>)
		{
			return Buffer->RemovalData.AllRemovalArray.IsEmpty();
		}
		else if constexpr (std::is_same_v<PayloadType, FSleepingEventData>)
		{
			return Buffer->SleepingData.IsEmpty();
		}
		else if constexpr (std::is_same_v<PayloadType, FCrumblingEventData>)
		{
			return Buffer->CrumblingData.AllCrumblingsArray.IsEmpty();
		}
		else
		{
			return false;
		}
	}
	
	template<typename PayloadType>
	const TMap<IPhysicsProxyBase*, TArray<int32>>* GetProxyToIndexMap(const PayloadType* Buffer)
	{
		if (!Buffer)
		{
			return nullptr;
		}

		if constexpr (std::is_same_v<PayloadType, FCollisionEventData>)
		{
			return &Buffer->PhysicsProxyToCollisionIndices.PhysicsProxyToIndicesMap;
		}
		else if constexpr (std::is_same_v<PayloadType, FBreakingEventData>)
		{
			return &Buffer->PhysicsProxyToBreakingIndices.PhysicsProxyToIndicesMap;
		}
		else if constexpr (std::is_same_v<PayloadType, FTrailingEventData>)
		{
			return nullptr; //&Buffer->PhysicsProxyToTrailingIndices.PhysicsProxyToIndicesMap;
		}
		else if constexpr (std::is_same_v<PayloadType, FRemovalEventData>)
		{
			return &Buffer->PhysicsProxyToRemovalIndices.PhysicsProxyToIndicesMap;
		}
		else if constexpr (std::is_same_v<PayloadType, FSleepingEventData>)
		{
			return nullptr;
		}
		else if constexpr (std::is_same_v<PayloadType, FCrumblingEventData>)
		{
			return &Buffer->PhysicsProxyToCrumblingIndices.PhysicsProxyToIndicesMap;
		}
		else
		{
			return nullptr;
		}
	}

	// An index into a FCollisionEventData array of FCollidingDataobjects obtained from
	// the PhysicsProxyToIndicesMap for a specific proxy.
	// Indices held in FCollisionEventData::PhysicsProxyToCollisionIndices are encoded
	// to include the proxy index in the collision data structure (0 or 1).
	// This is useful if you collect a set of collision indices for a set of proxies, and
	// the store a flat list of FCollidingDataIndexes. In this case it may be expensive
	// to determine which of the proxies in the collision belongs to one of our proxies.
	class FCollidingDataIndex
	{
	public:
		FCollidingDataIndex() 
			: ProxyIndex(0)
			, CollisionIndex(INDEX_NONE)
		{
		}

		// InProxyIndex: The index of the proxy in the FCollidingData (0 or 1)
		// InCollisionIndex: The index of the FCollidingData in the FCollisionEventData array
		FCollidingDataIndex(int32 InCollisionIndex, int32 InProxyIndex)
			: ProxyIndex(InProxyIndex)
			, CollisionIndex(InCollisionIndex)
		{
			check(InCollisionIndex >= 0);
			check(InProxyIndex >= 0);
			check(InProxyIndex <= 1);
		}

		// Reset to invalid state
		void Reset()
		{
			ProxyIndex = 0;
			CollisionIndex = INDEX_NONE;
		}

		// Is this index a valid index into the FCollidingData array?
		bool IsValid() const
		{
			return CollisionIndex >= 0;
		}

		// The index into the FCollidingData array
		int32 GetIndex() const
		{
			return CollisionIndex;
		}

		// The index of our body in the collision data. This will be zero or one.
		int32 GetProxyIndex() const
		{
			return ProxyIndex;
		}

	private:
		uint32 ProxyIndex : 1;
		int32 CollisionIndex : 31;
	};

	// It wouldn't matter if it were larger, but just in case more flags are added and we do something unexpected...
	static_assert(sizeof(FCollidingDataIndex) == sizeof(int32), "FCollidingDataIndex should be same size as int32");

	// Iterates over the FCollidingData objects for a set of proxies
	//
	// Usage:
	//	for (FCollisionEventDataIterator It(MyProxyList, CollisionData); It; ++It)
	//	{
	//		Chaos::FCollidingDataIndex CollidingDataIndex = It.GetCurrentCollidingDataIndex();
	//		if (CollidingDataIndex.IsValid())
	//		{
	//			// CollidingData is between one ot the proxies in MyProxyList and another (which in general may also be in MyProxyList!)
	//			const Chaos::FCollidingData& CollidingData = CollisionEventData.CollisionData.AllCollisionsArray[CollidingDataIndex.GetIndex()];
	//			
	//			Chaos::FVec3 Normal = (CollidingDataIndex.GetProxyIndex() == 0) ? CollidingData.Normal : -CollidingData.Normal
	//			float Mass = (CollidingDataIndex.GetProxyIndex() == 0) ? CollidingData.Mass1 : CollidingData.Mass2;
	//			// ...
	//		}}
	//	}
	//
	class FCollisionEventDataIterator
	{
	public:
		CHAOS_API FCollisionEventDataIterator(const TArrayView<const IPhysicsProxyBase*>& InProxies, const Chaos::FCollisionEventData& InCollisionEventData);

		// Reset the iterator to point at the first FCollidingData for the first proxy that has any collisions.
		CHAOS_API void Reset();

		// Move to the next collision. If we have reached the end of collisions for the current proxy, 
		// move to the next proxy with collisions and select its first collision.
		CHAOS_API void Next();

		// Get the index of the current collision in the CollisionEventData's CollidingData array
		// or INDEX_NONE if we have reached the end of the collisions for all proxies.
		CHAOS_API FCollidingDataIndex GetCurrentCollidingDataIndex() const;

		// Get the Proxy that the iterator is currently pointing to
		// or null if we have reached the end of the collisions for all proxies.
		CHAOS_API const IPhysicsProxyBase* GetCurrentProxy() const;

		// Have we reached the end of the collisions for all proxies?
		bool IsFinished() const
		{
			return (ProxyIndex >= Proxies.Num());
		}

		// Returns true if we are still iterating (have not reached the end)
		operator bool() const
		{
			return !IsFinished();
		}

		// Move to the next non-empty collision/proxy
		FCollisionEventDataIterator& operator++()
		{
			Next();
			return *this;
		}

		// NOTE: Deliberately not providing a post-increment operator
		// because it is more expensive to copy than users might expect 
		// (although not that bad really, so feel free to add one if
		// you really need it!)
		FCollisionEventDataIterator operator++(int) = delete;

	private:
		// Current Iteration state
		int32 ProxyIndex;
		int32 ProxyCollisionIndex;
		const TArray<int32>* ProxyCollisionIndices;

		// Set of proxies we care about
		TArrayView<const IPhysicsProxyBase*> Proxies;

		// Collision data from the whole scene
		const Chaos::FCollisionEventData& CollisionEventData;
	};

}
