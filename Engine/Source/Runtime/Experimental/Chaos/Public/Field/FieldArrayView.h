// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"

/**
 * TFieldArrayView : Templated fixed-size view of another array
 *
 * A statically sized view of an TArray of typed elements. The reference
 * TArray could be resized and the TFieldArrayView will still be valid 
 * For now only used for the fields. Should be extended if required to be used somewhere else. 
*/

template<typename InElementType>
class TFieldArrayView
{
public:
	using ElementType = InElementType;
	using SizeType = int32;

	/**
	 * Constructor.
	 */
	TFieldArrayView(TArray<ElementType>& InElementArray, const SizeType InArrayOffset, const SizeType InArrayNum)
		: ElementArray(InElementArray)
		, ArrayOffset(InArrayOffset)
		, ArrayNum(InArrayNum)
	{
		checkf(InArrayOffset + InArrayNum <= ElementArray.Num(), TEXT("Field array size out of bounds: %i from an array of size %i"), InArrayOffset + InArrayNum, ElementArray.Num());
	}

	/**
	 * Helper function for returning a typed pointer to the first array entry.
	 *
	 * @returns Pointer to first array entry or nullptr if ArrayMax == 0.
	 */
	FORCEINLINE ElementType* GetData() const
	{
		return ElementArray.GetData() + ArrayOffset;
	}

	/**
	 * Helper function returning the size of the inner type.
	 *
	 * @returns Size in bytes of array type.
	 */
	FORCEINLINE static constexpr size_t GetTypeSize()
	{
		return sizeof(ElementType);
	}

	/**
	 * Helper function returning the alignment of the inner type.
	 */
	FORCEINLINE static constexpr size_t GetTypeAlignment()
	{
		return alignof(ElementType);
	}

	/**
	 * Checks array invariants: if array size is greater than zero and less
	 * than maximum.
	 */
	FORCEINLINE void CheckInvariants() const
	{
		checkSlow(ArrayNum >= 0);
	}

	/**
	 * Checks if index is in array range.
	 *
	 * @param Index Index to check.
	 */
	FORCEINLINE void RangeCheck(SizeType Index) const
	{
		CheckInvariants();

		checkf((Index >= 0) & (Index < ArrayNum), TEXT("Array index out of bounds: %i from an array of size %i"), Index, ArrayNum); // & for one branch
	}

	/**
	 * Tests if index is valid, i.e. more or equal to zero, and less than the number of elements in the array.
	 *
	 * @param Index Index to test.
	 *
	 * @returns True if index is valid. False otherwise.
	 */
	FORCEINLINE bool IsValidIndex(SizeType Index) const
	{
		return (Index >= 0) && (Index < ArrayNum);
	}

	/**
	 * Returns true if the array view is empty and contains no elements.
	 *
	 * @returns True if the array view is empty.
	 * @see Num
	 */
	bool IsEmpty() const
	{
		return ArrayNum == 0;
	}

	/**
	 * Returns number of elements in array.
	 *
	 * @returns Number of elements in array.
	 */
	FORCEINLINE SizeType Num() const
	{
		return ArrayNum;
	}

	/**
	 * Array bracket operator. Returns reference to element at give index.
	 *
	 * @returns Reference to indexed element.
	 */
	FORCEINLINE ElementType& operator[](SizeType Index) const
	{
		RangeCheck(Index);
		return ElementArray[ArrayOffset + Index];
	}

	/**
	  * DO NOT USE DIRECTLY
      * STL-like iterators to enable range-based for loop support.
    */
	FORCEINLINE ElementType* begin() const { return GetData(); }
	FORCEINLINE ElementType* end() const { return GetData() + Num(); }

private:

	TArray<InElementType>& ElementArray;
	SizeType ArrayOffset;
	SizeType ArrayNum;
};
