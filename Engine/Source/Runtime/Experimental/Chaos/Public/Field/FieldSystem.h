// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ParticleHandleFwd.h"
#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Queue.h"
#include "Field/FieldSystemTypes.h"
#include "Field/FieldArrayView.h"
#include "Math/Vector.h"

/**
* FFieldContextIndex
*   The Context is passed into the field evaluation pipeline during evaluation. The Nodes
*   will have access to the samples and indices for evaluation. The MetaData is a optional
*   data package that the nodes will use during evaluation, the context does not assume
*   ownership of the metadata but assumes it will remain in scope during evaluation.
*/
struct FFieldContextIndex
{
	FFieldContextIndex(int32 InSample = INDEX_NONE, int32 InResult = INDEX_NONE)
		: Sample(InSample)
		, Result(InResult)
	{}

	static void ContiguousIndices(
		TArray<FFieldContextIndex>& Array,
		const int NumParticles,
		const bool bForce = true)
	{
		if(bForce)
		{
			Array.SetNum(NumParticles);
			for(int32 i = 0; i < Array.Num(); ++i)
			{
				Array[i].Result = i;
				Array[i].Sample = i;
			}
		}
	}

	int32 Sample;
	int32 Result;
};

/** Enum to specify on which array the final field output will be stored for future use in rban/cloth*/
enum class EFieldCommandOutputType : uint8
{
	LinearForce = 0,
	LinearVelocity = 1,
	AngularTorque = 2,
	AngularVelocity = 3,
	NumOutputs = 4
};

/** Enum to specify on which array the intermediate fields results are going to be stored */
enum class EFieldCommandResultType : uint8
{
	FinalResult = 0,
	TransientResult = 1,
	NumResults = 2
};

/** Enum to specify on whjich array the particle handles are going to be stored */
enum class EFieldCommandHandlesType : uint8
{
	FilteredHandles = 0,
	InsideHandles = 1,
	NumHandles = 2
};

/** List of datas that will be stored during field evaluation to avoid reallocation */
struct FFieldExecutionDatas
{
	/** Sample positions to be used to build the context */
	TArray<FVector> SamplePositions;

	/** Sample indices to be used to build the context  */
	TArray<FFieldContextIndex> SampleIndices;

	/** List of particles handles used during evaluation  */
	TArray<Chaos::FGeometryParticleHandle*> ParticleHandles[(uint8)EFieldCommandHandlesType::NumHandles];

	/** Results for the target results */
	TArray<FVector> FieldOutputs[(uint8)EFieldCommandOutputType::NumOutputs];

	/** Field vector targets results */
	TArray<FVector> VectorResults[(uint8)EFieldCommandResultType::NumResults];

	/** Field scalar targets results */
	TArray<float> ScalarResults[(uint8)EFieldCommandResultType::NumResults];

	/** Field integer targets results */
	TArray<int32> IntegerResults[(uint8)EFieldCommandResultType::NumResults];

	/** Field index results */
	TArray<FFieldContextIndex> IndexResults[(uint8)EFieldCommandResultType::NumResults];
};

/** Reset the results array with a given size and a default value */
template<typename FieldType>
FORCEINLINE void ResetResultsArray(const int32 FieldSize, TArray<FieldType>& FieldArray, const FieldType DefaultValue)
{
	FieldArray.SetNum(FieldSize, EAllowShrinking::No);
	for (int32 i = 0; i < FieldSize; ++i)
	{
		FieldArray[i] = DefaultValue;
	}
}

/** Reset the results arrays given a list of targets */
template<typename FieldType>
FORCEINLINE void ResetResultsArrays(const int32 FieldSize, const TArray<EFieldCommandOutputType>& FieldTargets, TArray<FieldType> FieldArray[(uint8)EFieldCommandOutputType::NumOutputs], const FieldType DefaultValue)
{
	for (const EFieldCommandOutputType& FieldTarget : FieldTargets)
	{
		if (FieldTarget < EFieldCommandOutputType::NumOutputs)
		{
			ResetResultsArray(FieldSize, FieldArray[FieldTarget], DefaultValue);
		}
	}
}

/** Empty the results array without deallocating when shrinking*/
template<typename FieldType>
FORCEINLINE void EmptyResultsArray(TArray<FieldType>& FieldArray)
{
	FieldArray.SetNum(0, EAllowShrinking::No);
}

/** Empty the results arrays given a list of targets */
template<typename FieldType>
FORCEINLINE void EmptyResultsArrays(const TArray<EFieldCommandOutputType>& FieldTargets, TArray<FieldType> FieldArray[(uint8)EFieldCommandOutputType::NumOutputs])
{
	for (const EFieldCommandOutputType& FieldTarget : FieldTargets)
	{
		if (FieldTarget < EFieldCommandOutputType::NumOutputs)
		{
			EmptyResultsArray(FieldArray[(uint8)FieldTarget]);
		}
	}
}

/**
* MetaData
*
* Metadata is used to attach state based information to the field evaluation 
* pipeline. Contexts and Commands can store metadata that can be used by
* the Evaluate() of the field node, or during the processing of the command.
*/


class FFieldSystemMetaData {
public:

	enum EMetaType
	{
		ECommandData_None = 0,
		ECommandData_ProcessingResolution,
		ECommandData_Results,
		ECommandData_Iteration,
		ECommandData_Culling,
		ECommandData_Filter
	};


	virtual ~FFieldSystemMetaData() = default;
	virtual EMetaType Type() const = 0;
	virtual FFieldSystemMetaData* NewCopy() const = 0;
};


class FFieldSystemMetaDataProcessingResolution : public FFieldSystemMetaData {
public:
	FFieldSystemMetaDataProcessingResolution(EFieldResolutionType ProcessingResolutionIn) : ProcessingResolution(ProcessingResolutionIn) {};
	virtual ~FFieldSystemMetaDataProcessingResolution() = default;
	virtual EMetaType Type() const { return EMetaType::ECommandData_ProcessingResolution; }
	virtual FFieldSystemMetaData* NewCopy() const { return new FFieldSystemMetaDataProcessingResolution(ProcessingResolution); }

	EFieldResolutionType ProcessingResolution;
};

class FFieldSystemMetaDataFilter : public FFieldSystemMetaData {
public:
	FFieldSystemMetaDataFilter(EFieldFilterType FilterTypeIn, EFieldObjectType ObjectTypeIn, EFieldPositionType PositionTypeIn) : FilterType(FilterTypeIn), ObjectType(ObjectTypeIn), PositionType(PositionTypeIn)  {};
	virtual ~FFieldSystemMetaDataFilter() = default;
	virtual EMetaType Type() const { return EMetaType::ECommandData_Filter; }
	virtual FFieldSystemMetaData* NewCopy() const { return new FFieldSystemMetaDataFilter(FilterType, ObjectType, PositionType); }

	EFieldFilterType FilterType;
	EFieldObjectType ObjectType;
	EFieldPositionType PositionType;
};

template<class T>
class FFieldSystemMetaDataResults : public FFieldSystemMetaData {
public:
	FFieldSystemMetaDataResults(const TFieldArrayView<T>& ResultsIn) : Results(ResultsIn) {};
	virtual ~FFieldSystemMetaDataResults() = default;
	virtual EMetaType Type() const { return EMetaType::ECommandData_Results; }
	virtual FFieldSystemMetaData* NewCopy() const { return new FFieldSystemMetaDataResults(Results); }

	const TFieldArrayView<T>& Results;
};

class FFieldSystemMetaDataIteration : public FFieldSystemMetaData {
public:
	FFieldSystemMetaDataIteration(int32 IterationsIn) : Iterations(IterationsIn) {};
	virtual ~FFieldSystemMetaDataIteration() = default;
	virtual EMetaType Type() const { return EMetaType::ECommandData_Iteration; }
	virtual FFieldSystemMetaData* NewCopy() const { return new FFieldSystemMetaDataIteration(Iterations); }

	int32 Iterations;
};

class FFieldSystemMetaDataCulling : public FFieldSystemMetaData
{
public:
	explicit FFieldSystemMetaDataCulling(TArray<FFieldContextIndex>& CullingIndicesIn)
		: bCullingActive(false)
		, CullingIndices(CullingIndicesIn)
	{};

	virtual ~FFieldSystemMetaDataCulling() = default;

	virtual EMetaType Type() const
	{
		return EMetaType::ECommandData_Culling;
	}
	virtual FFieldSystemMetaData* NewCopy() const
	{
		return new FFieldSystemMetaDataCulling(CullingIndices);
	}

	bool bCullingActive;
	TArray<FFieldContextIndex>& CullingIndices;
};

struct FFieldContext
{
	typedef  TMap<FFieldSystemMetaData::EMetaType, TUniquePtr<FFieldSystemMetaData> > UniquePointerMap;
	typedef  TMap<FFieldSystemMetaData::EMetaType, FFieldSystemMetaData * > PointerMap;

	FFieldContext() = delete;

	FFieldContext(const FFieldContext&) = delete;
	FFieldContext(FFieldContext&&) = delete;
	FFieldContext& operator =(const FFieldContext&) = delete;
	FFieldContext & operator =(FFieldContext&&) = delete;

	FFieldContext(const TFieldArrayView< FFieldContextIndex >& SampleIndicesIn, const TFieldArrayView<FVector>& SamplePositionsIn,
		const UniquePointerMap & MetaDataIn, const Chaos::FReal TimeSecondsIn, TArray<FVector>& VectorResultsIn, TArray<float>& ScalarResultsIn,
		TArray<int32>& IntegerResultsIn, TArray<FFieldContextIndex>& IndexResultsIn, TArray<FFieldContextIndex>& CullingResultsIn)
		: SampleIndices(SampleIndicesIn)
		, SamplePositions(SamplePositionsIn)
		, TimeSeconds(TimeSecondsIn)
		, VectorResults(VectorResultsIn)
		, ScalarResults(ScalarResultsIn)
		, IntegerResults(IntegerResultsIn)
		, IndexResults(IndexResultsIn)

	{
		for (const TPair<FFieldSystemMetaData::EMetaType, TUniquePtr<FFieldSystemMetaData>>& Meta : MetaDataIn)
		{
			MetaData.Add(Meta.Key) = Meta.Value.Get();
		}

		CullingData = MakeUnique<FFieldSystemMetaDataCulling>(CullingResultsIn);
		MetaData.Add(FFieldSystemMetaData::EMetaType::ECommandData_Culling, CullingData.Get());
	}
	FFieldContext(const TFieldArrayView< FFieldContextIndex >& SampleIndicesIn, const TFieldArrayView<FVector>& SamplePositionsIn,
		const PointerMap & MetaDataIn, const Chaos::FReal TimeSecondsIn, TArray<FVector>& VectorResultsIn, TArray<float>& ScalarResultsIn,
				TArray<int32>& IntegerResultsIn, TArray<FFieldContextIndex>& IndexResultsIn, TArray<FFieldContextIndex>& CullingResultsIn)
		: SampleIndices(SampleIndicesIn)
		, SamplePositions(SamplePositionsIn)
		, MetaData(MetaDataIn)
		, TimeSeconds(TimeSecondsIn)
		, VectorResults(VectorResultsIn)
		, ScalarResults(ScalarResultsIn)
		, IntegerResults(IntegerResultsIn)
		, IndexResults(IndexResultsIn)
	{
		CullingData = MakeUnique<FFieldSystemMetaDataCulling>(CullingResultsIn);
		MetaData.Add(FFieldSystemMetaData::EMetaType::ECommandData_Culling, CullingData.Get());
	}

	FFieldContext(FFieldExecutionDatas& ExecutionDatas,
		const UniquePointerMap& MetaDataIn, const Chaos::FReal TimeSecondsIn)
		: SampleIndices(ExecutionDatas.SampleIndices, 0, ExecutionDatas.SampleIndices.Num())
		, SamplePositions(ExecutionDatas.SamplePositions, 0, ExecutionDatas.SamplePositions.Num())
		, TimeSeconds(TimeSecondsIn)
		, VectorResults(ExecutionDatas.VectorResults[(uint8)EFieldCommandResultType::TransientResult])
		, ScalarResults(ExecutionDatas.ScalarResults[(uint8)EFieldCommandResultType::TransientResult])
		, IntegerResults(ExecutionDatas.IntegerResults[(uint8)EFieldCommandResultType::TransientResult])
		, IndexResults(ExecutionDatas.IndexResults[(uint8)EFieldCommandResultType::TransientResult])
	{
		for (const TPair<FFieldSystemMetaData::EMetaType, TUniquePtr<FFieldSystemMetaData>>& Meta : MetaDataIn)
		{
			MetaData.Add(Meta.Key) = Meta.Value.Get();
		}

		CullingData = MakeUnique<FFieldSystemMetaDataCulling>(ExecutionDatas.IndexResults[(uint8)EFieldCommandResultType::FinalResult]);
		MetaData.Add(FFieldSystemMetaData::EMetaType::ECommandData_Culling, CullingData.Get());
	}

	TFieldArrayView<FFieldContextIndex> GetEvaluatedSamples()
	{
		if(!CullingData->bCullingActive)
		{
			// No culling took place
			return SampleIndices;
		}

		// Culling fields created an evaluation set
		return TFieldArrayView<FFieldContextIndex>(CullingData->CullingIndices, 0, CullingData->CullingIndices.Num());
	}

	//
	// Ryan - TODO: This concept of having discreet sample data needs to change.  
	// I think we'd be better off supplying lambda accessors which can be specialized 
	// for each respective use case.  That means the method by which this data is 
	// traversed also needs to change; possibly to some load balanced threaded iterator 
	// or task based paradigm.

	TFieldArrayView<FFieldContextIndex> SampleIndices;
	TFieldArrayView<FVector> SamplePositions;

	PointerMap MetaData;
	TUniquePtr<FFieldSystemMetaDataCulling> CullingData;
	Chaos::FReal TimeSeconds;

	TArray<FVector>& VectorResults;
	TArray<float>& ScalarResults;
	TArray<int32>& IntegerResults;
	TArray<FFieldContextIndex>& IndexResults;
};
/** Get the vector execution array given a result type */
template<typename ResultType>
FORCEINLINE TArray<ResultType>& GetResultArray(FFieldContext& FieldContext);

/** Get the vector execution array given a result type */
template<>
FORCEINLINE TArray<FVector>& GetResultArray<FVector>(FFieldContext& FieldContext)
{
	return FieldContext.VectorResults;
}

/** Get the scalar execution array given a result type */
template<>
FORCEINLINE TArray<float>& GetResultArray<float>(FFieldContext& FieldContext)
{
	return FieldContext.ScalarResults;
}

/** Get the integer execution array given a result type */
template<>
FORCEINLINE TArray<int32>& GetResultArray<int32>(FFieldContext& FieldContext)
{
	return FieldContext.IntegerResults;
}

/** 
 * Limits the application of a meta data object to a single scope.
 * This has the effect of exposing metadata to downstream nodes but making sure
 * upstream nodes cannot see it.
 */
class FScopedFieldContextMetaData
{
	FScopedFieldContextMetaData() = delete;
	FScopedFieldContextMetaData(const FScopedFieldContextMetaData&) = delete;
	FScopedFieldContextMetaData(FScopedFieldContextMetaData&&) = delete;
	FScopedFieldContextMetaData& operator=(const FScopedFieldContextMetaData&) = delete;
	FScopedFieldContextMetaData& operator=(FScopedFieldContextMetaData&&) = delete;

public:

	explicit FScopedFieldContextMetaData(FFieldContext& InContext, FFieldSystemMetaData* InMetaData)
		: TargetContext(InContext)
	{
		check(InMetaData);
		MetaType = InMetaData->Type();
		TargetContext.MetaData.Add(MetaType, InMetaData);
	}

	~FScopedFieldContextMetaData()
	{
		TargetContext.MetaData.Remove(MetaType);
	}

private:
	FFieldSystemMetaData::EMetaType MetaType;
	FFieldContext& TargetContext;
};

/**
* FFieldNodeBase
*
*  Abstract base class for the field node evaluation. 
*
*/
class FFieldNodeBase
{

public:

	enum EFieldType
	{
		EField_None = 0,
		EField_Results,
		EField_Int32,
		EField_Float,
		EField_FVector,
	};

	enum ESerializationType
	{
		FieldNode_Null = 0,
		FieldNode_FUniformInteger,
		FieldNode_FRadialIntMask,
		FieldNode_FUniformScalar,
		FieldNode_FRadialFalloff,
		FieldNode_FPlaneFalloff,
		FieldNode_FBoxFalloff,
		FieldNode_FNoiseField,
		FieldNode_FUniformVector,
		FieldNode_FRadialVector,
		FieldNode_FRandomVector,
		FieldNode_FSumScalar,
		FieldNode_FSumVector,
		FieldNode_FConversionField,
		FieldNode_FCullingField,
		FieldNode_FWaveScalar,
		FieldNode_FReturnResultsTerminal,
		FieldNode_FRotatedRadialVector
	};

	FFieldNodeBase() : Name("") {}
	virtual ~FFieldNodeBase() = default;
	virtual EFieldType Type() const { check(false); return EFieldType::EField_None; }
	virtual ESerializationType SerializationType() const { check(false); return ESerializationType::FieldNode_Null; }
	virtual FFieldNodeBase * NewCopy() const = 0;
	virtual void Serialize(FArchive& Ar) { Ar << Name; }
	virtual bool operator==(const FFieldNodeBase& Node) { return Name.IsEqual(Node.GetName()); }

	/** Count the number of offsets/params that will be used by the world physics field */
	virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const {}

	/** Fill the offsets/params arrays that will be used by the world physics field */
	virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const {}

	/** Evaluate the maximum magnitude of the field graph  */
	virtual float EvalMaxMagnitude() const { return 1.0; }

	/** Compute the min/max spatial bounds of the field */
	virtual void ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const
	{
		MinBounds = FVector(-FLT_MAX);
		MaxBounds = FVector(FLT_MAX);
		CenterPosition = FVector::Zero();
	}

	FName GetName() const { return Name; }
	void  SetName(const FName & NameIn) { Name = NameIn; }

private:
	FName Name;
};


/**
* FieldNode<T>
*
*  Typed field nodes are used for the evaluation of specific types of data arrays.
*  For exampe, The FFieldNode<FVector>::Evaluate(...) will expect resutls 
*  of type TFieldArrayView<FVector>, and an example implementation is the UniformVectorField.
*
*/
template<class T>
class FFieldNode : public FFieldNodeBase
{
public:
	
	virtual ~FFieldNode() = default;

	virtual void Evaluate(FFieldContext&, TFieldArrayView<T>& Results) const = 0;

	static EFieldType StaticType();
	virtual EFieldType Type() const { return StaticType(); }

	/** Count the number of offsets/params that will be used by the world physics field */
	virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override
	{
		++NumOffsets;
		NumParams += 2;
	}

	/** Fill the offsets/params arrays that will be used by the world physics field */
	virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override
	{
		NodesOffsets.Add(NodesParams.Num());
		NodesParams.Add(static_cast<float>(Type()));
		NodesParams.Add(static_cast<float>(SerializationType()));
	}
};

template<> inline FFieldNodeBase::EFieldType FFieldNode<int32>::StaticType() { return EFieldType::EField_Int32; }
template<> inline FFieldNodeBase::EFieldType FFieldNode<float>::StaticType() { return EFieldType::EField_Float; }
template<> inline FFieldNodeBase::EFieldType FFieldNode<FVector>::StaticType() { return EFieldType::EField_FVector; }

/**
* FieldCommand
*
*   Field commands are issued on the game thread and trigger field
*   evaluation during game play. The Commands will store the root
*   node in the evaluation graph, and will trigger a full evaluation
*   of all the nodes in the graph. The MetaData within the command
*   will be passed to the evaluation of the field. 
*
*/
class FFieldSystemCommand
{
public:
	FFieldSystemCommand()
		: TargetAttribute("")
		, RootNode(nullptr)
		, CommandName("")
		, TimeCreation(0.0)
		, BoundingBox(FVector(-FLT_MAX), FVector(FLT_MAX))
		, PhysicsType(EFieldPhysicsType::Field_None)
		, MaxMagnitude(1.0)
		, CenterPosition(FVector::Zero())
	{}
	FFieldSystemCommand(const FName& TargetAttributeIn, FFieldNodeBase * RootNodeIn)
		: TargetAttribute(TargetAttributeIn)
		, RootNode(RootNodeIn)
		, CommandName("")
		, TimeCreation(0.0)
		, BoundingBox(FVector(-FLT_MAX), FVector(FLT_MAX))
		, PhysicsType(GetFieldPhysicsType(TargetAttributeIn))
		, MaxMagnitude(1.0)
		, CenterPosition(FVector::Zero())
	{}
	FFieldSystemCommand(const EFieldPhysicsType PhsyicsTypeIn, FFieldNodeBase* RootNodeIn)
		: TargetAttribute(GetFieldPhysicsName(PhsyicsTypeIn))
		, RootNode(RootNodeIn)
		, CommandName("")
		, TimeCreation(0.0)
		, BoundingBox(FVector(-FLT_MAX), FVector(FLT_MAX))
		, PhysicsType(PhsyicsTypeIn)
		, MaxMagnitude(1.0)
		, CenterPosition(FVector::Zero())
	{}

	// Commands are copied when moved from the one thread to 
	// another. This requires a full copy of all associated data. 
	FFieldSystemCommand(const FFieldSystemCommand& Other)
		: TargetAttribute(Other.RootNode ? Other.TargetAttribute:"")
		, RootNode(Other.RootNode?Other.RootNode->NewCopy():nullptr)
		, CommandName(Other.CommandName)
		, TimeCreation(Other.TimeCreation)
		, BoundingBox(Other.BoundingBox)
		, PhysicsType(Other.RootNode ? Other.PhysicsType : EFieldPhysicsType::Field_None)
		, MaxMagnitude(Other.MaxMagnitude)
		, CenterPosition(Other.CenterPosition)
	{
		for (const TPair<FFieldSystemMetaData::EMetaType, TUniquePtr<FFieldSystemMetaData>>& Meta : Other.MetaData)
		{
			MetaData.Add(Meta.Key).Reset(Meta.Value->NewCopy());
		}
	}

	bool HasMetaData(const FFieldSystemMetaData::EMetaType Key) const
	{
		return MetaData.Contains(Key) && GetMetaData(Key) != nullptr;
	}

	const TUniquePtr<FFieldSystemMetaData>& GetMetaData(
		const FFieldSystemMetaData::EMetaType Key) const
	{
		return MetaData[Key];
	}

	template <class TMetaData>
	const TMetaData* GetMetaDataAs(const FFieldSystemMetaData::EMetaType Key) const
	{
		return static_cast<const TMetaData*>(GetMetaData(Key).Get());
	}

	void SetMetaData(const FFieldSystemMetaData::EMetaType Key, TUniquePtr<FFieldSystemMetaData>&& Value)
	{
		MetaData.Add(Key, MoveTemp(Value));
	}

	void SetMetaData(const FFieldSystemMetaData::EMetaType Key, FFieldSystemMetaData* Value)
	{
		MetaData[Key].Reset(Value);
	}

	void InitFieldNodes(const double TimeSeconds, const FName& Name)
	{
		CommandName = Name;
		TimeCreation = (float)TimeSeconds;
	}

	CHAOS_API void Serialize(FArchive& Ar);
	CHAOS_API bool operator==(const FFieldSystemCommand&) const;
	bool operator!=(const FFieldSystemCommand& Other) const { return !this->operator==(Other); }

	FName TargetAttribute;
	TUniquePtr<FFieldNodeBase> RootNode;

	FName CommandName;
	float TimeCreation;

	FBox BoundingBox;
	EFieldPhysicsType PhysicsType;
	float MaxMagnitude;
	FVector CenterPosition;

	TMap<FFieldSystemMetaData::EMetaType, TUniquePtr<FFieldSystemMetaData> > MetaData;
};



/*
* Equality testing for pointer wrapped FieldNodes
*/
template<class T>
bool FieldsEqual(const TUniquePtr<T>& NodeA, const TUniquePtr<T>& NodeB)
{
	if (NodeA.IsValid() == NodeB.IsValid())
	{
		if (NodeA.IsValid())
		{
			if (NodeA->SerializationType() == NodeB->SerializationType())
			{
				return NodeA->operator==(*NodeB);
			}
		}
		else
		{
			return true;
		}
	}
	return false;
}



