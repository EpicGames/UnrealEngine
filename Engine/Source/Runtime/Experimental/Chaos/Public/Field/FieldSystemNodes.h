// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Field/FieldSystem.h"
#include "Math/Vector.h"
#include "Chaos/Box.h"

/**
* FUniformInteger
**/

class FUniformInteger : public FFieldNode<int32>
{
	typedef FFieldNode<int32> Super;

public:

	FUniformInteger(int32 MagnitudeIn = 0)
		: Super()
		, Magnitude(MagnitudeIn)
	{}
	virtual FFieldNodeBase * NewCopy() const override { return new FUniformInteger(Magnitude); }
	virtual ~FUniformInteger() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<int32>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FUniformInteger; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	int32 Magnitude;
};


/**
* RadialMaskField
**/

class FRadialIntMask : public FFieldNode<int32>
{
	typedef FFieldNode<int32> Super;

public:

	FRadialIntMask(float RadiusIn = 0, 
		FVector PositionIn = FVector(0, 0, 0), 
		int32 InteriorValueIn = 1.0, 
		int32 ExteriorValueIn = 0, 
		ESetMaskConditionType SetMaskConditionIn = ESetMaskConditionType::Field_Set_Always)
		: Super()
		, Radius(RadiusIn)
		, Position(PositionIn)
		, InteriorValue(InteriorValueIn)
		, ExteriorValue(ExteriorValueIn)
		, SetMaskCondition(SetMaskConditionIn)
	{}
	virtual FFieldNodeBase * NewCopy() const override { return new FRadialIntMask(Radius,Position,InteriorValue,ExteriorValue,SetMaskCondition); }
	virtual ~FRadialIntMask() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<int32>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FRadialIntMask; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	/** Compute the min/max spatial bounds of the field */
	CHAOS_API virtual void ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const override;

	float Radius;
	FVector Position;
	int32 InteriorValue;
	int32 ExteriorValue;
	ESetMaskConditionType SetMaskCondition;
};


/**
* FUniformScalar
**/

class FUniformScalar : public FFieldNode<float>
{
	typedef FFieldNode<float> Super;

public:

	FUniformScalar(float MagnitudeIn = 1.f)
		: Super()
		, Magnitude(MagnitudeIn)
	{}
	virtual FFieldNodeBase * NewCopy() const override { return new FUniformScalar(Magnitude); }

	virtual ~FUniformScalar() = default;

	CHAOS_API void Evaluate(FFieldContext&, TFieldArrayView<float>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node);

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FUniformScalar; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	float Magnitude;
};

/**
* FWaveScalar
**/

class FWaveScalar : public FFieldNode<float>
{
	typedef FFieldNode<float> Super;

public:

	FWaveScalar(float MagnitudeIn = 1.f, const FVector& PositionIn = FVector(0,0,0), const float WavelengthIn = 1000, const float PeriodIn = 1.0, 
		const EWaveFunctionType FunctionIn = EWaveFunctionType::Field_Wave_Cosine, const EFieldFalloffType FallofffIn = EFieldFalloffType::Field_Falloff_Linear)
		: Super()
		, Magnitude(MagnitudeIn)
		, Position(PositionIn)
		, Wavelength(WavelengthIn)
		, Period(PeriodIn)
		, Function(FunctionIn)
		, Falloff(FallofffIn)
	{}
	virtual FFieldNodeBase* NewCopy() const override { return new FWaveScalar(Magnitude, Position, Wavelength, Period, Function, Falloff); }

	virtual ~FWaveScalar() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<float>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FWaveScalar; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	/** Compute the min/max spatial bounds of the field */
	CHAOS_API virtual void ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const override;

	float Magnitude;
	FVector Position;
	float Wavelength;
	float Period;
	EWaveFunctionType Function;
	EFieldFalloffType Falloff;
};

/**
* RadialFalloff
**/

class FRadialFalloff : public FFieldNode<float>
{
	typedef FFieldNode<float> Super;

public:

	FRadialFalloff( 
		float MagnitudeIn = 1.f,
		float MinRangeIn = 0.f,
		float MaxRangeIn = 1.f,
		float DefaultIn = 0.f,
		float RadiusIn = 0.f,
		FVector PositionIn = FVector(0, 0, 0),
		EFieldFalloffType FalloffIn = EFieldFalloffType::Field_Falloff_Linear)
		: Super()
		, Magnitude(MagnitudeIn)
		, MinRange(MinRangeIn)
		, MaxRange(MaxRangeIn)
		, Default(DefaultIn)
		, Radius(RadiusIn)
		, Position(PositionIn)
		, Falloff(FalloffIn)
	{}
	virtual FFieldNodeBase * NewCopy() const override { return new FRadialFalloff(Magnitude,MinRange,MaxRange,Default,Radius,Position,Falloff); }

	virtual ~FRadialFalloff() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<float>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FRadialFalloff; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	/** Compute the min/max spatial bounds of the field */
	CHAOS_API virtual void ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const override;

	float Magnitude;
	float MinRange;
	float MaxRange;
	float Default;
	float Radius;
	FVector Position;
	EFieldFalloffType Falloff;

protected:
	template<EFieldFalloffType> void Evaluator(const FFieldContext& Context, TFieldArrayView<float>& Results) const;
};


/**
* FPlaneFalloff
**/
class FPlaneFalloff : public FFieldNode<float>
{
	typedef FFieldNode<float> Super;

public:

	FPlaneFalloff(
		float MagnitudeIn = 1.f,
		float MinRangeIn = 0.f,
		float MaxRangeIn = 1.f,
		float DefaultIn = 0.f,
		float DistanceIn = 0.f,
		FVector PositionIn = FVector(0, 0, 0),
		FVector NormalIn = FVector(0, 0, 1),
		EFieldFalloffType FalloffIn = EFieldFalloffType::Field_Falloff_Linear)

		: Super()
		, Magnitude(MagnitudeIn)
		, MinRange(MinRangeIn)
		, MaxRange(MaxRangeIn)
		, Default(DefaultIn)
		, Distance(DistanceIn)
		, Position(PositionIn)
		, Normal(NormalIn)
		, Falloff(FalloffIn)
	{}
	virtual FFieldNodeBase * NewCopy() const override { return new FPlaneFalloff(Magnitude, MinRange, MaxRange, Default, Distance, Position, Normal, Falloff); }
	virtual ~FPlaneFalloff() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<float>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FPlaneFalloff; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	/** Compute the min/max spatial bounds of the field */
	CHAOS_API virtual void ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const override;


	float Magnitude;
	float MinRange;
	float MaxRange;
	float Default;
	float Distance;
	FVector Position;
	FVector Normal;
	EFieldFalloffType Falloff;

protected :
	template<EFieldFalloffType> void Evaluator(const FFieldContext& Context, const FPlane& Plane, TFieldArrayView<float>& Results) const;

};

/**
* FBoxFalloff
**/
class FBoxFalloff : public FFieldNode<float>
{
	typedef FFieldNode<float> Super;

public:

	FBoxFalloff(
		float MagnitudeIn = 1.f,
		float MinRangeIn = 0.f,
		float MaxRangeIn = 1.f,
		float DefaultIn = 0.f,
		FTransform TransformIn = FTransform::Identity,
		EFieldFalloffType FalloffIn = EFieldFalloffType::Field_Falloff_Linear)

		: Super()
		, Magnitude(MagnitudeIn)
		, MinRange(MinRangeIn)
		, MaxRange(MaxRangeIn)
		, Default(DefaultIn)
		, Transform(TransformIn)
		, Falloff(FalloffIn)
	{}
	virtual FFieldNodeBase * NewCopy() const override { return new FBoxFalloff(Magnitude, MinRange, MaxRange, Default, Transform, Falloff); }
	virtual ~FBoxFalloff() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<float>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FBoxFalloff; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	/** Compute the min/max spatial bounds of the field */
	CHAOS_API virtual void ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const override;

	float Magnitude;
	float MinRange;
	float MaxRange;
	float Default;
	FTransform Transform;
	EFieldFalloffType Falloff;

protected:
	template<EFieldFalloffType> void Evaluator(const FFieldContext& Context, TFieldArrayView<float>& Results) const;

};


/**
* NoiseField
**/
class FNoiseField : public FFieldNode<float>
{
	typedef FFieldNode<float> Super;

public:

	FNoiseField(float MinRangeIn = 0.f, float MaxRangeIn = 0.f, FTransform TransformIn = FTransform::Identity)
		: Super()
		, MinRange(MinRangeIn)
		, MaxRange(MaxRangeIn)
		, Transform(TransformIn)
	{}
	virtual FFieldNodeBase * NewCopy() const override { return new FNoiseField(MinRange,MaxRange,Transform); }
	virtual ~FNoiseField() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<float>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FNoiseField; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	float MinRange;
	float MaxRange;
	FTransform Transform;
};

/**
* UniformVector
**/

class FUniformVector : public FFieldNode<FVector>
{
	typedef FFieldNode<FVector> Super;

public:

	FUniformVector(float MagnitudeIn = 1.f,
		FVector DirectionIn =FVector(0, 0, 0))
		: Super()
		, Magnitude(MagnitudeIn)
		, Direction(DirectionIn)
	{}
	virtual FFieldNodeBase * NewCopy() const override { return new FUniformVector(Magnitude,Direction); }
	virtual ~FUniformVector() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<FVector>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FUniformVector; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	float Magnitude;
	FVector Direction;
};

/**
* RadialVector
**/

class FRadialVector : public FFieldNode<FVector>
{
	typedef FFieldNode<FVector> Super;

public:

	FRadialVector(float MagnitudeIn = 1.f,
		FVector PositionIn = FVector(0, 0, 0))
		: Super()
		, Magnitude(MagnitudeIn)
		, Position(PositionIn)
	{}
	virtual FFieldNodeBase * NewCopy() const override { return new FRadialVector(Magnitude,Position); }
	virtual ~FRadialVector() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<FVector>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FRadialVector; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	/** Compute the min/max spatial bounds of the field */
	CHAOS_API virtual void ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const override;

	float Magnitude;
	FVector Position;

};

/**
* RotatedRadialVector
**/

class FRotatedRadialVector : public FFieldNode<FVector>
{
	typedef FFieldNode<FVector> Super;

public:

	FRotatedRadialVector(float MagnitudeIn = 1.f,
		FVector PositionIn = FVector(0, 0, 0),
		float AngleIn = 0.f,
		FVector AxisIn = FVector(0, 0, 1))
		: Super()
		, Magnitude(MagnitudeIn)
		, Position(PositionIn)
		, Angle(AngleIn)
		, Axis(AxisIn)
	{}
	virtual FFieldNodeBase* NewCopy() const override { return new FRotatedRadialVector(Magnitude, Position, Angle, Axis); }
	virtual ~FRotatedRadialVector() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<FVector>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FRotatedRadialVector; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	/** Compute the min/max spatial bounds of the field */
	CHAOS_API virtual void ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const override;

	float Magnitude;
	FVector Position;
	float Angle;
	FVector Axis;
};

/**
* RandomVector
**/

class FRandomVector : public FFieldNode<FVector>
{
	typedef FFieldNode<FVector> Super;

public:

	FRandomVector(float MagnitudeIn = 1.f)
		: Super()
		, Magnitude(MagnitudeIn)
	{}
	virtual FFieldNodeBase * NewCopy() const override { return new FRandomVector(Magnitude); }
	virtual ~FRandomVector() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<FVector>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FRandomVector; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	float Magnitude;
};

/**
* SumScalar
**/

class FSumScalar : public FFieldNode<float>
{
	typedef FFieldNode<float> Super;

public:

	FSumScalar(float MagnitudeIn = 1.f,
		FFieldNode<float> * ScalarRightIn = nullptr,
		FFieldNode<float> * ScalarLeftIn = nullptr,
		EFieldOperationType OperationIn = EFieldOperationType::Field_Multiply)
		: Super()
		, Magnitude(MagnitudeIn)
		, ScalarRight(TUniquePtr< FFieldNode<float> >(ScalarRightIn))
		, ScalarLeft(TUniquePtr< FFieldNode<float> >(ScalarLeftIn))
		, Operation(OperationIn)
	{}
	virtual FFieldNodeBase * NewCopy() const override
	{
		return new FSumScalar(Magnitude,
			ScalarRight.Get() ? static_cast<FFieldNode<float> *>(ScalarRight->NewCopy()) : nullptr,
			ScalarLeft.Get() ? static_cast<FFieldNode<float> *>(ScalarLeft->NewCopy()) : nullptr,
			Operation);
	}
	virtual ~FSumScalar() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<float>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FSumScalar; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	/** Compute the min/max spatial bounds of the field */
	CHAOS_API virtual void ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const override;

	/** Default setup for the sum scalar field */
	static void FillDefaultSetup(TArray<int32>& NodesOffsets, TArray<float>& NodesParams)
	{
		NodesOffsets.Add(NodesParams.Num());
		NodesParams.Add(static_cast<float>(Super::StaticType()));
		NodesParams.Add(static_cast<float>(FFieldNodeBase::ESerializationType::FieldNode_FSumScalar));
		NodesParams.Add(1.0);
		NodesParams.Add(1.0);
		NodesParams.Add(1.0);
		NodesParams.Add(static_cast<float>(EFieldOperationType::Field_Add));
	}

	float Magnitude;
	TUniquePtr< FFieldNode<float> > ScalarRight;
	TUniquePtr< FFieldNode<float> > ScalarLeft;
	EFieldOperationType Operation;
};

/**
* SumVector
**/

class FSumVector : public FFieldNode<FVector>
{
	typedef FFieldNode<FVector> Super;

public:

	FSumVector(float MagnitudeIn = 1.f,
		FFieldNode<float> * ScalarIn = nullptr,
		FFieldNode<FVector> * VectorRightIn = nullptr,
		FFieldNode<FVector> * VectorLeftIn = nullptr,
		EFieldOperationType OperationIn = EFieldOperationType::Field_Multiply )
		: Super()
		, Magnitude(MagnitudeIn)
		, Scalar(TUniquePtr< FFieldNode<float> >(ScalarIn) )
		, VectorRight(TUniquePtr< FFieldNode<FVector> >(VectorRightIn) )
		, VectorLeft(TUniquePtr< FFieldNode<FVector> >(VectorLeftIn) )
		, Operation(OperationIn)
	{}
	virtual FFieldNodeBase * NewCopy() const override
	{
		return new FSumVector(Magnitude,
			Scalar.Get() ? static_cast<FFieldNode<float> *>(Scalar->NewCopy()) : nullptr,
			VectorRight.Get() ? static_cast<FFieldNode<FVector> *>(VectorRight->NewCopy()) : nullptr,
			VectorLeft.Get() ? static_cast<FFieldNode<FVector> *>(VectorLeft->NewCopy()) : nullptr,
			Operation);
	}
	virtual ~FSumVector() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<FVector>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FSumVector; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	/** Compute the min/max spatial bounds of the field */
	CHAOS_API virtual void ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const override;

	/** Default setup for the sum vector field */
	static void FillDefaultSetup(TArray<int32>& NodesOffsets, TArray<float>& NodesParams)
	{
		NodesOffsets.Add(NodesParams.Num());
		NodesParams.Add(static_cast<float>(Super::StaticType()));
		NodesParams.Add(static_cast<float>(FFieldNodeBase::ESerializationType::FieldNode_FSumVector));
		NodesParams.Add(1.0);
		NodesParams.Add(0.0);
		NodesParams.Add(1.0);
		NodesParams.Add(1.0);
		NodesParams.Add(static_cast<float>(EFieldOperationType::Field_Add));
	}

	float Magnitude;
	TUniquePtr< FFieldNode<float> > Scalar;
	TUniquePtr< FFieldNode<FVector> > VectorRight;
	TUniquePtr< FFieldNode<FVector> > VectorLeft;
	EFieldOperationType Operation;
};


/**
* FConversionField
**/
template<class InT, class OutT>
class FConversionField : public FFieldNode<OutT>
{
	typedef FFieldNode<OutT> Super;

public:

	FConversionField(FFieldNode<InT> * InFieldIn = nullptr)
		: Super()
		, InputField(TUniquePtr< FFieldNode<InT> >(InFieldIn))
	{}
	virtual FFieldNodeBase * NewCopy() const override 
	{ 
		return new FConversionField(InputField.Get() ? static_cast<FFieldNode<InT> *>(InputField->NewCopy()) : nullptr);
	}
	virtual ~FConversionField() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<OutT>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FConversionField; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	/** Compute the min/max spatial bounds of the field */
	CHAOS_API virtual void ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const override;

	/** Default setup for the conversion field */
	static void FillDefaultSetup(TArray<int32>& NodesOffsets, TArray<float>& NodesParams)
	{
		NodesOffsets.Add(NodesParams.Num());
		NodesParams.Add(static_cast<float>(Super::StaticType()));
		NodesParams.Add(static_cast<float>(FFieldNodeBase::ESerializationType::FieldNode_FConversionField));
		NodesParams.Add(1.0);
	}

	TUniquePtr< FFieldNode<InT> > InputField;
};

/**
* FCullingField
**/

template<class T>
class FCullingField : public FFieldNode<T>
{
	typedef FFieldNode<T> Super;

public:

	FCullingField( 
		FFieldNode<float> * CullingIn = nullptr,
		FFieldNode<T> * InputIn = nullptr,
		EFieldCullingOperationType OperationIn = EFieldCullingOperationType::Field_Culling_Inside)

		: Super()
		, Culling(TUniquePtr< FFieldNode<float> >(CullingIn))
		, Input(TUniquePtr< FFieldNode<T> >(InputIn))
		, Operation(OperationIn)
	{}
	virtual FFieldNodeBase * NewCopy() const override
	{
		return new FCullingField<T>(
			Culling.Get() ? static_cast<FFieldNode<float> *>(Culling->NewCopy()) : nullptr,
			Input.Get() ? static_cast<FFieldNode<T> *>(Input->NewCopy()) : nullptr,
			Operation);
	}
	virtual ~FCullingField() = default;

	CHAOS_API virtual void Evaluate(FFieldContext&, TFieldArrayView<T>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node) override;

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const override { return FFieldNodeBase::ESerializationType::FieldNode_FCullingField; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;

	/** Count the number of offsets/params that will be used by the world physics field */
	CHAOS_API virtual void FillSetupCount(int32& NumOffsets, int32& NumParams) const override;

	/** Fill the offsets/params arrays that will be used by the world physics field */
	CHAOS_API virtual void FillSetupDatas(TArray<int32>& NodesOffsets, TArray<float>& NodesParams, const float CommandTime) const override;

	/** Evaluate the maximum magnitude of the field graph  */
	CHAOS_API virtual float EvalMaxMagnitude() const override;

	/** Compute the min/max spatial bounds of the field */
	CHAOS_API virtual void ComputeFieldBounds(FVector& MinBounds, FVector& MaxBounds, FVector& CenterPosition) const override;

	TUniquePtr< FFieldNode<float> > Culling;
	TUniquePtr< FFieldNode<T> > Input;
	EFieldCullingOperationType Operation;
};

#if PLATFORM_MAC
extern template class CHAOS_API FCullingField<int32>;
extern template class CHAOS_API FCullingField<float>;
extern template class CHAOS_API FCullingField<FVector>;
#endif

/**
* FReturnResultsTerminal
**/

template<class T>
class FReturnResultsTerminal : public FFieldNode<T>
{
	typedef FFieldNode<T> Super;

public:

	FReturnResultsTerminal() : Super()
	{}
	virtual FFieldNodeBase * NewCopy() const override
	{
		return new FReturnResultsTerminal<T>();
	}
	virtual ~FReturnResultsTerminal() = default;

	CHAOS_API void Evaluate(FFieldContext&, TFieldArrayView<T>& Results) const override;
	CHAOS_API virtual bool operator==(const FFieldNodeBase& Node);

	/** Serialization API */
	virtual FFieldNodeBase::ESerializationType SerializationType() const { return FFieldNodeBase::ESerializationType::FieldNode_FReturnResultsTerminal; }
	CHAOS_API virtual void Serialize(FArchive& Ar) override;
};

#if PLATFORM_MAC
extern template class CHAOS_API FReturnResultsTerminal<int32>;
extern template class CHAOS_API FReturnResultsTerminal<float>;
extern template class CHAOS_API FReturnResultsTerminal<FVector>;
#endif

/*
* Serialization Factory
*/
FFieldNodeBase * FieldNodeFactory(FFieldNodeBase::EFieldType BaseType, FFieldNodeBase::ESerializationType Type);
