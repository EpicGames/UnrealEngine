// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Field/FieldSystem.h"
#include "Math/Vector.h"

namespace Field
{
	// Basic scalar implementation of Perlin's improved noise, copied from VectorVM to remove the engine dependency.
	// http://mrl.nyu.edu/~perlin/noise/
	namespace PerlinNoise
	{
		void CHAOS_API Sample(float* Dst, float X, float Y, float Z);
	}
}