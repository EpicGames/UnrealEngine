// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryCollection/ManagedArrayCollection.h"
#include "UObject/ObjectMacros.h"

#include "FieldSystemTypes.generated.h"

/**
*
*/
UENUM(BlueprintType)
enum ESetMaskConditionType : int
{
	Field_Set_Always	        UMETA(DisplayName = "Set Always", ToolTip = "The particle output value will be equal to Interior-value if the particle position is inside a sphere / Exterior-value otherwise."),
	Field_Set_IFF_NOT_Interior  UMETA(DisplayName = "Merge Interior", ToolTip = "The particle output value will be equal to Interior-value if the particle position is inside the sphere or if the particle input value is already Interior-Value / Exterior-value otherwise."),
	Field_Set_IFF_NOT_Exterior  UMETA(DisplayName = "Merge Exterior", ToolTip = "The particle output value will be equal to Exterior-value if the particle position is outside the sphere or if the particle input value is already Exterior-Value / Interior-value otherwise."),
	//~~~
	//256th entry
	Field_MaskCondition_Max                 UMETA(Hidden)
};

/**
*
*/
UENUM(BlueprintType)
enum EWaveFunctionType : int
{
	Field_Wave_Cosine	 UMETA(DisplayName = "Cosine", ToolTip = "Cosine wave that will move in time."),
	Field_Wave_Gaussian  UMETA(DisplayName = "Gaussian", ToolTip = "Gaussian wave that will move in time."),
	Field_Wave_Falloff  UMETA(DisplayName = "Falloff", ToolTip = "The radial falloff radius will move along temporal wave."),
	Field_Wave_Decay  UMETA(DisplayName = "Decay", ToolTip = "The magnitude of the field will decay in time."),
	//~~~
	//256th entry
	Field_Wave_Max       UMETA(Hidden)
};

/**
*
*/
UENUM(BlueprintType)
enum EFieldOperationType : int
{
	Field_Multiply  UMETA(DisplayName = "Multiply", ToolTip = "Multiply the fields output values : Output = Left * Right"),
	Field_Divide    UMETA(DisplayName = "Divide", ToolTip = "Divide the fields output values : Output = Left / Right"),
	Field_Add       UMETA(DisplayName = "Add", ToolTip = "Add the fields output values : Output = Left + Right"),
	Field_Substract UMETA(DisplayName = "Subtract", ToolTip = "Subtract the fields output : Output = Left - Right"),
	//~~~
	//256th entry
	Field_Operation_Max                 UMETA(Hidden)
};

/**
*
*/
UENUM(BlueprintType)
enum EFieldCullingOperationType : int
{
	Field_Culling_Inside  UMETA(DisplayName = "Inside", ToolTip = "Evaluate the input field if the result of the culling field is equal to 0"),
	Field_Culling_Outside UMETA(DisplayName = "Outside", ToolTip = "Evaluate the input field if the result of the culling field is different from 0"),
	//~~~
	//256th entry
	Field_Culling_Operation_Max                 UMETA(Hidden)
};


/**
*
*/
UENUM(BlueprintType)
enum EFieldResolutionType : int
{
	Field_Resolution_Minimal  UMETA(DisplayName = "Minimum", ToolTip = "Apply the field to all the active particles"),
	Field_Resolution_DisabledParents  UMETA(DisplayName = "Parents", ToolTip = "Apply the field to all the parent particles"),
	Field_Resolution_Maximum  UMETA(DisplayName = "Maximum", ToolTip = "Apply the field to all the solver particles"),
	//~~~
	//256th entry
	Field_Resolution_Max      UMETA(Hidden)
};

/**
*
*/
UENUM(BlueprintType)
enum EFieldFilterType : int
{
	Field_Filter_Dynamic  UMETA(DisplayName = "Dynamic", ToolTip = "Apply the field to all the dynamic particles"),
	Field_Filter_Kinematic  UMETA(DisplayName = "Kinematic", ToolTip = "Apply the field to all the kinematic particles"),
	Field_Filter_Static  UMETA(DisplayName = "Static", ToolTip = "Apply the field to all the static particles"),
	Field_Filter_All  UMETA(DisplayName = "All", ToolTip = "Apply the field to all the solver particles"),
	Field_Filter_Sleeping  UMETA(DisplayName = "Sleeping", ToolTip = "Apply the field to all the sleeping particles"),
	Field_Filter_Disabled  UMETA(DisplayName = "Disabled", ToolTip = "Apply the field to all the disabled particles"),
	Field_Filter_DynamicAndSleeping  UMETA(DisplayName = "Dynamic & Sleeping", ToolTip = "Apply the field to all the dynamic and sleeping particles"),
	//~~~
	//256th entry
	Field_Filter_Max      UMETA(Hidden)
};

/**
*
*/
UENUM(BlueprintType)
enum EFieldObjectType : int
{
	Field_Object_Rigid  UMETA(DisplayName = "Rigid", ToolTip = "Apply the field to all the rigid particles"),
	Field_Object_Cloth  UMETA(DisplayName = "Cloth", ToolTip = "Apply the field to all the cloth particles"),
	Field_Object_Destruction  UMETA(DisplayName = "Destruction", ToolTip = "Apply the field to all the destruction particles"),
	Field_Object_Character UMETA(DisplayName = "Character", ToolTip = "Apply the field to all the character particles"),
	Field_Object_All  UMETA(DisplayName = "All", ToolTip = "Apply the field to all the objects particles"),
	//~~~
	//256th entry
	Field_Object_Max      UMETA(Hidden)
};

/**
*
*/
UENUM(BlueprintType)
enum EFieldPositionType : int
{
	Field_Position_CenterOfMass  UMETA(DisplayName = "CenterOfMass", ToolTip = "Apply the field to the particles center of mass position"),
	Field_Position_PivotPoint  UMETA(DisplayName = "PivotPoint", ToolTip = "Apply the field to the particles pivot point position"),
	//~~~
	//256th entry
	Field_Position_Max      UMETA(Hidden)
};

/**
*
*/
UENUM(BlueprintType)
enum EFieldFalloffType : int
{
	Field_FallOff_None			UMETA(DisplayName = "None", ToolTip = "No falloff function is used"),
	Field_Falloff_Linear		UMETA(DisplayName = "Linear", ToolTip = "The falloff function will be proportional to x"),
	Field_Falloff_Inverse		UMETA(DisplayName = "Inverse", ToolTip = "The falloff function will be proportional to 1.0/x"),
	Field_Falloff_Squared		UMETA(DisplayName = "Squared", ToolTip = "The falloff function will be proportional to x*x"),
	Field_Falloff_Logarithmic	UMETA(DisplayName = "Logarithmic", ToolTip = "The falloff function will be proportional to log(x)"),
	//~~~
	//256th entry
	Field_Falloff_Max           UMETA(Hidden)
};

/**
*
*/
UENUM(BlueprintType)
enum EFieldPhysicsType : int
{
	Field_None						UMETA(Hidden),
	Field_DynamicState				UMETA(DisplayName = "Dynamic State", ToolTip = "Set the dynamic state of a particle (static, dynamic, kinematic...)"),
	Field_LinearForce				UMETA(DisplayName = "Linear Force", ToolTip = "Add a vector field to the particles linear force."),
	Field_ExternalClusterStrain		UMETA(DisplayName = "External Strain", ToolTip = "Apply an external strain over the particles. If this strain is over the internal one, the cluster will break."),
	Field_Kill   					UMETA(DisplayName = "Kill Particle", ToolTip = "Disable the particles for which the field will be higher than 0."),
	Field_LinearVelocity			UMETA(DisplayName = "Linear Velocity", ToolTip = "Add a vector field to the particles linear velocity."),
	Field_AngularVelociy			UMETA(DisplayName = "Angular Velocity", ToolTip = "Add a vector field to the particles angular velocity."),
	Field_AngularTorque				UMETA(DisplayName = "Angular Torque", ToolTip = "Add a vector field to the particles angular torque."),
	Field_InternalClusterStrain		UMETA(DisplayName = "Internal Strain", ToolTip = "Add a strain field to the particles internal one."),
	Field_DisableThreshold			UMETA(DisplayName = "Disabled Threshold", ToolTip = "Disable the particles if their linear and angular velocity are less than the threshold."),
	Field_SleepingThreshold			UMETA(DisplayName = "Sleeping Threshold", ToolTip = "Set particles in sleeping mode if their linear and angular velocity are less than the threshold."),
	Field_PositionStatic			UMETA(DisplayName = "Position Static", ToolTip = "Add a position constraint to the particles to remain static", Hidden),
	Field_PositionAnimated			UMETA(DisplayName = "Position Animated", ToolTip = "Add a position constraint to the particles to follow its kinematic position", Hidden),
	Field_PositionTarget			UMETA(DisplayName = "Position Target", ToolTip = "Add a position constraint to the particles to follow a target position", Hidden),
	Field_DynamicConstraint			UMETA(DisplayName = "Dynamic Constraint", ToolTip = "Add the particles to a spring constraint holding them together", Hidden),
	Field_CollisionGroup			UMETA(DisplayName = "Collision Group", ToolTip = "Set the particles collision group."),
	Field_ActivateDisabled			UMETA(DisplayName = "Activate Disabled", ToolTip = "Activate all the disabled particles for which the field value will be 0"),
	Field_InitialLinearVelocity		UMETA(DisplayName = "Initial Linear Velocity", ToolTip = "Set the geometry collection initial linear velocity", Hidden),
	Field_InitialAngularVelocity	UMETA(DisplayName = "Initial Angular Velocity", ToolTip = "Set the geometry collection initial angular velocity", Hidden),
	Field_LinearImpulse				UMETA(DisplayName = "Linear Impulse", ToolTip = "Add a vector field affecting the particles linear impulse.", Hidden),
	//~~~
	//256th entry
	Field_PhysicsType_Max           UMETA(Hidden)
};

// TODO : Refactor these 3 enums to be in sync with the GetFieldTargetTypes
UENUM(BlueprintType)
enum EFieldVectorType : int
{
	Vector_LinearForce				UMETA(DisplayName = "Linear Force", ToolTip = "Add a vector field to the particles linear force."),
	Vector_LinearVelocity			UMETA(DisplayName = "Linear Velocity", ToolTip = "Add a vector field to the particles linear velocity."),
	Vector_AngularVelocity			UMETA(DisplayName = "Angular Velocity", ToolTip = "Add a vector field to the particles angular velocity."),
	Vector_AngularTorque			UMETA(DisplayName = "Angular Torque", ToolTip = "Add a vector field to the particles angular torque."),
	Vector_PositionTarget			UMETA(DisplayName = "Position Target", ToolTip = "Add a position constraint to the particles to follow a target position", Hidden),
	Vector_InitialLinearVelocity	UMETA(DisplayName = "Initial Linear Velocity", ToolTip = "Set the geometry collection initial linear velocity", Hidden),
	Vector_InitialAngularVelocity	UMETA(DisplayName = "Initial Angular Velocity", ToolTip = "Set the geometry collection initial angular velocity", Hidden),
	Vector_LinearImpulse			UMETA(DisplayName = "Linear Impulse", ToolTip = "Add a vector field affecting the particles linear impulse.", Hidden),

	//~~~
	//256th entry
	Vector_TargetMax           UMETA(Hidden)
};

UENUM(BlueprintType)
enum EFieldScalarType : int
{
	Scalar_ExternalClusterStrain		UMETA(DisplayName = "External Strain", ToolTip = "Apply an external strain over the particles. If this strain is over the internal one, the cluster will break."),
	Scalar_Kill   						UMETA(DisplayName = "Kill Particle", ToolTip = "Disable the particles for which the field will be higher than 0."),
	Scalar_DisableThreshold				UMETA(DisplayName = "Disabled Threshold", ToolTip = "Disable the particles if their linear and angular velocity are less than the threshold."),
	Scalar_SleepingThreshold			UMETA(DisplayName = "Sleeping Threshold", ToolTip = "Set particles in sleeping mode if their linear and angular velocity are less than the threshold."),
	Scalar_InternalClusterStrain		UMETA(DisplayName = "Internal Strain", ToolTip = "Add a strain field to the particles internal one."),
	Scalar_DynamicConstraint			UMETA(DisplayName = "Dynamic Constraint", ToolTip = "Add the particles to a spring constraint holding them together", Hidden),
	//~~~
	//256th entry
	Scalar_TargetMax           UMETA(Hidden)
};

UENUM(BlueprintType)
enum EFieldIntegerType : int
{
	Integer_DynamicState				UMETA(DisplayName = "Dynamic State", ToolTip = "Set the dynamic state of a particle (static, dynamic, kinematic...)"),
	Integer_ActivateDisabled			UMETA(DisplayName = "Activate Disabled", ToolTip = "Activate all the disabled particles for which the field value will be 0"),
	Integer_CollisionGroup				UMETA(DisplayName = "Collision Group", ToolTip = "Set the particles collision group."),
	Integer_PositionAnimated			UMETA(DisplayName = "Position Animated", ToolTip = "Add a position constraint to the particles to follow its kinematic position", Hidden),
	Integer_PositionStatic				UMETA(DisplayName = "Position Static", ToolTip = "Add a position constraint to the particles to remain static", Hidden),
	//~~~
	//256th entry
	Integer_TargetMax           UMETA(Hidden)
};

/** Defines the type of the output*/
UENUM()
enum EFieldOutputType : int
{
	/* Vector Field Type */
	Field_Output_Vector UMETA(DisplayName = "Vector Field"),

	/* Scalar Field Type */
	Field_Output_Scalar UMETA(DisplayName = "Scalar Field"),

	/* Integer field type */
	Field_Output_Integer UMETA(DisplayName = "Integer Field"),

	Field_Output_Max UMETA(Hidden)
};

inline
TArray<EFieldPhysicsType> GetFieldTargetTypes(EFieldOutputType OutputType)
{
	TArray<EFieldPhysicsType> PhysicsTypes;
	switch (OutputType)
	{
	case Field_Output_Vector:
	{
		PhysicsTypes = { EFieldPhysicsType::Field_LinearForce,
						 EFieldPhysicsType::Field_LinearVelocity,
						 EFieldPhysicsType::Field_AngularVelociy,
						 EFieldPhysicsType::Field_AngularTorque,
						 EFieldPhysicsType::Field_PositionTarget,
						 EFieldPhysicsType::Field_InitialLinearVelocity,
						 EFieldPhysicsType::Field_InitialAngularVelocity,
						 EFieldPhysicsType::Field_LinearImpulse };
		break;
	}
	case Field_Output_Scalar:
	{
		PhysicsTypes = { EFieldPhysicsType::Field_ExternalClusterStrain,
						 EFieldPhysicsType::Field_Kill,
						 EFieldPhysicsType::Field_DisableThreshold,
						 EFieldPhysicsType::Field_SleepingThreshold,
						 EFieldPhysicsType::Field_InternalClusterStrain,
					     EFieldPhysicsType::Field_DynamicConstraint };
		break;
	}
	case Field_Output_Integer:
	{
		PhysicsTypes = { EFieldPhysicsType::Field_DynamicState,
						 EFieldPhysicsType::Field_ActivateDisabled,
						 EFieldPhysicsType::Field_CollisionGroup,
						 EFieldPhysicsType::Field_PositionAnimated,
						 EFieldPhysicsType::Field_PositionStatic };
		break;
	}
	default: 
		break;
	}
	return MoveTemp(PhysicsTypes);
}

inline
EFieldOutputType GetFieldTargetIndex(const TArray<EFieldPhysicsType>& VectorTypes, 
											   const TArray<EFieldPhysicsType>& ScalarTypes, 
											   const TArray<EFieldPhysicsType>& IntegerTypes,
											   const EFieldPhysicsType FieldTarget, int32& TargetIndex)
{
	EFieldOutputType OutputType = EFieldOutputType::Field_Output_Max;

	TargetIndex = VectorTypes.Find(FieldTarget);
	if (TargetIndex == INDEX_NONE)
	{
		TargetIndex = ScalarTypes.Find(FieldTarget);
		if (TargetIndex == INDEX_NONE)
		{
			TargetIndex = IntegerTypes.Find(FieldTarget);
			if (TargetIndex != INDEX_NONE)
			{
				OutputType = EFieldOutputType::Field_Output_Integer;
			}
		}
		else
		{
			OutputType = EFieldOutputType::Field_Output_Scalar;
		}
	}
	else
	{
		OutputType = EFieldOutputType::Field_Output_Vector;
	}
	return OutputType;
}

inline
EFieldOutputType GetFieldTargetOutput(const EFieldPhysicsType FieldTarget)
{
	EFieldOutputType OutputType = EFieldOutputType::Field_Output_Max;

	static const TArray<EFieldPhysicsType> VectorTypes = GetFieldTargetTypes(EFieldOutputType::Field_Output_Vector);
	static const TArray<EFieldPhysicsType> ScalarTypes = GetFieldTargetTypes(EFieldOutputType::Field_Output_Scalar);
	static const TArray<EFieldPhysicsType> IntegerTypes = GetFieldTargetTypes(EFieldOutputType::Field_Output_Integer);

	int32 TargetIndex = INDEX_NONE;
	return GetFieldTargetIndex(VectorTypes, ScalarTypes, IntegerTypes, FieldTarget, TargetIndex);
}

inline
FName GetFieldOutputName(const EFieldOutputType Type)
{
	switch (Type)
	{
	case EFieldOutputType::Field_Output_Vector:
		return "Vector";
	case EFieldOutputType::Field_Output_Scalar:
		return "Scalar";
	case EFieldOutputType::Field_Output_Integer:
		return "Integer";
	}
	return "None";
}

inline const TArray<FName>& GetFieldPhysicsNames()
{
	static const TArray<FName> FieldPhysicsNames =
	{ "Node", "DynamicState", "LinearForce", "ExternalClusterStrain", "Kill", "LinearVelocity", "AngularVelocity",
	 "AngularTorque", "InternalClusterStrain", "DisableThreshold", "SleepingThreshold", "PositionStatic",
	 "PositionAnimated", "PositionTarget", "DynamicConstraint", "CollisionGroup", "ActivateDisabled", "InitialLinearVelocity", "InitialAngularVelocity", "LinearImpulse"};

	return FieldPhysicsNames;
}

inline const TArray<EFieldPhysicsType>& GetFieldPhysicsTypes()
{
	static const TArray<EFieldPhysicsType> FieldPhysicsTypes ={ EFieldPhysicsType::Field_None, 
																EFieldPhysicsType::Field_DynamicState, 
																EFieldPhysicsType::Field_LinearForce, 
																EFieldPhysicsType::Field_ExternalClusterStrain, 
																EFieldPhysicsType::Field_Kill, 
																EFieldPhysicsType::Field_LinearVelocity, 
																EFieldPhysicsType::Field_AngularVelociy,
																EFieldPhysicsType::Field_AngularTorque, 
																EFieldPhysicsType::Field_InternalClusterStrain, 
																EFieldPhysicsType::Field_DisableThreshold, 
																EFieldPhysicsType::Field_SleepingThreshold, 
																EFieldPhysicsType::Field_PositionStatic,
																EFieldPhysicsType::Field_PositionAnimated, 
																EFieldPhysicsType::Field_PositionTarget, 
																EFieldPhysicsType::Field_DynamicConstraint, 
																EFieldPhysicsType::Field_CollisionGroup,
																EFieldPhysicsType::Field_ActivateDisabled, 
																EFieldPhysicsType::Field_InitialLinearVelocity, 
																EFieldPhysicsType::Field_InitialAngularVelocity,
																EFieldPhysicsType::Field_LinearImpulse };

	return FieldPhysicsTypes;
}

inline const FName& GetFieldPhysicsName(EFieldPhysicsType PhysicsType)
{
	const TArray<FName>& FieldPhysicsNames = GetFieldPhysicsNames();
	static const FName NoneField;

	return (PhysicsType < FieldPhysicsNames.Num()) ? FieldPhysicsNames[PhysicsType] : NoneField;
}

inline EFieldPhysicsType GetFieldPhysicsType(const FName& PhysicsName)
{
	const TArray<FName>& FieldPhysicsNames = GetFieldPhysicsNames();
	const int32 PhysicsIndex = FieldPhysicsNames.Find(PhysicsName);

	return ((PhysicsIndex != INDEX_NONE) && (PhysicsIndex < EFieldPhysicsType::Field_PhysicsType_Max)) ? 
		(EFieldPhysicsType)PhysicsIndex : EFieldPhysicsType::Field_None;
}

/**
*
*/
UENUM(BlueprintType)
enum EFieldPhysicsDefaultFields : int
{
	Field_RadialIntMask				UMETA(DisplayName = "RadialIntMask"),
	Field_RadialFalloff				UMETA(DisplayName = "RadialFalloff"),
	Field_UniformVector				UMETA(DisplayName = "UniformVector"),
	Field_RadialVector				UMETA(DisplayName = "RadialVector"),
	Field_RadialVectorFalloff		UMETA(DisplayName = "RadialVectorFalloff"),
	//~~~
	//256th entry
	Field_EFieldPhysicsDefaultFields_Max                 UMETA(Hidden)
};

USTRUCT()
struct FFieldCollection : public FManagedArrayCollection
{
	GENERATED_USTRUCT_BODY()
public:
	static FName StaticType() { return FName("FFieldCollection"); }
	FFieldCollection() 
	{ 
		AddGroup("VectorField"); 
		AddAttribute<FVector3f>("Start", "VectorField");
		AddAttribute<FVector3f>("End", "VectorField");
		AddAttribute<FLinearColor>("Color", "VectorField");
	}
	
	int32 AddVectorToField(FVector3f Start, FVector3f End)
	{
		int32 Size = AddElements(1, "VectorField");
		ModifyAttribute<FVector3f>("Start", "VectorField")[Size] = Start;
		ModifyAttribute<FVector3f>("End", "VectorField")[Size] = End;
		ModifyAttribute<FLinearColor>("Color", "VectorField")[Size] = FLinearColor(0.6f, 0.6f, 0.6f);
		return Size;
	}

	void SetColorOnVector(int32 Index, FLinearColor Color)
	{
		if (0 <= Index && Index < NumElements("VectorField"))
		{
			ModifyAttribute<FLinearColor>("Color", "VectorField")[Index] = Color;
		}
	}
	
	TArray<TPair<FVector3f, FVector3f>> GetVectorField() const
	{
		TArray<TPair<FVector3f, FVector3f>> VectorField;
		if (FindAttribute<FVector3f>("Start", "VectorField") != nullptr && FindAttribute<FVector3f>("End", "VectorField") != nullptr)
		{
			const TManagedArray<FVector3f>& VectorFieldStart = GetAttribute<FVector3f>("Start", "VectorField");
			const TManagedArray<FVector3f>& VectorFieldEnd = GetAttribute<FVector3f>("End", "VectorField");
			for (int32 i = 0; i < NumElements("VectorField"); i++)
			{
				VectorField.Add(TPair<FVector3f, FVector3f>(VectorFieldStart[i], VectorFieldEnd[i]));
			}
		}
		return VectorField;
	}

	TArray<FLinearColor> GetVectorColor() const
	{
		if (FindAttribute<FLinearColor>("Color", "VectorField") != nullptr)
		{
			const TManagedArray<FLinearColor>& Color = GetAttribute<FLinearColor>("Color", "VectorField");
			return Color.GetConstArray();
		}
		else
		{
			return TArray<FLinearColor>();
		}
	}

};


