// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Framework/DebugSubstep.h"
#include "Templates/Function.h"
#include "Chaos/Declares.h"

#if CHAOS_DEBUG_SUBSTEP
#include "Async/AsyncWork.h"

namespace Chaos
{
class FPhysicsSolverBase;
	/**
	 * Async task to run the solver advance in its own debug thread, substep by substep.
	 */
	class FDebugSolverTask final : public FNonAbandonableTask
	{
		friend class FAsyncTask<FDebugSolverTask>;

	public:
		FDebugSolverTask(TFunction<void()> InStepFunction, FDebugSubstep& InDebugSubstep);
		~FDebugSolverTask() = default;

		/** Solver advances. */
		void DoWork();

	private:
		FDebugSolverTask(const FDebugSolverTask&) = delete;
		FDebugSolverTask& operator=(const FDebugSolverTask&) = delete;

		FORCEINLINE TStatId GetStatId() const
		{
			RETURN_QUICK_DECLARE_CYCLE_STAT(FDebugSolverTask, STATGROUP_ThreadPoolAsyncTasks);
		}

	private:
		TFunction<void()> StepFunction;
		FDebugSubstep& DebugSubstep;
	};

	/**
	 * List of solver tasks used to debug substep.
	 */
	class FDebugSolverTasks final
	{
	public:
		FDebugSolverTasks() : SolverToTaskMap() {}
		~FDebugSolverTasks() { Shutdown(); }

		/** Add debug task entry for the specified solver. */
		void Add(FPhysicsSolverBase* Solver);

		/** Remove the debug task entry for the specified solver, and delete its task if any was created. */
		void Remove(FPhysicsSolverBase* Solver);

		/** Run the specified step function in one go within the current thread, or in a debug thread substep by substep depending on the Solver's DebugSustep status. */
		void DebugStep(FPhysicsSolverBase* Solver, TFunction<void()> StepFunction);

		/** Shutdown all debug threads. */
		void Shutdown();

	private:
		TMap<FPhysicsSolverBase*, FAsyncTask<FDebugSolverTask>*> SolverToTaskMap;
	};
}  // namespace Chaos

#else  // #if CHAOS_DEBUG_SUBSTEP

namespace Chaos
{
class FPhysicsSolverBase;

	/**
	 * List of solver tasks stub for non debug builds.
	 */
	class FDebugSolverTasks final
	{
	public:
		FDebugSolverTasks() {}
		~FDebugSolverTasks() = default;

		void Add(FPhysicsSolverBase* /*Solver*/) {}
		void Remove(FPhysicsSolverBase* /*Solver*/) {}

		FORCEINLINE void DebugStep(FPhysicsSolverBase* /*Solver*/, TFunction<void()> StepFunction) { StepFunction(); }

		void Shutdown() {}
	};
}  // namespace Chaos

#endif  // #if CHAOS_DEBUG_SUBSTEP #else
