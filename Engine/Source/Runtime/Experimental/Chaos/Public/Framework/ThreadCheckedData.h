// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreGlobals.h"
#include "ChaosCheck.h"

namespace Chaos
{
	/// This is a wrapper around a piece of data which adds execution thread access checks.
	template <typename T, typename CheckT>
	class TThreadCheckedData
	{
	public:

		template <typename... ArgTs>
		TThreadCheckedData(ArgTs... Args) : Data(Args...) { }

		~TThreadCheckedData() = default;

		const T& Get() const
		{
			CheckT::CheckRead();
			return Data;
		}

		T& Get()
		{
			CheckT::CheckReadWrite();
			return Data;
		}

		T* operator->()
		{
			CheckT::CheckRead();
			return &Data;
		}

		const T* operator->() const
		{
			CheckT::CheckRead();
			return &Data;
		}

	private:

		T Data;
	};

	struct FCheckGameplayThread
	{
		static inline void CheckRead()
		{
			CHAOS_CHECK(IsInGameThread());
		}

		static inline void CheckReadWrite()
		{
			CHAOS_CHECK(IsInGameThread());
		}
	};

	struct FCheckPhysicsThread
	{
		static inline void CheckRead()
		{
			CheckReadWrite();
		}

		static inline void CheckReadWrite()
		{
			// TODO: Find a way to check that this code is being executed in
			// the correct physics thread. This may depend on the threading model.
		}
	};

	
	template <typename T>
	using TDataGT = TThreadCheckedData<T, FCheckGameplayThread>;

	template <typename T>
	using TDataPT = TThreadCheckedData<T, FCheckPhysicsThread>;
}
