// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AutoRTFM/OpenWrapper.h"
#include "CoreMinimal.h"
#include "Framework/ThreadContextEnum.h"
#include "UObject/ObjectMacros.h"
#include "PhysicsCoreTypes.h"
#include "ChaosLog.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Async/ParallelFor.h"
#include <atomic>
#include "HAL/CriticalSection.h"
#include "AutoRTFM.h"
#include "Templates/SharedPointer.h"
#include "ChaosInsights/ChaosInsightsMacros.h"
#include "Containers/SpscQueue.h"
#include "Containers/TransactionallySafeSpscQueue.h"

#ifndef PHYSICS_THREAD_CONTEXT
	#if (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)
		#define PHYSICS_THREAD_CONTEXT 1
	#else
		#define PHYSICS_THREAD_CONTEXT 0
	#endif
#endif



/**
 * Scene lock types
 * @see CHAOS_SCENE_LOCK_TYPE
 */ 
#define CHAOS_SCENE_LOCK_SCENE_GUARD 0            // Unfair RW lock
#define CHAOS_SCENE_LOCK_RWFIFO_SPINLOCK 1        // Fair RW spinlock, non yielding
#define CHAOS_SCENE_LOCK_RWFIFO_CRITICALSECTION 2 // Fair RW spinlock, yielding
#define CHAOS_SCENE_LOCK_FRWLOCK 3                // Recurrant RW lock based on FRwLock (uses platform sync primitives)
#define CHAOS_SCENE_LOCK_SIMPLE_MUTEX 4           // Just a critical section (not an RWLock). Provided For profiling/debugging only. Not recommended

/** Controls the scene lock type. See above. */ 
#if WITH_EDITOR
	#ifndef CHAOS_SCENE_LOCK_TYPE
	#define CHAOS_SCENE_LOCK_TYPE CHAOS_SCENE_LOCK_RWFIFO_CRITICALSECTION
	#endif
#else
	#ifndef CHAOS_SCENE_LOCK_TYPE
	#define CHAOS_SCENE_LOCK_TYPE CHAOS_SCENE_LOCK_FRWLOCK
	#endif
#endif


/**
 * \def CHAOS_SCENE_LOCK_CHECKS
 * Controls whether the runtime will check and emit errors when a read or write operation is attempted but an
 * appropriate read or write lock has not been taken by the caller
 * NOTE: Disable currently until this can be made to check with the per-instance thread counts.
 */
#ifndef CHAOS_SCENE_LOCK_CHECKS
	#if (!UE_BUILD_SHIPPING && !UE_BUILD_TEST)
		#define CHAOS_SCENE_LOCK_CHECKS 0
	#else
		#define CHAOS_SCENE_LOCK_CHECKS 0
	#endif
#endif

#if PHYSICS_THREAD_CONTEXT
namespace Chaos { class FPhysicsThreadContext; }
UE_DECLARE_THREAD_SINGLETON_TLS(Chaos::FPhysicsThreadContext, CHAOS_API)
#endif


namespace Chaos
{
	// Not intended for external callers, provided here to allow the locks below to record depths
	namespace ThreadingPrivate
	{
		// Control the current thread read/write depths
		CHAOS_API void IncReadDepth(void* Instance);
		CHAOS_API void IncWriteDepth(void* Instance);
		CHAOS_API void DecReadDepth(void* Instance);
		CHAOS_API void DecWriteDepth(void* Instance);

		// Get the calling thread's current read depth
		CHAOS_API uint32 GetThreadReadDepth(void* Instance);
	}

#if CHAOS_SCENE_LOCK_CHECKS

	// Not intended for external callers, provided here to allow the below check macros to function
	namespace ThreadingPrivate
	{
		// Checks assumptions for functions marked _AssumesLocked in the interface.
		CHAOS_API void CheckLockReadAssumption(const TCHAR* Context);
		CHAOS_API void CheckLockWriteAssumption(const TCHAR* Context);
	}

	/** Checks that the caller currently has a read or write lock open, emits an error if not locked */
	#define CHAOS_CHECK_READ_ASSUMPTION Chaos::ThreadingPrivate::CheckLockReadAssumption(ANSI_TO_TCHAR(__FUNCTION__))

	/** Checks that the caller currently has a write lock open, emits an error if not locked */
	#define CHAOS_CHECK_WRITE_ASSUMPTION Chaos::ThreadingPrivate::CheckLockWriteAssumption(ANSI_TO_TCHAR(__FUNCTION__))

	/** 
	 * Checks that the caller currently has a read or write lock open if an actor is currently bound to a solver.
	 * The actor is a derived child of IPhysicsProxyBase which holds a solver pointer which is set on the main
	 * thread during scene registration - if that isn't currently set the actor isn't under the control of the
	 * physics thread and can be operated on to initialize it without a lock.
	 * @see FPBDRigidsSolver::RegisterObject
	 */
	#define CHAOS_CHECK_READ_ASSUMPTION_ACTOR(Actor) if(Actor && Actor->GetSolverBase()) {Chaos::ThreadingPrivate::CheckLockReadAssumption(ANSI_TO_TCHAR(__FUNCTION__));}

	/**
	 * Checks that the caller currently has a write lock open if an actor is currently bound to a solver.
	 * The actor is a derived child of IPhysicsProxyBase which holds a solver pointer which is set on the main
	 * thread during scene registration - if that isn't currently set the actor isn't under the control of the
	 * physics thread and can be operated on to initialize it without a lock.
	 * @see FPBDRigidsSolver::RegisterObject
	 */
	#define CHAOS_CHECK_WRITE_ASSUMPTION_ACTOR(Actor) if(Actor && Actor->GetSolverBase()) {Chaos::ThreadingPrivate::CheckLockWriteAssumption(ANSI_TO_TCHAR(__FUNCTION__));}
	
	/**
	 * Checks that the caller currently has a read or write lock open when reading constraint properties for
	 * actors that are bound to a solver.
	 * @see CHAOS_CHECK_READ_ASSUMPTION_ACTOR
	 */
	#define CHAOS_CHECK_READ_ASSUMPTION_CONSTRAINT(Handle)  \
	if(Handle.Constraint && \
		((Handle.Constraint->GetParticleProxies()[0] && Handle.Constraint->GetParticleProxies()[0]->GetSolverBase()) || \
		(Handle.Constraint->GetParticleProxies()[1] && Handle.Constraint->GetParticleProxies()[1]->GetSolverBase()))) \
	{Chaos::ThreadingPrivate::CheckLockReadAssumption(ANSI_TO_TCHAR(__FUNCTION__));}

	/**
	 * Checks that the caller currently has a read or write lock open when reading constraint properties for
	 * actors that are bound to a solver.
	 * @see CHAOS_CHECK_WRITE_ASSUMPTION_ACTOR
	 */
	#define CHAOS_CHECK_WRITE_ASSUMPTION_CONSTRAINT(Handle)  \
	if(Handle.Constraint && \
		((Handle.Constraint->GetParticleProxies()[0] && Handle.Constraint->GetParticleProxies()[0]->GetSolverBase()) || \
		(Handle.Constraint->GetParticleProxies()[1] && Handle.Constraint->GetParticleProxies()[1]->GetSolverBase()))) \
	{Chaos::ThreadingPrivate::CheckLockWriteAssumption(ANSI_TO_TCHAR(__FUNCTION__));}

#else

	// Empty when not compiled in
	#define CHAOS_CHECK_READ_ASSUMPTION
	#define CHAOS_CHECK_WRITE_ASSUMPTION
	#define CHAOS_CHECK_READ_ASSUMPTION_ACTOR
	#define CHAOS_CHECK_WRITE_ASSUMPTION_ACTOR
	#define CHAOS_CHECK_READ_ASSUMPTION_CONSTRAINT
	#define CHAOS_CHECK_WRITE_ASSUMPTION_CONSTRAINT

#endif

/** Signals that we have entered a read lock to control the checks above */
#define CHAOS_RECORD_ENTER_READ_LOCK Chaos::ThreadingPrivate::IncReadDepth(this);

/** Signals that we have entered a write lock to control the checks above */
#define CHAOS_RECORD_ENTER_WRITE_LOCK Chaos::ThreadingPrivate::IncWriteDepth(this);

/** Signals that we have left a read lock to control the checks above */
#define CHAOS_RECORD_LEAVE_READ_LOCK Chaos::ThreadingPrivate::DecReadDepth(this);

/** Signals that we have left a write lock to control the checks above */
#define CHAOS_RECORD_LEAVE_WRITE_LOCK Chaos::ThreadingPrivate::DecWriteDepth(this);

#if PHYSICS_THREAD_CONTEXT

/** Debug helper to ensure threading mistakes are caught. Do not use for ship */
class FPhysicsThreadContext : public TThreadSingleton<FPhysicsThreadContext>
{
public:
	bool IsInPhysicsSimContext() const
	{
		return PhysicsSimContext > 0;
	}

	bool IsInGameThreadContext() const
	{
		return (IsInGameThread() || GameThreadContext > 0) && !bFrozenGameThread;
	}

	void IncPhysicsSimContext()
	{
		++PhysicsSimContext;
	}

	void DecPhysicsSimContext()
	{
		check(PhysicsSimContext > 0);	//double delete?
		--PhysicsSimContext;
	}

	void IncGameThreadContext()
	{
		++GameThreadContext;
	}

	void DecGameThreadContext()
	{
		check(GameThreadContext > 0);	//double delete?
		--GameThreadContext;
	}

	void FreezeGameThreadContext()
	{
		ensure(!bFrozenGameThread);
		bFrozenGameThread = true;
	}

	void UnFreezeGameThreadContext()
	{
		ensure(bFrozenGameThread);
		bFrozenGameThread = false;
	}

private:
	int32 PhysicsSimContext = 0;
	int32 GameThreadContext = 0;
	bool bFrozenGameThread = false;
};

struct FPhysicsThreadContextScope
{
	FPhysicsThreadContextScope(bool InParentIsPhysicsSimContext)
		: bParentIsPhysicsSimContext(InParentIsPhysicsSimContext)
	{
		if (bParentIsPhysicsSimContext)
		{
			FPhysicsThreadContext::Get().IncPhysicsSimContext();
		}
	}

	~FPhysicsThreadContextScope()
	{
		if (bParentIsPhysicsSimContext)
		{
			FPhysicsThreadContext::Get().DecPhysicsSimContext();
		}
	}

	bool bParentIsPhysicsSimContext;
};

struct FGameThreadContextScope
{
	FGameThreadContextScope(bool InParentIsGameThreadContext)
		: bParentIsGameThreadContext(InParentIsGameThreadContext)
	{
		if (bParentIsGameThreadContext)
		{
			FPhysicsThreadContext::Get().IncGameThreadContext();
		}
	}

	~FGameThreadContextScope()
	{
		if (bParentIsGameThreadContext)
		{
			FPhysicsThreadContext::Get().DecGameThreadContext();
		}
	}

	bool bParentIsGameThreadContext;
};

struct FFrozenGameThreadContextScope
{
	FFrozenGameThreadContextScope()
	{
		FPhysicsThreadContext::Get().FreezeGameThreadContext();
	}

	~FFrozenGameThreadContextScope()
	{
		FPhysicsThreadContext::Get().UnFreezeGameThreadContext();
	}
};


FORCEINLINE bool IsInPhysicsThreadContext()
{
	return FPhysicsThreadContext::Get().IsInPhysicsSimContext();
}

FORCEINLINE bool IsInGameThreadContext()
{
	return FPhysicsThreadContext::Get().IsInGameThreadContext();
}

FORCEINLINE void EnsureIsInPhysicsThreadContext()
{
	ensure(IsInPhysicsThreadContext());
}

FORCEINLINE void EnsureIsInGameThreadContext()
{
	ensure(IsInGameThreadContext());
}
#else
FORCEINLINE void EnsureIsInPhysicsThreadContext()
{
}

FORCEINLINE void EnsureIsInGameThreadContext()
{
}
#endif


	using EThreadingMode = EChaosThreadingMode;

	/**
	 * Recursive Read/Write lock object for protecting external data accesses for physics scenes.
	 * This is a fairly heavy lock designed to allow scene queries and user code to safely access
	 * external physics data.
	 *
	 * The lock also allows a thread to recursively lock data to avoid deadlocks on repeated writes
	 * or undefined behavior for nesting read locks.
	 *
	 * Fairness is determined by the underlying platform FRWLock type as this lock uses FRWLock
	 * as it's internal primitive
	 */
	class FPhysicsSceneGuard
	{
	public:
		FPhysicsSceneGuard()
		{
			TlsSlot = FPlatformTLS::AllocTlsSlot();
			CurrentWriterThreadId.Store(0);
		}

		~FPhysicsSceneGuard()
		{
			if(FPlatformTLS::IsValidTlsSlot(TlsSlot))
			{
				// Validate the lock as it shuts down
#if CHAOS_CHECKED
				ensureMsgf(CurrentWriterThreadId.Load() == 0, TEXT("Shutting down a physics scene guard but thread %u still holds a write lock"), CurrentWriterThreadId.Load());
#endif
				FPlatformTLS::FreeTlsSlot(TlsSlot);
			}
		}

		FPhysicsSceneGuard(const FPhysicsSceneGuard& InOther) = delete;
		FPhysicsSceneGuard(FPhysicsSceneGuard&& InOther) = delete;
		FPhysicsSceneGuard& operator=(const FPhysicsSceneGuard& InOther) = delete;
		FPhysicsSceneGuard& operator=(FPhysicsSceneGuard&& InOther) = delete;

		void ReadLock()
		{
			const FSceneLockTls ThreadData = ModifyTls([](FSceneLockTls& ThreadDataInner) {ThreadDataInner.ReadDepth++; });

			const uint32 ThisThreadId = FPlatformTLS::GetCurrentThreadId();

			// If we're already writing then don't attempt the lock, we already have exclusive access
			if(CurrentWriterThreadId.Load() != ThisThreadId && ThreadData.ReadDepth == 1)
			{
				InnerLock.ReadLock();
			}

#if PHYSICS_THREAD_CONTEXT
			//read lock means we can access game thread data, so set the right context
			FPhysicsThreadContext::Get().IncGameThreadContext();
#endif
		}

		void WriteLock()
		{
			ModifyTls([](FSceneLockTls& ThreadDataInner) {ThreadDataInner.WriteDepth++; });

			const uint32 ThisThreadId = FPlatformTLS::GetCurrentThreadId();

			if(CurrentWriterThreadId.Load() != ThisThreadId)
			{
				InnerLock.WriteLock();
				CurrentWriterThreadId.Store(ThisThreadId);
			}

#if PHYSICS_THREAD_CONTEXT
			//write lock means we can access game thread data, so set the right context
			FPhysicsThreadContext::Get().IncGameThreadContext();
#endif
		}

		void ReadUnlock()
		{
			const FSceneLockTls ThreadData = ModifyTls([](FSceneLockTls& ThreadDataInner)
				{
				if(ThreadDataInner.ReadDepth > 0)
					{
						ThreadDataInner.ReadDepth--;
					}
					else
					{
#if CHAOS_CHECKED
						ensureMsgf(false, TEXT("ReadUnlock called on physics scene guard when the thread does not hold the lock"));
#else
						UE_LOGF(LogChaos, Warning, "ReadUnlock called on physics scene guard when the thread does not hold the lock")
#endif
					}

				});

			const uint32 ThisThreadId = FPlatformTLS::GetCurrentThreadId();

			if(CurrentWriterThreadId.Load() != ThisThreadId && ThreadData.ReadDepth == 0)
			{
				InnerLock.ReadUnlock();
			}

#if PHYSICS_THREAD_CONTEXT
			//read lock is released, the gamethread context is gone
			FPhysicsThreadContext::Get().DecGameThreadContext();
#endif
		}

		void WriteUnlock()
		{
			const uint32 ThisThreadId = FPlatformTLS::GetCurrentThreadId();

			if(CurrentWriterThreadId.Load() == ThisThreadId)
			{
				const FSceneLockTls ThreadData = ModifyTls([](FSceneLockTls& ThreadDataInner) {ThreadDataInner.WriteDepth--; });

				if(ThreadData.WriteDepth == 0)
				{
					CurrentWriterThreadId.Store(0);
					InnerLock.WriteUnlock();
				}
			}
			else
			{
#if CHAOS_CHECKED
				ensureMsgf(false, TEXT("WriteUnlock called on physics scene guard when the thread does not hold the lock"));
#else
				UE_LOGF(LogChaos, Warning, "ReadUnlock called on physics scene guard when the thread does not hold the lock")
#endif
			}

#if PHYSICS_THREAD_CONTEXT
			//write lock is released, the gamethread context is gone
			FPhysicsThreadContext::Get().DecGameThreadContext();
#endif
		}

	private:

		// We use 32 bits to store our depths (16 read and 16 write) allowing a maximum
		// recursive lock of depth 65,536. This unions to whatever the platform ptr size
		// is so we can store this directly into TLS without allocating more storage
		class FSceneLockTls
		{
		public:

			FSceneLockTls()
				: WriteDepth(0)
				, ReadDepth(0)
			{}

			union
			{
				struct
				{
					uint16 WriteDepth;
					uint16 ReadDepth;
				};
				void* PtrDummy;
			};

		};

		// Helper for modifying the current TLS data
		template<typename CallableType>
		const FSceneLockTls ModifyTls(CallableType Callable)
		{
			checkSlow(FPlatformTLS::IsValidTlsSlot(TlsSlot));

			void* ThreadData = FPlatformTLS::GetTlsValue(TlsSlot);

			FSceneLockTls TlsData;
			TlsData.PtrDummy = ThreadData;

			Callable(TlsData);

			FPlatformTLS::SetTlsValue(TlsSlot, TlsData.PtrDummy);

			return TlsData;
		}

		uint32 TlsSlot;
		TAtomic<uint32> CurrentWriterThreadId;
		FRWLock InnerLock;
	};

	/**
	 * Templated RAII scope lock around generic mutex type
	 */
	template<typename MutexType>
	class TMutexScopeLock
	{
	public:
		TMutexScopeLock(MutexType& InMutex)
			: Mutex(InMutex)
		{
			Mutex.Lock();
		}

		~TMutexScopeLock()
		{
			Mutex.Unlock();
		}

	private:

		// No default, copy or move construction
		TMutexScopeLock() = delete;
		TMutexScopeLock(const TMutexScopeLock&) = delete;
		TMutexScopeLock(TMutexScopeLock&&) = delete;
		TMutexScopeLock& operator=(const TMutexScopeLock&) = delete;
		TMutexScopeLock& operator=(TMutexScopeLock&&) = delete;

		MutexType& Mutex;
	};

	/**
	 * A first-in, first-out "fair" read-write lock around a generic mutex
	 * Given a mutex (either just FCriticalSection or some custom lock) this class implements a fair lock.
	 * Any number of readers can enter the lock but as soon as a writer attempts to enter the lock all 
	 * subsequent readers are forced to wait until the current readers leave the lock and the writer gets a chance
	 * to perform its operation. Once the write is completed the waiting readers are able to resume.
	 * This ensures we do not end up in a situation where we have a write waiting but many reads end up constantly
	 * forcing the write to wait. In a physics context a write on the game thread is time-critical and we want
	 * that thread to resume as soon as possible by pausing any reads (scene queries) until the write is finished
	 */
	template<typename MutexType>
	class TRwFifoLock
	{
	public:

		TRwFifoLock()
			: NumReaders(0)
		{
			//ThreadingPrivate::CreateLockThreadData(this);
		}

		~TRwFifoLock() = default;

		void ReadLock()
		{
			TRACE_CHAOS_BEGIN_LOCK(Chaos::Insights::ELockEventType::RWLockReadLock);
			if(ThreadingPrivate::GetThreadReadDepth(this) == 0)
			{
				TMutexScopeLock<MutexType> Guard(Mutex);

				// We lock for this increment to halt if there's a writer waiting to enter the lock
				// In this case we will be forced to wait till the write completes
				++NumReaders;
			}
			else
			{
				// Only require a lock on the first acquisition. this allows recursive reads even while a
				// writer is holding the lock waiting to enter. The writer will be allowed to proceed when
				// all write scopes end
			++NumReaders;
			}

#if PHYSICS_THREAD_CONTEXT
			// Read lock means we can access game thread data, so set the right context
			FPhysicsThreadContext::Get().IncGameThreadContext();
#endif

			CHAOS_RECORD_ENTER_READ_LOCK;

			TRACE_CHAOS_ACQUIRE_LOCK();
		}

		void WriteLock()
		{
			TRACE_CHAOS_BEGIN_LOCK(Chaos::Insights::ELockEventType::RWLockWriteLock);

#if CHAOS_SCENE_LOCK_CHECKS
			if(ThreadingPrivate::GetThreadReadDepth(this) > 0)
			{
				ensureMsgf(false, TEXT("A thread holding a read lock on the physics scene attempted to upgrade to a write lock - this is not supported, performing an unsafe write."));

				// Still need to increment the context when we hit this case or we'll just crash later
#if PHYSICS_THREAD_CONTEXT
				// Write lock means we can access game thread data, so set the right context
				FPhysicsThreadContext::Get().IncGameThreadContext();
#endif

				return;
			}
#endif

			Mutex.Lock();

			// Spin until all readers are finished
			for(;;)
			{
				if(NumReaders.load() == 0)
				{
					// All readers now finished - writer can enter the lock properly (pass back to caller)
					break;
				}

				// Issue pause instruction - architecture dependent instruction to better handle
				// a spin lock not interfering with other threads on this core, this doesn't
				// actually yield the thread
				FPlatformProcess::Yield();
			}

			CHAOS_RECORD_ENTER_WRITE_LOCK;

#if PHYSICS_THREAD_CONTEXT
			// Write lock means we can access game thread data, so set the right context
			FPhysicsThreadContext::Get().IncGameThreadContext();
#endif
			TRACE_CHAOS_ACQUIRE_LOCK();
		}

		void ReadUnlock()
		{
			CHAOS_RECORD_LEAVE_READ_LOCK;

			// No locking here, just decrement atomic reader count
			--NumReaders;

#if PHYSICS_THREAD_CONTEXT
			// Read lock is released, the gamethread context is gone
			FPhysicsThreadContext::Get().DecGameThreadContext();
#endif
		
			TRACE_CHAOS_END_LOCK();
		}

		void WriteUnlock()
		{
			CHAOS_RECORD_LEAVE_WRITE_LOCK;

			Mutex.Unlock();
#if PHYSICS_THREAD_CONTEXT
			// Write lock is released, the gamethread context is gone
			FPhysicsThreadContext::Get().DecGameThreadContext();
#endif

			TRACE_CHAOS_END_LOCK();
		}

	private:
		MutexType Mutex;
		std::atomic<uint32> NumReaders;
	};

	/**
	 * A non-yielding, recursive spin lock
	 * Implements a first-in, first-out lock / mutex that won't yield back to the system.
	 * Intended for applications that must wake / resume at the earliest opportunity.
	 * Each thread attempting a write gets an atomically controlled counter to wait on so the lock is fair in that 
	 * the locks will be ordered according to the order Lock was called.
	 */
	class FPhysSpinLock
	{
	public:
		FPhysSpinLock()
			: Next(0)
			, Current(0)
			, WriterId(0)
			, Count()
		{}

		void Lock()
		{
			// Support recursive locking
			if(WriterId.load() == FPlatformTLS::GetCurrentThreadId())
			{
				Count++;
				return;
			}

			// Get the wait value - acquire operation so Current.load can't be reordered before this
			uint32 WaitFor = Next.fetch_add(1, std::memory_order_acquire);
			for(;;)
			{
				if(WaitFor == Current.load())
				{
					break;
				}

				// Issue pause instruction - architecture dependent instruction to better handle
				// a spin lock not interfering with other threads on this core, this doesn't
				// actually yield the thread
				FPlatformProcess::Yield();
			}

			// Lock acquired, store the thread ID for recursive locking
			WriterId.store(FPlatformTLS::GetCurrentThreadId());
			Count++;
		}

		void Unlock()
		{
			checkf(WriterId.load() == FPlatformTLS::GetCurrentThreadId(), TEXT("A thread unlocked without owning the lock (calling Lock first)"));
			checkf(Count > 0, TEXT("A thread unlocked a lock that had no outstanding lock scopes"));

			// Once all recursive locks are dropped, increment Current to allow the next thread in
			if(--Count == 0)
			{
				// Clear the lock owner
				WriterId.store(0);

				// Release the next thread - this must be the last operation as immediately
				// the next user of the lock will be allowed to take ownership
				++Current;
			}
		}

	private:
		std::atomic<uint32> Next;
		std::atomic<uint32> Current;
		std::atomic<uint32> WriterId;
		uint32 Count;
	};


	/**
	 * A recursive readwrite lock that uses FRwLock internally (this uses an efficient platform specific implementation)
	 */
	class FPhysicsRwLock
	{
		struct FRwLockInfo
		{
			FRwLockInfo(void* TlsSlotValue)
			{
				ThreadReadDepth = uint32(uint64(TlsSlotValue));
				ThreadWriteDepth = uint64(TlsSlotValue) >> 32;
			}
			void* GetTlsSlotValue()
			{
				uint64 ValueOut = uint64(ThreadReadDepth) | (uint64(ThreadWriteDepth) << 32);
				return (void*)ValueOut;
			}

			uint32 ThreadReadDepth = 0;
			uint32 ThreadWriteDepth = 0;
		};

	public:

		FPhysicsRwLock()
		{
			TlsSlot = FPlatformTLS::AllocTlsSlot();
			check(FPlatformTLS::IsValidTlsSlot(TlsSlot));
		}

		~FPhysicsRwLock()
		{
			check(FPlatformTLS::IsValidTlsSlot(TlsSlot));
			FPlatformTLS::FreeTlsSlot(TlsSlot);
		}

		void ReadLock()
		{
			TRACE_CHAOS_BEGIN_LOCK(Chaos::Insights::ELockEventType::RWLockReadLock)

			FRwLockInfo ThreadInfo(FPlatformTLS::GetTlsValue(TlsSlot));
			ThreadInfo.ThreadReadDepth++;
			FPlatformTLS::SetTlsValue(TlsSlot, ThreadInfo.GetTlsSlotValue());

			if (ThreadInfo.ThreadReadDepth + ThreadInfo.ThreadWriteDepth == 1)
			{
				RwLock.ReadLock();
			}

#if PHYSICS_THREAD_CONTEXT
			// Read lock means we can access game thread data, so set the right context
			FPhysicsThreadContext::Get().IncGameThreadContext();
#endif
			TRACE_CHAOS_ACQUIRE_LOCK()
		}
		void WriteLock()
		{
			TRACE_CHAOS_BEGIN_LOCK(Chaos::Insights::ELockEventType::RWLockWriteLock)

			FRwLockInfo ThreadInfo(FPlatformTLS::GetTlsValue(TlsSlot));
			ThreadInfo.ThreadWriteDepth++;
			FPlatformTLS::SetTlsValue(TlsSlot, ThreadInfo.GetTlsSlotValue());

#if CHAOS_SCENE_LOCK_CHECKS
			if (ThreadInfo.ThreadReadDepth > 0)
			{
				UE_LOGF(LogChaos, Warning, "Attempt to upgrade a read lock to a write lock. This is not supported. Writes will be unsafe")
			}
#endif
			if (ThreadInfo.ThreadReadDepth + ThreadInfo.ThreadWriteDepth == 1)
			{
				RwLock.WriteLock();
			}
#if PHYSICS_THREAD_CONTEXT
			// Write lock means we can access game thread data, so set the right context
			FPhysicsThreadContext::Get().IncGameThreadContext();
#endif
			TRACE_CHAOS_ACQUIRE_LOCK()
		}
		void ReadUnlock()
		{
			FRwLockInfo ThreadInfo(FPlatformTLS::GetTlsValue(TlsSlot));
			ThreadInfo.ThreadReadDepth--;
			FPlatformTLS::SetTlsValue(TlsSlot, ThreadInfo.GetTlsSlotValue());

			if (ThreadInfo.ThreadWriteDepth + ThreadInfo.ThreadReadDepth == 0)
			{
				RwLock.ReadUnlock();
			}

#if PHYSICS_THREAD_CONTEXT
			// Read lock is released, the gamethread context is gone
			FPhysicsThreadContext::Get().DecGameThreadContext();
#endif
			TRACE_CHAOS_END_LOCK()
		}
		void WriteUnlock()
		{
			FRwLockInfo ThreadInfo(FPlatformTLS::GetTlsValue(TlsSlot));
			ThreadInfo.ThreadWriteDepth--;
			FPlatformTLS::SetTlsValue(TlsSlot, ThreadInfo.GetTlsSlotValue());
			if (ThreadInfo.ThreadWriteDepth + ThreadInfo.ThreadReadDepth == 0)
			{
				RwLock.WriteUnlock();
			}

#if PHYSICS_THREAD_CONTEXT
			// Write lock is released, the gamethread context is gone
			FPhysicsThreadContext::Get().DecGameThreadContext();
#endif
			TRACE_CHAOS_END_LOCK()
		}

	private:
		FRWLock RwLock;
		uint32 TlsSlot;
	};


	/**
	 * A simple mutex based lock based on FCriticalSection. Reads are exclusive
	 */
	class FPhysicsSimpleMutexLock
	{
	public:
		FPhysicsSimpleMutexLock()
		{
		}

		~FPhysicsSimpleMutexLock() = default;

		void ReadLock()
		{
			Cs.Lock();
#if PHYSICS_THREAD_CONTEXT
			// Read lock means we can access game thread data, so set the right context
			FPhysicsThreadContext::Get().IncGameThreadContext();
#endif
		}
		void WriteLock()
		{
			Cs.Lock();
#if PHYSICS_THREAD_CONTEXT
			// Write lock means we can access game thread data, so set the right context
			FPhysicsThreadContext::Get().IncGameThreadContext();
#endif
		}
		void ReadUnlock()
		{
			Cs.Unlock();

#if PHYSICS_THREAD_CONTEXT
			// Read lock is released, the gamethread context is gone
			FPhysicsThreadContext::Get().DecGameThreadContext();
#endif
		}
		void WriteUnlock()
		{
			Cs.Unlock();

#if PHYSICS_THREAD_CONTEXT
			// Write lock is released, the gamethread context is gone
			FPhysicsThreadContext::Get().DecGameThreadContext();
#endif
		}

	private:
		FCriticalSection Cs;
	};


	/**
	 * Implements a RAII scoped write lock around a generic mutex.
	 */
	template<typename MutexType>
	class TPhysicsSceneGuardScopedWrite
	{
	public:
		TPhysicsSceneGuardScopedWrite(MutexType& InMutex)
			: Mutex(InMutex)
		{
			CSV_SCOPED_TIMING_STAT(PhysicsVerbose,AcquireSceneWriteLock);
			Mutex.WriteLock();
		}

		~TPhysicsSceneGuardScopedWrite()
		{
			Mutex.WriteUnlock();
		}

	private:
		TPhysicsSceneGuardScopedWrite() = delete;
		TPhysicsSceneGuardScopedWrite(const TPhysicsSceneGuardScopedWrite&) = delete;
		TPhysicsSceneGuardScopedWrite(TPhysicsSceneGuardScopedWrite&&) = delete;
		TPhysicsSceneGuardScopedWrite& operator=(const TPhysicsSceneGuardScopedWrite&) = delete;
		TPhysicsSceneGuardScopedWrite& operator=(TPhysicsSceneGuardScopedWrite&&) = delete;

		MutexType& Mutex;
	};

	/**
	 * Implements a RAII scoped read lock around a generic mutex.
	 */
	template<typename MutexType>
	class TPhysicsSceneGuardScopedRead
	{
	public:
		TPhysicsSceneGuardScopedRead(MutexType& InMutex)
			: Mutex(InMutex)
		{
			CSV_SCOPED_TIMING_STAT(PhysicsVerbose, AcquireSceneReadLock);
			Mutex.ReadLock();
		}

		~TPhysicsSceneGuardScopedRead()
		{
			Mutex.ReadUnlock();
		}

	private:
		TPhysicsSceneGuardScopedRead() = delete;
		TPhysicsSceneGuardScopedRead(const TPhysicsSceneGuardScopedRead&) = delete;
		TPhysicsSceneGuardScopedRead(TPhysicsSceneGuardScopedRead&&) = delete;
		TPhysicsSceneGuardScopedRead& operator=(const TPhysicsSceneGuardScopedRead&) = delete;
		TPhysicsSceneGuardScopedRead& operator=(TPhysicsSceneGuardScopedRead&&) = delete;

		MutexType& Mutex;
	};

#if CHAOS_SCENE_LOCK_TYPE == CHAOS_SCENE_LOCK_SCENE_GUARD
	using FPhysSceneLockNonTransactional = FPhysicsSceneGuard;
#elif CHAOS_SCENE_LOCK_TYPE == CHAOS_SCENE_LOCK_RWFIFO_SPINLOCK
	using FPhysSceneLockNonTransactional = TRwFifoLock<FPhysSpinLock>;
#elif CHAOS_SCENE_LOCK_TYPE == CHAOS_SCENE_LOCK_RWFIFO_CRITICALSECTION
	using FPhysSceneLockNonTransactional = TRwFifoLock<FCriticalSection>;
#elif CHAOS_SCENE_LOCK_TYPE == CHAOS_SCENE_LOCK_FRWLOCK
	using FPhysSceneLockNonTransactional = FPhysicsRwLock;
#elif CHAOS_SCENE_LOCK_TYPE == CHAOS_SCENE_LOCK_SIMPLE_MUTEX
	using FPhysSceneLockNonTransactional = FPhysicsSimpleMutexLock;
#endif

#if UE_AUTORTFM
	// A transactionally safe lock that works in the following novel ways:
	// - In the open (non-transactional):
	//   - Take the lock like before. Simple!
	//   - Free the lock like before too.
	// - In the closed (transactional):
	//   - During locking we query `TransactionalLockCount`:
	//	   - 0 means we haven't taken the lock within our transaction nest and need to acquire the lock.
	//     - Otherwise we already have the lock (and are preventing non-transactional code seeing any
	//       modifications we've made while holding the lock), so just bump `TransactionalLockCount`.
	//     - We also register an on-abort handler to release the lock should we abort (but we need to
	//       query `TransactionalLockCount` even there because we could be aborting an inner transaction
	//       and the parent transaction still wants to have the lock held!).
	//   - During unlocking we defer doing the unlock until the transaction commits.
	//
	// Thus with this approach we will hold this lock for the *entirety* of the transactional nest should
	// we take the lock during the transaction, thus preventing non-transactional code from seeing any
	// modifications we should make.
	//
	// If we are within a transaction, we pessimise our read-lock to a write-lock. Note: that it should
	// potentially be possible to have read-locks work correctly, but serious care will have to be taken to
	// ensure that we don't have:
	//   Open Thread     Closed Thread
	//   -----------     ReadLock
	//   -----------     ReadUnlock
	//   WriteLock       -------------
	//   WriteUnlock     -------------
	//   -----------     ReadLock      <- Invalid because the transaction can potentially observe side
	//                                    effects of the open-threads writes!
	struct FPhysSceneLockTransactionallySafe final
	{
		// Always open because the constructor arguments will create the underlying lock.
		UE_AUTORTFM_ALWAYS_OPEN
		FPhysSceneLockTransactionallySafe() : State(MakeShared<FState>())
		{
			if (AutoRTFM::IsTransactional())
			{
				const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([this]
					{
						AutoRTFM::PushOnAbortHandler(this, [this]
							{
								this->~FPhysSceneLockTransactionallySafe();
							});
					});

				ensure(AutoRTFM::ETransactionStatus::Executing == Status);
			}
		}

		~FPhysSceneLockTransactionallySafe()
		{
			if (AutoRTFM::IsTransactional())
			{
				const AutoRTFM::ETransactionStatus Status = AutoRTFM::Close([this]
					{
						AutoRTFM::PopOnAbortHandler(this);

						// We explicitly copy the state here for the case that `this` was stack
						// allocated and has already died before the on-commit is hit.
						AutoRTFM::OnCommit([State = AutoRTFM::TOpenWrapper{this->State}]
							{
								ensure(0 == State.Object->TransactionalLockCount);
							});
					});

				ensure(AutoRTFM::ETransactionStatus::Executing == Status);
			}

			// As the State was constructed in the open, it must be released in the open.
			AutoRTFM::Open([&] AUTORTFM_DISABLE { State = nullptr; });
		}

		void ReadLock()
		{
			if (!AutoRTFM::IsIdle())
			{
				// Transactionally pessimise ReadLock -> WriteLock.
				WriteLock();
			}
			else
			{
				State->Lock.ReadLock();
				ensure(0 == State->TransactionalLockCount);
			}
		}

		void ReadUnlock()
		{
			if (!AutoRTFM::IsIdle())
			{
				// Transactionally pessimise ReadUnlock -> WriteUnlock.
				WriteUnlock();
			}
			else
			{
				ensure(0 == State->TransactionalLockCount);
				State->Lock.ReadUnlock();
			}
		}

		void WriteLock()
		{
			if (!AutoRTFM::IsIdle())
			{
				UE_AUTORTFM_OPEN
					{
						// The transactional system which can increment TransactionalLockCount
						// is always single-threaded, thus this is safe to check without atomicity.
						if (0 == State->TransactionalLockCount)
						{
							State->Lock.WriteLock();
						}

						State->TransactionalLockCount += 1;
					};

				// We explicitly copy the state here for the case that `this` was stack
				// allocated and has already died before the on-abort is hit.
				AutoRTFM::OnAbort([State = AutoRTFM::TOpenWrapper{this->State}]
					{
						ensure(0 != State.Object->TransactionalLockCount);

						State.Object->TransactionalLockCount -= 1;

						if (0 == State.Object->TransactionalLockCount)
						{
							State.Object->Lock.WriteUnlock();
						}
					});
			}
			else
			{
				State->Lock.WriteLock();
				ensure(0 == State->TransactionalLockCount);
			}
		}

		void WriteUnlock()
		{
			if (!AutoRTFM::IsIdle())
			{
				// We explicitly copy the state here for the case that `this` was stack
				// allocated and has already died before the on-commit is hit.
				AutoRTFM::OnCommit([State = AutoRTFM::TOpenWrapper{this->State}]
					{
						ensure(0 != State.Object->TransactionalLockCount);

						State.Object->TransactionalLockCount -= 1;

						if (0 == State.Object->TransactionalLockCount)
						{
							State.Object->Lock.WriteUnlock();
						}
					});
			}
			else
			{
				ensure(0 == State->TransactionalLockCount);
				State->Lock.WriteUnlock();
			}
		}

	private:
		UE_NONCOPYABLE(FPhysSceneLockTransactionallySafe)

		struct FState final
		{
			FPhysSceneLockNonTransactional Lock;
			uint32 TransactionalLockCount = 0;
		};

		TSharedPtr<FState> State;
	};

#if UE_WITH_REMOTE_OBJECT_HANDLE
	// With remote object support, we wrap the underlying FPhysSceneLockTransactionallySafe
	// in an additional layer and expose callbacks to register additional logic to execute

	struct FPhysSceneLockRemoteObject;

	struct FPhysSceneLockCallbacks
	{
		void (*ReadLock)(FPhysSceneLockRemoteObject*) = nullptr;
		void (*ReadUnlock)(FPhysSceneLockRemoteObject*) = nullptr;
		void (*WriteLock)(FPhysSceneLockRemoteObject*) = nullptr;
		void (*WriteUnlock)(FPhysSceneLockRemoteObject*) = nullptr;
	};

	CHAOS_API extern FPhysSceneLockCallbacks GPhysSceneLockRemoteObjectCallbacks;

	struct FPhysSceneLockRemoteObject
	{
		void ReadLock()
		{
			if (GPhysSceneLockRemoteObjectCallbacks.ReadLock)
				GPhysSceneLockRemoteObjectCallbacks.ReadLock(this);
			else
				UnderlyingLock.ReadLock();
		}

		void ReadUnlock()
		{
			if (GPhysSceneLockRemoteObjectCallbacks.ReadUnlock)
				GPhysSceneLockRemoteObjectCallbacks.ReadUnlock(this);
			else
				UnderlyingLock.ReadUnlock();
		}

		void WriteLock()
		{
			if (GPhysSceneLockRemoteObjectCallbacks.WriteLock)
				GPhysSceneLockRemoteObjectCallbacks.WriteLock(this);
			else
				UnderlyingLock.WriteLock();
		}

		void WriteUnlock()
		{
			if (GPhysSceneLockRemoteObjectCallbacks.WriteUnlock)
				GPhysSceneLockRemoteObjectCallbacks.WriteUnlock(this);
			else
				UnderlyingLock.WriteUnlock();
		}

		FPhysSceneLockTransactionallySafe UnderlyingLock;
	};

	using FPhysSceneLock = FPhysSceneLockRemoteObject;
#else
	using FPhysSceneLock = FPhysSceneLockTransactionallySafe;
#endif
#else
	using FPhysSceneLock = FPhysSceneLockNonTransactional;
#endif

#if UE_WITH_REMOTE_OBJECT_HANDLE
template<typename T>
using TPhysicsSpscQueue = TTransactionallySafeSpscQueue<T>;
#else
template<typename T>
using TPhysicsSpscQueue = TSpscQueue<T>;
#endif

	// Stable types to use in calling code configured by the compiler switches above
	using FPhysicsSceneGuardScopedWrite = TPhysicsSceneGuardScopedWrite<FPhysSceneLock>;
	using FPhysicsSceneGuardScopedRead = TPhysicsSceneGuardScopedRead<FPhysSceneLock>;
}
