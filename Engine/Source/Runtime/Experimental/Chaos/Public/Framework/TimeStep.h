// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Defines.h"

namespace Chaos
{
	class ITimeStep
	{
	public:
		virtual ~ITimeStep() = default;

		/** 
		 * Reset the timestep to a default state. Called on restarting a simulation.
		 * Note we will not update till after an evolution so GetDt must succeed beore
		 * we call the first Update
		 */
		virtual void Reset() {}

		/**
		 * Perform any necessary update to calculate the next timestep, if any sleeps for
		 * synchronisation are required also perform them here
		 */
		virtual void Update() = 0;

		/** Get the next dt to use in simulation */
		virtual float GetCalculatedDt() const = 0;

		/** 
		 * Get how much time actually passed (Dt and any sleeps) mainly for stats.
		 * By default will just return Dt
		 */
		virtual float GetActualDt() const { return GetCalculatedDt(); }

		/** Called to set a target dt if necessary - derived timesteps can ignore if not needed */
		virtual void SetTarget(float InTarget) { (void)InTarget; }

		/** Called to retrieve the target time for the timestep. By default returns whatever the Dt was intended to be */
		virtual float GetTarget() const { return GetCalculatedDt(); }
	};

	class UE_DEPRECATED(4.27, "Deprecated, this class is to be deleted") FFixedTimeStep final : public ITimeStep
	{
	public:

		/** ITimeStep interface */
		virtual void Reset() override;
		virtual void Update() override;
		virtual float GetCalculatedDt() const override;
		virtual float GetActualDt() const override;
		virtual void SetTarget(float InTarget) override;
		virtual float GetTarget() const override;
		/** end ITimeStep interface */

	private:
		double LastTime;
		double CurrentTime;

		float TargetDt;
		float ActualDt;
	};

	class UE_DEPRECATED(4.27, "Deprecated, this class is to be deleted") FVariableTimeStep final : public ITimeStep
	{

		/** ITimeStep interface */
		virtual void Reset() override;
		virtual void Update() override;
		virtual float GetCalculatedDt() const override;
		/** end ITimeStep interface */

	private:
		double LastTime;

		float Dt;
	};

	class UE_DEPRECATED(4.27, "Deprecated, this class is to be deleted") FVariableWithCapTimestep final : public ITimeStep
	{
		/** ITimeStep interface */
		virtual void Reset() override;
		virtual void Update() override;
		virtual float GetCalculatedDt() const override;
		virtual float GetActualDt() const override;
		/** end ITimeStep interface */

	private:
		double LastTime;
		double CurrentTime;

		float Dt;
		float ActualDt;
	};

	class UE_DEPRECATED(4.27, "Deprecated, this class is to be deleted") FVariableMinimumWithCapTimestep final : public ITimeStep
	{
		/** ITimeStep interface */
		virtual void Reset() override;
		virtual void Update() override;
		virtual float GetCalculatedDt() const override;
		virtual float GetActualDt() const override;
		virtual void SetTarget(float InTarget) override;
		/** end ITimeStep interface */

	private:
		double LastTime;
		double CurrentTime;

		float Dt;
		float TargetDt;
		float ActualDt;
	};
}
