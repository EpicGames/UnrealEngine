// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"

namespace Chaos::Facades
{
	/**
	 * Provides an API to define anchoring properties on a collection
	 */
	class FCollectionAnchoringFacade
	{
	public:
		CHAOS_API FCollectionAnchoringFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FCollectionAnchoringFacade(const FManagedArrayCollection& InCollection);

		/** Create the facade attributes. */
		void DefineSchema() {}

		/** Valid if all accessors arrays are available */
		CHAOS_API bool IsValid() const;

		/** Is the facade defined from a constant collection. */
		bool IsConst() const { return InitialDynamicStateAttribute.IsConst(); }

		//
		//  Facade Functionality
		//

		CHAOS_API bool HasInitialDynamicStateAttribute() const;
		
		CHAOS_API EObjectStateType GetInitialDynamicState(int32 TransformIndex) const;
		CHAOS_API void SetInitialDynamicState(int32 TransformIndex, EObjectStateType State);
		CHAOS_API void SetInitialDynamicState(const TArray<int32>& TransformIndices, EObjectStateType State);
		
		CHAOS_API bool HasAnchoredAttribute() const;
		CHAOS_API void AddAnchoredAttribute();
		CHAOS_API void CopyAnchoredAttribute(const FCollectionAnchoringFacade& Other);

		CHAOS_API bool IsAnchored(int32 TransformIndex) const;
		CHAOS_API void SetAnchored(int32 TransformIndex, bool bValue);
		CHAOS_API void SetAnchored(const TArray<int32>& TransformIndices, bool bValue);
	
	private:
		/** Initial dynamic state of a transform ( can be changed at runtime ) */
		TManagedArrayAccessor<int32> InitialDynamicStateAttribute;

		/** Whether a transform is anchored ( orthogonal to the initial dynamic ) )*/
		TManagedArrayAccessor<bool> AnchoredAttribute;
	};
}
