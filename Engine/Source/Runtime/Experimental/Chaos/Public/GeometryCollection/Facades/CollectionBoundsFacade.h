// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "Math/MathFwd.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/Facades/CollectionSelectionFacade.h"
#include "Chaos/Triangle.h"

namespace GeometryCollection::Facades
{

	/**
	* FBoundsFacade
	* 
	* Defines common API for calculating the bounding box on a collection
	* 
	*/
	class FBoundsFacade
	{
	public:

		/**
		* FBoundsFacade Constuctor
		* @param FManagedArrayCollection : Collection input
		*/
		CHAOS_API FBoundsFacade(FManagedArrayCollection& InSelf);
		CHAOS_API FBoundsFacade(const FManagedArrayCollection& InSelf);

		/** Create the facade attributes. */
		CHAOS_API void DefineSchema();

		/** Is the facade defined constant. */
		bool IsConst() const { return BoundingBoxAttribute.IsConst(); }

		/** Is the Facade defined on the collection? */
		CHAOS_API bool IsValid() const;		

		/** UpdateBoundingBox */
		CHAOS_API void UpdateBoundingBox();

		/** BoundingBox access */
		const TManagedArray< FBox >& GetBoundingBoxes() const { return BoundingBoxAttribute.Get(); }

		/** 
		* Get center of bounding boxes 
		* warning : this is indexed on the geometry group not the transform group
		*/
		CHAOS_API TArray<FVector> GetCentroids() const;

		/** 
		* Get centroid in collection space ( center of the bounding boxes of each transform )
		* NOTE : Only transform with geometry associated with them will have a centroid , the rest will have a centroid set a the origin of the collection
		* IMPORTANT : The resulting array is indexed on transform group
		*/
		CHAOS_API TArray<FVector> GetTransformCentroidsInCollectionSpace() const;

		/** BoundingBox for the whole collection in Collection space */
		CHAOS_API FBox GetBoundingBoxInCollectionSpace() const;

		/** Returns the positions of the vertices of an FBox */
		static CHAOS_API TArray<FVector> GetBoundingBoxVertexPositions(const FBox& InBox);

		/** TransformToGeometryIndex Access */
		const TManagedArray< int32 >& GetTransformToGeometryIndex() const { return TransformToGeometryIndexAttribute.Get(); }

		/** BoundingSphere for the whole collection in Collection space */
		CHAOS_API FSphere GetBoundingSphereInCollectionSpace();

		/** Compute boundingBox for a point cloud */
		CHAOS_API FBox ComputeBoundingBox(const TArray<FVector> InPoints);

	protected:

		/** Transform based bounds evaluation, where the vertices are nested within a transform */
		void UpdateTransformBasedBoundingBox();

		/** Vertex based bounds evaluation, where the vertices are NOT nested within a transform */
		void UpdateVertexBasedBoundingBox();

	private:
		/** Compute a bounding sphere of a box */
		FSphere ComputeBoundingSphere(const FBox& InBoundingBox);

		/** Compute a bounding sphere of a point cloud 
		* These are different implementations and we use the better result */
		void ComputeBoundingSphereImpl(const TArray<FVector>& InVertices, FSphere& OutSphere) const;
		void ComputeBoundingSphereImpl2(const TArray<FVector>& InVertices, FSphere& OutSphere) const;
		void ComputeBoundingSphere(const TArray<FVector>& InVertices, FSphere& OutSphere) const ;

	private:
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;

		TManagedArrayAccessor<FBox>			BoundingBoxAttribute;
		TManagedArrayAccessor<FVector3f>	VertexAttribute;
		TManagedArrayAccessor<int32>		BoneMapAttribute;
		TManagedArrayAccessor<int32>		TransformToGeometryIndexAttribute;
		TManagedArrayAccessor<int32>		VertexStartAttribute;
		TManagedArrayAccessor<int32>		VertexCountAttribute;
	};

}
