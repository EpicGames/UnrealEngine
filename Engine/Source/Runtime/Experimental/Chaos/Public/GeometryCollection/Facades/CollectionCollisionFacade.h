// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace GeometryCollection::Facades
{

	/**
	* FCollisionFacade
	* 
	* Defines common API for storing collision information.
	* 
	*/
	class FCollisionFacade
	{
	public:

		// Attributes
		CHAOS_API static const FName IsCollisionEnabledAttributeName;

		/**
		* FCollisionFacade Constuctor
		*/

		CHAOS_API FCollisionFacade(FManagedArrayCollection& InCollection);

		CHAOS_API FCollisionFacade(const FManagedArrayCollection& InCollection);
		
		/** Define the facade */
		CHAOS_API void DefineSchema();

		/** Is the Facade const */
		bool IsConst() const { return Collection == nullptr; }

		CHAOS_API bool IsValid() const;

		CHAOS_API void SetCollisionEnabled(const TArray<int32>& VertexIndices);

		CHAOS_API bool IsCollisionEnabled(int32 VertexIndex) const;
	private:
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;

		TManagedArrayAccessor<bool> IsCollisionEnabledAttribute;
	};

}
