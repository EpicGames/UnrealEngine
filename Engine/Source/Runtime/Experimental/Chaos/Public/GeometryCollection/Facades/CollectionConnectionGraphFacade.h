// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace GeometryCollection::Facades
{

	/**
	 * Provides an API for connection-graph related attributes
	 */
	class FCollectionConnectionGraphFacade
	{
	public:
		CHAOS_API FCollectionConnectionGraphFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FCollectionConnectionGraphFacade(const FManagedArrayCollection& InCollection);

		/** Does the collection support the facade. */
		CHAOS_API bool IsValid() const;

		/** Is the facade defined constant. */
		bool IsConst() const
		{
			return ConnectionEdgeStartAttribute.IsConst();
		}

		/** Create the facade attributes. */
		CHAOS_API void DefineSchema();

		/** Remove the attributes */
		CHAOS_API void ClearAttributes();

		/** Connect two bones */
		CHAOS_API void ConnectWithContact(int32 BoneA, int32 BoneB, float ContactArea);

		/** Connect two bones */
		CHAOS_API void Connect(int32 BoneA, int32 BoneB);

		/** Enable or disable the Contact Area attribute */
		CHAOS_API void EnableContactAreas(bool bEnable, float DefaultContactArea = 1.0f);

		/** Reserve space for a number of additional connections */
		CHAOS_API void ReserveAdditionalConnections(int32 NumAdditionalConnections);

		/**  Connections between bones that have the same parent in the hierarchy */
		UE_DEPRECATED(5.3, "We have switched to an edge array connection representation. Please use the accessor functions (GetConnection, NumConnections, etc) to access the arrays of edge data.")
		TManagedArrayAccessor<TSet<int32>> ConnectionsAttribute;

		// Get the transform indices for the ConnectionIndex
		CHAOS_API TPair<int32, int32> GetConnection(int32 ConnectionIndex) const;
		// Get the contact area for the ConnectionIndex
		CHAOS_API float GetConnectionContactArea(int32 ConnectionIndex) const;

		CHAOS_API bool HasContactAreas() const;

		// Number of connection edges
		CHAOS_API int32 NumConnections() const;

		// Verifies the connections indices are valid indices into the Collection's Transform group
		CHAOS_API bool HasValidConnections() const;

		// Remove all edge connections, but keep the connection attributes
		CHAOS_API void ResetConnections();

	protected:
		 TManagedArrayAccessor<int32> ConnectionEdgeStartAttribute;
		 TManagedArrayAccessor<int32> ConnectionEdgeEndAttribute;
		 TManagedArrayAccessor<float> ConnectionEdgeContactAttribute;

#if UE_BUILD_DEBUG
		/** Optional Parent array for validating connections in debug */
		TManagedArrayAccessor<int32> ParentAttribute;
#endif 
	};

}
