// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/Facades/CollectionSelectionFacade.h"

namespace GeometryCollection::Facades
{
	/**
	* Transient constraint candidate data.
	*/
	struct FConstraintOverridesCandidateData
	{
		FConstraintOverridesCandidateData()
			: VertexIndex(INDEX_NONE)
			, BoneIndex(INDEX_NONE)
		{}

		int32 VertexIndex;
		int32 BoneIndex;
	};

	/**
	* Transient constraint target data.
	*/
	struct FConstraintOverridesTargetData
	{
		FConstraintOverridesTargetData()
			: VertexIndex(INDEX_NONE)
		{}

		int32 VertexIndex;
		FVector3f PositionTarget;
	};

	/**
	* Transient constraint candidates.  Typically stored in the rest collection.
	*/
	class FConstraintOverrideCandidateFacade
	{
	public:
		static CHAOS_API const FName GroupName;
		static CHAOS_API const FName TargetIndex;
		static CHAOS_API const FName BoneIndex;

		CHAOS_API FConstraintOverrideCandidateFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FConstraintOverrideCandidateFacade(const FManagedArrayCollection& InCollection);

		CHAOS_API void DefineSchema();
		bool IsConst() const { return Collection == nullptr; }
		CHAOS_API bool IsValid() const;

		CHAOS_API int32 Add(FConstraintOverridesCandidateData& InputData);
		CHAOS_API void Clear();
		CHAOS_API FConstraintOverridesCandidateData Get(const int32 Index) const;
		int32 Num() const { return TargetIndexAttribute.Num(); }

	private:
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;

		TManagedArrayAccessor<int32> TargetIndexAttribute;
		TManagedArrayAccessor<int32> BoneIndexAttribute;
	};

	/**
	* Transient constraint targets.  Typically stored in the simulation collection.
	*/
	class FConstraintOverrideTargetFacade
	{
	public:
		static CHAOS_API const FName GroupName;
		static CHAOS_API const FName TargetIndex;
		static CHAOS_API const FName TargetPosition;

		CHAOS_API FConstraintOverrideTargetFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FConstraintOverrideTargetFacade(const FManagedArrayCollection& InCollection);

		CHAOS_API void DefineSchema();
		bool IsConst() const { return Collection == nullptr; }
		CHAOS_API bool IsValid() const;

		CHAOS_API int32 Add(FConstraintOverridesTargetData& InputData);
		CHAOS_API void Clear();
		CHAOS_API FConstraintOverridesTargetData Get(const int32 Index) const;
		CHAOS_API int32 GetIndex(const int32 Index) const;
		CHAOS_API const FVector3f& GetPosition(const int32 Index) const;
		int32 Num() const { return TargetIndexAttribute.Num(); }

	private:
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;

		TManagedArrayAccessor<int32> TargetIndexAttribute;
		TManagedArrayAccessor<FVector3f> TargetPositionAttribute;
	};



} // namespace GeometryCollection::Facades
