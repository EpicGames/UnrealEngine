// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
	
namespace GeometryCollection::Facades
{
	class FCollectionCurveGeometryFacade
	{
	public:
		/** Collection group names */
		static CHAOS_API const FName CurvesGroup;
		static CHAOS_API const FName PointsGroup;
		
		/** Collection attribute names */
		static CHAOS_API const FName CurvePointOffsetsAttribute;
		static CHAOS_API const FName GeometryCurveOffsetsAttribute;
		static CHAOS_API const FName PointRestPositionsAttribute;
		static CHAOS_API const FName PointRestOrientationsAttribute;
		static CHAOS_API const FName PointCurveIndicesAttribute;
		static CHAOS_API const FName CurveGeometryIndicesAttribute;
		static CHAOS_API const FName GeometryGroupNamesAttribute;
		static CHAOS_API const FName GeometryCurveThicknessAttribute;
		static CHAOS_API const FName CurveSourceIndicesAttribute;

		CHAOS_API FCollectionCurveGeometryFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FCollectionCurveGeometryFacade(const FManagedArrayCollection& InCollection);

		/** Create the facade attributes. */
		CHAOS_API void DefineSchema();

		/** Is the facade defined constant. */
		bool IsConst() const { return Collection == nullptr; }

		/** Is the Facade defined on the collection? */
		CHAOS_API bool IsValid() const;
		
		/** Get the number of curves */
		int32 GetNumCurves() const { return CurvePointOffsets.Num(); }

		/** Get the number of geometry */
		int32 GetNumGeometry() const { return GeometryCurveOffsets.Num(); }

		/** Get the number of points */
		int32 GetNumPoints() const { return PointRestPositions.Num(); }
		
		/** Get the point rest positions */
		const TArray<FVector3f>& GetPointRestPositions() const { return PointRestPositions.Get().GetConstArray(); }

		/** Get the point rest orientations */
		const TArray<FQuat4f>& GetPointRestOrientations() const { return PointRestOrientations.Get().GetConstArray(); }

		/** Get the curve point offsets */
		const TArray<int32>& GetCurvePointOffsets() const { return CurvePointOffsets.Get().GetConstArray(); }

		/** Get the geometry curve offsets */
		const TArray<int32>& GetGeometryCurveOffsets() const { return GeometryCurveOffsets.Get().GetConstArray(); }

		/** Get the point curve indices */
		const TArray<int32>& GetPointCurveIndices() const { return PointCurveIndices.Get().GetConstArray(); }

		/** Get the curve geometry indices */
		const TArray<int32>& GetCurveGeometryIndices() const { return CurveGeometryIndices.Get().GetConstArray(); }

		/** Get the geometry group names */
		const TArray<FString>& GetGeometryGroupNames() const { return GeometryGroupNames.Get().GetConstArray(); }

		/** Get the geometry curve thickness */
		const TArray<float>& GetGeometryCurveThickness() const { return GeometryCurveThickness.Get().GetConstArray(); }

		/** Get the curve source indices*/
		const TArray<int32>& GetCurveSourceIndices() const { return CurveSourceIndices.Get().GetConstArray(); }
		
		/** Set the point rest positions */
		void SetPointRestPositions(const TArray<FVector3f>& InPointRestPositions) { PointRestPositions.Modify() = InPointRestPositions; UpdatePointRestOrientations(); }	

		/** Set the geometry  group names */
		void SetGeometryGroupNames(const TArray<FString>& InGeometryGroupNames) { GeometryGroupNames.Modify() = InGeometryGroupNames; }
		
		/** Set the geometry curve thickness */
		void SetGeometryCurveThickness(const TArray<float>& InGeometryCurveThickness) { GeometryCurveThickness.Modify() = InGeometryCurveThickness; }	
		
		/** Set the curve point offsets */
		void SetCurvePointOffsets(const TArray<int32>& InCurvePointOffsets) { CurvePointOffsets.Modify() = InCurvePointOffsets; UpdatePointCurveIndices(); }

		/** Set the geometry curve offsets */
		void SetGeometryCurveOffsets(const TArray<int32>& InGeometryCurveOffsets) { GeometryCurveOffsets.Modify() = InGeometryCurveOffsets; UpdateCurveGeometryIndices(); }

		/** Set the curve source indices */
		void SetCurveSourceIndices(const TArray<int32>& InCurveSourceIndices) { CurveSourceIndices.Modify() = InCurveSourceIndices; }
		
		/** Initialize the whole curve collection  */
		CHAOS_API void InitCurvesCollection(const TArray<FVector3f>& InPointRestPositions, const TArray<int32>& InCurvePointOffsets, const TArray<int32>& InGeometryCurveOffsets,
			const TArray<FString>& InGeometryGroupNames, const TArray<float>& InGeometryCurveThickness, const TArray<int32>& InCurveSourceIndices);

		/** Get the managed array collection */
		const FManagedArrayCollection& GetManagedArrayCollection() const {return ConstCollection;}
		
		/** Get the point positions of a specific curve */
		CHAOS_API void GetCurvePointPositions(const int32 CurveIndex, TArray<FVector3f>& OutPoints) const;

		/** 
		* Get the resampled positions of a specific curve 
		* the curve will be sampled at regular interval along the curve, to match the NumSamples parameters
		*/
		CHAOS_API void GetCurveResampledPositions(const int32 CurveIndex, int32 NumSamples, TArray<FVector3f>& OutPoints) const;

	protected :

		/** Build geometry collection from the curves one */
		void BuildGeometryCollection();
		
		/** Update the point rest orientations with parallel transport */
		void UpdatePointRestOrientations();
		
		/** Update the point curve indices from the offsets  */
		void UpdatePointCurveIndices();

		/** Update the curve object indices from the offsets  */
		void UpdateCurveGeometryIndices();
		
		/** Const collection the facade is linked to */
		const FManagedArrayCollection& ConstCollection;

		/** Non-const collection the facade is linked to */
		FManagedArrayCollection* Collection = nullptr;
		
		/** Points rest orientation */
		TManagedArrayAccessor<FQuat4f> PointRestOrientations;

		/** Points rest position */
		TManagedArrayAccessor<FVector3f> PointRestPositions;

		/** Curves point offset */
		TManagedArrayAccessor<int32> CurvePointOffsets;
                                                 
		/** Geometry curve offset */
		TManagedArrayAccessor<int32> GeometryCurveOffsets;

		/** Points curve index */
		TManagedArrayAccessor<int32> PointCurveIndices;

		/** Curves object index */
		TManagedArrayAccessor<int32> CurveGeometryIndices;

		/** Geometry group names */
		TManagedArrayAccessor<FString> GeometryGroupNames;

		/** Geometry curve thickness */
		TManagedArrayAccessor<float> GeometryCurveThickness;

		/** Curve source indices */
		TManagedArrayAccessor<int32> CurveSourceIndices;
	};

	class FCollectionCurveHierarchyFacade
	{
	public:

		/** Collection attribute names */
		static CHAOS_API const FName CurveParentIndicesAttribute;
		static CHAOS_API const FName CurveLodIndicesAttribute;

		CHAOS_API FCollectionCurveHierarchyFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FCollectionCurveHierarchyFacade(const FManagedArrayCollection& InCollection);

		/** Create the facade attributes. */
		CHAOS_API void DefineSchema();

		/** Is the facade defined constant. */
		bool IsConst() const { return Collection == nullptr; }

		/** Is the Facade defined on the collection? */
		CHAOS_API bool IsValid() const;
		
		/** Get the number of curves */
		int32 GetNumCurves() const { return CurveLodIndices.Num(); }
		
		/** Get the curve lod indices */
		const TArray<int32>& GetCurveLodIndices() const { return CurveLodIndices.Get().GetConstArray(); }

		/** Get the curve parent indices */
		const TArray<int32>& GetCurveParentIndices() const { return CurveParentIndices.Get().GetConstArray(); }
		
		/** Set the curve lod indices */
		void SetCurveLodIndices(const TArray<int32>& InCurveLodIndices) { CurveLodIndices.Modify() = InCurveLodIndices; }

		/** Set the curve parent indices */
		void SetCurveParentIndices(const TArray<int32>& InCurveParentIndices) { CurveParentIndices.Modify() = InCurveParentIndices; }

		/** Get the managed array collection */
		const FManagedArrayCollection& GetManagedArrayCollection() const {return ConstCollection;}
		
	protected :

		/** Build geometry collection from the curves one */
		void BuildGeometryCollection();
		
		/** Update the point rest orientations with parallel transport */
		void UpdatePointRestOrientations();
		
		/** Update the point curve indices from the offsets  */
		void UpdatePointCurveIndices();

		/** Update the curve object indices from the offsets  */
		void UpdateCurveGeometryIndices();
		
		/** Const collection the facade is linked to */
		const FManagedArrayCollection& ConstCollection;

		/** Non-const collection the facade is linked to */
		FManagedArrayCollection* Collection = nullptr;

		/** Curve parent indices*/
		TManagedArrayAccessor<int32> CurveParentIndices;

		/** Curve lod indices */
		TManagedArrayAccessor<int32> CurveLodIndices;
	};
	
	class FCollectionCurveAttributesFacade
	{
	public:

		/** Collection attribute names */
		static CHAOS_API const FName CurveCardGroupsAttribute;

		CHAOS_API FCollectionCurveAttributesFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FCollectionCurveAttributesFacade(const FManagedArrayCollection& InCollection);

		/** Create the facade attributes. */
		CHAOS_API void DefineSchema();

		/** Is the facade defined constant. */
		bool IsConst() const { return Collection == nullptr; }

		/** Is the Facade defined on the collection? */
		CHAOS_API bool IsValid() const;
		
		/** Get the number of curves */
		int32 GetNumCurves() const { return CurveCardGroups.Num(); }
		
		/** Get the curve card groups */
		const TArray<FName>& GetCurveCardGroups() const { return CurveCardGroups.Get().GetConstArray(); }

		/** Set the curve card groups */
		void SetCurveCardGroups(const TArray<FName>& InCurveCardGroups) { CurveCardGroups.Modify() = InCurveCardGroups; }

		/** Get the managed array collection */
		const FManagedArrayCollection& GetManagedArrayCollection() const {return ConstCollection;}
		
	protected :
		
		/** Const collection the facade is linked to */
		const FManagedArrayCollection& ConstCollection;

		/** Non-const collection the facade is linked to */
		FManagedArrayCollection* Collection = nullptr;

		/** Curve card groups */
		TManagedArrayAccessor<FName> CurveCardGroups;
	};
}

