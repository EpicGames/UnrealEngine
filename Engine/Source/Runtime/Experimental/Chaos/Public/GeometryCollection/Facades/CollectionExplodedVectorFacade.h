// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"


namespace GeometryCollection::Facades
{

	class FCollectionExplodedVectorFacade
	{
	public:
		CHAOS_API FCollectionExplodedVectorFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FCollectionExplodedVectorFacade(const FManagedArrayCollection& InCollection);

		/**
		 * returns true if all the necessary attributes are present
		 * if not then the API can be used to create
		 */
		CHAOS_API bool IsValid() const;

		/**
		 * Add the necessary attributes if they are missing
		 */
		CHAOS_API void DefineSchema();

		/**
		 * Adds exploded vector to global matrices translation 
		 */
		CHAOS_API void UpdateGlobalMatricesWithExplodedVectors(TArray<FMatrix>& InOutGlobalMatrices) const;
		CHAOS_API void UpdateGlobalMatricesWithExplodedVectors(TArray<FTransform>& InOutGlobalTransforms) const;

	private:

		TManagedArrayAccessor<FVector3f> ExplodedVectorAttribute;
	};

}
