// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"

namespace Chaos::Facades
{
	/**
	 * Provides an API to read and manipulate hierarchy in a managed array collection
	 */
	class FCollectionHierarchyFacade
	{
	public:
		enum class EPersistencePolicy : uint8
		{
			KeepExistingPersistence,
			MustBePersistent
		};

		CHAOS_API FCollectionHierarchyFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FCollectionHierarchyFacade(const FManagedArrayCollection& InCollection);

		/** Create the facade attributes. */
		void DefineSchema() {}

		/** Valid if parent and children arrays are available */
		CHAOS_API bool IsValid() const;

		/** Is this facade const access */
		bool IsConst() const { return ParentAttribute.IsConst(); }

		/** whether the level attribute is available */
		CHAOS_API bool HasLevelAttribute() const;

		/** whether the level attribute is persistent */
		CHAOS_API bool IsLevelAttributePersistent() const;

		/** Get the root index */
		CHAOS_API int32 GetRootIndex() const;

		/** Get the root indicies */
		CHAOS_API TArray<int32> GetRootIndices() const;

		/** Get direct children from a specific transform index. Assumes parent attribute is valid. */
		CHAOS_API TArray<int32> GetChildrenAsArray(int32 TransformIndex) const;

		/** Get set of children from a transform index, or nullptr if not available */
		CHAOS_API const TSet<int32>* FindChildren(int32 TransformIndex) const;

		/** Get parent of a specific transform index */
		CHAOS_API int32 GetParent(int32 TransformIndex) const;

		/**
		 * Enumerate transforms at or under TransformIndex where ShouldProcess returns true, stopping recursion and calling ProcessTransformFunc on such transforms.
		 */
		CHAOS_API void EnumerateChildTransforms(int32 TransformIndex, TFunctionRef<bool(int32)> ShouldProcess, TFunctionRef<void(int32)> ProcessTransformFunc) const;

		/** 
		* Get initial level of a specific transform index 
		* If the attribute is missing return INDEX_NONE
		*/
		CHAOS_API int32 GetInitialLevel(int32 TransformIndex) const;

		/** 
		* Update level attribute for all elements (and create it if it is missing ) 
		* @Param PersistencePolicy whether to make the attribute persistent or keep the existing state
		*/ 
		CHAOS_API void GenerateLevelAttribute();

		/**
		* Get transform indices in a depth first order 
		*/
		CHAOS_API TArray<int32> GetTransformArrayInDepthFirstOrder() const;

		/**
		* compute transform indices in a depth first order 
		*/
		CHAOS_API TArray<int32> ComputeTransformIndicesInDepthFirstOrder() const;

		/**
		* compute transform indices in a breadth first order ( root to leaves )
		*/
		CHAOS_API TArray<int32> ComputeTransformIndicesInBreadthFirstOrder() const;

		/**
		* generate a non serialize array that stores the indices of the transforms in a breath first order
		*/
		CHAOS_API void GenerateBreadthFirstOrderIndicesAttribute();

		/**
		* get the breadth first ordered transform indices from the stored attribute if any
		* may return empty array if the attribute does not exists
		*/
		CHAOS_API const TArray<int32>& GetBreadthFirstOrderIndicesFromAttribute() const;

		CHAOS_API int32 ComputeLevelAttributeMax() const;

	public:
		/** Get the root indicies */
		static CHAOS_API TArray<int32> GetRootIndices(const TManagedArrayAccessor<int32>& ParentAttribute);

	private:
		TManagedArrayAccessor<int32>		ParentAttribute;
		TManagedArrayAccessor<TSet<int32>>	ChildrenAttribute;
		TManagedArrayAccessor<int32>		LevelAttribute;
	};
}
