// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"

namespace GeometryCollection::Facades
{
	/**
	 * Provides an API to read and manipulate hierarchy in a managed array collection
	 */
	class FCollectionInstancedMeshFacade
	{
	public:
		CHAOS_API FCollectionInstancedMeshFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FCollectionInstancedMeshFacade(const FManagedArrayCollection& InCollection);

		/** Create the facade attributes. */
		CHAOS_API void DefineSchema();

		/** Valid if parent and children arrays are available */
		CHAOS_API bool IsValid() const;

		/** Is the facade defined constant. */
		CHAOS_API bool IsConst() const;

		/** get the total number of indices */
		CHAOS_API int32 GetNumIndices() const;

		/** get the instance mesh index for specific transform index */
		CHAOS_API int32 GetIndex(int32 TransformIndex) const;

		/** set the instance mesh index for specific transform index */
		CHAOS_API void SetIndex(int32 TransformIndex, int32 InstanceMeshIndex);


	private:
		TManagedArrayAccessor<int32> InstancedMeshIndexAttribute;
	};
}
