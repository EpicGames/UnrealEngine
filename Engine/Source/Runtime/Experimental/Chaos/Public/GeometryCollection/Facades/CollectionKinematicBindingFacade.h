// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/Facades/CollectionSelectionFacade.h"

namespace GeometryCollection::Facades
{
	/** Kinematic Facade */
	class FKinematicBindingFacade
	{
	public:

		typedef GeometryCollection::Facades::FSelectionFacade::FSelectionKey FBindingKey;

		//
		// Kinematics
		//
		static CHAOS_API const FName KinematicGroup;
		static CHAOS_API const FName KinematicBoneBindingIndex;
		static CHAOS_API const FName KinematicBoneBindingToGroup;

		CHAOS_API FKinematicBindingFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FKinematicBindingFacade(const FManagedArrayCollection& InCollection);

		/** Create the facade attributes. */
		CHAOS_API void DefineSchema();

		/** Is the facade defined constant. */
		bool IsConst() const { return Collection==nullptr; }

		/** Is the Facade defined on the collection? */
		CHAOS_API bool IsValid() const;

		//
		//  Skeletal Mesh Bone Bindings
		//
		CHAOS_API FBindingKey SetBoneBindings(const int32 BoneIndex, const TArray<int32>& Vertices, const TArray<float>& Weights);
		CHAOS_API void GetBoneBindings(const FBindingKey& Key, int32& OutBoneIndex, TArray<int32>& OutBoneVerts, TArray<float>& OutBoneWeights) const;

		CHAOS_API int32 AddKinematicBinding(const FBindingKey& Key);
		int32 NumKinematicBindings() const { return KinemaitcBoneBindingAttribute.Num(); }
		CHAOS_API FBindingKey GetKinematicBindingKey(int Index) const;

	private:
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;

		TManagedArrayAccessor<int32> KinemaitcBoneBindingAttribute;
		TManagedArrayAccessor<FString> KinemaitcBoneBindingToGroupAttribute;
	};
}
