// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionUVFacade.h"

namespace GeometryCollection::Facades
{

	class FCollectionMeshFacade : public FCollectionUVFacade
	{
	public:
		CHAOS_API FCollectionMeshFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FCollectionMeshFacade(const FManagedArrayCollection& InCollection);

		/**
		 * returns true if all the necessary attributes are present
		 * if not then the API can be used to create
		 */
		CHAOS_API bool IsValid() const;

		/**
		 * Add the necessary attributes if they are missing
		 */
		CHAOS_API void DefineSchema();

		/**
		 * Returns the vertex indicies for the bone
		 */
		CHAOS_API const TArray<int32> GetVertexIndices(int32 BoneIdx) const;

		/**
		 * Returns the vertex positions for the bone in bone space
		 */
		CHAOS_API const TArrayView<const FVector3f> GetVertexPositions(int32 BoneIdx) const;

		/**
		 * Returns the vertex positions in collection space
		 */
		CHAOS_API void GetVerticesInCollectionSpace(TArray<FVector3f>& OutPositions) const;

		/**
		* Returns the triangles for the bone 
		 */
		CHAOS_API const TArrayView<const FIntVector> GetTriangles(int32 BoneIdx) const;

		/**
		 * Returns the face indices for the bone
		 */
		CHAOS_API const TArray<int32> GetFaceIndices(int32 BoneIdx) const;	

		/**
		 * Returns the vertex indices of the face for the bone
		 */
//		const TArray<FIntVector> GetFaceVertexIndices(int32 BoneIdx) const;

		/**
		 * Bakes the transforms into the vertex positions and sets the bone transforms to identity
		 */
		CHAOS_API void BakeTransform(int32 TransformIdx, const FTransform& InTransform);

		CHAOS_API const TArray<int32> GetGeometryGroupIndexArray() const;

		TManagedArrayAccessor<int32> TransformToGeometryIndexAttribute;
		TManagedArrayAccessor<int32> TransformIndexAttribute;
		TManagedArrayAccessor<FVector3f> VertexAttribute;
		TManagedArrayAccessor<FVector3f> TangentUAttribute;
		TManagedArrayAccessor<FVector3f> TangentVAttribute;
		TManagedArrayAccessor<FVector3f> NormalAttribute;
		TManagedArrayAccessor<FLinearColor> ColorAttribute;
		TManagedArrayAccessor<int32> BoneMapAttribute;
		TManagedArrayAccessor<int32> VertexStartAttribute;
		TManagedArrayAccessor<int32> VertexCountAttribute;
		TManagedArrayAccessor<FIntVector> IndicesAttribute;
		TManagedArrayAccessor<bool> VisibleAttribute;
		TManagedArrayAccessor<int32> MaterialIndexAttribute;
		TManagedArrayAccessor<int32> MaterialIDAttribute;
		TManagedArrayAccessor<bool> InternalAttribute;
		TManagedArrayAccessor<int32> FaceStartAttribute;
		TManagedArrayAccessor<int32> FaceCountAttribute;
		TManagedArrayAccessor<int32> ParentAttribute;
		TManagedArrayAccessor<FTransform3f> TransformAttribute;
	};
}
