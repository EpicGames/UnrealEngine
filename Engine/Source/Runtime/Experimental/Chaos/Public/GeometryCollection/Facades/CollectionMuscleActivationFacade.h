// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/Facades/CollectionSelectionFacade.h"

namespace Chaos
{
	class FLinearCurve;
}

namespace GeometryCollection::Facades
{
	//Activation data for each muscle
	struct FMuscleActivationData
	{
		int32 GeometryGroupIndex; //Geometry group index of the muscle
		TArray<int32> MuscleActivationElement; //Contractible tetrahedra
		FIntVector2 OriginInsertionPair; //Muscle origin point and insertion point (to determine muscle length)
		float OriginInsertionRestLength; //Muscle origin-insertion rest length
		TArray<Chaos::PMatrix33d> FiberDirectionMatrix; //Per-element fiber direction orthogonal matrix: [v, w1, w2], v is the fiber direction
		TArray<float> ContractionVolumeScale; // Per-element volume scale for muscle contraction. Muscles gain volume during contraction if > 1. Volume-preserving if 1.
		float FiberLengthRatioAtMaxActivation = 0.5f; // How much muscle fibers shorten at max activation 1. A smaller value means more contraction in the fiber direction.
		float MuscleLengthRatioThresholdForMaxActivation = 0.75f; // Muscle length ratio (defined by origin-insertion distance) below this threshold is considered to reach max activation 1.
		float InflationVolumeScale = 1.f; // Increases muscle rest volume if > 1 and decreases muscle rest volume if < 1.
		TArray<TArray<FVector3f>> FiberStreamline; //Fiber streamline(s) for inverse dynamics
		TArray<float> FiberStreamlineRestLength; //Fiber streamline rest length(s)
	};
	
	class FMuscleActivationFacade
	{
	public:

		static CHAOS_API const FName GroupName;
		static CHAOS_API const FName GeometryGroupIndex;
		static CHAOS_API const FName MuscleActivationElement;
		static CHAOS_API const FName OriginInsertionPair;
		static CHAOS_API const FName OriginInsertionRestLength;
		static CHAOS_API const FName FiberDirectionMatrix;
		static CHAOS_API const FName ContractionVolumeScale;
		static CHAOS_API const FName FiberLengthRatioAtMaxActivation;
		static CHAOS_API const FName MuscleLengthRatioThresholdForMaxActivation;
		static CHAOS_API const FName InflationVolumeScale;
		static CHAOS_API const FName FiberStreamline;
		static CHAOS_API const FName FiberStreamlineRestLength;
		static CHAOS_API const FName MuscleActivationCurveName;
		static CHAOS_API const FName LengthActivationCurve;

		CHAOS_API FMuscleActivationFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FMuscleActivationFacade(const FManagedArrayCollection& InCollection);

		/** Create the facade attributes. */
		CHAOS_API void DefineSchema();

		/** Is the facade defined constant. */
		bool IsConst() const { return Collection == nullptr; }

		/** Is the Facade defined on the collection? */
		CHAOS_API bool IsValid() const;

		//
		//  Skeletal Mesh Bone Bindings
		//
		CHAOS_API int32 AddMuscleActivationData(const FMuscleActivationData& InputData);
		CHAOS_API bool UpdateMuscleActivationData(const int32 DataIndex, const FMuscleActivationData& InputData);
		CHAOS_API FMuscleActivationData GetMuscleActivationData(const int32 DataIndex) const;
		bool IsValidGeometryIndex(const int32 Index) const { return 0 <= Index && Index < ConstCollection.NumElements(FGeometryCollection::GeometryGroup); };
		bool IsValidElementIndex(const int32 Index) const { return 0 <= Index && Index < ConstCollection.NumElements("Tetrahedral"); };
		int32 NumMuscles() const { return MuscleActivationElementAttribute.Num(); }
		bool IsValidMuscleIndex(const int32 Index) const { return 0 <= Index && Index < NumMuscles(); };
		CHAOS_API int32 MuscleVertexOffset(const int32 MuscleIndex) const;
		CHAOS_API int32 NumMuscleVertices(const int32 MuscleIndex) const;
		CHAOS_API FString FindMuscleName(const int32 MuscleIndex) const;
		CHAOS_API int32 FindMuscleIndexByName(const FString MuscleName) const;
		CHAOS_API int32 FindMuscleGeometryIndex(const int32 MuscleIndex) const;
		CHAOS_API int32 RemoveInvalidMuscles();
		CHAOS_API bool SetUpMuscleActivation(const TArray<int32>& Origin, const TArray<int32>& Insertion, float ContractionVolumeScale = 1.f);
		CHAOS_API void UpdateGlobalMuscleActivationParameters(
			float InGlobalContractionVolumeScale,
			float InGlobalFiberLengthRatioAtMaxActivation,
			float InGlobalMuscleLengthRatioThresholdForMaxActivation,
			float InGlobalInflationVolumeScale);
		CHAOS_API bool UpdateMuscleActivationParameters(
			int32 MuscleIndex,
			float InContractionVolumeScale,
			float InFiberLengthRatioAtMaxActivation,
			float InMuscleLengthRatioThresholdForMaxActivation,
			float InInflationVolumeScale);
		CHAOS_API void UpdateGlobalLengthActivationCurve(const Chaos::FLinearCurve& InGlobalLengthActivationCurve);
		CHAOS_API void UpdateLengthActivationCurve(int32 MuscleIndex, const Chaos::FLinearCurve& InLengthActivationCurve);
		CHAOS_API Chaos::FLinearCurve GetLengthActivationCurve(int32 MuscleIndex) const;
		CHAOS_API TArray<TArray<TArray<FVector3f>>> BuildStreamlines(const TArray<int32>& Origin, const TArray<int32>& Insertion,
			 int32 NumLinesMultiplier, int32 MaxStreamlineIterations, int32 MaxPointsPerLine);
		CHAOS_API int32 AssignCurveName(const FString& CurveName, const FString& MuscleName);
		CHAOS_API TArray<int32> FindMuscleIndexByCurveName(const FString& CurveName) const;
	private:
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;
		TManagedArrayAccessor<int32> GeometryGroupIndexAttribute;
		TManagedArrayAccessor<TArray<int32>> MuscleActivationElementAttribute;
		TManagedArrayAccessor<FIntVector2> OriginInsertionPairAttribute;
		TManagedArrayAccessor<float> OriginInsertionRestLengthAttribute;
		TManagedArrayAccessor<TArray<Chaos::PMatrix33d>> FiberDirectionMatrixAttribute;
		TManagedArrayAccessor<TArray<float>> ContractionVolumeScaleAttribute;
		TManagedArrayAccessor<float> FiberLengthRatioAtMaxActivationAttribute;
		TManagedArrayAccessor<float> MuscleLengthRatioThresholdForMaxActivationAttribute;
		TManagedArrayAccessor<float> InflationVolumeScaleAttribute;
		TManagedArrayAccessor<TArray<TArray<FVector3f>>> FiberStreamlineAttribute;
		TManagedArrayAccessor<TArray<float>> FiberStreamlineRestLengthAttribute;
		TManagedArrayAccessor<FString> MuscleActivationCurveNameAttribute;
		TManagedArrayAccessor<Chaos::FLinearCurve> LengthActivationCurveAttribute;
	};
}
