// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionSelectionFacade.h"

namespace GeometryCollection::Facades
{
	struct FPositionTargetsData
	{
		TArray<int32> TargetIndex;
		TArray<int32> SourceIndex;
		TArray<float> TargetWeights;
		TArray<float> SourceWeights;
		UE_DEPRECATED(5.6, "TargetName will be removed and is no longer populated.")
		FString TargetName;
		UE_DEPRECATED(5.6, "SourceName will be removed and is no longer populated.")
		FString SourceName;
		float Stiffness = 0.f;
		float Damping = 0.f;
		bool bIsAnisotropic = false;
		bool bIsZeroRestLength = false;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FPositionTargetsData() = default;
		FPositionTargetsData(const FPositionTargetsData&) = default;
		FPositionTargetsData(FPositionTargetsData&&) = default;
		FPositionTargetsData& operator=(const FPositionTargetsData&) = default;
		FPositionTargetsData& operator=(FPositionTargetsData&&) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	/** Kinematic Facade */
	class FPositionTargetFacade
	{
	public:

		typedef GeometryCollection::Facades::FSelectionFacade::FSelectionKey FBindingKey;

		//
		// Kinematics
		//
		static CHAOS_API const FName GroupName;
		static CHAOS_API const FName TargetIndex;
		static CHAOS_API const FName SourceIndex;
		static CHAOS_API const FName Stiffness;
		static CHAOS_API const FName Damping;
		UE_DEPRECATED(5.6, "SourceName will be removed.")
		static CHAOS_API const FName SourceName;
		UE_DEPRECATED(5.6, "TargetName will be removed.")
		static CHAOS_API const FName TargetName;
		static CHAOS_API const FName TargetWeights;
		static CHAOS_API const FName SourceWeights;
		static CHAOS_API const FName IsAnisotropic;
		static CHAOS_API const FName IsZeroRestLength;
		
		CHAOS_API FPositionTargetFacade(FManagedArrayCollection& InCollection, const FName& InVerticesGroup = FGeometryCollection::VerticesGroup);
		CHAOS_API FPositionTargetFacade(const FManagedArrayCollection& InCollection, const FName& InVerticesGroup = FGeometryCollection::VerticesGroup);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FPositionTargetFacade(FPositionTargetFacade&&) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		/** Create the facade attributes. */
		CHAOS_API void DefineSchema();

		/** Is the facade defined constant. */
		bool IsConst() const { return Collection == nullptr; }

		/** Is the Facade defined on the collection? */
		CHAOS_API bool IsValid() const;

		//
		//  Skeletal Mesh Bone Bindings
		//
		CHAOS_API int32 AddPositionTarget(const FPositionTargetsData& InputData);
		CHAOS_API FPositionTargetsData GetPositionTarget(const int32 DataIndex) const;
		int32 NumPositionTargets() const { return TargetIndexAttribute.Num(); }
		
		/** Remove position targets with invalid indices*/
		CHAOS_API int32 RemoveInvalidPositionTarget();
		/** Remove position targets between two groups of vertices*/
		CHAOS_API int32 RemovePositionTargetBetween(TFunctionRef<bool(const int32 VertexIdx)> IsVertexGroup1, TFunctionRef<bool(const int32 VertexIdx)> IsVertexGroup2);

	protected:
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;
		const FName VerticesGroup;

		TManagedArrayAccessor<TArray<int32>> TargetIndexAttribute;
		TManagedArrayAccessor<TArray<int32>> SourceIndexAttribute;
		TManagedArrayAccessor<float> StiffnessAttribute;
		TManagedArrayAccessor<float> DampingAttribute;
		TManagedArrayAccessor<TArray<float>> TargetWeightsAttribute;
		TManagedArrayAccessor<TArray<float>> SourceWeightsAttribute;
		TManagedArrayAccessor<bool> IsAnisotropicAttribute;
		TManagedArrayAccessor<bool> IsZeroRestLengthAttribute;
	};
}
