// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

struct FManagedArrayCollection;

namespace GeometryCollection::Facades
{
	/**
	 * this class helps reading and writing the packed data used in the managed array
	 */
	struct FRemoveOnBreakData
	{
	public:
		inline static const FVector4f DisabledPackedData{ -1, 0, 0, 0 };

		CHAOS_API FRemoveOnBreakData();
		CHAOS_API FRemoveOnBreakData(const FVector4f& InPackedData);
		CHAOS_API FRemoveOnBreakData(bool bEnable, const FVector2f& BreakTimer, bool bClusterCrumbling, const FVector2f& RemovalTimer);

		CHAOS_API const FVector4f& GetPackedData() const;

		CHAOS_API bool IsEnabled() const;
		CHAOS_API bool GetClusterCrumbling() const;
		CHAOS_API FVector2f GetBreakTimer() const;
		CHAOS_API FVector2f GetRemovalTimer() const;
		
		CHAOS_API void SetEnabled(bool bEnable);
		CHAOS_API void SetClusterCrumbling(bool bClusterCrumbling);
		CHAOS_API void SetBreakTimer(float MinTime, float MaxTime);
		CHAOS_API void SetRemovalTimer(float MinTime, float MaxTime);

	private:
		FVector4f PackedData;
	};

	/**
	 * Provide an API to set and get removal on break data on a collection
	 */
	class FCollectionRemoveOnBreakFacade
	{
	public:
		CHAOS_API FCollectionRemoveOnBreakFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FCollectionRemoveOnBreakFacade(const FManagedArrayCollection& InCollection);

		/** Create the facade attributes. */
		CHAOS_API void DefineSchema();

		/** remove the facade attributes */
		CHAOS_API void RemoveSchema();

		/** Valid if related attributes are available */
		CHAOS_API bool IsValid() const;

		/** Is this facade const access */
		CHAOS_API bool IsConst() const;

		/** set a specific value to a array of transforms */
		CHAOS_API void SetFromIndexArray(const TArray<int32>& Indices, const FRemoveOnBreakData& Data);

		/** set a specific value to all transforms */
		CHAOS_API void SetToAll(const FRemoveOnBreakData& Data);

		/** Get a specific value by index */
		CHAOS_API const FRemoveOnBreakData GetData(int32 Index) const;

	private:
		TManagedArrayAccessor<FVector4f> RemoveOnBreakAttribute;
	};
}
