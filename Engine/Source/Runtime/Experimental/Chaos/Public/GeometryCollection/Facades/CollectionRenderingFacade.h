// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "Math/MathFwd.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/Facades/CollectionSelectionFacade.h"
#include "Chaos/Triangle.h"

namespace GeometryCollection::Facades
{

	/**
	* FRenderingFacade
	*
	* Defines common API for storing rendering data.
	*
	*/
	class FRenderingFacade
	{
	public:
		typedef FGeometryCollectionSection FTriangleSection;
		typedef TMap<FString, int32> FStringIntMap;
		/**
		* FRenderingFacade Constuctor
		* @param VertexDependencyGroup : GroupName the index attribute is dependent on.
		*/
		CHAOS_API FRenderingFacade(FManagedArrayCollection& InSelf);
		CHAOS_API FRenderingFacade(const FManagedArrayCollection& InSelf);

		/**Create the facade.*/
		CHAOS_API void DefineSchema();

		/** Is the facade defined constant. */
		bool IsConst() const { return Collection==nullptr; }

		/**Is the Facade defined on the collection?*/
		CHAOS_API bool IsValid() const;

		/**Does it support rendering surfaces.*/
		CHAOS_API bool CanRenderSurface() const;

		//
		// Facade API
		//

		/**Number of triangles to render.*/
		CHAOS_API int32 NumTriangles() const;

		/**Add a point to the rendering view.*/
		CHAOS_API void AddPoint(const FVector3f& InPoint);

		/**Add point cloud to the rendering view.*/
		CHAOS_API void AddPoints(TArray<FVector3f>&& InPoints);

		/**Add a triangle to the rendering view.*/
		CHAOS_API void AddTriangle(const Chaos::FTriangle& InTriangle);

		/** Given a list of triangles, expand them to faces with face-normals */
		CHAOS_API void AddFaces(const TArray<FVector3f>& InVertices, TArray<FIntVector>& InIndices, TArray<FLinearColor>& InColors);

		/**Add a box to the rendering view.*/
		CHAOS_API void AddBox(const FBox& InBox);
		CHAOS_API void AddBox(const FVector3f& InMinVertex, const FVector3f& InMaxVertex);

		/**Add boxes to the rendering view.*/
		CHAOS_API void AddBoxes(const TArray<FBox>& InBoxes);

		/**Add a sphere to the rendering view.*/
		CHAOS_API void AddSphere(const FSphere& InSphere, const FLinearColor& InColor);
		CHAOS_API void AddSphere(const FVector3f& InCenter, const float InRadius, const FLinearColor& InColor);

		/**Add spheres to the rendering view.*/
		CHAOS_API void AddSpheres(const TArray<FSphere>& InSpheres, const FLinearColor& InColor);

		/** Add a capsule to the rendering view */
		CHAOS_API void AddCapsule(const float Length, const float Radius, FLinearColor Color = FLinearColor::White, int32 Sides = 8);

		/** Add a surface to the rendering view.*/
		CHAOS_API void AddSurface(TArray<FVector3f>&& InVertices, TArray<FIntVector>&& InIndices, TArray<FVector3f>&& InNormals, TArray<FLinearColor>&& InColors);

		/** Add a surface with UV layers, Material IDs and material paths to the rendering view.*/
		CHAOS_API void AddSurface(TArray<FVector3f>&& InVertices, TArray<FIntVector>&& InIndices, TArray<FVector3f>&& InNormals, TArray<FLinearColor>&& InColors, TArray<TArray<FVector2f>>&& InUVs, TArray<int32>&& InMaterialIDs, TArray<FString>&& MaterialPaths);

		/** Add surface bone weights and indices to the rendering view - AddSurface must have been called before .*/
		CHAOS_API void AddSurfaceBoneWeightsAndIndices(TArray<TArray<float>>&& InBoneWeights, TArray<TArray<int32>>&& InBoneIndices);

		/**Add a tetrahedron to the rendering view.*/
		CHAOS_API void AddTetrahedron(const TArray<FVector3f>& InVertices, const FIntVector4& InIndices);

		/**Add a tetrahedrons to the rendering view.*/
		CHAOS_API void AddTetrahedrons(TArray<FVector3f>&& InVertices, TArray<FIntVector4>&& InIndices);

		/** GetIndices */
		const TManagedArray< FIntVector >& GetIndices() const { return IndicesAttribute.Get(); }

		/** GetMaterialID */
		const TManagedArray< int32 >& GetMaterialID() const { return MaterialIDAttribute.Get(); }

		/** GetTriangleSections */
		const TManagedArray< FTriangleSection >& GetTriangleSections() const { return TriangleSectionAttribute.Get(); }

		/** BuildMeshSections */
		CHAOS_API TArray<FTriangleSection> BuildMeshSections(const TArray<FIntVector>& Indices, TArray<int32> BaseMeshOriginalIndicesIndex, TArray<FIntVector>& RetIndices) const;


		//
		//  Vertices
		//

		/** GetVertices */
		const TManagedArray< FVector3f >& GetVertices() const { return VertexAttribute.Get(); }
		TManagedArray< FVector3f >& ModifyVertices() { check(!IsConst()); return VertexAttribute.Modify(); }
		
		/** GetNormals */
		const TManagedArray< FVector3f >& GetNormals() const { return VertexNormalAttribute.Get(); }
		TManagedArray< FVector3f >& ModifyNormals() { check(!IsConst()); return VertexNormalAttribute.Modify(); }

		/** GetVertexSelectionAttribute */
		const TManagedArray< int32 >& GetVertexSelection() const { return VertexSelectionAttribute.Get(); }
		TManagedArray< int32 >& ModifyVertexSelection() { check(!IsConst()); return VertexSelectionAttribute.Modify(); }

		/** GetVertexToGeometryIndexAttribute */
		const TManagedArray< int32 >& GetVertexToGeometryIndex() const { return VertexToGeometryIndexAttribute.Get(); }
		TManagedArray< int32 >& ModifyVertexToGeometryIndex() { check(!IsConst()); return VertexToGeometryIndexAttribute.Modify(); }

		/** HitProxyIDAttribute */
		const TManagedArray< int32 >& GetVertexHitProxyIndex() const { return VertexHitProxyIndexAttribute.Get(); }
		TManagedArray< int32 >& ModifyVertexHitProxyIndex() { check(!IsConst()); return VertexHitProxyIndexAttribute.Modify(); }

		/** NumVertices */
		int32 NumVertices() const { return VertexAttribute.Num(); }

		/** GetVertexColorAttribute */
		const TManagedArray<FLinearColor>& GetVertexColor() const { return VertexColorAttribute.Get(); }
		TManagedArray<FLinearColor>& ModifyVertexColor() { check(!IsConst()); return VertexColorAttribute.Modify(); }

		/** GetVertexUVAttribute */
		const TManagedArray<TArray<FVector2f>>& GetVertexUV() const { return VertexUVAttribute.Get(); }
		TManagedArray<TArray<FVector2f>>& ModifyVertexUV() { check(!IsConst()); return VertexUVAttribute.Modify(); }

		// optional per vertex bone attributes
		const TManagedArray<TArray<float>>* FindBoneWeights() const { return BoneWeightsAttribute.Find(); }
		const TManagedArray<TArray<int32>>* FindBoneIndices() const { return BoneIndicesAttribute.Find(); }

		//
		// Geometry Group Attributes
		//

		/** Geometry Group Start : */
		CHAOS_API int32 StartGeometryGroup(FString InName, const FTransform& InTm = FTransform::Identity);

		/** Geometry Group End : */
		CHAOS_API void EndGeometryGroup(int32 InGeometryGroupIndex);

		/** Modify the transform of a pre-declared group */
		CHAOS_API void SetGroupTransform(int32 InGeometryGroupIndex, const FTransform& InTm);

		int32 NumGeometry() const { return GeometryNameAttribute.Num(); }

		/** GeometryNameAttribute */
		const TManagedArray< FString >& GetGeometryName() const { return GeometryNameAttribute.Get(); }

		/** GeometryTransformAttribute */
		const TManagedArray<FTransform>& GetGeometryTransform() const { return GeometryTransformAttribute.Get(); }

		/** HitProxyIDAttribute */
		const TManagedArray< int32 >& GetGeometryHitProxyIndex() const { return GeometryHitProxyIndexAttribute.Get(); }
		      TManagedArray< int32 >& ModifyGeometryHitProxyIndex() {check(!IsConst());return GeometryHitProxyIndexAttribute.Modify(); }

		/** VertexStartAttribute */
		const TManagedArray< int32 >& GetVertexStart() const { return VertexStartAttribute.Get(); }

		/** VertexCountAttribute */
		const TManagedArray< int32 >& GetVertexCount() const { return VertexCountAttribute.Get(); }

		/** IndicesStartAttribute */
		const TManagedArray< int32 >& GetIndicesStart() const { return IndicesStartAttribute.Get(); }

		/** IndicesCountAttribute */
		const TManagedArray< int32 >& GetIndicesCount() const { return IndicesCountAttribute.Get(); }

		/** SelectionState */
		const TManagedArray< int32 >& GetSelectionState() const { return GeometrySelectionAttribute.Get(); }
		      TManagedArray< int32 >& ModifySelectionState() { check(!IsConst()); return GeometrySelectionAttribute.Modify(); }

		/** NumVerticesOnSelectedGeometry */
		CHAOS_API int32 NumVerticesOnSelectedGeometry() const;

		/** GetGeometryNameToIndexMap */
		CHAOS_API FStringIntMap GetGeometryNameToIndexMap() const;

		/** GetMaterialPathAttribute */
		const TManagedArray<FString>& GetMaterialPaths() const { return MaterialPathAttribute.Get(); }

		/** GetMaterialStart */
		const TManagedArray< int32 >& GetMaterialStart() const { return MaterialStartAttribute.Get(); }

		/** GetMaterialCount */
		const TManagedArray< int32 >& GetMaterialCount() const { return MaterialCountAttribute.Get(); }

		/** Total NumMaterials */
		int32 NumMaterials() const { return MaterialPathAttribute.Num(); }

	private : 
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;

		TManagedArrayAccessor<FVector3f> VertexAttribute;
		TManagedArrayAccessor<int32> VertexToGeometryIndexAttribute;
		TManagedArrayAccessor<int32> VertexSelectionAttribute;
		TManagedArrayAccessor<int32> VertexHitProxyIndexAttribute;
		TManagedArrayAccessor<FVector3f> VertexNormalAttribute;
		TManagedArrayAccessor<FLinearColor> VertexColorAttribute;
		TManagedArrayAccessor<TArray<FVector2f>> VertexUVAttribute;

		TManagedArrayAccessor<FIntVector> IndicesAttribute;
		TManagedArrayAccessor<int32> MaterialIDAttribute;

		TManagedArrayAccessor<FTriangleSection> TriangleSectionAttribute;
		TManagedArrayAccessor<FString> MaterialPathAttribute;

		TManagedArrayAccessor<FString> GeometryNameAttribute;
		TManagedArrayAccessor<FTransform> GeometryTransformAttribute;
		TManagedArrayAccessor<int32> GeometryHitProxyIndexAttribute;
		TManagedArrayAccessor<int32> VertexStartAttribute;
		TManagedArrayAccessor<int32> VertexCountAttribute;
		TManagedArrayAccessor<int32> IndicesStartAttribute;
		TManagedArrayAccessor<int32> IndicesCountAttribute;
		TManagedArrayAccessor<int32> MaterialStartAttribute;
		TManagedArrayAccessor<int32> MaterialCountAttribute;
		TManagedArrayAccessor<int32> GeometrySelectionAttribute;

		TManagedArrayAccessor<TArray<float>> BoneWeightsAttribute;
		TManagedArrayAccessor<TArray<int32>> BoneIndicesAttribute;
	};

}
