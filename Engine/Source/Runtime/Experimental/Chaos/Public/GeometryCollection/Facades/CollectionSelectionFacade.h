// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "GeometryCollection/ManagedArrayCollection.h"

namespace GeometryCollection::Facades
{

	/**
	* FSelectionFacade
	* 
	* Defines common API for storing a sib indexing of a separate group. For example, to store
	* a subset of vertices defined in the group "VerticesGroup", use the function :
	*    Key = this->AddSelection({1,2,3,4});
	* The key can be saved and later used to query the values:
	*	TArray<int32> SavedVertices;
	*   this->GetSelection(Key, SavedVertices);
	*
	* The following groups are created on the collection based on which API is called. 
	* 
	*	<Group> = FSelectionFacade::UnboundGroup_<DependencyGroup>
	*	- FindAttribute<TArray<int32>>(FVertexSetInterface::IndexAttribute, <Group>);
	* 
	*	<Group> = FSelectionFacade::WeightedUnboundGroup_<DependencyGroup>
	*	- FindAttribute<TArray<int32>>(FVertexSetInterface::IndexAttribute, <Group>);
	*	- FindAttribute<TArray<float>>(FVertexSetInterface::WeightAttribute, <Group>);
	* 
	*	<Group> = FSelectionFacade::BoundGroup_<DependencyGroup>
	*	- FindAttribute<TArray<int32>>(FVertexSetInterface::IndexAttribute, <Group>);
	*	- FindAttribute<int32>(FVertexSetInterface::BoneIndexAttribute,<Group>);
	*
	*	<Group> = FSelectionFacade::WeightedBoundGroup_<DependencyGroup>
	*	- FindAttribute<TArray<int32>>(FVertexSetInterface::IndexAttribute, <Group>);
	* 	- FindAttribute<TArray<float>>(FVertexSetInterface::WeightAttribute, <Group>);
	*	- FindAttribute<int32>(FVertexSetInterface::BoneIndexAttribute, <Group>);
	* 
	*/
	class FSelectionFacade
 	{
		CHAOS_API void InitUnboundedGroup(FName GroupName, FName DependencyGroup);
		CHAOS_API void InitWeightedUnboundedGroup(FName GroupName, FName DependencyGroup);
		CHAOS_API void InitBoundedGroup(FName GroupName, FName DependencyGroup, FName BoneDependencyGroup);
		CHAOS_API void InitWeightedBoundedGroup(FName GroupName, FName DependencyGroup, FName BoneDependencyGroup);

	public:
		enum class EErrorCode
		{
			None = 0,
			SelectionGroupCantBeFound = 1,
			AttributeCantBeFound = 2
		};

		// groups
		static CHAOS_API const FName UnboundGroup;
		static CHAOS_API const FName WeightedUnboundGroup;
		static CHAOS_API const FName BoundGroup;
		static CHAOS_API const FName WeightedBoundGroup;

		// Attributes
		static CHAOS_API const FName IndexAttribute;
		static CHAOS_API const FName WeightAttribute;
		static CHAOS_API const FName BoneIndexAttribute;

		struct FSelectionKey {
			FSelectionKey(int32 InIndex = INDEX_NONE, FName InGroupName = "") :
				Index(InIndex), GroupName(InGroupName) {}
			int32 Index;
			FName GroupName;
		};

		/**
		* FSelectionFacade Constuctor
		*/
		CHAOS_API FSelectionFacade(FManagedArrayCollection& InSelf);
		CHAOS_API FSelectionFacade(const FManagedArrayCollection& InSelf);

		/** Is the facade defined constant. */
		bool IsConst() const { return Collection==nullptr; }

		/**
		* Add the indices to the FVertexSetInterface::UnboundGroup 
		*/
		CHAOS_API FSelectionKey AddSelection(const TArray<int32>& Indices, FName DependencyGroup);

		/**
		* Add the indices to the FVertexSetInterface::WeightedUnboundGroup
		*/
		CHAOS_API FSelectionKey AddSelection(const TArray<int32>& Indices, const TArray<float>& Weights, FName DependencyGroup);

		/**
		* Add the indices to the FVertexSetInterface::BoundGroup
		*/
		CHAOS_API FSelectionKey AddSelection(const int32 BoneIndex, const TArray<int32>& Indices, FName DependencyGroup, FName BoneDependencyGroup = FName());

		/**
		* Add the indices to the FVertexSetInterface::BoundGroup
		*/
		CHAOS_API FSelectionKey AddSelection(const int32 BoneIndex, const TArray<int32>& Indices, const TArray<float>& Weights, FName DependencyGroup, FName BoneDependencyGroup = FName());

		/**
		* Return the vertex list from the given key 
		* @param Key : <GroupName and Index>
		* @param Indices : Return indices, empty if not found. 
		*/
		CHAOS_API void GetSelection(const FSelectionKey& Key, TArray<int32>& Indices) const;

		/**
		* Return the vertex list, bone index, and weights from the given key
		* @param Key : <GroupName and Index>
		* @param BoneIndex : Return BoneIndex, INDEX_NONE if not found.
		* @param Indices : Return indices, empty if not found.
		* @param Weights : Return vertex weights, empty if not found.
		*/
		CHAOS_API void GetSelection(const FSelectionKey& Key, TArray<int32>& Indices, TArray<float>& Weights) const;

		/**
		* Return the vertex list and bone index from the given key
		* @param Key : <GroupName and Index>
		* @param BoneIndex : Return BoneIndex, INDEX_NONE if not found.
		* @param Indices : Return indices, empty if not found.
		*/
		CHAOS_API void GetSelection(const FSelectionKey& Key, int32& BoneIndex, TArray<int32>& Indices) const;

		/**
		* Return the vertex list, bone index, and weights from the given key
		* @param Key : <GroupName and Index>
		* @param BoneIndex : Return BoneIndex, INDEX_NONE if not found.
		* @param Indices : Return indices, empty if not found.
		* @param Weights : Return vertex weights, empty if not found.
		*/
		CHAOS_API void GetSelection(const FSelectionKey& Key, int32& BoneIndex, TArray<int32>& Indices, TArray<float>& Weights) const;

		CHAOS_API void AddSelectionToCollection(const FName InGroupName, const TBitArray<>& InSelectionArray, const FString InAttribute, const bool bStoreAsFloat);

		CHAOS_API EErrorCode GetSelectionFromCollection(const FName InGroupName, const FString& InAttribute, TBitArray<>& OutSelectionArray) const;

	private:
		// const collection will be a null pointer, 
		// while non-const will be valid.
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;

		void RegisterSelectionInCollection(const FName InGroupName, FString InAttribute);
	};

}
