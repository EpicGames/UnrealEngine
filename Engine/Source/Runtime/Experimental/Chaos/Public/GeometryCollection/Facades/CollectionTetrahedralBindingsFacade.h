// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace GeometryCollection::Facades
{
	/**
	* Flesh deformer behavior in c++ (rather than hlsl).
	*/
	class FleshDeformerImpl
	{
	public:
		FleshDeformerImpl()
		{}

		/** Non-branching tangent basis vectors. Discontinuity at TangentZ.z == 0 */
		static CHAOS_API Chaos::PMatrix<float, 3, 3> GetTangentBasis(const FVector3f& TangentZ);
		/** Returns an orthogonal basis from triangle vertices. */
		static CHAOS_API Chaos::PMatrix<float, 3, 3> GetOrthogonalBasisVectors(const FVector3f& PtA, const FVector3f& PtB, const FVector3f& PtC);

		/**
		* Given an \p Offset vector relative to rest triangle configuration \p RestPtA, \p RestPtB,
		* \p RestPtC, returns a rotated offset vector relative to the current triangle configuration,
		* \p CurrPtA, \p CurrPtB, CurrPtC.
		*/
		static CHAOS_API FVector3f GetRotatedOffsetVector(const FVector3f& Offset, const FVector3f& RestPtA, const FVector3f& RestPtB, const FVector3f& RestPtC, const FVector3f& CurrPtA, const FVector3f& CurrPtB, const FVector3f& CurrPtC);
		static CHAOS_API FVector3f GetRotatedOffsetVector(const FIntVector4& Parents, const FVector3f& Offset, const TManagedArray<FVector3f>& RestVertices, const TArray<Chaos::TVector<Chaos::FRealSingle, 3>>& CurrVertices);

		/**
		* Evaluate the tetrahedral bindings for \p SurfaceIndex.
		*
		* \p SurfaceIndex - the bindings index to evaluate.
		* \p ParentsArray - bindings parents buffer.
		* \p WeightsArray - bindings weights buffer.
		* \p OffsetArray - bindings offset buffer.
		* \p RestVertices - tetrahedral mesh rest points.
		* \p CurrVertices - tetrahedral mesh current points.
		*/
		static CHAOS_API FVector3f GetEmbeddedPosition(
			const int32 SurfaceIndex,
			const TManagedArrayAccessor<FIntVector4>* ParentsArray,
			const TManagedArrayAccessor<FVector4f>* WeightsArray,
			const TManagedArrayAccessor<FVector3f>* OffsetArray,
			const TManagedArray<FVector3f>& RestVertices,
			const TArray<Chaos::TVector<Chaos::FRealSingle, 3>>& CurrVertices);
	};

	/**
	* FTetrahedralBindings
	* 
	* Interface for storing and retrieving bindings of surfaces (typically SkeletalMesh or StaticMesh) to
	* tetrahedral meshes.  Bindings data for each surface is grouped by a mesh id and a level of detail.
	*/
	class FTetrahedralBindings
 	{
	public:
		// groups
		static CHAOS_API const FName MeshBindingsGroupName;

		//
		// Attributes
		//

		static CHAOS_API const FName MeshIdAttributeName;

		//! Tet or Tri vertex indices.
		static CHAOS_API const FName ParentsAttributeName;
		//! Barycentric weight of each tet/tri vertex.
		static CHAOS_API const FName WeightsAttributeName;
		//! Offset vector from barycentric tri position.
		static CHAOS_API const FName OffsetsAttributeName;
		//! Per vertex amount for deformer masking.
		static CHAOS_API const FName MaskAttributeName;

		// Dependency
		static const FName TetrahedralGroupDependency;

		/**
		* FSelectionFacade Constuctor
		* @param VertixDependencyGroup : GroupName the index attribute is dependent on. 
		*/
		CHAOS_API FTetrahedralBindings(FManagedArrayCollection& InSelf);
		CHAOS_API FTetrahedralBindings(const FManagedArrayCollection& InSelf);
		CHAOS_API virtual ~FTetrahedralBindings();

		/** 
		* Create the facade schema. 
		*/
		CHAOS_API void DefineSchema();

		/** Returns \c true if the facade is operating on a read-only geometry collection. */
		bool IsConst() const { return MeshIdAttribute.IsConst(); }

		/** 
		* Returns \c true if the Facade defined on the collection, and is initialized to
		* a valid bindings group.
		*/
		CHAOS_API bool IsValid() const;

		/**
		* Given a \p MeshId (by convention \code Mesh->GetPrimaryAssetId() \endcode) and
		* a Level Of Detail rank, generate the associated bindings group name.
		*/
		static CHAOS_API FName GenerateMeshGroupName(const int32 TetMeshIdx, const FName& MeshId, const int32 LOD);

		/**
		* For a given \p MeshId and \p LOD, return the associated tetrahedral mesh index.
		*/
		CHAOS_API int32 GetTetMeshIndex(const FName& MeshId, const int32 LOD) const;

		/**
		* Returns \c true if the specified bindings group exists.
		*/
		CHAOS_API bool ContainsBindingsGroup(const int32 TetMeshIdx, const FName& MeshId, const int32 LOD) const;
		CHAOS_API bool ContainsBindingsGroup(const FName& GroupName) const;

		/**
		* Create a new bindings group, allocating new arrays.
		*/
		CHAOS_API void AddBindingsGroup(const int32 TetMeshIdx, const FName& MeshId, const int32 LOD);
		CHAOS_API void AddBindingsGroup(const FName& GroupName);
		
		/**
		* Initialize local arrays to point at a bindings group associated with \p MeshId 
		* and \p LOD.  Returns \c false if it doesn't exist.
		*/
		CHAOS_API bool ReadBindingsGroup(const int32 TetMeshIdx, const FName& MeshId, const int32 LOD);
		CHAOS_API bool ReadBindingsGroup(const FName& GroupName);

		/**
		* Removes a group from the list of bindings groups, removes the bindings arrays 
		* from the geometry collection, and removes the group if it's empty.
		*/
		CHAOS_API void RemoveBindingsGroup(const int32 TetMeshIdx, const FName& MeshId, const int32 LOD);
		CHAOS_API void RemoveBindingsGroup(const FName& GroupName);

		/**
		* Authors bindings data.
		* 
		* \p Parents are indicies of vertices; tetrahedron or surface triangle where final 
		*    elem is \c INDEX_NONE.
		* \p Weights are barycentric coordinates.
		* \p Offsets are vectors from the barycentric point to the location, in the case 
		*    of a surface binding.
		* \p Mask are per-vertex multipliers on the deformer, 0 for no deformation, 1.0 for 
		*    full deformation.
		*/
		CHAOS_API void SetBindingsData(const TArray<FIntVector4>& ParentsIn, const TArray<FVector4f>& WeightsIn, const TArray<FVector3f>& OffsetsIn, const TArray<float>& MaskIn);
		void SetBindingsData(const TArray<FIntVector4>& ParentsIn, const TArray<FVector4f>& WeightsIn, const TArray<FVector3f>& OffsetsIn)
		{
			TArray<float> MaskTmp; MaskTmp.SetNum(ParentsIn.Num());
			for (int32 i = 0; i < ParentsIn.Num(); i++)
			{
				MaskTmp[i] = 1.0;
			}
			SetBindingsData(ParentsIn, WeightsIn, OffsetsIn, MaskTmp);
		}

		/**
		* Get Parents array.
		*/
		const TManagedArrayAccessor<FIntVector4>* GetParentsRO() const { return Parents.Get(); }
		      TManagedArrayAccessor<FIntVector4>* GetParents() { check(!IsConst()); return Parents.Get(); }
		/**
		* Get Weights array.
		*/
		const TManagedArrayAccessor<FVector4f>* GetWeightsRO() const { return Weights.Get(); }
		      TManagedArrayAccessor<FVector4f>* GetWeights() { check(!IsConst()); return Weights.Get(); }
		/**
		* Get Offsets array.
		*/
		const TManagedArrayAccessor<FVector3f>* GetOffsetsRO() const { return Offsets.Get(); }
		      TManagedArrayAccessor<FVector3f>* GetOffsets() { check(!IsConst());  return Offsets.Get(); }
		/**
		* Get Mask array.
		*/
		const TManagedArrayAccessor<float>* GetMaskRO() const { return Masks.Get(); }
		      TManagedArrayAccessor<float>* GetMask() { check(!IsConst()); return Masks.Get(); }

		/**
		* Bindings evaluator.
		* 
		* Given binding parents, weights, offsets, and current tetrahedral mesh positions, get the
		* current bound position.
		*/
		class Evaluator
		{
		public:
			Evaluator(
				const TManagedArrayAccessor<FIntVector4>* Parents, 
				const TManagedArrayAccessor<FVector4f>* Weights, 
				const TManagedArrayAccessor<FVector3f>* Offsets,
				const TManagedArray<FVector3f>* RestVertices)
				: MinIndexValue(TNumericLimits<int32>::Max())
				, MaxIndexValue(INDEX_NONE)
				, ParentsArray(Parents)
				, WeightsArray(Weights)
				, OffsetsArray(Offsets)
				, RestVerticesArray(RestVertices)
			{}

			bool IsValid() const
			{ 
				return ParentsArray && WeightsArray && OffsetsArray && 
					(ParentsArray->Get().Num()==WeightsArray->Get().Num() && ParentsArray->Get().Num()==OffsetsArray->Get().Num()) &&
					(RestVerticesArray->IsValidIndex(MinIndex()) && RestVerticesArray->IsValidIndex(MaxIndex())); 
			}
			int32 NumVertices() const { return ParentsArray->Num(); }
			int32 MinIndex() const
			{
				if (MinIndexValue == TNumericLimits<int32>::Max())
				{
					for (const FIntVector4& Tet : ParentsArray->Get())
					{
						const int32 IndexEnd = Tet[3] == INDEX_NONE ? 3 : 4;
						for (int32 LocalIdx = 0; LocalIdx < IndexEnd; ++LocalIdx)
						{
							MinIndexValue = FMath::Min(MinIndexValue, Tet[LocalIdx]);
						}
					}
				}
				return MinIndexValue;
			}
			int32 MinIndexPosition() const
			{
				MinIndexValue = TNumericLimits<int32>::Max();
				int32 OutMinIndexPosition = INDEX_NONE;
				for (int32 TetIdx = 0; TetIdx < ParentsArray->Get().Num(); ++TetIdx)
				{
					const FIntVector4& Tet = ParentsArray->Get()[TetIdx];
					const int32 IndexEnd = Tet[3] == INDEX_NONE ? 3 : 4;
					for (int32 LocalIdx = 0; LocalIdx < IndexEnd; ++LocalIdx)
					{
						if (Tet[LocalIdx] < MinIndexValue)
						{
							MinIndexValue = FMath::Min(MinIndexValue, Tet[LocalIdx]);
							OutMinIndexPosition = TetIdx;
						}
					}
				}
				return OutMinIndexPosition;
			}
			int32 MaxIndex() const
			{
				if (MaxIndexValue == INDEX_NONE)
				{
					for (const FIntVector4& Tet : ParentsArray->Get())
					{
						const int32 IndexEnd = Tet[3] == INDEX_NONE ? 3 : 4;
						for (int32 LocalIdx = 0; LocalIdx < IndexEnd; ++LocalIdx)
						{
							MaxIndexValue = FMath::Max(MaxIndexValue, Tet[LocalIdx]);
						}
					}
				}
				return MaxIndexValue;
			}
			FVector3f GetEmbeddedPosition(const int32 Index, const TArray<Chaos::TVector<Chaos::FRealSingle, 3>>& CurrVertices) const
			{
				return FleshDeformerImpl::GetEmbeddedPosition(Index, ParentsArray, WeightsArray, OffsetsArray, *RestVerticesArray, CurrVertices);
			}

		private:
			mutable int32 MinIndexValue;
			mutable int32 MaxIndexValue;
			const TManagedArrayAccessor<FIntVector4>* ParentsArray;
			const TManagedArrayAccessor<FVector4f>* WeightsArray;
			const TManagedArrayAccessor<FVector3f>* OffsetsArray;
			const TManagedArray<FVector3f>* RestVerticesArray;
		};

		/**
		* Masked bindings evaluator.
		* 
		* Given binding parents, weights, offsets, mask weights, and current tetrahedral mesh positions, get 
		* the current bound position blended with the rig position.
		*/
		class MaskedEvaluator
		{
		public:
			MaskedEvaluator(
				const TManagedArrayAccessor<FIntVector4>* Parents, 
				const TManagedArrayAccessor<FVector4f>* Weights, 
				const TManagedArrayAccessor<FVector3f>* Offsets, 
				const TManagedArrayAccessor<float>* Masks,
				const TManagedArray<FVector3f>* RestVertices)
				: UnmaskedEval(Parents, Weights, Offsets, RestVertices)
				, ParentsArray(Parents)
				, MasksArray(Masks)
			{}

			bool IsValid() const { return UnmaskedEval.IsValid() && (ParentsArray->Get().Num() == MasksArray->Get().Num()); }
			int32 NumVertices() const { return UnmaskedEval.NumVertices(); }
			int32 MinIndex() const { return UnmaskedEval.MinIndex(); }
			int32 MaxIndex() const { return UnmaskedEval.MaxIndex(); }
			FVector3f GetEmbeddedPosition(const int32 Index, const FVector3f& RigPosition, const TArray<Chaos::TVector<Chaos::FRealSingle, 3>>& CurrVertices) const
			{
				const float Mask = MasksArray->Get()[Index];
				if (Mask < 1.0e-6)
				{
					return RigPosition;
				}
				else if (Mask > 1.0 - 1.0e-6)
				{
					return UnmaskedEval.GetEmbeddedPosition(Index, CurrVertices);
				}
				else
				{
					FVector3f EmbeddedPos = UnmaskedEval.GetEmbeddedPosition(Index, CurrVertices);
					return (1.0 - Mask) * RigPosition + Mask * EmbeddedPos;
				}
			}
		private:
			Evaluator UnmaskedEval;
			const TManagedArrayAccessor<FIntVector4>* ParentsArray;
			const TManagedArrayAccessor<float>* MasksArray;
		};

		/**
		* Initialize an \c Evaluator that samples bindings and computes resulting positions.
		*/
		TUniquePtr<Evaluator> InitEvaluator(const TManagedArray<FVector3f>* RestVertices) const
		{
			return TUniquePtr<Evaluator>(new Evaluator(Parents.Get(), Weights.Get(), Offsets.Get(), RestVertices));
		}

		/**
		* Initialize a \c MaskedEvaluator that samples bindings and computes resulting positions,
		* masked and blended by a rig evaluated position.
		*/
		TUniquePtr<MaskedEvaluator> InitMaskedEvaluator(const TManagedArray<FVector3f>* RestVertices) const
		{ return TUniquePtr<MaskedEvaluator>(new MaskedEvaluator(Parents.Get(), Weights.Get(), Offsets.Get(), Masks.Get(), RestVertices)); }

	private:
		TManagedArrayAccessor<FString> MeshIdAttribute;

		TUniquePtr<TManagedArrayAccessor<FIntVector4>> Parents;
		TUniquePtr<TManagedArrayAccessor<FVector4f>> Weights;
		TUniquePtr<TManagedArrayAccessor<FVector3f>> Offsets;
		TUniquePtr<TManagedArrayAccessor<float>> Masks;
	};

}
