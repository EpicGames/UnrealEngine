// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/PBDSoftsEvolutionFwd.h"
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/Facades/CollectionPositionTargetFacade.h"

namespace GeometryCollection::Facades
{
	struct TetrahedralParticleEmbedding
	{
		TetrahedralParticleEmbedding(
			int32 InParticleIndex = INDEX_NONE, 
			int32 InGeometryIndex = INDEX_NONE,
			int32 InTetrahedronIndex = INDEX_NONE,
			TArray<float> InBarycentricWeights = TArray<float>() )
			: ParticleIndex(InParticleIndex)
			, GeometryIndex(InGeometryIndex)
			, TetrahedronIndex(InTetrahedronIndex)
			, BarycentricWeights(InBarycentricWeights) {}

		int32 ParticleIndex;
		int32 GeometryIndex;
		int32 TetrahedronIndex;
		TArray<float> BarycentricWeights;
	};

	/**
	* FTetrahedralFacade
	*
	*/
	class FTetrahedralFacade
	{
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;

	public:

		/**
		* FSelectionFacade Constuctor
		* @param VertixDependencyGroup : GroupName the index attribute is dependent on.
		*/
		CHAOS_API FTetrahedralFacade(FManagedArrayCollection& InSelf);
		CHAOS_API FTetrahedralFacade(const FManagedArrayCollection& InSelf);
		CHAOS_API virtual ~FTetrahedralFacade();

		/**
		* Create the facade schema.
		*/
		CHAOS_API void DefineSchema();

		/** Returns \c true if the facade is operating on a read-only geometry collection. */
		bool IsConst() const { return Collection==nullptr; }

		/**
		* Returns \c true if the Facade defined on the collection, and is initialized to
		* a valid bindings group.
		*/
		CHAOS_API bool IsValid() const;

		/**
		*  Barycentric intersections with Tetarhedron
		*/
		CHAOS_API bool Intersection(
			const TConstArrayView<Chaos::Softs::FSolverVec3>& SamplePositions, 
			const TConstArrayView<Chaos::Softs::FSolverVec3>& TetarhedronPositions,
			TArray<TetrahedralParticleEmbedding>& OutIntersections) const;

		//
		// Attributes
		//

		const TManagedArrayAccessor<FIntVector4> Tetrahedron;
		const TManagedArrayAccessor<int32> TetrahedronStart;
		const TManagedArrayAccessor<int32> TetrahedronCount;
		const TManagedArrayAccessor<int32> VertexStart;
		const TManagedArrayAccessor<int32> VertexCount;
		const TManagedArrayAccessor<FVector3f> Vertex;
	};


}
