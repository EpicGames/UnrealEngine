// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"


namespace GeometryCollection::Facades
{
	/**
	*
	*/
	class FTetrahedralMetrics
	{
	public:
		// Groups

		// Attributes
		static CHAOS_API const FName SignedVolumeAttributeName;
		static CHAOS_API const FName AspectRatioAttributeName;

		CHAOS_API FTetrahedralMetrics(FManagedArrayCollection& InCollection);
		CHAOS_API FTetrahedralMetrics(const FManagedArrayCollection& InCollection);
		CHAOS_API virtual ~FTetrahedralMetrics();

		CHAOS_API void DefineSchema();
		bool IsConst() const { return SignedVolumeAttribute.IsConst(); }
		CHAOS_API bool IsValid() const;

		const TManagedArrayAccessor<float>& GetSignedVolumeRO() const { return SignedVolumeAttribute; }
		TManagedArrayAccessor<float>& GetSignedVolume() { check(!IsConst()); return SignedVolumeAttribute; }

		const TManagedArrayAccessor<float>& GetAspectRatioRO() const { return AspectRatioAttribute; }
		TManagedArrayAccessor<float>& GetAspectRatio() { check(!IsConst()); return AspectRatioAttribute; }

	private:
		TManagedArrayAccessor<float> SignedVolumeAttribute;
		TManagedArrayAccessor<float> AspectRatioAttribute;
	};

} // namespace GeometryCollection::Facades
