// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace GeometryCollection::Facades
{

	/**
	* FTetrahedralSkeletalBindings
	* 
	* Interface for storing and retrieving bindings of transforms of a SkeletalMesh to
	* tetrahedral meshes. All bindings are stored in a single group, and a lookup table
	* is aviable to match bindings to specifi meshs by name. 
	*/
	class FTetrahedralSkeletalBindings
 	{
	public:
		// Groups 
		static CHAOS_API const FName MeshBindingsGroupName;
		static CHAOS_API const FName MeshBindingsIdGroupName;

		// Attributes
		static CHAOS_API const FName MeshIdAttributeName;
		static CHAOS_API const FName MeshIndexAttributeName;
		static CHAOS_API const FName TetrahedronIndexAttributeName;
		static CHAOS_API const FName WeightsAttributeName;
		static CHAOS_API const FName SkeletalIndexAttributeName;

		/**
		* FSelectionFacade Constuctor
		* @param VertixDependencyGroup : GroupName the index attribute is dependent on. 
		*/
		CHAOS_API FTetrahedralSkeletalBindings(FManagedArrayCollection& InSelf);
		CHAOS_API FTetrahedralSkeletalBindings(const FManagedArrayCollection& InSelf);
		CHAOS_API virtual ~FTetrahedralSkeletalBindings();

		/** 
		* Create the facade schema. 
		*/
		CHAOS_API void DefineSchema();

		/** Returns \c true if the facade is operating on a read-only geometry collection. */
		bool IsConst() const { return MeshIdAttribute.IsConst(); }

		/** 
		* Returns \c true if the Facade defined on the collection, and is initialized to
		* a valid bindings group.
		*/
		CHAOS_API bool IsValid() const;

		//
		// API
		//
		static CHAOS_API FString GenerateMeshGroupName(const int32 TetMeshIdx, const FName& SkeletalMeshName, const int32 LOD = INDEX_NONE);
		CHAOS_API void SetBindings(const FString& InMeshGroupIndex, const TArray<int32>& InTetrahedronIndex, const TArray<FVector4f>& WeightsIn, const TArray<int32>& InSkeletalIndex);
		
		/** Returns \c false if bindings data was not found, or there was another error; \c true otherwise. */
		CHAOS_API bool CalculateBindings(const FString& InKey, const TArray<FVector3f>& InVertices, TArray<FVector>& OutPosition, TArray<bool>* OutInfluence = nullptr) const;
		
	private:
		TManagedArrayAccessor<FString> MeshIdAttribute;

		TManagedArrayAccessor<int32> MeshGroupIndexAttribute;
		TManagedArrayAccessor<int32> TetrahedronIndexAttribute;
		TManagedArrayAccessor<FVector4f> WeightsAttribute;
		TManagedArrayAccessor<int32> SkeletalIndexAttribute;

		TManagedArrayAccessor<FIntVector4> TetrahedronAttribute;
		TManagedArrayAccessor<FVector3f> VerticesAttribute;

	};

}
