// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"

namespace GeometryCollection::Facades
{
	/**
	 * Provides an API to read and manipulate hierarchy in a managed array collection
	 */
	class FCollectionTransformFacade
	{
	public:
		CHAOS_API FCollectionTransformFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FCollectionTransformFacade(const FManagedArrayCollection& InCollection);

		/** Creates the facade attributes. */
		void DefineSchema() {}

		/** Valid if parent and children arrays are available */
		CHAOS_API bool IsValid() const;

		/** Is the facade defined constant. */
		bool IsConst() const { return ParentAttribute.IsConst(); }

		/** Get the number of transforms */
		CHAOS_API int32 Num() const;


		/** Gets the root index */
		CHAOS_API TArray<int32> GetRootIndices() const;

		/** Gets the main root transform */
		CHAOS_API FTransform GetRootTransform() const;

		/**
		* Returns the parent indices from the collection. Null if not initialized.
		*/
		const TManagedArray< int32 >* GetParents() const { return ParentAttribute.Find(); }

		/**
		* Returns the child indicesfrom the collection. Null if not initialized.
		*/
		const TManagedArray< TSet<int32> >* FindChildren() const { return ChildrenAttribute.Find(); }

		/**
		* Returns the child indicesfrom the collection. Null if not initialized.
		*/
		const TManagedArray<FTransform3f>* FindTransforms() const { return TransformAttribute.Find(); }

		/**
		* Returns the bone names from the collection. Null if not initialized.
		*/
		const TManagedArray<FString>* FindBoneNames() const { return BoneNameAttribute.Find(); }

		/**
		* Returns array of transforms for transforming from bone space to collection space
		* Vertex(inCollectionSpace) = TransformComputed.TransformPosition(Vertex(inBoneSpace))
		*/
		CHAOS_API TArray<FTransform> ComputeCollectionSpaceTransforms() const;

		/**
		* Returns the transform for transforming from bone space to collection space for specified bone
		* Vertex(inCollectionSpace) = TransformComputed.TransformPosition(Vertex(inBoneSpace))
		*/
		CHAOS_API FTransform ComputeCollectionSpaceTransform(int32 BoneIdx) const;

		/** Transforms the pivot of a collection */
		CHAOS_API void SetPivot(const FTransform& InTransform);

		/** Transforms collection */
		CHAOS_API void Transform(const FTransform& InTransform);

		/** Transforms selected bones in the collection */
		CHAOS_API void Transform(const FTransform& InTransform, const TArray<int32>& InSelection);

		/** Check if the facade has the bone name attribute. */
		CHAOS_API bool HasBoneNameAttribute() const;

		/** Get a bone name from the index if the facade has the attribute defined. */
		CHAOS_API FString BoneName(int32 Index) const;

		/** Get a TMap from bone name to bone index if the facade has the attribute defined. */
		CHAOS_API TMap<FString, int32> BoneNameIndexMap() const;

		/** Builds a FMatrix from all the components */
		static CHAOS_API FMatrix BuildMatrix(const FVector& Translate,
			const uint8 RotationOrder,
			const FVector& Rotate,
			const FVector& Scale,
			const FVector& Shear,
			const float UniformScale,
			const FVector& RotatePivot,
			const FVector& ScalePivot,
			const bool InvertTransformation);

		/** Builds a FTransform from all the components */
		static CHAOS_API FTransform BuildTransform(const FVector& Translate,
			const uint8 RotationOrder,
			const FVector& Rotate,
			const FVector& Scale,
			const float UniformScale,
			const FVector& RotatePivot,
			const FVector& ScalePivot,
			const bool InvertTransformation);

		/** Sets the selected bone's transform to identity */
		CHAOS_API void SetBoneTransformToIdentity(int32 BoneIdx);

		/** Does the transform heirarchy have a cycle*/
		static CHAOS_API bool HasCycle(const TManagedArray<int32>& Parents, int32 Node);

		/** Does the transform heirarchy have a cycle*/
		static CHAOS_API bool HasCycle(const TManagedArray<int32>& Parents, const TArray<int32>& SelectedBones);

		/** Parent a single transform */
		CHAOS_API void ParentTransform(const int32 TransformIndex, const int32 ChildIndex);

		/**  Parent the list of transforms to the selected index. */
		CHAOS_API void ParentTransforms(const int32 TransformIndex, const TArray<int32>& SelectedBones);

		/**  Unparent the child index from its parent */
		CHAOS_API void UnparentTransform(const int32 ChildIndex);

		/** Adds a Identity transform and nests all roots under the new transform */
		CHAOS_API void EnforceSingleRoot(const FString & RootName);

		/** Computes the Min/Max values of the Level attribute */
		CHAOS_API void ComputeLevelsBounds(float& LevelsMin, float& LevelsMax);

	private:
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;

		TManagedArrayAccessor<int32>		ParentAttribute;
		TManagedArrayAccessor<TSet<int32>>	ChildrenAttribute;
		TManagedArrayAccessor<FTransform3f>	TransformAttribute;
		TManagedArrayAccessor<FString>	    BoneNameAttribute;
	};
}
