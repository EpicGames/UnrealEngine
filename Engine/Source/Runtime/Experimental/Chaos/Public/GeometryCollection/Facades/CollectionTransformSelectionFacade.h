// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "Math/MathFwd.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/Facades/CollectionSelectionFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Chaos/Triangle.h"

class FGeometryCollection;

namespace GeometryCollection::Facades
{

	/**
	* FCollectionTransformSelectionFacade
	* 
	* Defines common API for selecting transforms on a collection
	* 
	*/
	class FCollectionTransformSelectionFacade
	{
	public:

		/**
		* FCollectionTransformSelectionFacade Constuctor
		* @param FManagedArrayCollection : Collection input
		*/
		CHAOS_API FCollectionTransformSelectionFacade(const FManagedArrayCollection& InSelf);

		/** Create the facade attributes. */
		CHAOS_API void DefineSchema();

		/** Is the facade defined constant. */
		bool IsConst() const { return ParentAttribute.IsConst(); }

		/** Is the Facade defined on the collection? */
		CHAOS_API bool IsValid() const;		

		/**  */
		bool IsValidBone(const int32 Index) const { return Index >= 0 && Index < ParentAttribute.Num(); }

		/**  */
		CHAOS_API bool IsARootBone(const int32 Index) const;

		/**  */
		CHAOS_API bool HasSelectedAncestor(const TArray<int32>& InSelection, const int32 Index) const;

		/** Get bones filtered by the level as they would be in the editor; i.e., the bones with level <= the target Level, but with some additional logic for filtering, etc */
		CHAOS_API TArray<int32> GetBonesByLevel(const int32 Level, bool bOnlyClusteredOrRigid, bool bSkipFiltered) const;

		/** Get bones exactly at a target level. A target level of -1 will return all bones. */
		CHAOS_API TArray<int32> GetBonesExactlyAtLevel(const int32 TargetLevel, bool bOnlyClusteredOrRigid) const;

		/**  */
		CHAOS_API void Sanitize(TArray<int32>& InSelection, bool bFavorParents = true) const;

		/**  */
		CHAOS_API void RemoveRootNodes(TArray<int32>& InOutSelection) const;

		/**  */
		CHAOS_API void ConvertSelectionToRigidNodes(const int32 Index, TArray<int32>& InSelection) const;
		CHAOS_API void ConvertSelectionToRigidNodes(TArray<int32>& InSelection) const;

		/** Keep only the bones in the selection that match the input SimulationType */
		CHAOS_API void FilterSelectionBySimulationType(TArray<int32>& InOutSelection, FGeometryCollection::ESimulationTypes KeepSimulationType) const;

		/** Replace any Embedded or Rigid transforms in InOutSelection with their parent cluster. If there is just a rigid root node selected, optionally keep it if bLeaveRigidRoot is true. */
		CHAOS_API void ConvertSelectionToClusterNodes(TArray<int32>& InOutSelection, bool bLeaveRigidRoots = true) const;

		CHAOS_API void ConvertEmbeddedSelectionToParents(TArray<int32>& InOutSelection) const;

		/**  */
		bool CanSelectRootBones() const { return ParentAttribute.IsValid(); }
		CHAOS_API TArray<int32> SelectRootBones() const;

		/**  */
		CHAOS_API TArray<int32> SelectNone() const;

		/**  */
		bool CanSelectAll() const { return ParentAttribute.IsValid(); }
		CHAOS_API TArray<int32> SelectAll() const;

		/**  */
		bool CanSelectInverse() const { return ParentAttribute.IsValid(); }
		CHAOS_API void SelectInverse(TArray<int32>& InOutSelection) const;

		/**  */
		bool CanSelectRandom() const { return ParentAttribute.IsValid(); }
		CHAOS_API TArray<int32> SelectRandom(bool bDeterministic, int32 RandomSeed, float RandomThresholdVal) const;
		CHAOS_API TArray<int32> SelectRandom(const int32 NumElems, const bool bDeterministic, const int32 RandomSeed, const float RandomThresholdVal) const;

		/**  */
		bool CanSelectLeaf() const { return SimulationTypeAttribute.IsValid(); }
		CHAOS_API TArray<int32> SelectLeaf() const;

		/**  */
		bool CanSelectCluster() const { return SimulationTypeAttribute.IsValid(); }
		CHAOS_API TArray<int32> SelectCluster() const;

		/**  */
		bool CanSelectContact() const { return ParentAttribute.IsValid() && LevelAttribute.IsValid() && SimulationTypeAttribute.IsValid(); }
		CHAOS_API void SelectContact(TArray<int32>& InOutSelection, bool bIncludeNeighborsInParentLevels = true) const;

		/**  */
		bool CanSelectParent() const { return ParentAttribute.IsValid(); }
		CHAOS_API void SelectParent(TArray<int32>& InOutSelection) const;

		/**  */
		bool CanSelectChildren() const { return ChildrenAttribute.IsValid(); }
		CHAOS_API void SelectChildren(TArray<int32>& InOutSelection) const;

		/**  */
		bool CanSelectSiblings() const { return ParentAttribute.IsValid() && ChildrenAttribute.IsValid(); }
		CHAOS_API void SelectSiblings(TArray<int32>& InOutSelection) const;

		/**  */
		bool CanSelectLevel() const { return ParentAttribute.IsValid() && LevelAttribute.IsValid(); }
		CHAOS_API void SelectLevel(TArray<int32>& InOutSelection) const;

		/**  */
		static CHAOS_API void SelectByPercentage(TArray<int32>& InOutSelection, int32 Percentage, bool Deterministic, int32 RandomSeed);

		/** Select transforms by their Size or Relative Size */
		CHAOS_API TArray<int32> SelectBySize(float SizeMin, float SizeMax, bool bInclusive, bool bInsideRange, bool bUseRelativeSize = true) const;

		/**  */
		CHAOS_API TArray<int32> SelectByVolume(float VolumeMin, float VolumeMax, bool bInclusive, bool bInsideRange) const;

		/** Return a map from parent of bone to the selected bones with that parent */
		CHAOS_API TMap<int32, TArray<int32>> GetClusteredSelections(const TArray<int32>& InSelection) const;

		/**
		* Fill an array of bool that contains whether or not a vertex is contained in a transformed box
		* The transformed box must be in collection space
		*/
		CHAOS_API void SelectVerticesInTransformedBox(const FBox& InBox, const FTransform& InBoxTransform, TArray<bool>& OutVertexSelection) const;
		CHAOS_API void SelectVerticesInTransformedBox(const FBox& InBox, const FTransform& InBoxTransform, TBitArray<>& OutVertexSelection) const;

		/**
		* Fill an array of bool that contains whether or not a vertex is contained in a sphere
		* The sphere must be in collection space
		*/
		CHAOS_API void SelectVerticesInSphere(const FSphere& InSphere, TArray<bool>& OutVertexSelection) const;
		CHAOS_API void SelectVerticesInSphere(const FSphere& InSphere, TBitArray<>& OutVertexSelection) const;

		/**
		* Fill an array of bool that contains whether or not a vertex is on a specific side of a plane 
		* The plane must be defined in collection space
		*/
		CHAOS_API void SelectVerticesOnPlaneSide(const FPlane& InPlane, bool bUsePositiveSide, TArray<bool>& OutVertexSelection) const;
		CHAOS_API void SelectVerticesOnPlaneSide(const FPlane& InPlane, bool bUsePositiveSide, TBitArray<>& OutVertexSelection) const;

		/** 
		* Returns indices of bones where all the vertices or at least one vertex of the bone is inside InBox 
		*/
		bool CanSelectVerticesInBox() const { return ParentAttribute.IsValid(); }
		CHAOS_API TArray<int32> SelectVerticesInBox(const FBox& InBox, const FTransform& InBoxTransform, bool bAllVerticesInBox) const;

		/**
		* Returns indices of bones where centroid of the bone is inside InBox
		*/
		bool CanSelectCentroidInBox() const { return ParentAttribute.IsValid(); }
		CHAOS_API TArray<int32> SelectCentroidInBox(const FBox& InBox, const FTransform& InBoxTransform) const;

		/**
		* Returns indices of bones where BoundingBox of the bone is inside InBox
		*/
		bool CanSelectBoundingBoxInBox() const { return ParentAttribute.IsValid(); }
		CHAOS_API TArray<int32> SelectBoundingBoxInBox(const FBox& InBox, const FTransform& InBoxTransform) const;

		/**
		* Returns indices of bones where all the vertices or at least one vertex of the bone is inside a sphere
		*/
		bool CanSelectVerticesInSphere() const { return ParentAttribute.IsValid(); }
		CHAOS_API TArray<int32> SelectVerticesInSphere(const FSphere& InSphere, const FTransform& InSphereTransform, bool bAllVerticesInSphere) const;

		/**
		* Returns indices of bones where BoundingBox of the bone is inside a sphere
		*/
		bool CanSelectBoundingBoxInSphere() const { return ParentAttribute.IsValid(); }
		CHAOS_API TArray<int32> SelectBoundingBoxInSphere(const FSphere& InSphere, const FTransform& InSphereTransform) const;

		/**
		* Returns indices of bones where centroid of the bone is inside a sphere
		*/
		bool CanSelectCentroidInSphere() const { return ParentAttribute.IsValid(); }
		CHAOS_API TArray<int32> SelectCentroidInSphere(const FSphere& InSphere, const FTransform& InSphereTransform) const;

		/** 
		* Returns indices of bones where centroid of the bone is inside a convex hull
		*/
//		TArray<int32> SelectCentroidInConvex(const SOMECONVEXTYPE& InConvex, const FTransform& InTransform, const FTransform& InTransform) const;

		/**
		* Select by a float attribute in the collection, it can be any existing float attribute specified by GroupName/AttrName
		*/
		CHAOS_API TArray<int32> SelectByFloatAttribute(FString GroupName, FString AttrName, float Min, float Max, bool bInclusive, bool bInsideRange) const;

		/**
		* Select by an int attribute in the collection, it can be any existing int attribute specified by GroupName/AttrName 
		*/
		CHAOS_API TArray<int32> SelectByIntAttribute(FString GroupName, FString AttrName, int32 Min, int32 Max, bool bInclusive, bool bInsideRange) const;

		/**
		* Convert a VertexSelection to a TransformSelection, if bAllElementsMustBeSelected is true then all vertices of a transform must be selected for the transform to be selected 
		*/
		CHAOS_API TArray<int32> ConvertVertexSelectionToTransformSelection(const TArray<int32>& InVertexSelection, bool bAllElementsMustBeSelected) const;

		/**
		* Convert a FaceSelection to a TransformSelection, if bAllElementsMustBeSelected is true then all faces of a transform must be selected for the transform to be selected
		*/
		CHAOS_API TArray<int32> ConvertFaceSelectionToTransformSelection(const TArray<int32>& InFaceSelection, bool bAllElementsMustBeSelected) const;

		/**
		* Convert a VertexSelection to a FaceSelection, if bAllElementsMustBeSelected is true then all vertices of a face must be selectod for the face to be selected 
		*/
		CHAOS_API TArray<int32> ConvertVertexSelectionToFaceSelection(const TArray<int32>& InVertexSelection, bool bAllElementsMustBeSelected) const;

		/**
		* Convert a TransformSelection to a FaceSelection 
		*/
		CHAOS_API TArray<int32> ConvertTransformSelectionToFaceSelection(const TArray<int32>& InTransformSelection) const;

		/**
		* Convert a FaceSelection to a VertexSelection 
		*/
		CHAOS_API TArray<int32> ConvertFaceSelectionToVertexSelection(const TArray<int32>& InFaceSelection) const;

		/**
		* Convert a TransformSelection to a VertexSelection 
		*/
		CHAOS_API TArray<int32> ConvertTransformSelectionToVertexSelection(const TArray<int32>& InTransformSelection) const;

		/**
		* Convert a TransformSelection to a GeometrySelection
		*/
		CHAOS_API TArray<int32> ConvertTransformSelectionToGeometrySelection(const TArray<int32>& InTransformSelection) const;

		/**
		* Convert a GeometrySelection to a TransformSelection
		*/
		CHAOS_API TArray<int32> ConvertGeometrySelectionToTransformSelection(const TArray<int32>& InGeometrySelection) const;

		/**
		* Convert a GeometrySelection to a FaceSelection
		*/
		CHAOS_API TArray<int32> ConvertGeometrySelectionToFaceSelection(const TArray<int32>& InGeometrySelection) const;

		/**
		* Convert a GeometrySelection to a VertexSelection
		*/
		CHAOS_API TArray<int32> ConvertGeometrySelectionToVertexSelection(const TArray<int32>& InGeometrySelection) const;
		
		/**
		* Convert a FaceSelection to a GeometrySelection
		*/
		CHAOS_API TArray<int32> ConvertFaceSelectionToGeometrySelection(const TArray<int32>& InFaceSelection, bool bAllElementsMustBeSelected) const;

		/**
		* Convert a VertexSelection to a GeometrySelection
		*/
		CHAOS_API TArray<int32> ConvertVertexSelectionToGeometrySelection(const TArray<int32>& InVertexSelection, bool bAllElementsMustBeSelected) const;
		
		/**
		* Convert a TransformSelection to a CurveSelection
		*/
		CHAOS_API TArray<int32> ConvertTransformSelectionToCurveSelection(const TArray<int32>& InTransformSelection) const;

		/**
		* Convert a GeometrySelection to a CurveSelection
		*/
		CHAOS_API TArray<int32> ConvertGeometrySelectionToCurveSelection(const TArray<int32>& InGeometrySelection) const;

		/**
		* Convert a VertexSelection to a CurveSelection
		*/
		CHAOS_API TArray<int32> ConvertVertexSelectionToCurveSelection(const TArray<int32>& InVertexSelection, bool bAllElementsMustBeSelected) const;
		CHAOS_API void ConvertVertexSelectionToCurveSelection(const TBitArray<>& InVertexSelection, bool bAllElementsMustBeSelected, TBitArray<>& OutCurveSelection) const;

		/**
		* Convert a FaceSelection to a CurveSelection
		*/
		CHAOS_API TArray<int32> ConvertFaceSelectionToCurveSelection(const TArray<int32>& InFaceSelection, bool bAllElementsMustBeSelected) const;
		
		/**
		* Convert a CurveSelection to a VertexSelection
		*/
		CHAOS_API TArray<int32> ConvertCurveSelectionToVertexSelection(const TArray<int32>& InCurveSelection) const;
		
		/**
		* Convert a CurveSelection to a FaceSelection
		*/
		CHAOS_API TArray<int32> ConvertCurveSelectionToFaceSelection(const TArray<int32>& InCurveSelection) const;
		
		/**
		* Convert a CurveSelection to a GeometrySelection
		*/
		CHAOS_API TArray<int32> ConvertCurveSelectionToGeometrySelection(const TArray<int32>& InCurveSelection, bool bAllElementsMustBeSelected) const;
		
		/**
		* Convert a CurveSelection to a TransformSelection
		*/
		CHAOS_API TArray<int32> ConvertCurveSelectionToTransformSelection(const TArray<int32>& InCurveSelection, bool bAllElementsMustBeSelected) const;

		/**
		* Compute and return VertexToFaceIndices array
		*/
		CHAOS_API void GetVertexToFaceIndicesArray(TArray<TArray<int32>>& VertexToFaceIndices) const;

	private:
		// This a const facade
		const FManagedArrayCollection& ConstCollection;

		TManagedArrayAccessor<int32>		ParentAttribute;
		TManagedArrayAccessor<TSet<int32>>	ChildrenAttribute;
		TManagedArrayAccessor<int32>		LevelAttribute;
		TManagedArrayAccessor<int32>		SimulationTypeAttribute;
		TManagedArrayAccessor<int32>		TransformToGeometryIndexAttribute;
		TManagedArrayAccessor<int32>		TransformIndexAttribute;
		TManagedArrayAccessor<int32>		VertexStartAttribute;
		TManagedArrayAccessor<int32>		VertexCountAttribute;
		TManagedArrayAccessor<int32>		FaceStartAttribute;
		TManagedArrayAccessor<int32>		FaceCountAttribute;
		TManagedArrayAccessor<FIntVector>	IndicesAttribute;
	};

}
