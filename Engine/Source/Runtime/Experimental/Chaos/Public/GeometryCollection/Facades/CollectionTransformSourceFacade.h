// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace GeometryCollection::Facades
{

	/**
	* FTransformSource
	* 
	* Defines common API for storing owners of a transform hierarchy. For example, to store
	* the component that generated a transform hierirchy, use the function :
	*    Key = this->AddTransformSource(<Skeleton Asset Name>, <Skeleton Asset Guid>, {<Root transform indices in collection>});
	* 
	*    Key = this->AddTransformSource("", GUID, {1,5,7});
	*
	* The source root indices can be queries later using the name and guid:
	*	TSet<int32> SourceRoots = this->GetSourceTransformRoots("", GUID);
	*
	* The following groups are created on the collection based on which API is called. 
	* 
	*	<Group> = FTransformSource::TransformSourceGroup
	*	- FindAttribute<FString>(FTransformSource::SourceNameAttribute, <Group>);
	*	- FindAttribute<Guid>(FTransformSource::SourceGuidAttribute, <Group>);
	*	- FindAttribute<TSet<int32>>(FTransformSource::SourceRootsAttribute, <Group>);
	* 
	*/
	class FTransformSource
 	{
	public:

		// groups
		static CHAOS_API const FName TransformSourceGroupName;

		// Attributes
		static CHAOS_API const FName SourceNameAttributeName;
		static CHAOS_API const FName SourceGuidAttributeName;
		static CHAOS_API const FName SourceRootsAttributeName;
		static CHAOS_API const FName SourceSkeletalMeshNameAttributeName;
		/**
		* FSelectionFacade Constuctor
		* @param VertixDependencyGroup : GroupName the index attribute is dependent on. 
		*/
		CHAOS_API FTransformSource(FManagedArrayCollection& InSelf);
		CHAOS_API FTransformSource(const FManagedArrayCollection& InSelf);

		/** Create the facade. */
		CHAOS_API void DefineSchema();

		/** Is the facade defined constant. */
		bool IsConst() const { return SourceNameAttribute.IsConst(); }

		/** Is the Facade defined on the collection? */
		CHAOS_API bool IsValid() const;

		/**
		* Add a transform root mapping.
		* @param Name : Name of the owner of the transform set.
		* @param Guid : Guid of the owner of the transform set.
		* @param Roots : Root indices of the transform set.
		* @param SKMName : Name of the skeletal mesh owner of the transform set.
		*/
		CHAOS_API void AddTransformSource(const FString& Name, const FString& Guid, const TSet<int32>& Roots, const FString& SKMName = FString());

		/**
		* Query for root indices.  
		* @param Name : Name of the owner of the transform set.  
		* @param Guid : Guid of the owner of the transform set.
		*/
		CHAOS_API TSet<int32> GetTransformSource(const FString& Name, const FString& Guid, const FString& SKMName = FString()) const;

	private:
		TManagedArrayAccessor<FString> SourceNameAttribute;
		TManagedArrayAccessor<FString> SourceGuidAttribute;
		TManagedArrayAccessor< TSet<int32 > > SourceRootsAttribute;
		TManagedArrayAccessor<FString> SourceSkeletalMeshNameAttribute;
	};

}
