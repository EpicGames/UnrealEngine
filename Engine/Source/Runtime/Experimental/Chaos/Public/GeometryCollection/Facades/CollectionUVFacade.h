// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"

// Note: Prefer namespace GeometryCollection::UV for new code
namespace GeometryCollectionUV
{
	enum
	{
		MAX_NUM_UV_CHANNELS = 8,
	};
}

namespace GeometryCollection::UV
{
	// Names of each UV layer attribute. In a valid collection with UVs, the first layer must be present, and the other layers are optional
	extern CHAOS_API const FName UVLayerNames[GeometryCollectionUV::MAX_NUM_UV_CHANNELS];
	// The UV layers are always contained in the vertices group in a geometry collection
	extern CHAOS_API const FName VerticesGroupName;

	bool CHAOS_API HasValidUVs(const FManagedArrayCollection& Collection);
	bool CHAOS_API SetNumUVLayers(FManagedArrayCollection& Collection, int32 NumUVs);
	int32 CHAOS_API GetNumUVLayers(const FManagedArrayCollection& Collection);

	void CHAOS_API DefineUVSchema(FManagedArrayCollection& Collection);

	inline const TManagedArray<FVector2f>* FindUVLayer(const FManagedArrayCollection& Collection, int32 UVLayer)
	{
		return Collection.FindAttributeTyped<FVector2f>(UVLayerNames[UVLayer], VerticesGroupName);
	}
	inline TManagedArray<FVector2f>* FindUVLayer(FManagedArrayCollection& Collection, int32 UVLayer)
	{
		return Collection.FindAttributeTyped<FVector2f>(UVLayerNames[UVLayer], VerticesGroupName);
	}
	inline const TManagedArray<FVector2f>& GetUVLayer(const FManagedArrayCollection& Collection, int32 UVLayer)
	{
		checkSlow(UVLayer >= 0 && UVLayer < GeometryCollectionUV::MAX_NUM_UV_CHANNELS);
		return Collection.GetAttribute<FVector2f>(UVLayerNames[UVLayer], VerticesGroupName);
	}
	inline TManagedArray<FVector2f>& ModifyUVLayer(FManagedArrayCollection& Collection, int32 UVLayer)
	{
		checkSlow(UVLayer >= 0 && UVLayer < GeometryCollectionUV::MAX_NUM_UV_CHANNELS);
		return Collection.ModifyAttribute<FVector2f>(UVLayerNames[UVLayer], VerticesGroupName);
	}

	template<typename ManagedArrayType, int32 MaxArraySize>
	struct TArrayOfAttributesAccessor
	{
		TArray<ManagedArrayType*, TFixedAllocator<MaxArraySize>> Attributes;

		ManagedArrayType& operator[](int32 Idx) const
		{
			return *Attributes[Idx];
		}

		int32 Num() const
		{
			return Attributes.Num();
		}
	};

	using FUVLayers = TArrayOfAttributesAccessor<TManagedArray<FVector2f>, 8>;
	using FConstUVLayers = TArrayOfAttributesAccessor<const TManagedArray<FVector2f>, 8>;

	/**
	 * Find the currently-valid UV layers for a given collection
	 * @return A temporary accessor for all the active UV layers. Must be recreated if the collection attributes are changed / the number of UV layers is changed.
	 */
	inline FUVLayers FindActiveUVLayers(FManagedArrayCollection& Collection)
	{
		FUVLayers Layers;
		for (int32 LayerIdx = 0; LayerIdx < GeometryCollectionUV::MAX_NUM_UV_CHANNELS; ++LayerIdx)
		{
			TManagedArray<FVector2f>* Layer = Collection.FindAttributeTyped<FVector2f>(UVLayerNames[LayerIdx], VerticesGroupName);
			if (!Layer)
			{
				break;
			}
			Layers.Attributes.Add(Layer);
		}
		return Layers;
	}
	/**
	 * Find the currently-valid UV layers for a given collection
	 * @return A temporary accessor for all the active UV layers. Must be recreated if the collection attributes are changed / the number of UV layers is changed.
	 */
	inline FConstUVLayers FindActiveUVLayers(const FManagedArrayCollection& Collection)
	{
		FConstUVLayers Layers;
		for (int32 LayerIdx = 0; LayerIdx < GeometryCollectionUV::MAX_NUM_UV_CHANNELS; ++LayerIdx)
		{
			const TManagedArray<FVector2f>* Layer = Collection.FindAttributeTyped<FVector2f>(UVLayerNames[LayerIdx], VerticesGroupName);
			if (!Layer)
			{
				break;
			}
			Layers.Attributes.Add(Layer);
		}
		return Layers;
	}

	// Set all UVs in the array for the given vertex, stopping early if there are not enough matching UV layers in the collection
	inline void SetUVs(FManagedArrayCollection& Collection, int32 VertexIdx, TArrayView<const FVector2f> UVs)
	{
		for (int32 LayerIdx = 0; LayerIdx < UVs.Num(); ++LayerIdx)
		{
			TManagedArray<FVector2f>* Layer = FindUVLayer(Collection, LayerIdx);
			if (!ensure(Layer != nullptr))
			{
				break;
			}
			(*Layer)[VertexIdx] = UVs[LayerIdx];
		}
	}

	inline void MatchUVLayerCount(FManagedArrayCollection& ToCollection, const FManagedArrayCollection& FromCollection)
	{
		SetNumUVLayers(ToCollection, GetNumUVLayers(FromCollection));
	}
}

namespace GeometryCollection::Facades
{
	class FCollectionUVFacade
	{
	public:
		FCollectionUVFacade(FManagedArrayCollection& InCollection) : Collection(&InCollection), ConstCollection(&InCollection)
		{}
		FCollectionUVFacade(const FManagedArrayCollection& InCollection) : Collection(nullptr), ConstCollection(&InCollection)
		{}

		/**
		 * returns true if all the necessary attributes are present
		 * if not then the API can be used to create
		 */
		inline bool IsValid() const
		{
			return GeometryCollection::UV::HasValidUVs(*ConstCollection);
		}

		/**
		 * Add the necessary attributes if they are missing
		 */
		void DefineSchema()
		{
			GeometryCollection::UV::DefineUVSchema(*Collection);
		}

		inline bool SetNumUVLayers(int32 UVLayers)
		{
			return GeometryCollection::UV::SetNumUVLayers(*Collection, UVLayers);
		}

		inline int32 GetNumUVLayers() const
		{
			return GeometryCollection::UV::GetNumUVLayers(*ConstCollection);
		}

		inline const TManagedArray<FVector2f>& GetUVLayer(int32 UVLayer) const
		{
			return GeometryCollection::UV::GetUVLayer(*ConstCollection, UVLayer);
		}
		inline TManagedArray<FVector2f>& ModifyUVLayer(int32 UVLayer)
		{
			return GeometryCollection::UV::ModifyUVLayer(*Collection, UVLayer);
		}
		inline const TManagedArray<FVector2f>* FindUVLayer(int32 UVLayer) const
		{
			return GeometryCollection::UV::FindUVLayer(*ConstCollection, UVLayer);
		}
		inline TManagedArray<FVector2f>* FindUVLayer(int32 UVLayer)
		{
			return GeometryCollection::UV::FindUVLayer(*Collection, UVLayer);
		}
		inline FVector2f& ModifyUV(int32 VertexIdx, int32 LayerIdx)
		{
			return ModifyUVLayer(LayerIdx)[VertexIdx];
		}
		inline const FVector2f& GetUV(int32 VertexIdx, int32 LayerIdx) const
		{
			return GetUVLayer(LayerIdx)[VertexIdx];
		}
		inline void SetUV(int32 VertexIdx, int32 LayerIdx, const FVector2f& UV)
		{
			ModifyUVLayer(LayerIdx)[VertexIdx] = UV;
		}
		inline GeometryCollection::UV::FUVLayers FindActiveUVLayers()
		{
			return GeometryCollection::UV::FindActiveUVLayers(*Collection);
		}
		inline GeometryCollection::UV::FConstUVLayers FindActiveUVLayers() const
		{
			return GeometryCollection::UV::FindActiveUVLayers(*ConstCollection);
		}

		// Set all UVs in the array for the given vertex, stopping early if there are not enough matching UV layers in the collection
		inline void SetUVs(int32 VertexIdx, TArrayView<const FVector2f> UVs)
		{
			GeometryCollection::UV::SetUVs(*Collection, VertexIdx, UVs);
		}

	private:
		FManagedArrayCollection* Collection;
		const FManagedArrayCollection* ConstCollection;
	};

}