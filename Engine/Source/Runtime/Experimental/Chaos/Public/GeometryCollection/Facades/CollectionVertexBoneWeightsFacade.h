// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/UnrealString.h"
#include "Containers/Array.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionSelectionFacade.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

namespace GeometryCollection::Facades
{

	/**
	* FVertexBoneWeightsFacade
	* 
	* Defines common API for storing a vertex weights bound to a bone. This mapping is from the 
	* the vertex to the bone index. Kinematic array specifies whether vertices are considered kinematic. 
	* Non-kinematic vertices can also have associated bone indices and weights.
	* 
	* Then arrays can be accessed later by:
	*	const TManagedArray< TArray<int32> >* BoneIndices = FVertexBoneWeightsFacade::GetBoneIndices(this);
	*	const TManagedArray< TArray<float> >* BoneWeights = FVertexBoneWeightsFacade::GetBoneWeights(this);
	* 
	* The following attributes are created on the collection:
	* 
	*	- FindAttribute<TArray<int32>>(FVertexSetInterface::IndexAttribute, FGeometryCollection::VerticesGroup);
	*	- FindAttribute<TArray<float>>(FVertexSetInterface::WeightAttribute, FGeometryCollection::VerticesGroup);
	* 
	*/
	class FVertexBoneWeightsFacade
	{
	public:

		// Attributes
		static CHAOS_API const FName BoneIndicesAttributeName;
		static CHAOS_API const FName BoneWeightsAttributeName;
		static CHAOS_API const FName KinematicWeightAttributeName;
		static CHAOS_API const FName GeometryLODAttributeName;
		static CHAOS_API const FName SkeletalMeshAttributeName;
		
		/**
		* FVertexBoneWeightsFacade Constuctor
		*/
		CHAOS_API FVertexBoneWeightsFacade(FManagedArrayCollection& InSelf, const bool bInternalWeights = true);
		CHAOS_API FVertexBoneWeightsFacade(const FManagedArrayCollection& InSelf, const bool bInternalWeights = true);

		/** Define the facade */
		CHAOS_API void DefineSchema();

		/** Is the Facade const */
		bool IsConst() const { return Collection == nullptr; }

		/** Is the Facade defined on the collection? */
		CHAOS_API bool IsValid() const;

		/** Does the collection have bone weight and indices defined? */
		CHAOS_API bool HasValidBoneIndicesAndWeights() const;

		/** Add bone weight based on the kinematic bindings. */
		CHAOS_API void AddBoneWeightsFromKinematicBindings();

		/** Add single bone/weight to vertex */
		CHAOS_API void AddBoneWeight(int32 VertexIndex, int32 BoneIndex, float BoneWeight);

		/** Modify bone weight based on the kinematic bindings. */
		CHAOS_API void ModifyBoneWeight(int32 VertexIndex, const TArray<int32>& VertexBoneIndices, const TArray<float>& VertexBoneWeights);

		/** Modify kinematic weight based */
		CHAOS_API void ModifyKinematicWeight(int32 VertexIndex, const float KinematicWeight);

		/** Set vertex to be kinematic/dynamic */
		CHAOS_API void SetVertexKinematic(int32 VertexIndex, bool Value = true);

		/** Set vertex to be kinematic/dynamic */
		CHAOS_API void SetVertexArrayKinematic(const TArray<int32>& VertexIndices, bool Value = true);
		
		/** Return if the vertex is kinematic
		Pre 5.5 we did not have per-vertex kinematic attribute. 
		This supports defining kinematics without per-vertex kinematic attribute. */
		CHAOS_API bool IsKinematicVertex(int32 VertexIndex) const;
		
		/** Modify the geometry skelmesh and LOD. */
		CHAOS_API void ModifyGeometryBinding(const int32 GeometryIndex, const TObjectPtr<UObject>& SkeletalMesh, const int32 GeometryLOD);

		/** Normalize bone weights */
		CHAOS_API void NormalizeBoneWeights();

		/** Return number of vertices */
		int32 NumVertices() const { return ConstCollection.NumElements(FGeometryCollection::VerticesGroup); };
		
		/** Return number of bones */
		int32 NumBones() const { return ConstCollection.NumElements(FGeometryCollection::TransformGroup); };
		
		/** Return number of geometries */
		int32 NumGeometry() const { return ConstCollection.NumElements(FGeometryCollection::GeometryGroup); };

		/** Return the vertex bone indices from the collection. Null if not initialized.  */
		CHAOS_API const TManagedArray< TArray<int32> >* FindBoneIndices()  const;
		CHAOS_API const TManagedArray< TArray<int32> >& GetBoneIndices() const;

		/** Return the vertex bone weights from the collection. Null if not initialized. */
		CHAOS_API const TManagedArray< TArray<float> >* FindBoneWeights()  const;
		CHAOS_API const TManagedArray< TArray<float> >& GetBoneWeights() const;

		/** Return the vertex kinematic weights from the collection. Null if not initialized. */
		const TManagedArray<float>* FindKinematicWeights()  const { return KinematicWeightAttribute.Find(); }
		const TManagedArray<float>& GetKinematicWeights() const { return KinematicWeightAttribute.Get(); }

		/** Return the skeletal meshes from the collection. Null if not initialized. */
		const TManagedArray<TObjectPtr<UObject>>* FindSkeletalMeshes()  const { return SkeletalMeshAttribute.Find(); }
		const TManagedArray<TObjectPtr<UObject>>& GetSkeletalMeshes() const { return SkeletalMeshAttribute.Get(); }
		
		/** Return the geometry LODs from the collection. Null if not initialized. */
		const TManagedArray<int32>* FindGeomketryLODs()  const { return GeometryLODAttribute.Find(); }
		const TManagedArray<int32>& GetGeometryLODs() const { return GeometryLODAttribute.Get(); }
		
		/** Get the managed array collection */
		const FManagedArrayCollection& GetManagedArrayCollection() const {return ConstCollection;}

	private:

		/** Deprecated names for attributes */
		static const FName DeprecatedBoneIndicesAttributeName;
		static const FName DeprecatedKinematicFlagAttributeName;
		
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;

		/** Bone indices for each vertices */
		TManagedArrayAccessor<TArray<int32>> BoneIndicesAttribute;

		/** Bone weights for each vertices */
		TManagedArrayAccessor<TArray<float>> BoneWeightsAttribute;

		/** Kinematic weights for each vertices */
		TManagedArrayAccessor<float> KinematicWeightAttribute;
		
		/** Geometry LODSs the skin weights are linked to */
		TManagedArrayAccessor<int32> GeometryLODAttribute;

		/** Geometry skeletal meshes the skin weights are linked to*/
		TManagedArrayAccessor<TObjectPtr<UObject>> SkeletalMeshAttribute;

		/** Internal weights flag to specify if the bone indices are relative to the transform group or to an external skelkmesh */
		const bool bInternalWeights = true;
	};

}
