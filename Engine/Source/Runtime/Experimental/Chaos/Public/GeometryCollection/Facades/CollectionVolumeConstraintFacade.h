// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollection.h"

namespace GeometryCollection::Facades
{
	/** Volume Constraint Facade */
	class FVolumeConstraintFacade
	{
	public:
		// Attributes
		static CHAOS_API const FName GroupName;
		static CHAOS_API const FName VolumeIndex;
		static CHAOS_API const FName Stiffness;

		CHAOS_API FVolumeConstraintFacade(FManagedArrayCollection& InCollection);
		CHAOS_API FVolumeConstraintFacade(const FManagedArrayCollection& InCollection);
	
		FVolumeConstraintFacade(FVolumeConstraintFacade&&) = default;

		/** Create the facade attributes. */
		CHAOS_API void DefineSchema();

		/** Is the facade defined constant. */
		bool IsConst() const { return Collection == nullptr; }

		/** Is the Facade defined on the collection? */
		CHAOS_API bool IsValid() const;

		//
		//  Accessors of constraints
		//
		CHAOS_API int32 AddVolumeConstraint(const FIntVector4& NewVolumeIndex, float NewStiffness);
		CHAOS_API FIntVector4 GetVolumeIndex(const int32 AttributeIndex) const;
		CHAOS_API float GetStiffness(const int32 AttributeIndex) const;
		int32 NumVolumeConstraints() const { return VolumeIndexAttribute.Num(); }
		
		/** Remove volume constraint with invalid indices*/
		CHAOS_API int32 RemoveInvalidVolumeConstraint();
		/** Remove volume constraint between two groups of vertices*/
		CHAOS_API int32 RemoveVolumeConstraintBetween(TFunctionRef<bool(const int32 VertexIdx)> IsVertexGroup1, TFunctionRef<bool(const int32 VertexIdx)> IsVertexGroup2);

	protected:
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;

		TManagedArrayAccessor<FIntVector4> VolumeIndexAttribute;
		TManagedArrayAccessor<float> StiffnessAttribute;
	};
}
