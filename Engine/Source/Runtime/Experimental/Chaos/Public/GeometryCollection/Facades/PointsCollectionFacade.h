// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "GeometryCollection/ManagedArrayCollection.h"

namespace GeometryCollection::Facades
{

	/**
	* FPointsCollectionFacade
	* 
	* Defines common API for storing points with attributes in a collection.
	* The points/attributes are stored in the Vertices group.
	* 
	*/

	class FPointsCollectionFacade
	{
	public:
		/**
		* FPointsCollectionFacade Constuctor
		*/
		CHAOS_API FPointsCollectionFacade(FManagedArrayCollection& InSelf);
		CHAOS_API FPointsCollectionFacade(const FManagedArrayCollection& InSelf);

		/** Is the facade defined constant. */
		bool IsConst() const { return Collection==nullptr; }

		/**
		* Add points to the collection
		*/
		CHAOS_API void AddPoints(const TArray<FVector>& InPoints);
		CHAOS_API void AddPointsWithFloatAttribute(const TArray<FVector>& InPoints, const FName InAttributeName, const TArray<float>& InValues);
		CHAOS_API void AddPointsWithIntAttribute(const TArray<FVector>& InPoints, const FName InAttributeName, const TArray<int32>& InValues);
		CHAOS_API void AddPointsWithVectorAttribute(const TArray<FVector>& InPoints, const FName InAttributeName, const TArray<FVector>& InValues);

		CHAOS_API void GetPointsWithFloatAttribute(TArray<FVector>& OutPoints, const FName InAttributeName, TArray<float>& OutValues, const int32 InTransformIndex = 0);

	private:

		// const collection will be a null pointer, 
		// while non-const will be valid.
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* Collection = nullptr;
	};
}
