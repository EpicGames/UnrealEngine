// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "GeometryCollection/ManagedArrayCollection.h"
#include "UObject/NameTypes.h"

#include "PointsFacade.generated.h"

UENUM()
enum class EFilterByAttributeOperation : uint8
{
	Equal UMETA(DisplayName = "=="),
	NotEqual UMETA(DisplayName = "!="),
	Greater UMETA(DisplayName = ">"),
	GreaterOrEqual UMETA(DisplayName = ">="),
	Smaller UMETA(DisplayName = "<"),
	SmallerOrEqual UMETA(DisplayName = "<="),
	InRangeInclusive UMETA(DisplayName = "<= X <="),
	InRangeExclusive UMETA(DisplayName = "< X <")
};

namespace GeometryCollection::Facades
{
	/**
	* FPointsFacade
	* 
	* Defines common API for storing points with attributes in a collection.
	* The points/attributes are stored in the Points group.
	* 
	*/

	class FPointsFacade
	{
	public:
		CHAOS_API static const FName PointsGroup;
		CHAOS_API static const FName PointAttribute;

		/**
		* FPointsFacade Constuctor
		*/
		CHAOS_API FPointsFacade(FManagedArrayCollection& InSelf);
		CHAOS_API FPointsFacade(const FManagedArrayCollection& InSelf);

		FPointsFacade(const FPointsFacade&) = delete;
		FPointsFacade& operator= (const FPointsFacade&) = delete;

		// Get Access to the ConstCollection
		CHAOS_API const FManagedArrayCollection& GetConstCollection() const;

		/** Is the facade defined constant. */
		bool IsConst() const { return MutableCollection == nullptr; }

		/** Is the facade defined constant. */
		CHAOS_API bool IsValid() const;

		/** Adds neccessary group/Attribute */
		void DefineSchema();

		/**
		* Get number of points in the collection
		*/
		CHAOS_API int32 GetNumPoints() const;

		/**
		* Get number of attributes on the points
		*/
		CHAOS_API int32 GetNumAttrs() const;

		/**
		* Get list of attributes on the points
		*/
		CHAOS_API TArray<FName> GetAttrs() const;
		CHAOS_API FString GetAttrsAsString(bool bAddCommaSeparator = true) const;

		/**
		* Add points to the collection
		*/
		CHAOS_API void AddPoints(const TArray<FVector3f>& InPoints);
		CHAOS_API void AddPoints(const TArray<FVector>& InPoints);
		CHAOS_API void AddPoints(const FPointsFacade& InPointsFacade);

		CHAOS_API void AppendPoints(const TArray<FVector3f>& InPoints);
		CHAOS_API void AppendPoints(const TArray<FVector>& InPoints);
		CHAOS_API void AppendPoints(const FPointsFacade& InPointsFacade);

		/**
		* Delete points from the collection
		*/
		CHAOS_API void DeletePoints(const TArray<int32>& InSortedDeletionList);

		/**
		* Add attribute to the points
		*/
		CHAOS_API bool AddFloatAttribute(const FName InAttribute, const TArray<float>& InValues);
		CHAOS_API bool AddBoolAttribute(const FName InAttribute, const TArray<bool>& InValues);
		CHAOS_API bool AddBoolAttribute(const FName InAttribute, const TBitArray<>& InValues);
		CHAOS_API bool AddIntAttribute(const FName InAttribute, const TArray<int32>& InValues);
		CHAOS_API bool AddVector2Attribute(const FName InAttribute, const TArray<FVector2f>& InValues);
		CHAOS_API bool AddVector3Attribute(const FName InAttribute, const TArray<FVector3f>& InValues);

		/**
		* Check if attribute exist
		*/
		CHAOS_API bool HasAttribute(const FName& InAttribute) const;

		/**
		* Get attribute type
		*/
		CHAOS_API EManagedArrayType GetAttributeType(const FName& InAttribute) const;

		/**
		* Filter points by a float attribute value
		*/
		CHAOS_API void FilterByFloatAttributeToCollection(const FName InAttribute, const EFilterByAttributeOperation InOperation, float InValue, float InValue2, FManagedArrayCollection& OutCollection) const;
		CHAOS_API void FilterByFloatAttributeToArray(const FName InAttribute, const EFilterByAttributeOperation InOperation, float InValue, float InValue2, TArray<FVector>& OutPoints) const;
		CHAOS_API void FilterByFloatAttributeToSelection(const FName InAttribute, const EFilterByAttributeOperation InOperation, float InValue, float InValue2, TArray<int32>& OutPointSelection) const;

		/**
		* Get the points as a TArray<FVector> (it returns the point positions, all attributes will be lost)
		*/
		CHAOS_API TArray<FVector> GetPointsAsArray() const;
		CHAOS_API TArray<FVector3f> GetPointsAsFloatArray() const;

		/**
		* Get the values of a specified attribute as an array
		*/
		CHAOS_API const TArray<float>& GetFloatAttributeValues(const FName InAttribute) const;
		CHAOS_API const TBitArray<>& GetBoolAttributeValues(const FName InAttribute) const;
		CHAOS_API const TArray<int32>& GetIntAttributeValues(const FName InAttribute) const;
		CHAOS_API const TArray<FVector2f>& GetVector2AttributeValues(const FName InAttribute) const;
		CHAOS_API const TArray<FVector3f>& GetVector3AttributeValues(const FName InAttribute) const;

		/**
		* Get the element of a specified attribute from the array
		*/
		CHAOS_API const float GetFloatAttributeValue(const FName InAttribute, const int32 ElementIndex) const;
		CHAOS_API const bool GetBoolAttributeValue(const FName InAttribute, const int32 ElementIndex) const;
		CHAOS_API const int32 GetIntAttributeValue(const FName InAttribute, const int32 ElementIndex) const;
		CHAOS_API const FVector2f GetVector2AttributeValue(const FName InAttribute, const int32 ElementIndex) const;
		CHAOS_API const FVector3f GetVector3AttributeValue(const FName InAttribute, const int32 ElementIndex) const;

		/**
		* Transform points
		*/
		CHAOS_API void TransformPoints(const FTransform& InTransform);

		/**
		* Generate points in an AABB
		*/
		CHAOS_API bool GeneratePointsInBox(const FBox& InBox, float InPointSeparation);

		/**
		* Generate points in an AABB on a plane
		*/
		CHAOS_API bool GeneratePointsOnPlane(const FBox& InBox, const int32 InPlaneOrientation, float InPointSeparation, float InOffset);

		/**
		* Computes the smallest AABB for the points
		*/
		CHAOS_API FBox GetBoundingBox();

		/**
		* Computes the smallest bounding sphere for the points
		*/
		CHAOS_API FSphere GetBoundingSphere();

	private:
		const FManagedArrayCollection& ConstCollection;
		FManagedArrayCollection* MutableCollection = nullptr;
	};
}
