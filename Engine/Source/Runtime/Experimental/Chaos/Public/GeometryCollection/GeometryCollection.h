// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "TransformCollection.h"
#include "Misc/Crc.h"

#include "GeometryCollection/GeometryCollectionConvexPropertiesInterface.h"
#include "GeometryCollection/GeometryCollectionProximityPropertiesInterface.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/Facades/CollectionUVFacade.h"

namespace Chaos
{
	class FChaosArchive;
}
class FGeometryCollectionConvexPropertiesInterface;
class FGeometryCollectionProximityPropertiesInterface;

struct FGeometryCollectionDefaults
{
	FLinearColor DefaultVertexColor = FLinearColor::White;
};

/**
* FGeometryCollection (FTransformCollection)
*/
class FGeometryCollection : public FTransformCollection, 
	public FGeometryCollectionConvexPropertiesInterface,
	public FGeometryCollectionProximityPropertiesInterface
{

public:
	typedef FTransformCollection Super;

	CHAOS_API FGeometryCollection(FGeometryCollectionDefaults InDefaults = FGeometryCollectionDefaults() );
	CHAOS_API ~FGeometryCollection();
	FGeometryCollection(FGeometryCollection &) = delete;
	FGeometryCollection& operator=(const FGeometryCollection &) = delete;
	FGeometryCollection(FGeometryCollection &&) = default;
	FGeometryCollection& operator=(FGeometryCollection &&) = default;
	MANAGED_ARRAY_COLLECTION_INTERNAL(FGeometryCollection);

		/***
		*  Attribute Groups
		*
		*   These attribute groups are predefined data member of the FGeometryCollection.
		*
		*   VerticesGroup ("Vertices")
		*
		*			FVectorArray      Vertex         = GetAttribute<FVector3f>("Vertex", VerticesGroup)
		*			FInt32Array       BoneMap        = GetAttribute<Int32>("BoneMap", VerticesGroup, {"Transform"})
		*           FVectorArray      Normal         = GetAttribute<FVector3f>("Normal", MaterialGroup)
		*			FVector2DArray    UVs            = GetAttribute<TArray<FVector2D>>("UVs", MaterialGroup)
		*           FVectorArray      TangentU       = GetAttribute<FVector3f>("TangentU", MaterialGroup)
		*           FVectorArray      TangentV       = GetAttribute<FVector3f>("TangentV", MaterialGroup)
		*           FLinearColorArray Color          = GetAttribute<FLinearColor>("Color", MaterialGroup)
		*
		*		The VerticesGroup will store per-vertex information about the geometry. For
		*       example, the "Position" attribute stores a FVector array for the relative
		*       offset of a vertex from the geometries geometric center, and the "BoneMap"
		*       attribute stores an index in to the TransformGroups Transform array so that
		*       the local space vertices may be mapped in to world space positions.
		*
		*	FacesGroup ("Faces")
		*		Default Attributes :
		*
		*            FIntVectorArray   Indices       = GetAttribute<FIntVector>("Indices", FacesGroup, {"Faces"})
		*            FBoolArray        Visible       = GetAttribute<bool>("Visible", FacesGroup)
		*            FInt32Array       MaterialIndex = GetAttribute<Int32>("MaterialIndex", FacesGroup)
		*            FInt32Array       MaterialID    = GetAttribute<Int32>("MaterialID", FacesGroup)
		*
		*       The FacesGroup will store the triangulated face data, and any other information
		*       that is associated with the faces of the geometry. The "Triangle" attribute is
		*       stored as Vector<int,3>, and represents the vertices of a individual triangle.
		*
		*	GeometryGroup ("Geometry")
		*		Default Attributes :
		*
		*			FInt32Array       TransformIndex = GetAttribute<Int32>("TransformIndex", GeometryGroup, {"Transform"})
		*			FBoxArray		  BoundingBox = GetAttribute<FBox>("BoundingBox", GeometryGroup)
		*			FIntArray		  FaceStart = GetAttribute<int32>("FaceStart", GeometryGroup)
		*			FIntArray		  FaceCount = GetAttribute<int32>("FaceCount", GeometryGroup)
		*			FIntArray		  VertexStart = GetAttribute<int32>("VertexStart", GeometryGroup)
		*			FIntArray		  VertexCount = GetAttribute<int32>("VertexCount", GeometryGroup)
		*
		*       The GeometryGroup will store the transform indices, bounding boxes and any other information
		*       that is associated with the geometry.
		*
		*	MaterialGroup ("Material")
		*		Default Attributes	:
		*
		*			FGeometryCollectionSection	Sections = GetAttribute<FGeometryCollectionSection>("Sections", MaterialGroup)
		*
		*		 The set of triangles which are rendered with the same material
		*/

	static CHAOS_API const FName VerticesGroup; // Vertices
	static CHAOS_API const FName FacesGroup;	  // Faces
	static CHAOS_API const FName GeometryGroup; // Geometry
	static CHAOS_API const FName BreakingGroup; // Breaking
	static CHAOS_API const FName MaterialGroup; // Materials

	static CHAOS_API const FName SimulatableParticlesAttribute;
	static CHAOS_API const FName SimulationTypeAttribute;
	static CHAOS_API const FName StatusFlagsAttribute;
	static CHAOS_API const FName ExternalCollisionsAttribute;
	static CHAOS_API const FName ColorAttribute;
	static CHAOS_API const FName BoneColorAttribute;
	static CHAOS_API const FName TransformToGeometryIndexAttribute;
	static CHAOS_API const FName BoundingBoxAttribute;
	static CHAOS_API const FName TransformIndexAttribute;
	static CHAOS_API const FName VertexStartAttribute;
	static CHAOS_API const FName VertexCountAttribute;
	static CHAOS_API const FName InnerRadiusAttribute;
	static CHAOS_API const FName OuterRadiusAttribute;
	static CHAOS_API const FName FaceStartAttribute;
	static CHAOS_API const FName FaceCountAttribute;
	static CHAOS_API const FName FaceIndicesAttribute;
	static CHAOS_API const FName FaceVisibleAttribute;
	static CHAOS_API const FName MaterialIndexAttribute;
	static CHAOS_API const FName MaterialIDAttribute;
	static CHAOS_API const FName VertexPositionAttribute;
	static CHAOS_API const FName VertexNormalAttribute;
	static CHAOS_API const FName VertexTangentUAttribute;
	static CHAOS_API const FName VertexTangentVAttribute;
	static CHAOS_API const FName VertexBoneMapAttribute;

	enum ESimulationTypes : uint8
	{
		FST_None = 0,
		FST_Rigid = 1,
		FST_Clustered = 2,

		FST_Max = 3
	};

	enum ENodeFlags : uint32
	{
		// additional flags
		FS_None = 0,

		// identify nodes that should be removed from the simulation instead of becoming a fractured body
		FS_RemoveOnFracture = 0x00000004,

		FS_IgnoreCollisionInParentCluster = 0x00000008

	};

	static CHAOS_API bool AreCollisionParticlesEnabled();

	/**
	 * Create a GeometryCollection from Vertex and Indices arrays
	 */
	static CHAOS_API FGeometryCollection* NewGeometryCollection(const TArray<float>& RawVertexArray, const TArray<int32>& RawIndicesArray, bool ReverseVertexOrder = true, FGeometryCollectionDefaults InDefaults = FGeometryCollectionDefaults());
	static CHAOS_API void Init(FGeometryCollection* Collection, const TArray<float>& RawVertexArray, const TArray<int32>& RawIndicesArray, bool ReverseVertexOrder = true);
	static CHAOS_API void DefineGeometrySchema(FManagedArrayCollection&);

	/**
	* Create a GeometryCollection from Vertex, Indices, BoneMap, Transform, BoneHierarchy arrays
	*/
	static CHAOS_API FGeometryCollection* NewGeometryCollection(const TArray<float>& RawVertexArray,
		const TArray<int32>& RawIndicesArray,
		const TArray<int32>& RawBoneMapArray,
		const TArray<FTransform>& RawTransformArray,
		const TManagedArray<int32>& RawLevelArray,
		const TManagedArray<int32>& RawParentArray,
		const TManagedArray<TSet<int32>>& RawChildrenArray,
		const TManagedArray<int32>& RawSimulationTypeArray,
		const TManagedArray<int32>& RawStatusFlagsArray,
		FGeometryCollectionDefaults InDefaults = FGeometryCollectionDefaults());

	/**
	* Append a FGeometryCollection to a FGeometryCollection
	*/
	CHAOS_API void AppendCollection(const FGeometryCollection& InCollection);

	/** 
	* Append a single geometric object to a FGeometryCollection 
	*/
	CHAOS_API int32 AppendGeometry(const FGeometryCollection & GeometryCollection, int32 MaterialIDOffset = 0, bool ReindexAllMaterials = true, const FTransform& TransformRoot = FTransform::Identity);

	/**
	* Append single embedded geometry. Returns true if the operation succeeds.
	*/
	CHAOS_API bool AppendEmbeddedInstance(int32 InExemplarIndex, int32 InParentIndex, const FTransform& InTransform = FTransform::Identity);

	/**
	 * Reindex exemplar indices to reflect removed exemplars.
	 */
	CHAOS_API void ReindexExemplarIndices(TArray<int32>& SortedRemovedIndices);

	/**
	* Remove Geometry and update dependent elements
	*/
	CHAOS_API virtual void RemoveElements(const FName & Group, const TArray<int32>& SortedDeletionList, FProcessingParameters Params = FProcessingParameters()) override;

	/**
	* Empty each managed array in each group in order to reset the collection to an initial (empty) state. 
	*/
	CHAOS_API void Empty();

	/**
	* reset internal state
	*/
	CHAOS_API virtual void Reset() override;

	/**
	* Reorders elements in a group. NewOrder must be the same length as the group.
	*/
	CHAOS_API virtual void ReorderElements(FName Group, const TArray<int32>& NewOrder) override;

	//
	//
	//

	// Initialize any interfaces on the geometry collection (i.e., the FGeometryCollectionConvexPropertiesInterface)
	CHAOS_API virtual void InitializeInterfaces();


	/**
	*  Update bounding box entries for the geometry
	*/
	CHAOS_API void UpdateBoundingBox();
	static CHAOS_API void UpdateBoundingBox(FManagedArrayCollection&, bool bSkipCheck=false);

	/**  
	* GetBoundingBox 
	*/
	CHAOS_API FBoxSphereBounds GetBoundingBox() const;

	/**
	 * Update the visibility of specified geometry nodes
	 */
	CHAOS_API void UpdateGeometryVisibility(const TArray<int32>& NodeList, bool VisibilityState);

	/**
	* Reindex sections to keep polys with same materials together to reduce the number of draw calls
	*/
	CHAOS_API void ReindexMaterials();
	static CHAOS_API void ReindexMaterials(FManagedArrayCollection&);

	/**
	* Builds mesh sections for a given index buffer that could be a subset.
	* Currently, this call assumes that the indices are ordered by MaterialID
	* #todo(dmp): Refactor this and ReindexMaterials to share code
	*/
	CHAOS_API TArray<FGeometryCollectionSection> BuildMeshSections(const TArray<FIntVector> &Indices, const TArray<int32>& BaseMeshOriginalIndicesIndex, TArray<FIntVector> &RetIndices) const;
	//
	//
	//

	/** Returns true if there is anything to render */
	CHAOS_API bool HasVisibleGeometry() const;

	/** Returns true if the vertices are contiguous*/
	CHAOS_API bool HasContiguousVertices() const;

	/** Returns true if the faces are contiguous*/
	CHAOS_API bool HasContiguousFaces() const;

	/** Returns true if the render faces are contiguous*/
	CHAOS_API bool HasContiguousRenderFaces() const;

	/** Returns number of UV layers represented by UV array. A Valid Geometry Collection has the same count for every vertex */
	CHAOS_API int32 NumUVLayers() const;

	/** Update a geometry collection to have the target number of UV layers (must be in the range [1, MAX_UV_LAYERS)) */
	CHAOS_API bool SetNumUVLayers(int32 NumLayers);

	FORCEINLINE bool IsGeometry(int32 Element) const { return TransformToGeometryIndex[Element] != INDEX_NONE; }
	FORCEINLINE bool IsClustered(int32 Element) const { const TManagedArray<int32>& SimType = SimulationType;  return !!(SimType[Element] == ESimulationTypes::FST_Clustered); }
	FORCEINLINE bool IsRigid(int32 Element) const { const TManagedArray<int32>& SimType = SimulationType;  return !!(SimType[Element] == ESimulationTypes::FST_Rigid); }
	FORCEINLINE bool IsTransform(int32 Element) const { return !IsGeometry(Element); }
	FORCEINLINE void SetFlags(int32 Element, int32 InFlags) { TManagedArray<int32>& Status = StatusFlags; Status[Element] |= InFlags; }
	FORCEINLINE void ClearFlags(int32 Element, int32 InFlags) { TManagedArray<int32>& Status = StatusFlags; Status[Element] = Status[Element] & ~InFlags; }
	FORCEINLINE bool HasFlags(int32 Element, int32 InFlags) const { const TManagedArray<int32>& Status = StatusFlags; return (Status[Element] & InFlags) != 0; }

	/** Return true if the Element contains any visible faces. */
	CHAOS_API bool IsVisible(int32 Element) const;

	/** Connection of leaf geometry */
	CHAOS_API TArray<TArray<int32>> ConnectionGraph();

	//
	//
	//

	/** Serialize */
	CHAOS_API void Serialize(Chaos::FChaosArchive& Ar);

	/**   */
	CHAOS_API void WriteDataToHeaderFile(const FString &Name, const FString &Path);

	/**  */
	CHAOS_API void WriteDataToOBJFile(const FString &Name, const FString &Path, const bool WriteTopology=true, const bool WriteAuxStructures=true);

	//
	//
	//
	FGeometryCollectionDefaults Defaults;
	CHAOS_API virtual void SetDefaults(FName Group, uint32 StartSize, uint32 NumElements) override;

	// Transform Group
	TManagedArray<int32>		TransformToGeometryIndex;
	TManagedArray<int32>        SimulationType;
	TManagedArray<int32>        StatusFlags;
	TManagedArray<int32>		InitialDynamicState;
	TManagedArray<int32>		ExemplarIndex;

	// Vertices Group
	TManagedArray<FVector3f>		 Vertex;

	// Note: UVs have been reworked, and unfortunately there is not a safe path to provide the original UVs managed array as a deprecated accessor.
	// They are now stored in dynamically allocated attributes per UV channel (/ layer)
	// See Facades/CollectionUVFacade.h for a more complete interface to access UV layers,
	// but accesses of the form Collection.UVs[Vertex][Layer] can be replaced with Collection.GetUV(Vertex, Layer) (or ModifyUV)
	FVector2f& ModifyUV(int32 VertexIndex, int32 UVLayer)
	{
		return GeometryCollection::UV::ModifyUVLayer(*this, UVLayer)[VertexIndex];
	}
	const FVector2f& GetUV(int32 VertexIndex, int32 UVLayer) const
	{
		return GeometryCollection::UV::GetUVLayer(*this, UVLayer)[VertexIndex];
	}
	inline TManagedArray<FVector2f>* FindUVLayer(int32 UVLayer)
	{
		return GeometryCollection::UV::FindUVLayer(*this, UVLayer);
	}
	inline const TManagedArray<FVector2f>* FindUVLayer(int32 UVLayer) const
	{
		return GeometryCollection::UV::FindUVLayer(*this, UVLayer);
	}

	TManagedArray<FLinearColor>      Color;
	TManagedArray<FVector3f>         TangentU;
	TManagedArray<FVector3f>         TangentV;
	TManagedArray<FVector3f>         Normal;
	TManagedArray<int32>             BoneMap;

	// Faces Group
	TManagedArray<FIntVector>   Indices;
	TManagedArray<bool>         Visible;
	TManagedArray<int32>        MaterialIndex;
	TManagedArray<int32>        MaterialID;
	TManagedArray<bool>         Internal;

	// Geometry Group
	TManagedArray<int32>        TransformIndex;
	TManagedArray<FBox>			BoundingBox;
	TManagedArray<float>		InnerRadius;
	TManagedArray<float>		OuterRadius;
	TManagedArray<int32>		VertexStart;
	TManagedArray<int32>		VertexCount;
	TManagedArray<int32>		FaceStart;
	TManagedArray<int32>		FaceCount;

	// Material Group
	TManagedArray<FGeometryCollectionSection> Sections;
	
protected:

	/**
	 * Virtual helper function called by CopyMatchingAttributesFrom; adds attributes 'default, but optional' attributes that are present in InCollection
	 * This is used by FGeometryCollection to make sure all UV layers are copied over by CopyMatchingAttributesFrom()
	 */
	virtual void MatchOptionalDefaultAttributes(const FManagedArrayCollection& InCollection) override
	{
		GeometryCollection::UV::MatchUVLayerCount(*this, InCollection);
	}

	CHAOS_API void Construct();

	/**
	* Remove Geometry elements i.e. verts, faces, etc, leaving the transform nodes intact
	*/
	CHAOS_API void RemoveGeometryElements(const TArray<int32>& SortedGeometryIndicesToDelete);

	/**
	* Update Face Attributes based on changes in the group.
	*/
	CHAOS_API bool BuildFaceToGeometryMapping(bool InSaved=false);
	CHAOS_API void UpdateFaceGroupElements();

	/**
	* Build and Update Vertex Attributes based on changes in the group.
	*/
	CHAOS_API bool BuildVertexToGeometryMapping(bool InSaved = false);
	CHAOS_API void UpdateVerticesGroupElements();

	/** 
	* Reorder geometry elements. i.e. verts faces etc are reordered so we can get contiguous memory access
	*/
	CHAOS_API void ReorderGeometryElements(const TArray<int32>& NewOrder);

	/**
	* Reorder geometry elements based on the new transform order. i.e. verts faces etc are reordered so we can get contiguous memory access
	*/
	CHAOS_API void ReorderTransformElements(const TArray<int32>& NewOrder);

	/**
	 * @return the latest Version number used in serialization
	 */
	constexpr int32 GetLatestVersionNumber() const
	{
		return 10;
	}

	CHAOS_API virtual void Append(const FManagedArrayCollection& InCollection) override;

public:
	/* Backwards compatibility */
	CHAOS_API void UpdateOldAttributeNames();

	CHAOS_API TArray<int32> TransformSelectionToGeometryIndices(const TArray<int32>& Transforms);
};

FORCEINLINE Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, FGeometryCollection& Value)
{
	Value.Serialize(Ar);
	return Ar;
}

