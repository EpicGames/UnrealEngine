// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ManagedArray.h"
#include "GeometryCollection/TransformCollection.h"

class FGeometryCollection;
class FGeometryDynamicCollection;

namespace GeometryCollectionAlgo
{
	struct FFaceEdge
	{
		int32 VertexIdx1;
		int32 VertexIdx2;

		friend inline uint32 GetTypeHash(const FFaceEdge& Other)
		{
			return HashCombine(::GetTypeHash(Other.VertexIdx1), ::GetTypeHash(Other.VertexIdx2));
		}

		friend bool operator==(const FFaceEdge& A, const FFaceEdge& B)
		{
			return A.VertexIdx1 == B.VertexIdx1 && A.VertexIdx2 == B.VertexIdx2;
		}
	};

	/*
	* Print the parent hierarchy of the collection.
	*/
	void 
	CHAOS_API 
	PrintParentHierarchy(const FGeometryCollection * Collection);

	/**
	* Build a contiguous array of integers
	*/
	void
	CHAOS_API
	ContiguousArray(TArray<int32> & Array, int32 Length);
		
	/**
	* Offset list for re-incrementing deleted elements.
	*/
	void 
	CHAOS_API
	BuildIncrementMask(const TArray<int32> & SortedDeletionList, const int32 & Size, TArray<int32> & Mask);

	/**
	*
	*/
	void 
	CHAOS_API
	BuildLookupMask(const TArray<int32> & SortedDeletionList, const int32 & Size, TArray<bool> & Mask);


	/*
	*
	*/
	void
	CHAOS_API
	BuildTransformGroupToGeometryGroupMap(const FGeometryCollection& GeometryCollection, TArray<int32> & TransformToGeometry);


	/*
	*
	*/
	void
	CHAOS_API
	BuildFaceGroupToGeometryGroupMap(const FGeometryCollection& GeometryCollection, const TArray<int32>& TransformToGeometryMap, TArray<int32> & FaceToGeometry);


	/**
	* Make sure the deletion list is correctly formed.
	*/
	void 
	CHAOS_API
	ValidateSortedList(const TArray<int32>&SortedDeletionList, const int32 & ListSize);

	/*
	*  Check if the Collection has multiple transform roots.
	*/
	bool 
	CHAOS_API 
	HasMultipleRoots(FGeometryCollection * Collection);

	/*
	*
	*/
	bool 
	CHAOS_API
	HasCycle(const TManagedArray<int32>& Parents, int32 Node);

	/*
	*
	*/
	bool 
	CHAOS_API
	HasCycle(const TManagedArray<int32>& Parents, const TArray<int32>& SelectedBones);

	/*
	* Parent a single transform
	*/
	void
	CHAOS_API 
	ParentTransform(FManagedArrayCollection* ManagedArrayCollection, const int32 TransformIndex, const int32 ChildIndex);
		
	/*
	*  Parent the list of transforms to the selected index. 
	*/
	void 
	CHAOS_API 
	ParentTransforms(FManagedArrayCollection* ManagedArrayCollection, const int32 TransformIndex, const TArray<int32>& SelectedBones);


	/*
	*  Unparent the child index from its parent
	*/
	void
	CHAOS_API
	UnparentTransform(FManagedArrayCollection* ManagedArrayCollection, const int32 ChildIndex);

	/*
	*  Find the average position of the transforms.
	*/
	FVector
	CHAOS_API
	AveragePosition(FGeometryCollection* Collection, const TArray<int32>& Indices);


	/*
	*  Global Matrices of the specified index.
	*/
	FTransform CHAOS_API GlobalMatrix(const TManagedArray<FTransform>& RelativeTransforms, const TManagedArray<int32>& Parents, int32 Index);
	FTransform CHAOS_API GlobalMatrix(const TManagedArray<FTransform3f>& RelativeTransforms, const TManagedArray<int32>& Parents, int32 Index);
	FTransform CHAOS_API GlobalMatrix(TArrayView<const FTransform> RelativeTransforms, TArrayView<const int32> Parents, int32 Index);
	FTransform CHAOS_API GlobalMatrix(TArrayView<const FTransform3f> RelativeTransforms, TArrayView<const int32> Parents, int32 Index);
	FTransform3f CHAOS_API GlobalMatrix3f(const TManagedArray<FTransform3f>& RelativeTransforms, const TManagedArray<int32>& Parents, int32 Index);


	/*
	*  Global Matrices of the collection based on list of indices
	*/
	void CHAOS_API GlobalMatrices(const TManagedArray<FTransform>& RelativeTransforms, const TManagedArray<int32>& Parents, const TArray<int32>& Indices, TArray<FTransform>& Transforms);
	void CHAOS_API GlobalMatrices(const TManagedArray<FTransform3f>& RelativeTransforms, const TManagedArray<int32>& Parents, const TArray<int32>& Indices, TArray<FTransform3f>& Transforms);

	/*
	 *  Recursively traverse from a root node down
	 */
	void CHAOS_API GlobalMatricesFromRoot(const int32 ParentTransformIndex, const TManagedArray<FTransform>& RelativeTransforms, const TManagedArray<TSet<int32>>& Children, TArray<FMatrix>& Transforms);

	/*
	*  Global Matrices of the collection, transforms will be resized to fit
	*/
	template<typename MatrixType>
	void CHAOS_API GlobalMatrices(const TManagedArray<FTransform>& RelativeTransforms, const TManagedArray<int32>& Parents, const TManagedArray<FTransform>& UniformScale, TArray<MatrixType>& Transforms);

	template<typename MatrixType, typename TransformType>
	void CHAOS_API GlobalMatrices(const TManagedArray<TransformType>& RelativeTransforms, const TManagedArray<int32>& Parents, TArray<MatrixType>& Transforms);

	/*
	*  Gets pairs of elements whose bounding boxes overlap.
	*/
	void CHAOS_API GetOverlappedPairs(FGeometryCollection* Collection, int Level, TSet<TTuple<int32, int32>>& OutOverlappedPairs);
	
	/*
	*  Prepare for simulation - placeholder function
	*/
	void 
	CHAOS_API
	PrepareForSimulation(FGeometryCollection* GeometryCollection, bool CenterAtOrigin=true);

	/*
	*  Moves the geometry to center of mass aligned, with option to re-center bones around origin of actor
	*/
	void
	CHAOS_API
	ReCenterGeometryAroundCentreOfMass(FGeometryCollection* GeometryCollection, bool CenterAtOrigin = true);

	void
	CHAOS_API
	FindOpenBoundaries(const FGeometryCollection* GeometryCollection, const float CoincidentVertexTolerance, TArray<TArray<TArray<int32>>> &BoundaryVertexIndices);

	void
	CHAOS_API
	TriangulateBoundaries(FGeometryCollection* GeometryCollection, const TArray<TArray<TArray<int32>>> &BoundaryVertexIndices, bool bWoundClockwise = true, float MinTriangleAreaSq = 1e-4f);

	void
	CHAOS_API
	AddFaces(FGeometryCollection* GeometryCollection, const TArray<TArray<FIntVector>> &Faces);

	/**
	 * Sets new sizes (face/vertex counts) for each geometry
	 * Added vertices and faces contain arbitrary data; other elements remain unchanged
	 */
	void
	CHAOS_API
	ResizeGeometries(FGeometryCollection* GeometryCollection, const TArray<int32>& FaceCounts, const TArray<int32>& VertexCounts, bool bDoValidation = true);

	void
	CHAOS_API
	ComputeCoincidentVertices(const FGeometryCollection* GeometryCollection, const float Tolerance, TMap<int32, int32>& CoincidentVerticesMap, TSet<int32>& VertexToDeleteSet);

	void
	CHAOS_API
	DeleteCoincidentVertices(FGeometryCollection* GeometryCollection, float Tolerance = 1e-2f);

	void
	CHAOS_API
	ComputeZeroAreaFaces(const FGeometryCollection* GeometryCollection, const float Tolerance, TSet<int32>& FaceToDeleteSet);

	void
	CHAOS_API
	DeleteZeroAreaFaces(FGeometryCollection* GeometryCollection, float Tolerance = 1e-4f);

	void
	CHAOS_API
	ComputeHiddenFaces(const FGeometryCollection* GeometryCollection, TSet<int32>& FaceToDeleteSet);

	void
	CHAOS_API
	DeleteHiddenFaces(FGeometryCollection* GeometryCollection);

	void
	CHAOS_API
	ComputeStaleVertices(const FGeometryCollection* GeometryCollection, TSet<int32>& VertexToDeleteSet);

	void
	CHAOS_API
	DeleteStaleVertices(FGeometryCollection* GeometryCollection);

	void
	CHAOS_API
	ComputeEdgeInFaces(const FGeometryCollection* GeometryCollection, TMap<FFaceEdge, int32>& FaceEdgeMap);

	void
	CHAOS_API
	PrintStatistics(const FGeometryCollection* GeometryCollection);

	/*
	* Geometry validation - Checks if the geometry group faces ranges fall within the size of the faces group
	*/
	bool
	CHAOS_API
	HasValidFacesFor(const FGeometryCollection* GeometryCollection, int32 GeometryIndex);

	/*
	* Geometry validation - Checks if the geometry group verts ranges fall within the size of the vertices group
	*/
	bool
	CHAOS_API
	HasValidIndicesFor(const FGeometryCollection* GeometryCollection, int32 GeometryIndex);

	/*
	* Geometry validation - Checks if the geometry group indices appear out of range
	*/
	bool
	CHAOS_API
	HasInvalidIndicesFor(const FGeometryCollection* GeometryCollection, int32 GeometryIndex);

	/*
	* Geometry validation - Checks if there are any faces that are not referenced by the geometry groups
	*/
	bool
	CHAOS_API
	HasResidualFaces(const FGeometryCollection* GeometryCollection);

	/*
	* Geometry validation - Checks if there are any vertices that are not referenced by the geometry groups
	*/
	bool
	CHAOS_API
	HasResidualIndices(const FGeometryCollection* GeometryCollection);

	/*
	* Performs all of the above geometry validation checks
	*/
	bool
	CHAOS_API
	HasValidGeometryReferences(const FGeometryCollection* GeometryCollection);


	/*
	* Computes the order of transform indices so that children in a tree always appear before their parents. Handles forests
	*/
	TArray<int32>
	CHAOS_API
	ComputeRecursiveOrder(const FManagedArrayCollection& Collection);


	// For internal use only 
	namespace Private
	{
		void CHAOS_API GlobalMatrices(const FGeometryDynamicCollection& DynamicCollection, TArray<FTransform>& Transforms);
		void CHAOS_API GlobalMatrices(const FGeometryDynamicCollection& DynamicCollection, TArray<FTransform3f>& Transforms);
		void CHAOS_API GlobalMatrices(const FGeometryDynamicCollection& DynamicCollection, const TArray<int32>& Indices, TArray<FTransform>& OutGlobalTransforms);
		FTransform CHAOS_API GlobalMatrix(const FGeometryDynamicCollection& DynamicCollection, int32 Index);
	}

}
