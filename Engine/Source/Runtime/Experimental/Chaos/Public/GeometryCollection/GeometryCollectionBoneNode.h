// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Set.h"
#include "CoreMinimal.h"

struct /*UE_DEPRECATED(4.22, "Use the split out Level, Parent, Children.. Managed Arrays instead")*/ FGeometryCollectionBoneNode
{
	static const int32 InvalidBone = -1;
	static const int32 InvalidLevel = -1;

	enum ENodeFlags : uint32
	{
		// A node is currently either a geometry node (bit set) or a null node with a transform only (bit zero)
		FS_Geometry = 0x00000001,

		// additional flags
		FS_Clustered = 0x00000002,

		// Gets deleted from world instead of becoming a fractured chunk in the world
		FS_RemoveOnFracture = 0x00000004
	};

	FGeometryCollectionBoneNode(int32 LevelIn, int32 ParentIn, uint32 StatusFlagsIn)
	{ 
		Level = LevelIn;
		Parent = ParentIn;
		StatusFlags = StatusFlagsIn;
		Children.Reset();
	}

	FGeometryCollectionBoneNode()
		: Level(InvalidLevel)
		, Parent(InvalidBone)
		, StatusFlags(ENodeFlags::FS_Geometry)
	{ }

	FGeometryCollectionBoneNode(EForceInit) : Level(InvalidLevel), Parent(InvalidBone), StatusFlags(ENodeFlags::FS_Geometry)
	{
		Children.Reset();
	}

	FGeometryCollectionBoneNode(const FGeometryCollectionBoneNode& Other)
	{
		Level = Other.Level;
		Parent = Other.Parent;
		Children = Other.Children;
		StatusFlags = Other.StatusFlags;
	}

	FORCEINLINE bool IsGeometry() const { return !!(StatusFlags & FS_Geometry); }
	FORCEINLINE bool IsClustered() const { return !!(StatusFlags & FS_Clustered); }
	FORCEINLINE bool IsTransform() const { return !IsGeometry(); }
	FORCEINLINE void SetFlags(uint32 InFlags) { StatusFlags |= InFlags; }
	FORCEINLINE void ClearFlags(uint32 InFlags) { StatusFlags = StatusFlags & ~InFlags; }
	FORCEINLINE bool HasFlags(uint32 InFlags) const { return (StatusFlags & InFlags) != 0; }

	// Serializer.
	friend FArchive& operator<<(FArchive& Ar, FGeometryCollectionBoneNode& Node)
	{
		return Ar << Node.Level << Node.Parent << Node.Children << Node.StatusFlags;
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	/** Assignment */
	FORCEINLINE void operator=(const FGeometryCollectionBoneNode& Other)
	{
		this->Level = Other.Level;
		this->Parent = Other.Parent;
		this->Children = Other.Children;
		this->StatusFlags = Other.StatusFlags;
	}

	FORCEINLINE FString ToString() const 
	{
		FString Result("{");
		
		Result += "Parent : ";
		if( Parent==InvalidBone)
			Result += "Root";
		else
			Result += FString::Printf(TEXT("%d"), Parent);

		Result += ", Level : ";
		if (Level == InvalidLevel)
			Result += "None";
		else
			Result += FString::Printf(TEXT("%d"), Level);

		Result += ", Children [";
		for (auto& Elem : Children)
		{
			Result += FString::Printf(TEXT("%d, "), Elem);
		}

		Result += "]}";
		return Result;
	}

	/** Level in Hierarchy : 0 is usually but not necessarily always the root */
	int32 Level;

	/** Parent bone index : use InvalidBone for root parent */
	int32 Parent;

	/** Child bone indices */
	TSet<int32> Children;

	/** Flags to store any state for each node */
	uint32 StatusFlags;
}; 
