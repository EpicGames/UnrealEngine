// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayCollection.h"

class FGeometryCollection;

class FGeometryCollectionClusteringUtility
{
public:
	/**
	* Creates a cluster in the node hierarchy by re-parenting the Selected Bones off a new node in the hierarchy
	* It makes more sense to think that the Selected Bones are all at the same level in the hierarchy however
	* it will re-parent multiple levels at the InsertAtIndex location bone
	*
	* e.g. if you have a flat chunk hierarchy after performing Voronoi fracturing
	*   L0         Root
	*               |
	*          ----------
	*          |  |  |  |
	*   L1     A  B  C  D
	*
	* Cluster A & B at insertion point A, results in
	*   L0         Root
	*               |
	*          ----------
	*          |     |  |
	*   L1     E     C  D
	*          |
	*         ----
	*         |  |
	*   L2    A  B
	*
	* Node E has no geometry of its own, only a transform by which to control A & B as a single unit
	* 
	* @return Index of the New Node
	*/
	static CHAOS_API int32 ClusterBonesUnderNewNode(FGeometryCollection* GeometryCollection, const int32 InsertAtIndex, const TArray<int32>& SelectedBones, bool CalcNewLocalTransform, bool Validate = true);

	// Same as ClusterBonesUnderNewNode, but specify the parent of the new node instead of a sibling
	static CHAOS_API int32 ClusterBonesUnderNewNodeWithParent(FGeometryCollection* GeometryCollection, const int32 ParentOfNewNode, const TArray<int32>& SelectedBones, bool CalcNewLocalTransform, bool Validate = true);

	/** Cluster all existing bones under a new root node, so there is now only one root node and a completely flat hierarchy underneath it */
	static CHAOS_API void ClusterAllBonesUnderNewRoot(FGeometryCollection* GeometryCollection, FName RootName = NAME_None, bool bUpdateChildBoneNames = true);

	/** Cluster all source bones under an existing node, algorithm chooses best node to add to 'closest to root' */
	static CHAOS_API void ClusterBonesUnderExistingNode(FGeometryCollection* GeometryCollection, const TArray<int32>& SourceElements);

	/** Cluster all source bones under an existing node, existing node is specifiedas MergeNode */
	static CHAOS_API void ClusterBonesUnderExistingNode(FGeometryCollection* GeometryCollection, int32 MergeNode, const TArray<int32>& SourceElements);

	/** Clusters using one of ClusterBonesUnderNewNode or ClusterBonesUnderExistingNode based on the context of what is being clustered */
	static CHAOS_API void ClusterBonesByContext(FGeometryCollection* GeometryCollection, int32 MergeNode, const TArray<int32>& SourceElementsIn);

	/** Returns true if bone hierarchy contains more than one root node */
	static CHAOS_API bool ContainsMultipleRootBones(FGeometryCollection* GeometryCollection);

	/** Finds the root bone in the hierarchy, the one with an invalid parent bone index */
	static CHAOS_API void GetRootBones(const FGeometryCollection* GeometryCollection, TArray<int32>& RootBonesOut);

	/** Return true if the specified bone is a root bone */
	static CHAOS_API bool IsARootBone(const FGeometryCollection* GeometryCollection, int32 InBone);
	
	/** Finds all Bones in same cluster as the one specified */
	static CHAOS_API void GetClusteredBonesWithCommonParent(const FGeometryCollection* GeometryCollection, int32 SourceBone, TArray<int32>& BonesOut);

	/** Get list of all bones above the specified hierarchy level */
	static CHAOS_API void GetBonesToLevel(const FGeometryCollection* GeometryCollection, int32 Level, TArray<int32>& BonesOut, bool bOnlyClusteredOrRigid = true, bool bSkipFiltered = true);
	
	/** Get list of child bones down from the source bone below the specified hierarchy level */
	static CHAOS_API void GetChildBonesFromLevel(const FGeometryCollection* GeometryCollection, int32 SourceBone, int32 Level, TArray<int32>& BonesOut);

	/** Get list of child bones down from the source bone at the specified hierarchy level */
	static CHAOS_API void GetChildBonesAtLevel(const FGeometryCollection* GeometryCollection, int32 SourceBone, int32 Level, TArray<int32>& BonesOut);

	/** Recursively Add all children to output bone list from source bone down to the leaf nodes */
	static CHAOS_API void RecursiveAddAllChildren(const TManagedArray<TSet<int32>>& Children, int32 SourceBone, TArray<int32>& BonesOut);

	/**
	 * Search hierarchy for the parent of the specified bone, where the parent exists at the given level in the hierarchy.
	 * If bSkipFiltered, also skip to parents of any rigid/embedded nodes that will be filtered from the outliner.
	 */
	static CHAOS_API int32 GetParentOfBoneAtSpecifiedLevel(const FGeometryCollection* GeometryCollection, int32 SourceBone, int32 Level, bool bSkipFiltered = false);

	/**
	* Maintains the bone naming convention of
	*  Root "Name"
	*  Level 1 "Name_001", "Name_0002", ...
	*  Level 2 children of "Name_0001" are "Name_0001_0001", "Name_0001_0002",.. etc
	* from the given bone index down through the hierarchy to the leaf nodes
	*/
	static CHAOS_API void RecursivelyUpdateChildBoneNames(int32 BoneIndex, const TManagedArray<TSet<int32>>& Children, TManagedArray<FString>& BoneNames, bool OverrideBoneNames = false);
	
	/** Recursively update the hierarchy level of all the children below this bone */
	static CHAOS_API void UpdateHierarchyLevelOfChildren(FGeometryCollection* GeometryCollection, int32 ParentElement);
	static CHAOS_API void UpdateHierarchyLevelOfChildren(FManagedArrayCollection& InCollection, int32 ParentElement);

	/** Collapse hierarchy at specified level */
	static CHAOS_API void CollapseLevelHierarchy(int8 Level, FGeometryCollection* GeometryCollection);

	/** Collapse hierarchy of selected bones at specified level */
	static CHAOS_API void CollapseSelectedHierarchy(int8 Level, const TArray<int32>& SelectedBones, FGeometryCollection* GeometryCollection);

	/** reparent source bones under root bone */
	static CHAOS_API void ClusterBonesUnderExistingRoot(FGeometryCollection* GeometryCollection, const TArray<int32>& SourceElements);

	/** moved the selected bones closer to the root */
	static CHAOS_API void CollapseHierarchyOneLevel(FGeometryCollection* GeometryCollection, TArray<int32>& SourceElements);

	/** return true of Test Node exists on tree branch specified by TreeElement Node*/
	static CHAOS_API bool NodeExistsOnThisBranch(const FGeometryCollection* GeometryCollection, int32 TestNode, int32 TreeElement);

	/** Rename a bone node, will automatically update all child node names if requested (current default) */
	static CHAOS_API void RenameBone(FGeometryCollection* GeometryCollection, int32 BoneIndex, const FString& NewName, bool UpdateChildren = true);

	/** return an array of all child leaf nodes below the specified node. If bOnlyRigids is true, the first Rigid node dound is considered a leaf, regardless of an children it might have. */
	static CHAOS_API void GetLeafBones(const FManagedArrayCollection* GeometryCollection, int BoneIndex, bool bOnlyRigids, TArray<int32>& LeafBonesOut);

	/** move the selected node up a level in direction of root */
	static CHAOS_API void MoveUpOneHierarchyLevel(FGeometryCollection* GeometryCollection, const TArray<int32>& SelectedBones);

	/** Find the lowest common ancestor index of currently selected nodes. Returns INDEX_NODE if there is no common ancestor. */
	static CHAOS_API int32 FindLowestCommonAncestor(const FManagedArrayCollection* GeometryCollection, const TArray<int32>& SelectedBones);

	/** Delete any cluster nodes discovered to have no children. Returns true if any nodes were removed. */
	static CHAOS_API bool RemoveDanglingClusters(FGeometryCollection* GeometryCollection);

	/** Simplify the geometry collection by removing cluster nodes w/ a parent and only one child, re-parenting the child to its grand-parent. Returns true if any nodes were removed. */
	static CHAOS_API bool RemoveClustersOfOnlyOneChild(FGeometryCollection* GeometryCollection);

	static CHAOS_API void ValidateResults(FGeometryCollection* GeometryCollection);

	static CHAOS_API int32 PickBestNodeToMergeTo(const FManagedArrayCollection* Collection, const TArray<int32>& SourceElements);

private:
	// #todo: intend to remove reliance on custom attributes for slider by making use of Rest/Dynamic collections
	static void ResetSliderTransforms(TManagedArray<FTransform>& ExplodedTransforms, TManagedArray<FTransform>& Transforms);
	
	static void RecursivelyUpdateHierarchyLevelOfChildren(TManagedArray<int32>& Levels, const TManagedArray<TSet<int32>>& Children, int32 ParentElement);

	static int32 FindLowestCommonAncestor(const FManagedArrayCollection* GeometryCollection, int32 N0, int32 N1);
};
