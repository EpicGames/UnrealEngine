// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionSimulationCoreTypes.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
#include "UObject/ObjectMacros.h"

namespace Chaos { class FErrorReporter; }

namespace Chaos
{
	class FTriangleMesh;
	class FLevelSet;

	template <typename T, int d>
	class TParticles;
	using FParticles = TParticles<FReal, 3>;
}



class FCollisionStructureManager
{
public:
	CHAOS_API FCollisionStructureManager();
	virtual ~FCollisionStructureManager() = default;

	typedef TArray<Chaos::FVec3> FPoints;
	typedef Chaos::FBVHParticles FSimplicial;
	typedef Chaos::FImplicitObject FImplicit;

	/**
	 * Culls particles inside \p Implicit, and coincident particles (defined by 
	 * being within 1/20'th of the size of the domain from one another), truncates
	 * at \p CollisionParticlesMaxInput, and returns a bounding volume hierarchy
	 * of the remainder.
	 */
	static CHAOS_API FSimplicial* NewSimplicial(
		const Chaos::FParticles& Vertices,
		const Chaos::FTriangleMesh& TriMesh,
		const Chaos::FImplicitObject* Implicit,
		const int32 CollisionParticlesMaxInput);
	
	/**
	 * Culls particles by importance (See \c FTriangleMesh::GetVertexImportanceOrdering()),
	 * and returns a bounding volume hierarchy of the remainder.
	 */
	static CHAOS_API FSimplicial* NewSimplicial(
		const Chaos::FParticles& AllParticles,
		const TManagedArray<int32>& BoneMap,
		const ECollisionTypeEnum CollisionType,
		Chaos::FTriangleMesh& TriMesh,
		const float CollisionParticlesFraction);

	/**
	 * Calls \c SetDoCollide(false) and \c SetConvex(false) on \p Implicit if 
	 * \p CollisionType is \c ECollisionTypeEnum::Chaos_Surface_Volumetric.
	 */
	static CHAOS_API void UpdateImplicitFlags(
		FImplicit* Implicit, 
		const ECollisionTypeEnum CollisionType);

	/**
	 * Build a box, sphere, or level set based on \p ImplicitType.
	 * 
	 *	\p ErrorReporter - level set only
	 *	\p MeshParticles - level set only
	 *	\p TriMesh - level set only
	 *	\p CollisionBounds - box and level set
	 *	\p Radius - sphere only
	 *	\p MinRes - level set only
	 *	\p MaxRes - level set only
	 *	\p CollisionObjectReduction - shrink percentage; value of 10 reduces by 
	 *	   10%, 0 does nothing, 100 shrinks to zero.
	 *	\p CollisionType - param forwarded to \c UpdateImplictFlags().
	 *	\p ImplicitType - type of implicit shape to build.
	 */
	static CHAOS_API Chaos::FImplicitObjectRef NewImplicit(
		Chaos::FErrorReporter ErrorReporter,
		const Chaos::FParticles& MeshParticles,
		const Chaos::FTriangleMesh& TriMesh,
		const FBox& CollisionBounds,
		const Chaos::FReal Radius,
		const int32 MinRes,
		const int32 MaxRes,
		const float CollisionObjectReduction,
		const ECollisionTypeEnum CollisionType,
		const EImplicitTypeEnum ImplicitType);

	static CHAOS_API Chaos::FImplicitObjectRef NewImplicitBox(
		const FBox& CollisionBounds,
		const float CollisionObjectReduction,
		const ECollisionTypeEnum CollisionType);

	static CHAOS_API Chaos::FImplicitObjectRef NewImplicitSphere(
		const Chaos::FReal Radius,
		const float CollisionObjectReduction,
		const ECollisionTypeEnum CollisionType);

	static CHAOS_API Chaos::FImplicitObjectRef NewImplicitCapsule(
		const Chaos::FReal Radius,
		const Chaos::FReal Length,
		const float CollisionObjectReduction,
		const ECollisionTypeEnum CollisionType);

	static CHAOS_API Chaos::FImplicitObjectRef NewImplicitCapsule(
		const FBox& CollisionBounds,
		const float CollisionObjectReduction,
		const ECollisionTypeEnum CollisionType);

	static CHAOS_API Chaos::FImplicitObjectRef NewImplicitConvex(
		const TArray<int32>& ConvexIndices,
		const TManagedArray<Chaos::FConvexPtr>* ConvexGeometry,
		const ECollisionTypeEnum CollisionType,
		const FTransform& MassTransform,
		const Chaos::FReal CollisionMarginFraction,
		const float CollisionObjectReduction);

	static CHAOS_API Chaos::FImplicitObjectRef NewImplicitLevelset(
		Chaos::FErrorReporter ErrorReporter,
		const Chaos::FParticles& MeshParticles,
		const Chaos::FTriangleMesh& TriMesh,
		const FBox& CollisionBounds,
		const int32 MinRes,
		const int32 MaxRes,
		const float CollisionObjectReduction,
		const ECollisionTypeEnum CollisionType);

	static CHAOS_API Chaos::FLevelSetRef NewLevelset(
		Chaos::FErrorReporter ErrorReporter,
		const Chaos::FParticles& MeshParticles,
		const Chaos::FTriangleMesh& TriMesh,
		const FBox& CollisionBounds,
		const int32 MinRes,
		const int32 MaxRes,
		const ECollisionTypeEnum CollisionType);

	static CHAOS_API FVector CalculateUnitMassInertiaTensor(
		const FBox& BoundingBox,
		const Chaos::FReal Radius,
		const EImplicitTypeEnum ImplicitType);

	static CHAOS_API Chaos::FReal CalculateVolume(
		const FBox& BoundingBox,
		const Chaos::FReal Radius,
		const EImplicitTypeEnum ImplicitType);
};
