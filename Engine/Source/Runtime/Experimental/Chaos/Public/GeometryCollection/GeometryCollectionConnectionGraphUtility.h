// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "ChaosSolverConfiguration.h"

class FGeometryCollection;

class FGeometryCollectionConnectionGraphGenerator
{
public:
	static CHAOS_API void UpdateConnectivityGraph(FGeometryCollection& Collection, int32 ClusterTransformIndex, EClusterUnionMethod ConnectionMethod);
	
private:
	using FConnections = TSet<int32>;
	using FConnectionGraph = TArray<FConnections>;
	struct FVoronoiNeighbors
	{
		TArray<FVector> Points;
		TArray<TArray<int>> Neighbors;
	};
	
	static FConnectionGraph ComputeConnectivityGraph(const FGeometryCollection& Collection, int32 ClusterTransformIndex, EClusterUnionMethod ConnectionMethod);
	static void ConnectChildren(FConnectionGraph& Graph, const TArray<int32>& Children, int32 ChildIndexA, int32 ChildIndexB);
	static FVoronoiNeighbors ComputeVoronoiNeighbors(const FGeometryCollection& Collection, int32 ClusterTransformIndex);
	static void ComputeConnectivityGraphUsingDelaunayTriangulation(FConnectionGraph& Graph, const FGeometryCollection& Collection, int32 ClusterTransformIndex);
	static void FixConnectivityGraphUsingDelaunayTriangulation(FConnectionGraph& Graph, const FGeometryCollection& Collection, int32 ClusterTransformIndex);
	static void UpdateConnectivityGraphUsingPointImplicit(FConnectionGraph& Graph, const FGeometryCollection& Collection, int32 ClusterTransformIndex);
	static void CommitToCollection(FConnectionGraph& Graph, FGeometryCollection& Collection, int32 ClusterTransformIndex);
};