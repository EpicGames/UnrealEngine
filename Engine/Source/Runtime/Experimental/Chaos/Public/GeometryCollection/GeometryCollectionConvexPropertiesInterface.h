// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"

class FGeometryCollection;

class FGeometryCollectionConvexPropertiesInterface : public FManagedArrayInterface
{
public :
	typedef FManagedArrayInterface Super;
	using FManagedArrayInterface::ManagedCollection;

	// Convex Properties Group Name
	static CHAOS_API const FName ConvexPropertiesGroup; 
	// Attribute
	static CHAOS_API const FName ConvexIndexAttribute;
	// Attribute
	static CHAOS_API const FName ConvexEnable;
	// Attribute
	static CHAOS_API const FName ConvexFractionRemoveAttribute;
	// Attribute
	static CHAOS_API const FName ConvexSimplificationThresholdAttribute;
	// Attribute
	static CHAOS_API const FName ConvexCanExceedFractionAttribute;
	// Attribute
	static CHAOS_API const FName ConvexRemoveOverlapsMethodAttribute;
	// Attribute
	static CHAOS_API const FName ConvexRemoveOverlapsShrinkAttribute;


	struct FConvexCreationProperties {
		bool Enable = true;
		float FractionRemove = 0.5f;
		float SimplificationThreshold = 10.0f;
		float CanExceedFraction = 0.5f;
		EConvexOverlapRemoval RemoveOverlaps = EConvexOverlapRemoval::All;
		float OverlapRemovalShrinkPercent = 0.0f;
	};

	CHAOS_API FGeometryCollectionConvexPropertiesInterface(FManagedArrayCollection* InGeometryCollection);

	CHAOS_API void InitializeInterface() override;

	CHAOS_API void CleanInterfaceForCook() override;
	
	CHAOS_API void RemoveInterfaceAttributes() override;

	CHAOS_API FConvexCreationProperties GetConvexProperties(int TransformGroupIndex = INDEX_NONE) const;
	CHAOS_API void SetConvexProperties(const FConvexCreationProperties&, int TransformGroupIndex = INDEX_NONE);
private:

	void SetDefaultProperty();
	FConvexCreationProperties GetDefaultProperty() const;
};

