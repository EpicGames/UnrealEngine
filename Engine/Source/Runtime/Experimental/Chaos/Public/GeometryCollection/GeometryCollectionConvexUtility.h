// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Chaos/ImplicitFwd.h"
#include "CompGeom/ConvexDecomposition3.h"

#include "GeometryCollectionConvexUtility.generated.h"

class FGeometryCollection;


UENUM()
enum class EConvexOverlapRemoval : int32
{
	// Do not remove overlaps between convex hulls
	None = 0,
	// Remove all overlaps between neighboring convex hulls
	All = 1,
	// Only remove overlaps on convex hulls of clusters, ignoring leaf-leaf overlaps
	OnlyClusters = 2,
	// Only remove overlaps between overlapping clusters, ignoring leaf-leaf and cluster-leaf overlaps
	OnlyClustersVsClusters = 3
};

UENUM()
enum class EGenerateConvexMethod : uint8
{
	// Convert from external collision shapes (if available)
	ExternalCollision,
	// Compute all convex hulls from geometry
	ComputedFromGeometry,
	// Intersect external collision shapes with computed convex hulls
	IntersectExternalWithComputed
};

// Controls which bones are allowed to merge their convex hulls
UENUM(meta=(DisplayName="Allow Convex Bone Merge Method"))
enum class EAllowConvexMergeMethod : uint8
{
	// Only allow merging convex hulls of bones that are in proximity
	ByProximity UMETA(DisplayName = "By Bone Proximity"),
	// Allow any pair of convex hulls to merge
	Any
};

// Controls which individual convex hulls are considered for merges
UENUM()
enum class EConvexHullProximityFilter : uint8
{
	// Do not require convex hull geometry to be in proximity to merge
	// Note: May still use bone proximity to limit possible merges in the cluster merge case, via the Allow Convex Bone Merge Method options
	None,
	// Only consider convex hull merges if their bounding boxes are in proximity
	BoundingBox
};

namespace UE::GeometryCollectionConvexUtility
{
	// Used to pass computed convex hulls along with metadata
	// Note these hulls are typically computed in a shared coordinate space, in contrast to the final hulls on the geometry collection which are in the local space of each bone
	struct FConvexHulls
	{
		TArray<::Chaos::FConvexPtr> Hulls;

		// Mapping from geometry collection bones indices to indices in the Hulls array. A Set is used to support multiple hulls per bone.
		TArray<TSet<int32>> TransformToHullsIndices;

		// Percent the hulls are down-scaled
		double OverlapRemovalShrinkPercent;

		// The pivots used for scaling by OverlapRemovalShrinkPercent
		TArray<FVector> Pivots;
	};

	// Compute surface area and contact estimates for two convex hulls in the same local space, with the second hull optionally expanded
	void CHAOS_API HullIntersectionStats(const ::Chaos::FConvex* HullA, const ::Chaos::FConvex* HullB, float HullBExpansion, float& OutArea, float& OutMaxArea, float& OutSharpContact, float& OutMaxSharpContact);

	// Compute the intersection of ClipHull and UpdateHull, with ClipHull optionally offset, and place the result in ResultHull. Note ResultHull can point to the same FConvex as UpdateHull.
	// If optional transforms are provided, ClipHull will be transformed into the local space of UpdateHull, and finally transformed by UpdateToResultTransform. Otherwise, all FConvex are assumed to be in the same coordinate space.
	void CHAOS_API IntersectConvexHulls(::Chaos::FConvex* ResultHull, const ::Chaos::FConvex* ClipHull, float ClipHullOffset, const ::Chaos::FConvex* UpdateHull,
		const FTransform* ClipHullTransform = nullptr, const FTransform* UpdateHullTransform = nullptr, const FTransform* UpdateToResultTransform = nullptr, double SimplificationDistanceThreshold = 0.0);

	// Get the existing convex hulls from the collection, transformed to the global/shared space of the overall collection
	// @param bLeafOnly		Only include the convex hulls of leaf (rigid) nodes
	bool CHAOS_API GetExistingConvexHullsInSharedSpace(const FManagedArrayCollection* Collection, FConvexHulls& OutConvexHulls, bool bLeafOnly = false);

}

class FGeometryCollectionConvexUtility
{
public:

	struct FGeometryCollectionConvexData
	{
		TManagedArray<TSet<int32>>& TransformToConvexIndices;
		TManagedArray<Chaos::FConvexPtr>& ConvexHull;
	};

	/** Ensure that convex hull data exists for the Geometry Collection and construct it if not (or if some data is missing. */
	static CHAOS_API FGeometryCollectionConvexData GetValidConvexHullData(FGeometryCollection* GeometryCollection);

	/** Get convex hull data for the Geometry Collection if it is present */
	static CHAOS_API TOptional<FGeometryCollectionConvexData> GetConvexHullDataIfPresent(FManagedArrayCollection* GeometryCollection);

	/** @return true if convex hull data is present */
	static CHAOS_API bool HasConvexHullData(const FManagedArrayCollection* GeometryCollection);

	/**
	 Create non-overlapping convex hull data for all transforms in the geometry collection (except transforms where it would be better to just use the hulls of the children) 

	 @param GeometryCollection					The collection to add convex hulls to
	 @param FractionAllowRemove					The fraction of a convex body we can cut away to remove overlaps with neighbors, before we fall back to using the hulls of the children directly.  (Does not affect leaves of hierarchy)
	 @param SimplificationDistanceThreshold		Approximate minimum distance between vertices, below which we remove vertices to generate a simpler convex shape.  If 0.0, no simplification will occur.
	 @param CanExceedFraction					The fraction by which the convex body volume on a cluster can exceed the volume of the geometry under that cluster (a value of 1 == exceed by 100% == convex hull has 2x the volume of the geometry)
	 @param OverlapRemovalMethod				If bRemoveOverlaps, control which overlaps are removed
	 @param OverlapRemovalShrinkPercent			Compute overlaps based on objects shrunk by this percentage, so objects that would not overlap with this value set as their 'Collision Object Reduction Percentage' will not be cut
	 @param ComputedLeafHullsToModify			Optional pre-computed hulls for geometry of rigid leaves, in the top-level coordinate space of the geometry collection. If passed in, the data will be updated and moved.
	 */
	static CHAOS_API FGeometryCollectionConvexData CreateNonOverlappingConvexHullData(FGeometryCollection* GeometryCollection, double FractionAllowRemove = .3, double SimplificationDistanceThreshold = 0.0, double CanExceedFraction = .5, 
		EConvexOverlapRemoval OverlapRemovalMethod = EConvexOverlapRemoval::All, double OverlapRemovalShrinkPercent = 0.0, UE::GeometryCollectionConvexUtility::FConvexHulls* ComputedLeafHullsToModify = nullptr);

	struct FClusterConvexHullSettings
	{
		FClusterConvexHullSettings() = default;
		FClusterConvexHullSettings(int32 ConvexCount, double ErrorToleranceInCm, bool bUseExternalCollisionIfAvailable = true)
			: ConvexCount(ConvexCount), ErrorToleranceInCm(ErrorToleranceInCm), bUseExternalCollisionIfAvailable(bUseExternalCollisionIfAvailable)
		{}

		int32 ConvexCount = 4;
		double ErrorToleranceInCm = 0.0;
		bool bUseExternalCollisionIfAvailable = true;
		// Option to apply the same hull merging settings to leaf hulls
		bool bAllowMergingLeafHulls = false;
		// How to allow merges between hulls on different bones
		EAllowConvexMergeMethod AllowMergesMethod = EAllowConvexMergeMethod::ByProximity;
		// How to filter by proximity for all merges (including within bones)
		EConvexHullProximityFilter ProximityFilter = EConvexHullProximityFilter::None;
		float ProximityDistanceThreshold = 0.1f;

		UE::Geometry::FSphereCovering* EmptySpace = nullptr;
	};

	static CHAOS_API void GenerateClusterConvexHullsFromChildrenHulls(FGeometryCollection& Collection, const FClusterConvexHullSettings& Settings, const TArrayView<const int32> TransformSubset);
	static CHAOS_API void GenerateClusterConvexHullsFromChildrenHulls(FGeometryCollection& Collection, const FClusterConvexHullSettings& Settings);
	static CHAOS_API void GenerateClusterConvexHullsFromLeafHulls(FGeometryCollection& Collection, const FClusterConvexHullSettings& Settings, const TArrayView<const int32> OptionalTransformSubset);
	static CHAOS_API void GenerateClusterConvexHullsFromLeafHulls(FGeometryCollection& Collection, const FClusterConvexHullSettings& Settings);

	struct FMergeConvexHullSettings
	{
		int32 MaxConvexCount = -1;
		double ErrorToleranceInCm = 0.0;
		// Optional externally-provided empty space, to be used for all hull merges
		UE::Geometry::FSphereCovering* EmptySpace = nullptr;

		// Optional settings to compute targeted empty space per-bone
		UE::Geometry::FNegativeSpaceSampleSettings* ComputeEmptySpacePerBoneSettings = nullptr;

		EConvexHullProximityFilter ProximityFilter = EConvexHullProximityFilter::None;
		float ProximityDistanceThreshold = 0.1f;
	};
	// Merge convex hulls that are currently on each (selected) transform. If convex hulls are not present, does nothing.
	// @params OptionalSphereCoveringOut		If non-null, will be filled with spheres from all used sphere covering.
	static CHAOS_API void MergeHullsOnTransforms(FManagedArrayCollection& Collection, const FGeometryCollectionConvexUtility::FMergeConvexHullSettings& Settings, bool bRestrictToSelection, const TArrayView<const int32> OptionalTransformSelection,
		UE::Geometry::FSphereCovering* OptionalSphereCoveringOut = nullptr);

	// Additional settings for filtering when the EGenerateConvexMethod::IntersectExternalWithComputed is applied
	struct FIntersectionFilters
	{
		FIntersectionFilters() : OnlyIntersectIfComputedIsSmallerFactor(1.0), MinExternalVolumeToIntersect(0.0)
		{}
		double OnlyIntersectIfComputedIsSmallerFactor;
		double MinExternalVolumeToIntersect;
	};

	// Settings to control convex decompositions. Note the default values are set to *not* perform any decomposition
	struct FConvexDecompositionSettings
	{
		FConvexDecompositionSettings() = default;

		///
		/// These settings are filters that control whether convex decomposition is performed
		///
		
		// If greater than zero, the minimum geometry volume to consider for convex decomposition
		float MinGeoVolumeToDecompose = 0.f;

		// If the geo volume / hull volume ratio is greater than this, do not consider convex decomposition
		float MaxGeoToHullVolumeRatioToDecompose = 1.f;


		///
		/// If we do perform a convex decomposition, these settings control the decomposition process
		///
		
		// If > 0, specify the maximum number of convex hulls to create.  Hull merges will be attempted until this number of hulls remains, or no merges are possible.
		int32 MaxHullsPerGeometry = 1;

		// Stop splitting when hulls have error less than this (expressed in cm; will be cubed for volumetric error). Overrides NumOutputHulls if non-zero.
		float ErrorTolerance = 0.f;

		// Optionally specify a minimum thickness (in cm) for convex parts; parts below this thickness will always be merged away. Will merge hulls with greater error than ErrorTolerance when needed.
		float MinThicknessTolerance = 0.f;

		// How far to go beyond the target number of hulls for initial decomposition (before merges) -- larger values will require more computation but can find better convex decompositions
		int32 NumAdditionalSplits = 4;

		// How much to thicken a hull on failure to construct -- to allow 3D decomposition of planar or degenerate meshes and parts-of-meshes
		//~ Note Chaos::FConvex has its own process to thicken hulls that are smaller than .1 units across (specified by Chaos::TriQuadPrismInflation)
		//~ Applying some smaller thickening earlier in the decomposition process just allows degenerate parts to make it through the decomposition.
		float ThickenHullOnFailure = UE_KINDA_SMALL_NUMBER;

		/** Whether to use a sphere cover to define negative space that should not be covered by convex hulls */
		bool bProtectNegativeSpace = false;

		/** When protecting negative space, only look for space that is connected out to the convex hull. This removes inaccessable internal negative space from consideration. */
		bool bOnlyConnectedToHull = true;

		/** Amount of space to leave between convex hulls and protected negative space */
		float NegativeSpaceTolerance = 2.0f;

		/** Spheres smaller than this are not included in the negative space */
		float NegativeSpaceMinRadius = 10.0f;

		/** Directly-specified empty space to use *in addition* to any that would be added via bProtectNegativeSpace. If provided, this covering will be used even if bProtectNegativeSpace is false. */
		const UE::Geometry::FSphereCovering* CustomEmptySpace = nullptr;

		/** Transform to apply to the custom empty space sphere covering, relative to the top-level/world space of the geometry collection. */
		FTransform CustomEmptySpaceWorldTransform = FTransform::Identity;
	};

	// Settings to control how convex hulls are generated for rigid/leaf nodes, from the geometry and/or imported collision shapes
	struct FLeafConvexHullSettings
	{
		FLeafConvexHullSettings() : SimplificationDistanceThreshold(1.0), GenerateMethod(EGenerateConvexMethod::ExternalCollision)
		{}

		explicit FLeafConvexHullSettings(double SimplificationDistanceThreshold, EGenerateConvexMethod GenerateMethod = EGenerateConvexMethod::ExternalCollision) :
			SimplificationDistanceThreshold(SimplificationDistanceThreshold),
			GenerateMethod(GenerateMethod)
		{}

		double SimplificationDistanceThreshold;
		EGenerateConvexMethod GenerateMethod;

		// Intersection filters only apply if GenerateMethod == EGenerateConvexMethod::IntersectExternalWithComputed
		FIntersectionFilters IntersectFilters;

		// Convex decomposition settings, applied to convex hulls generated from geometry
		FConvexDecompositionSettings DecompositionSettings;

		// For GenerateMethod == IntersectExternalWithComputed, whether to compute the intersection before computing convex hulls
		// Note: It seems more logical for this setting to be true, but we expose it as an option because in some special cases the results when it was false were preferred
		bool bComputeIntersectionsBeforeHull = true;
	};

	// Computed sphere covering, and optional associated transform/bone
	struct FSphereCoveringInfo
	{
		FTransform Transform = FTransform::Identity;
		int32 SourceBoneIndex = INDEX_NONE;
		UE::Geometry::FSphereCovering SphereCovering;
	};
	
	static CHAOS_API void GenerateLeafConvexHulls(
		FGeometryCollection& Collection, bool bRestrictToSelection, 
		const TArrayView<const int32> TransformSubset, const FLeafConvexHullSettings& Settings,
		TArray<FSphereCoveringInfo>* OutComputedNavigableSpheres = nullptr
	);

	/** Returns the convex hull of the vertices contained in the specified geometry. */
	static CHAOS_API Chaos::FConvexPtr GetConvexHull(const FGeometryCollection* GeometryCollection, int32 GeometryIndex);

	UE_DEPRECATED(5.4, "Please Use GetConvexHull instead.")
	static TUniquePtr<Chaos::FConvex> FindConvexHull(const FGeometryCollection* GeometryCollection, int32 GeometryIndex)
	{
		check(false);
		return nullptr;
	}

	/** 
	 * Copy convex hulls on the specified transforms from a source collection to the collection 
	 * @param GeometryCollection The collection to copy convex hulls to
	 * @param TransformIndices The transforms on the GeometryCollection to update w/ copied convex hulls. Must be same length as SourceTransformIndices array.
	 * @param SourceCollection The collection to copy convex hulls from
	 * @param SourceTransformIndices The transforms on the SourceCollection to copy from. Must be same length as TransformIndices array.
	 * @param bSkipIfEmpty Whether to skip copying from transforms that have no convex hulls
	 */
	static CHAOS_API bool CopyConvexHulls(FManagedArrayCollection& Collection, const TArray<int32>& TransformIndices,
		const FManagedArrayCollection& SourceCollection, const TArray<int32>& SourceTransformIndices, bool bSkipIfEmpty = false);

	/** Delete the convex hulls pointed at by the transform indices provided. */
	static CHAOS_API void RemoveConvexHulls(FManagedArrayCollection* GeometryCollection, const TArray<int32>& TransformsToClearHullsFrom);

	/** Delete the convex hulls that are null */
	static CHAOS_API void RemoveEmptyConvexHulls(FManagedArrayCollection& GeometryCollection);

	/** Set default values for convex hull related managed arrays. */
	static CHAOS_API void SetDefaults(FGeometryCollection* GeometryCollection, FName Group, uint32 StartSize, uint32 NumElements);

	/** Get the HasCustomConvex flags.  If they're missing, either add them (if bAddIfMissing) or return nullptr */
	static CHAOS_API TManagedArray<int32>* GetCustomConvexFlags(FGeometryCollection* GeometryCollection, bool bAddIfMissing = false);

	/** @return true if the GeometryCollection has convex data with no null pointers and no invalid indices */
	static CHAOS_API bool ValidateConvexData(const FManagedArrayCollection* GeometryCollection);

	/** Set Volume and Size attributes on the Collection (will be called by CreateNonOverlappingConvexHullData -- Volumes must be up to date for convex calc) */
	static CHAOS_API void SetVolumeAttributes(FManagedArrayCollection* Collection);

	/**
	 * Copy convex hulls from *below* FromTransformIdx over to all live at ToTransformIdx.
	 * The two geometry collections can be the same but do not need to be.
	 * Note: This will also set HasCustomConvex flags for the ToTransformIdx.
	 * 
	 * @param FromCollection	The collection to copy from
	 * @param FromTransformIdx	The transform indices whose child nodes should be copied from (or, if it's a leaf, the hull on the leaf will be copied instead)
	 * @param ToCollection		The collection to copy to (can be the same as FromCollection)
	 * @param ToTransformIdx	The transform indices whose convexes will be replaced with *copies* of the child convexes.  Must be same length as FromTransformIdx; can be the same array.
	 * @param bLeafOnly			If true, we will only collect convexes from leaf bones, not from clusters.
	 */
	static CHAOS_API void CopyChildConvexes(const FGeometryCollection* FromCollection, const TArrayView<const int32>& FromTransformIdx, FGeometryCollection* ToCollection, const TArrayView<const int32>& ToTransformIdx, bool bLeafOnly);

	struct FTransformedConvex
	{
		Chaos::FConvexPtr Convex;
		FTransform Transform;
	};

	// Compute just the hulls of the leaf / rigid nodes that hold geometry directly, with no cluster hulls and no overlap removal by cutting
	// This is an initial step of several algorithms: The CreateNonOverlappingConvexHullData function as well as convex-based proximity detection (TODO: and the auto-embed algorithm?)
	// (TODO: Make auto-embed use this instead of the full hulls?)
	// @param GlobalTransformArray				GeometryCollection's transforms to global space, as computed by GeometryCollectionAlgo::GlobalMatrices
	// @param SkipBoneFn						Indicator function returns true for transform indices that do not need a convex hull to be computed, if non-null
	// @param OptionalDecompositionSettings		Optionally generate multiple convex hulls per transform, if these settings are provided
	// @param OptionalIntersectConvexHulls		Convex hulls to optionally intersect with the computed hulls, so that the resulting hulls will not extend outside of these provided hulls.
	// @param OptionalTransformToIntersectHulls	Mapping from transforms to the OptionalIntersectConvexHulls
	static CHAOS_API UE::GeometryCollectionConvexUtility::FConvexHulls ComputeLeafHulls(FGeometryCollection* GeometryCollection, const TArray<FTransform>& GlobalTransformArray, double SimplificationDistanceThreshold = 0.0, double OverlapRemovalShrinkPercent = 0.0,
		TFunction<bool(int32)> SkipBoneFn = nullptr, const FConvexDecompositionSettings* OptionalDecompositionSettings = nullptr,
		const TArray<FTransformedConvex>* OptionalIntersectConvexHulls = nullptr,
		const TArray<TSet<int32>>* OptionalTransformToIntersectHulls = nullptr,
		TArray<FGeometryCollectionConvexUtility::FSphereCoveringInfo>* OutComputedNavigableSpheres = nullptr);

	// generate a list of convex out of a hierarchy of implciit shapes
	// suported shapes are scaled / transformed implicits as well as Boxes, convexes, spheres and capsules
	// levelset, tapered capsule, cylinder, heightfields and trimesh are not supported
	// array will not be erased and convex will be added if any already exists
	static CHAOS_API void ConvertImplicitToConvexArray(const Chaos::FImplicitObject& InImplicit, const FTransform& Transform, TArray<FTransformedConvex>& InOutConvex);

private:
	static void ConvertScaledImplicitToConvexArray(
		const Chaos::FImplicitObject& Implicit,
		const FTransform& WorldSpaceTransform, bool bInstanced,
		TArray<FTransformedConvex>& InOutConvex);

	static void ConvertInstancedImplicitToConvexArray(
		const Chaos::FImplicitObject& Implicit,
		const FTransform& Transform,
		TArray<FTransformedConvex>& InOutConvex);

	static void CreateConvexHullAttributesIfNeeded(FManagedArrayCollection& GeometryCollection);

	// Implementation for GenerateClusterConvexHullsFromLeafHulls, supporting the full-collection and subset cases
	static void GenerateClusterConvexHullsFromLeafOrChildrenHullsInternal(FGeometryCollection& Collection, const FClusterConvexHullSettings& Settings, bool bOnlySubset, bool bUseDirectChildren, const TArrayView<const int32> OptionalTransformSubset);
};

