// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"

class FGeometryCollection;

class FGeometryCollectionProximityPropertiesInterface : public FManagedArrayInterface
{
public :
	typedef FManagedArrayInterface Super;
	using FManagedArrayInterface::ManagedCollection;

	// Proximity Properties Group Name
	static CHAOS_API const FName ProximityPropertiesGroup;
	
	// Attribute: Method to determine proximity
	static CHAOS_API const FName ProximityDetectionMethod;
	// Attribute: For convex hull proximity, what separation distance can still be considered as 'proximity'
	static CHAOS_API const FName ProximityDistanceThreshold;
	// Attribute: Whether to use the computed proximity graph as a connection graph
	static CHAOS_API const FName ProximityAsConnectionGraph;
	// Attribute: How to compute contact area for connection graph edges, to be used by the simulation
	static CHAOS_API const FName ProximityConnectionGraphContactAreaMethod;
	// Attribute: If greater than zero, filter proximity connections by requiring an amount of 'contact' as computed by the Contact Method
	// This is a second filter applied after initial proximity connections were determined by any Proximity Detection Method,
	// and can be used to reduce unsupported or spurious/glancing proximities.
	static CHAOS_API const FName ProximityRequireContactAmount;
	// Attribute: How to define 'contact' for the above Require Contact Amount.
	static CHAOS_API const FName ProximityContactMethod;

	struct FProximityProperties
	{
		EProximityMethod Method = EProximityMethod::Precise;
		float DistanceThreshold = 1.0f;
		float RequireContactAmount = 0.0f;
		EProximityContactMethod ContactMethod = EProximityContactMethod::MinOverlapInProjectionToMajorAxes;
		EConnectionContactMethod ContactAreaMethod = EConnectionContactMethod::None;
		bool bUseAsConnectionGraph = false;
	};

	CHAOS_API FGeometryCollectionProximityPropertiesInterface(FGeometryCollection* InGeometryCollection);

	CHAOS_API void InitializeInterface() override;

	CHAOS_API void CleanInterfaceForCook() override;
	
	CHAOS_API void RemoveInterfaceAttributes() override;

	CHAOS_API FProximityProperties GetProximityProperties() const;
	CHAOS_API void SetProximityProperties(const FProximityProperties&);
};

