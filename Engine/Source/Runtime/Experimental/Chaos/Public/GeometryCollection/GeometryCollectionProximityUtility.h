// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"

#include "GeometryCollectionProximityUtility.generated.h"

class FGeometryCollection;
namespace UE::GeometryCollectionConvexUtility
{
	struct FConvexHulls;
}

namespace Chaos::Facades
{
	class FCollectionHierarchyFacade;
}

UENUM()
enum class EProximityMethod : int32
{
	// Precise proximity mode looks for geometry with touching vertices or touching, coplanar, opposite-facing triangles. This works well with geometry fractured using our fracture tools.
	Precise,
	// Convex Hull proximity mode looks for geometry with overlapping convex hulls (with an optional offset)
	ConvexHull
};

UENUM()
enum class EProximityContactMethod : uint8
{
	// Rejects proximity if the bounding boxes do not overlap by more than Contact Threshold centimeters in any major axis direction (or at least half the max possible). This can filter out corner connections of box-like shapes.
	MinOverlapInProjectionToMajorAxes,
	// Rejects proximity if the intersection of convex hulls (allowing for optional offset) follows a sharp, thin region which is not wider than Contact Threshold centimeters (or at least half the max possible).
	ConvexHullSharpContact,
	// Rejects proximity if the surface area of the intersection of convex hulls (allowing for optional offset) is smaller than Contact Threshold squared (or at least half the max possible).
	ConvexHullAreaContact
	//~ TODO: Add other methods for filtering overlaps, e.g. based on approximate surface area of the contact
};

// How contact is computed on the connection graph
UENUM()
enum class EConnectionContactMethod : uint8
{
	// Do not compute contact areas
	None,
	// Define contact based on the surface area of the intersection of the convex hulls, allowing for optional offset
	ConvexHullContactArea
};

class FGeometryCollectionProximityUtility
{
public:
	CHAOS_API FGeometryCollectionProximityUtility(FGeometryCollection* InCollection);

	CHAOS_API void UpdateProximity(UE::GeometryCollectionConvexUtility::FConvexHulls* OptionalComputedHulls = nullptr);

	// Update proximity data if it is not already present
	CHAOS_API void RequireProximity(UE::GeometryCollectionConvexUtility::FConvexHulls* OptionalComputedHulls = nullptr);

	CHAOS_API void InvalidateProximity();

	CHAOS_API void EnumerateNeighbors(const Chaos::Facades::FCollectionHierarchyFacade& Hierarchy, int32 BoneIdx, TFunctionRef<void(int32)> NeighborFunc, bool bIncludeNeighborsInParentLevels = true, bool bFilterDuplicates = true);

	// Method to compute the precise proximity, regardless of the proximity configuration on the collection
	CHAOS_API static TArray<TSet<int32>> ComputePreciseProximity(const FGeometryCollection& InCollection);

	// Stores stats about the contact between two geometries
	struct FGeometryContactEdge
	{
		int32 GeometryIndices[2];
		// Area estimate for contact region
		float ContactArea;
		// Maximum area for contact for this pair (half the smallest surface area)
		float MaxContactArea;
		// 'Sharp contact' width estimate
		float SharpContactWidth;
		// Estimate of maximum possible 'sharp contact' for this pair
		float MaxSharpContact;

		FGeometryContactEdge() = default;
		FGeometryContactEdge(int32 GeoIdxA, int32 GeoIdxB, float ContactArea, float MaxContactArea, float SharpContactWidth, float MaxSharpContact) :
			GeometryIndices{ GeoIdxA, GeoIdxB }, ContactArea(ContactArea), MaxContactArea(MaxContactArea), SharpContactWidth(SharpContactWidth), MaxSharpContact(MaxSharpContact)
		{}
	};
	// Note: This computes connections from lower to higher geometry indices, assuming connections are symmetric
	static CHAOS_API TArray<FGeometryContactEdge> ComputeConvexGeometryContactFromProximity(FGeometryCollection* Collection, float DistanceTolerance, UE::GeometryCollectionConvexUtility::FConvexHulls& LocalHulls);

	// @param ContactEdges	Optional pre-computed proximity contact edges, to be used for computing contact areas on the connection graph edges
	CHAOS_API void CopyProximityToConnectionGraph(const TArray<FGeometryContactEdge>* ContactEdges = nullptr);
	CHAOS_API void ClearConnectionGraph();

private:
	FGeometryCollection* Collection;

	// This threshold not exposed to the user via Properties.DistanceThreshold because it doesn't behave very well if increased to large values --
	// the computation would become closer to O(n^2), and the results would likely be confusing/inconsistent
	static constexpr float PreciseProximityThreshold = .01f;
};

