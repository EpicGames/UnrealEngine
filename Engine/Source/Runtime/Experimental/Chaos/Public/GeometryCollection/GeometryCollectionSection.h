// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Set.h"

class HHitProxy;
struct FManagedArrayCollection;

/**
* A set of triangles which are rendered with the same material.
*/
struct FGeometryCollectionSection
{
	/** Constructor. */
	FGeometryCollectionSection()
		: MaterialID(0)
		, FirstIndex(0)
		, NumTriangles(0)
		, MinVertexIndex(0)
		, MaxVertexIndex(0)
		, HitProxy(NULL)
	{ }	
	
	/** Serializer. */
	friend FArchive& operator<<(FArchive& Ar, FGeometryCollectionSection& Section)
	{
		return Ar << Section.MaterialID << Section.FirstIndex << Section.NumTriangles << Section.MinVertexIndex << Section.MaxVertexIndex;
	}

	bool Serialize(FArchive& Ar)
	{
		Ar << *this;
		return true;
	}

	static CHAOS_API TArray<FGeometryCollectionSection>
	BuildMeshSections(const FManagedArrayCollection& InCollection, const TArray<FIntVector>& InputIndices, const TArray<int32>& BaseMeshOriginalIndicesIndex, TArray<FIntVector>& RetIndices);

	/** The index of the material with which to render this section. */
	int32 MaterialID;

	/** Range of vertices and indices used when rendering this section. */
	int32 FirstIndex;
	int32 NumTriangles;
	int32 MinVertexIndex;
	int32 MaxVertexIndex;

	HHitProxy* HitProxy;

	static const int InvalidIndex = -1;
};
