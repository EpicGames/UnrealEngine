// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Field/FieldSystem.h"
#include "GeometryCollection/RecordedTransformTrack.h"
#include "GeometryCollectionSimulationTypes.h"
#include "Chaos/ClusterCreationParameters.h"
#include "Chaos/CollisionFilterData.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "Chaos/PBDRigidClusteringTypes.h"

class FGeometryCollection;
class FGeometryDynamicCollection;


struct FCollectionLevelSetData
{
	FCollectionLevelSetData()
		: MinLevelSetResolution(5)
		, MaxLevelSetResolution(10)
		, MinClusterLevelSetResolution(25)
		, MaxClusterLevelSetResolution(50)
	{}

	int32 MinLevelSetResolution;
	int32 MaxLevelSetResolution;
	int32 MinClusterLevelSetResolution;
	int32 MaxClusterLevelSetResolution;
};

struct FCollectionCollisionParticleData
{
	FCollectionCollisionParticleData()
		: CollisionParticlesFraction(1.f)
		, MaximumCollisionParticles(60)
	{}

	float CollisionParticlesFraction;
	int32 MaximumCollisionParticles;
};



struct FCollectionCollisionTypeData
{
	FCollectionCollisionTypeData()
		: CollisionType(ECollisionTypeEnum::Chaos_Surface_Volumetric)
		, ImplicitType(EImplicitTypeEnum::Chaos_Implicit_Sphere)
		, LevelSetData()
		, CollisionParticleData()
		, CollisionObjectReductionPercentage(0.f)
		, CollisionMarginFraction(0.f)
	{
	}

	ECollisionTypeEnum CollisionType;
	EImplicitTypeEnum ImplicitType;
	FCollectionLevelSetData LevelSetData;
	FCollectionCollisionParticleData CollisionParticleData;
	float CollisionObjectReductionPercentage;
	float CollisionMarginFraction;
};

struct FSharedSimulationSizeSpecificData
{
	FSharedSimulationSizeSpecificData()
		: MaxSize(0.f)
		, DamageThreshold(5000.f)
		, CollisionShapesData({ FCollectionCollisionTypeData() })
	{
	}

	float MaxSize;
	float DamageThreshold;
	TArray<FCollectionCollisionTypeData> CollisionShapesData;

	bool operator<(const FSharedSimulationSizeSpecificData& Rhs) const { return MaxSize < Rhs.MaxSize; }
};

//
//
//
enum class ESimulationInitializationState : uint8 { Unintialized = 0, Activated, Created, Initialized };


/**
*  Simulation Parameters
*/
struct FSharedSimulationParameters
{
	FSharedSimulationParameters()
	: MinimumMassClamp(0.1f)								// todo : Expose to users with better initial values
	, MaximumMassClamp(1e5f)								// todo : Expose to users with better initial values
	, MinimumBoundingExtentClamp(0.1f)						// todo : Expose to users with better initial values
	, MaximumBoundingExtentClamp(1e6f)						// todo : Expose to users with better initial values
	, MinimumInertiaTensorDiagonalClamp(UE_SMALL_NUMBER)	// todo : Expose to users with better initial values
	, MaximumInertiaTensorDiagonalClamp(1e20f)				// todo : Expose to users with better initial values
	, MaximumCollisionParticleCount(60)
	, Mass(1.0f)
	, bMassAsDensity(true)
	, bUseImportedCollisionImplicits(false)
	{
		SizeSpecificData.AddDefaulted();
	}

	FSharedSimulationParameters(ECollisionTypeEnum InCollisionType
		,EImplicitTypeEnum InImplicitType
		,int32 InMinLevelSetResolution
		,int32 InMaxLevelSetResolution
		,int32 InMinClusterLevelSetResolution
		,int32 InMaxClusterLevelSetResolution
		,bool InMassAsDensity
		,float InMass
		, float InMinimumMassClamp
		, float InMaximumMassClamp
		, float InMinimumBoundingExtentClamp
		, float InMaximumBoundingExtentClamp
		, float InMinimumInertiaTensorDiagonalClamp
		, float InMaximumInertiaTensorDiagonalClamp
		, float InCollisionParticlesFraction
		, int32 InMaximumCollisionParticleCount
		, float InCollisionMarginFraction
		, bool InUseImportedCollisionImplicits )
	: MinimumMassClamp(InMinimumMassClamp)
	, MaximumMassClamp(InMinimumMassClamp)
	, MinimumBoundingExtentClamp(InMinimumBoundingExtentClamp)
	, MaximumBoundingExtentClamp(InMinimumBoundingExtentClamp)
	, MinimumInertiaTensorDiagonalClamp(InMinimumInertiaTensorDiagonalClamp)
	, MaximumInertiaTensorDiagonalClamp(InMaximumInertiaTensorDiagonalClamp)
	, MaximumCollisionParticleCount(InMaximumCollisionParticleCount)
	, Mass(InMass)
	, bMassAsDensity(InMassAsDensity)
	, bUseImportedCollisionImplicits(InUseImportedCollisionImplicits)
	{
		SizeSpecificData.AddDefaulted();
		if (ensure(SizeSpecificData.Num() && SizeSpecificData[0].CollisionShapesData.Num()))
		{
			SizeSpecificData[0].CollisionShapesData[0].CollisionType = InCollisionType;
			SizeSpecificData[0].CollisionShapesData[0].ImplicitType = InImplicitType;
			SizeSpecificData[0].CollisionShapesData[0].CollisionMarginFraction = InCollisionMarginFraction;
			SizeSpecificData[0].CollisionShapesData[0].LevelSetData.MinLevelSetResolution = InMinLevelSetResolution;
			SizeSpecificData[0].CollisionShapesData[0].LevelSetData.MaxLevelSetResolution = InMaxLevelSetResolution;
			SizeSpecificData[0].CollisionShapesData[0].LevelSetData.MinClusterLevelSetResolution = InMinClusterLevelSetResolution;
			SizeSpecificData[0].CollisionShapesData[0].LevelSetData.MaxClusterLevelSetResolution = InMaxClusterLevelSetResolution;
			SizeSpecificData[0].CollisionShapesData[0].CollisionParticleData.CollisionParticlesFraction = InCollisionParticlesFraction;
			SizeSpecificData[0].CollisionShapesData[0].CollisionParticleData.MaximumCollisionParticles = InMaximumCollisionParticleCount;
		}
	}

	TArray<FSharedSimulationSizeSpecificData> SizeSpecificData;
	float MinimumMassClamp;
	float MaximumMassClamp;
	float MinimumBoundingExtentClamp;
	float MaximumBoundingExtentClamp;
	float MinimumInertiaTensorDiagonalClamp;
	float MaximumInertiaTensorDiagonalClamp;
	int32 MaximumCollisionParticleCount;
	float Mass;
	bool bMassAsDensity : 1;
	bool bUseImportedCollisionImplicits : 1;

	float MinimumVolumeClamp() const { return MinimumBoundingExtentClamp * MinimumBoundingExtentClamp * MinimumBoundingExtentClamp; }
	float MaximumVolumeClamp() const { return MaximumBoundingExtentClamp * MaximumBoundingExtentClamp * MaximumBoundingExtentClamp; }
};

#define SIMULATIONPARAMETERS_CACHE_PARAMETERS 1

struct FSimulationParameters
{
	FSimulationParameters()
		: Name("")
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, RestCollection(nullptr)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		, RestCollectionShared(nullptr)
		, WorldTransform(FTransform::Identity)
		, DamageThreshold({ 500000.f, 50000.f, 5000.f })
		, InitialRootIndex(INDEX_NONE)
		, ClusterGroupIndex(0)
		, MaxClusterLevel(100)
		, MaxSimulatedLevel(100)
		, ObjectType(EObjectStateTypeEnum::Chaos_NONE)
		, InitialVelocityType(EInitialVelocityTypeEnum::Chaos_Initial_Velocity_None)
		, DamageModel(EDamageModelTypeEnum::Chaos_Damage_Model_UserDefined_Damage_Threshold)
		, DamageEvaluationModel(Chaos::EDamageEvaluationModel::StrainFromDamageThreshold)
		, ClusterConnectionMethod(Chaos::FClusterCreationParameters::EConnectionMethod::PointImplicit)
		, ConnectionGraphBoundsFilteringMargin(0)
		, CollisionGroup(0)
		, CollisionSampleFraction(1.0)
		, InitialLinearVelocity(FVector(0))
		, InitialAngularVelocity(FVector(0))
		, MaterialOverrideMassScaleMultiplier(1.0f)
		, Simulating(false)
		, EnableClustering(true)
		, bUseSizeSpecificDamageThresholds(false)
		, bUseMaterialDamageModifiers(false)
		, bUsePerClusterOnlyDamageThreshold(false)
		, StartAwake(true)
		, bForceUpdateActiveTransforms(false)
		, bGenerateBreakingData(false)
		, bGenerateCollisionData(false)
		, bGenerateTrailingData(false)
		, bGenerateCrumblingData(false)
		, bGenerateCrumblingChildrenData(false)
		, bGenerateGlobalBreakingData(false)
		, bGenerateGlobalCollisionData(false)
		, bGenerateGlobalCrumblingData(false)
		, bGenerateGlobalCrumblingChildrenData(false)
		, bGenerateGlobalTrailingData(false)
		, EnableGravity(true)
		, UseInertiaConditioning(true)
		, UseCCD(false)
		, UseMACD(false)
		, AllowPartialIslandSleep(true)
		, bEnableStrainOnCollision(true)
		, bUseStaticMeshCollisionForTraces(false)
		, bOptimizeConvexes(true)
		, bUseSimplicialsWhenAvailable(false)
		, bUseDamagePropagation(false)
		, bOptimizeForRuntimeMemory(false)
		, bGyroscopicTorqueEnabled(false)
		, PositionSolverIterations(8)
		, VelocitySolverIterations(1)
		, ProjectionSolverIterations(1)
		, BreakDamagePropagationFactor(1.0f)
		, ShockDamagePropagationFactor(0.0f)
		, LinearDamping(0.01f)
		, AngularDamping(0)
		, InitialOverlapDepenetrationVelocity(-1.0f)
		, SleepThresholdMultiplier(1.0f)
		, GravityGroupIndex(0)
		, OneWayInteractionLevel(INDEX_NONE)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, SimulationFilterData()
		, QueryFilterData()
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		, ShapeFilterData()
		, ShapeInstanceData()
		, UserData(nullptr)
#if SIMULATIONPARAMETERS_CACHE_PARAMETERS
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, RecordedTrack(nullptr)
		, CacheBeginTime(0.0f)
		, ReverseCacheBeginTime(0.0f)
		, CacheType(EGeometryCollectionCacheType::None)
		, bClearCache(false)
		, bOwnsTrack(false)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	{}

	FSimulationParameters(const FSimulationParameters& Other)
		: Name(Other.Name)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, RestCollection(Other.RestCollection)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		, RestCollectionShared(Other.RestCollectionShared)
		, Shared(Other.Shared)
		, WorldTransform(Other.WorldTransform)
		, InitializationCommands(Other.InitializationCommands)
		, DamageThreshold(Other.DamageThreshold)
		, InitialRootIndex(Other.InitialRootIndex)
		, ClusterGroupIndex(Other.ClusterGroupIndex)
		, MaxClusterLevel(Other.MaxClusterLevel)
		, MaxSimulatedLevel(Other.MaxSimulatedLevel)
		, ObjectType(Other.ObjectType)
		, InitialVelocityType(Other.InitialVelocityType)
		, DamageModel(Other.DamageModel)
		, DamageEvaluationModel(Other.DamageEvaluationModel)
		, ClusterConnectionMethod(Other.ClusterConnectionMethod)
		, ConnectionGraphBoundsFilteringMargin(Other.ConnectionGraphBoundsFilteringMargin)
		, CollisionGroup(Other.CollisionGroup)
		, CollisionSampleFraction(Other.CollisionSampleFraction)
		, InitialLinearVelocity(Other.InitialLinearVelocity)
		, InitialAngularVelocity(Other.InitialAngularVelocity)
		, PhysicalMaterialHandle(Other.PhysicalMaterialHandle)
		, MaterialOverrideMassScaleMultiplier(Other.MaterialOverrideMassScaleMultiplier)
		, Simulating(Other.Simulating)
		, EnableClustering(Other.EnableClustering)
		, bUseSizeSpecificDamageThresholds(Other.bUseSizeSpecificDamageThresholds)
		, bUseMaterialDamageModifiers(Other.bUseMaterialDamageModifiers)
		, bUsePerClusterOnlyDamageThreshold(Other.bUsePerClusterOnlyDamageThreshold)
		, StartAwake(Other.StartAwake)
		, bForceUpdateActiveTransforms(Other.bForceUpdateActiveTransforms)
		, bGenerateBreakingData(Other.bGenerateBreakingData)
		, bGenerateCollisionData(Other.bGenerateCollisionData)
		, bGenerateTrailingData(Other.bGenerateTrailingData)
		, bGenerateCrumblingData(Other.bGenerateCrumblingData)
		, bGenerateCrumblingChildrenData(Other.bGenerateCrumblingChildrenData)
		, bGenerateGlobalBreakingData(Other.bGenerateGlobalBreakingData)
		, bGenerateGlobalCollisionData(Other.bGenerateGlobalCollisionData)
		, bGenerateGlobalCrumblingData(Other.bGenerateGlobalCrumblingData)
		, bGenerateGlobalCrumblingChildrenData(Other.bGenerateGlobalCrumblingChildrenData)
		, bGenerateGlobalTrailingData(Other.bGenerateGlobalTrailingData)
		, EnableGravity(Other.EnableGravity)
		, UseInertiaConditioning(Other.UseInertiaConditioning)
		, UseCCD(Other.UseCCD)
		, UseMACD(Other.UseMACD)
		, AllowPartialIslandSleep(Other.AllowPartialIslandSleep)
		, bEnableStrainOnCollision(Other.bEnableStrainOnCollision)
		, bUseStaticMeshCollisionForTraces(Other.bUseStaticMeshCollisionForTraces)
		, bOptimizeConvexes(Other.bOptimizeConvexes)
		, bUseSimplicialsWhenAvailable(Other.bUseSimplicialsWhenAvailable)
		, bUseDamagePropagation(Other.bUseDamagePropagation)
		, bOptimizeForRuntimeMemory(Other.bOptimizeForRuntimeMemory)
		, bGyroscopicTorqueEnabled(Other.bGyroscopicTorqueEnabled)
		, PositionSolverIterations(Other.PositionSolverIterations)
		, VelocitySolverIterations(Other.VelocitySolverIterations)
		, ProjectionSolverIterations(Other.ProjectionSolverIterations)
		, BreakDamagePropagationFactor(Other.BreakDamagePropagationFactor)
		, ShockDamagePropagationFactor(Other.ShockDamagePropagationFactor)
		, LinearDamping(Other.LinearDamping)
		, AngularDamping(Other.AngularDamping)
		, InitialOverlapDepenetrationVelocity(Other.InitialOverlapDepenetrationVelocity)
		, SleepThresholdMultiplier(Other.SleepThresholdMultiplier)
		, GravityGroupIndex(Other.GravityGroupIndex)
		, OneWayInteractionLevel(Other.OneWayInteractionLevel)
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, SimulationFilterData(Other.SimulationFilterData)
		, QueryFilterData(Other.QueryFilterData)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		, ShapeFilterData(Other.ShapeFilterData)
		, ShapeInstanceData(Other.ShapeInstanceData)
		, UserData(Other.UserData)
#if SIMULATIONPARAMETERS_CACHE_PARAMETERS
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, RecordedTrack(Other.RecordedTrack)
		, CacheBeginTime(Other.CacheBeginTime)
		, ReverseCacheBeginTime(Other.ReverseCacheBeginTime)
		, CacheType(Other.CacheType)
		, bClearCache(Other.bClearCache)
		, bOwnsTrack(false)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	{
	}

	~FSimulationParameters()
	{
#if SIMULATIONPARAMETERS_CACHE_PARAMETERS
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (bOwnsTrack)
		{
			delete const_cast<FRecordedTransformTrack*>(RecordedTrack);
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	}

	FString Name;
	
	UE_DEPRECATED(5.4, "Raw pointer no longer in use, instead prefer RestCollectionShared")
	const FGeometryCollection* RestCollection;

	TSharedPtr<const FGeometryCollection> RestCollectionShared;
	FSharedSimulationParameters Shared;

	FTransform WorldTransform;
	FTransform PrevWorldTransform;

	TArray<FFieldSystemCommand> InitializationCommands;

	TArray<float> DamageThreshold;

	int32 InitialRootIndex;
	int32 ClusterGroupIndex;
	int16 MaxClusterLevel;
	int16 MaxSimulatedLevel;

	EObjectStateTypeEnum ObjectType;
	
	EInitialVelocityTypeEnum InitialVelocityType;

	/** this is the user expose damage model, used for creation of the particles */
	EDamageModelTypeEnum DamageModel; 

	/** this is the lower level damage model for clustering, used at runm time */
	Chaos::EDamageEvaluationModel DamageEvaluationModel;

	Chaos::FClusterCreationParameters::EConnectionMethod ClusterConnectionMethod;
	float ConnectionGraphBoundsFilteringMargin;

	int32 CollisionGroup;
	float CollisionSampleFraction;

	FVector3f InitialLinearVelocity;
	FVector3f InitialAngularVelocity;

	Chaos::FMaterialHandle PhysicalMaterialHandle;

	float MaterialOverrideMassScaleMultiplier;

	bool Simulating : 1;
	bool EnableClustering : 1;
	bool bUseSizeSpecificDamageThresholds : 1;
	bool bUseMaterialDamageModifiers : 1;
	bool bUsePerClusterOnlyDamageThreshold : 1;
	bool StartAwake : 1;
	bool bForceUpdateActiveTransforms : 1;

	bool bGenerateBreakingData : 1;
	bool bGenerateCollisionData : 1;
	bool bGenerateTrailingData : 1;
	bool bGenerateCrumblingData : 1;
	bool bGenerateCrumblingChildrenData : 1; 

	bool bGenerateGlobalBreakingData : 1;
	bool bGenerateGlobalCollisionData : 1;
	bool bGenerateGlobalCrumblingData : 1;
	bool bGenerateGlobalCrumblingChildrenData : 1;
	bool bGenerateGlobalTrailingData : 1;

	bool EnableGravity : 1;
	bool UseInertiaConditioning : 1;
	bool UseCCD : 1;
	bool UseMACD : 1;
	bool AllowPartialIslandSleep : 1;
	bool bEnableStrainOnCollision : 1;
	bool bUseStaticMeshCollisionForTraces : 1;
	bool bOptimizeConvexes : 1;
	bool bUseSimplicialsWhenAvailable : 1;

	bool bUseDamagePropagation : 1;
	bool bOptimizeForRuntimeMemory : 1;
	bool bGyroscopicTorqueEnabled : 1;

	int32 PositionSolverIterations;
	int32 VelocitySolverIterations;
	int32 ProjectionSolverIterations;

	float BreakDamagePropagationFactor;
	float ShockDamagePropagationFactor;

	float LinearDamping;
	float AngularDamping;
	float InitialOverlapDepenetrationVelocity;
	float SleepThresholdMultiplier;

	int32 GravityGroupIndex;
	int32 OneWayInteractionLevel;

	UE_DEPRECATED(5.8, "Use ShapeFilterData and ShapeInstanceData instead")
	FCollisionFilterData SimulationFilterData;
	UE_DEPRECATED(5.8, "Use ShapeFilterData and ShapeInstanceData instead")
	FCollisionFilterData QueryFilterData;
	Chaos::Filter::FShapeFilterData ShapeFilterData;
	Chaos::Filter::FInstanceData ShapeInstanceData;

	void* UserData;

#if SIMULATIONPARAMETERS_CACHE_PARAMETERS
	UE_DEPRECATED(5.5, "No longer used")
	const FRecordedTransformTrack* RecordedTrack;

	UE_DEPRECATED(5.5, "No longer used")
	float CacheBeginTime;

	UE_DEPRECATED(5.5, "No longer used")
	float ReverseCacheBeginTime;

	UE_DEPRECATED(5.5, "No longer used")
	EGeometryCollectionCacheType CacheType;

	UE_DEPRECATED(5.5, "No longer used")
	bool bClearCache : 1;

	UE_DEPRECATED(5.5, "No longer used")
	bool bOwnsTrack : 1;

	UE_DEPRECATED(5.5, "No longer used and underlying variable is deprecated")
	bool IsCacheRecording()
	{ 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CacheType == EGeometryCollectionCacheType::Record || CacheType == EGeometryCollectionCacheType::RecordAndPlay; 
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	UE_DEPRECATED(5.5, "No longer used and underlying variable is deprecated")
	bool IsCachePlaying()
	{ 
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CacheType == EGeometryCollectionCacheType::Play || CacheType == EGeometryCollectionCacheType::RecordAndPlay;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
#endif
};
