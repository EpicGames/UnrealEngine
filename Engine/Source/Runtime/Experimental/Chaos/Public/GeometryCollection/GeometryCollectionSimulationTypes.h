// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/ObjectMacros.h"
#include "Field/FieldSystemTypes.h"
#include "Chaos/PBDRigidClusteringTypes.h"

#include "GeometryCollectionSimulationTypes.generated.h"

UENUM()
enum class ECollisionTypeEnum : uint8
{
	Chaos_Volumetric         UMETA(DisplayName = "Implicit-Implicit"),
	Chaos_Surface_Volumetric UMETA(DisplayName = "Particle-Implicit"),
	//
	Chaos_Max                UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EImplicitTypeEnum : uint8
{
	Chaos_Implicit_Box UMETA(DisplayName = "Box"),
	Chaos_Implicit_Sphere UMETA(DisplayName = "Sphere"),
	Chaos_Implicit_Capsule UMETA(DisplayName = "Capsule"),
	Chaos_Implicit_LevelSet UMETA(DisplayName = "Level Set"),
	Chaos_Implicit_None UMETA(DisplayName = "None"),
	Chaos_Implicit_Convex UMETA(DisplayName = "Convex"),
	//
	Chaos_Max                UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EObjectStateTypeEnum : uint8
{
	Chaos_NONE = 0 UMETA(Hidden, DisplayName = "None"),
	Chaos_Object_Sleeping  = 1 UMETA(DisplayName = "Sleeping"),
	Chaos_Object_Kinematic = 2 UMETA(DisplayName = "Kinematic"),
	Chaos_Object_Static = 3    UMETA(DisplayName = "Static"),
	Chaos_Object_Dynamic = 4 UMETA(DisplayName = "Dynamic"),
	Chaos_Object_UserDefined = 100 UMETA(DisplayName = "User Defined"),
	//
	Chaos_Max UMETA(Hidden)
};

UENUM(BlueprintType)
enum class EGeometryCollectionPhysicsTypeEnum : uint8
{
	Chaos_AngularVelocity          UMETA(DisplayName = "Angular Velocity", ToolTip = "Add a vector field to the particles angular velocity."),
	Chaos_DynamicState             UMETA(DisplayName = "Dynamic State", ToolTip = "Set the dynamic state of a particle (static, dynamic, kinematic...)"),
	Chaos_LinearVelocity           UMETA(DisplayName = "Linear Velocity", ToolTip = "Add a vector field to the particles linear velocity."),
	Chaos_InitialAngularVelocity   UMETA(DisplayName = "Initial Angular Velocity", ToolTip = "Initial particles angular velocity."),
	Chaos_InitialLinearVelocity    UMETA(DisplayName = "Initial Linear Velocity", ToolTip = "Initial particles linear velocity."),
	Chaos_CollisionGroup           UMETA(DisplayName = "Collision Group", ToolTip = "Set the particles collision group."),
	Chaos_LinearForce              UMETA(DisplayName = "Linear Force", ToolTip = "Add a vector field to the particles linear force."),
	Chaos_AngularTorque            UMETA(DisplayName = "Angular Torque", ToolTip = "Add a vector field to the particles angular torque."),
	Chaos_DisableThreshold         UMETA(DisplayName = "Disable Threshold", ToolTip = "Disable the particles if their linear and angular velocity are less than the threshold."),
	Chaos_SleepingThreshold        UMETA(DisplayName = "Sleeping Threshold", ToolTip = "Set particles in sleeping mode if their linear and angular velocity are less than the threshold."),
	Chaos_ExternalClusterStrain    UMETA(DisplayName = "External Strain", ToolTip = "Apply an external strain over the particles. If this strain is over the internal one, the cluster will break."),
	Chaos_InternalClusterStrain    UMETA(DisplayName = "Internal Strain", ToolTip = "Add a strain field to the particles internal one."),
	Chaos_LinearImpulse			   UMETA(DisplayName = "Linear Impulse", ToolTip = "Add a vector field to apply an impulse to the particles."),
	//
	Chaos_Max						UMETA(Hidden)
};


inline EFieldPhysicsType GetGeometryCollectionPhysicsType(const EGeometryCollectionPhysicsTypeEnum GeoCollectionType)
{
	static const TArray<EFieldPhysicsType> PhysicsTypes = {
		EFieldPhysicsType::Field_AngularVelociy,
		EFieldPhysicsType::Field_DynamicState,
		EFieldPhysicsType::Field_LinearVelocity, 
		EFieldPhysicsType::Field_InitialAngularVelocity,
		EFieldPhysicsType::Field_InitialLinearVelocity,
		EFieldPhysicsType::Field_CollisionGroup, 
		EFieldPhysicsType::Field_LinearForce,
		EFieldPhysicsType::Field_AngularTorque,
		EFieldPhysicsType::Field_DisableThreshold,
		EFieldPhysicsType::Field_SleepingThreshold, 
		EFieldPhysicsType::Field_ExternalClusterStrain,
		EFieldPhysicsType::Field_InternalClusterStrain,
		EFieldPhysicsType::Field_LinearImpulse,
		EFieldPhysicsType::Field_PhysicsType_Max
	};

	return PhysicsTypes[(uint8)GeoCollectionType];
}

UENUM(BlueprintType)
enum class EInitialVelocityTypeEnum : uint8
{
	//Chaos_Initial_Velocity_Animation UMETA(DisplayName = "Animation"),
	Chaos_Initial_Velocity_User_Defined UMETA(DisplayName = "User Defined"),
	//Chaos_Initial_Velocity_Field UMETA(DisplayName = "Field"),
	Chaos_Initial_Velocity_None UMETA(DisplayName = "None"),
	//
	Chaos_Max                UMETA(Hidden)
};


UENUM(BlueprintType)
enum class EEmissionPatternTypeEnum : uint8
{
	Chaos_Emission_Pattern_First_Frame UMETA(DisplayName = "First Frame"),
	Chaos_Emission_Pattern_On_Demand UMETA(DisplayName = "On Demand"),
	//
	Chaos_Max                UMETA(Hidden)
};


UENUM(BlueprintType)
enum class EDamageModelTypeEnum : uint8
{
	/** Using damage threshold set based on level of the cluster */
	Chaos_Damage_Model_UserDefined_Damage_Threshold UMETA(DisplayName = "User-Defined Damage Threshold"),

	/** Using damage threshold set using the physical material strength and how connected a cluster is */
	Chaos_Damage_Model_Material_Strength_And_Connectivity_DamageThreshold UMETA(DisplayName = "Material Strength And Connectivity Damage Threshold"),

	Chaos_Max UMETA(Hidden)
};

// convert user level damage model to chaos damage evaluation model
inline Chaos::EDamageEvaluationModel GetDamageEvaluationModel(const EDamageModelTypeEnum DamageModel)
{
	switch (DamageModel)
	{
	case EDamageModelTypeEnum::Chaos_Damage_Model_UserDefined_Damage_Threshold: 
		return Chaos::EDamageEvaluationModel::StrainFromDamageThreshold;
	case EDamageModelTypeEnum::Chaos_Damage_Model_Material_Strength_And_Connectivity_DamageThreshold: 
		return Chaos::EDamageEvaluationModel::StrainFromMaterialStrengthAndConnectivity;
	default:
		ensure(false); // unexpected
		return Chaos::EDamageEvaluationModel::StrainFromDamageThreshold;
	}
}