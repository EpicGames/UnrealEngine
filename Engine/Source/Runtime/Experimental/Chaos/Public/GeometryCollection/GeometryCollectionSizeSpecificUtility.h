// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionSimulationCoreTypes.h"
#include "Templates/SharedPointer.h"

namespace GeometryCollection 
{
	namespace SizeSpecific
	{
		//
		//  Given a volume, return the first index of the size specific data that matches this volume. 
		//
		int32 CHAOS_API FindIndexForVolume(const TArray<FSharedSimulationSizeSpecificData>& SizeSpecificData, Chaos::FReal Volume);

		//
		//  Given a FBox, return the first index of the size specific data that matches this volume. 
		//
		int32 CHAOS_API FindIndexForVolume(const TArray<FSharedSimulationSizeSpecificData>& SizeSpecificData, const FBox& Bounds);

	}
}