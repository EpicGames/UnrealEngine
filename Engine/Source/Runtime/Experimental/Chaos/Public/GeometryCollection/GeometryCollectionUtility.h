// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Templates/SharedPointer.h"
#include "Async/ParallelFor.h"

namespace GeometryCollection 
{
	/***
	* Add the geometry group to a collection. Mostly for backwards compatibility with older files.
	*/
	void
	CHAOS_API
	AddGeometryProperties(FManagedArrayCollection* Collection);

	/****
	* MakeMeshElement
	*   Utility to create an arbitrary triangulated mesh using the FGeometryCollection format.
	*/
	template <class TV3_PTS, class TV3_NORM, class TV2, class TV_INT3>
	TSharedPtr<FGeometryCollection> MakeMeshElement(
		const TArray<TV3_PTS>& PointsIn,
		const TArray<TV3_NORM>& NormalsIn,
		const TArray<TV_INT3>& TrianglesIn,
		const TArray<TV2>& UVsIn,
		const FTransform& Xf,
		const FTransform& GeoXf = FTransform::Identity,
		const int NumberOfMaterials = 2)
	{
		FGeometryCollection* RestCollection = new FGeometryCollection();
		RestCollection->AddElements(PointsIn.Num(), FGeometryCollection::VerticesGroup);
		RestCollection->AddElements(TrianglesIn.Num(), FGeometryCollection::FacesGroup);
		RestCollection->AddElements(1, FGeometryCollection::TransformGroup);

		TManagedArray<FVector3f>& Vertices = RestCollection->Vertex;
		TManagedArray<FVector3f>& Normals = RestCollection->Normal;
		TManagedArray<FVector3f>& TangentU = RestCollection->TangentU;
		TManagedArray<FVector3f>& TangentV = RestCollection->TangentV;
		TManagedArray<FVector2f>& UV0 = *RestCollection->FindUVLayer(0);
		TManagedArray<FLinearColor>& Colors = RestCollection->Color;
		TManagedArray<FIntVector>& Indices = RestCollection->Indices;
		TManagedArray<bool>& Visible = RestCollection->Visible;
		TManagedArray<int32>& MaterialIndex = RestCollection->MaterialIndex;
		TManagedArray<int32>& MaterialID = RestCollection->MaterialID;
		TManagedArray<bool>& Internal = RestCollection->Internal;
		TManagedArray<FTransform3f>& Transform = RestCollection->Transform;
		TManagedArray<int32>& SimulationType = RestCollection->SimulationType;
		TManagedArray<int32>& BoneMap = RestCollection->BoneMap;

		// Set particle info
		Transform[0] = FTransform3f(Xf);
		Transform[0].NormalizeRotation();
		SimulationType[0] = FGeometryCollection::ESimulationTypes::FST_Rigid;

		// Set vertex info
		for (int32 Idx = 0; Idx < PointsIn.Num(); ++Idx)
		{
			Vertices[Idx] = (FVector3f)GeoXf.TransformPosition(FVector(PointsIn[Idx][0], PointsIn[Idx][1], PointsIn[Idx][2])); // transform points by GeoXf
			Normals[Idx] = NormalsIn.Num() > Idx ? FVector3f(NormalsIn[Idx][0], NormalsIn[Idx][1], NormalsIn[Idx][2]) : FVector3f(0);

			FVector2D UV = UVsIn.Num() > Idx ? FVector2D(UVsIn[Idx][0], UVsIn[Idx][1]) : FVector2D(0);
			UV0[Idx] = FVector2f(UV);
			
			Colors[Idx] = FLinearColor::White;
			BoneMap[Idx] = 0;
		}

		// Set face info
		const int NumberOfEachMaterial = TrianglesIn.Num() / NumberOfMaterials;
		for (int32 Idx = 0; Idx < TrianglesIn.Num(); ++Idx)
		{
			const auto& Tri = TrianglesIn[Idx];
			Indices[Idx] = FIntVector(Tri[0], Tri[1], Tri[2]);

			Visible[Idx] = true;
			Internal[Idx] = false;
			MaterialIndex[Idx] = Idx;
			MaterialID[Idx] = Idx / NumberOfEachMaterial;

			for (int32 Axis = 0; Axis < 3; ++Axis)
			{
				const FVector3f& Normal = Normals[Tri[Axis]];
				const FVector3f Edge = (Vertices[Tri[(Axis + 1) % 3]] - Vertices[Tri[Axis]]);
				TangentU[Tri[Axis]] = (Edge ^ Normal).GetSafeNormal();
				TangentV[Tri[Axis]] = (Normal ^ TangentU[Tri[Axis]]).GetSafeNormal();
			}
		}

		// GeometryGroup
		GeometryCollection::AddGeometryProperties(RestCollection);

		// Add the materail sections to simulate NumberOfMaterials on the object
		TManagedArray<FGeometryCollectionSection>& Sections = RestCollection->Sections;

		// the first 6 indices are material 0
		int FirstElement = RestCollection->AddElements(NumberOfMaterials, FGeometryCollection::MaterialGroup);
		for (int Element = 0; Element < NumberOfMaterials; Element++)
		{
			Sections[Element].MaterialID = Element;
			Sections[Element].FirstIndex = (Element * NumberOfEachMaterial) * 3;
			Sections[Element].NumTriangles = NumberOfEachMaterial;
			Sections[Element].MinVertexIndex = 0;
			Sections[Element].MaxVertexIndex = Vertices.Num() - 1;
		}

		return TSharedPtr<FGeometryCollection>(RestCollection);
	}

	/****
	* MakeCubeElement
	*   Utility to create a triangulated unit cube using the FGeometryCollection format.
	*/
	TSharedPtr<FGeometryCollection> 
	CHAOS_API 
	MakeCubeElement(const FTransform& center, FVector Scale = FVector(1.f), int NumberOfMaterials = 2);

	/****
	* SetupCubeGridExample
	*   Utility to create a grid (10x10x10) of triangulated unit cube using the FGeometryCollection format.
	*/
	void 
	CHAOS_API 
	SetupCubeGridExample(TSharedPtr<FGeometryCollection> GeometryCollection);


	/****
	* Setup Nested Hierarchy Example	
	*/
	void
	CHAOS_API
	SetupNestedBoneCollection(FGeometryCollection * Collection);

	/****
	* Setup Two Clustered Cubes : 
	* ... geometry       { (-9,0,0) && (9,0,0)}
	* ... center of mass { (-10,0,0) && (10,0,0)}
	*/
	void
	CHAOS_API
	SetupTwoClusteredCubesCollection(FGeometryCollection * Collection);

	/***
	* Ensure Material indices are setup correctly. Mostly for backwards compatibility with older files. 
	*/
	void 
	CHAOS_API 
	MakeMaterialsContiguous(FGeometryCollection * Collection);

	/***
	* Transfers attributes from one collection to another based on the nearest vertex
	* #todo(dmp): We can add a lot of modes here, such as:
	* - transfer between different attribute groups
	* - derive attribute values based on different proximity based kernels
	*/
	template<class T>
	void
	AttributeTransfer(const FGeometryCollection * FromCollection, FGeometryCollection * ToCollection, const FName FromAttributeName, const FName ToAttributeName);

	/***
	* Generate GUID in the FTransformCollection::TransformGroup for id tracking. 
	* The GUIDs will not be saved during serialization, and can be used to bind
	* and entry to an location in the collection. NOTE: GUIDs are expensive to
	* maintain so they should not be kept for cooked content. 
	*/
	void 
	CHAOS_API
	GenerateTemporaryGuids(FManagedArrayCollection* Collection, int32 StartIdx = 0, bool bForceInit=false);

	/***
	* Compute inner and outer radius from a set of vertices 
	* @param Vertices		array containing the vertices used to computer the radii
	* @param VertexStart	start index of the range of vertex to use within the array
	* @param VertexCount	Number of vertices to use from VertexStart
	* @param VertexCount	Total number of vertices to use (from VertexStart)
	* @param OutInnerRadius	Computed InnerRadius ( existing value will be overriden )
	* @param OutOuterRadius	Computed outerRadius ( existing value will be overriden )
	*/
	void 
	CHAOS_API
	ComputeInnerAndOuterRadiiFromGeometryVertices(const TManagedArray<FVector3f>& Vertices, const int32 VertexStart, const int32 VertexCount, float& OutInnerRadius, float& OutOuterRadius);

};

// AttributeTransfer implementation
template<class T>
void GeometryCollection::AttributeTransfer(const FGeometryCollection * FromCollection, FGeometryCollection * ToCollection, const FName FromAttributeName, const FName ToAttributeName)
{
	// #todo(dmp): later on we will support different attribute groups for transfer		
	const TManagedArray<T> &FromAttribute = FromCollection->GetAttribute<T>(FromAttributeName, FGeometryCollection::VerticesGroup);
	TManagedArray<T> &ToAttribute = ToCollection->ModifyAttribute<T>(ToAttributeName, FGeometryCollection::VerticesGroup);

	const TManagedArray<FVector3f> &FromVertex = FromCollection->Vertex;
	TManagedArray<FVector3f> &ToVertex = ToCollection->Vertex;

	// for each vertex in ToCollection, find the closest in FromCollection based on vertex position
	// #todo(dmp): should we be evaluating the transform hierarchy here, or just do it in local space?
	// #todo(dmp): use spatial hash rather than n^2 lookup
	ParallelFor(ToCollection->NumElements(FGeometryCollection::VerticesGroup), [&](int32 ToIndex)
	{
		int32 ClosestFromIndex = -1;
		Chaos::FReal ClosestDist = UE_MAX_FLT;
		for (int32 FromIndex = 0, ni = FromVertex.Num(); FromIndex < ni ; ++FromIndex)
		{
			Chaos::FReal CurrDist = FVector3f::DistSquared(FromVertex[FromIndex], ToVertex[ToIndex]);
			if (CurrDist < ClosestDist)
			{
				ClosestDist = CurrDist;
				ClosestFromIndex = FromIndex;
			}
		}

		// If there is a valid position in FromCollection, transfer attribute
		if (ClosestFromIndex != -1)
		{
			ToAttribute[ToIndex] = FromAttribute[ClosestFromIndex];
		}
	});
}