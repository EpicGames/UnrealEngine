// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Containers/Array.h"
#include "GeometryCollection/GeometryCollectionSection.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "Chaos/ChaosArchive.h"
#include "UObject/DestructionObjectVersion.h"
//
#include "ChaosLog.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/BVHParticles.h"
#include "Math/Vector.h"
#include "Chaos/Matrix.h"
#include "Templates/MemoryOps.h"
#include "Templates/TypeHash.h"

struct FManagedArrayCollection;

template <typename T>
void TryBulkSerializeManagedArray(Chaos::FChaosArchive& Ar, TArray<T>& Array)
{
	Ar << Array;
}

//Note: see TArray::BulkSerialize for requirements
inline void TryBulkSerializeManagedArray(Chaos::FChaosArchive& Ar, TArray<FVector3f>& Array)
{
	Array.BulkSerialize(Ar);
}

inline void TryBulkSerializeManagedArray(Chaos::FChaosArchive& Ar, TArray<FGuid>& Array)
{
	Array.BulkSerialize(Ar);
}

inline void TryBulkSerializeManagedArray(Chaos::FChaosArchive& Ar, TArray<FIntVector>& Array)
{
	Array.BulkSerialize(Ar);
}

inline void TryBulkSerializeManagedArray(Chaos::FChaosArchive& Ar, TArray<FVector2f>& Array)
{
	Array.BulkSerialize(Ar);
}

inline void TryBulkSerializeManagedArray(Chaos::FChaosArchive& Ar, TArray<float>& Array)
{
	Array.BulkSerialize(Ar);
}

inline void TryBulkSerializeManagedArray(Chaos::FChaosArchive& Ar, TArray<FQuat4f>& Array)
{
	Array.BulkSerialize(Ar);
}

inline void TryBulkSerializeManagedArray(Chaos::FChaosArchive& Ar, TArray<bool>& Array)
{
	Array.BulkSerialize(Ar);
}

inline void TryBulkSerializeManagedArray(Chaos::FChaosArchive& Ar, TArray<int32>& Array)
{
	Array.BulkSerialize(Ar);
}

inline void TryBulkSerializeManagedArray(Chaos::FChaosArchive& Ar, TArray<uint8>& Array)
{
	Array.BulkSerialize(Ar);
}

inline void TryBulkSerializeManagedArray(Chaos::FChaosArchive& Ar, TArray<FUintVector2>& Array)
{
	Array.BulkSerialize(Ar);
}

inline void TryBulkSerializeManagedArray(Chaos::FChaosArchive& Ar, TArray<FIntVector2>& Array)
{
	Array.BulkSerialize(Ar);
}

inline void TryBulkSerializeManagedArray(Chaos::FChaosArchive& Ar, TArray<FIntVector4>& Array)
{
	Array.BulkSerialize(Ar);
}


// --------------------------------------------------------------------------
// utility functions to estimate the allocated size of managed arrays
// --------------------------------------------------------------------------
namespace ManagedArrayTypeSize
{
	template<typename T>
	inline SIZE_T GetAllocatedSize(const T& Value)
	{
		return sizeof(T);
	}

	template<typename T>
	inline SIZE_T GetAllocatedSize(const TArray<T>& Array)
	{
		return Array.GetAllocatedSize();
	}

	template<typename T>
	inline SIZE_T GetAllocatedSize(const TBitArray<>& Array)
	{
		return Array.GetAllocatedSize();
	}

	template<typename T>
	inline SIZE_T GetAllocatedSize(const TSet<T>& Set)
	{
		return Set.GetAllocatedSize();
	}

	template<typename T>
	inline SIZE_T GetAllocatedSize(const TUniquePtr<T>& Ptr)
	{
		return Ptr ? ManagedArrayTypeSize::GetAllocatedSize(*Ptr) : 0;
	}
	
	template<typename T>
	inline SIZE_T GetAllocatedSize(const TRefCountPtr<T>& Ptr)
	{
		return Ptr ? ManagedArrayTypeSize::GetAllocatedSize(*Ptr) : 0;
	}

	template<typename T, ESPMode Mode>
	inline SIZE_T GetAllocatedSize(const TSharedPtr<T, Mode>& Ptr)
	{
		return Ptr ? ManagedArrayTypeSize::GetAllocatedSize(*Ptr) : 0;
	}

	inline SIZE_T GetAllocatedSize(const Chaos::FImplicitObject3* ImplicitObjectPtr)
	{
		return ImplicitObjectPtr ? sizeof(Chaos::FImplicitObject3) : 0;
	}

	inline SIZE_T GetAllocatedSize(const Chaos::FBVHParticlesFloat3& BVHParticles)
	{
		return BVHParticles.GetAllocatedSize();
	}
}


/***
*  Managed Array Base
*
*  The ManagedArrayBase allows a common base class for the
*  the template class ManagedArray<T>. (see ManagedArray)
*
*/
class FManagedArrayBase : public FNoncopyable
{
	friend FManagedArrayCollection;
protected:
	/**
	* Protected access to array resizing. Only the managers of the Array
	* are allowed to perform a resize. (see friend list above).
	*/
	virtual void Resize(const int32 Num) {};

	/**
	* Protected access to array reservation. Only the managers of the Array
	* are allowed to perform a reserve. (see friend list above).
	*/
	virtual void Reserve(const int32 Num) {};

	/**
	 * Reorder elements given a new ordering. Sizes must match
	 * @param NewOrder Mapping from indices in the new array -> indices in the old array
	 */
	virtual void Reorder(const TArray<int32>& NewOrder) = 0;

	/** 
	 * Reindex given a lookup table
	 * @param InverseNewOrder Mapping from indices into the old array -> indices in the new array
	 */
	//todo: this should really assert, but material is currently relying on both faces and vertices
	virtual void ReindexFromLookup(const TArray<int32>& InverseNewOrder) { }

	/**
	* Init from a predefined Array
	*/
	virtual void Init(const FManagedArrayBase& ) {};

	/**
	* Convert from a predefined Array, the managed array itself should have defined its conversion procedure
	*/
	virtual void Convert(const FManagedArrayBase&) { ensureMsgf(false, TEXT("Type change not supported")); /* This type has no conversion process defined*/ };

	/**
	* Copy a range of values from the ConstArray into this
	*/
	virtual void CopyRange(const FManagedArrayBase& ConstArray, int32 Start, int32 Stop, int32 Offset = 0) {};

	/**
	* Set default values. 
	*/
	virtual void SetDefaults(uint32 StartSize, uint32 NumElements, bool bHasGroupIndexDependency) {};

	/**
	* Get allocated memory 
	*/
	virtual SIZE_T GetAllocatedSize() const { return 0; }

public:
	FManagedArrayBase()
	{
		ClearDirtyFlag();
	}
	
	virtual ~FManagedArrayBase() = default;

	FORCEINLINE void ClearDirtyFlag()
	{
		bIsDirty = false;
	}
	
	FORCEINLINE_DEBUGGABLE void MarkDirty()
	{
		bIsDirty = true;
	}
	
	FORCEINLINE bool IsDirty() const
	{
		return bIsDirty;
	}
	
	//todo(ocohen): these should all be private with friend access to managed collection
	
	/** Perform a memory move between the two arrays */
	virtual void ExchangeArrays(FManagedArrayBase& Src) = 0;

	/** Remove elements */
	virtual void RemoveElements(const TArray<int32>& SortedDeletionList)
	{
		check(false);
	}

	/** The length of the array.*/
	virtual int32 Num() const 
	{
		return 0; 
	};

	/** The reserved length of the array.*/
	virtual int32 Max() const
	{
		return 0;
	};

	/** Serialization */
	virtual void Serialize(Chaos::FChaosArchive& Ar) 
	{
		check(false);
	}

	/** TypeSize */
	virtual size_t GetTypeSize() const
	{
		return 0;
	}

	/**
	* Reindex - Adjust index dependent elements.  
	*   Offsets is the size of the dependent group;
	*   Final is post resize of dependent group used for bounds checking on remapped indices.
	*/
	virtual void Reindex(const TArray<int32> & Offsets, const int32 & FinalSize, const TArray<int32> & SortedDeletionList, const TSet<int32> & DeletionSet) { }

#if 0 //not needed until per instance serialization
	/** Swap elements*/
	virtual void Swap(int32 Index1, int32 Index2) = 0;
#endif

	/** Empty the array. */
	virtual void Empty()
	{
		check(false);
	}
private:
	bool bIsDirty;
};

template <typename T>
class TManagedArrayBase;

template <typename T>
void InitHelper(TArray<T>& Array, const TManagedArrayBase<T>& NewTypedArray, int32 Size);
template <typename T>
void InitHelper(TArray<TUniquePtr<T>>& Array, const TManagedArrayBase<TUniquePtr<T>>& NewTypedArray, int32 Size);
template <typename T>
void InitHelper(TArray<TRefCountPtr<T>>& Array, const TManagedArrayBase<TRefCountPtr<T>>& NewTypedArray, int32 Size);
template <typename T>
void CopyRangeHelper(TArray<T>& Target, const TManagedArrayBase<T>& Source, int32 Start, int32 Stop, int32 Offset);
template <typename T>
void CopyRangeHelper(TArray<TUniquePtr<T>>& Array, const TManagedArrayBase<TUniquePtr<T>>& ConstArray, int32 Start, int32 Stop, int32 Offset);
template <typename T>
void CopyRangeHelper(TArray<TRefCountPtr<T>>& Array, const TManagedArrayBase<TRefCountPtr<T>>& ConstArray, int32 Start, int32 Stop, int32 Offset);

/***
*  Managed Array
*
*  Restricts clients ability to resize the array external to the containing manager. 
*/
template<class InElementType>
class TManagedArrayBase : public FManagedArrayBase
{

public:

	using ElementType = InElementType;

	/**
	* Constructor (default) Build an empty shared array
	*
	*/	
	FORCEINLINE TManagedArrayBase()
	{}

	/**
	* Constructor (TArray)
	*
	*/
	FORCEINLINE TManagedArrayBase(const TArray<ElementType>& Other)
		: Array(Other)
	{}

	/**
	* Copy Constructor (default)
	*/
	FORCEINLINE TManagedArrayBase(const TManagedArrayBase<ElementType>& Other) = delete;

	/**
	* Move Constructor
	*/
	FORCEINLINE TManagedArrayBase(TManagedArrayBase<ElementType>&& Other)
		: Array(MoveTemp(Other.Array))
	{}
	FORCEINLINE TManagedArrayBase(TArray<ElementType>&& Other)
		: Array(MoveTemp(Other))
	{}

	/**
	* Assignment operator
	*/
	FORCEINLINE TManagedArrayBase& operator=(TManagedArrayBase<ElementType>&& Other)
	{
		return this->operator=(MoveTemp(Other.Array));
	}

	FORCEINLINE TManagedArrayBase& operator=(TArray<ElementType>&& Other)
	{
		// ryan - is it okay to check that the size matches?
		ensureMsgf(Array.Num() == 0 || Array.Num() == Other.Num(), TEXT("TManagedArrayBase<T>::operator=(TArray<T>&&) : Invalid array size."));
		Array = MoveTemp(Other);
		return *this;
	}

	/**
	* Virtual Destructor 
	*
	*/
	virtual ~TManagedArrayBase() = default;


	virtual void RemoveElements(const TArray<int32>& SortedDeletionList) override
	{
		if (SortedDeletionList.Num() == 0)
		{
			return;
		}

		int32 RangeStart = SortedDeletionList.Last();
		for (int32 ii = SortedDeletionList.Num()-1 ; ii > -1 ; --ii)
		{
			if (ii == 0)
			{
				Array.RemoveAt(SortedDeletionList[0], RangeStart - SortedDeletionList[0] + 1, EAllowShrinking::No);

			}
			else if (SortedDeletionList[ii] != (SortedDeletionList[ii - 1]+1)) // compare this and previous values to make sure the difference is only 1.
			{
				Array.RemoveAt(SortedDeletionList[ii], RangeStart - SortedDeletionList[ii] + 1, EAllowShrinking::No);
				RangeStart = SortedDeletionList[ii-1];
			}
		}

 		Array.Shrink();
	}

	/**
	* Init from a predefined Array of matching type
	*/
	virtual void Init(const FManagedArrayBase& NewArray) override
	{
		ensureMsgf(NewArray.GetTypeSize() == GetTypeSize(),TEXT("TManagedArrayBase<T>::Init : Invalid array types."));
		const TManagedArrayBase<ElementType> & NewTypedArray = static_cast< const TManagedArrayBase<ElementType>& >(NewArray);
		int32 Size = NewTypedArray.Num();

		Resize(Size);
		InitHelper(Array, NewTypedArray, Size);
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return ManagedArrayTypeSize::GetAllocatedSize(Array);
	}

	/**
	* Copy from a predefined Array of matching type
	*/
	virtual void CopyRange(const FManagedArrayBase& ConstArray, int32 Start, int32 Stop, int32 Offset = 0) override
	{
		ensureMsgf(ConstArray.GetTypeSize() == GetTypeSize(), TEXT("TManagedArrayBase<T>::Init : Invalid array types."));
		if (ensureMsgf(Stop + Offset <= Array.Num(), TEXT("Error : Index out of bounds")))
		{
			const TManagedArrayBase<ElementType>& TypedConstArray = static_cast<const TManagedArrayBase<ElementType>&>(ConstArray);
			CopyRangeHelper(Array, TypedConstArray, Start, Stop, Offset);
		}
	}

	/**
	 * Fill the array with \p Value.
	 */
	void Fill(const ElementType& Value)
	{
		for (int32 Idx = 0; Idx < Array.Num(); ++Idx)
			Array[Idx] = Value;
	}

#if 0
	virtual void Swap(int32 Index1, int32 Index2) override
	{
		Exchange(Array[Index1], Array[Index2]);
	}
#endif

	virtual void ExchangeArrays(FManagedArrayBase& NewArray) override
	{
		//It's up to the caller to make sure that the two arrays are of the same type
		ensureMsgf(NewArray.GetTypeSize() == GetTypeSize(), TEXT("TManagedArrayBase<T>::Exchange : Invalid array types."));
		TManagedArrayBase<ElementType>& NewTypedArray = static_cast<TManagedArrayBase<ElementType>& >(NewArray);

		Exchange(*this, NewTypedArray);
	}

	/**
	* Returning a reference to the element at index.
	*
	* @returns Array element reference
	*/
	FORCEINLINE ElementType & operator[](int Index)
	{
		// @todo : optimization
		// TArray->operator(Index) will perform checks against the 
		// the array. It might be worth implementing the memory
		// management directly on the ManagedArray, to avoid the
		// overhead of the TArray.
		return Array[Index];
	}
	FORCEINLINE const ElementType & operator[](int Index) const
	{
		return Array[Index];
	}

	/**
	* Helper function for returning the internal const array
	*
	* @returns const array of all the elements
	*/
	FORCEINLINE const TArray<ElementType>& GetConstArray()
	{
		return Array;
	}

	FORCEINLINE const TArray<ElementType>& GetConstArray() const
	{
		return Array;
	}
	
	/**
	* Helper function for returning a typed pointer to the first array entry.
	*
	* @returns Pointer to first array entry or nullptr if ArrayMax == 0.
	*/
	FORCEINLINE ElementType* GetData()
	{
		return Array.GetData();
	}

	/**
	* Helper function for returning a typed pointer to the first array entry.
	*
	* @returns Pointer to first array entry or nullptr if ArrayMax == 0.
	*/
	FORCEINLINE const ElementType * GetData() const
	{
		return Array.GetData();
	}

	/**
	* Helper function returning the size of the inner type.
	*
	* @returns Size in bytes of array type.
	*/
	FORCEINLINE size_t GetTypeSize() const override
	{
		return sizeof(ElementType);
	}

	/**
	* Returning the size of the array
	*
	* @returns Array size
	*/
	FORCEINLINE int32 Num() const override
	{
		return Array.Num();
	}

	FORCEINLINE int32 Max() const override
	{
		return Array.Max();
	}

	FORCEINLINE bool Contains(const ElementType& Item) const
	{
		return Array.Contains(Item);
	}

	/**
	* Find first index of the element
	*/
	int32 Find(const ElementType& Item) const
	{
		return Array.Find(Item);
	}

	/**
	* Count the number of entries match \p Item.
	*/
	int32 Count(const ElementType& Item) const
	{
		int32 Num = 0;
		for (int32 Idx = 0; Idx < Array.Num(); ++Idx)
			Num += Array[Idx] == Item ? 1 : 0;
		return Num;
	}

	/**
	* return true if index is in array range.
	*
	* @param Index Index to check.
	*/
	FORCEINLINE bool IsValidIndex(int32 Index) const
	{
		return Array.IsValidIndex(Index);
	}

	/**
	* Checks if index is in array range.
	*
	* @param Index Index to check.
	*/
	FORCEINLINE void RangeCheck(int32 Index) const
	{
		checkf((Index >= 0) & (Index < Array.Num()), TEXT("Array index out of bounds: %i from an array of size %i"), Index, Array.Num());
	}

	/**
	* Serialization Support
	*
	* @param Chaos::FChaosArchive& Ar
	*/
	virtual void Serialize(Chaos::FChaosArchive& Ar)
	{		
		Ar.UsingCustomVersion(FDestructionObjectVersion::GUID);
		int Version = 1;
		Ar << Version;
	
		if (Ar.CustomVer(FDestructionObjectVersion::GUID) < FDestructionObjectVersion::BulkSerializeArrays)
		{
			Ar << Array;
		}
		else
		{
			TryBulkSerializeManagedArray(Ar, Array);
		}
	}

	/**
	* This hash is using HashCombineFast and should not be serialized!
	*/
	FORCEINLINE uint32 GetTypeHash() const
	{
		return GetArrayHash(Array.GetData(), Array.Num());
	}

	typedef typename TArray<InElementType>::RangedForIteratorType		RangedForIteratorType;
	typedef typename TArray<InElementType>::RangedForConstIteratorType	RangedForConstIteratorType;

	/**
	* DO NOT USE DIRECTLY
	* STL-like iterators to enable range-based for loop support.
	*/
	FORCEINLINE RangedForIteratorType      begin()			{ return Array.begin(); }
	FORCEINLINE RangedForConstIteratorType begin() const	{ return Array.begin(); }
	FORCEINLINE RangedForIteratorType      end  ()			{ return Array.end(); }
	FORCEINLINE RangedForConstIteratorType end  () const	{ return Array.end(); }

protected:
	/**
	* Protected Resize to prevent external resizing of the array
	*
	* @param New array size.
	*/
	void Resize(const int32 Size) 
	{ 
		Array.SetNum(Size,EAllowShrinking::Yes);
	}

	/**
	* Protected Reserve to prevent external reservation of the array
	*
	* @param New array reservation size.
	*/
	void Reserve(const int32 Size)
	{
		Array.Reserve(Size);
	}

	/**
	* Protected clear to prevent external clearing of the array
	*
	*/
	void Empty()
	{
		Array.Empty();
	}

	void Reorder(const TArray<int32>& NewOrder) override
	{
		const int32 NumElements = Num();
		check(NewOrder.Num() == NumElements);
		TArray<InElementType> NewArray;
		NewArray.AddDefaulted(NumElements);
		for (int32 OriginalIdx = 0; OriginalIdx < NumElements; ++OriginalIdx)
		{
			NewArray[OriginalIdx] = MoveTemp(Array[NewOrder[OriginalIdx]]);
		}
		Exchange(Array, NewArray);
	}

	TArray<InElementType> Array;

};

template<class T>
inline uint32 GetTypeHash(const TManagedArrayBase<T>& ManagedArray)
{
	return ManagedArray.GetTypeHash();
}

template <typename T>
void InitHelper(TArray<T>& Array, const TManagedArrayBase<T>& NewTypedArray, int32 Size)
{
	check(Size <= Array.Num() && Size <= NewTypedArray.Num());
	CopyAssignItems(Array.GetData(), NewTypedArray.GetData(), Size);
}


template <typename TSrc, typename TDst>
void InitHelper(TArray<TDst>& Array, const TManagedArrayBase<TSrc>& NewTypedArray, int32 Size)
{
	for (int32 Index = 0; Index < Size; Index++)
	{
		Array[Index] = TDst(NewTypedArray[Index]);
	}
}

template <typename T>
void InitHelper(TArray<TUniquePtr<T>>& Array, const TManagedArrayBase<TUniquePtr<T>>& NewTypedArray, int32 Size)
{
	check(false);
}

template <typename T>
void InitHelper(TArray<TRefCountPtr<T>>& Array, const TManagedArrayBase<TRefCountPtr<T>>& NewTypedArray, int32 Size)
{
	for (int32 Index = 0; Index < Size; Index++)
	{
		if (NewTypedArray[Index])
		{
			Array[Index] = NewTypedArray[Index];
		}
	}
}

template <typename T>
void CopyRangeHelper(TArray<T>& Target, const TManagedArrayBase<T>& Source, int32 Start, int32 Stop, int32 Offset)
{
	const int32 NumCopy = FMath::Min(FMath::Min(Source.Num() - Start, Stop - Start), Target.Num() - Start - Offset);
	if (NumCopy > 0)
	{
		check(Start >= 0 && Start + Offset >= 0);
		CopyAssignItems(Target.GetData() + Start + Offset, Source.GetData() + Start, NumCopy);
	}
}

template <typename T>
void CopyRangeHelper(TArray<TUniquePtr<T>>& Target, const TManagedArrayBase<TUniquePtr<T>>& Source, int32 Start, int32 Stop, int32 Offset)
{
	check(false);
}

template <typename T>
void CopyRangeHelper(TArray<TRefCountPtr<T>>& Target, const TManagedArrayBase<TRefCountPtr<T>>& Source, int32 Start, int32 Stop, int32 Offset)
{
	for (int32 Sdx = Start, Tdx = Start+Offset; Sdx<Source.Num() && Tdx<Target.Num() && Sdx<Stop; Sdx++, Tdx++)
	{
		Target[Tdx] = Source[Sdx];
	}
}

/***
*  BitArray Managed Array base
*/
class FManagedBitArrayBase : public FManagedArrayBase
{

public:

	FORCEINLINE FManagedBitArrayBase()
	{}

	FORCEINLINE FManagedBitArrayBase(const FManagedBitArrayBase& Other) = delete;

	FORCEINLINE FManagedBitArrayBase(FManagedBitArrayBase&& Other)
		: Array(MoveTemp(Other.Array))
	{}

	FORCEINLINE FManagedBitArrayBase& operator=(FManagedBitArrayBase&& Other)
	{
		return this->operator=(MoveTemp(Other.Array));
	}

	FORCEINLINE FManagedBitArrayBase& operator=(TBitArray<>&& Other)
	{
		// ryan - is it okay to check that the size matches?
		ensureMsgf(Array.Num() == 0 || Array.Num() == Other.Num(), TEXT("FManagedBitArrayBase::operator=(TArray<T>&&) : Invalid array size."));
		Array = MoveTemp(Other);
		return *this;
	}

	virtual ~FManagedBitArrayBase() = default;

	virtual void RemoveElements(const TArray<int32>& SortedDeletionList) override
	{
		if (SortedDeletionList.Num() == 0)
		{
			return;
		}

		// try to batch as many element as possible
		int32 RangeStart = SortedDeletionList.Last();
		for (int32 ii = SortedDeletionList.Num() - 1; ii > -1; --ii)
		{
			const int32 NumTopRemove = (RangeStart - SortedDeletionList[ii] + 1);
			if (ii == 0)
			{
				Array.RemoveAt(SortedDeletionList[0], NumTopRemove);
			}
			else if (SortedDeletionList[ii] != (SortedDeletionList[ii - 1] + 1)) // compare this and previous values to make sure the difference is only 1.
			{
				Array.RemoveAt(SortedDeletionList[ii], NumTopRemove);
				RangeStart = SortedDeletionList[ii - 1];
			}
		}
	}

	/**
	* Init from a predefined Array of matching type
	*/
	virtual void Init(const FManagedArrayBase& NewArray) override
	{
		ensureMsgf(NewArray.GetTypeSize() == GetTypeSize(), TEXT("FManagedBitArrayBase::Init : Invalid array types."));
		const FManagedBitArrayBase& TypedConstArray = static_cast<const FManagedBitArrayBase&>(NewArray);
		
		const int32 Size = TypedConstArray.Num();
		Resize(Size);
		for (int32 Index = 0; Index < Size; Index++)
		{
			Array[Index] = TypedConstArray[Index];
		}
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return Array.GetAllocatedSize();
	}

	/**
	* Copy from a predefined Array of matching type
	*/
	virtual void CopyRange(const FManagedArrayBase& ConstArray, int32 Start, int32 Stop, int32 Offset = 0) override
	{
		ensureMsgf(ConstArray.GetTypeSize() == GetTypeSize(), TEXT("TManagedArrayBase<T>::Init : Invalid array types."));
		if (ensureMsgf(Stop + Offset <= Array.Num(), TEXT("Error : Index out of bounds")))
		{
			const FManagedBitArrayBase& TypedConstArray = static_cast<const FManagedBitArrayBase&>(ConstArray);
			for (int32 Sdx = Start, Tdx = Start + Offset; Sdx < ConstArray.Num() && Tdx < Array.Num() && Sdx < Stop; Sdx++, Tdx++)
			{
				Array[Tdx] = TypedConstArray[Sdx];
			}
		}
	}

	/**
	 * Fill the array with \p Value.
	 */
	void Fill(const bool Value)
	{
		for (int32 Idx = 0; Idx < Array.Num(); ++Idx)
		{
			Array[Idx] = Value;
		}
	}

	void Fill(const TArray<bool>& BoolArray)
	{
		check(BoolArray.Num() == Array.Num());
		for (int32 Idx = 0; Idx < Array.Num(); Idx++)
		{
			Array[Idx] = BoolArray[Idx];
		}
	}

	virtual void ExchangeArrays(FManagedArrayBase& NewArray) override
	{
		//It's up to the caller to make sure that the two arrays are of the same type
		ensureMsgf(NewArray.GetTypeSize() == GetTypeSize(), TEXT("FManagedBitArrayBase::Exchange : Invalid array types."));
		FManagedBitArrayBase& NewTypedArray = static_cast<FManagedBitArrayBase&>(NewArray);

		Exchange(*this, NewTypedArray);
	}
	
	/**
	* return true if index is in array range.
	*
	* @param Index Index to check.
	*/
	FORCEINLINE bool IsValidIndex(int32 Index) const
	{
		return Array.IsValidIndex(Index);
	}

	/**
	* Returning a reference to the element at index.
	*
	* @returns Array element reference
	*/
	FORCEINLINE FBitReference operator[](int Index)
	{
		// @todo : optimization
		// TArray->operator(Index) will perform checks against the 
		// the array. It might be worth implementing the memory
		// management directly on the ManagedArray, to avoid the
		// overhead of the TArray.
		return Array[Index];
	}
	FORCEINLINE const FConstBitReference operator[](int Index) const
	{
		return Array[Index];
	}

	/**
	* Helper function for returning the internal const array
	*
	* @returns const array of all the elements
	*/
	FORCEINLINE const TBitArray<>& GetConstArray()
	{
		return Array;
	}

	FORCEINLINE const TBitArray<>& GetConstArray() const
	{
		return Array;
	}

	/**
	* Helper function to convert the bitArray to a bool array 
	* this is creating a new array and copy convert each bit to a bool
	*/
	TArray<bool> GetAsBoolArray() const
	{
		TArray<bool> BoolArray;
		BoolArray.SetNumUninitialized(Array.Num());
		for (int32 Idx = 0; Idx < Array.Num(); Idx++)
		{
			BoolArray[Idx] = Array[Idx];
		}
		return BoolArray;
	}


	/**
	* Helper function for returning a typed pointer to the first array entry.
	*
	* @returns Pointer to first array entry or nullptr if ArrayMax == 0.
	*/
	//FORCEINLINE ElementType* GetData()
	//{
	//	return Array.GetData();
	//}

	/**
	* Helper function for returning a typed pointer to the first array entry.
	*
	* @returns Pointer to first array entry or nullptr if ArrayMax == 0.
	*/
	//FORCEINLINE const ElementType* GetData() const
	//{
	//	return Array.GetData();
	//}

	/**
	* Helper function returning the size of the inner type.
	*
	* @returns Size in bytes of array type.
	*/
	FORCEINLINE size_t GetTypeSize() const override
	{
		// this is not true but this ios the smallest we can represent
		return 1;
	}

	/**
	* Returning the size of the array
	*
	* @returns Array size
	*/
	FORCEINLINE int32 Num() const override
	{
		return Array.Num();
	}

	FORCEINLINE int32 Max() const override
	{
		return Array.Max();
	}

	FORCEINLINE bool Contains(const bool Item) const
	{
		return Array.Contains(Item);
	}

	/**
	* Find first index of the element
	*/
	int32 Find(const bool Item) const
	{
		return Array.Find(Item);
	}

	/**
	* Count the number of entries match \p Item.
	*/
	int32 Count(const bool Item) const
	{
		const int32 NumSetBits = Array.CountSetBits();
		return (Item) ? NumSetBits : (Array.Num() - NumSetBits);
	}

	/**
	* Checks if index is in array range.
	*
	* @param Index Index to check.
	*/
	FORCEINLINE void RangeCheck(int32 Index) const
	{
		checkf((Index >= 0) & (Index < Array.Num()), TEXT("Array index out of bounds: %i from an array of size %i"), Index, Array.Num());
	}

	/**
	* Serialization Support
	*
	* @param Chaos::FChaosArchive& Ar
	*/
	virtual void Serialize(Chaos::FChaosArchive& Ar)
	{
		// we need to keep the backward compatibility with TManagedArray<bool> when it inherited from TManagedArrayBase<T>
		Ar.UsingCustomVersion(FDestructionObjectVersion::GUID);
		int Version = 1;
		Ar << Version;

		// for now always go through a bool array, in the future we can have a more optimized path 
		TArray<bool> BoolArray;
		if (Ar.IsSaving())
		{
			BoolArray = GetAsBoolArray();
		}
		if (Ar.CustomVer(FDestructionObjectVersion::GUID) < FDestructionObjectVersion::BulkSerializeArrays)
		{
			Ar << BoolArray;
		}
		else
		{
			TryBulkSerializeManagedArray(Ar, BoolArray);
		}
		if (Ar.IsLoading())
		{
			Resize(BoolArray.Num());
			Fill(BoolArray);
		}

	}
protected:
	/**
	* Protected Resize to prevent external resizing of the array
	*
	* @param New array size.
	*/
	void Resize(const int32 Size)
	{
		if (Size > Array.Num())
		{
			Array.Add(false, Size - Array.Num());
		}
		else if (Size < Array.Num())
		{
			const int32 NumToRemove = (Array.Num() - Size);
			Array.RemoveAt((Array.Num() - NumToRemove), NumToRemove);
		}
	}

	/**
	* Protected Reserve to prevent external reservation of the array
	*
	* @param New array reservation size.
	*/
	void Reserve(const int32 Size)
	{
		Array.Reserve(Size);
	}

	/**
	* Protected clear to prevent external clearing of the array
	*
	*/
	void Empty()
	{
		Array.Empty();
	}

	void Reorder(const TArray<int32>& NewOrder) override
	{
		const int32 NumElements = Num();
		check(NewOrder.Num() == NumElements);
		TBitArray<> NewArray(false, NumElements);
		for (int32 OriginalIdx = 0; OriginalIdx < NumElements; ++OriginalIdx)
		{
			NewArray[OriginalIdx] = Array[NewOrder[OriginalIdx]];
		}
		Exchange(Array, NewArray);
	}

	TBitArray<> Array;
};

//
//
//
#define UNSUPPORTED_UNIQUE_ARRAY_COPIES(TYPE, NAME) \
template<> inline void InitHelper(TArray<TYPE>& Array, const TManagedArrayBase<TYPE>& NewTypedArray, int32 Size) { \
	UE_LOGF(LogChaos,Warning, "Cannot make a copy of unique array of type (%ls) within the managed array collection. Regenerate unique pointer attributes if needed.", NAME); }\
template<> inline void CopyRangeHelper(TArray<TYPE>& Target, const TManagedArrayBase<TYPE>& Source, int32 Start, int32 Stop, int32 Offset) {\
	UE_LOGF(LogChaos, Warning, "Cannot make a range copy of unique array of type (%ls) within the managed array collection. Regenerate unique pointer attributes if needed.", NAME); \
}

typedef TUniquePtr<Chaos::TGeometryParticle<Chaos::FReal, 3>> LOCAL_MA_UniqueTGeometryParticle;
UNSUPPORTED_UNIQUE_ARRAY_COPIES(LOCAL_MA_UniqueTGeometryParticle, TEXT("Chaos::TGeometryParticle"));

typedef TUniquePtr<Chaos::TPBDRigidParticle<Chaos::FReal, 3>> LOCAL_MA_UniqueTPBDRigidParticle;
UNSUPPORTED_UNIQUE_ARRAY_COPIES(LOCAL_MA_UniqueTPBDRigidParticle, TEXT("Chaos::TPBDRigidParticle"));

typedef TUniquePtr<Chaos::FBVHParticles, TDefaultDelete<Chaos::FBVHParticles>> LOCAL_MA_UniqueTBVHParticles;
UNSUPPORTED_UNIQUE_ARRAY_COPIES(LOCAL_MA_UniqueTBVHParticles, TEXT("Chaos::FBVHParticles"));

typedef TUniquePtr<TArray<UE::Math::TVector<float>>> LOCAL_MA_UniqueTArrayTVector;
UNSUPPORTED_UNIQUE_ARRAY_COPIES(LOCAL_MA_UniqueTArrayTVector, TEXT("TArray<UE::Math::TVector<float>>"));

template<class InElementType>
class TManagedArray : public TManagedArrayBase<InElementType>
{
public:
	FORCEINLINE TManagedArray()
	{}

	FORCEINLINE TManagedArray(const TArray<InElementType>& Other)
		: TManagedArrayBase<InElementType>(Other)
	{}

	FORCEINLINE TManagedArray(TManagedArray<InElementType>&& Other)
		: TManagedArrayBase<InElementType>(MoveTemp(Other))
	{}

	FORCEINLINE TManagedArray(TArray<InElementType>&& Other)
		: TManagedArrayBase<InElementType>(MoveTemp(Other))
	{}

	FORCEINLINE TManagedArray& operator=(TManagedArray<InElementType>&& Other)
	{
		TManagedArrayBase<InElementType>::operator=(MoveTemp(Other));
		return *this;
	}

	FORCEINLINE TManagedArray(const TManagedArray<InElementType>& Other) = delete;

	virtual ~TManagedArray() = default;
};

template<>
class TManagedArray<int32> : public TManagedArrayBase<int32>
{
public:
    using TManagedArrayBase<int32>::Num;

	FORCEINLINE TManagedArray()
	{}

	FORCEINLINE TManagedArray(const TArray<int32>& Other)
		: TManagedArrayBase<int32>(Other)
	{}

	FORCEINLINE TManagedArray(const TManagedArray<int32>& Other) = delete;
	FORCEINLINE TManagedArray(TManagedArray<int32>&& Other) = default;
	FORCEINLINE TManagedArray(TArray<int32>&& Other)
		: TManagedArrayBase<int32>(MoveTemp(Other))
	{}
	FORCEINLINE TManagedArray& operator=(TManagedArray<int32>&& Other) = default;

	virtual ~TManagedArray() = default;

	virtual void Reindex(const TArray<int32> & Offsets, const int32 & FinalSize, const TArray<int32> & SortedDeletionList, const TSet<int32>& DeletionSet) override
	{
		UE_LOGF(LogChaos, VeryVerbose, "TManagedArray<int32>[%p]::Reindex()",this);

		int32 ArraySize = Num(), MaskSize = Offsets.Num();
		for (int32 Index = 0; Index < ArraySize; Index++)
		{
			int32 RemapVal = this->operator[](Index);
			if (0 <= RemapVal)
			{
				ensure(RemapVal < MaskSize);
				if (DeletionSet.Contains(this->operator[](Index)))
				{
					this->operator[](Index) = INDEX_NONE;
				}
				else
				{
					this->operator[](Index) -= Offsets[RemapVal];
				}
				ensure(-1 <= this->operator[](Index));
			}
		}
	}

	virtual void ReindexFromLookup(const TArray<int32>& InverseNewOrder) override
	{
		const int32 ArraySize = Num();
		for (int32 Index = 0; Index < ArraySize; ++Index)
		{
			int32& Mapping = this->operator[](Index);
			if (Mapping >= 0)
			{
				Mapping = InverseNewOrder[Mapping];
			}
		}
	}

	virtual void SetDefaults(uint32 StartSize, uint32 NumElements, bool bHasGroupIndexDependency) override
	{
		if (bHasGroupIndexDependency)
		{
			for (uint32 Index = StartSize; Index < StartSize + NumElements; ++Index)
			{
				this->operator[](Index) = INDEX_NONE;
			}
		}
	}
};


template<>
class TManagedArray<FTransform3f> : public TManagedArrayBase<FTransform3f>
{
public:
	FORCEINLINE TManagedArray()
	{}

	FORCEINLINE TManagedArray(const TArray<FTransform3f>& Other)
		: TManagedArrayBase<FTransform3f>(Other)
	{}

	FORCEINLINE TManagedArray(const TManagedArray<FTransform3f>& Other) = delete;
	FORCEINLINE TManagedArray(TManagedArray<FTransform3f>&& Other) = default;
	FORCEINLINE TManagedArray(TArray<FTransform3f>&& Other)
		: TManagedArrayBase<FTransform3f>(MoveTemp(Other))
	{}
	FORCEINLINE TManagedArray& operator=(TManagedArray<FTransform3f>&& Other) = default;

	virtual ~TManagedArray() = default;

protected:
	/**
	* Init from a predefined Array of matching type
	*/
	virtual void Convert(const FManagedArrayBase& NewArray) override
	{
		check(NewArray.GetTypeSize() != GetTypeSize());
		check(NewArray.GetTypeSize() == 2 * GetTypeSize());
		check(NewArray.GetTypeSize() == sizeof(FTransform));
		const TManagedArrayBase<FTransform>& NewTypedArray = static_cast<const TManagedArrayBase<FTransform>&>(NewArray);
		const int32 Size = NewTypedArray.Num();
		Resize(Size);
		InitHelper(Array, NewTypedArray, Size);
	}
};

template<>
class TManagedArray<bool> : public FManagedBitArrayBase
{
protected:
	/**
	* Init from a predefined Array of matching type
	*/
	virtual void Convert(const FManagedArrayBase& NewArray) override
	{
		check(NewArray.GetTypeSize() != GetTypeSize());
		check(NewArray.GetTypeSize() == sizeof(int32));
		
		const TManagedArrayBase<int32>& NewTypedArray = static_cast<const TManagedArrayBase<int32>&>(NewArray);
		const int32 Size = NewTypedArray.Num();
		Resize(Size);
		for (int32 Index = 0; Index < Size; Index++)
		{
			Array[Index] = NewTypedArray[Index] != INDEX_NONE;
		}
	}
};

template<>
class TManagedArray<TSet<int32>> : public TManagedArrayBase<TSet<int32>>
{
public:
	using TManagedArrayBase<TSet<int32>>::Num;

	FORCEINLINE TManagedArray()
	{}

	FORCEINLINE TManagedArray(const TArray<TSet<int32>>& Other)
		: TManagedArrayBase< TSet<int32> >(Other)
	{}

	FORCEINLINE TManagedArray(const TManagedArray<TSet<int32>>& Other) = delete;
	FORCEINLINE TManagedArray(TManagedArray<TSet<int32>>&& Other) = default;
	FORCEINLINE TManagedArray(TArray<TSet<int32>>&& Other)
		: TManagedArrayBase<TSet<int32>>(MoveTemp(Other))
	{}
	FORCEINLINE TManagedArray& operator=(TManagedArray<TSet<int32>>&& Other) = default;

	virtual ~TManagedArray() = default;
	
	virtual void Reindex(const TArray<int32> & Offsets, const int32 & FinalSize, const TArray<int32> & SortedDeletionList, const TSet<int32>& DeletionSet) override
	{
		UE_LOGF(LogChaos, VeryVerbose, "TManagedArray<TArray<int32>>[%p]::Reindex()", this);
		
		int32 ArraySize = Num(), MaskSize = Offsets.Num();

		for (int32 Index = 0; Index < ArraySize; Index++)
		{
			TSet<int32>& NewSet = this->operator[](Index);
			
			for (int32 Del : SortedDeletionList)
			{
				NewSet.Remove(Del);
			}

			TSet<int32> OldSet = this->operator[](Index);	//need a copy since we're modifying the entries in the set (can't edit in place because value desyncs from hash)
			NewSet.Reset();	//maybe we should remove 

			for (int32 StaleEntry : OldSet)
			{
				const int32 NewEntry = StaleEntry >= 0 ? StaleEntry - Offsets[StaleEntry] : StaleEntry;	//only remap if valid
				NewSet.Add(NewEntry);
			}
		}
	}

	virtual void ReindexFromLookup(const TArray<int32> & InverseNewOrder) override
	{

		int32 ArraySize = Num();
		
		for (int32 Index = 0; Index < ArraySize; Index++)
		{
			TSet<int32>& NewSet = this->operator[](Index);
			TSet<int32> OldSet = this->operator[](Index);	//need a copy since we're modifying the entries in the set
			NewSet.Reset();	//maybe we should remove 

			for (int32 StaleEntry : OldSet)
			{
				const int32 NewEntry = StaleEntry >= 0 ? InverseNewOrder[StaleEntry] : StaleEntry;	//only remap if valid
				NewSet.Add(NewEntry);
			}
		}
	}
};

template<>
class TManagedArray<FIntVector> : public TManagedArrayBase<FIntVector>
{
public:
    using TManagedArrayBase<FIntVector>::Num;

	FORCEINLINE TManagedArray()
	{}

	FORCEINLINE TManagedArray(const TArray<FIntVector>& Other)
		: TManagedArrayBase<FIntVector>(Other)
	{}

	FORCEINLINE TManagedArray(const TManagedArray<FIntVector>& Other) = delete;
	FORCEINLINE TManagedArray(TManagedArray<FIntVector>&& Other) = default;
	FORCEINLINE TManagedArray(TArray<FIntVector>&& Other)
		: TManagedArrayBase<FIntVector>(MoveTemp(Other))
	{}
	FORCEINLINE TManagedArray& operator=(TManagedArray<FIntVector>&& Other) = default;

	virtual ~TManagedArray() = default;

	virtual void Reindex(const TArray<int32> & Offsets, const int32 & FinalSize, const TArray<int32> & SortedDeletionList, const TSet<int32>& DeletionSet) override
	{
		UE_LOGF(LogChaos, VeryVerbose, "TManagedArray<FIntVector>[%p]::Reindex()", this);
		int32 ArraySize = Num(), MaskSize = Offsets.Num();
		for (int32 Index = 0; Index < ArraySize; Index++)
		{
			const FIntVector & RemapVal = this->operator[](Index);
			for (int i = 0; i < 3; i++)
			{
				if (0 <= RemapVal[i])
				{
					ensure(RemapVal[i] < MaskSize);
					if (DeletionSet.Contains(this->operator[](Index)[i]))
					{
						this->operator[](Index)[i] = INDEX_NONE;
					}
					else
					{
						this->operator[](Index)[i] -= Offsets[RemapVal[i]];
					}
					ensure(-1 <= this->operator[](Index)[i] && this->operator[](Index)[i] <= FinalSize);
				}
			}
		}
	}

	virtual void ReindexFromLookup(const TArray<int32> & InverseNewOrder) override
	{
		int32 ArraySize = Num();
		for (int32 Index = 0; Index < ArraySize; Index++)
		{
			FIntVector& RemapVal = this->operator[](Index);
			for (int i = 0; i < 3; i++)
			{
				if (RemapVal[i] >= 0)
				{
					RemapVal[i] = InverseNewOrder[RemapVal[i]];
				}
			}
		}
	}

	virtual void SetDefaults(uint32 StartSize, uint32 NumElements, bool bHasGroupIndexDependency) override
	{
		if (bHasGroupIndexDependency)
		{
			for (uint32 Index = StartSize; Index < StartSize + NumElements; ++Index)
			{
				this->operator[](Index) = FIntVector(INDEX_NONE);
			}
		}
	}
};

template<>
class TManagedArray<FIntVector2> : public TManagedArrayBase<FIntVector2>
{
public:
	using TManagedArrayBase<FIntVector2>::Num;

	FORCEINLINE TManagedArray()
	{}

	FORCEINLINE TManagedArray(const TArray<FIntVector2>& Other)
		: TManagedArrayBase<FIntVector2>(Other)
	{}

	FORCEINLINE TManagedArray(const TManagedArray<FIntVector2>& Other) = delete;
	FORCEINLINE TManagedArray(TManagedArray<FIntVector2>&& Other) = default;
	FORCEINLINE TManagedArray(TArray<FIntVector2>&& Other)
		: TManagedArrayBase<FIntVector2>(MoveTemp(Other))
	{}
	FORCEINLINE TManagedArray& operator=(TManagedArray<FIntVector2>&& Other) = default;

	virtual ~TManagedArray() = default;

	virtual void Reindex(const TArray<int32>& Offsets, const int32& FinalSize, const TArray<int32>& SortedDeletionList, const TSet<int32>& DeletionSet) override
	{
		UE_LOGF(LogChaos, VeryVerbose, "TManagedArray<FIntVector>[%p]::Reindex()", this);
		int32 ArraySize = Num(), MaskSize = Offsets.Num();
		for (int32 Index = 0; Index < ArraySize; Index++)
		{
			const FIntVector2& RemapVal = this->operator[](Index);
			for (int i = 0; i < 2; i++)
			{
				if (0 <= RemapVal[i])
				{
					ensure(RemapVal[i] < MaskSize);
					if (DeletionSet.Contains(this->operator[](Index)[i]))
					{
						this->operator[](Index)[i] = INDEX_NONE;
					}
					else
					{
						this->operator[](Index)[i] -= Offsets[RemapVal[i]];
					}
					ensure(-1 <= this->operator[](Index)[i] && this->operator[](Index)[i] <= FinalSize);
				}
			}
		}
	}

	virtual void ReindexFromLookup(const TArray<int32>& InverseNewOrder) override
	{
		int32 ArraySize = Num();
		for (int32 Index = 0; Index < ArraySize; Index++)
		{
			FIntVector2& RemapVal = this->operator[](Index);
			for (int i = 0; i < 2; i++)
			{
				if (RemapVal[i] >= 0)
				{
					RemapVal[i] = InverseNewOrder[RemapVal[i]];
				}
			}
		}
	}

	virtual void SetDefaults(uint32 StartSize, uint32 NumElements, bool bHasGroupIndexDependency) override
	{
		if (bHasGroupIndexDependency)
		{
			for (uint32 Index = StartSize; Index < StartSize + NumElements; ++Index)
			{
				this->operator[](Index) = FIntVector2(INDEX_NONE);
			}
		}
	}
};

template<>
class TManagedArray<TArray<FIntVector2>> : public TManagedArrayBase<TArray<FIntVector2>>
{
public:
	using TManagedArrayBase<TArray<FIntVector2>>::Num;

	FORCEINLINE TManagedArray()
	{}

	FORCEINLINE TManagedArray(const TArray<TArray<FIntVector2>>& Other)
		: TManagedArrayBase<TArray<FIntVector2>>(Other)
	{}

	FORCEINLINE TManagedArray(const TManagedArray<TArray<FIntVector2>>& Other) = delete;
	FORCEINLINE TManagedArray(TManagedArray<TArray<FIntVector2>>&& Other) = default;
	FORCEINLINE TManagedArray(TArray<TArray<FIntVector2>>&& Other)
		: TManagedArrayBase<TArray<FIntVector2>>(MoveTemp(Other))
	{}
	FORCEINLINE TManagedArray& operator=(TManagedArray<TArray<FIntVector2>>&& Other) = default;

	virtual ~TManagedArray() = default;

	virtual void Reindex(const TArray<int32>& Offsets, const int32& FinalSize, const TArray<int32>& SortedDeletionList, const TSet<int32>& DeletionSet) override
	{
		UE_LOGF(LogChaos, VeryVerbose, "TManagedArray<FIntVector>[%p]::Reindex()", this);
		int32 ArraySize = Num(), MaskSize = Offsets.Num();
		for (int32 Index = 0; Index < ArraySize; Index++)
		{
			const TArray<FIntVector2>& RemapValArray = this->operator[](Index);
			for (int32 ArrayIndex = 0; ArrayIndex < RemapValArray.Num(); ArrayIndex++)
			{
				const FIntVector2& RemapVal = RemapValArray[ArrayIndex];
				for (int i = 0; i < 2; i++)
				{
					if (0 <= RemapVal[i])
					{
						ensure(RemapVal[i] < MaskSize);
						if (DeletionSet.Contains(this->operator[](Index)[ArrayIndex][i]))
						{
							this->operator[](Index)[ArrayIndex][i] = INDEX_NONE;
						}
						else
						{
							this->operator[](Index)[ArrayIndex][i] -= Offsets[RemapVal[i]];
						}
						ensure(-1 <= this->operator[](Index)[ArrayIndex][i] && this->operator[](Index)[ArrayIndex][i] <= FinalSize);
					}
				}
			}
		}
	}

	virtual void ReindexFromLookup(const TArray<int32>& InverseNewOrder) override
	{
		int32 ArraySize = Num();
		for (int32 Index = 0; Index < ArraySize; Index++)
		{
			TArray<FIntVector2>& RemapValArray = this->operator[](Index);
			for (int32 ArrayIndex = 0; ArrayIndex < RemapValArray.Num(); ArrayIndex++)
			{
				FIntVector2& RemapVal = RemapValArray[ArrayIndex];
				for (int i = 0; i < 2; i++)
				{
					if (RemapVal[i] >= 0)
					{
						RemapVal[i] = InverseNewOrder[RemapVal[i]];
					}
				}
			}
		}
	}
};

template<>
class TManagedArray<FIntVector4> : public TManagedArrayBase<FIntVector4>
{
public:
	using TManagedArrayBase<FIntVector4>::Num;

	FORCEINLINE TManagedArray()
	{}

	FORCEINLINE TManagedArray(const TArray<FIntVector4>& Other)
		: TManagedArrayBase<FIntVector4>(Other)
	{}

	FORCEINLINE TManagedArray(const TManagedArray<FIntVector4>& Other) = delete;
	FORCEINLINE TManagedArray(TManagedArray<FIntVector4>&& Other) = default;
	FORCEINLINE TManagedArray(TArray<FIntVector4>&& Other)
		: TManagedArrayBase<FIntVector4>(MoveTemp(Other))
	{}
	FORCEINLINE TManagedArray& operator=(TManagedArray<FIntVector4>&& Other) = default;

	virtual ~TManagedArray() = default;

	virtual void Reindex(const TArray<int32>& Offsets, const int32& FinalSize, const TArray<int32>& SortedDeletionList, const TSet<int32>& DeletionSet) override
	{
		UE_LOGF(LogChaos, VeryVerbose, "TManagedArray<FIntVector>[%p]::Reindex()", this);
		int32 ArraySize = Num(), MaskSize = Offsets.Num();
		for (int32 Index = 0; Index < ArraySize; Index++)
		{
			const FIntVector4& RemapVal = this->operator[](Index);
			for (int i = 0; i < 4; i++)
			{
				if (0 <= RemapVal[i])
				{
					ensure(RemapVal[i] < MaskSize);
					if (DeletionSet.Contains(this->operator[](Index)[i]))
					{
						this->operator[](Index)[i] = INDEX_NONE;
					}
					else
					{
						this->operator[](Index)[i] -= Offsets[RemapVal[i]];
					}
					ensure(-1 <= this->operator[](Index)[i] && this->operator[](Index)[i] <= FinalSize);
				}
			}
		}
	}

	virtual void ReindexFromLookup(const TArray<int32>& InverseNewOrder) override
	{
		int32 ArraySize = Num();
		for (int32 Index = 0; Index < ArraySize; Index++)
		{
			FIntVector4& RemapVal = this->operator[](Index);
			for (int i = 0; i < 4; i++)
			{
				if (RemapVal[i] >= 0)
				{
					RemapVal[i] = InverseNewOrder[RemapVal[i]];
				}
			}
		}
	}

	virtual void SetDefaults(uint32 StartSize, uint32 NumElements, bool bHasGroupIndexDependency) override
	{
		if (bHasGroupIndexDependency)
		{
			for (uint32 Index = StartSize; Index < StartSize + NumElements; ++Index)
			{
				this->operator[](Index) = FIntVector4(INDEX_NONE);
			}
		}
	}
};

template<>
class TManagedArray<TArray<int32>> : public TManagedArrayBase<TArray<int32>>
{
public:
	using TManagedArrayBase<TArray<int32>>::Num;

	FORCEINLINE TManagedArray()
	{}

	FORCEINLINE TManagedArray(const TArray<TArray<int32>>& Other)
		: TManagedArrayBase<TArray<int32>>(Other)
	{}

	FORCEINLINE TManagedArray(const TManagedArray<TArray<int32>>& Other) = delete;
	FORCEINLINE TManagedArray(TManagedArray<TArray<int32>>&& Other) = default;
	FORCEINLINE TManagedArray(TArray<TArray<int32>>&& Other)
		: TManagedArrayBase<TArray<int32>>(MoveTemp(Other))
	{}
	FORCEINLINE TManagedArray& operator=(TManagedArray<TArray<int32>>&& Other) = default;

	virtual ~TManagedArray() = default;

	virtual void Reindex(const TArray<int32>& Offsets, const int32& FinalSize, const TArray<int32>& SortedDeletionList, const TSet<int32>& DeletionSet) override
	{
		UE_LOGF(LogChaos, VeryVerbose, "TManagedArray<FIntVector>[%p]::Reindex()", this);
		int32 ArraySize = Num(), MaskSize = Offsets.Num();
		for (int32 Index = 0; Index < ArraySize; Index++)
		{
			const TArray<int32>& RemapValArray = this->operator[](Index);
			for (int32 ArrayIndex = 0; ArrayIndex < RemapValArray.Num(); ArrayIndex++)
			{
				if (0 <= RemapValArray[ArrayIndex])
				{
					ensure(RemapValArray[ArrayIndex] < MaskSize);
					if (DeletionSet.Contains(this->operator[](Index)[ArrayIndex]))
					{
						this->operator[](Index)[ArrayIndex] = INDEX_NONE;
					}
					else
					{
						this->operator[](Index)[ArrayIndex] -= Offsets[RemapValArray[ArrayIndex]];
					}
					ensure(-1 <= this->operator[](Index)[ArrayIndex] && this->operator[](Index)[ArrayIndex] <= FinalSize);
				}
			}
		}
	}

	virtual void ReindexFromLookup(const TArray<int32>& InverseNewOrder) override
	{
		int32 ArraySize = Num();
		for (int32 Index = 0; Index < ArraySize; Index++)
		{
			TArray<int32>& RemapValArray = this->operator[](Index);
			for (int32 ArrayIndex = 0; ArrayIndex < RemapValArray.Num(); ArrayIndex++)
			{
				if (RemapValArray[ArrayIndex] >= 0)
				{
					RemapValArray[ArrayIndex] = InverseNewOrder[RemapValArray[ArrayIndex]];
				}
			}
		}
	}
};

template<>
class TManagedArray<TArray<FIntVector3>> : public TManagedArrayBase<TArray<FIntVector3>>
{
public:
	using TManagedArrayBase<TArray<FIntVector3>>::Num;

	TManagedArray() = default;

	TManagedArray(const TArray<TArray<FIntVector3>>& Other)
		: TManagedArrayBase<TArray<FIntVector3>>(Other)
	{}

	TManagedArray(const TManagedArray<TArray<FIntVector3>>& Other) = delete;
	TManagedArray(TManagedArray<TArray<FIntVector3>>&& Other) = default;
	TManagedArray(TArray<TArray<FIntVector3>>&& Other)
		: TManagedArrayBase<TArray<FIntVector3>>(MoveTemp(Other))
	{}
	
	TManagedArray& operator=(TManagedArray<TArray<FIntVector3>>&& Other) = default;

	virtual ~TManagedArray() override = default;

	virtual void Reindex(const TArray<int32>& Offsets, const int32& FinalSize, const TArray<int32>& SortedDeletionList, const TSet<int32>& DeletionSet) override
	{
		UE_LOGF(LogChaos, VeryVerbose, "TManagedArray<FIntVector>[%p]::Reindex()", this);
		const int32 ArraySize = Num();
		const int32 MaskSize = Offsets.Num();
		for (int32 Index = 0; Index < ArraySize; ++Index)
		{
			const TArray<FIntVector3>& RemapValArray = this->operator[](Index);
			for (int32 ArrayIndex = 0; ArrayIndex < RemapValArray.Num(); ++ArrayIndex)
			{
				const FIntVector3& RemapVal = RemapValArray[ArrayIndex];
				for (int32 i = 0; i < FIntVector3::Num(); ++i)
				{
					if (0 <= RemapVal[i])
					{
						ensure(RemapVal[i] < MaskSize);
						if (DeletionSet.Contains(this->operator[](Index)[ArrayIndex][i]))
						{
							this->operator[](Index)[ArrayIndex][i] = INDEX_NONE;
						}
						else
						{
							this->operator[](Index)[ArrayIndex][i] -= Offsets[RemapVal[i]];
						}
						ensure(-1 <= this->operator[](Index)[ArrayIndex][i] && this->operator[](Index)[ArrayIndex][i] <= FinalSize);
					}
				}
			}
		}
	}

	virtual void ReindexFromLookup(const TArray<int32>& InverseNewOrder) override
	{
		const int32 ArraySize = Num();
		for (int32 Index = 0; Index < ArraySize; ++Index)
		{
			TArray<FIntVector3>& RemapValArray = this->operator[](Index);
			for (int32 ArrayIndex = 0; ArrayIndex < RemapValArray.Num(); ++ArrayIndex)
			{
				FIntVector3& RemapVal = RemapValArray[ArrayIndex];
				for (int32 i = 0; i < FIntVector3::Num(); ++i)
				{
					if (RemapVal[i] >= 0)
					{
						RemapVal[i] = InverseNewOrder[RemapVal[i]];
					}
				}
			}
		}
	}
};
