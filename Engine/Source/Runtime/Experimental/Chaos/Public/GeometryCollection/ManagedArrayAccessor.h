// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "GeometryCollection/ManagedArrayCollection.h"

namespace ManageArrayAccessor
{
	enum class EPersistencePolicy : uint8
	{
		KeepExistingPersistence,
		MakePersistent
	};
};

/**
 * this class wraps a managed array
 * this provides a convenient API for optional attributes in a collection facade
 */
template <typename T>
struct TManagedArrayAccessor
{
public:
	TManagedArrayAccessor(FManagedArrayCollection& InCollection, const FName& InAttributeName, const FName& InAttributeGroup, const FName& InDefaultGroupDependency = NAME_None)
		: ConstCollection(InCollection)
		, Collection(&InCollection)
		, Name(InAttributeName)
		, Group(InAttributeGroup)
		, DefaultGroupDependency(InDefaultGroupDependency)
		, AttributeArray(InCollection.FindAttributeTyped<T>(InAttributeName, InAttributeGroup))
		, ConstAttributeArray(AttributeArray)
	{}

	TManagedArrayAccessor(const FManagedArrayCollection& InCollection, const FName& InAttributeName, const FName& InAttributeGroup, const FName& InDefaultGroupDependency = NAME_None)
		: ConstCollection(InCollection)
		, Collection(nullptr)
		, Name(InAttributeName)
		, Group(InAttributeGroup)
		, DefaultGroupDependency(InDefaultGroupDependency)
		, AttributeArray(nullptr)
		, ConstAttributeArray(InCollection.FindAttributeTyped<T>(InAttributeName, InAttributeGroup))
	{}

	const FManagedArrayCollection& GetConstCollection() const { return ConstCollection; }
	FManagedArrayCollection* GetCollection() { check(!IsConst()); return Collection; }

	FName GetName() const { return Name; }

	FName GetGroup() const { return Group; }

	FName GetGroupDependency() const { return IsValid() ? ConstCollection.GetDependency(Name, Group) : DefaultGroupDependency; }

	bool IsConst() const { return Collection == nullptr; }

	bool IsValid() const { return ConstAttributeArray != nullptr; }

	bool IsPersistent() const { return ConstCollection.IsAttributePersistent(Name, Group); }

	bool IsValidIndex(int32 Index) const 
	{ 
		check(IsValid());
		return (Index >= 0) && (Index < ConstAttributeArray->Num());
	}

	int32 AddElements(int32 NumElements) const 
	{ 
		check(!IsConst());
		return Collection->AddElements(NumElements, Group);
	}

	void RemoveElements(int32 NumElements, int32 Position)
	{
		check(!IsConst());
		return Collection->RemoveElements(Group, NumElements, Position);
	}

	void SetNumElements(int32 InNumElements)
	{
		check(!IsConst());
		check(InNumElements >= 0);
		const int32 NumElements = Num();
		if (const int32 Delta = InNumElements - NumElements)
		{
			if (Delta > 0)
			{
				Collection->AddElements(Delta, Group);
			}
			else
			{
				Collection->RemoveElements(Group, -Delta, InNumElements);
			}
		}
	}

	/** array style accessor */
	const T& operator[](int32 Index) const
	{
		check(IsValid());
		return (*ConstAttributeArray)[Index];
	}

	/** get the attribute for read only */
	const TManagedArray<T>& Get() const
	{
		check(IsValid());
		return *ConstAttributeArray;
	}

	/** find the attribute for read only */
	const TManagedArray<T>* Find() const
	{
		return ConstAttributeArray;
	}

	/** get the attribute for modification */
	TManagedArray<T>& Modify() 
	{
		check(AttributeArray!=nullptr && !IsConst());
		AttributeArray->MarkDirty();
		return *AttributeArray;
	}

	/** per index modification */
	void ModifyAt(int32 Index, const T& NewValue)
	{
		check(AttributeArray != nullptr && !IsConst());
		AttributeArray->MarkDirty();
		(*AttributeArray)[Index] = NewValue;
	}

	/** add the attribute if it does not exists yet */
	TManagedArray<T>& Add(ManageArrayAccessor::EPersistencePolicy PersistencePolicy = ManageArrayAccessor::EPersistencePolicy::MakePersistent,
		FName InGroupDependency = FName(NAME_None))
	{
		check(!IsConst());
		if (PersistencePolicy == ManageArrayAccessor::EPersistencePolicy::MakePersistent && !IsPersistent())
		{
			Remove();
		}

		if (!Collection->HasGroup(Group))
		{
			Collection->AddGroup(Group);
		}

		FName LocalGroupDependency = DefaultGroupDependency;
		if (!InGroupDependency.IsNone())
		{
			LocalGroupDependency = InGroupDependency;
		}

		bool bSaved = (PersistencePolicy == ManageArrayAccessor::EPersistencePolicy::MakePersistent);
		FManagedArrayCollection::FConstructionParameters Params(LocalGroupDependency, bSaved);
		ConstAttributeArray = AttributeArray = &Collection->AddAttribute<T>(Name, Group, Params);
		return *AttributeArray;
	}

	/** add and fill the attribute if it does not exist yet */
	void AddAndFill(const T& Value, 
		ManageArrayAccessor::EPersistencePolicy PersistencePolicy = ManageArrayAccessor::EPersistencePolicy::MakePersistent,
		FName InGroupDependency = FName(NAME_None))
	{
		check(!IsConst());
		if (!Collection->HasAttribute(Name, Group))
		{
			Add(PersistencePolicy, InGroupDependency);
			AttributeArray->Fill(Value);
		}
	}

	/** Fill the attribute with a specific value */
	void Fill(const T& Value)
	{
		check(!IsConst());
		if (AttributeArray)
		{
			AttributeArray->Fill(Value);
		}
	}

	/** copy from another attribute ( create if necessary ) */
	void Copy(const TManagedArrayAccessor<T>& FromAttribute )
	{
		check(!IsConst());
		Collection->CopyAttribute(FromAttribute.ConstCollection, FromAttribute.Name, Name, Group);
		// update attribute array if necessary
		AttributeArray = Collection->FindAttributeTyped<T>(Name, Group);
		ConstAttributeArray = AttributeArray;
	}

	void Remove()
	{
		check(!IsConst());
		Collection->RemoveAttribute(Name, Group);
		ConstAttributeArray = AttributeArray = nullptr;
	}

	int32 Num() const
	{
		return ConstAttributeArray
			? ConstAttributeArray->Num()
			: ConstCollection.NumElements(Group); // more expensive to fetch
	}

private:

	// the non-const Collection will be null if the accessor is const, while ConstCollection is always set
	const FManagedArrayCollection& ConstCollection;
	FManagedArrayCollection* Collection = nullptr;

	FName Name;
	FName Group;
	const FName DefaultGroupDependency; // Will be used by Add if a non-NAME_None group dependency is specified by that method.
	TManagedArray<T>* AttributeArray;
	const TManagedArray<T>* ConstAttributeArray;
};
