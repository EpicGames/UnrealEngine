// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArray.h"
#include "GeometryCollection/ManagedArrayTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "GeometryCollection/GeometryCollectionSection.h"

#include "ManagedArrayCollection.generated.h"

class FSimulationProperties;

namespace Chaos
{
	class FChaosArchive;
}

struct FAttributeAndGroupId
{
	FName AttributeName;
	FName GroupName;

	bool operator==(const FAttributeAndGroupId& Other) const
	{
		return ((AttributeName == Other.AttributeName) && (GroupName == Other.GroupName));
	}
};

/**
* ManagedArrayCollection
*
*   The ManagedArrayCollection is an entity system that implements a homogeneous, dynamically allocated, manager of
*   primitive TArray structures. The collection stores groups of TArray attributes, where each attribute within a
*   group is the same length. The collection will make use of the TManagedArray class, providing limited interaction
*   with the underlying TArray. 
*
*   For example:
*
	FManagedArrayCollection* Collection(NewObject<FManagedArrayCollection>());
	Collection->AddElements(10, "GroupBar"); // Create a group GroupBar and add 10 elements.
	Collection->AddAttribute<FVector3f>("AttributeFoo", "GroupBar"); // Add a FVector array named AttributeFoo to GroupBar.
	TManagedArray<FVector3f>&  Foo = Collection->GetAttribute<FVector3f>("AttributeFoo", "GroupBar"); // Get AttributeFoo
	for (int32 i = 0; i < Foo.Num(); i++)
	{
		Foo[i] = FVector(i, i, i); // Update AttribureFoo's elements
	}
*
*
*
*/
USTRUCT()
struct FManagedArrayCollection
{
	GENERATED_USTRUCT_BODY()
	friend FSimulationProperties;

public:


	CHAOS_API FManagedArrayCollection();
	CHAOS_API virtual ~FManagedArrayCollection();

	CHAOS_API FManagedArrayCollection(const FManagedArrayCollection& In);
	CHAOS_API FManagedArrayCollection& operator=(const FManagedArrayCollection& In);
	CHAOS_API FManagedArrayCollection(FManagedArrayCollection&&);
	CHAOS_API FManagedArrayCollection& operator=(FManagedArrayCollection&&);

	CHAOS_API bool operator==(const FManagedArrayCollection& Other) const;  // Slow, required for comparing to a default collection during UPROPERTY serialization

	static CHAOS_API int8 Invalid;
	typedef EManagedArrayType EArrayType;
	/**
	*
	*/
	struct FConstructionParameters {

		FConstructionParameters(FName GroupIndexDependencyIn = NAME_None
			, bool SavedIn = true, bool bInAllowCircularDependency = false)
			: GroupIndexDependency(GroupIndexDependencyIn)
			, Saved(SavedIn)
			, bAllowCircularDependency(bInAllowCircularDependency)
		{}

		FName GroupIndexDependency;
		bool Saved;
		bool bAllowCircularDependency;
	};

	struct FProcessingParameters
	{
		FProcessingParameters() : bDoValidation(true), bReindexDependentAttibutes(true) {}

		bool bDoValidation ;
		bool bReindexDependentAttibutes;
	};

	struct FManagedType
	{
		FManagedType(EManagedArrayType InType, FName InName, FName InGroup)
			: Type(InType), Name(InName), Group(InGroup) {}
		EManagedArrayType Type = EManagedArrayType::FNoneType;
		FName Name;
		FName Group;
	};

	template<class T>
	struct TManagedType : FManagedType
	{
		TManagedType(FName InName, FName InGroup) :
			FManagedType(ManagedArrayType<T>(), InName, InGroup) {}			
	};

	/**
	* Type name for this class.
	*/
	static FName StaticType() { return FName("FManagedArrayCollection"); }

	virtual bool IsAType(FName InTypeName) const { return InTypeName.IsEqual(FManagedArrayCollection::StaticType()); }

	template<typename T>
	bool IsA() { return IsAType(T::StaticType()); }

	template<class T>
	T* Cast() { return IsAType(T::StaticType())?static_cast<T*>(this):nullptr; }

	template<class T>
	const T* Cast() const { return IsAType(T::StaticType())? static_cast<const T*>(this) : nullptr; }

	/**
	* Add an attribute of Type(T) to the group
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	* @return reference to the created ManagedArray<T>
	*/
	template<typename T>
	TManagedArray<T>& AddAttribute(FName Name, FName Group, FConstructionParameters Parameters = FConstructionParameters())
	{
		if (!HasAttribute(Name, Group))
		{
			AddNewAttributeImpl<T>(Name, Group, Parameters);
		}
		return ModifyAttribute<T>(Name, Group);
	}

	/**
	* Add an attribute of Type(T) to the group or find the existing attribute with the same name and type.
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	* @return pointer to the added/found managed array. If there is a type mismatch with an existing attribute, returns nullptr.
	*/
	template<typename T>
	TManagedArray<T>* FindOrAddAttributeTyped(FName Name, FName Group, FConstructionParameters Parameters = FConstructionParameters())
	{
		if (!HasAttribute(Name, Group))
		{
			AddNewAttributeImpl<T>(Name, Group, Parameters);
		}
		return ModifyAttributeTyped<T>(Name, Group);
	}


	/**
	* Duplicate the ManagedArrayCollection as the specified templated type
	* @return unowned pointer to new collection.
	*/
	template<class T = FManagedArrayCollection>
	T* NewCopy() const
	{
		T* Collection = new T();
		for (const FName& GroupName : Collection->GroupNames())
		{
			Collection->EmptyGroup(GroupName); //Geometry collection has 1 element in ConvexProperties group when initialized
		}
		CopyTo(Collection);
		return Collection;
	}

	CHAOS_API void CopyTo(FManagedArrayCollection* Collection, const TArray<FName>& GroupsToSkip = TArray<FName>(), TArray<TTuple<FName, FName>> AttributesToSkip = TArray<TTuple<FName, FName>>()) const;

	/**
	* Add an external attribute of Type(T) to the group for size management. Lifetime is managed by the caller, must make sure the array is alive when the collection is
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	* @param ValueIn - The array to be managed
	*/
	template<typename T>
	void AddExternalAttribute(FName Name, FName Group, TManagedArray<T>& ValueIn, FConstructionParameters Parameters = FConstructionParameters())
	{
		check(!HasAttribute(Name, Group));

		if (!HasGroup(Group))
		{
			AddGroup(Group);
		}

		const int32 InitialSize = NumElements(Group);
		FName GroupIndexDependency;
		if (ensure(Parameters.bAllowCircularDependency || !IsConnected(Parameters.GroupIndexDependency, Group)))
		{
			GroupIndexDependency = Parameters.GroupIndexDependency;
		}
		else
		{
			GroupIndexDependency = NAME_None;
		}

		FValueType NewAttribute = FValueType::MakeExternal<T>(&ValueIn, NumElements(Group), GroupIndexDependency, Parameters.Saved);
		Map.Add({ Name, Group }, MoveTemp(NewAttribute));
	}

	/**
	* Create a group on the collection. Adding attribute will also create unknown groups.
	* @param Group - The group name
	*/
	CHAOS_API void AddGroup(FName Group);

	/**
	* Returns the number of attributes in a group.
	*/
	CHAOS_API int32 NumAttributes(FName Group) const;

	/**
	* List all the attributes in a group names.
	* @return list of group names
	*/
	CHAOS_API TArray<FName> AttributeNames(FName Group) const;

	/**
	* Add elements to a group
	* @param NumberElements - The number of array entries to add
	* @param Group - The group to append entries to.
	* @return starting index of the new ManagedArray<T> entries.
	*/
	CHAOS_API int32 AddElements(int32 NumberElements, FName Group);

	/**
	* Insert elements to a group
	* @param NumberElements - The number of array entries to add
	* @param Position - The position in the managed array where to insert entries.
	* @param Group - The group to append entries to.
	* @return starting index of the new ManagedArray<T> entries (same as Position).
	*/
	CHAOS_API int32 InsertElements(int32 NumberElements, int32 Position, FName Group);

	/**
	* Append Collection and reindex dependencies on this collection. 
	* @param InCollection : Collection to add. 
	*/
	CHAOS_API virtual void Append(const FManagedArrayCollection& Collection);

	/**
	* Returns attribute(Name) of Type(T) from the group
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	* @return ManagedArray<T> &
	*/
	template<typename T>
	TManagedArray<T>* FindAttribute(FName Name, FName Group)
	{
		const FKeyType Key = FManagedArrayCollection::MakeMapKey(Name, Group);
		if (FValueType* FoundValue = Map.Find(Key))
		{
			return FoundValue->ModifyTypedPtr<T>();
		}
		return nullptr;
	};

	template<typename T>
	const TManagedArray<T>* FindAttribute(FName Name, FName Group) const
	{
		if (const FValueType* FoundValue = Map.Find({ Name, Group }))
		{
			return FoundValue->GetTypedPtr<T>();
		}
		return nullptr;
	};

	/**
	* Returns attribute(Name) of Type(T) from the group if and only if the types of T and the array match
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	* @return ManagedArray<T> &
	*/
	template<typename T>
	TManagedArray<T>* FindAttributeTyped(FName Name, FName Group)
	{
		if (FValueType* FoundValue = Map.Find({ Name, Group }))
		{
			if(FoundValue->IsSameType<T>())
			{
				return FoundValue->ModifyTypedPtr<T>();
			}
		}
		return nullptr;
	};

	template<typename T>
	const TManagedArray<T>* FindAttributeTyped(FName Name, FName Group) const
	{
		if (const FValueType* FoundValue = Map.Find({ Name, Group }))
		{
			if(FoundValue->IsSameType<T>())
			{
				return FoundValue->GetTypedPtr<T>();
			}
		}
		return nullptr;
	};

	/**
	* Returns attribute access of Type(T) from the group for modification
	* this will mark the collection dirty
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	* @return ManagedArray<T> &
	*/
	template<typename T>
	TManagedArray<T>& ModifyAttribute(FName Name, FName Group)
	{
		check(HasAttribute(Name, Group))
		return Map[{Name, Group}].ModifyTyped<T>();
	}

	/**
	* Returns attribute access of Type(T) from the group for modification if and only if the types of T and the array match
	* this will mark the collection dirty
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	* @return ManagedArray<T> *
	*/
	template<typename T>
	TManagedArray<T>* ModifyAttributeTyped(FName Name, FName Group)
	{
		if (FValueType* FoundValue = Map.Find({ Name, Group }))
		{
			if (FoundValue->IsSameType<T>())
			{
				return FoundValue->ModifyTypedPtr<T>();
			}
		}
		return nullptr;
	}
	
	/**
	* Returns attribute access of Type(T) from the group
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	* @return ManagedArray<T> &
	*/
	template<typename T>
	const TManagedArray<T>& GetAttribute(FName Name, FName Group) const
	{
		check(HasAttribute(Name, Group));
		return Map[{Name, Group}].GetTyped<T>();
	};

	/**
	* Clear the internal data. 
	*/
	virtual void Reset() { Map.Reset(); GroupInfo.Reset(); MakeDirty(); }


	/**
	* Remove the element at index and reindex the dependent arrays 
	*/
	CHAOS_API virtual void RemoveElements(const FName & Group, const TArray<int32> & SortedDeletionList, FProcessingParameters Params = FProcessingParameters());

	/**
	* Merge the element indexed SortedMergeList[Idx] to the element indexed MergeRemapIndex[Idx]
	* first redirect references of SortedMergeList[Idx] then delete
	*/
	CHAOS_API virtual void MergeElements(const FName& Group, const TArray<int32>& SortedMergeList, const TArray<int32>& MergeRemapIndex, FProcessingParameters Params = FProcessingParameters());

	/**
	* Remove the elements at Position and reindex the dependent arrays 
	* @param Group - The group that manages the attributes
	* @param NumberElements - The number of array entries to remove
	* @param Position - The position from which to remove entries
	*/
	CHAOS_API virtual void RemoveElements(const FName& Group, int32 NumberElements, int32 Position);


	/**
	* Remove the attribute from the collection, and clear the memory.
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	*/
	CHAOS_API void RemoveAttribute(FName Name, FName Group);


	/**
	* Remove the group from the collection.
	* @param Group - The group that manages the attribute
	*/
	CHAOS_API void RemoveGroup(FName Group);

	/**
	* List all the group names.
	*/
	CHAOS_API TArray<FName> GroupNames() const;

	/**
	* Whether the collection is ampty ( no group / no attributes ) 
	*/
	bool IsEmpty() const
	{
		return GroupInfo.IsEmpty() && Map.IsEmpty();
	}

	/**
	* Check for the existence of a attribute
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	*/
	CHAOS_API bool HasAttribute(FName Name, FName Group) const;

	/**
	* Check for the existence of a attribute
	* @param TArray<ManagedType> - Attributes to search for
	*/
	CHAOS_API bool HasAttributes(const TArray<FManagedType>& Types) const;

	/**
	* Check for the existence of a group
	* @param Group - The group name
	*/
	FORCEINLINE bool HasGroup(FName Group) const { return GroupInfo.Contains(Group); }

	/**
	* Return attribute type
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	*/
	CHAOS_API EArrayType GetAttributeType(FName Name, FName Group) const;

	/**
	* Check if an attribute is dirty
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	*/
	CHAOS_API bool IsAttributeDirty(FName Name, FName Group) const;

	/**
	* Check if an attribute is persistent - ie its data will be serialized
	* If the attribute does not exists this will return false
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	*/
	CHAOS_API bool IsAttributePersistent(FName Name, FName Group) const;
	
	/**
	*
	*/
	CHAOS_API void SetDependency(FName Name, FName Group, FName DependencyGroup, bool bAllowCircularDependency = false);

	/**
	* Return the group index dependency for the specified attribute.
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	*/
	CHAOS_API FName GetDependency(FName Name, FName Group) const;

	/**
	*
	*/
	CHAOS_API void RemoveDependencyFor(FName Group);

	/**
	* Copy an attribute. Will perform an implicit group sync. Attribute must exist in both InCollection and this collection
	* @param Name - The name of the attribute
	* @param Group - The group that manages the attribute
	*/
	CHAOS_API void CopyAttribute(const FManagedArrayCollection& InCollection, FName Name, FName Group);

	/**
	* Copy an attribute. Will perform an implicit group sync.
	* @param SrcName - The name of the attribute being copied from
	* @param DestName - The name of the attribute being copied to
	* @param Group - The group that manages the attribute
	*/
	CHAOS_API void CopyAttribute(const FManagedArrayCollection& InCollection, FName SrcName, FName DestName, FName Group);

	/**
	* Copy an attribute from one group to another group. Will perform an implicit group sync.
	* @param SrcName - The name of the attribute being copied from
	* @param DestName - The name of the attribute being copied to
	* @param SrcGroup - The group that manages the attribute being copied from 
	* @param DstGroup - The group that manages the attribute being copied to
	*/
	CHAOS_API void CopyAttribute(const FManagedArrayCollection& InCollection, FName SrcName, FName DestName, FName SrcGroup, FName DstGroup);

	/**
	* Copy attributes that match the input collection. This is a utility to easily sync collections
	* @param InCollection - All groups from this collection found in the input will be sized accordingly
	* @param SkipList - Group/Attrs to skip. Keys are group names, values are attributes in those groups.
	*/
	CHAOS_API void CopyMatchingAttributesFrom(const FManagedArrayCollection& InCollection, const TMap<FName, TSet<FName>>* SkipList=nullptr);

	/**
	* Copy attributes that match the input collection. This is a utility to easily sync collections
	* This version is recommend to be used as it is more performant overall
	*	- it only resize the necessary groups
	*	- it takes a array view for the skip list
	*	- it has a more contained logic that reduces the number of lookups for attributes in the maps
	* @param InCollection - All groups from this collection found in the input will be sized accordingly
	* @param SkipList - Group/Attrs to skip. Keys are group names, values are attributes in those groups.
	*/
	CHAOS_API void CopyMatchingAttributesFrom(const FManagedArrayCollection& FromCollection, const TArrayView<const FAttributeAndGroupId> SkipList);

	/**
	* Number of elements in a group
	* @return the group size, and Zero(0) if group does not exist.
	*/
	CHAOS_API int32 NumElements(FName Group) const;

	/**
	* Resize a group
	* @param Size - The size of the group
	* @param Group - The group to resize
	*/
	CHAOS_API void Resize(int32 Size, FName Group);

	/**
	* Reserve a group
	* @param Size - The size of the group
	* @param Group - The group to reserve
	*/
	CHAOS_API void Reserve(int32 Size, FName Group);

	/**
	* Empty the group 
	*  @param Group - The group to empty
	*/
	CHAOS_API void EmptyGroup(FName Group);

	/**
	* Reorders elements in a group. NewOrder must be the same length as the group.
	*/
	CHAOS_API virtual void ReorderElements(FName Group, const TArray<int32>& NewOrder);
#if 0	//not needed until we support per instance serialization
	/**
	* Swap elements within a group
	* @param Index1 - The first location to be swapped
	* @param Index2 - The second location to be swapped
	* @param Group - The group to resize
	*/
	CHAOS_API void SwapElements(int32 Index1, int32 Index2, FName Group);
#endif

	/**
	* Updated for Render ( Mark it dirty ) question: is this the right place for this?
	*/
	void MakeDirty() { bDirty = true; }
	void MakeClean() { bDirty = false; }
	bool IsDirty() const { return bDirty; }

	/**
	* Serialize
	*/
	CHAOS_API virtual void Serialize(Chaos::FChaosArchive& Ar);

	CHAOS_API bool Serialize(FArchive& Ar);

	/**
	* Cycle Checking. 
	* Search for TargetNode from the StartingNode. 
	*/
	CHAOS_API bool IsConnected(FName StartingNode, FName TargetNode);


	/**
	* Dump the contents to a FString
	*/
	CHAOS_API FString ToString() const;

	/**
	* Get allocated memory size 
	*/
	CHAOS_API SIZE_T GetAllocatedSize() const;

	/**
	 * Get information on the total storage required for each element in each group
	 */
	 CHAOS_API void GetElementSizeInfoForGroups(TArray<TPair<FName, SIZE_T>>& OutSizeInfo) const;

private:

	template<typename T>
	void AddNewAttributeImpl(FName Name, FName Group, const FConstructionParameters& Parameters)
	{
		checkSlow(!HasAttribute(Name, Group));
		if (!HasGroup(Group))
		{
			AddGroup(Group);
		}

		static const FName EmptyName = NAME_None;
		
		const int32 InitialSize = NumElements(Group);
		FName GroupIndexDependency;
		if (ensure(Parameters.bAllowCircularDependency || !IsConnected(Parameters.GroupIndexDependency, Group)))
		{
			GroupIndexDependency = Parameters.GroupIndexDependency;
		}
		else
		{
			GroupIndexDependency = EmptyName;
		}

		TUniquePtr<TManagedArray<T>> ArrayPtr(new TManagedArray<T>());
		FValueType NewAttribute = FValueType::MakeManaged(MoveTemp(ArrayPtr), InitialSize, GroupIndexDependency, Parameters.Saved);
		Map.Add({ Name, Group }, MoveTemp(NewAttribute));
	}

	/**
	* Insert elements to a group without reordering elements
	* @param NumberElements - The number of array entries to add
	* @param Position - The position in the managed array where to insert entries.
	* @param Group - The group to append entries to.
	* @return new order after insertion.
	*/
	TArray<int32> InsertElementsNoReorder(int32 NumberElements, int32 Position, FName Group);

	/****
	*  Mapping Key/Value
	*
	*    The Key and Value pairs for the attribute arrays allow for grouping
	*    of array attributes based on the Name,Group pairs. Array attributes
	*    that share Group names will have the same lengths.
	*
	*    The FKeyType is a tuple of <FName,FName> where the Get<0>() parameter
	*    is the AttributeName, and the Get<1>() parameter is the GroupName.
	*    Construction of the FKeyType will follow the following pattern:
	*    FKeyType("AttributeName","GroupName");
	*
	*/
	typedef TTuple<FName, FName> FKeyType;
	struct FGroupInfo
	{
		int32 Size;
	};

	static inline FKeyType MakeMapKey(FName Name, FName Group) // inline for linking issues
	{
		return FKeyType(Name, Group);
	}

	friend struct FManagedArrayCollectionValueTypeWrapper;
	struct FValueType
	{
	private:
		EArrayType ArrayType;
		FName GroupIndexDependency;
		bool bPersistent;
		bool bExternalValue;	//External arrays have external memory management.

		TSharedPtr<FManagedArrayBase, ESPMode::ThreadSafe> SharedManagedArray; // external arrays have this set this to null
		FManagedArrayBase* ManagedArray = nullptr;;

	private:
		/**
		* Create a typed attribute
		* the array is managed ( as opposed to being external ) and will be deleted by the destructor
		*/
		template<typename T>
		FValueType(TUniquePtr<TManagedArray<T>>&& ArrayPtr, int32 InitialSize, FName InGroupIndexDependency, bool bInPersistent)
			: ArrayType(ManagedArrayType<T>())
			, GroupIndexDependency(InGroupIndexDependency)
			, bPersistent(bInPersistent)
			, bExternalValue(false)
			, SharedManagedArray(ArrayPtr.Release())
			, ManagedArray(SharedManagedArray.Get())
		{
			MakeUniqueForWrite();
			ManagedArray->Reserve(InitialSize); // make sure we have the exact size to avoid over-allocation
			ManagedArray->Resize(InitialSize);
		};

		/**
		* Create a external typed attribute
		* the array is external and it is the responsability of the original owner to destroy it
		*/
		template<typename T>
		FValueType(TManagedArray<T>* ExternalArrayPtr, int32 InitialSize, FName InGroupIndexDependency, bool bInPersistent)
			: ArrayType(ManagedArrayType<T>())
			, GroupIndexDependency(InGroupIndexDependency)
			, bPersistent(bInPersistent)
			, bExternalValue(true)
			, SharedManagedArray(nullptr)
			, ManagedArray(ExternalArrayPtr)
		{
			ManagedArray->Reserve(InitialSize); // make sure we have the exact size to avoid over-allocation
			ManagedArray->Resize(InitialSize);
		};

		CHAOS_API void MakeUniqueForWrite();

	public:
		/**
		* Create a typed attribute
		* the array is managed ( as opposed to being external ) and will be deleted by the destructor
		*/
		template <typename T>
		static FValueType MakeManaged(TUniquePtr<TManagedArray<T>>&& ArrayPtr, int32 InitialSize, FName InGroupIndexDependency, bool bInPersistent)
		{
			return FValueType(MoveTemp(ArrayPtr), InitialSize, InGroupIndexDependency, bInPersistent);
		}

		template <typename T>
		static FValueType MakeExternal(TManagedArray<T>* ExternalArrayPtr, int32 InitialSize, FName InGroupIndexDependency, bool bInPersistent)
		{
			return FValueType(ExternalArrayPtr, InitialSize, InGroupIndexDependency, bInPersistent);
		}

		CHAOS_API FValueType();
		CHAOS_API FValueType(const FValueType& Other);
		CHAOS_API FValueType(FValueType&& Other);

		CHAOS_API ~FValueType();

		EArrayType GetArrayType() const { return ArrayType; }
		FName GetGroupIndexDependency() const { return GroupIndexDependency; };
		bool IsPersistent() const { return bPersistent; }
		bool IsExternal() const { return bExternalValue; }
		bool IsDirty() const { return ManagedArray->IsDirty(); }

		void SetGroupIndexDependency(FName NewGroupDependency) { GroupIndexDependency = NewGroupDependency; }

		template <typename T>
		bool IsSameType() const { return (ArrayType == ManagedArrayType<T>()); }
		
		template <typename T>
		const TManagedArray<T>* GetTypedPtr() const
		{ 
			check(IsSameType<T>());
			
			const TManagedArray<T>* TypedManagedArray = static_cast<const TManagedArray<T>*>(ManagedArray);
			check(TypedManagedArray != nullptr);
			return TypedManagedArray;
		}

		template <typename T>
		const TManagedArray<T>& GetTyped() const
		{
			return *GetTypedPtr<T>();
		}

		const FManagedArrayBase& Get() const { return *ManagedArray; }

		template <typename T>
		TManagedArray<T>* ModifyTypedPtr()
		{ 
			check(IsSameType<T>());
			MakeUniqueForWrite();

			TManagedArray<T>* TypedManagedArray = static_cast<TManagedArray<T>*>(ManagedArray);
			check(TypedManagedArray != nullptr);

			TypedManagedArray->MarkDirty();
			return TypedManagedArray;
		}

		template <typename T>
		TManagedArray<T>& ModifyTyped()
		{
			return *ModifyTypedPtr<T>();
		}

		FManagedArrayBase& Modify();

		void Reserve(int32 ReservedSize);
		void Resize(int32 NewSize);
		void InitFrom(const FValueType& Other);
		void Exchange(FValueType& Other);
		void Convert(FValueType& Other);
		void CopyFrom(const FValueType& Other);
		void Empty();
		void RemoveGroupIndexDependency(FName Group);

		FValueType& operator=(const FValueType& Other) = delete;
		FValueType& operator=(FValueType&& Other) = delete;

		void Serialize(FArchive& Ar);
	};


	TMap< FKeyType, FValueType> Map;	//data is owned by the map explicitly
	TMap< FName, FGroupInfo> GroupInfo;
	bool bDirty;

protected:

	CHAOS_API virtual void SetDefaults(FName Group, uint32 StartSize, uint32 NumElements);

	/**
	 * Virtual helper function called by CopyMatchingAttributesFrom; adds attributes 'default, but optional' attributes that are present in InCollection
	 * This is used by FGeometryCollection to make sure all UV layers are copied over by CopyMatchingAttributesFrom()
	 */
	virtual void MatchOptionalDefaultAttributes(const FManagedArrayCollection& InCollection)
	{}

	/**
	* Size and order a group so that it matches the group found in the input collection.
	* @param InCollection - The collection we are ordering our group against. 
	* @param Group - The group that manages the attribute
	*/
	CHAOS_API void SyncGroupSizeFrom(const FManagedArrayCollection& InCollection, FName Group);
	
	/**
	* Size and order a dstgroup so that it matches the srcgroup found in the input collection.
	* @param InCollection - The collection we are ordering our group against. 
	* @param SrcGroup - The group that manages the attribute in the input collection
	* @param DstGroup - The group that manages the attribute in the output collection
	*/
	CHAOS_API void SyncGroupSizeFrom(const FManagedArrayCollection& InCollection, FName SrcGroup, FName DstGroup);

	/**
	 * Version to indicate need for conditioning to current expected data layout during serialization loading.
	 */
	int32 Version;


	/**
	*   operator<<(FGroupInfo)
	*/
	friend FArchive& operator<<(FArchive& Ar, FGroupInfo& GroupInfo);

	/**
	*   operator<<(FValueType)
	*/
	friend FArchive& operator<<(FArchive& Ar, FValueType& ValueIn);

};

template<> struct TStructOpsTypeTraits<FManagedArrayCollection> : public TStructOpsTypeTraitsBase2<FManagedArrayCollection>
{
	enum
	{
		WithSerializer = true,
		WithIdenticalViaEquality = true  // Required for usage as a UPROPERTY, otherwise the reflection system won't see a difference with a default collection when serializing
	};
};

/*
*  FManagedArrayInterface
*/
class FManagedArrayInterface
{
public:
	FManagedArrayInterface() : ManagedCollection(nullptr) {}
	FManagedArrayInterface(FManagedArrayCollection* InManagedArray)
		: ManagedCollection(InManagedArray) {}

	virtual void InitializeInterface() = 0;
	virtual void CleanInterfaceForCook() = 0; 
	virtual void RemoveInterfaceAttributes() = 0;

protected:
	FManagedArrayCollection* ManagedCollection;

};


#define MANAGED_ARRAY_COLLECTION_INTERNAL(TYPE_NAME)					\
	static FName StaticType() { return FName(#TYPE_NAME); }				\
	virtual bool IsAType(FName InTypeName) const override {				\
		return InTypeName.IsEqual(TYPE_NAME::StaticType())				\
				|| Super::IsAType(InTypeName);							\
	}
