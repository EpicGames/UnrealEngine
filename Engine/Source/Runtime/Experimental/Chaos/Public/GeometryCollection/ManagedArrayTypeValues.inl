// Copyright Epic Games, Inc. All Rights Reserved.

// usage
//
// General purpose ManagedArrayCollection::ArrayType definition

#ifndef MANAGED_ARRAY_TYPE
#error MANAGED_ARRAY_TYPE macro is undefined.
#endif

// NOTE: new types must be added at the bottom to keep serialization from breaking

MANAGED_ARRAY_TYPE(FVector3f, Vector)
MANAGED_ARRAY_TYPE(FIntVector, IntVector)
MANAGED_ARRAY_TYPE(FVector2f, Vector2D)
MANAGED_ARRAY_TYPE(FLinearColor, LinearColor)
MANAGED_ARRAY_TYPE(int32, Int32)
MANAGED_ARRAY_TYPE(bool, Bool)
MANAGED_ARRAY_TYPE(FTransform, Transform)
MANAGED_ARRAY_TYPE(FString, String)
MANAGED_ARRAY_TYPE(float, Float)
MANAGED_ARRAY_TYPE(FQuat4f, Quat)
MANAGED_ARRAY_TYPE(FGeometryCollectionBoneNode, BoneNode)
MANAGED_ARRAY_TYPE(FGeometryCollectionSection, MeshSection)
MANAGED_ARRAY_TYPE(FBox, Box)
MANAGED_ARRAY_TYPE(TSet<int32>, IntArray)
MANAGED_ARRAY_TYPE(FGuid, Guid)
MANAGED_ARRAY_TYPE(uint8, UInt8)
MANAGED_ARRAY_TYPE(TArray<FVector3f>*, VectorArrayPointer)
MANAGED_ARRAY_TYPE(TUniquePtr<TArray<FVector3f>>, VectorArrayUniquePointer)
MANAGED_ARRAY_TYPE(Chaos::FImplicitObject3*, FImplicitObject3Pointer)
MANAGED_ARRAY_TYPE(TUniquePtr<Chaos::FImplicitObject3>, FImplicitObject3UniquePointer)
MANAGED_ARRAY_TYPE(Chaos::TSerializablePtr<Chaos::FImplicitObject3>, FImplicitObject3SerializablePtr)
MANAGED_ARRAY_TYPE(Chaos::FBVHParticlesFloat3, FBVHParticlesFloat3Pointer)
MANAGED_ARRAY_TYPE(TUniquePtr<Chaos::FBVHParticlesFloat3>, FBVHParticlesFloat3UniquePointer)
MANAGED_ARRAY_TYPE(Chaos::FPBDRigidParticleHandle*, TPBDRigidParticleHandle3fPtr)
MANAGED_ARRAY_TYPE(Chaos::FPBDGeometryCollectionParticleHandle*, TPBDGeometryCollectionParticleHandle3fPtr)
MANAGED_ARRAY_TYPE(TUniquePtr<Chaos::FGeometryParticle>, TGeometryParticle3fUniquePtr)
MANAGED_ARRAY_TYPE(Chaos::ThreadSafeSharedPtr_FImplicitObject, FImplicitObject3ThreadSafeSharedPointer)
MANAGED_ARRAY_TYPE(Chaos::NotThreadSafeSharedPtr_FImplicitObject, FImplicitObject3SharedPointer)
MANAGED_ARRAY_TYPE(Chaos::FPBDRigidClusteredParticleHandle*, TPBDRigidClusteredParticleHandle3fPtr)
MANAGED_ARRAY_TYPE(TUniquePtr<Chaos::FConvex>, FConvexUniquePtr)
MANAGED_ARRAY_TYPE(TArray<FVector2f>, Vector2DArray)
MANAGED_ARRAY_TYPE(double, Double)
MANAGED_ARRAY_TYPE(FIntVector4, IntVector4)
MANAGED_ARRAY_TYPE(FVector3d, Vector3d)
MANAGED_ARRAY_TYPE(FIntVector2, IntVector2)
MANAGED_ARRAY_TYPE(TArray<FIntVector2>, IntVector2Array)
MANAGED_ARRAY_TYPE(TArray<int32>, Int32Array)
MANAGED_ARRAY_TYPE(TArray<float>, FloatArray)
MANAGED_ARRAY_TYPE(FVector4f, Vector4f)
MANAGED_ARRAY_TYPE(TArray<FVector3f>, FVectorArray)
MANAGED_ARRAY_TYPE(TUniquePtr<Chaos::FPBDRigidParticle>, TPBDRigidParticle3fUniquePtr)
MANAGED_ARRAY_TYPE(Chaos::FImplicitObjectPtr, FImplicitObjectRefCountedPtr)
MANAGED_ARRAY_TYPE(Chaos::FConvexPtr, FConvexRefCountedPtr)
MANAGED_ARRAY_TYPE(FTransform3f, Transform3f)
MANAGED_ARRAY_TYPE(TArray<FIntVector3>, IntVector3Array)
MANAGED_ARRAY_TYPE(TArray<FVector4f>, Vector4fArray)
MANAGED_ARRAY_TYPE(Chaos::PMatrix33d, PMatrix33d)
MANAGED_ARRAY_TYPE(TArray<Chaos::PMatrix33d>, PMatrix33dArray)
MANAGED_ARRAY_TYPE(TArray<TArray<FVector3f>>, FVector3fNestedArray)
MANAGED_ARRAY_TYPE(FUintVector2, UintVector2)
MANAGED_ARRAY_TYPE(TObjectPtr<UObject>, UObjectArray)
MANAGED_ARRAY_TYPE(Chaos::FLinearCurve, LinearCurve)
MANAGED_ARRAY_TYPE(FName, Name)
MANAGED_ARRAY_TYPE(FSoftObjectPath, SoftObjectPath)
// NOTE: new types must be added at the bottom to keep serialization from breaking


#undef MANAGED_ARRAY_TYPE
