// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Containers/UnrealString.h"
#include "GeometryCollection/GeometryCollectionBoneNode.h"
#include "GeometryCollection/GeometryCollectionSection.h"
#include "GeometryCollection/ManagedArray.h"
#include "Math/Color.h"
#include "Math/IntVector.h"
#include "Math/Quat.h"
#include "Math/Transform.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Box.h"
#include "UObject/SoftObjectPath.h"

#include "Chaos/ImplicitObject.h"
#include "Chaos/BVHParticles.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/Convex.h"
#include "Chaos/Matrix.h"
#include "Chaos/Curve.h"

inline FArchive& operator<<(FArchive& Ar, TArray<FVector3f>*& ValueIn)
{
	check(false);	//We don't serialize raw pointers to arrays. Use unique ptr
	return Ar;
}

inline FArchive& operator<<(FArchive& Ar, TUniquePtr<TArray<FVector3f>>& ValueIn)
{
	bool bExists = ValueIn.Get() != nullptr;
	Ar << bExists;
	if (bExists)
	{
		if (Ar.IsLoading())
		{
			ValueIn = MakeUnique<TArray<FVector3f>>();
		}
		Ar << *ValueIn;
	}

	return Ar;
}

inline FArchive& operator<<(FArchive& Ar, Chaos::FImplicitObject*& ValueIn)
{
	check(false);	//We don't serialize raw pointers to implicit objects. Use unique ptr
	return Ar;
}

template <typename T, int d, bool bPersistent>
inline FArchive& operator<<(FArchive& Ar, Chaos::TPBDRigidParticleHandleImp<T, d, bPersistent>*& Particle)
{
	verifyf(false, TEXT("TPBDRigidParticleHandleImp* should never be serialized!  Use unique ptr."));
	return Ar;
}

// ---------------------------------------------------------
//
// General purpose EManagedArrayType definition. 
// This defines things like 
//     EManagedArrayType::FVectorType
// see ManagedArrayTypeValues.inl for specific types.
//
#define MANAGED_ARRAY_TYPE(a,A) F##A##Type,
enum class EManagedArrayType : uint8
{
	FNoneType,
#include "ManagedArrayTypeValues.inl"
};
#undef MANAGED_ARRAY_TYPE

// ---------------------------------------------------------
//  ManagedArrayType<T>
//    Templated function to return a EManagedArrayType.
//
template<class T> inline EManagedArrayType ManagedArrayType();
#define MANAGED_ARRAY_TYPE(a,A) template<> inline EManagedArrayType ManagedArrayType<a>() { return EManagedArrayType::F##A##Type; }
#include "ManagedArrayTypeValues.inl"
#undef MANAGED_ARRAY_TYPE


// ---------------------------------------------------------
//  ManagedArrayType<T>
//     Returns a new EManagedArray shared pointer based on 
//     passed type.
//
inline FManagedArrayBase* NewManagedTypedArray(EManagedArrayType ArrayType)
{
	switch (ArrayType)
	{
#define MANAGED_ARRAY_TYPE(a,A)	case EManagedArrayType::F##A##Type:\
		return new TManagedArray<a>();
#include "ManagedArrayTypeValues.inl"
#undef MANAGED_ARRAY_TYPE
	}
	check(false);
	return nullptr;
}
