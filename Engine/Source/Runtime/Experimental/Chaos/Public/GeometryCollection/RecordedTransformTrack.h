// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "HAL/IConsoleManager.h"
#include "Serialization/ArchiveCountMem.h"
#include "Features/IModularFeatures.h"

#include "RecordedTransformTrack.generated.h"

USTRUCT()
struct FSolverCollisionData
{
	GENERATED_BODY()

	FSolverCollisionData()
		: Location(FVector(0.f))
		, AccumulatedImpulse(FVector(0.f))
		, Normal(FVector(0.f))
		, Velocity1(FVector(0.f))
		, Velocity2(FVector(0.f))
		, AngularVelocity1(FVector(0.f))
		, AngularVelocity2(FVector(0.f))
		, Mass1(0.f)
		, Mass2(0.f)
		, ParticleIndex(INDEX_NONE)
		, LevelsetIndex(INDEX_NONE)
		, ParticleIndexMesh(INDEX_NONE)
		, LevelsetIndexMesh(INDEX_NONE)
	{}

	FSolverCollisionData(FVector InLocation
		, FVector InAccumulatedImpulse
		, FVector InNormal
		, FVector InVelocity1
		, FVector InVelocity2
		, FVector InAngularVelocity1
		, FVector InAngularVelocity2
		, float InMass1
		, float InMass2
		, int32 InParticleIndex
		, int32 InLevelsetIndex
		, int32 InParticleIndexMesh
		, int32 InLevelsetIndexMesh)
		: Location(InLocation)
		, AccumulatedImpulse(InAccumulatedImpulse)
		, Normal(InNormal)
		, Velocity1(InVelocity1)
		, Velocity2(InVelocity2)
		, AngularVelocity1(InAngularVelocity1)
		, AngularVelocity2(InAngularVelocity2)
		, Mass1(InMass1)
		, Mass2(InMass2)
		, ParticleIndex(InParticleIndex)
		, LevelsetIndex(InLevelsetIndex)
		, ParticleIndexMesh(InParticleIndexMesh)
		, LevelsetIndexMesh(InLevelsetIndexMesh)
	{}

	UPROPERTY()
	FVector Location;

	UPROPERTY()
	FVector AccumulatedImpulse;

	UPROPERTY()
	FVector Normal;

	UPROPERTY()
	FVector Velocity1;

	UPROPERTY()
	FVector Velocity2;

	UPROPERTY()
	FVector AngularVelocity1;
	
	UPROPERTY()
	FVector AngularVelocity2;

	UPROPERTY()
	float Mass1;
	
	UPROPERTY()
	float Mass2;

	UPROPERTY()
	int32 ParticleIndex;
	
	UPROPERTY()
	int32 LevelsetIndex;

	UPROPERTY()
	int32 ParticleIndexMesh;
	
	UPROPERTY()
	int32 LevelsetIndexMesh;
};

USTRUCT()
struct FSolverBreakingData
{
	GENERATED_BODY()

	FSolverBreakingData()
		: Location(FVector(0.f))
		, Velocity(FVector(0.f))
		, AngularVelocity(FVector(0.f))
		, Mass(0.f)
		, ParticleIndex(INDEX_NONE)
		, ParticleIndexMesh(INDEX_NONE)
	{}

	UPROPERTY()
	FVector Location;

	UPROPERTY()
	FVector Velocity;
	
	UPROPERTY()
	FVector AngularVelocity;
	
	UPROPERTY()
	float Mass;
	
	UPROPERTY()
	int32 ParticleIndex;
	
	UPROPERTY()
	int32 ParticleIndexMesh;
};

USTRUCT()
struct FSolverTrailingData
{
	GENERATED_BODY()

	FSolverTrailingData()
		: Location(FVector(0.f))
		, Velocity(FVector(0.f))
		, AngularVelocity(FVector(0.f))
		, Mass(0.f)
		, ParticleIndex(INDEX_NONE)
		, ParticleIndexMesh(INDEX_NONE)
	{}

	FSolverTrailingData(FVector InLocation
		, FVector InVelocity
		, FVector InAngularVelocity
		, float InMass
		, int32 InParticleIndex
		, int32 InParticleIndexMesh)
		: Location(InLocation)
		, Velocity(InVelocity)
		, AngularVelocity(InAngularVelocity)
		, Mass(InMass)
		, ParticleIndex(InParticleIndex)
		, ParticleIndexMesh(InParticleIndexMesh)
	{}

	UPROPERTY()
	FVector Location;

	UPROPERTY()
	FVector Velocity;

	UPROPERTY()
	FVector AngularVelocity;

	UPROPERTY()
	float Mass;

	UPROPERTY()
	int32 ParticleIndex;

	UPROPERTY()
	int32 ParticleIndexMesh;

	friend inline uint32 GetTypeHash(const FSolverTrailingData& Other)
	{
		return ::GetTypeHash(Other.ParticleIndex);
	}

	friend bool operator==(const FSolverTrailingData& A, const FSolverTrailingData& B)
	{
		return A.ParticleIndex == B.ParticleIndex;
	}
};

UENUM()
enum class EGeometryCollectionCacheType : uint8
{
	None,
	Record,
	Play,
	RecordAndPlay
};

/**
 * Structure describing necessary data to record the results of a physics simulation for rigids.
 * Primarily for recording the results of a geometry collection simulation for later playback
 */
USTRUCT()
struct FRecordedFrame
{
	GENERATED_BODY()

	/**
	 * List of transforms recorded for this frame.
	 * During execution maps one-to-one with the number of particles. This is handed off for processing when written back to the collection.
	 * This processing will remove redundant transforms and populate the indices in FRecordedFrame to handle the remapping
	 */
	UPROPERTY()
	TArray<FTransform> Transforms;

	/**
	 * After processing, this will map a transform index within this frame out to a body/particle index for the component.
	 * This will map back to where the transform was before processing.
	 */
	UPROPERTY()
	TArray<int32> TransformIndices;

	/**
	 * Maps to the position in the Transforms array of the *previous* frame that this entry existed. They can move around as particles
	 * either become disabled/enabled or if a particle didn't move since the last frame. This allows us to pick up previous positions
	 * for velocities etc.
	 */
	UPROPERTY()
	TArray<int32> PreviousTransformIndices;

	/**
	 * Per-particle list of whether or not they were disabled on this frame.
	 * #BGallagher change this to be event based - expensive on memory and beginframe iterations
	 */
	UPROPERTY()
	TArray<bool> DisabledFlags;

	UPROPERTY()
	TArray<FSolverCollisionData> Collisions;

	UPROPERTY()
	TArray<FSolverBreakingData> Breakings;

	UPROPERTY()
	TSet<FSolverTrailingData> Trailings;

	UPROPERTY()
	float Timestamp = 0.0f;

	void Reset(int32 InNum = 0)
	{
		Transforms.Reset(InNum);
		DisabledFlags.Reset(InNum);

		if(InNum > 0)
		{
			Transforms.AddDefaulted(InNum);
			DisabledFlags.AddDefaulted(InNum);
		}

		Timestamp = -MAX_flt;

		Collisions.Reset();
		Breakings.Reset();
		Trailings.Reset();
	}
};

USTRUCT()
struct FRecordedTransformTrack
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FRecordedFrame> Records;

	float GetDt() const
	{
		return Records.Num() > 1 ? Records[1].Timestamp - Records[0].Timestamp : 0;
	}

	/**
	 * Validates that the time is within the track timestamp bounds (inclusive)
	 */
	bool IsTimeValid(float InTime) const
	{
		if(Records.Num() > 1)
		{
			const FRecordedFrame& First = Records[0];
			const FRecordedFrame& Last = Records.Last();

			return First.Timestamp <= InTime && InTime <= Last.Timestamp;
		}

		return false;
	}

	/**
	 * Finds the index of the key immediately before the provided time. Or the exact
	 * frame in the case of an exact time match
	 */
	int32 FindLastKeyBefore(float InTime) const
	{
		const int32 NumKeys = Records.Num();
		if(NumKeys > 0)
		{
			float FirstKeyTime = Records[0].Timestamp;
			float LastKeyTime = Records[NumKeys - 1].Timestamp;

			if(LastKeyTime <= InTime)
			{
				// Past the end of the records
				return NumKeys - 1;
			}

			if(FirstKeyTime >= InTime)
			{
				// Before the beginning of the records
				return 0;
			}

			// Linear search
			// #BG TODO Do something else, binary search
			for(int32 KeyIndex = 1; KeyIndex < NumKeys; ++KeyIndex)
			{
				const float CurrTime = Records[KeyIndex].Timestamp;
				if(CurrTime > InTime)
				{
					// Just passed over the specified time, last record is before
					return KeyIndex - 1;
				}
			}
		}

		return INDEX_NONE;
	}

	/**
	 * Find a frame at InTime if available within a specified tolarance of the timestamp
	 */
	const FRecordedFrame* FindRecordedFrame(float InTime, float InTolerance = UE_SMALL_NUMBER) const
	{
		for(const FRecordedFrame& Frame : Records)
		{
			if(FMath::IsNearlyEqual(Frame.Timestamp, InTime, InTolerance))
			{
				return &Frame;
			}
		}

		return nullptr;
	}

	/**
	 * Find a frame at InTime if available within a specified tolarance of the timestamp
	 */
	FRecordedFrame* FindRecordedFrame(float InTime, float InTolerance = UE_SMALL_NUMBER)
	{
		for(FRecordedFrame& Frame : Records)
		{
			if(FMath::IsNearlyEqual(Frame.Timestamp, InTime, InTolerance))
			{
				return &Frame;
			}
		}

		return nullptr;
	}

	/**
	 * Find a frame index at InTime if available within a specified tolarance of the timestamp
	 */
	int32 FindRecordedFrameIndex(float InTime, float InTolerance = UE_SMALL_NUMBER) const
	{
		const int32 NumFrames = Records.Num();
		for(int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			const FRecordedFrame& Frame = Records[FrameIndex];
			if(FMath::IsNearlyEqual(Frame.Timestamp, InTime, InTolerance))
			{
				return FrameIndex;
			}
		}

		return INDEX_NONE;
	}

	/**
	 * Given a time, return either one or two frames. If InTime is exactly on a frame then only that frame is returned.
	 * If we are beyond the end of the track only the last is returned. For times in the middle of the track that are
	 * not exactly on a frame then the frame before and after the time are returned.
	 */
	void GetFramesForTime(float InTime, const FRecordedFrame*& OutFirst, const FRecordedFrame*& OutSecond) const
	{
		// If we're exactly on a frame, just return that
		if(const FRecordedFrame* ExactFrame = FindRecordedFrame(InTime))
		{
			OutFirst = ExactFrame;
			OutSecond = nullptr;

			return;
		}

		const int32 KeyBeforeIndex = FindLastKeyBefore(InTime);
		const int32 KeyAfterIndex = KeyBeforeIndex + 1;

		// Past then end, just return the last frame
		if(KeyBeforeIndex == Records.Num() - 1)
		{
			OutFirst = &Records.Last();
			OutSecond = nullptr;

			return;
		}

		// Somewhere in the middle, return both
		OutFirst = &Records[KeyBeforeIndex];
		OutSecond = &Records[KeyAfterIndex];
	}

	/**
	 * Given a time, get an interpolated transform from the track for a provided particle
	 */
	FTransform GetTransformAtTime(int32 InIndex, float InTime) const
	{
		// If we're exactly on a frame, just return that
		if(const FRecordedFrame* ExactFrame = FindRecordedFrame(InTime))
		{
			return ExactFrame->Transforms[InIndex];
		}

		// Otherwise interpolate
		const int32 KeyBeforeIndex = FindLastKeyBefore(InTime);
		const int32 KeyAfterIndex = KeyBeforeIndex + 1;

		if(KeyBeforeIndex == Records.Num() - 1)
		{
			return Records.Last().Transforms[InIndex];
		}

		const FRecordedFrame& BeforeFrame = Records[KeyBeforeIndex];
		const FRecordedFrame& AfterFrame = Records[KeyAfterIndex];

		const float BeforeTime = BeforeFrame.Timestamp;
		const float AfterTime = AfterFrame.Timestamp;
		const float Alpha = (InTime - BeforeTime) / (AfterTime - BeforeTime);

		FTransform Result;
		Result.Blend(BeforeFrame.Transforms[InIndex], AfterFrame.Transforms[InIndex], Alpha);

		return Result;
	}

	/**
	 * Get whether a particle is disabled at a provided time
	 */
	bool GetDisabledAtTime(int32 InIndex, float InTime) const
	{
		// If we're exactly on a frame, just return that
		if(const FRecordedFrame* ExactFrame = FindRecordedFrame(InTime))
		{
			return ExactFrame->DisabledFlags[InIndex];
		}

		const int32 KeyBeforeIndex = FindLastKeyBefore(InTime);
		return Records[KeyBeforeIndex].DisabledFlags[InIndex];
	}

	/**
	 * Given two times, find out if the provided particle was ever active within them
	 */
	bool GetWasActiveInWindow(int32 InIndex, float InBeginTime, float InEndTime) const
	{
		if(InBeginTime == InEndTime)
		{
			return !GetDisabledAtTime(InIndex, InBeginTime);
		}

		if(InBeginTime > InEndTime)
		{
			Swap(InBeginTime, InEndTime);
		}

		const int32 KeyBeforeBeginIndex = FindLastKeyBefore(InBeginTime);
		const int32 KeyBeforeEndIndex = FindLastKeyBefore(InEndTime);
		const int32 Offset = KeyBeforeEndIndex - KeyBeforeBeginIndex;

		if(Offset < 2)
		{
			return !Records[KeyBeforeBeginIndex].DisabledFlags[InIndex];
		}

		for(int32 WindowKeyIndex = 0; WindowKeyIndex < Offset; ++WindowKeyIndex)
		{
			if(!Records[KeyBeforeBeginIndex + WindowKeyIndex].DisabledFlags[InIndex])
			{
				return true;
			}
		}

		return false;
	}

	/**
	 * Get an interpolated linear velocity for the specified particle at the provided time - using SampleWidth to pick out a window of interpolation
	 */
	FVector GetLinearVelocityAtTime(int32 InIndex, float InTime, float SampleWidth = 1.0f / 120.0f) const
	{
		if(Records.Num() == 0)
		{
			return FVector::ZeroVector;
		}
		// We're at the beginning of the cache, zero velocity (also guarantees we have at least SampleWidth before InTime)
		if(FMath::Abs(InTime - Records[0].Timestamp) <= (SampleWidth + UE_SMALL_NUMBER))
		{
			return FVector::ZeroVector;
		}

		FTransform Prev = GetTransformAtTime(InIndex, InTime - SampleWidth);
		FTransform Curr = GetTransformAtTime(InIndex, InTime);

		return (Curr.GetTranslation() - Prev.GetTranslation()) / SampleWidth;
	}

	/**
	 * Get an interpolated angular velocity for the specified particle at the provided time - using SampleWidth to pick out a window of interpolation
	 */
	FVector GetAngularVelocityAtTime(int32 InIndex, float InTime, float SampleWidth = 1.0f / 120.0f) const
	{
		if(Records.Num() == 0)
		{
			return FVector::ZeroVector;
		}
		// We're at the beginning of the cache, zero velocity (also guarantees we have at least SampleWidth before InTime)
		if(FMath::Abs(InTime - Records[0].Timestamp) <= (SampleWidth + UE_SMALL_NUMBER))
		{
			return FVector::ZeroVector;
		}

		FTransform Prev = GetTransformAtTime(InIndex, InTime - SampleWidth);
		FTransform Curr = GetTransformAtTime(InIndex, InTime);

		FQuat Delta = Curr.GetRotation() * Prev.GetRotation().Inverse();
		FVector Axis;
		FVector::FReal Angle;
		Delta.ToAxisAndAngle(Axis, Angle);

		return (Axis * Angle) / SampleWidth;
	}

	static CHAOS_API FRecordedTransformTrack ProcessRawRecordedData(const FRecordedTransformTrack& InCache);

};
