// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayCollection.h"
#include "Misc/Crc.h"

/**
* TransformCollection (ManagedArrayCollection)
*
* Stores the TArray<T> groups necessary to process transform hierarchies.
*
* @see FTransformCollectionComponent
*/
class FTransformCollection : public FManagedArrayCollection
{
public:
	typedef FManagedArrayCollection Super;

	CHAOS_API FTransformCollection();
	CHAOS_API ~FTransformCollection();
	FTransformCollection(FTransformCollection &) = delete;
	FTransformCollection& operator=(const FTransformCollection&) = delete;
	FTransformCollection(FTransformCollection&&) = default;
	FTransformCollection& operator=(FTransformCollection&&) = default;
	MANAGED_ARRAY_COLLECTION_INTERNAL(FTransformCollection);

	/***
	*  Attribute Groups
	*
	*   These attribute groups are predefined data member of the FTransformCollection.
	*
	*   TransformGroup ("Transform")
	*		Default Attributes :
	*
	*          FTransform3fArray Transform =  GetAttribute<FTransform3f>("Transform", TransformGroup)
	*		   FInt32Array Level = GetAttribute<int32>("Level", TransformGroup) FIX
	*		   FInt32Array Parent = GetAttribute<int32>("Parent", TransformGroup) FIX
	*		   FInt32Array Children = GetAttribute<TSet<int32>>("Children", TransformGroup) FIX
	*
	*       The TransformGroup defines transform information for each Vertex. All positional
	*       information stored within Vertices and Geometry groups should be relative to its
	*       TransformGroup Transform.
	*       Parent defines the parent index of one transform node relative to another (Invalid is no parent exists, i.e. is root)
	*       Children defines the child indices of the transform node in the transform hierarchy (leaf nodes will have no children)
	*       Level is the distance from the root node at level 0. Leaf nodes will have the highest level number.
	*/
	static CHAOS_API const FName TransformGroup;
	static CHAOS_API const FName TransformAttribute;
	static CHAOS_API const FName ParentAttribute;
	static CHAOS_API const FName ChildrenAttribute;
	static CHAOS_API const FName ParticlesAttribute;
	static CHAOS_API const FName LevelAttribute;
	static CHAOS_API const FName ConvexGroup;
	static CHAOS_API const FName ConvexHullAttribute;
	static CHAOS_API const FName BoneNameAttribute;
	static CHAOS_API const FName BoneColorAttribute;
	static CHAOS_API const FName BoneHierarchyAttribute;
	static CHAOS_API const FName GeometryIndexAttribute;
	
	/** Schema */
	static CHAOS_API void DefineTransformSchema(FManagedArrayCollection&);


	/** Serialize */
	CHAOS_API virtual void Serialize(Chaos::FChaosArchive& Ar) override;

	/*
	* SingleTransform:
	*   Create a single transform.
	*/
	static CHAOS_API FTransformCollection SingleTransform(const FTransform& TransformRoot = FTransform::Identity);

	/**
	* Append Collection and reindex dependencies on this collection.
	* @param InCollection : FManagedArrayCollection to add.
	*/
	CHAOS_API virtual void Append(const FManagedArrayCollection& Collection) override;

	/**
	* Append Collection and reindex dependencies on this collection.
	* @param InCollection : FTransformCollection to add.
	*/
	CHAOS_API void AppendCollection(const FTransformCollection& Collection);

	/*
	* AppendTransform:
	*   Append a transform at the end of the collection without
	*   parenting. 
	*/
	CHAOS_API int32 AppendTransform(const FTransformCollection & GeometryCollection, const FTransform& TransformRoot = FTransform::Identity);

	/*
	* ParentTransforms
	*   Parent Transforms under the specified node using local parent
	*   hierarchy compensation. .
	*/
	CHAOS_API void ParentTransforms(const int32 TransformIndex, const int32 ChildIndex);
	CHAOS_API void ParentTransforms(const int32 TransformIndex, const TArray<int32>& SelectedBones);
	CHAOS_API void UnparentTransform(const int32 ChildIndex);

	/*
	* RelativeTransformation
	*   Modify the specified index by the local matrix offset. 
	*/
	CHAOS_API void RelativeTransformation(const int32& Index, const FTransform& LocalOffset); 

	/**
	* RemoveElements
	*   Remove elements from the transform collection. Transform children are re-parented
	*   under the deleted elements parent using local parent compensation [relative local matrices].
	* 
	*/
	CHAOS_API virtual void RemoveElements(const FName & Group, const TArray<int32> & SortedDeletionList, FProcessingParameters Params = FProcessingParameters()) override;

	/**
	* MergeElements
	*   Merge elements from the transform collection. Updates parent and children hierachy to the remapped index
	*	Deletes merged elements 
	*/
	CHAOS_API virtual void MergeElements(const FName& Group, const TArray<int32>& SortedMergeList, const TArray<int32>& MergeRemapIndex, FProcessingParameters Params = FProcessingParameters()) override;

	/**
	* reset internal state
	*/
	CHAOS_API virtual void Reset() override;

	// Transform Group
	TManagedArray<FTransform3f> Transform;
	TManagedArray<FString>      BoneName;
	TManagedArray<FLinearColor> BoneColor;
	TManagedArray<int32>        Parent;
	TManagedArray<TSet<int32>>  Children;


protected:

	/** Construct */
	CHAOS_API void Construct();
};
