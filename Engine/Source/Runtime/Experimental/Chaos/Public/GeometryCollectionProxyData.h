// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Crc.h"

#include "GeometryCollection/ManagedArrayCollection.h"
#include "GeometryCollection/GeometryCollectionCollisionStructureManager.h"

/*
* Managed arrays for simulation data used by the GeometryCollectionProxy
*/

/**
* FTransformDynamicCollection (FManagedArrayCollection)
*
* Stores per instance data for transforms and hierarchy information
*/
class FTransformDynamicCollection : public FManagedArrayCollection
{
public:
	typedef FManagedArrayCollection Super;

	UE_DEPRECATED(5.4, "No longer handles a raw ptr, use the version taking TSharedPtr")
	CHAOS_API FTransformDynamicCollection(const FGeometryCollection* InRestCollection);
	CHAOS_API FTransformDynamicCollection(TSharedPtr<const FGeometryCollection> InRestCollection);
	FTransformDynamicCollection(FTransformDynamicCollection&) = delete;
	FTransformDynamicCollection& operator=(const FTransformDynamicCollection&) = delete;
	FTransformDynamicCollection(FTransformDynamicCollection&&) = delete;
	FTransformDynamicCollection& operator=(FTransformDynamicCollection&&) = delete;

	CHAOS_API const FTransform3f& GetTransform(int32 Index) const;
	CHAOS_API void SetTransform(int32 Index, const FTransform3f& Transform);
	CHAOS_API int32 GetNumTransforms() const;
	CHAOS_API void ResetInitialTransforms();

	CHAOS_API const TManagedArray<bool>& GetHasParent() const;
	CHAOS_API bool GetHasParent(int32 Index) const;
	CHAOS_API void SetHasParent(int32 Index, bool Value);
	CHAOS_API int32 GetParent(int32 Index) const;

	CHAOS_API bool HasChildren(int32 Index) const;

	/** check if the transform is a cluster - ie has originally children as per rest collection */
	CHAOS_API bool IsCluster(int32 Index) const;

	template<typename Lambda>
	void IterateThroughChildren(int32 Index, Lambda&& LambdaIt) const
	{
		if (RestCollectionShared && RestCollectionShared->Children.IsValidIndex(Index))
		{	
			const TSet<int32>& Children = RestCollectionShared->Children[Index];
			for (const int32 Child : Children)
			{
				if (GetHasParent(Child))
				{
					bool bContinue = LambdaIt(Child);
					if (!bContinue)
					{
						break;
					}
				}
			}
		}
	}

protected:
	UE_DEPRECATED(5.4, "Raw pointer no longer in use, instead access the shared version")
	const FGeometryCollection* RestCollection;
	TSharedPtr<const FGeometryCollection> RestCollectionShared;
private:
	TManagedArray<bool>         HasParent;
	TManagedArray<FTransform3f> Transform;
	bool bTransformHasChanged;

	/** Construct */
	CHAOS_API void Construct();
	void InitializeTransforms();
};


/**
* FGeometryDynamicCollection (FTransformDynamicCollection)
*
* Stores per instance data for simulation level information
*/

class FGeometryDynamicCollection : public FTransformDynamicCollection
{
public:
	UE_DEPRECATED(5.4, "No longer handles a raw ptr, use the version taking TSharedPtr")
	CHAOS_API FGeometryDynamicCollection(const FGeometryCollection* InRestCollection);
	CHAOS_API FGeometryDynamicCollection(TSharedPtr<const FGeometryCollection> InRestCollection);
	FGeometryDynamicCollection(FGeometryDynamicCollection&) = delete;
	FGeometryDynamicCollection& operator=(const FGeometryDynamicCollection&) = delete;
	FGeometryDynamicCollection(FGeometryDynamicCollection&&) = delete;
	FGeometryDynamicCollection& operator=(FGeometryDynamicCollection&&) = delete;

	typedef FTransformDynamicCollection Super;

	static CHAOS_API const FName ActiveAttribute;
	static CHAOS_API const FName DynamicStateAttribute;
	static CHAOS_API const FName ImplicitsAttribute;
	static CHAOS_API const FName ShapesQueryDataAttribute;
	static CHAOS_API const FName ShapesSimDataAttribute;
	static CHAOS_API const FName SharedImplicitsAttribute;
	static CHAOS_API const FName SimplicialsAttribute;
	static CHAOS_API const FName SimulatableParticlesAttribute;
	static CHAOS_API const FName InternalClusterParentTypeAttribute;

	UE_DEPRECATED(5.4, "CollisionMaskAttribute is no longer supported")
	static CHAOS_API const FName CollisionMaskAttribute;

	UE_DEPRECATED(5.4, "CollisionGroupAttribute is no longer supported")
	static CHAOS_API const FName CollisionGroupAttribute;

	// Transform Group
	TManagedArray<bool> Active;
	
	TManagedArray<uint8> DynamicState; 
	static_assert(sizeof(EObjectStateTypeEnum) <= sizeof(uint8)); // DynamicState must fit  EObjectStateTypeEnum

	UE_DEPRECATED(5.5, "Simplicials array is optional and should be queried using the FindAttribute method")
	TManagedArray<TUniquePtr<FCollisionStructureManager::FSimplicial>> Simplicials;

	TManagedArray<bool> SimulatableParticles;

	UE_DEPRECATED(5.4, "CollisionStructureID attribute is no longer supported")
	TManagedArray<int32> CollisionStructureID;

	UE_DEPRECATED(5.4, "CollisionMask attribute is no longer supported")
	TManagedArray<int32> CollisionMask;

	UE_DEPRECATED(5.4, "CollisionGroup attribute is no longer supported - you can still set the collision group on the geometry collection")
	TManagedArray<int32> CollisionGroup;

public:

	CHAOS_API const TManagedArrayAccessor<int32> GetInitialLevels() const;

	const TManagedArray<uint8>& GetInternalClusterParentTypeAttribute() const { return InternalClusterParentType; }
	TManagedArray<uint8>& GetInternalClusterParentTypeAttribute() { return InternalClusterParentType; }

	CHAOS_API void AddVelocitiesAttributes();
	const TManagedArray<FVector3f>* GetLinearVelocitiesAttribute() const { return OptionalLinearVelocityAttribute; }
	const TManagedArray<FVector3f>* GetAngularVelocitiesAttribute() const { return OptionalAngularVelocityAttribute; };
	TManagedArray<FVector3f>* GetLinearVelocitiesAttribute() { return OptionalLinearVelocityAttribute; }
	TManagedArray<FVector3f>* GetAngularVelocitiesAttribute() { return OptionalAngularVelocityAttribute; };

	CHAOS_API void AddAnimateTransformAttribute();
	const TManagedArray<bool>* GetAnimateTransformAttribute() const { return OptionalAnimateTransformAttribute; }
	TManagedArray<bool>* GetAnimateTransformAttribute() { return OptionalAnimateTransformAttribute; }

	struct FInitialVelocityFacade
	{
		FInitialVelocityFacade(FGeometryDynamicCollection& DynamicCollection);
		FInitialVelocityFacade(const FGeometryDynamicCollection& DynamicCollection);

		bool IsValid() const;
		void DefineSchema();
		void Fill(const FVector3f& InitialLinearVelocity, const FVector3f& InitialAngularVelocity);
		void CopyFrom(const FGeometryDynamicCollection& SourceCollection);

		TManagedArrayAccessor<FVector3f> InitialLinearVelocityAttribute;
		TManagedArrayAccessor<FVector3f> InitialAngularVelocityAttribute;
	};

	FInitialVelocityFacade GetInitialVelocityFacade() { return FInitialVelocityFacade(*this); }
	FInitialVelocityFacade GetInitialVelocityFacade() const { return FInitialVelocityFacade(*this); }

	CHAOS_API void CopyInitialVelocityAttributesFrom(const FGeometryDynamicCollection& SourceCollection);

	void MakeDirty() { bDirty = true; }
	void MakeClean() { bDirty = false; }
	bool IsDirty() const { return bDirty; }

	virtual void Reset() override
	{
		Super::Reset();
		MakeDirty();
	}

private:
	TManagedArray<uint8>	  InternalClusterParentType;

	TManagedArray<FVector3f>* OptionalLinearVelocityAttribute;
	TManagedArray<FVector3f>* OptionalAngularVelocityAttribute;
	TManagedArray<bool>*	  OptionalAnimateTransformAttribute;
	bool bDirty = false;
};

/**
 * Provides an API for dynamic state related attributes
 * physics state , broken state, current parent (normal or internal clusters )
 * To be used with the dynamic collection
 */
class FGeometryCollectionDynamicStateFacade
{
public:
	CHAOS_API FGeometryCollectionDynamicStateFacade(FGeometryDynamicCollection& InCollection);

	/** returns true if all the necessary attributes are present */
	CHAOS_API bool IsValid() const;

	/** return true if the transform is active */
	CHAOS_API bool IsActive(int32 TransformIndex) const;

	/** return true if the transform is in a dynamic or sleeping state */
	CHAOS_API bool IsDynamicOrSleeping(int32 TransformIndex) const;

	/** return true if the transform is in a sleeping state */
	CHAOS_API bool IsSleeping(int32 TransformIndex) const;

	/** whether there's children attached to this transform (Cluster) */
	CHAOS_API bool HasChildren(int32 TransformIndex) const;
	
	/** return true if the transform has broken off its parent */
	CHAOS_API bool HasBrokenOff(int32 TransformIndex) const;

	/** return true if the transform has an internal cluster parent */
	CHAOS_API bool HasInternalClusterParent(int32 TransformIndex) const;

	/** return true if the transform has an internal cluster parent in a dynamic state */
	CHAOS_API bool HasDynamicInternalClusterParent(int32 TransformIndex) const;

	/** Return true if the transform has a cluster union parent. */
	CHAOS_API bool HasClusterUnionParent(int32 TransformIndex) const;
	
private:
	/** Active state, true means that the transform is active or broken off from its parent */
	TManagedArrayAccessor<bool> ActiveAttribute;

	/** physics state of the transform (Dynamic, kinematic, static, sleeping) */
	TManagedArrayAccessor<uint8> DynamicStateAttribute;

	/** type of internal state parent */
	TManagedArrayAccessor<uint8> InternalClusterParentTypeAttribute;

	FGeometryDynamicCollection& DynamicCollection;
};

/**
 * Buffer structure for communicating simulation state between game and physics
 * threads.
 */
class FGeometryCollectionResults: public FRefCountedObject
{
public:
	FGeometryCollectionResults();

	int32 GetNumEntries() const { return States.Num(); }

	void Reset();

	void InitArrays(const FGeometryDynamicCollection& Collection)
	{
		const int32 NumTransforms = Collection.NumElements(FGeometryCollection::TransformGroup);
		ModifiedTransformIndices.Init(false, NumTransforms);
#if WITH_EDITORONLY_DATA
		if (Damages.Num() != NumTransforms)
		{
			Damages.SetNumUninitialized(NumTransforms);
		}
#endif	
	}

	using FEntryIndex = int32;

	struct FState
	{
		uint16 DynamicState: 8; // need to fit EObjectStateTypeEnum
		uint16 DisabledState: 1;
		uint16 HasDecayed : 1; // particle has been released but disabled right away
		uint16 HasInternalClusterParent: 1;
		uint16 DynamicInternalClusterParent: 1;
		uint16 HasClusterUnionParent: 1;
		// 3 bits left
	};

	struct FStateData
	{
		int32  TransformIndex;
		int32  InternalClusterUniqueIdx;
		bool   HasParent;
		FState State;
	};

	struct FPositionData
	{
		Chaos::FVec3 ParticleX;
		Chaos::FRotation3 ParticleR;
	};

	struct FVelocityData
	{
		Chaos::FVec3f ParticleV;
		Chaos::FVec3f ParticleW;
	};

#if WITH_EDITORONLY_DATA
	struct FDamageData
	{
		float Damage = 0;
		float DamageThreshold = 0;
	};

	void SetDamages(int32 TransformIndex, const FDamageData& DamageData)
	{
		Damages[TransformIndex] = DamageData;
	}

	const FDamageData& GetDamages(int32 TransformIndex) const
	{
		return Damages[TransformIndex];
	}
#endif

	inline FEntryIndex GetEntryIndexByTransformIndex(int32 TransformIndex) const
	{
		if (ModifiedTransformIndices[TransformIndex])
		{
			return ModifiedTransformIndices.CountSetBits(0, TransformIndex + 1) - 1;
		}
		return INDEX_NONE;
	}

	inline const FStateData& GetState(FEntryIndex EntryIndex) const
	{
		return States[EntryIndex];
	}

	inline const FPositionData& GetPositions(FEntryIndex EntryIndex) const
	{
		return Positions[EntryIndex];
	}

	inline const FVelocityData& GetVelocities(FEntryIndex EntryIndex) const
	{
		return Velocities[EntryIndex];
	}

	inline void SetSolverDt(const Chaos::FReal SolverDtIn)
	{
		SolverDt = SolverDtIn;
	}

	inline void SetState(int32 EntryIndex, const FStateData& StateData)
	{
		States[EntryIndex] = StateData;
	}

	inline FEntryIndex AddEntry(int32 TransformIndex)
	{
		ModifiedTransformIndices[TransformIndex] = true;
		const FEntryIndex EntryIndex = States.AddDefaulted();
		ensure(GetEntryIndexByTransformIndex(TransformIndex) == EntryIndex);
		Positions.AddDefaulted();
		Velocities.AddDefaulted();
		return EntryIndex;
	}

	inline void SetPositions(FEntryIndex EntryIndex, const FPositionData& PositionData)
	{
		Positions[EntryIndex] = PositionData;
	}

	inline void SetVelocities(FEntryIndex EntryIndex, const FVelocityData& VelocityData)
	{
		Velocities[EntryIndex] = VelocityData;
	}

	inline const TBitArray<>& GetModifiedTransformIndices() const
	{
		return ModifiedTransformIndices;
	}

private:
	Chaos::FReal SolverDt;

	// we only store the data for modified transforms
	// ModifiedTransformIndices contains which transform has been set 
	// use the API to retrieve the entry Index matching a specific transform index
	TBitArray<> ModifiedTransformIndices;
	TArray<FStateData> States;
	TArray<FPositionData> Positions;
	TArray<FVelocityData> Velocities;

#if WITH_EDITORONLY_DATA
	// use to display impulse statistics in editor
	// this is indexed on the transform index
	TArray<FDamageData> Damages;
#endif

public:
	uint8 IsObjectDynamic: 1;
	uint8 IsObjectLoading: 1;
	uint8 IsRootBroken : 1;
};
