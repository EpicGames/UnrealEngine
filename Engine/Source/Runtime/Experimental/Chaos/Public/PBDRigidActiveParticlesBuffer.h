// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Framework/MultiBufferResource.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"

namespace Chaos
{
	class FSingleParticlePhysicsProxy;

	/**
	 * Solver specific data buffered for use on Game thread
	 */
	struct FPBDRigidDirtyParticlesBufferOut
	{
		TArray<FSingleParticlePhysicsProxy*> DirtyGameThreadParticles;
		// Some particle types (clustered) only exist on the game thread, but we
		// still need to pull data over via their proxies.
		TSet<IPhysicsProxyBase*> PhysicsParticleProxies;
	};


	class FPBDRigidDirtyParticlesBuffer
	{
		friend class FPBDRigidDirtyParticlesBufferAccessor;

	public:
		CHAOS_API FPBDRigidDirtyParticlesBuffer(const Chaos::EMultiBufferMode& InBufferMode, bool bInSingleThreaded);

		CHAOS_API void CaptureSolverData(FPBDRigidsSolver* Solver);

		CHAOS_API void ReadLock();
		CHAOS_API void ReadUnlock();
		CHAOS_API void WriteLock();
		CHAOS_API void WriteUnlock();
	
	private:
		const FPBDRigidDirtyParticlesBufferOut* GetSolverOutData() const
		{
			return SolverDataOut->GetConsumerBuffer();
		}

		/**
		 * Fill data from solver destined for the game thread - used to limit the number of objects updated on the game thread
		 */
		CHAOS_API void BufferPhysicsResults(FPBDRigidsSolver* Solver);

		/**
		 * Flip should be performed on physics thread side non-game thread
		 */
		void FlipDataOut()
		{
			SolverDataOut->FlipProducer();
		}

		Chaos::EMultiBufferMode BufferMode;
		FRWLock ResourceOutLock;
		bool bUseLock;

		// Physics thread to game thread
		TUniquePtr<IBufferResource<FPBDRigidDirtyParticlesBufferOut>> SolverDataOut;
	};

	class FPBDRigidDirtyParticlesBufferAccessor
	{
	public:
		FPBDRigidDirtyParticlesBufferAccessor(FPBDRigidDirtyParticlesBuffer* InManager) : Manager(InManager)
		{
			check(InManager);
			Manager->ReadLock();
		}

		const FPBDRigidDirtyParticlesBufferOut* GetSolverOutData() const
		{
			return Manager->GetSolverOutData();
		}

		~FPBDRigidDirtyParticlesBufferAccessor()
		{
			Manager->ReadUnlock();
		}

	private:
		FPBDRigidDirtyParticlesBuffer* Manager;
	};
}
