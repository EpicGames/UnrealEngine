// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Defines.h"
#include "Chaos/Framework/MultiBufferResource.h"
#include "Chaos/Framework/PhysicsProxy.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Chaos/PBDRigidParticles.h"
#include "Chaos/PBDRigidsEvolutionGBF.h"
#include "Chaos/PBDCollisionConstraints.h"
#include "Chaos/PBDRigidDynamicSpringConstraints.h"
#include "Chaos/PBDPositionConstraints.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "Chaos/PBDSuspensionConstraints.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/PerParticleGravity.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Serialization/SolverSerializer.h"
#include "Chaos/Transform.h"
#include "Chaos/Framework/PhysicsProxy.h"
#include "ChaosSolversModule.h"
#include "CoreMinimal.h"
#include "Containers/Queue.h"
#include "EventManager.h"
#include "Field/FieldSystem.h"
#include "PBDRigidActiveParticlesBuffer.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"
#include "PhysicsProxy/JointConstraintProxy.h"
#include "PhysicsProxy/SuspensionConstraintProxy.h"
#include "SolverEventFilters.h"

class FPhysInterface_Chaos;
struct FChaosSolverConfiguration;
class FSkeletalMeshPhysicsProxy;
class FStaticMeshPhysicsProxy;
class FPerSolverFieldSystem;

#define PBDRIGID_PREALLOC_COUNT 1024
#define KINEMATIC_GEOM_PREALLOC_COUNT 100
#define GEOMETRY_PREALLOC_COUNT 100

extern int32 ChaosSolverParticlePoolNumFrameUntilShrink;

DECLARE_MULTICAST_DELEGATE_TwoParams(FSolverParticleUnregisteredGT, Chaos::FSingleParticlePhysicsProxy* /*Proxy*/, Chaos::FUniqueIdx /*UniqueIdx*/);

namespace ChaosTest
{
	template <typename TSolver>
	void AdvanceSolverNoPushHelper(TSolver* Solver, Chaos::FReal Dt);
}

namespace Chaos::Private
{
	class FTaskDispatcherSolver;

	UE_INTERNAL CHAOS_API bool IsUsingSingleThreadPullData();
}
/**
*
*/
namespace Chaos
{
	class FCharacterGroundConstraint;
	class FCharacterGroundConstraintProxy;
	class FClusterUnionPhysicsProxy;

	template <typename T,typename R,int d>
	class ISpatialAccelerationCollection;

	class FAccelerationStructureHandle;

	enum class ELockType : uint8
	{
		Read,
		Write
	};

	template<ELockType LockType>
	struct TSolverSimMaterialScope
	{
		TSolverSimMaterialScope() = delete;
	};

	/**
	*
	*/
	class FPBDRigidsSolver : public FPhysicsSolverBase
	{

		CHAOS_API FPBDRigidsSolver(const EMultiBufferMode BufferingModeIn, UObject* InOwner, FReal AsyncDt);
		CHAOS_API virtual ~FPBDRigidsSolver();

	public:

		typedef FPhysicsSolverBase Super;

		friend class FPersistentPhysicsTask;
		friend class ::FChaosSolversModule;

		template<EThreadingMode Mode>
		friend class FDispatcher;
		
		friend class FEventDefaults;

		friend class FPhysInterface_Chaos;
		friend class FPhysScene_ChaosInterface;
		friend class FPBDRigidDirtyParticlesBuffer;

		void* PhysSceneHack;	//This is a total hack for now to get at the owning scene

		typedef FPBDRigidsSOAs FParticlesType;
		typedef FPBDRigidDirtyParticlesBuffer FDirtyParticlesBuffer;

		typedef Chaos::FGeometryParticle FParticle;
		typedef Chaos::FGeometryParticleHandle FHandle;
		typedef Chaos::FPBDRigidsEvolutionGBF FPBDRigidsEvolution;

		typedef FPBDRigidDynamicSpringConstraints FRigidDynamicSpringConstraints;
		typedef FPBDPositionConstraints FPositionConstraints;

		using FJointConstraints = FPBDJointCombinedConstraints;
		//
		// Execution API
		//

		CHAOS_API void ChangeBufferMode(Chaos::EMultiBufferMode InBufferMode);

		//
		//  Object API
		//

		CHAOS_API void RegisterObject(FSingleParticlePhysicsProxy* Proxy);
		CHAOS_API void UnregisterObject(FSingleParticlePhysicsProxy* Proxy);

		CHAOS_API void RegisterObject(FGeometryCollectionPhysicsProxy* InProxy);
		CHAOS_API void UnregisterObject(FGeometryCollectionPhysicsProxy* InProxy);

		CHAOS_API void RegisterObject(FClusterUnionPhysicsProxy* Proxy);
		CHAOS_API void UnregisterObject(FClusterUnionPhysicsProxy* Proxy);

		CHAOS_API void RegisterObject(Chaos::FJointConstraint* GTConstraint);
		CHAOS_API void UnregisterObject(Chaos::FJointConstraint* GTConstraint);

		CHAOS_API void RegisterObject(Chaos::FSuspensionConstraint* GTConstraint);
		CHAOS_API void UnregisterObject(Chaos::FSuspensionConstraint* GTConstraint);

		CHAOS_API void RegisterObject(Chaos::FCharacterGroundConstraint* GTConstraint);
		CHAOS_API void UnregisterObject(Chaos::FCharacterGroundConstraint* GTConstraint);

		/** Fired on the game thread when a single particle physics proxy is unregistered.
		    Called during UnregisterObject before the proxy is enqueued for destruction. */
		UE_INTERNAL CHAOS_API FDelegateHandle AddParticleUnregisteredGTCallback(FSolverParticleUnregisteredGT::FDelegate InDelegate);
		UE_INTERNAL CHAOS_API bool            RemoveParticleUnregisteredGTCallback(FDelegateHandle InHandle);

		//
		//  Simulation API
		//

		/**/
		FDirtyParticlesBuffer* GetDirtyParticlesBuffer() const { return MDirtyParticlesBuffer.Get(); }

		CHAOS_API int32 NumJointConstraints() const;
		CHAOS_API int32 NumCollisionConstraints() const;

		//Make friend with unit test code so we can verify some behavior
		template <typename TSolver>
		friend void ChaosTest::AdvanceSolverNoPushHelper(TSolver* Solver, FReal Dt);

		/**/
		CHAOS_API void Reset();

		/**/
		CHAOS_API void StartingSceneSimulation();

		/**/
		CHAOS_API void CompleteSceneSimulation();

		/**/
		CHAOS_API void UpdateGameThreadStructures();



		/**/
		void SetCurrentFrame(const int32 CurrentFrameIn) { CurrentFrame = CurrentFrameIn; }
		int32& GetCurrentFrame() { return CurrentFrame; }

#if WITH_CHAOS_VISUAL_DEBUGGER
		virtual int32 GetCVDFrameNumber() const override { return CurrentFrame; }
#endif

		/**/
		void SetPositionIterations(const int32 InNumIterations) { GetEvolution()->SetNumPositionIterations(InNumIterations); }
		void SetVelocityIterations(const int32 InNumIterations) { GetEvolution()->SetNumVelocityIterations(InNumIterations); }
		void SetProjectionIterations(const int32 InNumIterations) { GetEvolution()->SetNumProjectionIterations(InNumIterations); }
		void SetCollisionCullDistance(const FReal InCullDistance) { GetEvolution()->GetCollisionConstraints().SetCullDistance(InCullDistance); }
		void SetVelocityBoundsExpansion(const FReal BoundsVelocityMultiplier, const FReal MaxBoundsVelocityExpansion);
		void SetVelocityBoundsExpansionMACD(const FReal BoundsVelocityMultiplier, const FReal MaxBoundsVelocityExpansion);
		void SetCollisionMaxPushOutVelocity(const FReal InMaxPushOutVelocity) { GetEvolution()->GetCollisionConstraints().SetMaxPushOutVelocity(InMaxPushOutVelocity); }
		void SetCollisionDepenetrationVelocity(const FRealSingle InVelocity) { GetEvolution()->GetCollisionConstraints().SetDepenetrationVelocity(InVelocity); }

		/**/
		void SetGenerateCollisionData(bool bDoGenerate) { GetEventFilters()->SetGenerateCollisionEvents(bDoGenerate); }
		void SetGenerateBreakingData(bool bDoGenerate)
		{
			GetEventFilters()->SetGenerateBreakingEvents(bDoGenerate);
			GetEvolution()->GetRigidClustering().SetGenerateClusterBreaking(bDoGenerate);
		}
		void SetGenerateTrailingData(bool bDoGenerate) { GetEventFilters()->SetGenerateTrailingEvents(bDoGenerate); }
		void SetGenerateRemovalData(bool bDoGenerate) { GetEventFilters()->SetGenerateRemovalEvents(bDoGenerate); }
		void SetCollisionFilterSettings(const FSolverCollisionFilterSettings& InCollisionFilterSettings) { GetEventFilters()->GetCollisionFilter()->UpdateFilterSettings(InCollisionFilterSettings); }
		void SetBreakingFilterSettings(const FSolverBreakingFilterSettings& InBreakingFilterSettings) { GetEventFilters()->GetBreakingFilter()->UpdateFilterSettings(InBreakingFilterSettings); }
		void SetTrailingFilterSettings(const FSolverTrailingFilterSettings& InTrailingFilterSettings) { GetEventFilters()->GetTrailingFilter()->UpdateFilterSettings(InTrailingFilterSettings); }
		void SetRemovalFilterSettings(const FSolverRemovalFilterSettings& InRemovalFilterSettings) { GetEventFilters()->GetRemovalFilter()->UpdateFilterSettings(InRemovalFilterSettings); }

		/**
		 * @brief True if the simulation is running in deterministic mode
		 * This will be true if determinism is explicitly requested (via SetIsDeterministic()) or if required
		 * by some other system like Rewind/Resim support.
		*/
		CHAOS_API bool IsDetemerministic() const;

		/**
		 * @brief Request that the sim be deterministic (or not)
		 * @note Even if set to false, the sim may still be deterministic if some other feature is enabled and requires it.
		 * @see IsDetemerministic()
		*/
		CHAOS_API void SetIsDeterministic(const bool bInIsDeterministic);

		UE_DEPRECATED(5.5, "No longer used, see combined constraint in FPBDRigidsSolver::GetJointCombinedConstraints")
		FPBDJointConstraints& GetJointConstraints() { return MEvolution->GetJointCombinedConstraints().LinearConstraints; }
		UE_DEPRECATED(5.5, "No longer used, see combined constraint in FPBDRigidsSolver::GetJointCombinedConstraints")
		const FPBDJointConstraints& GetJointConstraints() const { return MEvolution->GetJointCombinedConstraints().LinearConstraints; }

		/**/
		FJointConstraints& GetJointCombinedConstraints() { return MEvolution->GetJointCombinedConstraints(); }
		const FJointConstraints& GetJointCombinedConstraints() const { return MEvolution->GetJointCombinedConstraints(); }

		FPBDSuspensionConstraints& GetSuspensionConstraints() { return MEvolution->GetSuspensionConstraints(); }
		const FPBDSuspensionConstraints& GetSuspensionConstraints() const { return MEvolution->GetSuspensionConstraints(); }
		CHAOS_API void SetSuspensionTarget(Chaos::FSuspensionConstraint* GTConstraint, const FVector& TargetPos, const FVector& Normal, bool Enabled, const IPhysicsProxyBase* GroundProxy = nullptr);

		FCharacterGroundConstraintContainer& GetCharacterGroundConstraints() { return MEvolution->GetCharacterGroundConstraints(); }
		const FCharacterGroundConstraintContainer& GetCharacterGroundConstraints() const { return MEvolution->GetCharacterGroundConstraints(); }

		CHAOS_API void EnableRewindCapture(int32 NumFrames, bool InUseCollisionResimCache, TUniquePtr<IRewindCallback>&& RewindCallback);
		CHAOS_API void EnableRewindCapture(int32 NumFrames, bool InUseCollisionResimCache);
		CHAOS_API void EnableRewindCapture(int32 NumFrames);
		CHAOS_API void EnableRewindCapture();

		/**/
		FPBDRigidsEvolution* GetEvolution() { return MEvolution.Get(); }
		FPBDRigidsEvolution* GetEvolution() const { return MEvolution.Get(); }

		FParticlesType& GetParticles() { return Particles; }
		const FParticlesType& GetParticles() const { return Particles; }
		
		/**/
		FEventManager* GetEventManager() { return MEventManager.Get(); }
		virtual void FlipEventManagerBuffer() { MEventManager->FlipBuffersIfRequired(); }

		/**/
		FSolverEventFilters* GetEventFilters() { return MSolverEventFilters.Get(); }
		FSolverEventFilters* GetEventFilters() const { return MSolverEventFilters.Get(); }

		/**/
		CHAOS_API void SyncEvents_GameThread();

		/**/
		CHAOS_API void PreIntegrateDebugDraw(FReal Dt) const;
		CHAOS_API void PreSolveDebugDraw(FReal Dt) const;
		CHAOS_API void PostTickDebugDraw(FReal Dt) const;

		// Visual debugger (VDB) push methods
		UE_DEPRECATED(5.4, "This method will be removed in the future")
		void PostEvolutionVDBPush() const {};

		TArray<FGeometryCollectionPhysicsProxy*>& GetGeometryCollectionPhysicsProxies_Internal()
		{
			return GeometryCollectionPhysicsProxies_Internal;
		}

		TArray<FGeometryCollectionPhysicsProxy*>& GetGeometryCollectionPhysicsProxiesField_Internal()
		{
			return GeometryCollectionPhysicsProxiesField_Internal;
		}

		const TArray<FJointConstraintPhysicsProxy*>& GetJointConstraintPhysicsProxies_Internal() const
		{
			return JointConstraintPhysicsProxies_Internal;
		}

		/** Events hooked up to the Chaos material manager */
		CHAOS_API void UpdateMaterial(Chaos::FMaterialHandle InHandle, const Chaos::FChaosPhysicsMaterial& InNewData);
		CHAOS_API void CreateMaterial(Chaos::FMaterialHandle InHandle, const Chaos::FChaosPhysicsMaterial& InNewData);
		CHAOS_API void DestroyMaterial(Chaos::FMaterialHandle InHandle);
		CHAOS_API void UpdateMaterialMask(Chaos::FMaterialMaskHandle InHandle, const Chaos::FChaosPhysicsMaterialMask& InNewData);
		CHAOS_API void CreateMaterialMask(Chaos::FMaterialMaskHandle InHandle, const Chaos::FChaosPhysicsMaterialMask& InNewData);
		CHAOS_API void DestroyMaterialMask(Chaos::FMaterialMaskHandle InHandle);

		/** Access to the internal material mirrors */
		const THandleArray<FChaosPhysicsMaterial>& GetQueryMaterials_External() const { return QueryMaterials_External; }
		const THandleArray<FChaosPhysicsMaterialMask>& GetQueryMaterialMasks_External() const { return QueryMaterialMasks_External; }
		const THandleArray<FChaosPhysicsMaterial>& GetSimMaterials() const { return SimMaterials; }
		const THandleArray<FChaosPhysicsMaterialMask>& GetSimMaterialMasks() const { return SimMaterialMasks; }

		/** Copy the simulation material list to the query material list, to be done when the SQ commits an update */
		CHAOS_API void SyncQueryMaterials_External();

		UE_DEPRECATED(5.5, "Deprecated, use GetUseCollisionResimCache() instead")
		bool RewindUsesCollisionResimCache() const { return GetUseCollisionResimCache(); }

		UE_DEPRECATED(5.7, "Deprecated, This functionality is now handled by FRewindData::ProcessDirtyPTParticles")
		void FinalizeRewindData(const TParticleView<FPBDRigidParticles>& DirtyParticles) { };

		FPerSolverFieldSystem& GetPerSolverField() { return *PerSolverField; }
		const FPerSolverFieldSystem& GetPerSolverField() const { return *PerSolverField; }

		CHAOS_API void UpdateExternalAccelerationStructure_External(ISpatialAccelerationCollection<FAccelerationStructureHandle,FReal,3>*& ExternalStructure);
		const ISpatialAccelerationCollection<FAccelerationStructureHandle, FReal, 3>* GetInternalAccelerationStructure_Internal() const
		{
			return MEvolution->GetSpatialAcceleration();
		}

		/** Apply a solver configuration to this solver, set externally by the owner of a solver (see UPhysicsSettings for world solver settings) */
		CHAOS_API void ApplyConfig(const FChaosSolverConfiguration& InConfig);

		virtual void KillSafeAsyncTasks() override
		{
			GetEvolution()->KillSafeAsyncTasks();
		}

		virtual bool AreAnyTasksPending() const override
		{
			if (IsPendingTasksComplete() == false || GetEvolution()->AreAnyTasksPending())
			{
				return true;
			}

			return false;
		}

		CHAOS_API void BeginDestroy();

		/** Update the particles parameters based on field evaluation */
		CHAOS_API void FieldParameterUpdateCallback(
			Chaos::FPBDPositionConstraints& PositionTarget,
			TMap<int32, int32>& TargetedParticles);

		/** Update the particles forces based on field evaluation */
		CHAOS_API void FieldForcesUpdateCallback();

		// Update the counter in Stats and the CSV profiler
		CHAOS_API void ResetStatCounters();
		CHAOS_API void UpdateStatCounters() const;
		CHAOS_API void UpdateExpensiveStatCounters() const;

		// Access particle proxy from physics thread useful for cross thread communication
		CHAOS_API FSingleParticlePhysicsProxy* GetParticleProxy_PT(const FUniqueIdx& Idx);
		CHAOS_API const FSingleParticlePhysicsProxy* GetParticleProxy_PT(const FUniqueIdx& Idx) const;
		CHAOS_API FSingleParticlePhysicsProxy* GetParticleProxy_PT(const FGeometryParticleHandle& Handle);
		CHAOS_API const FSingleParticlePhysicsProxy* GetParticleProxy_PT(const FGeometryParticleHandle& Handle) const;

		// Interop utilities
		CHAOS_API void SetParticleDynamicMisc(FPBDRigidParticleHandle* Rigid, const FParticleDynamicMisc& DynamicMisc);

		// Apply callbacks internally 
		CHAOS_API virtual void ApplyCallbacks_Internal() override;
		CHAOS_API FSolverSerializer& GetSerializer();

		CHAOS_API void OnEvolutionPreIntegrate(FReal Dt);
		CHAOS_API void OnEvolutionPostIntegrate(FReal Dt);

	protected:

#if CHAOS_DEBUG_NAME
		virtual void OnDebugNameChanged() override final;
#endif

	private:

		/**/
		template <typename LambdaType>
		int32 DispatchTasksBufferPhysicsResults(int32 DispatchBatchIndex, int32 NumDirty, LambdaType ReturnHandle);

		CHAOS_API void BufferPhysicsResults();
		CHAOS_API void BufferPhysicsResultsSingleThreaded(
			TArray<FSingleParticlePhysicsProxy*>& ActiveRigid, 
			TArray<FGeometryCollectionPhysicsProxy*>& ActiveGC, 
			TArray<FClusterUnionPhysicsProxy*>& ActiveClusterUnions);
	
		/**/
		CHAOS_API virtual void PrepareAdvanceBy(const FReal DeltaTime) override;
		CHAOS_API virtual void AdvanceSolverBy(const FSubStepInfo& SubStepInfo) override;
		CHAOS_API virtual void PushPhysicsState(const FReal ExternalDt, const int32 NumSteps, const int32 NumExternalSteps) override;
		CHAOS_API virtual void SetExternalTimestampConsumed_Internal(const int32 Timestamp) override;

		CHAOS_API void ClearCollisionEventsForProxy(const IPhysicsProxyBase* Proxy);

		CHAOS_API void ApplyCVars();

		CHAOS_API void UpdateIsDeterministic();

		CHAOS_API void DebugDrawShapes(const bool bShowStatic, const bool bShowKinematic, const bool bShowDynamic) const;

		//
		// Solver Data
		//
		int32 CurrentFrame;
		bool bHasFloor;
		bool bIsFloorAnalytic;
		FReal FloorHeight;
		bool bIsDeterministic;

		// Task dispatcher manage the task graph logic
		TUniquePtr<Private::FTaskDispatcherSolver> TaskDispatcher;

		// Physics material mirrors for the solver. These should generally stay in sync with the global material list from
		// the game thread. This data is read only in the solver as we should never need to update it here. External threads can
		// Enqueue commands to change parameters.
		//
		// There are two copies here to enable SQ to lock only the solvers that it needs to handle the material access during a query
		// instead of having to lock the entire physics state of the runtime.
		// NOTE: Must be before MEvolution because they are passed to it in the constructor
		THandleArray<FChaosPhysicsMaterial> QueryMaterials_External;
		THandleArray<FChaosPhysicsMaterialMask> QueryMaterialMasks_External;
		THandleArray<FChaosPhysicsMaterial> SimMaterials;
		THandleArray<FChaosPhysicsMaterialMask> SimMaterialMasks;

		FParticleUniqueIndicesMultithreaded UniqueIndices;
		FParticlesType Particles;
		TUniquePtr<FPBDRigidsEvolution> MEvolution;
		TUniquePtr<FEventManager> MEventManager;
		TUniquePtr<FSolverEventFilters> MSolverEventFilters;
		TUniquePtr<FDirtyParticlesBuffer> MDirtyParticlesBuffer;

		//
		// Proxies
		//
		TSharedPtr<FCriticalSection> MCurrentLock;
		TSparseArray< FSingleParticlePhysicsProxy* > SingleParticlePhysicsProxies_PT;
		TArray< FGeometryCollectionPhysicsProxy* > GeometryCollectionPhysicsProxies_Internal; // PT
		TArray< FGeometryCollectionPhysicsProxy* > GeometryCollectionPhysicsProxiesField_Internal; // PT
		TArray< FClusterUnionPhysicsProxy* > ClusterUnionPhysicsProxies_Internal; // PT
		TArray< FJointConstraintPhysicsProxy* > JointConstraintPhysicsProxies_Internal; // PT
		TArray< FCharacterGroundConstraintProxy* > CharacterGroundConstraintProxies_Internal;

		TUniquePtr<FPerSolverFieldSystem> PerSolverField;

		FSolverSerializer Serializer;

		FSolverParticleUnregisteredGT EventParticleUnregisteredGT;

		struct FPendingDestroyInfo
		{
			FSingleParticlePhysicsProxy* Proxy;
			int32 DestroyOnStep;
			FGeometryParticleHandle* Handle;
			FUniqueIdx UniqueIdx;

			const bool operator==(const FPendingDestroyInfo Other) const
			{
				return Proxy == Other.Proxy;
			}
		};

		template<typename ProxyType>
		struct FPendingProxyDestroyInfo
		{
			ProxyType* Proxy;
			int32 DestroyOnStep;

			const bool operator==(const FPendingProxyDestroyInfo Other) const
			{
				return Proxy == Other.Proxy;
			}
		};

		TArray<FPendingDestroyInfo> PendingDestroyPhysicsProxy;
		TArray<FPendingProxyDestroyInfo<FGeometryCollectionPhysicsProxy>> PendingDestroyGeometryCollectionPhysicsProxy;
		TArray<FPendingProxyDestroyInfo<FClusterUnionPhysicsProxy>> PendingDestroyClusterUnionProxy;

		CHAOS_API void ProcessSinglePushedData_Internal(FPushPhysicsData& PushData);
		CHAOS_API virtual void ProcessPushedData_Internal(FPushPhysicsData& PushData) override;
		CHAOS_API void DestroyPendingProxies_Internal();

		/** Process callback lifecycle (add/remove) and SimCommands from a skipped frame's PushData.
		 *  Skips dirty proxy processing and input assignment. Used during adaptive resim coalescing.
		 *  Caller must set MTime to the correct value for the skipped frame before calling. */
		void ProcessSkippedFramePushData_Internal(FPushPhysicsData& PushData, int32 SkippedFrame);

		CHAOS_API virtual void ConditionalApplyRewind_Internal() override;

		/** Check if we are resimming or not */
		virtual bool IsResimming() const {return GetEvolution()->IsResimming();}

		/** Sets if we are resimming or not */
		void SetIsResimming(bool bIsResimming);

#if CHAOS_DEBUG_DRAW
	public:
		CHAOS_API virtual void SetDebugDrawScene(const ChaosDD::Private::FChaosDDScenePtr& InCDDScene) override final;

	private:
		// NOTE: A physics frame may consist of multiple sub-steps (ticks) so this debug draw timeline only
		// captures debug draw that happens outside of the tick(s). See FPBDRigidsEvolutionGBF::SetDebugDrawScene
		ChaosDD::Private::FChaosDDScenePtr CDDScene;
		ChaosDD::Private::FChaosDDTimelinePtr CDDFrameTimeline;
#endif
	};

	template<>
	struct TSolverSimMaterialScope<ELockType::Read>
	{
		TSolverSimMaterialScope() = delete;


		explicit TSolverSimMaterialScope(FPhysicsSolverBase* InSolver)
			: Solver(InSolver)
		{
			check(Solver);
			Solver->SimMaterialLock.ReadLock();
		}

		~TSolverSimMaterialScope()
		{
			Solver->SimMaterialLock.ReadUnlock();
		}

	private:
		FPhysicsSolverBase* Solver;
	};

	template<>
	struct TSolverSimMaterialScope<ELockType::Write>
	{
		TSolverSimMaterialScope() = delete;

		explicit TSolverSimMaterialScope(FPhysicsSolverBase* InSolver)
			: Solver(InSolver)
		{
			check(Solver);
			Solver->SimMaterialLock.WriteLock();
		}

		~TSolverSimMaterialScope()
		{
			Solver->SimMaterialLock.WriteUnlock();
		}

	private:
		FPhysicsSolverBase* Solver;
	};

}; // namespace Chaos
