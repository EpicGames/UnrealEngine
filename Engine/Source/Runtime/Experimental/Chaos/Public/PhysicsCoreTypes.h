// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Chaos/Real.h"

#include "PhysicsCoreTypes.generated.h"

UENUM()
enum class EChaosSolverTickMode : uint8
{
	Fixed,
	Variable,
	VariableCapped,
	VariableCappedWithTarget,
};

UENUM()
enum class EChaosThreadingMode : uint8
{
	DedicatedThread UMETA(Hidden),
	TaskGraph,
	SingleThread,
	Num UMETA(Hidden),
	Invalid UMETA(Hidden)
};

UENUM()
enum class EChaosBufferMode : uint8
{
	Double,
	Triple,
	Num UMETA(Hidden),
	Invalid UMETA(Hidden)
};

namespace Chaos
{
	template<class T, int>
	class TPBDRigidParticles;
}
typedef Chaos::TPBDRigidParticles<Chaos::FReal, 3> FParticlesType;
