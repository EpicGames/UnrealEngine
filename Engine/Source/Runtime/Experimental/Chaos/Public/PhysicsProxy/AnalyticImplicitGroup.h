// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Box.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Levelset.h"
#include "Chaos/MassProperties.h"
#include "Chaos/Sphere.h"
#include "Chaos/TaperedCylinder.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"

/**
 * A collection of analytic implicit shapes parented to a single transform, 
 * in a hierarchy of transforms.
 *
 * Currently we use this class to represent bones even if they don't have any 
 * implicit shapes.
 */
class FAnalyticImplicitGroup
{
public:
	FAnalyticImplicitGroup()
		: BoneName(NAME_None)
		, BoneIndex(INDEX_NONE)
		, ParentBoneIndex(INDEX_NONE)
		, RigidBodyId(INDEX_NONE)
		, RigidBodyState(EObjectStateTypeEnum::Chaos_Object_Kinematic)
		, Parent(nullptr)
	{}
	FAnalyticImplicitGroup(const FName &InBoneName, const int32 InBoneIndex)
		: BoneName(InBoneName)
		, BoneIndex(InBoneIndex)
		, ParentBoneIndex(INDEX_NONE)
		, RigidBodyId(INDEX_NONE)
		, RigidBodyState(EObjectStateTypeEnum::Chaos_Object_Kinematic)
		, Parent(nullptr)
	{}
	FAnalyticImplicitGroup(const FAnalyticImplicitGroup &) = delete;
	FAnalyticImplicitGroup(FAnalyticImplicitGroup &&Other)
		: BoneName(MoveTemp(Other.BoneName))
		, BoneIndex(MoveTemp(Other.BoneIndex))
		, ParentBoneIndex(MoveTemp(Other.ParentBoneIndex))

		, RigidBodyId(MoveTemp(Other.RigidBodyId))
		, RigidBodyState(MoveTemp(Other.RigidBodyState))

		, Spheres(MoveTemp(Other.Spheres))
		, Boxes(MoveTemp(Other.Boxes))
		, Capsules(MoveTemp(Other.Capsules))
		, TaperedCylinders(MoveTemp(Other.TaperedCylinders))
		, ConvexHulls(MoveTemp(Other.ConvexHulls))
		, LevelSets(MoveTemp(Other.LevelSets))

		, Transforms(MoveTemp(Other.Transforms))
		, Parent(MoveTemp(Other.Parent))
		, Children(MoveTemp(Other.Children))
	{}
	~FAnalyticImplicitGroup()
	{
		for (Chaos::FSphere* Sphere : Spheres) if (Sphere) delete Sphere;
		for (Chaos::TBox<Chaos::FReal, 3>* Box : Boxes) if (Box) delete Box;
		for (Chaos::FCapsule* Capsule : Capsules) if (Capsule) delete Capsule;
		for (Chaos::FTaperedCylinder* TaperedCylinder : TaperedCylinders) if (TaperedCylinder) delete TaperedCylinder;
		for (Chaos::FConvex* ConvexHull : ConvexHulls) if (ConvexHull) delete ConvexHull;
		for (Chaos::FLevelSet* LevelSet : LevelSets) if (LevelSet) delete LevelSet;
	}

	void Init(const int32 NumStructures, const bool DoCollGeom=true)
	{
		Transforms.Reserve(NumStructures);
		if (DoCollGeom)
		{
			CollisionPoints.SetNum(NumStructures);
			CollisionTriangles.SetNum(NumStructures);
		}
	}

	/** The number of analytic shapes in this group. */
	int32 NumStructures() const { return Transforms.Num(); }

	bool IsValid() { return BoneIndex != INDEX_NONE; }

	const FName& GetBoneName() const { return BoneName; }
	const int32 GetBoneIndex() const { return BoneIndex; }

	void SetParentBoneIndex(const int32 InParentBoneIndex) { ParentBoneIndex = InParentBoneIndex; }
	const int32 GetParentBoneIndex() const { return ParentBoneIndex; }

	void SetRigidBodyId(const int32 InRigidBodyId) { RigidBodyId = InRigidBodyId; }
	int32 GetRigidBodyId() const { return RigidBodyId; }

	void SetRigidBodyState(const EObjectStateTypeEnum State) { RigidBodyState = State; }
	EObjectStateTypeEnum GetRigidBodyState() const { return RigidBodyState; }

	int32 Add(const FTransform &InitialXf, Chaos::FSphere *Sphere) { Spheres.Add(Sphere); return Transforms.Insert(InitialXf, Spheres.Num()-1); }
	int32 Add(const FTransform &InitialXf, Chaos::TBox<Chaos::FReal, 3> *Box) { Boxes.Add(Box); return Transforms.Insert(InitialXf, Spheres.Num()+Boxes.Num()-1); }
	int32 Add(const FTransform &InitialXf, Chaos::FCapsule *Capsule) { Capsules.Add(Capsule); return Transforms.Insert(InitialXf, Spheres.Num()+Boxes.Num()+Capsules.Num()-1); }
	int32 Add(const FTransform &InitialXf, Chaos::FTaperedCylinder *TaperedCylinder) { TaperedCylinders.Add(TaperedCylinder); return Transforms.Insert(InitialXf, Spheres.Num()+Boxes.Num()+Capsules.Num()+TaperedCylinders.Num()-1); }
	int32 Add(const FTransform &InitialXf, Chaos::FConvex *ConvexHull) { ConvexHulls.Add(ConvexHull); return Transforms.Insert(InitialXf, Spheres.Num()+Boxes.Num()+Capsules.Num()+TaperedCylinders.Num()+ConvexHulls.Num()-1); }
	int32 Add(const FTransform &InitialXf, Chaos::FLevelSet *LevelSet) { LevelSets.Add(LevelSet); return Transforms.Add(InitialXf); }

	void SetCollisionTopology(
		const int32 Index,
		TArray<Chaos::FVec3>&& Points,
		TArray<Chaos::TVec3<int32>>&& Triangles)
	{ CollisionPoints[Index] = MoveTemp(Points); CollisionTriangles[Index] = MoveTemp(Triangles); }

	const TArray<FTransform>& GetInitialStructureTransforms() const { return Transforms; }
	void ResetTransforms() { Transforms.Init(FTransform::Identity, Transforms.Num()); }

	Chaos::FMassProperties BuildMassProperties(
		const Chaos::FReal Density,
		Chaos::FReal& TotalMass)
	{
		// Make sure this function is being called before we've transfered ownership 
		// of our implicit shapes to the simulator.
		checkSlow(!Spheres.Contains(nullptr) && !Boxes.Contains(nullptr) && !Capsules.Contains(nullptr) && 
			!TaperedCylinders.Contains(nullptr) && !ConvexHulls.Contains(nullptr) && !LevelSets.Contains(nullptr));

		const int32 Num = NumStructures();
		TArray<Chaos::FMassProperties> MPArray;
		TArray<Chaos::FAABB3> BBoxes;
		MPArray.SetNum(Num);
		BBoxes.SetNum(Num);

		int32 TransformIndex = 0;
		for (Chaos::FSphere* Sphere : Spheres)
		{
			const FTransform& Xf = Transforms[TransformIndex];
			BBoxes[TransformIndex] = Sphere->BoundingBox().TransformedAABB(Xf);
			Chaos::FMassProperties &MP = MPArray[TransformIndex++];
			MP.Volume = Sphere->GetVolume();
			MP.CenterOfMass = Xf.TransformPositionNoScale(Sphere->GetCenterOfMass());
			MP.RotationOfMass = Xf.TransformRotation(Sphere->GetRotationOfMass());
		}
		for (Chaos::TBox<Chaos::FReal, 3>* Box : Boxes)
		{
			const FTransform& Xf = Transforms[TransformIndex];
			BBoxes[TransformIndex] = Box->BoundingBox().TransformedAABB(Xf);
			Chaos::FMassProperties&MP = MPArray[TransformIndex++];
			MP.Volume = Box->GetVolume();
			MP.CenterOfMass = Xf.TransformPositionNoScale(Box->GetCenterOfMass());
			MP.RotationOfMass = Xf.TransformRotation(Box->GetRotationOfMass());
		}
		for (Chaos::FCapsule* Capsule : Capsules)
		{
			const FTransform& Xf = Transforms[TransformIndex];
			BBoxes[TransformIndex] = Capsule->BoundingBox().TransformedAABB(Xf);
			Chaos::FMassProperties &MP = MPArray[TransformIndex++];
			MP.Volume = Capsule->GetVolume();
			MP.CenterOfMass = Xf.TransformPositionNoScale(FVector3d(Capsule->GetCenterOfMassf()));
			MP.RotationOfMass = Xf.TransformRotation(Capsule->GetRotationOfMass());
		}
		for (Chaos::FTaperedCylinder* TaperedCylinder : TaperedCylinders)
		{
			const FTransform& Xf = Transforms[TransformIndex];
			BBoxes[TransformIndex] = TaperedCylinder->BoundingBox().TransformedAABB(Xf);
			Chaos::FMassProperties &MP = MPArray[TransformIndex++];
			MP.Volume = TaperedCylinder->GetVolume();
			MP.CenterOfMass = Xf.TransformPositionNoScale(TaperedCylinder->GetCenterOfMass());
			MP.RotationOfMass = Xf.TransformRotation(TaperedCylinder->GetRotationOfMass());
		}
		for (Chaos::FConvex* Convex : ConvexHulls)
		{
			const FTransform& Xf = Transforms[TransformIndex];
			BBoxes[TransformIndex] = Convex->BoundingBox().TransformedAABB(Xf);
			Chaos::FMassProperties &MP = MPArray[TransformIndex++];
			MP.Volume = Convex->BoundingBox().GetVolume();
			MP.CenterOfMass = Xf.TransformPositionNoScale(Convex->BoundingBox().Center());
			MP.RotationOfMass = Xf.TransformRotation(Convex->BoundingBox().GetRotationOfMass());
		}
		for (Chaos::FLevelSet* LevelSet : LevelSets)
		{
			const FTransform& Xf = Transforms[TransformIndex];
			BBoxes[TransformIndex] = LevelSet->BoundingBox().TransformedAABB(Xf);
			Chaos::FMassProperties &MP = MPArray[TransformIndex++];
			MP.Volume = LevelSet->BoundingBox().GetVolume();
			MP.CenterOfMass = Xf.TransformPositionNoScale(LevelSet->BoundingBox().Center());
			MP.RotationOfMass = Xf.TransformRotation(LevelSet->BoundingBox().GetRotationOfMass());
		}

		// Find overlap and adjust volumes accordingly.  We do this by determining 
		// how much their AABB's overlap for sake of speed and simplicity.  Ideally
		// we'd do something more accurate...
		for (int32 i=0; i < Num-1; i++)
		{
			const Chaos::FAABB3& BoxI = BBoxes[i];
			for (int32 j = i+1; j < Num; j++)
			{
				const Chaos::FAABB3& BoxJ = BBoxes[j];
				if (BoxI.Intersects(BoxJ))
				{
					Chaos::FAABB3 BoxIJ = BoxI.GetIntersection(BoxJ);
					const Chaos::FReal VolIJ = BoxIJ.GetVolume();
					if (VolIJ > UE_KINDA_SMALL_NUMBER)
					{
						const Chaos::FReal VolI = BoxI.GetVolume();
						const Chaos::FReal VolJ = BoxJ.GetVolume();
						const Chaos::FReal PctOverlapI = VolI > UE_KINDA_SMALL_NUMBER ? VolIJ / VolI : 0.f;
						const Chaos::FReal PctOverlapJ = VolJ > UE_KINDA_SMALL_NUMBER ? VolIJ / VolJ : 0.f;
						// Split the overlapping volume between i and j
						MPArray[i].Volume *= static_cast<Chaos::FReal>(1.0 - PctOverlapI / 2.0);
						MPArray[j].Volume *= static_cast<Chaos::FReal>(1.0 - PctOverlapJ / 2.0);
					}
				}
			}
		}

		TotalMass = 0.f;
		TransformIndex = 0;
		for (Chaos::FSphere* Sphere : Spheres)
		{
			Chaos::FMassProperties &MP = MPArray[TransformIndex++];
			Chaos::FReal Mass = Density * MP.Volume;
			TotalMass += Mass;
			MP.InertiaTensor = Sphere->GetInertiaTensor(Mass);
		}
		for (Chaos::TBox<Chaos::FReal, 3>* Box : Boxes)
		{
			Chaos::FMassProperties &MP = MPArray[TransformIndex++];
			Chaos::FReal Mass = Density * MP.Volume;
			TotalMass += Mass;
			MP.InertiaTensor = Box->GetInertiaTensor(Mass);
		}
		for (Chaos::FCapsule* Capsule : Capsules)
		{
			Chaos::FMassProperties &MP = MPArray[TransformIndex++];
			Chaos::FReal Mass = Density * MP.Volume;
			TotalMass += Mass;
			MP.InertiaTensor = Capsule->GetInertiaTensor(Mass);
		}
		for (Chaos::FTaperedCylinder* TaperedCylinder : TaperedCylinders)
		{
			Chaos::FMassProperties &MP = MPArray[TransformIndex++];
			Chaos::FReal Mass = Density * MP.Volume;
			TotalMass += Mass;
			MP.InertiaTensor = TaperedCylinder->GetInertiaTensor(Mass);
		}
		for (Chaos::FConvex* Convex : ConvexHulls)
		{
			Chaos::FMassProperties &MP = MPArray[TransformIndex++];
			Chaos::FReal Mass = Density * MP.Volume;
			TotalMass += Mass;
			MP.InertiaTensor = Convex->BoundingBox().GetInertiaTensor(Mass);
		}
		for (Chaos::FLevelSet* LevelSet : LevelSets)
		{
			Chaos::FMassProperties &MP = MPArray[TransformIndex++];
			Chaos::FReal Mass = Density * MP.Volume;
			TotalMass += Mass;
			MP.InertiaTensor = LevelSet->BoundingBox().GetInertiaTensor(Mass);
		}

		return Chaos::Combine(MPArray);
	}

	TArray<Chaos::FVec3>* BuildSamplePoints(
		const Chaos::FReal ParticlesPerUnitArea,
		const int32 MinParticles,
		const int32 MaxParticles)
	{
		// Make sure this function is being called before we've transfered ownership 
		// of our implicit shapes to the simulator.
		checkSlow(!Spheres.Contains(nullptr) && !Boxes.Contains(nullptr) && !Capsules.Contains(nullptr) && 
			!TaperedCylinders.Contains(nullptr) && !ConvexHulls.Contains(nullptr) && !LevelSets.Contains(nullptr));

		ContiguousCollisionPoints.Reset();
		const int32 Num = NumStructures();
		if (Num == 0)
		{
			return &ContiguousCollisionPoints;
		}

		int32 TransformIndex = 0;
		for (Chaos::FSphere* Sphere : Spheres)
		{
			TArray<Chaos::FVec3>& Points = CollisionPoints[TransformIndex];
			if (!Points.Num())
				Points = Sphere->ComputeSamplePoints(ParticlesPerUnitArea, MinParticles, MaxParticles);
			CullDeepPoints(Points, TransformIndex);
			TransformIndex++;
		}
		for (Chaos::TBox<Chaos::FReal, 3>* Box : Boxes)
		{
			TArray<Chaos::FVec3>& Points = CollisionPoints[TransformIndex];
			if (!Points.Num())
				Points = Box->ComputeSamplePoints();
			CullDeepPoints(Points, TransformIndex);
			TransformIndex++;
		}
		for (Chaos::FCapsule* Capsule : Capsules)
		{
			TArray<Chaos::FVec3>& Points = CollisionPoints[TransformIndex];
			if (!Points.Num())
				Points = Capsule->ComputeSamplePoints(ParticlesPerUnitArea, MinParticles, MaxParticles);
			CullDeepPoints(Points, TransformIndex);
			TransformIndex++;
		}
		for (Chaos::FTaperedCylinder* TaperedCylinder : TaperedCylinders)
		{
			TArray<Chaos::FVec3>& Points = CollisionPoints[TransformIndex];
			if (!Points.Num())
				Points = TaperedCylinder->ComputeSamplePoints(ParticlesPerUnitArea, false, MinParticles, MaxParticles);
			CullDeepPoints(Points, TransformIndex);
			TransformIndex++;
		}
		for (Chaos::FConvex* Convex : ConvexHulls)
		{
			TArray<Chaos::FVec3>& Points = CollisionPoints[TransformIndex];
			if (!Points.Num())
			{
				const Chaos::FAABB3& BBox = Convex->BoundingBox();
				Chaos::FSphere Sphere(BBox.Center(), BBox.Extents().Size() / 2);
				Points = Sphere.ComputeSamplePoints(ParticlesPerUnitArea, MinParticles, MaxParticles);
				Chaos::FVec3 Normal;
				for (Chaos::FVec3 &Pt : Points)
				{
					const Chaos::FReal Phi = Convex->PhiWithNormal(Pt, Normal);
					Pt += Normal * -Phi;
					//check(FMath::Abs(Convex->SignedDistance(Pt)) <= KINDA_SMALL_NUMBER);
				}
			}
			CullDeepPoints(Points, TransformIndex);
			TransformIndex++;
		}
		for (Chaos::FLevelSet* LevelSet : LevelSets)
		{
			TArray<Chaos::FVec3>& Points = CollisionPoints[TransformIndex];
			if (!Points.Num())
			{
				const Chaos::FAABB3& BBox = LevelSet->BoundingBox();
				Chaos::FSphere Sphere(BBox.Center(), BBox.Extents().Size() / 2);
				Points = Sphere.ComputeSamplePoints(ParticlesPerUnitArea, MinParticles, MaxParticles);
				Chaos::FVec3 Normal;
				for (Chaos::FVec3 &Pt : Points)
				{
					const Chaos::FReal Phi = LevelSet->PhiWithNormal(Pt, Normal);
					Pt += Normal * -Phi;
					//check(FMath::Abs(LevelSet->SignedDistance(Pt)) <= KINDA_SMALL_NUMBER);
				}
			}
			CullDeepPoints(Points, TransformIndex);
			TransformIndex++;
		}

		if (Num > 1)
		{
			int32 NumPoints = 0;
			for (const auto& PtArray : CollisionPoints)
				NumPoints += PtArray.Num();
			ContiguousCollisionPoints.Reserve(NumPoints);
		}
		for (TransformIndex = 0; TransformIndex < Num; TransformIndex++)
		{
			TArray<Chaos::FVec3> &PtArray = CollisionPoints[TransformIndex];
			const FTransform& Xf = Transforms[TransformIndex];
			if(!Xf.Equals(FTransform::Identity))
			{
				for (Chaos::FVec3& Pt : PtArray)
					Pt = Xf.TransformPosition(Pt);
			}
			if (Num == 1)
			{
				// Free memory we're not going to use
				ContiguousCollisionPoints.Empty();
				return &PtArray;
			}
			ContiguousCollisionPoints.Append(PtArray);	
		}

		// Free memory we're not going to use
		CollisionPoints.Empty();
		return &ContiguousCollisionPoints;
	}

	TArray<Chaos::TVec3<int32>> BuildSampleTopology() const
	{
		int32 NumTris = 0;
		for (const TArray<Chaos::TVec3<int32>>& Tris : CollisionTriangles)
		{
			NumTris += Tris.Num();
		}

		TArray<Chaos::TVec3<int32>> AllTriangles;
		AllTriangles.Reserve(NumTris);
		int32 Offset = 0;
		for (int32 Index=0; Index < CollisionTriangles.Num(); Index++)
		{
			for (const Chaos::TVec3<int32>& Tri : CollisionTriangles[Index])
			{
				AllTriangles.Add(Tri + Offset);
			}
			Offset += CollisionPoints[Index].Num();
		}

		return AllTriangles;
	}

	/**
	 * Build the implicit object representation of this object.
	 *
	 * Transfers ownership of sub structures to the returned implicit object.
	 */
	Chaos::FImplicitObject* BuildSimImplicitObject()
	{
		// TODO: We make copies of implicit objects owned by this class, so we
		// give the solver memory it can own.  It'd be nice if we could transfer
		// or share the memory this class owns to/with the solver.

		const int32 Num = NumStructures();
		if (Num == 0)
		{
			return nullptr;
		}
		else if (Num == 1)
		{
			if (Transforms[0].Equals(FTransform::Identity))
			{
				return TransferImplicitObj(0);
			}
			else
			{
				// Make a copy and transfer ownership to the transformed implicit.
				Chaos::FImplicitObjectPtr ObjPtr(TransferImplicitObj(0));
				return new Chaos::TImplicitObjectTransformed<Chaos::FReal, 3, true>(
					MoveTemp(ObjPtr),
					Chaos::FRigidTransform3(Transforms[0]));
			}
		}
		else
		{
			// Make copies of the implicits owned by transformed immplicits, and 
			// transfer ownership of the transformed implicits to the implicit union.
			TArray<Chaos::FImplicitObjectPtr> ImplicitObjects;
			ImplicitObjects.Reserve(Num);
			for (int i = 0; i < Num; i++)
			{
				Chaos::FImplicitObjectPtr ObjPtr(TransferImplicitObj(i));

				const FTransform &Xf = Transforms[i];
				if (Xf.Equals(FTransform::Identity))
				{
					// If we ever support animated substructures, we'll need all
					// of them wrapped by a transform.
					ImplicitObjects.Add(MoveTemp(ObjPtr));
				}
				else
				{
					ImplicitObjects.Add(
						Chaos::FImplicitObjectPtr(
							new Chaos::TImplicitObjectTransformed<Chaos::FReal, 3, true>(
								MoveTemp(ObjPtr),
								Chaos::FRigidTransform3(Xf))));
				}
			}
			return new Chaos::FImplicitObjectUnion(MoveTemp(ImplicitObjects));
		}
	}

protected:
	friend class FBoneHierarchy;

	void SetParent(FAnalyticImplicitGroup *InParent)
	{ Parent = InParent; }
	const FAnalyticImplicitGroup* GetParent() const
	{ return Parent; }

	void AddChild(FAnalyticImplicitGroup *Child)
	{ Children.Add(Child); }
	const TArray<FAnalyticImplicitGroup*>& GetChildren() const
	{ return Children; }

	void ClearHierarchy()
	{ Parent = nullptr; Children.Reset(); }

	template<class TImplicitShape>
	void CullDeepPoints(TArray<Chaos::FVec3>& Points, const TImplicitShape& Shape, const FTransform& Xf)
	{
		const Chaos::FAABB3& BBox = Shape.BoundingBox();
		const Chaos::FReal Tolerance = -BBox.Extents().Max() / (Chaos::FReal)100.; // -1/100th the largest dimension
		if (Xf.Equals(FTransform::Identity))
		{
			for (int32 i = Points.Num(); i--; )
			{
				const Chaos::FVec3& LocalPoint = Points[i];
				const Chaos::FReal Phi = Shape.SignedDistance(LocalPoint);
				if (Phi < Tolerance)
				{
					Points.RemoveAt(i);
				}
			}
		}
		else
		{
			const FTransform InvXf = Xf.Inverse();
			for (int32 i = Points.Num(); i--; )
			{
				const Chaos::FVec3 LocalPoint = InvXf.TransformPosition(Points[i]);
				const Chaos::FReal Phi = Shape.SignedDistance(LocalPoint);
				if (Phi < Tolerance)
				{
					Points.RemoveAt(i);
				}
			}
		}
	}

	void CullDeepPoints(TArray<Chaos::FVec3>& Points, const int32 SkipIndex)
	{
		int32 TransformIndex = 0;
		for (Chaos::FSphere* Sphere : Spheres)
		{
			if (TransformIndex != SkipIndex)
			{
				CullDeepPoints(Points, *Sphere, Transforms[TransformIndex]);
			}
			TransformIndex++;
		}
		for (Chaos::TBox<Chaos::FReal, 3>* Box : Boxes)
		{
			if (TransformIndex != SkipIndex)
			{
				CullDeepPoints(Points, *Box, Transforms[TransformIndex]);
			}
			TransformIndex++;
		}
		for (Chaos::FCapsule* Capsule : Capsules)
		{
			if (TransformIndex != SkipIndex)
			{
				CullDeepPoints(Points, *Capsule, Transforms[TransformIndex]);
			}
			TransformIndex++;
		}
		for (Chaos::FTaperedCylinder* TaperedCylinder : TaperedCylinders)
		{
			if (TransformIndex != SkipIndex)
			{
				CullDeepPoints(Points, *TaperedCylinder, Transforms[TransformIndex]);
			}
			TransformIndex++;
		}
		for (Chaos::FConvex* Convex : ConvexHulls)
		{
			if (TransformIndex != SkipIndex)
			{
				CullDeepPoints(Points, *Convex, Transforms[TransformIndex]);
			}
			TransformIndex++;
		}
		for (Chaos::FLevelSet* LevelSet : LevelSets)
		{
			if (TransformIndex != SkipIndex)
			{
				CullDeepPoints(Points, *LevelSet, Transforms[TransformIndex]);
			}
			TransformIndex++;
		}
	}

	Chaos::FImplicitObject* TransferImplicitObj(int32 Idx)
	{
		Chaos::FImplicitObject* Obj = nullptr;
		if (Idx < Spheres.Num())
		{
			Obj = Spheres[Idx]; 
			Spheres[Idx] = nullptr;
			return Obj;
		}
		Idx -= Spheres.Num();

		if (Idx < Boxes.Num())
		{
			Obj = Boxes[Idx];
			Boxes[Idx] = nullptr;
			return Obj;
		}
		Idx -= Boxes.Num();

		if (Idx < Capsules.Num())
		{
			Obj = Capsules[Idx];
			Capsules[Idx] = nullptr;
			return Obj;
		}
		Idx -= Capsules.Num();

		if (Idx < TaperedCylinders.Num())
		{
			Obj = TaperedCylinders[Idx];
			TaperedCylinders[Idx] = nullptr;
			return Obj;
		}
		Idx -= TaperedCylinders.Num();

		if (Idx < ConvexHulls.Num())
		{
			Obj = ConvexHulls[Idx];
			ConvexHulls[Idx] = nullptr;
			return Obj;
		}
		Idx -= ConvexHulls.Num();

		if (Idx < LevelSets.Num())
		{
			Obj = LevelSets[Idx];
			LevelSets[Idx] = nullptr;
			return Obj;
		}

		check(false); // Index out of range!
		return Obj;
	}

protected:
	//friend class USkeletalMeshSimulationComponent;
	friend struct FPhysicsAssetSimulationUtil;

	FName BoneName;
	int32 BoneIndex;
	int32 ParentBoneIndex;

	int32 RigidBodyId;
	EObjectStateTypeEnum RigidBodyState;

	// FkSphereElem and FKTaperedCapsuleElem ends
	TArray<Chaos::FSphere*> Spheres;
	// FKBoxElem
	TArray<Chaos::TBox<Chaos::FReal, 3>*> Boxes;
	// FKSphylElem - Z axis is capsule axis
	TArray<Chaos::FCapsule*> Capsules;
	// FKTaperedCapsuleElem - Z axis is the capsule axis
	TArray<Chaos::FTaperedCylinder*> TaperedCylinders;
	// FKConvexElem
	TArray<Chaos::FConvex*> ConvexHulls;
	// Chaos::TConvex replacement
	TArray<Chaos::FLevelSet*> LevelSets;

	TArray<Chaos::FVec3> ContiguousCollisionPoints;
	TArray<TArray<Chaos::FVec3>> CollisionPoints;
	TArray<TArray<Chaos::TVec3<int32>>> CollisionTriangles;

	TArray<FTransform> Transforms;
	FTransform RefBoneXf;

	FAnalyticImplicitGroup* Parent;
	TArray<FAnalyticImplicitGroup*> Children;
};
