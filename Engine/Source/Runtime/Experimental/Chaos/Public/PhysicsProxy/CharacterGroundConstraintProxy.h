// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Framework/PhysicsProxyBase.h"

namespace Chaos
{
	class FCharacterGroundConstraint;
	class FCharacterGroundConstraintHandle;
	class FDirtyChaosProperties;
	struct FDirtyCharacterGroundConstraintData;
	class FDirtyPropertiesManager;
	class FPBDRigidsSolver;

	/// Proxy class to manage access to and syncing of character ground constraint data
	/// between the game thread and physics thread
	class FCharacterGroundConstraintProxy : public IPhysicsProxyBase
	{
	public:
		using Base = IPhysicsProxyBase;

		CHAOS_API FCharacterGroundConstraintProxy(FCharacterGroundConstraint* InConstraintGT, FCharacterGroundConstraintHandle* InConstraintPT = nullptr, UObject* InOwner = nullptr);
	
		//////////////////////////////////////////////////////////////////////////
		/// Member Access

		FCharacterGroundConstraint* GetGameThreadAPI() { return Constraint_GT; }
		const FCharacterGroundConstraint* GetGameThreadAPI() const { return Constraint_GT; }

		FCharacterGroundConstraintHandle* GetPhysicsThreadAPI() { return Constraint_PT; }
		const FCharacterGroundConstraintHandle* GetPhysicsThreadAPI() const { return Constraint_PT; }

		//////////////////////////////////////////////////////////////////////////
		/// State Management

		/// Gets the data from the game thread via the RemoteData and creates a constraint on the physics thread
		CHAOS_API void InitializeOnPhysicsThread(FPBDRigidsSolver* InSolver, FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData);

		/// Pushes any changed data from the game thread constraint to the remote data
		CHAOS_API void PushStateOnGameThread(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData);

		/// Reads changed data from the game thread constraint into the physics thread constraint via the remote data
		CHAOS_API void PushStateOnPhysicsThread(FPBDRigidsSolver* InSolver, const FDirtyPropertiesManager& Manager, int32 DataIdx, const FDirtyChaosProperties& RemoteData);

		/// Deletes the game thread constraint
		CHAOS_API void DestroyOnGameThread();

		/// Removes references to the physics thread constraint and deletes it
		CHAOS_API void DestroyOnPhysicsThread(FPBDRigidsSolver* InSolver);

		//////////////////////////////////////////////////////////////////////////
		/// IPhysicsProxyBase Implementation

		virtual void* GetHandleUnsafe() const override { return Constraint_PT; }

		//////////////////////////////////////////////////////////////////////////
		/// Manage Output Data

		/// Write the output constraint data from the physics thread to the buffer
		CHAOS_API void BufferPhysicsResults(FDirtyCharacterGroundConstraintData& Buffer);

		/// Write the output constraint data from the buffer to the game thread 
		CHAOS_API bool PullFromPhysicsState(const FDirtyCharacterGroundConstraintData& Buffer, const int32 SolverSyncTimestamp);

	private:
		FCharacterGroundConstraint* Constraint_GT;
		FCharacterGroundConstraintHandle* Constraint_PT;
		bool bInitialized = false;
	};

} // namespace Chaos
