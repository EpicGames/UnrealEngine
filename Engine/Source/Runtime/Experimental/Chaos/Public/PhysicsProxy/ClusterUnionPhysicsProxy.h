// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ClusterCreationParameters.h"
#include "Chaos/ClusterUnionManager.h"
#include "Chaos/Framework/PhysicsProxy.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PhysicsObject.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Framework/Threading.h"
#include "PBDRigidsSolver.h"
#include "Templates/UniquePtr.h"

namespace Chaos
{
	struct FDirtyClusterUnionData;

	struct FClusterUnionChildData
	{
		FUniqueIdx ParticleIdx;
		FTransform ChildToParent;
		IPhysicsProxyBase* Proxy = nullptr;
		void* CachedOwner = nullptr;
		int32 BoneId = INDEX_NONE;
	};

	struct FClusterUnionInitData
	{
		void* UserData;
		FTransform InitialTransform;
		uint32 ActorId = INDEX_NONE;
		uint32 ComponentId = INDEX_NONE;
		bool bCheckConnectivity = true;
		bool bUnbreakable = false;
		bool bGenerateConnectivityEdges = true;
		int32 GravityGroupOverride = INDEX_NONE;
		bool bCCDEnabled = false;
		bool bMACDEnabled = false;
#if CHAOS_DEBUG_NAME
		TSharedPtr<FString, ESPMode::ThreadSafe> DebugName;
#endif
	};

	/**
	 * Extra data that needs to be synced between the PT and GT for cluster unions.
	 */
	struct FClusterUnionSyncedData
	{
		// Whether the cluster is anchored or not.
		bool bIsAnchored = true;

		// Whether we assigned new geometry from the PT in PullFromPhysicsState
		bool bDidSyncGeometry = false;

		// Data on every child particle in the cluster union.
		TArray<FClusterUnionChildData> ChildParticles;
	};
	

	class FClusterUnionPhysicsProxy : public TPhysicsProxy<FClusterUnionPhysicsProxy, void, FClusterUnionProxyTimestamp>
	{
		using Base = TPhysicsProxy<FClusterUnionPhysicsProxy, void, FClusterUnionProxyTimestamp>;
	public:
		// The mismatch here is fine since we really only need to sync a few properties between the PT and the GT.
		using FExternalParticle = TPBDRigidParticle<FReal, 3>;
		using FInternalParticle = TPBDRigidClusteredParticleHandle<Chaos::FReal, 3>;

		FClusterUnionPhysicsProxy() = delete;
		CHAOS_API FClusterUnionPhysicsProxy(UObject* InOwner, const FClusterCreationParameters& InParameters, const FClusterUnionInitData& InInitData);

		// Add physics objects to the cluster union. Should only be called from the game thread.
		CHAOS_API void AddPhysicsObjects_External(const TArray<FPhysicsObjectHandle>& Objects);
		const FClusterUnionSyncedData& GetSyncedData_External() const { return SyncedData_External; }

		// Remove physics objects from the cluster union.
		CHAOS_API void RemovePhysicsObjects_External(const TSet<FPhysicsObjectHandle>& Objects);

		// Set/Remove any anchors on the cluster particle.
		CHAOS_API void SetIsAnchored_External(bool bIsAnchored);

		// Set Object State.
		CHAOS_API EObjectStateType GetObjectState_External() const;
		CHAOS_API void SetObjectState_External(EObjectStateType State);
		
		CHAOS_API ESleepType GetSleepType_External() const;
		CHAOS_API void SetSleepType_External(const ESleepType SleepType);

		CHAOS_API bool GetDisabled_External() const;
		CHAOS_API void SetDisabled_External(const bool Disabled);

		CHAOS_API bool GetGravityEnabled_External() const;
		CHAOS_API void SetGravityEnabled_External(const bool Enabled);

		// Explicitly wake the physics object. The wakes the physics object if it is sleeping and resets any sleep state (whether awake or sleeping)
		CHAOS_API void Wake_External();

		// Set the cluster mass.
		// NOTE: When a cluster breaks its mass will be recalculated from the remaining children on the PT, effectively undoing this work.
		CHAOS_API void SetMass_External(Chaos::FReal Mass);

		// Set GT geometry - this is only for smoothing over any changes until the PT syncs back to the GT.
		CHAOS_API void SetGeometry_External(const Chaos::FImplicitObjectPtr& Geometry, const TArray<FPBDRigidParticle*>& ShapeParticles);

		// Merge GT geometry into the existing union
		CHAOS_API void MergeGeometry_External(TArray<Chaos::FImplicitObjectPtr>&& ImplicitGeometries, const TArray<FPBDRigidParticle*>& ShapeParticles);

		// Remove GT shapes from the existing unions
		CHAOS_API void RemoveShapes_External(const TArray<FPBDRigidParticle*>& ShapeParticles);
		
		UE_DEPRECATED(5.4, "Please use SetGeometry_External instead")
		void SetSharedGeometry_External(const TSharedPtr<Chaos::FImplicitObject, ESPMode::ThreadSafe>& Geometry, const TArray<FPBDRigidParticle*>& ShapeParticles)
		{
			check(false);
		}

		// Cluster union proxy initialization happens in two first on the game thread (external) then on the
		// physics thread (internal). Cluster unions are a primarily physics concept so the things exposed to
		// an external context is just there to be able to safely query the state on the physics thread and to
		// be able to add/remove things to the proxy itself (rather than to move it, for example).
		CHAOS_API void Initialize_External();

		bool IsInitializedOnPhysicsThread() const { return bIsInitializedOnPhysicsThread; }
		CHAOS_API void Initialize_Internal(FPBDRigidsSolver* RigidsSolver, FPBDRigidsSolver::FParticlesType& Particles);

		FExternalParticle* GetParticle_External() const { return Particle_External.Get(); }
		FInternalParticle* GetParticle_Internal() const { return Particle_Internal; }
		virtual void* GetHandleUnsafe() const override { return Particle_Internal; }
		FPhysicsObjectHandle GetPhysicsObjectHandle() const { return PhysicsObject.Get(); }

		bool HasChildren_External() const { return !SyncedData_External.ChildParticles.IsEmpty(); }
		CHAOS_API bool HasChildren_Internal() const;

		bool IsAnchored_External() const { return SyncedData_External.bIsAnchored; }

		CHAOS_API void SetXR_External(const FVector& X, const FQuat& R);
		CHAOS_API void SetLinearVelocity_External(const FVector& V);
		CHAOS_API void SetAngularVelocity_External(const FVector& W);
		CHAOS_API void SetChildToParent_External(FPhysicsObjectHandle Child, const FTransform& RelativeTransform, bool bLock);
		CHAOS_API void BulkSetChildToParent_External(const TArray<FPhysicsObjectHandle>& Objects, const TArray<FTransform>& Transforms, bool bLock);
		CHAOS_API void ChangeMainParticleStatus_External(const TArray<FPhysicsObjectHandle>& Objects, bool bIsMain);

		//
		// These functions take care of marshaling data back and forth between the game thread
		// and the physics thread.
		//
		CHAOS_API void PushToPhysicsState(const FDirtyPropertiesManager& Manager, int32 DataIdx, const FDirtyProxy& Dirty);
		CHAOS_API bool PullFromPhysicsState(const FDirtyClusterUnionData& PullData, int32 SolverSyncTimestamp, const FDirtyClusterUnionData* NextPullData = nullptr, const FRealSingle* Alpha = nullptr, const FDirtyRigidParticleReplicationErrorData* Error = nullptr, const Chaos::FReal AsyncFixedTimeStep = 0);

		CHAOS_API void BufferPhysicsResults_Internal(FDirtyClusterUnionData& BufferData);
		CHAOS_API void BufferPhysicsResults_External(FDirtyClusterUnionData& BufferData);

		CHAOS_API void SyncRemoteData(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData) const;
		CHAOS_API void ClearAccumulatedData();

		FProxyInterpolationBase* GetInterpolationData() { return InterpolationData.Get(); }
		const FProxyInterpolationBase* GetInterpolationData() const { return InterpolationData.Get(); }

		FClusterUnionIndex GetClusterUnionIndex() const { return ClusterUnionIndex; }

		void ForceSetGeometryChildParticles_External(TArray<FExternalParticle*>&& InParticles) { GeometryChildParticles_External = InParticles; }

		CHAOS_API void SetEnableStrainOnCollision_External(bool bEnable);
		bool GetEnableStrainOnCollision_Internal() const { return bEnableStrainOnCollision_Internal; }

	private:
		bool bIsInitializedOnPhysicsThread: 1 = false;
		bool bEnableStrainOnCollision_Internal : 1 = true;

		FClusterCreationParameters ClusterParameters;
		const FClusterUnionInitData InitData;

		FPhysicsObjectUniquePtr PhysicsObject;
		TUniquePtr<FExternalParticle> Particle_External;
		FClusterUnionSyncedData SyncedData_External;

		FInternalParticle* Particle_Internal = nullptr;
		FClusterUnionIndex ClusterUnionIndex = INDEX_NONE;

		TUniquePtr<FProxyInterpolationBase> InterpolationData;

	public:
		/** Get or create a derived FProxyInterpolationBase that handles render interpolation error corrections */
		template<typename ErrorDataType>
		ErrorDataType* GetOrCreateErrorInterpolationData()
		{
			if (!InterpolationData.IsValid())
			{
				InterpolationData = MakeUnique<ErrorDataType>();
			}
			else if (InterpolationData.Get()->GetInterpolationType() != ErrorDataType::InterpolationType)
			{
				InterpolationData = MakeUnique<ErrorDataType>(InterpolationData.Get()->GetPullDataInterpIdx_External(), InterpolationData.Get()->GetInterpChannel_External());
			}

			return static_cast<ErrorDataType*>(InterpolationData.Get());
		}
		
		template<typename ErrorDataType>
		const ErrorDataType* GetErrorInterpolationData() const
		{
			return static_cast<ErrorDataType*>(InterpolationData.Get());
		}

		void ResetInterpolationData()
		{
			InterpolationData.Reset();
		}

	private:
		// An array of a particles that exist in the external implicit object union.
		// Note that this array should only be used for book-keeping. It is generally
		// unsafe to try and access the particles within this array.
		TArray<FExternalParticle*> GeometryChildParticles_External;

		//~ Begin TPhysicsProxy Interface
	public:
		bool IsSimulating() const { return true; }
		void UpdateKinematicBodiesCallback(const FParticlesType& InParticles, const float InDt, const float InTime, FKinematicProxy& InKinematicProxy) {}
		void StartFrameCallback(const float InDt, const float InTime) {}
		void EndFrameCallback(const float InDt) {}
		void CreateRigidBodyCallback(FParticlesType& InOutParticles) {}
		void DisableCollisionsCallback(TSet<TTuple<int32, int32>>& InPairs) {}
		void AddForceCallback(FParticlesType& InParticles, const float InDt, const int32 InIndex) {}
		void BindParticleCallbackMapping(Chaos::TArrayCollectionArray<PhysicsProxyWrapper>& PhysicsProxyReverseMap, Chaos::TArrayCollectionArray<int32>& ParticleIDReverseMap) {}
		static constexpr EPhysicsProxyType ConcreteType() { return EPhysicsProxyType::ClusterUnionProxy; }
		void SyncBeforeDestroy() {}
		void OnRemoveFromScene() {}
		bool IsDirty() { return false; }

		//~ End TPhysicsProxy Interface
	};

}
