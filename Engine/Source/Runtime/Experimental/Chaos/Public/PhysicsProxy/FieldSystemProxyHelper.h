// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Field/FieldSystem.h"
#include "Chaos/Array.h"
#include "Chaos/ConstraintHandle.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Defines.h"
#include "Chaos/PBDRigidClusteringAlgo.h"
#include "Chaos/Particles.h"
#include <limits>

#include "GeometryCollectionPhysicsProxy.h"

namespace Chaos
{
	/**
	 * Build the sample points positions and indices based on the resolution and filter type 
	 * @param    LocalProxy Physics proxy from which to extract the particle handles
	 * @param    RigidSolver Rigid solver owning the particles
	 * @param    FieldCommand Field command used to extract the resolution and filter meta data
	 * @param    ExecutionDatas Field Datas stored during evaluation 
	 * @param    PrevResolutionType Resolution of the previous command
	 * @param    PrevFilterType Filter of the previous command
	 * @param    PrevObjectType Object Type of the previous command
	 * @param    PrevPositionType Position type of the previous command
	 */
	template <typename PhysicsProxy>
    static bool BuildFieldSamplePoints(
		PhysicsProxy* LocalProxy,
		Chaos::FPBDRigidsSolver* RigidSolver,
		const FFieldSystemCommand& FieldCommand, 
		FFieldExecutionDatas& ExecutionDatas,
		EFieldResolutionType& PrevResolutionType, 
		EFieldFilterType& PrevFilterType, 
		EFieldObjectType& PrevObjectType,
		EFieldPositionType& PrevPositionType)
	{
		if(!LocalProxy || !RigidSolver)
		{
			return false;
		}
		const EFieldResolutionType ResolutionType =
			FieldCommand.HasMetaData(FFieldSystemMetaData::EMetaType::ECommandData_ProcessingResolution) ?
			FieldCommand.GetMetaDataAs<FFieldSystemMetaDataProcessingResolution>( 
				FFieldSystemMetaData::EMetaType::ECommandData_ProcessingResolution)->ProcessingResolution :
			EFieldResolutionType::Field_Resolution_Minimal;

		EFieldFilterType FilterType = EFieldFilterType::Field_Filter_Max;
		EFieldObjectType ObjectType = EFieldObjectType::Field_Object_Max;
		EFieldPositionType PositionType = EFieldPositionType::Field_Position_Max;

		if (FieldCommand.HasMetaData(FFieldSystemMetaData::EMetaType::ECommandData_Filter))
		{
			const FFieldSystemMetaDataFilter* MetaDataFilter = FieldCommand.GetMetaDataAs<FFieldSystemMetaDataFilter>(FFieldSystemMetaData::EMetaType::ECommandData_Filter);
			FilterType = MetaDataFilter->FilterType;
			ObjectType = MetaDataFilter->ObjectType;
			PositionType = MetaDataFilter->PositionType;
		}

		TArray<Chaos::FGeometryParticleHandle*>& FilteredHandles = ExecutionDatas.ParticleHandles[(uint8)EFieldCommandHandlesType::FilteredHandles];
		TArray<Chaos::FGeometryParticleHandle*>& InsideHandles = ExecutionDatas.ParticleHandles[(uint8)EFieldCommandHandlesType::InsideHandles];

		//if (LocalProxy && ( (PrevResolutionType != ResolutionType) || (PrevFilterType != FilterType) || (PrevObjectType != ObjectType) || (PrevPositionType != PositionType) || FilteredHandles.Num() == 0))
		{
			if (FilterType != EFieldFilterType::Field_Filter_Max)
			{
				LocalProxy->GetFilteredParticleHandles(FilteredHandles, RigidSolver, FilterType, ObjectType);
			}
			else
			{
				LocalProxy->GetRelevantParticleHandles(FilteredHandles, RigidSolver, ResolutionType);
			}

			PrevResolutionType = ResolutionType;
			PrevFilterType = FilterType;
			PrevObjectType = ObjectType;
			PrevPositionType = PositionType;

			ExecutionDatas.SamplePositions.SetNum(FilteredHandles.Num(), EAllowShrinking::No);
			ExecutionDatas.SampleIndices.SetNum(FilteredHandles.Num(), EAllowShrinking::No);
			InsideHandles.SetNum(FilteredHandles.Num(), EAllowShrinking::No);

			auto FillExecutionDatas = [&ExecutionDatas,&FieldCommand,&InsideHandles](FVec3 SamplePosition, Chaos::FGeometryParticleHandle* ParticleHandle, int32& HandleIndex)
			{
				if (FPBDRigidClusteredParticleHandle* ClusterHandle = ParticleHandle->CastToClustered())
				{
					// Disabled clustered particles that are driven by a parent, contain particle 
					// positions in local space. The field system requires the transformation of 
					// the disabled child particles into world space.
					if (ClusterHandle->Disabled() == true)
					{
						FRigidTransform3 RelativeFrame = ClusterHandle->ChildToParent();

						// go up the chain until no parent or a non-disabled parent ( should normally stop at a non disabled one )
						FPBDRigidClusteredParticleHandle* ParentHandle = ClusterHandle->Parent();
						while (ParentHandle && ParentHandle->Disabled())
						{
							RelativeFrame = RelativeFrame * ParentHandle->ChildToParent();
							ParentHandle = ParentHandle->Parent();
						}

						if (ParentHandle && ParentHandle->Disabled() == false)
						{
							const FRigidTransform3 ParentWorldTM(ParentHandle->GetP(), ParentHandle->GetQ());
							const FRigidTransform3 ChildFrame = RelativeFrame * ParentWorldTM;
							SamplePosition = ChildFrame.GetTranslation();
						}
					}
				}


				if (FieldCommand.BoundingBox.IsInside(SamplePosition))
				{
					ExecutionDatas.SamplePositions[HandleIndex] = SamplePosition;
					ExecutionDatas.SampleIndices[HandleIndex] = FFieldContextIndex(HandleIndex, HandleIndex);
					InsideHandles[HandleIndex] = ParticleHandle;
					++HandleIndex;
				}
			};

			int32 HandleIndex = 0;
			if (PositionType == EFieldPositionType::Field_Position_CenterOfMass)
			{
				for (int32 Idx = 0; Idx < FilteredHandles.Num(); ++Idx)
				{
					if (Chaos::FPBDRigidParticleHandle* RigidHandle = FilteredHandles[Idx]->CastToRigidParticle())
					{
						const FVec3 SamplePosition = FParticleUtilities::GetCoMWorldPosition(RigidHandle);
						FillExecutionDatas(SamplePosition, RigidHandle, HandleIndex);
					}
				}
			}
			else
			{
				for (int32 Idx = 0; Idx < FilteredHandles.Num(); ++Idx)
				{
					if (Chaos::FGeometryParticleHandle* FilteredHandle = FilteredHandles[Idx])
					{
						const FVec3& SamplePosition = FilteredHandle->GetX();
						FillExecutionDatas(SamplePosition, FilteredHandle, HandleIndex);
					}
				}
			}
			ExecutionDatas.SamplePositions.SetNum(HandleIndex, EAllowShrinking::No);
			ExecutionDatas.SampleIndices.SetNum(HandleIndex, EAllowShrinking::No);
			InsideHandles.SetNum(HandleIndex, EAllowShrinking::No);
		}
		return InsideHandles.Num() > 0;
	}

	/**
	 * Init the dynamics state of the particle handles to be processed by the field
	 * @param    ParticleHandles Particle hadles that will be used to init the dynamic state
	 * @param    FieldContext Field context to retrieve the evaluated samples
	 * @param    LocalResults Array to store the dynamic state
	 */
	static void InitDynamicStateResults(const TArray<Chaos::FGeometryParticleHandle*>& ParticleHandles, FFieldContext& FieldContext, TArray<int32>& LocalResults)
	{
		for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
		{
			const Chaos::EObjectStateType InitState = (ParticleHandles[Index.Sample]->ObjectState() != Chaos::EObjectStateType::Uninitialized) ?
				ParticleHandles[Index.Sample]->ObjectState() : Chaos::EObjectStateType::Dynamic;
			LocalResults[Index.Result] = static_cast<int32>(InitState);
		}
	}

	/**
	 * Init the enable/disable boolean array of the particle handles to be processed by the field
	 * @param    ParticleHandles Particle hadles that will be used to init the enable/disable booleans
	 * @param    FieldContext Field context to retrieve the evaluated samples
	 * @param    LocalResults Array to store the enable/disable boolean
	 */
	static void InitActivateDisabledResults(const TArray<Chaos::FGeometryParticleHandle*>& ParticleHandles, FFieldContext& FieldContext, TArray<int32>& LocalResults)
	{
		for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
		{
			Chaos::FPBDRigidParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();
			if (RigidHandle)
			{
				LocalResults[Index.Result] = RigidHandle->Disabled();
			}
		}
	}

	/**
	 * Set the dynamic state of a particle handle
	 * @param    Rigidsolver Rigid solver owning the particle handle
	 * @param    FieldState Field state that will be set on the handle
	 * @param    RigidHandle Particle hadle on which the state will be set
	 */
	static void SetParticleDynamicState(Chaos::FPBDRigidsSolver* RigidSolver,
		const Chaos::EObjectStateType FieldState, Chaos::FPBDRigidParticleHandle* RigidHandle)
	{
		const bool bIsGC = (RigidHandle->GetParticleType() == Chaos::EParticleType::GeometryCollection) ||
			(RigidHandle->GetParticleType() == Chaos::EParticleType::Clustered && !RigidHandle->CastToClustered()->InternalCluster());

		if (!bIsGC)
		{
			RigidSolver->GetEvolution()->SetParticleObjectState(RigidHandle, FieldState);
		}
		else
		{
			// @todo(chaos): this should also call Evolution.SetParticleObjectState, but to do that
			// we need to change how the ActiveParticlesArray is handled in RigidParticleSOAs.
			// Instead we manually do the parts of SetParticleObjectState that are still required 
			// while avoiding the SOA management.
			RigidHandle->SetObjectStateLowLevel(FieldState);
			if (!RigidHandle->Disabled())
			{
				RigidSolver->GetEvolution()->GetIslandManager().AddParticle(RigidHandle);

				// This is mainly just to refresh the views. Note that we do not want to enable disabled particles.
				RigidSolver->GetEvolution()->GetParticles().EnableParticle(RigidHandle);
			}
		}
	}

	/**
	 * Report the dynamic state result onto the handle
	 * @param    Rigidsolver Rigid solver owning the particle handle
	 * @param    FieldState Field state that will be set on the handle
	 * @param    RigidHandle Particle hadle on which the state will be set
	 * @param    HasInitialLinearVelocity Boolean to check if we have to set the initial linear velocity 
	 * @param    InitialLinearVelocity Initial linear velocity to potentially set onto the handle
	 * @param    HasInitialAngularVelocity Boolean to check if we have to set the initial angular velocity 
	 * @param    InitialAngularVelocity Initial angular velocity to potentially set onto the handle
	 */
	static bool ReportDynamicStateResult(Chaos::FPBDRigidsSolver* RigidSolver,
		const Chaos::EObjectStateType FieldState, Chaos::FPBDRigidParticleHandle* RigidHandle,
		const bool HasInitialLinearVelocity, const Chaos::FVec3& InitialLinearVelocity,
		const bool HasInitialAngularVelocity, const Chaos::FVec3& InitialAngularVelocity)
	{
		const Chaos::EObjectStateType HandleState = RigidHandle->ObjectState();

		// Do we need to be sure the mass > 0 only for the dynamic state
		const bool bHasStateChanged = ((FieldState != Chaos::EObjectStateType::Dynamic) ||
			(FieldState == Chaos::EObjectStateType::Dynamic && RigidHandle->M() > FLT_EPSILON)) && (HandleState != FieldState);  

		if (bHasStateChanged)
		{
			SetParticleDynamicState(RigidSolver, FieldState, RigidHandle);

			if (FieldState == Chaos::EObjectStateType::Kinematic || FieldState == Chaos::EObjectStateType::Static)
			{
				RigidHandle->SetVf(Chaos::FVec3f(0));
				RigidHandle->SetWf(Chaos::FVec3f(0));
			}
			else if (FieldState == Chaos::EObjectStateType::Dynamic)
			{
				if (HasInitialLinearVelocity)
				{
					RigidHandle->SetV(InitialLinearVelocity);
				}
				if (HasInitialAngularVelocity)
				{
					RigidHandle->SetW(InitialAngularVelocity);
				}
			}
		}
		return bHasStateChanged;
	}

	/**
	 * Update all the clustered particles object state to static/kinematic if one of its children state has been changed to static/kinematic
	 * @param    Rigidsolver Rigid solver owning the particle handle
	 * @param    UpdatedParticles List of particles that had their state updated
	 */
	static void UpdateSolverParticlesState(Chaos::FPBDRigidsSolver* RigidSolver, const TFieldArrayView<FFieldContextIndex>& UpdatedParticleIndices, const TArray<Chaos::FGeometryParticleHandle*>& Particles)
	{
		if (UpdatedParticleIndices.Num() > 0)
		{
			TSet<FPBDRigidClusteredParticleHandle*> TopParentClusters;
			for (const FFieldContextIndex& ParticleIndex: UpdatedParticleIndices)
			{
				if (FPBDRigidClusteredParticleHandle* ClusteredHandle = Particles[ParticleIndex.Sample]->CastToClustered())
				{
					FPBDRigidClusteredParticleHandle* TopParent = ClusteredHandle;
					while (FPBDRigidClusteredParticleHandle* DirectParent = TopParent->Parent())
					{
						TopParent = DirectParent;
					}
					TopParentClusters.Add(TopParent);
				}
			}

			const FRigidClustering& Clustering = RigidSolver->GetEvolution()->GetRigidClustering();
			for (FPBDRigidClusteredParticleHandle* TopParentClusteredHandle: TopParentClusters)
			{
				if (TopParentClusteredHandle && !TopParentClusteredHandle->Disabled())
				{
					if (TopParentClusteredHandle->ClusterIds().NumChildren)
					{
						UpdateKinematicProperties(TopParentClusteredHandle, Clustering.GetChildrenMap(), *RigidSolver->GetEvolution());
					}
					else
					{
						// if the cluster is dynamic let's make sure we clear kinematic target to avoid animated ones to have their velocity reset by the kinematic target update
						// for particles with children this is taken care in UpdateKinematicProperties
						if (TopParentClusteredHandle->ObjectState() == EObjectStateType::Dynamic)
						{
							RigidSolver->GetEvolution()->SetParticleKinematicTarget(TopParentClusteredHandle, FKinematicTarget());
						}
					}
				}
			}
		}
	}

	/**
	 * Update the handle sleeping linear and angular theshold
	 * @param    Rigidsolver Rigid solver owning the particle handle
	 * @param    RigidHandle Particle handle on which the threshold will be updated
	 * @param    ResultThreshold Threshoild to be set onto the handle
	 */
	static void UpdateMaterialSleepingThreshold(Chaos::FPBDRigidsSolver* RigidSolver, Chaos::FPBDRigidParticleHandle* RigidHandle, const Chaos::FReal ResultThreshold)
	{
		// if no per particle physics material is set, make one
		if (!RigidSolver->GetEvolution()->GetPerParticlePhysicsMaterial(RigidHandle).IsValid())
		{
			TUniquePtr<Chaos::FChaosPhysicsMaterial> NewMaterial = MakeUnique< Chaos::FChaosPhysicsMaterial>();
			NewMaterial->SleepingLinearThreshold = ResultThreshold;
			NewMaterial->SleepingAngularThreshold = ResultThreshold;

			RigidSolver->GetEvolution()->SetPhysicsMaterial(RigidHandle, MakeSerializable(NewMaterial));
			RigidSolver->GetEvolution()->SetPerParticlePhysicsMaterial(RigidHandle, NewMaterial);
		}
		else
		{
			const TUniquePtr<Chaos::FChaosPhysicsMaterial>& InstanceMaterial = RigidSolver->GetEvolution()->GetPerParticlePhysicsMaterial(RigidHandle);

			if (ResultThreshold != InstanceMaterial->DisabledLinearThreshold)
			{
				InstanceMaterial->SleepingLinearThreshold = ResultThreshold;
				InstanceMaterial->SleepingAngularThreshold = ResultThreshold;
			}
		}
	}

	/**
	 * Update the handle disable linear and angular theshold
	 * @param    Rigidsolver Rigid solver owning the particle handle
	 * @param    RigidHandle Particle handle on which the threshold will be updated
	 * @param    ResultThreshold Threshoild to be set onto the handle
	 */
	static void UpdateMaterialDisableThreshold(Chaos::FPBDRigidsSolver* RigidSolver, Chaos::FPBDRigidParticleHandle* RigidHandle, const Chaos::FReal ResultThreshold)
	{
		// if no per particle physics material is set, make one
		if (!RigidSolver->GetEvolution()->GetPerParticlePhysicsMaterial(RigidHandle).IsValid())
		{
			TUniquePtr<Chaos::FChaosPhysicsMaterial> NewMaterial = MakeUnique< Chaos::FChaosPhysicsMaterial>();
			NewMaterial->DisabledLinearThreshold = ResultThreshold;
			NewMaterial->DisabledAngularThreshold = ResultThreshold;

			RigidSolver->GetEvolution()->SetPhysicsMaterial(RigidHandle, MakeSerializable(NewMaterial));
			RigidSolver->GetEvolution()->SetPerParticlePhysicsMaterial(RigidHandle, NewMaterial);
		}
		else
		{
			const TUniquePtr<Chaos::FChaosPhysicsMaterial>& InstanceMaterial = RigidSolver->GetEvolution()->GetPerParticlePhysicsMaterial(RigidHandle);

			if (ResultThreshold != InstanceMaterial->DisabledLinearThreshold)
			{
				InstanceMaterial->DisabledLinearThreshold = ResultThreshold;
				InstanceMaterial->DisabledAngularThreshold = ResultThreshold;
			}
		}
	}

	/**
	 * Update the particle handles integer parameters based on the field evaluation
	 * @param    RigidSolver Rigid solver owning the particles
	 * @param    FieldCommand Field command to be used for the parameter field evaluatation
	 * @param    ParticleHandles List of particle handles extracted from the field command meta data
	 * @param	 FieldContext Field context that will be used for field evaluation
	 * @param    PositionTarget Chaos position contraint in which each target will be added
	 * @param    TargetedParticles List of particles (source/target) that will be filled by the PositionTarget/static parameter 
	 * @param    FinalResults Array in which will be stored the field nodes evaulation
	 */
	static void FieldIntegerParameterUpdate(Chaos::FPBDRigidsSolver* RigidSolver, const FFieldSystemCommand& FieldCommand,
		TArray<Chaos::FGeometryParticleHandle*>& ParticleHandles, FFieldContext& FieldContext, 
		Chaos::FPBDPositionConstraints& PositionTarget,
		TMap<int32, int32>& TargetedParticles, TArray<int32>& FinalResults)
	{
		TFieldArrayView<int32> ResultsView(FinalResults, 0, FinalResults.Num());

		if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_DynamicState)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_DynamicState);
			{
				InitDynamicStateResults(ParticleHandles, FieldContext, FinalResults);

				static_cast<const FFieldNode<int32>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);

				bool bHasStateChanged = false;
				
				const TFieldArrayView<FFieldContextIndex>& EvaluatedSamples = FieldContext.GetEvaluatedSamples();
				for (const FFieldContextIndex& Index : EvaluatedSamples)
				{
					Chaos::FPBDRigidParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();
					if (RigidHandle)
					{
						const int32 CurrResult = ResultsView[Index.Result];
						check(CurrResult < std::numeric_limits<int8>::max());

						const int8 ResultState = static_cast<int8>(CurrResult);
						bHasStateChanged |= ReportDynamicStateResult(RigidSolver, static_cast<Chaos::EObjectStateType>(ResultState), RigidHandle,
							false, Chaos::FVec3(0), false, Chaos::FVec3(0));
					}
				}
				if (bHasStateChanged)
				{
					UpdateSolverParticlesState(RigidSolver, EvaluatedSamples, ParticleHandles);
				}
			}
		}
		else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_ActivateDisabled)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_ActivateDisabled);
			{
				InitActivateDisabledResults(ParticleHandles, FieldContext, FinalResults);

				static_cast<const FFieldNode<int32>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					Chaos::FPBDRigidParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();
					if (RigidHandle && RigidHandle->Disabled() && ResultsView[Index.Result] == 0)
					{
						RigidSolver->GetEvolution()->EnableParticle(RigidHandle);
						SetParticleDynamicState(RigidSolver, Chaos::EObjectStateType::Dynamic, RigidHandle);
					}
				}
			}
		}
		else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_CollisionGroup)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_CollisionGroup);
			{
				static_cast<const FFieldNode<int32>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					Chaos::FPBDRigidClusteredParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToClustered();
					if (RigidHandle)
					{
						RigidHandle->SetCollisionGroup(ResultsView[Index.Result]);
					}
				}
			}
		}
		else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_PositionStatic)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_PositionStatic);
			{
				static_cast<const FFieldNode<int32>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					Chaos::FPBDRigidClusteredParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToClustered();
					if (RigidHandle && ResultsView[Index.Result])
					{
						if (TargetedParticles.Contains(Index.Sample))
						{
							const int32 ConstraintIndex = TargetedParticles[Index.Sample];
							PositionTarget.Replace(ConstraintIndex, ParticleHandles[Index.Sample]->GetX());
						}
						else
						{
							const int32 ConstraintIndex = PositionTarget.NumConstraints();
							PositionTarget.AddConstraint(RigidHandle, RigidHandle->GetX());
							TargetedParticles.Add(Index.Sample, ConstraintIndex);
						}
					}
				}
			}
		}
		else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_DynamicConstraint)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_DynamicConstraint);
			{
				UE_LOGF(LogChaos, Error, "Dynamic constraint target currently not supported by chaos");
			}
		}
	}

	/**
	 * Update the particle handles scalar parameters based on the field evaluation
	 * @param    RigidSolver Rigid solver owning the particles
	 * @param    FieldCommand Field command to be used for the parameter field evaluatation
	 * @param    ParticleHandles List of particle handles extracted from the field command meta data
	 * @param	 FieldContext Field context that will be used for field evaluation
	 * @param    PositionTarget Chaos position contraint in which each target will be added
	 * @param    TargetedParticles List of particles (source/target) that will be filled by the PositionTarget/Static parameter
	 * @param    FinalResults Array in which will be stored the field nodes evaulation
	 */
	static void FieldScalarParameterUpdate(Chaos::FPBDRigidsSolver* RigidSolver, const FFieldSystemCommand& FieldCommand,
		TArray<Chaos::FGeometryParticleHandle*>& ParticleHandles, FFieldContext& FieldContext, 
		Chaos::FPBDPositionConstraints& PositionTarget,
		TMap<int32, int32>& TargetedParticles, TArray<float>& FinalResults)
	{
		TFieldArrayView<float> ResultsView(FinalResults, 0, FinalResults.Num());

		if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_ExternalClusterStrain)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_ExternalClusterStrain);
			{
				FRigidClustering& RigidClustering = RigidSolver->GetEvolution()->GetRigidClustering();
				static_cast<const FFieldNode<float>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					const float ExternalStrainValue = ResultsView[Index.Result];
					if (ExternalStrainValue > 0)
					{
						if (Chaos::FPBDRigidClusteredParticleHandle* ClusteredParticle = ParticleHandles[Index.Sample]->CastToClustered())
						{
							const FRealSingle CurrentExternalStrains = ClusteredParticle->GetExternalStrain();
							RigidClustering.SetExternalStrain(ClusteredParticle, FMath::Max(CurrentExternalStrains, ExternalStrainValue));
						}
					}
				}
			}
		}
		else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_Kill)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_Kill);
			{
				static_cast<const FFieldNode<float>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					Chaos::FPBDRigidParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();
					if (RigidHandle && ResultsView[Index.Result] > 0.0)
					{
						RigidSolver->GetEvolution()->DisableParticle(RigidHandle);
					}
				}
			}
		}
		else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_SleepingThreshold)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_SleepingThreshold);
			{
				static_cast<const FFieldNode<float>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					Chaos::FPBDRigidParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();
					if (RigidHandle && ResultsView.Num() > 0)
					{
						UpdateMaterialSleepingThreshold(RigidSolver, RigidHandle, ResultsView[Index.Result]);
					}
				}
			}
		}
		else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_DisableThreshold)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_DisableThreshold);
			{
				static_cast<const FFieldNode<float>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					Chaos::FPBDRigidParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();

					if (RigidHandle && RigidHandle->ObjectState() == Chaos::EObjectStateType::Dynamic && ResultsView.Num() > 0)
					{
						UpdateMaterialDisableThreshold(RigidSolver, RigidHandle, ResultsView[Index.Result]);
					}
				}
			}
		}
		else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_InternalClusterStrain)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_InternalClusterStrain);
			{
				FRigidClustering& RigidClustering = RigidSolver->GetEvolution()->GetRigidClustering();
				static_cast<const FFieldNode<float>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					Chaos::FPBDRigidClusteredParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToClustered();
					if (RigidHandle && RigidHandle->ObjectState() == Chaos::EObjectStateType::Dynamic)
					{
						RigidClustering.SetInternalStrain(RigidHandle, RigidHandle->GetInternalStrains() + ResultsView[Index.Result]);
					}
				}
			}
		}
	}

	/**
	 * Update the particle handles vector parameters based on the field evaluation
	 * @param    RigidSolver Rigid solver owning the particles
	 * @param    FieldCommand Field command to be used for the parameter field evaluatation
	 * @param    ParticleHandles List of particle handles extracted from the field command meta data
	 * @param	 FieldContext Field context that will be used for field evaluation
	 * @param    PositionTarget Chaos position contraint in which each target will be added
	 * @param    TargetedParticles List of particles (source/target) that will be filled by the PositionTarget/Static parameter
	 * @param    FinalResults Array in which will be stored the field nodes evaulation
	 */
	static void FieldVectorParameterUpdate(Chaos::FPBDRigidsSolver* RigidSolver, const FFieldSystemCommand& FieldCommand,
		TArray<Chaos::FGeometryParticleHandle*>& ParticleHandles, FFieldContext& FieldContext, 
		Chaos::FPBDPositionConstraints& PositionTarget,
		TMap<int32, int32>& TargetedParticles, TArray<FVector>& FinalResults)
	{
		TFieldArrayView<FVector> ResultsView(FinalResults, 0, FinalResults.Num());

		Chaos::FPBDRigidsSolver::FPBDRigidsEvolution* Evolution = RigidSolver->GetEvolution();

		if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_LinearVelocity)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_LinearVelocity);
			{
				static_cast<const FFieldNode<FVector>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					const FVector DeltaV = ResultsView[Index.Result];
					if (!DeltaV.IsZero())
					{
						Chaos::FPBDRigidParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();
						if (RigidHandle && RigidHandle->IsDynamic())
						{
							RigidHandle->SetV(RigidHandle->GetV() + DeltaV);
							if (Evolution && RigidHandle->IsSleeping())
							{
								Evolution->WakeParticle(RigidHandle);
							}
						}
					}
				}
			}
		}
		else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_LinearImpulse)
		{
			SCOPE_CYCLE_COUNTER(STAT_ForceUpdateField_LinearImpulse);
			{
				static_cast<const FFieldNode<FVector>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					const FVector DeltaImpulse = ResultsView[Index.Result];
					if (!DeltaImpulse.IsZero())
					{
						Chaos::FPBDRigidParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();
						if (RigidHandle && RigidHandle->IsDynamic())
						{
							const FVec3 CurrentImpulseVelocity = RigidHandle->LinearImpulseVelocity();
							RigidHandle->SetLinearImpulseVelocity(CurrentImpulseVelocity + (DeltaImpulse * RigidHandle->InvM()));
							if (Evolution && RigidHandle->IsSleeping())
							{
								Evolution->WakeParticle(RigidHandle);
							}
						}
					}
				}
			}
		}
		else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_AngularVelociy)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_AngularVelocity);
			{
				static_cast<const FFieldNode<FVector>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);

				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					const FVector DeltaW = ResultsView[Index.Result];
					if (!DeltaW.IsZero())
					{
						Chaos::FPBDRigidParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();
						if (RigidHandle && RigidHandle->IsDynamic())
						{
							RigidHandle->SetW(RigidHandle->GetW() + DeltaW);
							if (Evolution && RigidHandle->IsSleeping())
							{
								Evolution->WakeParticle(RigidHandle);
							}
						}
					}
				}
			}
		}
		else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_PositionTarget)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_PositionTarget);
			{
				static_cast<const FFieldNode<FVector>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);
				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					Chaos::FPBDRigidClusteredParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToClustered();
					if (RigidHandle && ResultsView[Index.Result] != FVector(FLT_MAX))
					{
						if (TargetedParticles.Contains(Index.Sample))
						{
							const int32 ConstraintIndex = TargetedParticles[Index.Sample];
							PositionTarget.Replace(ConstraintIndex, ResultsView[Index.Result]);
						}
						else
						{
							const int32 ConstraintIndex = PositionTarget.NumConstraints();
							PositionTarget.AddConstraint(RigidHandle, ResultsView[Index.Result]);
							TargetedParticles.Add(Index.Sample, ConstraintIndex);
						}
					}
				}
			}
		}
		else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_PositionAnimated)
		{
			SCOPE_CYCLE_COUNTER(STAT_ParamUpdateField_PositionAnimated);
			{
				UE_LOGF(LogChaos, Error, "Position Animated target currently not supported by chaos");
			}
		}
	}


	static void FieldVectorForceUpdate(Chaos::FPBDRigidsSolver* RigidSolver, const FFieldSystemCommand& FieldCommand,
		TArray<Chaos::FGeometryParticleHandle*>& ParticleHandles, FFieldContext& FieldContext, TArray<FVector>& FinalResults)
	{
		TFieldArrayView<FVector> ResultsView(FinalResults, 0, FinalResults.Num());

		static_cast<const FFieldNode<FVector>*>(FieldCommand.RootNode.Get())->Evaluate(FieldContext, ResultsView);

		if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_LinearForce)
		{
			SCOPE_CYCLE_COUNTER(STAT_ForceUpdateField_LinearForce);
			{
				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					Chaos::FPBDRigidParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();
					if (RigidHandle && !RigidHandle->Disabled() && (RigidHandle->ObjectState() == Chaos::EObjectStateType::Dynamic || RigidHandle->ObjectState() == Chaos::EObjectStateType::Sleeping))
					{
						if (RigidHandle->Sleeping())
						{
							RigidHandle->SetObjectStateLowLevel(Chaos::EObjectStateType::Dynamic);
						}
						RigidHandle->AddForce(ResultsView[Index.Result]);
					}
				}
			}
		}
		else if (FieldCommand.PhysicsType == EFieldPhysicsType::Field_AngularTorque)
		{
			SCOPE_CYCLE_COUNTER(STAT_ForceUpdateField_AngularTorque);
			{
				for (const FFieldContextIndex& Index : FieldContext.GetEvaluatedSamples())
				{
					Chaos::FPBDRigidParticleHandle* RigidHandle = ParticleHandles[Index.Sample]->CastToRigidParticle();
					if (RigidHandle && (RigidHandle->ObjectState() == Chaos::EObjectStateType::Dynamic || RigidHandle->ObjectState() == Chaos::EObjectStateType::Sleeping))
					{
						if (RigidHandle->Sleeping())
						{
							RigidHandle->SetObjectStateLowLevel(Chaos::EObjectStateType::Dynamic);
						}
						RigidHandle->AddTorque(ResultsView[Index.Result]);
					}
				}
			}
		}
	}
}

FORCEINLINE bool IsParameterFieldValid(const FFieldSystemCommand& FieldCommand)
{
	if (FieldCommand.RootNode->Type() == FFieldNodeBase::EFieldType::EField_Int32)
	{
		return (FieldCommand.PhysicsType == EFieldPhysicsType::Field_DynamicState) ||
			(FieldCommand.PhysicsType == EFieldPhysicsType::Field_ActivateDisabled) ||
			(FieldCommand.PhysicsType == EFieldPhysicsType::Field_CollisionGroup);

	}
	else if (FieldCommand.RootNode->Type() == FFieldNodeBase::EFieldType::EField_Float)
	{
		return (FieldCommand.PhysicsType == EFieldPhysicsType::Field_ExternalClusterStrain) ||
			(FieldCommand.PhysicsType == EFieldPhysicsType::Field_Kill) ||
			(FieldCommand.PhysicsType == EFieldPhysicsType::Field_SleepingThreshold) ||
			(FieldCommand.PhysicsType == EFieldPhysicsType::Field_DisableThreshold) ||
			(FieldCommand.PhysicsType == EFieldPhysicsType::Field_InternalClusterStrain);
	}
	else if (FieldCommand.RootNode->Type() == FFieldNodeBase::EFieldType::EField_FVector)
	{
		return (FieldCommand.PhysicsType == EFieldPhysicsType::Field_LinearVelocity) ||
				(FieldCommand.PhysicsType == EFieldPhysicsType::Field_AngularVelociy) ||
				(FieldCommand.PhysicsType == EFieldPhysicsType::Field_LinearImpulse);
	}
	return false;
}

FORCEINLINE bool IsForceFieldValid(const FFieldSystemCommand& FieldCommand)
{
	if (FieldCommand.RootNode->Type() == FFieldNodeBase::EFieldType::EField_FVector)
	{
		return (FieldCommand.PhysicsType == EFieldPhysicsType::Field_LinearForce) ||
			(FieldCommand.PhysicsType == EFieldPhysicsType::Field_AngularTorque);
	}
	return false;
}
	
