// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollectionProxyData.h"

#include "Chaos/Framework/PhysicsProxy.h"
#include "GeometryCollection/ManagedArray.h"
#include "GeometryCollection/GeometryCollectionCollisionStructureManager.h"
#include "Chaos/CollisionFilterData.h"
#include "Chaos/Framework/BufferedData.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PhysicsObject.h"
#include "Containers/Array.h"
#include "PBDRigidsSolver.h"
#include "Chaos/Defines.h"
#include "Chaos/GeometryParticlesfwd.h"


namespace Chaos
{
	template <typename T> class TSerializablePtr;
	class FErrorReporter;
	struct FClusterCreationParameters;

	struct FDirtyGeometryCollectionData;

	class FPBDCollisionConstraints;
	class FPBDRigidsEvolutionBase;
}

/**
 * index abstraction for HitResults
 * this allows regular bones and internal clusters to be represented by a unique int32 index that can be passed to HitResults
 * and in return understood by the geometry collection physics proxy 
 */ 
struct FGeometryCollectionItemIndex
{
public:
	static FGeometryCollectionItemIndex CreateInternalClusterItemIndex(int32 ClusterUniqueIdx)
	{
		return FGeometryCollectionItemIndex(ClusterUniqueIdx, true);
	}

	static FGeometryCollectionItemIndex CreateTransformItemIndex(int32 TransformIndex)
	{
		return FGeometryCollectionItemIndex(TransformIndex, false);
	}

	static FGeometryCollectionItemIndex CreateFromExistingItemIndex(int32 ItemIndex)
	{
		FGeometryCollectionItemIndex Result(INDEX_NONE, false);
		Result.ItemIndex = ItemIndex;
		return Result;
	}
	
	static FGeometryCollectionItemIndex CreateInvalidItemIndex()
	{
		return FGeometryCollectionItemIndex(INDEX_NONE, false);
	}
	
public:
	FGeometryCollectionItemIndex(const FGeometryCollectionItemIndex& Other)
		: ItemIndex(Other.ItemIndex)
	{}
	
	bool IsInternalCluster() const
	{
		return ItemIndex < INDEX_NONE;
	}

	int32 GetInternalClusterIndex() const
	{
		check(IsInternalCluster());
		return (ItemIndex - InternalClusterBaseIndex);
	}

	int32 GetTransformIndex() const
	{
		check(!IsInternalCluster());
		return ItemIndex;
	}
	
	bool IsValid() const
	{
		return ItemIndex != INDEX_NONE;
	}

	int32 GetItemIndex() const
	{
		return ItemIndex;
	}

	bool operator==(const FGeometryCollectionItemIndex& Other) const
	{
		return ItemIndex == Other.ItemIndex;
	}
	
private:
	static const int32 InternalClusterBaseIndex = TNumericLimits<int32>::Lowest();   

	FGeometryCollectionItemIndex(int32 Index, bool bInternalCluster)
		: ItemIndex(INDEX_NONE)
	{
		if (Index > INDEX_NONE)
		{
			ItemIndex = Index + (bInternalCluster? InternalClusterBaseIndex: 0); 	
		}
	}
	
	int32 ItemIndex;
};

class FStubGeometryCollectionData : public Chaos::FParticleData 
{
public:
	typedef Chaos::FParticleData Base;

	FStubGeometryCollectionData(const FGeometryCollectionResults* DataIn=nullptr)
		: Base(Chaos::EParticleType::GeometryCollection)
		, Data(DataIn)
	{}

	void Reset() 
	{
		Base::Reset(); // Sets Type to EParticleType::Static
	}

	const FGeometryCollectionResults* GetStateData() const { return Data; }

private:
	const FGeometryCollectionResults* Data;
};

/**
 * Class to manage sharing data between the game thread and the simulation thread 
 * (which may not be different than the game thread) for a \c FGeometryDynamicCollection.
 */
class FGeometryCollectionPhysicsProxy : public TPhysicsProxy<FGeometryCollectionPhysicsProxy, FStubGeometryCollectionData, FGeometryCollectionProxyTimestamp>
{
public:
	typedef TPhysicsProxy<FGeometryCollectionPhysicsProxy, FStubGeometryCollectionData, FGeometryCollectionProxyTimestamp> Base;
	typedef FCollisionStructureManager::FSimplicial FSimplicial;
	typedef Chaos::TPBDRigidParticle<Chaos::FReal, 3> FParticle;
	typedef Chaos::TPBDRigidParticleHandle<Chaos::FReal, 3> FParticleHandle;
	typedef Chaos::TPBDRigidClusteredParticleHandle<Chaos::FReal, 3> FClusterHandle;

	/** Proxy publics */
	using IPhysicsProxyBase::GetSolver;

	FGeometryCollectionPhysicsProxy() = delete;
	/**
	 * \p InOwner
	 * \p InDynamicCollection game thread owned geometry collection.
	 * \p InInitFunc callback invoked from \c Initialize().
	 * \p InCacheSyncFunc callback invoked from \c PullFromPhysicsState().
	 * \p InFinalSyncFunc callback invoked from \c SyncBeforeDestory().
	 */
	UE_DEPRECATED(5.6, "Use the constructor that use a shared ref  for the game thread collection")
	CHAOS_API FGeometryCollectionPhysicsProxy(
		UObject* InOwner, 
		FGeometryDynamicCollection& GameThreadCollection,
		const FSimulationParameters& SimulationParameters,
		FCollisionFilterData InSimFilter,
		FCollisionFilterData InQueryFilter,
		FGuid InCollectorGuid = FGuid::NewGuid(),
		const Chaos::EMultiBufferMode BufferMode=Chaos::EMultiBufferMode::TripleGuarded);

	UE_DEPRECATED(5.8, "Use the constructor that takes FShapeFilterData and FInstanceData instead")
	CHAOS_API FGeometryCollectionPhysicsProxy(
		UObject* InOwner,
		TSharedRef<FGeometryDynamicCollection> GameThreadCollection,
		const FSimulationParameters& SimulationParameters,
		FCollisionFilterData InSimFilter,
		FCollisionFilterData InQueryFilter,
		FGuid InCollectorGuid = FGuid::NewGuid(),
		const Chaos::EMultiBufferMode BufferMode = Chaos::EMultiBufferMode::TripleGuarded);

	CHAOS_API FGeometryCollectionPhysicsProxy(
		UObject* InOwner,
		TSharedRef<FGeometryDynamicCollection> GameThreadCollection,
		const FSimulationParameters& SimulationParameters,
		const Chaos::Filter::FShapeFilterData& InShapeFilterData,
		const Chaos::Filter::FInstanceData& InShapeInstanceData,
		FGuid InCollectorGuid = FGuid::NewGuid(),
		const Chaos::EMultiBufferMode BufferMode = Chaos::EMultiBufferMode::TripleGuarded);

	CHAOS_API virtual ~FGeometryCollectionPhysicsProxy();

	/**
	 * Construct \c PTDynamicCollection, copying attributes from the game thread, 
	 * and prepare for simulation.
	 */
	CHAOS_API void Initialize(Chaos::FPBDRigidsEvolutionBase* Evolution);
	void Reset() { }

	bool IsInitializedOnPhysicsThread() const { return bIsInitializedOnPhysicsThread; }

	/** 
	 * Finish initialization on the physics thread. 
	 *
	 * Called by solver command registered by \c FPBDRigidsSolver::RegisterObject().
	 */
	CHAOS_API void InitializeBodiesPT(
		Chaos::FPBDRigidsSolver* RigidsSolver,
		typename Chaos::FPBDRigidsSolver::FParticlesType& Particles);

	/** */
	static CHAOS_API void InitializeDynamicCollection(FGeometryDynamicCollection& DynamicCollection, const FGeometryCollection& RestCollection, const FSimulationParameters& Params);

	/** */
	bool IsSimulating() const { return Parameters.Simulating; }

	/**
	 * Pushes current game thread particle state into the \c GameToPhysInterchange.
	 *
	 * Redirects to \c BufferGameState(), and returns nullptr as this class manages 
	 * data transport to the physics thread itself, without allocating memory.
	 */
	Chaos::FParticleData* NewData() { BufferGameState(); return nullptr; }
	CHAOS_API void BufferGameState();

	/** Called at the end of \c FPBDRigidsSolver::PushPhysicsStateExec(). */
	void ClearAccumulatedData() {}

	/** Push PT state into the \c PhysToGameInterchange. */
	CHAOS_API void BufferPhysicsResults_Internal(Chaos::FPBDRigidsSolver* CurrentSolver, Chaos::FDirtyGeometryCollectionData& BufferData);

	/** Push GT state into the \c PhysToGameInterchange for async physics */
	CHAOS_API void BufferPhysicsResults_External(Chaos::FDirtyGeometryCollectionData& BufferData);

	/** Push data from the game thread to the physics thread */
	CHAOS_API void PushStateOnGameThread(Chaos::FPBDRigidsSolver* InSolver);

	/** apply the state changes on the physics thread */
	CHAOS_API void PushToPhysicsState();
	
	/** Does nothing as \c BufferPhysicsResults() already did this. */
	CHAOS_API void FlipBuffer();
	
	/** 
	 * Pulls data out of the PhysToGameInterchange and updates \c GTDynamicCollection. 
	 * Called from FPhysScene_ChaosInterface::SyncBodies(), NOT the solver.
	 */
	CHAOS_API bool PullFromPhysicsState(const Chaos::FDirtyGeometryCollectionData& BufferData, const int32 SolverSyncTimestamp, const Chaos::FDirtyGeometryCollectionData* NextPullData = nullptr, const Chaos::FRealSingle* Alpha = nullptr, const Chaos::FDirtyRigidParticleReplicationErrorData* Error = nullptr, const Chaos::FReal AsyncFixedTimeStep = 0);

	bool IsDirty() { return false; }

	static constexpr EPhysicsProxyType ConcreteType() { return EPhysicsProxyType::GeometryCollectionType; }

	CHAOS_API void SyncBeforeDestroy();
	CHAOS_API void OnRemoveFromSolver(Chaos::FPBDRigidsSolver *RBDSolver);
	CHAOS_API void OnRemoveFromScene();
	CHAOS_API void OnUnregisteredFromSolver();

	void SetCollisionParticlesPerObjectFraction(float CollisionParticlesPerObjectFractionIn) 
	{CollisionParticlesPerObjectFraction = CollisionParticlesPerObjectFractionIn;}

	UE_DEPRECATED(5.4, "Use GetSolverClusterHandle_Internal instead")
	TArray<FClusterHandle*>& GetSolverClusterHandles() { return SolverClusterHandles; }

	UE_DEPRECATED(5.4, "Use GetParticle_Internal instead")
	TArray<FClusterHandle*>& GetSolverParticleHandles() { return SolverParticleHandles; }

	FClusterHandle* GetSolverClusterHandle_Internal(int32 Index) const
	{
		const int32 ParticleIndex = FromTransformToParticleIndex[Index];
		if (ParticleIndex != INDEX_NONE)
		{
			return SolverClusterHandles[ParticleIndex];
		}
		return nullptr;
	}

	const FGeometryCollectionResults* GetConsumerResultsGT() const 
	{ return PhysToGameInterchange.PeekConsumerBuffer(); }

	/** Enqueue a field \p Command to be processed by \c ProcessCommands() or \c FieldForcesUpdateCallback(). Game thread only */
	CHAOS_API void BufferFieldCommand_External(FFieldSystemCommand&& Command);

	UE_DEPRECATED(5.5, "Use BufferFieldCommand_Internal instead when calling on the physics thread or the _external version when calling on the gamethread")
	CHAOS_API void BufferCommand(Chaos::FPBDRigidsSolver* RigidsSolver, const FFieldSystemCommand& Command);

	/** Enqueue a field \p Command to be processed by \c ProcessCommands() or \c FieldForcesUpdateCallback(). Physics thread only*/
	CHAOS_API void BufferFieldCommand_Internal(Chaos::FPBDRigidsSolver* RigidsSolver, const FFieldSystemCommand& Command);

	CHAOS_API void FieldForcesUpdateCallback(Chaos::FPBDRigidsSolver* RigidSolver);

	CHAOS_API void FieldParameterUpdateCallback(Chaos::FPBDRigidsSolver* RigidSolver, const bool bUpdateViews = true);

	static CHAOS_API bool NeedToInitializeSharedCollisionStructures(const FGeometryCollection& RestCollection);
	static CHAOS_API void InitializeSharedCollisionStructures(Chaos::FErrorReporter& ErrorReporter, FGeometryCollection& RestCollection, const FSharedSimulationParameters& SharedParams);

	void UpdateKinematicBodiesCallback(const FParticlesType& InParticles, const float InDt, const float InTime, FKinematicProxy& InKinematicProxy) {}
	void StartFrameCallback(const float InDt, const float InTime) {}
	void EndFrameCallback(const float InDt) {}
	void BindParticleCallbackMapping(Chaos::TArrayCollectionArray<PhysicsProxyWrapper>& PhysicsProxyReverseMap, Chaos::TArrayCollectionArray<int32>& ParticleIDReverseMap) {}
	void CreateRigidBodyCallback(FParticlesType& InOutParticles) {}
	void DisableCollisionsCallback(TSet<TTuple<int32, int32>>& InPairs) {}
	void AddForceCallback(FParticlesType& InParticles, const float InDt, const int32 InIndex) {}

	bool IsGTCollectionDirty() const { return GameThreadCollection.IsDirty(); }

	// set the world transform ( this needs to be called on the game thread )
	CHAOS_API void SetWorldTransform_External(const FTransform& WorldTransform);

	// Update cached world transforms (GT side, and mirror Parameters.WorldTransform on PT)
	// without dispatching kinematic targets or cluster-union updates.
	// Use when the component moves on the game thread but the simulation must not be perturbed.
	CHAOS_API void UpdateWorldTransformCaches_External(const FTransform& WorldTransform);

	const FTransform& GetPreviousWorldTransform_External() const { return PreviousWorldTransform_External; }
	const FTransform& GetWorldTransform_External() { return WorldTransform_External; }

	// todo(chaos): Remove this and move to a cook time approach of the SM data based on the GC property
	// Set whether the GC should be using collision from the Static Mesh or the GC itself for game thread traces ( this needs to be called on the game thread )
	CHAOS_API void SetUseStaticMeshCollisionForTraces_External(bool bInUseStaticMeshCollisionForTraces);

	UE_DEPRECATED(5.4, "Use GetParticle_Internal instead")
	const TArray<FClusterHandle*> GetParticles() const
	{
		return SolverParticleHandles;
	}

	const TArray<FClusterHandle*>GetUnorderedParticles_Internal() const
	{
		return SolverParticleHandles;
	}

	FClusterHandle* GetParticle_Internal(int32 Index) const
	{
		if (FromTransformToParticleIndex.IsValidIndex(Index))
		{
			const int32 ParticleIndex = FromTransformToParticleIndex[Index];
			if (SolverParticleHandles.IsValidIndex(ParticleIndex))
			{
				return SolverParticleHandles[ParticleIndex];
			}
		}
		return nullptr;
	}

	const FSimulationParameters& GetSimParameters() const
	{
		return Parameters;
	}

	FSimulationParameters& GetSimParameters()
	{
		return Parameters;
	}

	FGeometryDynamicCollection& GetPhysicsCollection()
	{
		return PhysicsThreadCollection;
	}

	FGeometryDynamicCollection& GetExternalCollection()
	{
		return GameThreadCollection;
	}

	UE_DEPRECATED(5.4, "Use GetUnorderedParticles_External instead")
	TArray<TUniquePtr<FParticle>>& GetExternalParticles()
	{
		return GTParticles;
	}

	TArray<TUniquePtr<FParticle>>& GetUnorderedParticles_External()
	{
		return GTParticles;
	}

	CHAOS_API FParticle* GetParticleByIndex_External(int32 Index);
	CHAOS_API const FParticle* GetParticleByIndex_External(int32 Index) const;

	CHAOS_API FParticleHandle* GetParticleByIndex_Internal(int32 Index);
	CHAOS_API const FParticleHandle* GetParticleByIndex_Internal(int32 Index) const;

	FParticle* GetInitialRootParticle_External() { return GetParticleByIndex_External(Parameters.InitialRootIndex); }
	const FParticle* GetInitialRootParticle_External() const { return GetParticleByIndex_External(Parameters.InitialRootIndex); }

	FParticleHandle* GetInitialRootParticle_Internal() { return GetParticleByIndex_Internal(Parameters.InitialRootIndex); }
	const FParticleHandle* GetInitialRootParticle_Internal() const { return GetParticleByIndex_Internal(Parameters.InitialRootIndex); }

	/**
	*  * Get all the geometry collection particle handles based on the processing resolution
	 */
	CHAOS_API void GetRelevantParticleHandles(
		TArray<Chaos::TGeometryParticleHandle<Chaos::FReal, 3>*>& Handles,
		const Chaos::FPBDRigidsSolver* RigidSolver,
		EFieldResolutionType ResolutionType);

	/**
	 * Get all the geometry collection particle handles filtered by object state
	 */
	CHAOS_API void GetFilteredParticleHandles(
		TArray<Chaos::TGeometryParticleHandle<Chaos::FReal, 3>*>& Handles,
		const Chaos::FPBDRigidsSolver* RigidSolver,
		const EFieldFilterType FilterType,
		const EFieldObjectType ObjectType);
		
	/* Implemented so we can construct TAccelerationStructureHandle. */
	virtual void* GetHandleUnsafe() const override { return nullptr; }

	int32 GetTransformGroupIndexFromHandle(const FParticleHandle* Handle) const
	{
		if (const int32* TransformGroupIndex = HandleToTransformGroupIndex.Find(Handle))
		{
			return *TransformGroupIndex;
		}
		return INDEX_NONE;
	}

	void GetTransformGroupIndicesFromHandles(const TArray<FParticleHandle*> Handles, TArray<int32>& IndicesOut) const
	{
		IndicesOut.SetNumUninitialized(Handles.Num());
		for (int32 HandleIndex = 0; HandleIndex < Handles.Num(); ++HandleIndex)
		{
			IndicesOut[HandleIndex] = INDEX_NONE;
			if (const int32* TransformGroupIndex = HandleToTransformGroupIndex.Find(Handles[HandleIndex]))
			{
				IndicesOut[HandleIndex] = (*TransformGroupIndex);
			}
		}
	}

	FGeometryCollectionItemIndex GetInternalClusterParentItemIndex_External(int32 ChildTransformIndex) const
	{
		// first find the GTParticle matching the Transform index
		if (FromTransformToParticleIndex.IsValidIndex(ChildTransformIndex))
		{
			const int32 ParticleIndex = FromTransformToParticleIndex[ChildTransformIndex];

			if (GTParticles.IsValidIndex(ParticleIndex))
			{
				const TUniquePtr<FParticle>& ChildGTParticle = GTParticles[ParticleIndex];
				if (const FGTParticleIndices* Indices = GTParticleToIndices.Find(ChildGTParticle.Get()))
				{
					if (Indices->InternalClusterUniqueId != INDEX_NONE)
					{
						return FGeometryCollectionItemIndex::CreateInternalClusterItemIndex(Indices->InternalClusterUniqueId);
					}
				}
			}
		}
		return FGeometryCollectionItemIndex::CreateInvalidItemIndex();
	}

	const TArray<int32>* FindInternalClusterChildrenTransformIndices_External(FGeometryCollectionItemIndex ItemIndex) const
	{
		if (ensure(ItemIndex.IsInternalCluster()))
		{
			return InternalClusterUniqueIdxToChildrenTransformIndices.Find(ItemIndex.GetInternalClusterIndex());
		}
		return nullptr;
	}
	
	FGeometryCollectionItemIndex GetItemIndexFromGTParticle_External(const FParticle* GTPParticle) const
	{
		// internal cluster have  no representation on the GT, so we use the child GT particle to find the matching internal cluster unique index 
		if (const FGTParticleIndices* Indices = GTParticleToIndices.Find(GTPParticle))
		{
			if (Indices->InternalClusterUniqueId != INDEX_NONE)
			{
				return FGeometryCollectionItemIndex::CreateInternalClusterItemIndex(Indices->InternalClusterUniqueId);
			}
			// regular particle that has a matching transform index 
			if (Indices->TransformGroupIndex != INDEX_NONE)
			{
				return FGeometryCollectionItemIndex::CreateTransformItemIndex(Indices->TransformGroupIndex);
			}
		}
		return FGeometryCollectionItemIndex::CreateInvalidItemIndex();
	}

	FGeometryCollectionItemIndex GetItemIndexFromGTParticleNoInternalCluster_External(const FParticle* GTPParticle) const
	{
		if (const FGTParticleIndices* Indices = GTParticleToIndices.Find(GTPParticle))
		{
			if (Indices->TransformGroupIndex != INDEX_NONE)
			{
				return FGeometryCollectionItemIndex::CreateTransformItemIndex(Indices->TransformGroupIndex);
			}
		}
		return FGeometryCollectionItemIndex::CreateInvalidItemIndex();
	}
	
	CHAOS_API FName GetTransformName_External(FGeometryCollectionItemIndex ItemIndex) const;

	bool GetIsObjectDynamic() const { return IsObjectDynamic; }

	CHAOS_API void DisableParticles_External(TArray<int32>&& TransformGroupIndices);

	CHAOS_API void ApplyForceAt_External(FVector Force, FVector WorldLocation);
	CHAOS_API void ApplyImpulseAt_External(FVector Force, FVector WorldLocation);
	CHAOS_API void BreakClusters_External(TArray<FGeometryCollectionItemIndex>&& ItemIndices);
	CHAOS_API void BreakActiveClusters_External();
	CHAOS_API void SetAnchoredByIndex_External(int32 Index, bool bAnchored);
	CHAOS_API void SetAnchoredByTransformedBox_External(const FBox& Box, const FTransform& Transform, bool bAnchored, int32 MaxLevel = INDEX_NONE);
	CHAOS_API void RemoveAllAnchors_External();
	CHAOS_API void ApplyExternalStrain_External(FGeometryCollectionItemIndex ItemIndex, const FVector& WorldLocation, float Radius, int32 PropagationDepth, float PropagationFactor, float StrainValue);
	CHAOS_API void ApplyInternalStrain_External(FGeometryCollectionItemIndex ItemIndex, const FVector& WorldLocation, float Radius, int32 PropagationDepth, float PropagationFactor, float StrainValue);
	CHAOS_API void ApplyBreakingLinearVelocity_External(FGeometryCollectionItemIndex ItemIndex, const FVector& LinearVelocity);
	CHAOS_API void ApplyBreakingAngularVelocity_External(FGeometryCollectionItemIndex ItemIndex, const FVector& AngularVelocity);
	CHAOS_API void ApplyLinearVelocity_External(FGeometryCollectionItemIndex ItemIndex, const FVector& LinearVelocity);
	CHAOS_API void ApplyAngularVelocity_External(FGeometryCollectionItemIndex ItemIndex, const FVector& AngularVelocity);
	CHAOS_API void SetLinearDamping_External(float InDamping);
	CHAOS_API void SetLinearDampingPerTransform_External(const TArray<int32>& TransformIndices, float InDamping);
	CHAOS_API void SetAngularDamping_External(float InDamping);
	CHAOS_API void SetAngularDampingPerTransform_External(const TArray<int32>& TransformIndices, float InDamping);

	CHAOS_API void SetProxyDirty_External();

	CHAOS_API void SetEnableDamageFromCollision_External(bool bEnable);
	CHAOS_API void SetNotifyBreakings_External(bool bNotify);
	CHAOS_API void SetNotifyRemovals_External(bool bNotify);
	CHAOS_API void SetNotifyCrumblings_External(bool bNotify, bool bIncludeChildren);
	CHAOS_API void SetNotifyTrailings_External(bool bNotify);

	CHAOS_API void SetNotifyGlobalBreakings_External(bool bNotify);
	CHAOS_API void SetNotifyGlobalRemovals_External(bool bNotify);
	CHAOS_API void SetNotifyGlobalCrumblings_External(bool bNotify, bool bIncludeChildren);
	CHAOS_API void SetNotifyGlobalTrailings_External(bool bNotify);	

	CHAOS_API float ComputeMaterialBasedDamageThreshold_Internal(Chaos::FPBDRigidClusteredParticleHandle& ClusteredParticle) const;

	UE_INTERNAL CHAOS_API void FlushDirtyParticles(Chaos::FPBDRigidsSolver& Solver);

	FProxyInterpolationBase* GetInterpolationData() { return InterpolationData.Get(); }
	const FProxyInterpolationBase* GetInterpolationData() const { return InterpolationData.Get(); }

	void ResetInterpolationData()
	{
		InterpolationData.Reset();
	}
	
	enum class EReplicationMode: uint8
	{
		Unknown,
		Server,
		Client,
	};

	void SetReplicationMode(EReplicationMode Mode) { ReplicationMode = Mode; }
	EReplicationMode GetReplicationMode() const { return ReplicationMode; }

	UE_DEPRECATED(5.7, "Use the overload that takes a FCombinedShapeFilterData instead") CHAOS_API void UpdateFilterData_External(const FCollisionFilterData& NewSimFilter, const FCollisionFilterData& NewQueryFilter);
	CHAOS_API void UpdateFilterData_External(const Chaos::Filter::FCombinedShapeFilterData& NewShapeFilter);

	struct FParticleCollisionFilterData
	{
		// These must be explicitly declared to get rid of deprecation warnings.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FParticleCollisionFilterData() = default;
		FParticleCollisionFilterData(const FParticleCollisionFilterData&) = default;
		FParticleCollisionFilterData(FParticleCollisionFilterData&&) = default;

		FParticleCollisionFilterData& operator=(const FParticleCollisionFilterData&) = default;
		FParticleCollisionFilterData& operator=(FParticleCollisionFilterData&&) = default;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		int32 ParticleIndex = INDEX_NONE;
		bool bIsValid = false;
		bool bQueryEnabled = false;
		bool bSimEnabled = false;
		UE_DEPRECATED(5.8, "Use ShapeFilterData and FilterInstanceData")
		FCollisionFilterData QueryFilter;
		UE_DEPRECATED(5.8, "Use ShapeFilterData and FilterInstanceData")
		FCollisionFilterData SimFilter;
		Chaos::Filter::FShapeFilterData ShapeFilterData;
		Chaos::Filter::FInstanceData FilterInstanceData;
	};

	CHAOS_API void UpdatePerParticleFilterData_External(const TArray<FParticleCollisionFilterData>& Data);
	
	CHAOS_API void SetDamageThresholds_External(const TArray<float>& DamageThresholds);
	CHAOS_API void SetDamagePropagationData_External(bool bEnabled, float BreakDamagePropagationFactor, float ShockDamagePropagationFactor);
	CHAOS_API void SetDamageModel_External(EDamageModelTypeEnum DamageModel);
	CHAOS_API void SetUseMaterialDamageModifiers_External(bool bUseMaterialDamageModifiers);
	CHAOS_API void SetMaterialOverrideMassScaleMultiplier_External(float InMultiplier);
	CHAOS_API void SetEnableGravity_External(bool EnableGravity_External);
	CHAOS_API void SetGravityGroupIndex_External(int32 GravityGroupIndex);
	CHAOS_API void SetOneWayInteractionLevel_External(int32 OneWayInteractionLevel);
	CHAOS_API void SetPhysicsMaterial_External(const Chaos::FMaterialHandle& MaterialHandle);
	/** 
	 * Traverses the parents of TransformGroupIdx counting number of levels,
	 * and sets levels array value for TransformGroupIdx and its parents if not yet initialized.
	 * If level is already set, retrieve stored level.
	 * Uninitialized level array should be correct size and defaulted to zeros.
	 */
	static CHAOS_API int32 CalculateAndSetLevel(int32 TransformGroupIdx, const TManagedArray<int32>& Parent, TManagedArray<int32>& Levels);

	void SetPostPhysicsSyncCallback(TFunction<void()> Callback)
	{
		PostPhysicsSyncCallback = Callback;
	}

	void SetPostParticlesCreatedCallback(TFunction<void()> Callback)
	{
		PostParticlesCreatedCallback = Callback;
	}
	
	CHAOS_API FClusterHandle* GetInitialRootHandle_Internal() const;

	CHAOS_API TArray<Chaos::FPhysicsObjectHandle> GetAllPhysicsObjects() const ;
	CHAOS_API TArray<Chaos::FPhysicsObjectHandle> GetAllPhysicsObjectIncludingNulls() const;
	CHAOS_API Chaos::FPhysicsObjectHandle GetPhysicsObjectByIndex(int32 Index) const;
	CHAOS_API void RebaseAllGameThreadCollectionTransformsOnNewWorldTransform_External();

	UE_DEPRECATED(5.4, "Use GetNumTransforms instead")
	int32 GetNumParticles() const { return NumTransforms; }
	int32 GetNumTransforms() const { return NumTransforms; }

	// todo(chaos): Remove this and move to a cook time approach of the SM data based on the GC property
	using FCreateTraceCollisionGeometryCallback = TFunction<void(const FTransform& InToLocal, TArray<Chaos::FImplicitObjectPtr>& OutGeoms, Chaos::FShapesArray& OutShapes)>;
	void SetCreateTraceCollisionGeometryCallback(FCreateTraceCollisionGeometryCallback InCreateGeometryCallback) { CreateTraceCollisionGeometryCallback = InCreateGeometryCallback; }

	CHAOS_API void CreateChildrenGeometry_Internal();

	int32 GetFromParticleToTransformIndex(int32 Index) const { check(FromParticleToTransformIndex.IsValidIndex(Index));  return FromParticleToTransformIndex[Index]; }

	bool GetSkipChildToParentUpdateWhenInClusterUnion() const { return bSkipChildToParentUpdateWhenInClusterUnion; };
	void SetSkipChildToParentUpdateWhenInClusterUnion(bool bValue) { bSkipChildToParentUpdateWhenInClusterUnion = bValue; };

protected:

	bool RebaseParticleGameThreadCollectionTransformOnNewWorldTransform_External(int32 ParticleIndex, const TManagedArray<FTransform>& MassToLocal, bool bIsComponentTransformScaled, const FTransform& ComponentScaleTransform);

	CHAOS_API float ComputeMaterialBasedDamageThreshold_Internal(int32 TransformIndex) const;

	/**
	* Compute user defined damage threshold for a specific transform
	* this account for component level damage threshold as well as size specific ones
	* @param DynamicCollection dynamic collection to use
	* @param TransformIndex index of the transform to compute the threshold for
	* #return damage threshold value
	*/
	CHAOS_API float ComputeUserDefinedDamageThreshold_Internal(int32 TransformIndex) const;

	/** adjust scalar mass to account for per component scale properties ( from material override and world transform scale ) */ 
	CHAOS_API float AdjustMassForScale(float Mass) const;

	/** adjust inertia to account for per component scale properties ( from material override and world transform scale ) */
	CHAOS_API Chaos::FVec3f AdjustInertiaForScale(const Chaos::FVec3f& Inertia) const;

	CHAOS_API Chaos::TPBDGeometryCollectionParticleHandle<Chaos::FReal, 3>* BuildNonClusters_Internal(const uint32 CollectionClusterIndex, Chaos::FPBDRigidsSolver* RigidsSolver, float Mass, Chaos::FVec3f Inertia, const Chaos::FUniqueIdx* ExistingIndex);

	/**
	 * Build a physics thread cluster parent particle.
	 *	\p CollectionClusterIndex - the source geometry collection transform index.
	 *	\p ChildHandles - physics particle handles of the cluster children.
	 *  \p ChildTransformGroupIndices - geometry collection indices of the children.
	 *  \P Parameters - uh, yeah...  Other parameters.
	 */

	CHAOS_API Chaos::FPBDRigidClusteredParticleHandle* BuildClusters_Internal(
		const uint32 CollectionClusterIndex, 
		TArray<Chaos::FPBDRigidParticleHandle*>& ChildHandles,
		const TArray<int32>& ChildTransformGroupIndices,
		const Chaos::FClusterCreationParameters & Parameters,
		const Chaos::FUniqueIdx* ExistingIndex);

	CHAOS_API void SetSleepingState(const Chaos::FPBDRigidsSolver& RigidsSolver);
	CHAOS_API void DirtyAllParticles(const Chaos::FPBDRigidsSolver& RigidsSolver);

	/** 
	 * Traverses the parents of \p TransformIndex in \p GeometryCollection, counting
	 * the number of levels until the next parent is \c INDEX_NONE.
	 */
	static CHAOS_API int32 CalculateHierarchyLevel(const FGeometryDynamicCollection& DynamicCollection, int32 TransformIndex);

	CHAOS_API void SetClusteredParticleKinematicTarget_Internal(Chaos::FPBDRigidClusteredParticleHandle* Handle, const FTransform& WorldTransform);

	CHAOS_API void PrepareBufferData(Chaos::FDirtyGeometryCollectionData& BufferData, const FGeometryDynamicCollection& ThreadCollection,  Chaos::FReal SolverLastDt = 0.0);

	CHAOS_API void CreateNonClusteredParticles(Chaos::FPBDRigidsSolver* RigidsSolver,	const FGeometryCollection& RestCollection, const FGeometryDynamicCollection& DynamicCollection);

	CHAOS_API Chaos::FPBDRigidClusteredParticleHandle* FindClusteredParticleHandleByItemIndex_Internal(FGeometryCollectionItemIndex ItemIndex) const;

	CHAOS_API void UpdateDamageThreshold_Internal();

	/** Scale the cluster particles geometry (creates if necessary an additional TImplicitObjectScaled object into the implicits hierarchy) */
	CHAOS_API void ScaleClusterGeometry_Internal(const FVector& WorldScale);

	CHAOS_API void SetWorldTransform_Internal(const FTransform& WorldTransform, bool bInSkipChildToParentUpdateWhenInClusterUnion = false);
	UE_DEPRECATED(5.7, "Use the overload that takes a FCombinedShapeFilterData instead") CHAOS_API void SetFilterData_Internal(const FCollisionFilterData& NewSimFilter, const FCollisionFilterData& NewQueryFilter);
	CHAOS_API void SetFilterData_Internal(const Chaos::Filter::FCombinedShapeFilterData& NewShapeFilter);
	CHAOS_API void SetPerParticleFilterData_Internal(const TArray<FParticleCollisionFilterData>& PerParticleData);
	CHAOS_API void SetDamagePropagationData_Internal(bool bEnabled, float BreakDamagePropagationFactor, float ShockDamagePropagationFactor);
	CHAOS_API void SetEnableGravity_Internal(bool bEnabled);
	CHAOS_API void SetDamageThresholds_Internal(const TArray<float>& DamageThresholds);
	CHAOS_API void SetDamageModel_Internal(EDamageModelTypeEnum DamageModel);
	CHAOS_API void SetUseMaterialDamageModifiers_Internal(bool bUseMaterialDamageModifiers);
	CHAOS_API void SetMaterialOverrideMassScaleMultiplier_Internal(float InMultiplier);
	CHAOS_API void SetGravityGroupIndex_Internal(int32 GravityGroupIndex);
	CHAOS_API void SetOneWayInteractionLevel_Internal(int32 InOneWayInteractionLevel);
	CHAOS_API void SetPhysicsMaterial_Internal(const Chaos::FMaterialHandle& MaterialHandle);
	CHAOS_API void SetLinearDamping_Internal(float InDamping);
	CHAOS_API void SetLinearDampingPerTransform_Internal(const TArray<int32>& TransformIndices, float InDamping);
	CHAOS_API void SetAngularDamping_Internal(float InDamping);
	CHAOS_API void SetAngularDampingPerTransform_Internal(const TArray<int32>& TransformIndices ,float InDamping);

private:

	static TBitArray<> CalculateClustersToCreateFromChildren(const FGeometryDynamicCollection& DynamicCollection, int32 NumTransforms);
	static int32 CalculateEffectiveParticles(const FGeometryDynamicCollection& DynamicCollection, int32 NumTransform, int32 MaxSimulatedLevel, bool bEnableClustering, const UObject* Owner, TBitArray<>& EffectiveParticles);

	void CreateGTParticles(TManagedArray<Chaos::FImplicitObjectPtr>& Implicits, Chaos::FPBDRigidsEvolutionBase* Evolution, bool bInitializeRootOnly);
	void CreateChildrenGeometry_External();
	void SyncParticles_External();
	
	/**
	 * Since geometry collections only buffer data that has changed, when PullFromPhysicsState is given both PrevData and NextData it must
	 * examine *both* PrevData and NextData for data about a particle (since that particle's data coudl be in PrevData and not NextData).
	 * PullNonInterpolatableDataFromSinglePhysicsState will do the work to pull the non-interpolatable data (which could include X/R/V/W in certain scenarios)
	 * to the game thread.
	 * 
	 * @param BufferData The buffered physics data to pull data from (could be PrevData or NextData).
	 * @param bForcePullXRVW Whether or not to pull interpolatable data as non-interpolatable data (i.e. X/R/V/W). This happens only when NextData doesn't exist.
	 * @param Seen The bit array of the previously handled FDirtyGeometryCollectionData (should be null in the case of handling NextData, and should be the TBitArray on NextData when handling PrevData).
	 * @return A boolean indicating whether or not a change to the GT state was detected.
	 */
	bool PullNonInterpolatableDataFromSinglePhysicsState(const Chaos::FDirtyGeometryCollectionData& BufferData, bool bForcePullXRVW, const TBitArray<>* Seen);

	FSimulationParameters Parameters;

	TArray<Chaos::FPhysicsObjectUniquePtr> PhysicsObjects;

	// todo : we should probably keep a simulation parameter copy on the game thread instead 
	FTransform WorldTransform_External;
	FTransform PreviousWorldTransform_External;

	//
	//  Proxy State Information
	//
	int32 NumTransforms;
	int32 NumEffectiveParticles;
	int32 BaseParticleIndex;
	// Per object collision fraction.
	float CollisionParticlesPerObjectFraction;

	/** structure that contains the necessary information for processing fields */
	struct FFieldData
	{
		/** field command to execute */
		TArray<FFieldSystemCommand> Commands;

		/** Field Datas stored during evaluation */
		FFieldExecutionDatas ExecutionDatas;
	};
	// data is allocated on demand on the physics thread only ( see GetOrCreateFieldData_Internal )
	TUniquePtr<FFieldData> FieldData_Internal;

	FFieldData& GetOrCreateFieldData_Internal();

	EReplicationMode ReplicationMode = EReplicationMode::Unknown;

	uint8 bIsGameThreadWorldTransformDirty : 1;
	uint8 bHasBuiltGeometryOnPT : 1;
	uint8 bHasBuiltGeometryOnGT : 1;
	/* set to true once InitializeBodiesPT has been called*/
	bool bIsInitializedOnPhysicsThread : 1 = false;
	//
	// Buffer Results State Information
	//
	bool IsObjectDynamic : 1; // Records current dynamic state
	bool IsObjectLoading : 1; // Indicate when loaded
	bool IsObjectDeleting : 1; // Indicate when pending deletion

	/** when true and part of a cluster union this will skip the Child to parent update part */
	bool bSkipChildToParentUpdateWhenInClusterUnion : 1; 

	TArray<FParticleHandle*> SolverClusterID;
	TArray<FClusterHandle*> SolverClusterHandles; // make a TArray of the base clase with type
	TArray<FClusterHandle*> SolverParticleHandles;// make a TArray of base class and join with above
	TMap<FParticleHandle*, int32> HandleToTransformGroupIndex;
	TMap<int32, FClusterHandle*> UniqueIdxToInternalClusterHandle;
	TArray<Chaos::FUniqueIdx> UniqueIdxs;
	TArray<int32> FromParticleToTransformIndex;
	TArray<int32> FromTransformToParticleIndex;
	TBitArray<> EffectiveParticles;
	TBitArray<> ParticlesToDirty;

	// Game thread particles 
	TArray<TUniquePtr<FParticle>> GTParticles;

	struct FGTParticleIndices
	{
		/** Correponding transform index of the GTparticle */
		int32 TransformGroupIndex = INDEX_NONE;

		/** unique index of the parent the GT particle is a child of internal cluster  */
		int32 InternalClusterUniqueId = INDEX_NONE;
	};

	TMap<FParticle*, FGTParticleIndices> GTParticleToIndices;
	TMap<int32, TArray<int32>> InternalClusterUniqueIdxToChildrenTransformIndices;

	// These are read on both threads and should not be changed
	Chaos::Filter::FShapeFilterData ShapeFilterData;
	Chaos::Filter::FInstanceData ShapeInstanceData;

	// todo(chaos): Remove this and move to a cook time approach of the SM data based on the GC property
	FCreateTraceCollisionGeometryCallback CreateTraceCollisionGeometryCallback;
	
	// called after we sync the physics thread data ( called on the game thread )
	TFunction<void()> PostPhysicsSyncCallback;
	TFunction<void()> PostParticlesCreatedCallback;
	
	// The Simulation data is copied between the game and physics thread. It is 
	// expected that the two data sets will diverge, based on how the simulation
	// uses the data, but at the start of the simulation the PhysicsThreadCollection
	// is a deep copy from the GameThreadCollection. 
	FGeometryDynamicCollection PhysicsThreadCollection;
	FGeometryDynamicCollection& GameThreadCollection; // now initialize from the SharedRef passed in the constructor 
	const TSharedPtr<FGeometryDynamicCollection> GameThreadCollectionSharedPtr; // shoul dnever be null

	// Currently this is using triple buffers for game-physics and 
	// physics-game thread communication, but not for any reason other than this 
	// is the only implementation we currently have of a guarded buffer - a buffer 
	// that tracks it's own state, rather than having other mechanisms determine 
	// whether or not the contents of the buffer have been updated.  A double 
	// buffer would probably be fine, as that seems to be the assumption the logic
	// currently managing the exchange is built upon.  However, I believe that 
	// logic locks, and the triple buffer would enable a decoupled lock-free 
	// paradigm, at least for this component of the handshake.
	Chaos::FGuardedTripleBuffer<FGeometryCollectionResults> PhysToGameInterchange;

	TUniquePtr<FProxyInterpolationBase> InterpolationData;

public:
	/** Get or create a derived FProxyInterpolationBase that handles render interpolation error corrections */
	template<typename ErrorDataType>
	ErrorDataType* GetOrCreateErrorInterpolationData()
	{
		if (!InterpolationData.IsValid())
		{
			InterpolationData = MakeUnique<ErrorDataType>();
		}
		else if (InterpolationData.Get()->GetInterpolationType() != ErrorDataType::InterpolationType)
		{
			InterpolationData = MakeUnique<ErrorDataType>(InterpolationData.Get()->GetPullDataInterpIdx_External(), InterpolationData.Get()->GetInterpChannel_External());
		}

		return static_cast<ErrorDataType*>(InterpolationData.Get());
	}
	
	template<typename ErrorDataType>
	const ErrorDataType* GetErrorInterpolationData() const
	{
		return static_cast<ErrorDataType*>(InterpolationData.Get());
	}

private:
#if WITH_EDITORONLY_DATA
	// this is used as a unique ID when collecting data from runtime
	FGuid CollectorGuid;
#endif
};

/**
 * the watcher collects runtime data about
 * damage on each piece of the geometry collection 
 */
struct FDamageCollector
{
public:
	struct FDamageData
	{
		float DamageThreshold= 0;
		float MaxDamages = 0;
		bool  bIsBroken = false;
	};

	CHAOS_API void Reset(int32 NumTransforms);
	int32 Num() const { return DamageData.Num(); }
	const FDamageData& operator[](int32 TransformIndex) const 
	{ 
		static FDamageData DefaultDamageData;
		if (DamageData.IsValidIndex(TransformIndex))
			return DamageData[TransformIndex]; 
		return DefaultDamageData;
	}
	CHAOS_API void SampleDamage(int32 TransformIndex, float Damage, float DamageThreshold);
	
private:
	TArray<FDamageData> DamageData;
};

struct FRuntimeDataCollector
{
public:
	CHAOS_API void Clear();
	CHAOS_API void AddCollector(const FGuid& Guid, int32 TransformNum);
	CHAOS_API void RemoveCollector(const FGuid& Guid);

	CHAOS_API FDamageCollector* Find(const FGuid& Guid);

	static CHAOS_API FRuntimeDataCollector& GetInstance();
	
private:
	// collectors by geometry collection Guids
	TMap<FGuid,FDamageCollector> Collectors;
};

CHAOS_API TUniquePtr<Chaos::FTriangleMesh> CreateTriangleMesh(const int32 FaceStart,const int32 FaceCount,const TManagedArray<bool>& Visible,const TManagedArray<FIntVector>& Indices, bool bRotateWinding = true);
CHAOS_API void BuildSimulationData(Chaos::FErrorReporter& ErrorReporter, FGeometryCollection& GeometryCollection, const FSharedSimulationParameters& SharedParams);
