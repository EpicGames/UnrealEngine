// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/Framework/MultiBufferResource.h"
#include "Chaos/Framework/PhysicsProxy.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "Chaos/ParticleHandle.h"
#include "PhysicsCoreTypes.h"
#include "Chaos/Defines.h"
#include "JointConstraintProxyFwd.h"
#include "Framework/Threading.h"
#include "Chaos/PBDJointConstraintData.h"
#include "RewindData.h"

namespace Chaos
{
class FJointConstraint;

class FPBDRigidsEvolutionGBF;

struct FDirtyJointConstraintData;

template <bool bExternal>
class TThreadedJointConstraintPhysicsProxyBase;

using FJointConstraintHandle_External = TThreadedJointConstraintPhysicsProxyBase<true>;
using FJointConstraintHandle_Internal = TThreadedJointConstraintPhysicsProxyBase<false>;

class FJointConstraintPhysicsProxy : public IPhysicsProxyBase
{
	using Base = IPhysicsProxyBase;

public:
	FJointConstraintPhysicsProxy() = delete;
	FJointConstraintPhysicsProxy(FJointConstraint* InConstraint, FPBDJointConstraintHandle* InHandle, UObject* InOwner = nullptr);

	static FGeometryParticleHandle* GetParticleHandleFromProxy(IPhysicsProxyBase* ProxyBase);

	//
	//  Lifespan Management
	//

	void CHAOS_API InitializeOnPhysicsThread(FPBDRigidsSolver* InSolver, FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData);

	void CHAOS_API PushStateOnGameThread(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData);

	void CHAOS_API PushStateOnPhysicsThread(FPBDRigidsSolver* InSolver, const FDirtyPropertiesManager& Manager, int32 DataIdx, const FDirtyChaosProperties& RemoteData);

	void CHAOS_API DestroyOnPhysicsThread(FPBDRigidsSolver* InSolver);

	void CHAOS_API DestroyOnGameThread();

	FORCEINLINE Chaos::FJointConstraintHandle_External& GetGameThreadAPI()
	{
		return (Chaos::FJointConstraintHandle_External&)*this;
	}

	FORCEINLINE const Chaos::FJointConstraintHandle_External& GetGameThreadAPI() const
	{
		return (const Chaos::FJointConstraintHandle_External&)*this;
	}

	//Note this is a pointer because the internal handle may have already been deleted
	FORCEINLINE Chaos::FJointConstraintHandle_Internal* GetPhysicsThreadAPI()
	{
		return GetHandle() == nullptr ? nullptr : (Chaos::FJointConstraintHandle_Internal*)this;
	}

	//Note this is a pointer because the internal handle may have already been deleted
	FORCEINLINE const Chaos::FJointConstraintHandle_Internal* GetPhysicsThreadAPI() const
	{
		return GetHandle() == nullptr ? nullptr : (const Chaos::FJointConstraintHandle_Internal*)this;
	}


	//
	// Member Access
	//

	FPBDJointConstraintHandle* GetHandle() { return Constraint_PT; }
	const FPBDJointConstraintHandle* GetHandle() const { return Constraint_PT; }

	virtual void* GetHandleUnsafe() const override { return Constraint_PT; }

	void SetHandle(FPBDJointConstraintHandle* InHandle)	{ Constraint_PT = InHandle; }

	FJointConstraint* GetConstraint(){ return Constraint_GT; }
	const FJointConstraint* GetConstraint() const { return Constraint_GT; }

	//
	// Threading API
	//
	
	/**/
	void BufferPhysicsResults(FDirtyJointConstraintData& Buffer);

	/**/
	bool CHAOS_API PullFromPhysicsState(const FDirtyJointConstraintData& Buffer, const int32 SolverSyncTimestamp);
	
private:
	FJointConstraint* Constraint_GT;
	FPBDJointConstraintHandle* Constraint_PT;
	FParticlePair OriginalParticleHandles_PT;
	bool bInitialized = false;
};


/** Wrapper class that routes all reads and writes to the appropriate joint data. This is helpful for cases where we want to both write to a joint and a network buffer for example*/
template <bool bExternal>
class TThreadedJointConstraintPhysicsProxyBase : protected FJointConstraintPhysicsProxy
{
public:

#define CHAOS_INNER_JOINT_PROPERTY(OuterProp, FuncName, Inner, InnerType)\
	const InnerType& Get##FuncName() const { return ReadRef([](const auto& Data){ return Data.Inner;}); }\
	void Set##FuncName(const InnerType& Val) { Write([&Val](auto& Data){ Data.Inner = Val;}); }\

#include "Chaos/JointProperties.inl"

private:

	void VerifyContext() const
	{
#if PHYSICS_THREAD_CONTEXT
		//Are you using the wrong API type for the thread this code runs in?
		//GetGameThreadAPI should be used for gamethread, GetPhysicsThreadAPI should be used for callbacks and internal physics thread
		//Note if you are using a ParallelFor you must use PhysicsParallelFor to ensure the right context is inherited from parent thread
		if (bExternal)
		{
			//if proxy is registered with solver, we need a lock
			if (GetSolverBase() != nullptr)
			{
				ensure(IsInGameThreadContext());
			}
		}
		else
		{
			ensure(IsInPhysicsThreadContext());
		}
#endif
	}

	template <typename TLambda>
	const auto& ReadRef(const TLambda& Lambda) const { VerifyContext(); return bExternal ? Lambda(GetConstraint()) : Lambda(GetHandle()); }

	template <typename TLambda>
	void Write(const TLambda& Lambda)
	{
		VerifyContext();
		if (bExternal)
		{
			Lambda(GetConstraint());
		}
		else
		{
			//Mark entire joint as dirty from PT. TODO: use property system
			FPhysicsSolverBase* SolverBase = GetSolverBase();	//internal so must have solver already
			if (FRewindData* RewindData = SolverBase->GetRewindData())
			{
				RewindData->MarkDirtyJointFromPT(*GetHandle());
			}

			Lambda(GetHandle());
		}
	}
};

}