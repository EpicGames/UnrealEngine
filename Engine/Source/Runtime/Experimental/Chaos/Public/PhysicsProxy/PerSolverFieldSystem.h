// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Field/FieldSystem.h"
#include "Field/FieldSystemTypes.h"
#include "Chaos/PBDPositionConstraints.h"
#include "Chaos/Defines.h"

class FPerSolverFieldSystem
{
public:

	/**
	 * Services queued \c FFieldSystemCommand commands.
	 *
	 * Supported fields:
	 * EFieldPhysicsType::Field_DynamicState
	 * EFieldPhysicsType::Field_ActivateDisabled
	 * EFieldPhysicsType::Field_ExternalClusterStrain (clustering)
	 * EFieldPhysicsType::Field_Kill
	 * EFieldPhysicsType::Field_LinearVelocity
	 * EFieldPhysicsType::Field_AngularVelociy
	 * EFieldPhysicsType::Field_SleepingThreshold
	 * EFieldPhysicsType::Field_DisableThreshold
	 * EFieldPhysicsType::Field_InternalClusterStrain (clustering)
	 * EFieldPhysicsType::Field_CollisionGroup
	 * EFieldPhysicsType::Field_PositionStatic
	 * EFieldPhysicsType::Field_PositionTarget
	 * EFieldPhysicsType::Field_PositionAnimated
	 * EFieldPhysicsType::Field_DynamicConstraint
	 */
	CHAOS_API void FieldParameterUpdateCallback(
		Chaos::FPBDRigidsSolver* InSolver, 
		Chaos::FPBDPositionConstraints& PositionTarget,
		TMap<int32, int32>& TargetedParticles);

	/**
	 * Services queued \c FFieldSystemCommand commands.
	 *
	 * Supported fields:
	 * EFieldPhysicsType::Field_LinearForce
	 * EFieldPhysicsType::Field_AngularTorque
	 */
	CHAOS_API void FieldForcesUpdateCallback(
		Chaos::FPBDRigidsSolver* RigidSolver);

	/**
	 * Compute field linear velocity/force and angular velocity/torque given a list of samples (positions + indices)
	 *
	 * Supported fields:
	 * FieldPhysicsType::Field_LinearVelocity
	 * EFieldPhysicsType::Field_LinearForce
	 * EFieldPhysicsType::Field_AngularVelocity
	 * EFieldPhysicsType::Field_AngularrTorque
	 */
	CHAOS_API void ComputeFieldRigidImpulse(const Chaos::FReal SolverTime);

	/**
	 * Compute field linear velocity/force given a list of samples (positions + indices)
	 *
	 * Supported fields:
	 * EFieldPhysicsType::Field_LinearVelocity
	 * EFieldPhysicsType::Field_LinearForce
	 */
	CHAOS_API void ComputeFieldLinearImpulse(const Chaos::FReal SolverTime);

	/** Add the transient field command */
	CHAOS_API void AddTransientCommand(const FFieldSystemCommand& FieldCommand);

	/** Add the persistent field command */
	CHAOS_API void AddPersistentCommand(const FFieldSystemCommand& FieldCommand);

	/** Remove the transient field command */
	CHAOS_API void RemoveTransientCommand(const FFieldSystemCommand& FieldCommand);

	/** Remove the persistent field command */
	CHAOS_API void RemovePersistentCommand(const FFieldSystemCommand& FieldCommand);

	/** Get all the non const transient field commands */
	TArray<FFieldSystemCommand>& GetTransientCommands() { return TransientCommands; }

	/** Get all the const transient field commands */
	const TArray<FFieldSystemCommand>& GetTransientCommands() const { return TransientCommands; }

	/** Get all the non const persistent field commands */
	TArray<FFieldSystemCommand>& GetPersistentCommands() { return PersistentCommands; }

	/** Get all the const persistent field commands */
	const TArray<FFieldSystemCommand>& GetPersistentCommands() const { return PersistentCommands; }

	/**
	 * Generates a mapping between the Position array and the results array. 
	 *
	 * When \p ResolutionType is set to \c Maximum the complete particle mapping 
	 * is provided from the \c Particles.X to \c Particles.Attribute. 
	 * When \c Minimum is set only the ActiveIndices and the direct children of 
	 * the active clusters are set in the \p IndicesArray.
	 */

	static CHAOS_API void GetRelevantParticleHandles(
		TArray<Chaos::FGeometryParticleHandle*>& ParticleHandles,
		const Chaos::FPBDRigidsSolver* RigidSolver,
		const EFieldResolutionType ResolutionType);

	/**
	 * Generates a mapping between the Position array and the results array.
	 *
	 * When \p FilterType is set to \c Active the complete particle mapping
	 * is provided from the \c Particles.X to \c Particles.Attribute.
	 */

	static CHAOS_API void GetFilteredParticleHandles(
		TArray<Chaos::FGeometryParticleHandle*>& ParticleHandles,
		const Chaos::FPBDRigidsSolver* RigidSolver,
		const EFieldFilterType FilterType,
		const EFieldObjectType ObjectType);

	/** Check if a per solver field system has no commands. */
	bool IsEmpty() const { return (TransientCommands.Num() == 0) && (PersistentCommands.Num() == 0); }

	/** Get the non const array of sample positions */
	const TArray<FVector>& GetSamplePositions() const { return ExecutionDatas.SamplePositions; }

	/** Get the const array of sample positions */
	TArray<FVector>& GetSamplePositions() { return ExecutionDatas.SamplePositions; }

	/** Get the const array of sample indices */
	const TArray<FFieldContextIndex>& GetSampleIndices() const { return ExecutionDatas.SampleIndices; }

	/** Get the non const array of sample indices */
	TArray<FFieldContextIndex>& GetSampleIndices() { return ExecutionDatas.SampleIndices; }

	/** Get the non const array of the output results given an output type*/
	TArray<FVector>& GetOutputResults(const EFieldCommandOutputType OutputType) { return ExecutionDatas.FieldOutputs[(uint8)OutputType]; }

	/** Get the const array of the output results given an output type*/
	const TArray<FVector>& GetOutputResults(const EFieldCommandOutputType OutputType) const { return ExecutionDatas.FieldOutputs[(uint8)OutputType]; }

private:

	/** Forces update callback implementation */
	CHAOS_API void FieldForcesUpdateInternal(
		Chaos::FPBDRigidsSolver* RigidSolver,
		TArray<FFieldSystemCommand>& Commands, 
		const bool IsTransient);

	/** Parameter update callback implementation */
	CHAOS_API void FieldParameterUpdateInternal(
		Chaos::FPBDRigidsSolver* RigidSolver,
		Chaos::FPBDPositionConstraints& PositionTarget,
		TMap<int32, int32>& PositionTargetedParticles,
		TArray<FFieldSystemCommand>& Commands, 
		const bool IsTransient);

	/** Field Datas stored during evaluation */
	FFieldExecutionDatas ExecutionDatas;

	/** Transient commands to be processed by the chaos solver */
	TArray<FFieldSystemCommand> TransientCommands;

	/** Persistent commands to be processed by the chaos solver */
	TArray<FFieldSystemCommand> PersistentCommands;
};
