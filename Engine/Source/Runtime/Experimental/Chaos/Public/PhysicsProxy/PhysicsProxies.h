// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** 
 * Convenience header for working with object storage in physics code.
 * When using FPhysicsProxyStorage::ForEachPhysicsProxy a lambda in the 
 * generic for [](auto* Obj) {} is expected which will require all solver
 * objects are fully defined when used, including this header will include
 * the currently supported set of objects.
 */

#include "SingleParticlePhysicsProxy.h"
#include "PerSolverFieldSystem.h"
#include "GeometryCollectionPhysicsProxy.h"
#include "SkeletalMeshPhysicsProxy.h"
#include "StaticMeshPhysicsProxy.h"
