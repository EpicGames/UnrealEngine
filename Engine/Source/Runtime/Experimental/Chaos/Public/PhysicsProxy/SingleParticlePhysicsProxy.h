// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Real.h"
#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/Framework/MultiBufferResource.h"
#include "Chaos/Framework/PhysicsProxy.h"
#include "Chaos/PBDPositionConstraints.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/PhysicsObject.h"
#include "PhysicsCoreTypes.h"
#include "Chaos/Defines.h"
#include "Chaos/PullPhysicsDataImp.h"
#include "Chaos/Core.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"
#include "Framework/Threading.h"
#include "Math/NumericLimits.h"
#include "RewindData.h"


namespace Chaos
{

class FPBDRigidsEvolutionGBF;

struct FDirtyRigidParticleData;

class FInitialState
{
public:
	FInitialState()
	    : Mass(0.f)
	    , InvMass(0.f)
	    , InertiaTensor(1.f)
	{}

	FInitialState(FReal MassIn, FReal InvMassIn, FVector InertiaTensorIn)
	    : Mass(MassIn)
	    , InvMass(InvMassIn)
	    , InertiaTensor(InertiaTensorIn)
	{}

	FReal GetMass() const { return Mass; }
	FReal GetInverseMass() const { return InvMass; }
	FVector GetInertiaTensor() const { return InertiaTensor; }

private:
	FReal Mass;
	FReal InvMass;
	FVector InertiaTensor;
};

class FRigidBodyHandle_External;
class FRigidBodyHandle_Internal;

class FSingleParticlePhysicsProxy : public IPhysicsProxyBase
{
public:
	using PARTICLE_TYPE = FGeometryParticle;
	using FParticleHandle = FGeometryParticleHandle;

	static FSingleParticlePhysicsProxy* Create(TUniquePtr<FGeometryParticle>&& Particle, UObject* InOwner);

	FSingleParticlePhysicsProxy() = delete;
	FSingleParticlePhysicsProxy(const FSingleParticlePhysicsProxy&) = delete;
	FSingleParticlePhysicsProxy(FSingleParticlePhysicsProxy&&) = delete;
	CHAOS_API virtual ~FSingleParticlePhysicsProxy();

	FProxyInterpolationBase* GetInterpolationData() { return InterpolationData.Get(); }
	const FProxyInterpolationBase* GetInterpolationData() const { return InterpolationData.Get(); }

	void ResetInterpolationData()
	{
		InterpolationData.Reset();
	}
	
	FORCEINLINE FRigidBodyHandle_External& GetGameThreadAPI()
	{
		return (FRigidBodyHandle_External&)*this;
	}

	FORCEINLINE const FRigidBodyHandle_External& GetGameThreadAPI() const
	{
		return (const FRigidBodyHandle_External&)*this;
	}

	//Note this is a pointer because the internal handle may have already been deleted
	FORCEINLINE FRigidBodyHandle_Internal* GetPhysicsThreadAPI()
	{
		return GetHandle_LowLevel() == nullptr ? nullptr : (FRigidBodyHandle_Internal*)this;
	}

	//Note this is a pointer because the internal handle may have already been deleted
	FORCEINLINE const FRigidBodyHandle_Internal* GetPhysicsThreadAPI() const
	{
		return GetHandle_LowLevel() == nullptr ? nullptr : (const FRigidBodyHandle_Internal*)this;
	}

	//Returns the underlying physics thread particle. Note this should only be needed for internal book keeping type tasks. API may change, use GetPhysicsThreadAPI instead
	FParticleHandle* GetHandle_LowLevel()
	{
		return Handle;
	}

	//Returns the underlying physics thread particle. Note this should only be needed for internal book keeping type tasks. API may change, use GetPhysicsThreadAPI instead
	const FParticleHandle* GetHandle_LowLevel() const
	{
		return Handle;
	}

	virtual void* GetHandleUnsafe() const override
	{
		return Handle;
	}

	void SetHandle(FParticleHandle* InHandle)
	{
		Handle = InHandle;
	}

	// Threading API

	CHAOS_API void PushToPhysicsState(const FDirtyPropertiesManager& Manager,int32 DataIdx,const FDirtyProxy& Dirty,FShapeDirtyData* ShapesData, FReal ExternalDt);

	/**/
	CHAOS_API void ClearAccumulatedData();

	/**/
	CHAOS_API void BufferPhysicsResults(FDirtyRigidParticleData&);

	/**/
	CHAOS_API void BufferPhysicsResults_External(FDirtyRigidParticleData&);

	/**/
	CHAOS_API bool PullFromPhysicsState(const FDirtyRigidParticleData& PullData, int32 SolverSyncTimestamp, const FDirtyRigidParticleData* NextPullData = nullptr, const FRealSingle* Alpha = nullptr, const FDirtyRigidParticleReplicationErrorData* Error = nullptr, const Chaos::FReal AsyncFixedTimeStep = 0);

	/**/
	CHAOS_API bool IsDirty();


	/**/
	CHAOS_API EWakeEventEntry GetWakeEvent() const;

	/**/
	CHAOS_API void ClearEvents();

	//Returns the underlying game thread particle. Note this should only be needed for internal book keeping type tasks. API may change, use GetGameThreadAPI instead
	PARTICLE_TYPE* GetParticle_LowLevel()
	{
		return Particle.Get();
	}

	//Returns the underlying game thread particle. Note this should only be needed for internal book keeping type tasks. API may change, use GetGameThreadAPI instead
	const PARTICLE_TYPE* GetParticle_LowLevel() const
	{
		return Particle.Get();
	}

	FPBDRigidParticle* GetRigidParticleUnsafe()
	{
		return static_cast<FPBDRigidParticle*>(GetParticle_LowLevel());
	}

	const FPBDRigidParticle* GetRigidParticleUnsafe() const
	{
		return static_cast<const FPBDRigidParticle*>(GetParticle_LowLevel());
	}

	FPhysicsObjectHandle GetPhysicsObject()
	{
		return Reference.Get();
	}

	const FPhysicsObjectHandle GetPhysicsObject() const
	{
		return Reference.Get();
	}

	/** Get or create a derived FProxyInterpolationBase that handles render interpolation error corrections */
	template<typename ErrorDataType>
	ErrorDataType* GetOrCreateErrorInterpolationData()
	{
		if (!InterpolationData.IsValid())
		{
			InterpolationData = MakeUnique<ErrorDataType>();
		}
		else if (InterpolationData.Get()->GetInterpolationType() != ErrorDataType::InterpolationType)
		{
			InterpolationData = MakeUnique<ErrorDataType>(InterpolationData.Get()->GetPullDataInterpIdx_External(), InterpolationData.Get()->GetInterpChannel_External());
		}

		return static_cast<ErrorDataType*>(InterpolationData.Get());
	}
	
	template<typename ErrorDataType>
	const ErrorDataType* GetErrorInterpolationData() const
	{
		return static_cast<ErrorDataType*>(InterpolationData.Get());
	}

protected:
	TUniquePtr<PARTICLE_TYPE> Particle;
	FParticleHandle* Handle;
	FPhysicsObjectUniquePtr Reference;

	int32 GravityGroupIndex;

private:
	TUniquePtr<FProxyInterpolationBase> InterpolationData;

	//use static Create
	CHAOS_API FSingleParticlePhysicsProxy(TUniquePtr<PARTICLE_TYPE>&& InParticle, FParticleHandle* InHandle, UObject* InOwner = nullptr);
};

/** Wrapper class that routes all reads and writes to the appropriate particle data. This is helpful for cases where we want to both write to a particle and a network buffer for example*/
template <bool bExternal>
class TThreadedSingleParticlePhysicsProxyBase : protected FSingleParticlePhysicsProxy
{
	TThreadedSingleParticlePhysicsProxyBase() = delete;	//You should only ever new FSingleParticlePhysicsProxy, derived types are simply there for API constraining, no new data
public:

	FSingleParticlePhysicsProxy* GetProxy() { return static_cast<FSingleParticlePhysicsProxy*>(this); }

	bool CanTreatAsKinematic() const
	{
		return Read([](auto* Particle) { return Particle->CastToKinematicParticle() != nullptr; });
	}

	bool CanTreatAsRigid() const
	{
		return Read([](auto* Particle) { return Particle->CastToRigidParticle() != nullptr; });
	}

	//API for static particle
	const FVec3& X() const { return ReadRef([](auto* Particle) -> const auto& { return Particle->GetX(); }); }
	const FVec3& GetX() const { return ReadRef([](auto* Particle) -> const auto& { return Particle->GetX(); }); }

protected:
	void SetXBase(const FVec3& InX, bool bInvalidate = true) { Write([&InX, bInvalidate, this](auto* Particle)
	{
		if (bInvalidate)
		{
			auto Dyn = Particle->CastToRigidParticle();
			if (Dyn && Dyn->ObjectState() == EObjectStateType::Sleeping)
			{
				SetObjectStateHelper(*GetProxy(), *Dyn, EObjectStateType::Dynamic, true);
			}
		}
		Particle->SetX(InX, bInvalidate);
	});}
public:

	FUniqueIdx UniqueIdx() const { return Read([](auto* Particle) { return Particle->UniqueIdx(); }); }
	void SetUniqueIdx(const FUniqueIdx UniqueIdx, bool bInvalidate = true) { Write([UniqueIdx, bInvalidate](auto* Particle) { Particle->SetUniqueIdx(UniqueIdx, bInvalidate); }); }

	FRotation3 R() const { return Read([](auto* Particle) -> const auto { return Particle->GetR(); }); }
	FRotation3 GetR() const { return Read([](auto* Particle) -> const auto { return Particle->GetR(); }); }

protected:
	void SetRBase(const FRotation3& InR, bool bInvalidate = true){ Write([&InR, bInvalidate, this](auto* Particle)
	{
			if (bInvalidate)
			{
				auto Dyn = Particle->CastToRigidParticle();
				if (Dyn && Dyn->ObjectState() == EObjectStateType::Sleeping)
				{
					SetObjectStateHelper(*GetProxy(), *Dyn, EObjectStateType::Dynamic, true);
				}
			}
			Particle->SetR(InR, bInvalidate);
	});}
public:
	
	UE_DEPRECATED(5.4, "Use GetGeometry instead.")
	const TSharedPtr<FImplicitObject, ESPMode::ThreadSafe>& SharedGeometryLowLevel() const
	{
		check(false);
		static TSharedPtr<FImplicitObject, ESPMode::ThreadSafe> DummyPtr(nullptr);
		return DummyPtr;
	}

	
#if CHAOS_DEBUG_NAME
	const TSharedPtr<FString, ESPMode::ThreadSafe>& DebugName() const { return ReadRef([](auto* Ptr) -> const auto& { return Ptr->DebugName(); }); }
	void SetDebugName(const TSharedPtr<FString, ESPMode::ThreadSafe>& InDebugName) { Write([&InDebugName](auto* Ptr) { Ptr->SetDebugName(InDebugName); }); }
#endif

	const FImplicitObjectRef GetGeometry() const { return Read([](auto* Ptr) { FImplicitObjectRef ImplicitRef = Ptr->GetGeometry(); return ImplicitRef; }); }

	UE_DEPRECATED(5.4, "Please use GetGeometry instead")
	TSerializablePtr<FImplicitObject> Geometry() const { check(false); return TSerializablePtr<FImplicitObject>(); }

	const FShapesArray& ShapesArray() const { return ReadRef([](auto* Ptr) -> const auto& { return Ptr->ShapesArray(); }); }

	EObjectStateType ObjectState() const { return Read([](auto* Ptr) { return Ptr->ObjectState(); }); }

	EParticleType ObjectType() const { return Read([](auto* Ptr) { return Ptr->ObjectType(); }); }

	FSpatialAccelerationIdx SpatialIdx() const { return Read([](auto* Ptr) { return Ptr->SpatialIdx(); }); }
	void SetSpatialIdx(FSpatialAccelerationIdx Idx) { Write([Idx](auto* Ptr) { Ptr->SetSpatialIdx(Idx); }); }

	//API for kinematic particle
	const FVec3 V() const
	{
		return Read([](auto* Particle)
		{
			if (auto Kinematic = Particle->CastToKinematicParticle())
			{
				return Kinematic->GetV();
			}
			
			return FVec3(0);
		});
	}
	const FVec3 GetV() const
	{
		return V();
	}

protected:
	void SetVBase(const FVec3& InV, bool bInvalidate = true)
	{
		Write([&InV, bInvalidate, this](auto* Particle)
		{
			if (auto Kinematic = Particle->CastToKinematicParticle())
			{
				if (bInvalidate)
				{
					auto Dyn = Particle->CastToRigidParticle();
					if (Dyn && Dyn->ObjectState() == EObjectStateType::Sleeping && !InV.IsNearlyZero())
					{
						SetObjectStateHelper(*GetProxy(), *Dyn, EObjectStateType::Dynamic, true);
					}
				}

				Kinematic->SetV(InV, bInvalidate);
			}
		});
	}
public:

	const FVec3 W() const
	{
		return Read([](auto* Particle)
		{
			if (auto Kinematic = Particle->CastToKinematicParticle())
			{
				return Kinematic->GetW();
			}

			return FVec3(0);
		});
	}

	const FVec3 GetW() const
	{
		return W();
	}

protected:
	void SetWBase(const FVec3& InW, bool bInvalidate = true)
	{
		Write([&InW, bInvalidate, this](auto* Particle)
		{
			if (auto Kinematic = Particle->CastToKinematicParticle())
			{
				if (bInvalidate)
				{
					auto Dyn = Particle->CastToRigidParticle();
					if (Dyn && Dyn->ObjectState() == EObjectStateType::Sleeping && !InW.IsNearlyZero())
					{
						SetObjectStateHelper(*GetProxy(), *Dyn, EObjectStateType::Dynamic, true);
					}
				}
				
				Kinematic->SetW(InW, bInvalidate);
			}
		});
	}
public:

	void SetKinematicTarget(const FRigidTransform3& InTargetTransform, bool bInvalidate = true)
	{
		SetKinematicTarget(FKinematicTarget::MakePositionTarget(InTargetTransform), bInvalidate);
	}

	void SetKinematicTarget(const FKinematicTarget& InKinematicTarget, bool bInvalidate = true)
	{
		Write([&InKinematicTarget, bInvalidate](auto* Ptr)
		{
			if (auto Kinematic = Ptr->CastToKinematicParticle())
			{
				Kinematic->SetKinematicTarget(InKinematicTarget, bInvalidate);
			}
		});
	}
	
	//API for dynamic particle

	bool GravityEnabled() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->GravityEnabled();
			}

			return false;
		});
	}

	void SetGravityEnabled(const bool InGravityEnabled)
	{
		Write([InGravityEnabled](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->SetGravityEnabled(InGravityEnabled);
			}
		});
	}

	int32 GravityGroupIndex() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->GravityGroupIndex();
			}

			return -1;
		});
	}

	void SetGravityGroupIndex(const uint32 InGravityGroupIndex)
	{
		Write([InGravityGroupIndex](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->SetGravityGroupIndex(InGravityGroupIndex);
				}
			});
	}

	bool UpdateKinematicFromSimulation() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->UpdateKinematicFromSimulation();
			}

			return false;
		});
	}

	void SetUpdateKinematicFromSimulation(const bool InUpdateKinematicFromSimulation)
	{
		Write([InUpdateKinematicFromSimulation](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->SetUpdateKinematicFromSimulation(InUpdateKinematicFromSimulation);
			}
		});
	}

	bool GyroscopicTorqueEnabled() const
	{
		return Read([](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->GyroscopicTorqueEnabled();
				}

				return false;
			});
	}

	void SetGyroscopicTorqueEnabled(const bool InGyroscopicTorqueEnabled)
	{
		Write([InGyroscopicTorqueEnabled](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->SetGyroscopicTorqueEnabled(InGyroscopicTorqueEnabled);
				}
			});
	}

	bool CCDEnabled() const
	{
		return Read([](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->CCDEnabled();
				}

				return false;
			});
	}

	void SetCCDEnabled(const bool InCCDEnabled)
	{
		Write([InCCDEnabled](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->SetCCDEnabled(InCCDEnabled);
				}
			});
	}

	bool MACDEnabled() const
	{
		return Read([](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->MACDEnabled();
				}

				return false;
			});
	}

	void SetMACDEnabled(const bool InMACDEnabled)
	{
		Write([InMACDEnabled](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->SetMACDEnabled(InMACDEnabled);
				}
			});
	}

	bool PartialIslandSleepAllowed() const
	{
		return Read([](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->PartialIslandSleepAllowed();
				}

				return false;
			});
	}

	void SetPartialIslandSleepAllowed(const bool InPartialIslandSleepAllowed)
	{
		Write([InPartialIslandSleepAllowed](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->SetPartialIslandSleepAllowed(InPartialIslandSleepAllowed);
				}
			});
	}

	void SetPositionSolverIterations(const int32 PositionSolverIterationsIn)
	{
		Write([PositionSolverIterationsIn](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->SetPositionSolverIterations(PositionSolverIterationsIn);
				}
			});
	}

	void SetVelocitySolverIterations(const int32 VelocitySolverIterationsIn)
	{
		Write([VelocitySolverIterationsIn](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->SetVelocitySolverIterations(VelocitySolverIterationsIn);
				}
			});
	}

	void SetProjectionSolverIterations(const int32 ProjectionSolverIterationsIn)
	{
		Write([ProjectionSolverIterationsIn](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->SetProjectionSolverIterations(ProjectionSolverIterationsIn);
				}
			});
	}

	bool OneWayInteraction() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->OneWayInteraction();
			}

			return false;
		});
	}

	void SetOneWayInteraction(const bool bInOneWayInteraction)
	{
		Write([bInOneWayInteraction](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->SetOneWayInteraction(bInOneWayInteraction);
			}
		});
	}

	bool InertiaConditioningEnabled() const
	{
		return Read([](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->InertiaConditioningEnabled();
				}

				return false;
			});
	}

	void SetInertiaConditioningEnabled(const bool bInEnabled)
	{
		Write([bInEnabled](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->SetInertiaConditioningEnabled(bInEnabled);
				}
			});
	}

	EResimType ResimType() const
	{
		return Read([](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					return Rigid->ResimType();
				}

				return EResimType::FrozenDuringResim;
			});
	}

	void SetResimType(EResimType ResimType)
	{
		Write([ResimType](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->SetResimType(ResimType);
			}
		});
	}

	const FVec3 Acceleration() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->Acceleration();
			}

			return FVec3(0);
		});
	}
	
	void SetAcceleration(const FVec3& Acceleration, bool bInvalidate = true)
	{
		Write([&Acceleration, bInvalidate, this](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					if (Rigid->ObjectState() == EObjectStateType::Sleeping || Rigid->ObjectState() == EObjectStateType::Dynamic)
					{
						if (bInvalidate)
						{
							SetObjectStateHelper(*GetProxy(), *Rigid, EObjectStateType::Dynamic, true);
						}

						Rigid->SetAcceleration(Acceleration);
					}
				}
			});
	}

	void AddForce(const FVec3& InForce, bool bInvalidate = true)
	{
		Write([&InForce, bInvalidate, this](auto* Particle)
		{
			if (auto* Rigid = Particle->CastToRigidParticle())
			{
				if (Rigid->ObjectState() == EObjectStateType::Sleeping || Rigid->ObjectState() == EObjectStateType::Dynamic)
				{
					if (bInvalidate)
					{
						SetObjectStateHelper(*GetProxy(), *Rigid, EObjectStateType::Dynamic, true);
					}

					Rigid->AddForce(InForce, bInvalidate);
				}
			}
		});
	}

	void SetAngularAcceleration(const FVec3& AngularAcceleration, bool bInvalidate = true)
	{
		Write([&AngularAcceleration, bInvalidate, this](auto* Particle)
			{
				if (auto Rigid = Particle->CastToRigidParticle())
				{
					if (Rigid->ObjectState() == EObjectStateType::Sleeping || Rigid->ObjectState() == EObjectStateType::Dynamic)
					{
						if (bInvalidate)
						{
							SetObjectStateHelper(*GetProxy(), *Rigid, EObjectStateType::Dynamic, true);
						}

						Rigid->SetAngularAcceleration(AngularAcceleration);
					}
				}
			});
	}

	const FVec3 AngularAcceleration() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->AngularAcceleration();
			}

			return FVec3(0);
		});
	}

	void AddTorque(const FVec3& InTorque, bool bInvalidate = true)
	{
		Write([&InTorque, bInvalidate, this](auto* Particle)
		{
			if (auto* Rigid = Particle->CastToRigidParticle())
			{
				if (Rigid->ObjectState() == EObjectStateType::Sleeping || Rigid->ObjectState() == EObjectStateType::Dynamic)
				{
					if (bInvalidate)
					{
						SetObjectStateHelper(*GetProxy(), *Rigid, EObjectStateType::Dynamic, true);
					}

					Rigid->AddTorque(InTorque, bInvalidate);
				}
			}
		});
	}

	const FVec3 LinearImpulseVelocity() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->LinearImpulseVelocity();
			}

			return FVec3(0);
		});
	}

	const FVec3 LinearImpulse() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->LinearImpulseVelocity() * Rigid->M();
			}

			return FVec3(0);
		});
	}

	void SetLinearImpulse(const FVec3& InLinearImpulse, bool bIsVelocity, bool bInvalidate = true)
	{
		Write([&InLinearImpulse, bIsVelocity, bInvalidate, this](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				if (Rigid->ObjectState() == EObjectStateType::Sleeping || Rigid->ObjectState() == EObjectStateType::Dynamic)
				{
					if (bInvalidate)
					{
						SetObjectStateHelper(*GetProxy(), *Rigid, EObjectStateType::Dynamic, true);
					}

					if (bIsVelocity)
					{
						Rigid->SetLinearImpulseVelocity(InLinearImpulse, bInvalidate);
					}
					else
					{
						Rigid->SetLinearImpulseVelocity(InLinearImpulse * Rigid->InvM(), bInvalidate);
					}
				}
			}
		});
	}

	const FVec3 AngularImpulseVelocity() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->AngularImpulseVelocity();
			}

			return FVec3(0);
		});
	}

	const FVec3 AngularImpulse() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				const FMatrix33 WorldI = Utilities::ComputeWorldSpaceInertia(Rigid->GetR() * Rigid->RotationOfMass(), Rigid->I());
				return WorldI * Rigid->AngularImpulseVelocity();
			}

			return FVec3(0);
		});
	}

	void SetAngularImpulse(const FVec3& InAngularImpulse, bool bIsVelocity, bool bInvalidate = true)
	{
		Write([&InAngularImpulse, bIsVelocity, bInvalidate, this](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				if (Rigid->ObjectState() == EObjectStateType::Sleeping || Rigid->ObjectState() == EObjectStateType::Dynamic)
				{
					if (bInvalidate)
					{
						SetObjectStateHelper(*GetProxy(), *Rigid, EObjectStateType::Dynamic, true);
					}

					if (bIsVelocity)
					{
						Rigid->SetAngularImpulseVelocity(InAngularImpulse, bInvalidate);
					}
					else
					{
						const FMatrix33 WorldInvI = Utilities::ComputeWorldSpaceInertia(Rigid->GetR() * Rigid->RotationOfMass(), Rigid->InvI());
						Rigid->SetAngularImpulseVelocity(WorldInvI * InAngularImpulse, bInvalidate);
					}
				}
			}
		});
	}

	const Chaos::TVec3<FRealSingle> I() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->I();
			}

			return Chaos::TVec3<FRealSingle>(0, 0, 0);
		});
	}

	void SetI(const Chaos::TVec3<FRealSingle>& InI)
	{
		Write([&InI](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				Rigid->SetI(InI);
			}
		});
	}

	const Chaos::TVec3<FRealSingle> InvI() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->InvI();
			}

			return Chaos::TVec3<FRealSingle>(0, 0, 0);
		});
	}

	void SetInvI(const Chaos::TVec3<FRealSingle>& InInvI)
	{
		Write([&InInvI](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				Rigid->SetInvI(InInvI);
			}
		});
	}

	const FReal M() const
	{
		return Read([](auto* Particle) -> FReal
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->M();
			}

			return 0;
		});
	}

	void SetM(const FReal InM)
	{
		Write([&InM](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				Rigid->SetM(InM);
			}
		});
	}

	const FReal InvM() const
	{
		return Read([](auto* Particle) -> FReal
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->InvM();
			}

			return 0;
		});
	}

	void SetInvM(const FReal InInvM)
	{
		Write([&InInvM](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				Rigid->SetInvM(InInvM);
			}
		});
	}

	const FVec3 CenterOfMass() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->CenterOfMass();
			}

			return FVec3(0);
		});
	}

	void SetCenterOfMass(const FVec3& InCenterOfMass, bool bInvalidate = true)
	{
		Write([&InCenterOfMass, bInvalidate](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				Rigid->SetCenterOfMass(InCenterOfMass, bInvalidate);
			}
		});
	}

	const FRotation3 RotationOfMass() const
	{
		return Read([](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->RotationOfMass();
			}

			return FRotation3::FromIdentity();
		});
	}

	void SetRotationOfMass(const FRotation3& InRotationOfMass, bool bInvalidate = true)
	{
		Write([&InRotationOfMass, bInvalidate](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				Rigid->SetRotationOfMass(InRotationOfMass, bInvalidate);
			}
		});
	}

	const FReal LinearEtherDrag() const
	{
		return Read([](auto* Particle) -> FReal
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->LinearEtherDrag();
			}

			return 0;
		});
	}

	void SetLinearEtherDrag(const FReal InLinearEtherDrag)
	{
		Write([&InLinearEtherDrag](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				Rigid->SetLinearEtherDrag(InLinearEtherDrag);
			}
		});
	}

	const FReal AngularEtherDrag() const
	{
		return Read([](auto* Particle) -> FReal
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->AngularEtherDrag();
			}

			return 0;
		});
	}

	void SetAngularEtherDrag(const FReal InAngularEtherDrag)
	{
		Write([&InAngularEtherDrag](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				Rigid->SetAngularEtherDrag(InAngularEtherDrag);
			}
		});
	}

protected:
	void SetObjectStateBase(const EObjectStateType InState, bool bAllowEvents = false, bool bInvalidate = true)
	{
		Write([InState, bAllowEvents, bInvalidate, this](auto* Ptr)
		{
			if (auto Rigid = Ptr->CastToRigidParticle())
			{
				SetObjectStateHelper(*this, *Rigid, InState, bAllowEvents, bInvalidate);
			}
		});
	}
public:

	void SetSleepType(ESleepType InSleepType)
	{
		Write([InSleepType](auto* Particle)
		{
			if (auto Rigid = Particle->CastToRigidParticle())
			{
				return Rigid->SetSleepType(InSleepType);
			}
		});
	}

	ESleepType SleepType() const
	{
		if (auto Rigid = Particle->CastToRigidParticle())
		{
			return Rigid->SleepType();
		}

		return ESleepType::MaterialSleep;
	}

protected:
	void VerifyContext() const
	{
#if PHYSICS_THREAD_CONTEXT
		//Are you using the wrong API type for the thread this code runs in?
		//GetGameThreadAPI should be used for gamethread, GetPhysicsThreadAPI should be used for callbacks and internal physics thread
		//Note if you are using a ParallelFor you must use PhysicsParallelFor to ensure the right context is inherited from parent thread
		if(bExternal)
		{
			//if proxy is registered with solver, we need a lock
			if(GetSolverBase() != nullptr)
			{
				EnsureIsInGameThreadContext();
			}
		}
		else
		{
			EnsureIsInPhysicsThreadContext();
		}
#endif
	}

private:

	template <typename TLambda>
	auto Read(const TLambda& Lambda) const { VerifyContext(); return bExternal ? Lambda(GetParticle_LowLevel()) : Lambda(GetHandle_LowLevel()); }

	template <typename TLambda>
	const auto& ReadRef(const TLambda& Lambda) const { VerifyContext(); return bExternal ? Lambda(GetParticle_LowLevel()) : Lambda(GetHandle_LowLevel()); }

	template <typename TLambda>
	auto& ReadRef(const TLambda& Lambda) { VerifyContext(); return bExternal ? Lambda(GetParticle_LowLevel()) : Lambda(GetHandle_LowLevel()); }

	template <typename TLambda>
	void Write(const TLambda& Lambda)
	{
		VerifyContext();
		if (bExternal)
		{
			Lambda(GetParticle_LowLevel());
		}
		else
		{
			//Mark entire particle as dirty from PT. TODO: use property system
			FPhysicsSolverBase* SolverBase = GetSolverBase();	//internal so must have solver already
			if(FRewindData* RewindData = SolverBase->GetRewindData())
			{
				RewindData->MarkDirtyFromPT(*GetHandle_LowLevel());
			}

			Lambda(GetHandle_LowLevel());
		}
	}
};

static_assert(sizeof(TThreadedSingleParticlePhysicsProxyBase<true>) == sizeof(FSingleParticlePhysicsProxy), "Derived types only used to constrain API, all data lives in base class ");
static_assert(sizeof(TThreadedSingleParticlePhysicsProxyBase<false>) == sizeof(FSingleParticlePhysicsProxy), "Derived types only used to constrain API, all data lives in base class ");


class FRigidBodyHandle_External : public TThreadedSingleParticlePhysicsProxyBase<true>
{
	FRigidBodyHandle_External() = delete;	//You should only ever new FSingleParticlePhysicsProxy, derrived types are simply there for API constraining, no new data

public:
	using Base = TThreadedSingleParticlePhysicsProxyBase<true>;
	using Base::VerifyContext;

	void SetIgnoreAnalyticCollisions(bool bIgnoreAnalyticCollisions) { VerifyContext(); GetParticle_LowLevel()->SetIgnoreAnalyticCollisions(bIgnoreAnalyticCollisions); }
	void UpdateShapeBounds() { VerifyContext(); GetParticle_LowLevel()->UpdateShapeBounds(); }

	void UpdateShapeBounds(const FTransform& Transform) { VerifyContext(); GetParticle_LowLevel()->UpdateShapeBounds(Transform); }

	void SetShapeCollisionTraceType(int32 InShapeIndex, EChaosCollisionTraceFlag TraceType) { VerifyContext(); GetParticle_LowLevel()->SetShapeCollisionTraceType(InShapeIndex, TraceType); }

	void SetShapeSimCollisionEnabled(int32 InShapeIndex, bool bInEnabled) { VerifyContext(); GetParticle_LowLevel()->SetShapeSimCollisionEnabled(InShapeIndex, bInEnabled); }
	void SetShapeQueryCollisionEnabled(int32 InShapeIndex, bool bInEnabled) { VerifyContext(); GetParticle_LowLevel()->SetShapeQueryCollisionEnabled(InShapeIndex, bInEnabled); }
	UE_DEPRECATED(5.8, "Use SetShapeFilterData instead")
	void SetShapeSimData(int32 InShapeIndex, const FCollisionFilterData& SimData)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		VerifyContext();
		GetParticle_LowLevel()->SetShapeSimData(InShapeIndex, SimData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	CHAOS_API void SetShapeFilterData(int32 InShapeIndex, const Filter::FShapeFilterData& FilterData);
	CHAOS_API void SetShapeFilterInstanceData(int32 InShapeIndex, const Filter::FInstanceData& InstanceData);

	void SetParticleID(const FParticleID& ParticleID)
	{
		VerifyContext();
		GetParticle_LowLevel()->SetParticleID(ParticleID);
	}

	void SetX(const FVec3& InX, bool bInvalidate = true)
	{
		VerifyContext();
		SetXBase(InX, bInvalidate);
		FSingleParticleProxyTimestamp& SyncTS = GetSyncTimestampAs<FSingleParticleProxyTimestamp>(); 
		SyncTS.OverWriteX.Set(GetSolverSyncTimestamp_External(), InX);
	}

	void SetR(const FRotation3& InR, bool bInvalidate = true)
	{
		VerifyContext();
		SetRBase(InR, bInvalidate);
		FSingleParticleProxyTimestamp& SyncTS = GetSyncTimestampAs<FSingleParticleProxyTimestamp>();
		SyncTS.OverWriteR.Set(GetSolverSyncTimestamp_External(), InR);
	}

	void SetV(const FVec3& InV, bool bInvalidate = true)
	{
		VerifyContext();
		SetVBase(InV, bInvalidate);
		if (InV == FVec3(0))	//should we use an explicit API instead?
		{
			//external thread is setting velocity to 0 so we want to freeze object until sim catches up
			//but we also want position to snap to where it currently is on external thread
			SetX(X(), bInvalidate);
		}
		FSingleParticleProxyTimestamp& SyncTS = GetSyncTimestampAs<FSingleParticleProxyTimestamp>();
		SyncTS.OverWriteV.Set(GetSolverSyncTimestamp_External(), InV);
	}

	void SetW(const FVec3& InW, bool bInvalidate = true)
	{
		VerifyContext();
		SetWBase(InW, bInvalidate);
		
		if (InW == FVec3(0))	//should we use an explicit API instead?
		{
			//external thread is setting velocity to 0 so we want to freeze object until sim catches up
			//but we also want position to snap to where it currently is on external thread
			SetR(R(), bInvalidate);
		}
		FSingleParticleProxyTimestamp& SyncTS = GetSyncTimestampAs<FSingleParticleProxyTimestamp>();
		SyncTS.OverWriteW.Set(GetSolverSyncTimestamp_External(), InW);
	}

	void SetObjectState(const EObjectStateType InState, bool bAllowEvents = false, bool bInvalidate = true)
	{
		VerifyContext();
		if (auto Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			FSingleParticleProxyTimestamp& SyncTS = GetSyncTimestampAs<FSingleParticleProxyTimestamp>();
			SyncTS.ObjectStateTimestamp = GetSolverSyncTimestamp_External();
			if (InState != EObjectStateType::Dynamic && Rigid->ObjectState() == EObjectStateType::Dynamic)
			{
				//we want to snap the particle to its current state on the external thread. This is because the user wants the object to fully stop right now
				//the internal thread will continue if async is on, but eventually it will see this snap
				SetV(FVec3(0), bInvalidate);
				SetW(FVec3(0), bInvalidate);
			}

			if (InState == EObjectStateType::Kinematic && Rigid->ObjectState() != EObjectStateType::Kinematic)
			{
				// NOTE: using ClearKinematicTarget() here would just clean the dirty flag, but we actually
				// want to make sure the kinematic target mode is set to "None", which is how it's default constructed.
				SetKinematicTarget(Chaos::FKinematicTarget(), bInvalidate);
			}
		}

		SetObjectStateBase(InState, bAllowEvents, bInvalidate);
	}

	int32 Island() const
	{
		VerifyContext();
		if (const TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			return Rigid->Island();
		}

		return INDEX_NONE;
	}
	// TODO(stett): Make the setter private. It is public right now to provide access to proxies.
	void SetIsland(const int32 InIsland)
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetIsland(InIsland);
		}
	}

	void ClearEvents()
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->ClearEvents();
		}
	}

	EWakeEventEntry GetWakeEvent()
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			return Rigid->GetWakeEvent();
		}

		return EWakeEventEntry::None;
	}

	void ClearForces(bool bInvalidate = true)
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>*Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{

			Rigid->ClearForces(bInvalidate);
		}
	}

	void ClearTorques(bool bInvalidate = true)
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>*Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->ClearTorques(bInvalidate);
		}
	}

	void* UserData() const { VerifyContext(); return GetParticle_LowLevel()->UserData(); }
	void SetUserData(void* InUserData) { VerifyContext(); GetParticle_LowLevel()->SetUserData(InUserData); }
	
	void SetGeometry(const Chaos::FImplicitObjectPtr& ImplicitGeometryPtr)
	{
		VerifyContext();
		GetParticle_LowLevel()->SetGeometry(ImplicitGeometryPtr);
	}

	void MarkNonFrequentDataDirty()
	{
		VerifyContext();

		FGeometryParticle* GTParticle = GetParticle_LowLevel();
		if (GTParticle)
		{
			// Mark NonFrequentData as dirty.
			GTParticle->ForceDirty(EChaosPropertyFlags::NonFrequentData);

			// Manually add proxy to solver's dirty list (replicating what MarkDirty does)
			if (IPhysicsProxyBase* Proxy = GetProxy())
			{
				if (FPhysicsSolverBase* SolverBase = Proxy->GetSolverBase())
				{
					SolverBase->AddDirtyProxy(Proxy);
				}
			}
		}
	}
	
	UE_DEPRECATED(5.4, "Use SetGeometry with FImplicitObjectPtr instead.")
	void SetGeometry(TUniquePtr<FImplicitObject>&& UniqueGeometry)
	{
		check(false);
	}
	
	UE_DEPRECATED(5.4, "Use SetGeometry with FImplicitObjectPtr instead.")
	void SetGeometry(TSharedPtr<const FImplicitObject, ESPMode::ThreadSafe> SharedGeometry)
	{
		check(false);
	}
	
	void RemoveShape(FPerShapeData* InShape, bool bWakeTouching) { VerifyContext(); GetParticle_LowLevel()->RemoveShape(InShape, bWakeTouching); }

	void MergeShapesArray(FShapesArray&& OtherShapesArray) { VerifyContext(); GetParticle_LowLevel()->MergeShapesArray(MoveTemp(OtherShapesArray)); }

	void MergeGeometry(TArray<Chaos::FImplicitObjectPtr>&& Objects) { VerifyContext(); GetParticle_LowLevel()->MergeGeometry(MoveTemp(Objects)); }

    UE_DEPRECATED(5.4, "Please use MergeGeometry with FImplicitObjectPtr instead.")
	void MergeGeometry(TArray<TUniquePtr<FImplicitObject>>&& Objects) { check(false); }

	bool IsKinematicTargetDirty() const
	{
		VerifyContext();
		if (auto Kinematic = GetParticle_LowLevel()->CastToKinematicParticle())
		{
			return Kinematic->IsKinematicTargetDirty();
		}
		return false;
	}

	void ClearKinematicTarget()
	{
		VerifyContext();
		if (auto Kinematic = GetParticle_LowLevel()->CastToKinematicParticle())
		{
			return Kinematic->ClearKinematicTarget();
		}
	}

	void SetSmoothEdgeCollisionsEnabled(bool bEnabled)
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>*Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			if (bEnabled)
			{
				Rigid->AddCollisionConstraintFlag(ECollisionConstraintFlags::CCF_SmoothEdgeCollisions);
			}
			else
			{
				Rigid->RemoveCollisionConstraintFlag(ECollisionConstraintFlags::CCF_SmoothEdgeCollisions);
			}
		}
	}

	void SetCCDEnabled(bool bEnabled)
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetCCDEnabled(bEnabled);
		}
	}

	bool CCDEnabled() const
	{
		VerifyContext();
		if (const TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			return Rigid->CCDEnabled();
		}

		return false;
	}

	void SetMACDEnabled(bool bEnabled)
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>*Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetMACDEnabled(bEnabled);
		}
	}

	bool MACDEnabled() const
	{
		VerifyContext();
		if (const TPBDRigidParticle<FReal, 3>*Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			return Rigid->MACDEnabled();
		}

		return false;
	}

	void SetPartialIslandSleepAllowed(bool bEnabled)
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>*Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetPartialIslandSleepAllowed(bEnabled);
		}
	}

	bool PartialIslandSleepAllowed() const
	{
		VerifyContext();
		if (const TPBDRigidParticle<FReal, 3>*Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			return Rigid->PartialIslandSleepAllowed();
		}

		return false;
	}

	Private::FIterationSettings IterationSettings() const
	{
		VerifyContext();
		if (const TPBDRigidParticle<FReal, 3>*Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			return Rigid->IterationSettings();
		}

		return Private::FIterationSettings(0, 0, 0);
	}

	void SetPositionSolverIterationCount(int32 PositionSolverIterationCountIn)
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetPositionSolverIterations(PositionSolverIterationCountIn);
		}
	}


	void SetVelocitySolverIterationCount(int32 VelocitySolverIterationCountIn)
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetVelocitySolverIterations(VelocitySolverIterationCountIn);
		}
	}


	void SetProjectionSolverIterationCount(int32 ProjectionSolverIterationCountIn)
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetProjectionSolverIterations(ProjectionSolverIterationCountIn);
		}
	}

	void SetMaxLinearSpeedSq(FReal InNewSpeed)
	{
		VerifyContext();
		if(TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetMaxLinearSpeedSq(InNewSpeed);
		}
	}

	FReal GetMaxLinearSpeedSq()
	{
		VerifyContext();
		if(const TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			return Rigid->MaxLinearSpeedSq();
		}

		return TNumericLimits<FReal>::Max();
	}

	void SetMaxAngularSpeedSq(FReal InNewSpeed)
	{
		VerifyContext();
		if(TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetMaxAngularSpeedSq(InNewSpeed);
		}
	}

	FReal GetMaxAngularSpeedSq()
	{
		VerifyContext();
		if(const TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			return Rigid->MaxAngularSpeedSq();
		}

		return TNumericLimits<FReal>::Max();
	}

	FRealSingle GetInitialOverlapDepenetrationVelocity()
	{
		VerifyContext();
		if (const TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			return Rigid->InitialOverlapDepenetrationVelocity();
		}

		return 0;
	}

	void SetInitialOverlapDepenetrationVelocity(FRealSingle InNewSpeed)
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetInitialOverlapDepenetrationVelocity(InNewSpeed);
		}
	}

	void SetSleepThresholdMultiplier(FRealSingle Multiplier)
	{
		VerifyContext();
		if (TPBDRigidParticle<FReal, 3>* Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetSleepThresholdMultiplier(Multiplier);
		}
	}

	void SetDisabled(bool bDisable)
	{
		VerifyContext();
		if (auto Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetDisabled(bDisable);
		}
	}

	bool Disabled() const
	{
		VerifyContext();
		if (auto Rigid = GetParticle_LowLevel()->CastToRigidParticle())
		{
			return Rigid->Disabled();
		}

		return false;
	}
};

static_assert(sizeof(FRigidBodyHandle_External) == sizeof(FSingleParticlePhysicsProxy), "Derived types only used to constrain API, all data lives in base class ");


class FRigidBodyHandle_Internal : public TThreadedSingleParticlePhysicsProxyBase<false>
{
	FRigidBodyHandle_Internal() = delete;	//You should only ever new FSingleParticlePhysicsProxy, derived types are simply there for API constraining, no new data

public:
	using Base = TThreadedSingleParticlePhysicsProxyBase<false>;

	const FVec3 PreV() const
	{
		VerifyContext();
		if (auto Rigid = GetHandle_LowLevel()->CastToRigidParticle())
		{
			return Rigid->GetPreV();
		}
		return FVec3(0);
	}

	const FVec3 PreW() const
	{
		VerifyContext();
		if (auto Rigid = GetHandle_LowLevel()->CastToRigidParticle())
		{
			return Rigid->GetPreW();
		}
		return FVec3(0);
	}

	void SetX(const FVec3& InX, bool bInvalidate = true)
	{
		VerifyContext();
		SetXBase(InX, bInvalidate);
		if (auto Rigid = GetHandle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetP(InX);
		}
	}

	void SetR(const FRotation3& InR, bool bInvalidate = true)
	{
		VerifyContext();
		SetRBase(InR, bInvalidate);
		if (auto Rigid = GetHandle_LowLevel()->CastToRigidParticle())
		{
			Rigid->SetQ(InR);
		}
	}

	void SetV(const FVec3& InV, bool bInvalidate = true)
	{
		VerifyContext();
		SetVBase(InV, bInvalidate);
	}

	void SetW(const FVec3& InW, bool bInvalidate = true)
	{
		VerifyContext();
		SetWBase(InW, bInvalidate);
	}

	void SetObjectState(const EObjectStateType InState, bool bAllowEvents = false, bool bInvalidate = true)
	{
		VerifyContext();
		SetObjectStateBase(InState, bAllowEvents, bInvalidate);
	}
};

static_assert(sizeof(FRigidBodyHandle_Internal) == sizeof(FSingleParticlePhysicsProxy), "Derived types only used to constrain API, all data lives in base class ");

inline FSingleParticlePhysicsProxy* FSingleParticlePhysicsProxy::Create(TUniquePtr<FGeometryParticle>&& Particle, UObject* InOwner = nullptr)
{
	ensure(Particle->GetProxy() == nullptr);	//not already owned by another proxy. TODO: use TUniquePtr
	auto Proxy = new FSingleParticlePhysicsProxy(MoveTemp(Particle), nullptr, InOwner);
	return Proxy;
}
}
