// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Framework/PhysicsProxy.h"
#include "Chaos/Framework/BufferedData.h"
#include "BoneHierarchy.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"

#include "Framework/TripleBufferedData.h"

// @todo(chaos): remove this file

namespace Chaos
{
	class FParticleData;
}

struct FSkeletalMeshPhysicsProxyParams
{
	FSkeletalMeshPhysicsProxyParams()
		: Name("")
		, InitialTransform(FTransform::Identity)
		, InitialLinearVelocity(FVector::ZeroVector)
		, InitialAngularVelocity(FVector::ZeroVector)

		, ObjectType(EObjectStateTypeEnum::Chaos_Object_Kinematic)

		, CollisionType(ECollisionTypeEnum::Chaos_Volumetric)
		, ParticlesPerUnitArea(0.1f)
		, MinNumParticles(0)
		, MaxNumParticles(50)
		, MinRes(5)
		, MaxRes(10)
		, CollisionGroup(0)
#if 0
		, bEnableClustering(false)
		, ClusterGroupIndex(0)
		, MaxClusterLevel(100)
		, DamageThreshold(250.f)
#endif
		, Density(2.4f)
		, MinMass(0.001f)
		, MaxMass(1.e6f)

		, bSimulating(false)
	{}

	FString Name;

	//
	// Analytic implicit representation
	//

	FBoneHierarchy BoneHierarchy;

	//
	// Mesh
	//

	TArray<FVector> MeshVertexPositions;
	TArray<FIntVector> Triangles;

	//
	// Transform hierarchy
	//

	FTransform InitialTransform;
	FTransform LocalToWorld;
	FVector InitialLinearVelocity;
	FVector InitialAngularVelocity;

	Chaos::TSerializablePtr<Chaos::FChaosPhysicsMaterial> PhysicalMaterial;	// @todo(ccaulfield): should be per-shape
	EObjectStateTypeEnum ObjectType;												// @todo(ccaulfield): should be per-body

	ECollisionTypeEnum CollisionType;
	float ParticlesPerUnitArea;
	int32 MinNumParticles;
	int32 MaxNumParticles;
	int32 MinRes;
	int32 MaxRes;
	int32 CollisionGroup;
#if 0
	bool bEnableClustering;
	int32 ClusterGroupIndex;
	int32 MaxClusterLevel;
	float DamageThreshold;
#endif
	float Density;
	float MinMass;
	float MaxMass;

	bool bSimulating;
};

// @todo(ccaulfield): make the IO structures private again - only the hierarchy should be required outside the PhysicsProxy
struct FSkeletalMeshPhysicsProxyInputs
{
	TArray<FTransform> Transforms;
	TArray<FVector> LinearVelocities;
	TArray<FVector> AngularVelocities;
};

struct FSkeletalMeshPhysicsProxyOutputs //: public Chaos::FParticleData
{
	TArray<FTransform> Transforms;
	TArray<FVector> LinearVelocities;
	TArray<FVector> AngularVelocities;
	void Reset() {
		Transforms = TArray<FTransform>();
		LinearVelocities = TArray<FVector>();
		AngularVelocities = TArray<FVector>();
	}
};


class FSkeletalMeshPhysicsProxy : public TPhysicsProxy<FSkeletalMeshPhysicsProxy, FSkeletalMeshPhysicsProxyOutputs, FProxyTimestampBase>
{
	typedef TPhysicsProxy<FSkeletalMeshPhysicsProxy, FSkeletalMeshPhysicsProxyOutputs, FProxyTimestampBase> Base;
public:


	using FInitFunc = TFunction<void(FSkeletalMeshPhysicsProxyParams& OutParams)>;
	using FInputFunc = TFunction<bool(const float Dt, FSkeletalMeshPhysicsProxyParams& OutParams)>;


	FSkeletalMeshPhysicsProxy() = delete;
	CHAOS_API FSkeletalMeshPhysicsProxy(UObject* InOwner, const FInitFunc& InitFunc);
	CHAOS_API ~FSkeletalMeshPhysicsProxy();

	/** Solver Object interface */
	CHAOS_API void Initialize();
	CHAOS_API bool IsSimulating() const;
	CHAOS_API void UpdateKinematicBodiesCallback(const FParticlesType& Particles, const float Dt, const float Time, FKinematicProxy& Proxy);
	CHAOS_API void StartFrameCallback(const float InDt, const float InTime);
	CHAOS_API void EndFrameCallback(const float InDt);
	CHAOS_API void CreateRigidBodyCallback(FParticlesType& InOutParticles);
	CHAOS_API void ParameterUpdateCallback(FParticlesType& InParticles, const float InTime);
	CHAOS_API void DisableCollisionsCallback(TSet<TTuple<int32, int32>>& InPairs);
	CHAOS_API void AddForceCallback(FParticlesType& InParticles, const float InDt, const int32 InIndex);

	CHAOS_API void BindParticleCallbackMapping(Chaos::TArrayCollectionArray<PhysicsProxyWrapper>& PhysicsProxyReverseMap, Chaos::TArrayCollectionArray<int32>& ParticleIDReverseMap);

	void BufferCommand(Chaos::FPhysicsSolver* InSolver, const FFieldSystemCommand& InCommmand) {}

	FSkeletalMeshPhysicsProxyOutputs* NewData() { return nullptr; }
	CHAOS_API void SyncBeforeDestroy();
	CHAOS_API void OnRemoveFromScene();
	void PushToPhysicsState(const Chaos::FParticleData*) {};
	void ClearAccumulatedData() {}
	CHAOS_API void BufferPhysicsResults();
	CHAOS_API void FlipBuffer();
	CHAOS_API bool PullFromPhysicsState(const int32 SolverSyncTimestamp);
	bool IsDirty() { return false; }
	static constexpr EPhysicsProxyType ConcreteType() { return EPhysicsProxyType::SkeletalMeshType; }
	/** ----------------------- */

	/**
	 *
	 */
	CHAOS_API void Reset();

	/**
	 * Capture the current animation pose for use by the physics.
	 * Called by game thread via the owning component's tick.
	 */
	CHAOS_API void CaptureInputs(const float Dt, const FInputFunc& InputFunc);

	/** 
	 */
	const FSkeletalMeshPhysicsProxyOutputs* GetOutputs() const { return nullptr; }

	const FBoneHierarchy& GetBoneHierarchy() const { return Parameters.BoneHierarchy; }

private:
	using FJointConstraints = Chaos::FPBDJointConstraints;

	FSkeletalMeshPhysicsProxyParams Parameters;
};
