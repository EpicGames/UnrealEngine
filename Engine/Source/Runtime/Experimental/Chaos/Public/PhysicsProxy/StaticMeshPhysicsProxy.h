// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Framework/PhysicsProxy.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
#include "Chaos/Framework/BufferedData.h"

// @todo(chaos): remove this file

namespace Chaos
{
	class FParticleData;
}

struct FPhysicsProxyKinematicUpdate
{
	FTransform NewTransform;
	FVector NewVelocity;
};

struct FStubSkeletalMeshData //: public Chaos::FParticleData 
{
	void Reset() { };
};

class FStaticMeshPhysicsProxy : public TPhysicsProxy<FStaticMeshPhysicsProxy, FStubSkeletalMeshData, FProxyTimestampBase>
{
	typedef  TPhysicsProxy<FStaticMeshPhysicsProxy, FStubSkeletalMeshData, FProxyTimestampBase> Base;
public:

	struct FShapeParams
	{
		FVector BoxExtents;
		FVector2D CapsuleHalfHeightAndRadius;
		float SphereRadius;

		FShapeParams() 
		{ 
			FMemory::Memset(this, 0, sizeof(FShapeParams));
		}
	};

	struct Params
	{
		Params()
			: Name("")
			, InitialTransform(FTransform::Identity)
			, InitialLinearVelocity(FVector::ZeroVector)
			, InitialAngularVelocity(FVector::ZeroVector)
			, ObjectType(EObjectStateTypeEnum::Chaos_Object_Dynamic)
			, ShapeType(EImplicitTypeEnum::Chaos_Max)
			, bSimulating(false)
			, TargetTransform(nullptr)
			, Mass(0.0f)
			, MinRes(5)
			, MaxRes(10)
		{}

		FString Name;
		Chaos::FParticles MeshVertexPositions;
		TArray<Chaos::TVector<int32, 3> > TriIndices;
		FShapeParams ShapeParams;
		FTransform InitialTransform;
		FVector InitialLinearVelocity;
		FVector InitialAngularVelocity;
		EObjectStateTypeEnum ObjectType;
		EImplicitTypeEnum ShapeType;
		bool bSimulating;
		FTransform* TargetTransform;
		Chaos::TSerializablePtr<Chaos::FChaosPhysicsMaterial> PhysicalMaterial;
		float Mass;
		int32 MinRes;
		int32 MaxRes;
	};

	// Engine interface functions
	using FCallbackInitFunc = TFunction<void(FStaticMeshPhysicsProxy::Params&)>;
	using FSyncDynamicFunc = TFunction<void(const FTransform&)>;

	FStaticMeshPhysicsProxy() = delete;
	CHAOS_API FStaticMeshPhysicsProxy(UObject* InOwner, FCallbackInitFunc InInitFunc, FSyncDynamicFunc InSyncFunc);

	CHAOS_API void Initialize();
	CHAOS_API void Reset();

	/** Stores latest update, to be applied at next opportunity (via UpdateKinematicBodiesCallback). */
	CHAOS_API void BufferKinematicUpdate(const FPhysicsProxyKinematicUpdate& InParamUpdate);

	/** Solver Object interface */
	CHAOS_API bool IsSimulating() const;
	CHAOS_API void UpdateKinematicBodiesCallback(const FParticlesType& Particles, const float Dt, const float Time, FKinematicProxy& Proxy);
	CHAOS_API void StartFrameCallback(const float InDt, const float InTime);
	CHAOS_API void EndFrameCallback(const float InDt);
	CHAOS_API void BindParticleCallbackMapping(Chaos::TArrayCollectionArray<PhysicsProxyWrapper> & PhysicsProxyReverseMap, Chaos::TArrayCollectionArray<int32> & ParticleIDReverseMap);
	CHAOS_API void CreateRigidBodyCallback(FParticlesType& InOutParticles);
	CHAOS_API void ParameterUpdateCallback(FParticlesType& InParticles, const float InTime);
	CHAOS_API void DisableCollisionsCallback(TSet<TTuple<int32, int32>>& InPairs);
	CHAOS_API void AddForceCallback(FParticlesType& InParticles, const float InDt, const int32 InIndex);
	void BufferCommand(Chaos::FPhysicsSolver* InSolver, const FFieldSystemCommand& InCommand) {};

	void SyncBeforeDestroy() {};
	CHAOS_API void OnRemoveFromScene();
	void PushToPhysicsState(const Chaos::FParticleData*) {};
	void ClearAccumulatedData() {}
	CHAOS_API void BufferPhysicsResults();
	CHAOS_API void FlipBuffer();
	CHAOS_API bool PullFromPhysicsState(const int32 SolverSyncTimestamp);
	bool IsDirty() { return false; }
	FStubSkeletalMeshData* NewData() { return nullptr; }
	static constexpr EPhysicsProxyType ConcreteType() { return EPhysicsProxyType::StaticMeshType; }
	/** ----------------------- */

private:
};
