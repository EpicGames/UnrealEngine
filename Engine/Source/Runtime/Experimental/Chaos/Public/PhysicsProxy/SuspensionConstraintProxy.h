// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/Framework/MultiBufferResource.h"
#include "Chaos/Framework/PhysicsProxy.h"
#include "Chaos/PBDSuspensionConstraints.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "Chaos/ParticleHandle.h"
#include "PhysicsCoreTypes.h"
#include "Chaos/Defines.h"
#include "Chaos/PBDConstraintBaseData.h"

namespace Chaos
{
	class FSuspensionConstraint;

	class FPBDRigidsEvolutionGBF;

class FSuspensionConstraintPhysicsProxy : public IPhysicsProxyBase
{
public:
	using Base = IPhysicsProxyBase;
	
	FSuspensionConstraintPhysicsProxy() = delete;
	CHAOS_API FSuspensionConstraintPhysicsProxy(FSuspensionConstraint* InConstraint, FPBDSuspensionConstraintHandle* InHandle, UObject* InOwner = nullptr);

	bool IsInitialized() const { return bInitialized; }
	void SetInitialized() { bInitialized = true; }

	static CHAOS_API FGeometryParticleHandle* GetParticleHandleFromProxy(IPhysicsProxyBase* ProxyBase);

	//
	//  Lifespan Management
	//

	CHAOS_API void InitializeOnPhysicsThread(FPBDRigidsSolver* InSolver, FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData);

	// Merge to perform a remote sync
	CHAOS_API void PushStateOnGameThread(FDirtyPropertiesManager& Manager, int32 DataIdx, FDirtyChaosProperties& RemoteData);

	CHAOS_API void PushStateOnPhysicsThread(FPBDRigidsSolver* InSolver, const FDirtyPropertiesManager& Manager, int32 DataIdx, const FDirtyChaosProperties& RemoteData);
	// Merge to perform a remote sync - END

	CHAOS_API void DestroyOnGameThread();
	CHAOS_API void DestroyOnPhysicsThread(FPBDRigidsSolver* InSolver);

	CHAOS_API void UpdateTargetOnPhysicsThread(FPBDRigidsSolver* InSolver, const FVector& TargetPos, const IPhysicsProxyBase* GroundProxy, const FVector& Normal, bool Enabled);


	//
	// Member Access
	//

	FPBDSuspensionConstraintHandle* GetHandle()
	{
		return Constraint_PT;
	}

	const FPBDSuspensionConstraintHandle* GetHandle() const
	{
		return Constraint_PT;
	}

	virtual void* GetHandleUnsafe() const override
	{
		return Constraint_PT;
	}

	void SetHandle(FPBDSuspensionConstraintHandle* InHandle)
	{
		Constraint_PT = InHandle;
	}

	FSuspensionConstraint* GetConstraint()
	{
		return Constraint_GT;
	}

	const FSuspensionConstraint* GetConstraint() const
	{
		return Constraint_GT;
	}
	
private:
	FSuspensionConstraint* Constraint_GT;
	FPBDSuspensionConstraintHandle* Constraint_PT;
	bool bInitialized;
};
}
