// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/Core.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "Serialization/BufferArchive.h"
#include "Containers/Set.h"
#include "Containers/CircularBuffer.h"
#include "Chaos/ResimCacheBase.h"
#include "Chaos/PBDJointConstraints.h"
#include "Chaos/AABB.h"
#include "Chaos/PhysicsProxyHandle.h"

#ifndef VALIDATE_REWIND_DATA
#define VALIDATE_REWIND_DATA 0
#endif

#ifndef DEBUG_REWIND_DATA
#define DEBUG_REWIND_DATA 0
#endif

#ifndef DEBUG_NETWORK_PHYSICS
#define DEBUG_NETWORK_PHYSICS 0
#endif

class IPhysicsProxyBase;
struct FProxyTimestampBase;

namespace Chaos
{
	namespace Private
	{
		class FPBDIsland;
	}

	namespace ResimDefines
	{
		extern const CHAOS_API FName BubbleResimDebugDrawTag;
	}

	namespace ResimulationCVars
	{
		extern float DesyncLeniencyValue;
	}

/** Base rewind history used in the rewind data */
struct FBaseRewindHistory
{
	FORCEINLINE virtual ~FBaseRewindHistory() = default;

	/** Create a new, empty instance with the same concrete type as this object */
	virtual TUniquePtr<FBaseRewindHistory> CreateNew() const = 0;

	/** Create a polymorphic copy of the history */
	virtual TUniquePtr<FBaseRewindHistory> Clone() const = 0;

	/** Initialize history */
	virtual void Initialize() { }

	/** Set the package map for serialization */
	FORCEINLINE virtual void SetPackageMap(class UPackageMap* InPackageMap) {}

	/** Check if the history buffer contains an entry for the given frame*/
	FORCEINLINE virtual bool HasValidData(const int32 ValidFrame) const { return false; }

	/** Find how many entries are valid in frame range
	* @param StartFrame = Included
	* @param EndFrame = Included
	* @param bIncludeUnimportant = If to include unimportant data entries
	* @param bIncludeImportant = If to include important data entries */
	FORCEINLINE virtual int32 CountValidData(const uint32 StartFrame, const uint32 EndFrame, const bool bIncludeUnimportant = true, const bool bIncludeImportant = false) { return 0; }
	
	/** Find how many entries that have been marked as altered, meaning the server has altered the data so that it doesn't match the received data from the client */
	FORCEINLINE virtual int32 CountAlteredData(const bool bIncludeUnimportant = true, const bool bIncludeImportant = false) { return 0; }

	/** Mark frame important or unimportant
	* @param Frame = use INDEX_NONE to set importance on all entries */
	FORCEINLINE virtual void SetImportant(const bool bImportant, const int32 Frame = INDEX_NONE) {}

	/** Extract data from the history buffer at a given time */
	FORCEINLINE virtual bool ExtractData(const int32 ExtractFrame, const bool bResetSolver, void* HistoryData, const bool bExactFrame = false) { return true; }
	
	/** Call ApplyData on each frame data within range */
	FORCEINLINE virtual void ApplyDataRange(const int32 FromFrame, const int32 ToFrame, void* ActorComponent, const bool bOnlyImportant = false) {}

	/** Iterate over and merge data */
	FORCEINLINE virtual void MergeData(const int32 FromFrame, void* ToData) {}

	/** Record data into the history buffer at a given time */
	FORCEINLINE virtual bool RecordData(const int32 RecordFrame, const void* HistoryData) { return true; }

	/** Record data in order and allow growing the history */
	FORCEINLINE virtual bool RecordDataGrowingOrdered(const void* HistoryData) { return true; }

	/** Set if this history should only allow overriding of data if it has a higher value than current recorded data */
	FORCEINLINE virtual void SetRecordDataIncremental(const bool bInIncremental) {}

	/** Copy all data from local history into into @param OutHistory
	* @param bIncludeUnimportant = If to copy unimportant data entries
	* @param bIncludeImportant = If to copy important data entries */
	virtual bool CopyAllData(Chaos::FBaseRewindHistory& OutHistory, bool bIncludeUnimportant = true, bool bIncludeImportant = false) { return false; }

	/** Copy altered data from local history into into @param OutHistory
	* @param bIncludeUnimportant = If to copy unimportant data entries
	* @param bIncludeImportant = If to copy important data entries */
	virtual bool CopyAlteredData(Chaos::FBaseRewindHistory& OutHistory, bool bIncludeUnimportant = true, bool bIncludeImportant = false) { return false; }

	/** Copy data from local history into @param OutHistory
	* @param StartFrame = Included
	* @param EndFrame = Included 
	* @param bIncludeUnimportant = If to copy unimportant data entries
	* @param bIncludeImportant = If to copy important data entries */
	virtual bool CopyData(Chaos::FBaseRewindHistory& OutHistory, const uint32 StartFrame, const uint32 EndFrame, bool bIncludeUnimportant = true, bool bIncludeImportant = false) { return false; }

	/** Copy all data and record it ordered and growing */
	virtual bool CopyAllDataGrowingOrdered(Chaos::FBaseRewindHistory& OutHistory) { return false; }

	/** Create a polymorphic copy of only a range of frames, applying the frame offset to the copies
	* @param StartFrame = Included
	* @param EndFrame = Excluded */
	virtual TUniquePtr<FBaseRewindHistory> CopyFramesWithOffset(const uint32 StartFrame, const uint32 EndFrame, const int32 FrameOffset) = 0;

	/** Copy new data (received from the network) into this history
	* Returns frame to resimulate from if @param CompareDataForRewind is set to true and compared data differ enough 
	* @param TryInjectAtFrame puts the latest of the received data entries at the specified frame (if there isn't data for that frame yet), used to inject the latest data at the head of the buffer if it's empty */
	virtual int32 ReceiveNewData(FBaseRewindHistory& NewData, const int32 FrameOffset, const bool CompareDataForRewind = false, const bool bImportant = false, int32 TryInjectAtFrame = INDEX_NONE) { return INDEX_NONE; }
	
	UE_DEPRECATED(5.7, "Deprecated, use the ReceiveNewData call with parameter InjectAtFrameIfEmpty, pass in nullptr to opt out of implementing a function.")
	virtual int32 ReceiveNewData(FBaseRewindHistory& NewData, const int32 FrameOffset, const bool CompareDataForRewind = false, const bool bImportant = false) 
	{
		return ReceiveNewData(NewData, FrameOffset, CompareDataForRewind, bImportant, INDEX_NONE);
	}

	/** Serialize the data to or from a network archive */
	UE_DEPRECATED(5.6, "Deprecated, use the NetSerialize call with parameter that takes a function DataSetupFunction, pass in nullptr to opt out of implementing a function.")
	virtual void NetSerialize(FArchive& Ar, UPackageMap* PackageMap) { NetSerialize(Ar, PackageMap, [](void* Data, const int32 DataIndex){}); }

	/** Serialize the data to or from a network archive */
	virtual void NetSerialize(FArchive& Ar, UPackageMap* PackageMap, TUniqueFunction<void(void* Data, const int32 DataIndex)> DataSetupFunction) {}
	
	/** Validate data in history buffer received from clients on the server */
	virtual void ValidateDataInHistory(const void* ActorComponent) {}

	/** Print custom string along with values for each entry in history */
	FORCEINLINE virtual void DebugData(const FString& DebugText) { }

	/** Get arrays of frame values for each entry in the history */
	FORCEINLINE virtual void DebugData(const Chaos::FBaseRewindHistory& NewData, TArray<int32>& LocalFrames, TArray<int32>& ServerFrames, TArray<int32>& InputFrames) { }

	/** Legacy interface to rewind states */
	UE_DEPRECATED(5.6, "Deprecated, RewindStates is no longer viable. Any custom states can be applied during IRewindCallback::ProcessInputs_Internal during resimulation. Example FNetworkPhysicsCallback")
		FORCEINLINE virtual bool RewindStates(const int32 RewindFrame, const bool bResetSolver) { return false; }

	/** Legacy interface to apply inputs */
	UE_DEPRECATED(5.6, "Deprecated, ApplyInputs is no longer viable. Any custom inputs can be applied during IRewindCallback::ProcessInputs_Internal during resimulation. Example FNetworkPhysicsCallback")
		FORCEINLINE virtual bool ApplyInputs(const int32 ApplyFrame, const bool bResetSolver) { return false; }

	/** Return the most up to date frame entry in history, returns INDEX_NONE if no frame was found */
	virtual const int32 GetLatestFrame() const { return INDEX_NONE; }

	/** Return the least up to date frame entry in history, returns INT_MAX if no frame was found */
	virtual const int32 GetEarliestFrame() const { return INT_MAX; }

	/** Return the max size of the history */
	virtual const int32 GetHistorySize() const { return 0; }

	/** Return if history has valid data */
	virtual const bool HasDataInHistory() const { return false; }

	/** Resize the history */
	virtual void ResizeDataHistory(const int32 FrameCount, const EAllowShrinking AllowShrinking = EAllowShrinking::Default) {}

	/** Perform a fast reset, marking the data history as reset but not clearing the data or resetting collections */
	virtual void ResetFast() {}
};

/** Templated data history holding a data buffer */
template<typename DataType>
struct TDataRewindHistory : public FBaseRewindHistory
{
	FORCEINLINE TDataRewindHistory(const int32 FrameCount, const bool bIsHistoryLocal) :
		bIsLocalHistory(bIsHistoryLocal), bIncremental(false), DataHistory(), LatestFrame(INDEX_NONE), CurrentFrame(0), CurrentIndex(0), NumFrames(FrameCount)
	{
		DataHistory.SetNum(NumFrames);
	}

	FORCEINLINE TDataRewindHistory(const int32 FrameCount) :
		bIsLocalHistory(false), bIncremental(false), DataHistory(), LatestFrame(INDEX_NONE), CurrentFrame(0), CurrentIndex(0), NumFrames(FrameCount)
	{
		DataHistory.SetNum(NumFrames);
	}
	FORCEINLINE virtual ~TDataRewindHistory() = default;

	/** Initialize history */
	virtual void Initialize()
	{
		LatestFrame = INDEX_NONE;
		// Iterate over current data to find latest frame
		for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			LatestFrame = FMath::Max(LatestFrame, DataHistory[FrameIndex].LocalFrame);
		}
	}

protected:

	/** Get the closest (min/max) valid data from the data frame */
	FORCEINLINE int32 ClosestData(const int32 DataFrame, const bool bMinData)
	{
		int32 ClosestIndex = INDEX_NONE;
		for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			const int32 ValidFrame = bMinData ? FMath::Max(0, DataFrame - FrameIndex) : DataFrame + FrameIndex;
			const int32 ValidIndex = GetFrameIndex(ValidFrame);

			if (DataHistory[ValidIndex].LocalFrame == ValidFrame)
			{
				ClosestIndex = ValidIndex;
				break;
			}
		}
		return ClosestIndex;
	}

public : 

	/** Check if the history buffer contains an entry for the given frame*/
	FORCEINLINE virtual bool HasValidData(const int32 ValidFrame) const override
	{
		const int32 ValidIndex = GetFrameIndex(ValidFrame);
		return ValidFrame == DataHistory[ValidIndex].LocalFrame;
	}

	/** Extract states at a given time */
	FORCEINLINE virtual bool ExtractData(const int32 ExtractFrame, const bool bResetSolver, void* HistoryData, const bool bExactFrame = false) override
	{
		// Early out if we are trying to extract data later than latest frame but the latest data is more than the whole buffer size old, don't extrapolate that far.
		if (ExtractFrame - NumFrames > GetLatestFrame())
		{
			return false;
		}

		const int32 ExtractIndex = GetFrameIndex(ExtractFrame);
		if (ExtractFrame == DataHistory[ExtractIndex].LocalFrame)
		{
			CurrentFrame = ExtractFrame;
			CurrentIndex = ExtractIndex;
			*static_cast<DataType*>(HistoryData) = DataHistory[CurrentIndex];
			return true;
		}
		else if(!bExactFrame)
		{
#if DEBUG_NETWORK_PHYSICS
			if (bResetSolver)
			{
				UE_LOGF(LogChaos, Warning, "		Unable to extract data at frame %d while rewinding the simulation", ExtractFrame);
			}
#endif
			const int32 MinFrameIndex = ClosestData(ExtractFrame, true);
			const int32 MaxFrameIndex = ClosestData(ExtractFrame, false);

			if (MinFrameIndex != INDEX_NONE
				&& MaxFrameIndex != INDEX_NONE 
				&& (MinFrameIndex != MaxFrameIndex)
				&& (DataHistory[MinFrameIndex].LocalFrame != DataHistory[MaxFrameIndex].LocalFrame))
			{
				DataType& ExtractedData = *static_cast<DataType*>(HistoryData);
				ExtractedData = DataHistory[MinFrameIndex];

				const int32 DeltaFrame = FMath::Abs(ExtractFrame - DataHistory[MinFrameIndex].LocalFrame);

				ExtractedData.LocalFrame = ExtractFrame;
				ExtractedData.ServerFrame += DeltaFrame;
				ExtractedData.bDataAltered = true; // This history entry is now altered and doesn't correspond to the source entry

PRAGMA_DISABLE_DEPRECATION_WARNINGS
				ExtractedData.DoInterpolateData(DataHistory[MinFrameIndex], DataHistory[MaxFrameIndex]);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if DEBUG_NETWORK_PHYSICS
				UE_LOGF(LogChaos, Log, "		Interpolating data between frame %d and %d - > [%d]", DataHistory[MinFrameIndex].LocalFrame, DataHistory[MaxFrameIndex].LocalFrame, ExtractedData.LocalFrame);
#endif
				return true;
			}
			else if (MinFrameIndex != INDEX_NONE)
			{
				DataType& ExtractedData = *static_cast<DataType*>(HistoryData);
				ExtractedData = DataHistory[MinFrameIndex];

				const int32 DeltaFrame = FMath::Abs(ExtractFrame - DataHistory[MinFrameIndex].LocalFrame);

				ExtractedData.LocalFrame = ExtractFrame;
				ExtractedData.ServerFrame += DeltaFrame;
				ExtractedData.bDataAltered = true; // This history entry is now altered and doesn't correspond to the source entry

#if DEBUG_NETWORK_PHYSICS
				UE_LOGF(LogChaos, Log, "		Setting data to frame %d", DataHistory[MinFrameIndex].LocalFrame);
#endif
				return true;
			}
			else
			{
#if DEBUG_NETWORK_PHYSICS
				UE_LOGF(LogChaos, Log, "		Failed to find data bounds : Min = %d | Max = %d", MinFrameIndex, MaxFrameIndex);
#endif
				return false;
			}
		}
		return false;
	}

	FORCEINLINE virtual void MergeData(int32 FromFrame, void* ToData) override
	{
		if (!ToData)
		{
			return;
		}

		DataType& MergeToData = *static_cast<DataType*>(ToData);

		const int32 ToFrame = MergeToData.LocalFrame;
		const int32 ToServerFrame = MergeToData.ServerFrame;

		// Save target data as a stable reference for interpolation and as the final *this.
		// Older frames are accumulated separately so that each MergeData call receives
		// *this = newer frame and From = older adjacent frame, matching the API contract.
		const DataType TargetData = MergeToData;
		DataType AccumulatedOlder;
		bool bSeeded = false;

		for (; FromFrame < ToFrame; FromFrame++)
		{
			const int32 FromIndex = GetFrameIndex(FromFrame);

			// If the frame was never recorded, synthesize it via interpolation between the nearest
			// earlier known frame and the target. This writes directly into DataHistory[FromIndex]
			// as a side effect, filling the gap so the merge accumulation can pick it up below.
			if (FromFrame != DataHistory[FromIndex].LocalFrame)
			{
				const int32 MinFrameIndex = ClosestData(FromFrame, true);
				if (MinFrameIndex != INDEX_NONE)
				{
					DataType& NewData = DataHistory[FromIndex];
					const int32 DeltaFrame = ToFrame - FromFrame;
					NewData.LocalFrame = FromFrame;
					NewData.ServerFrame = ToServerFrame - DeltaFrame;
					NewData.bDataAltered = true;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
					NewData.DoInterpolateData(DataHistory[MinFrameIndex], TargetData);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
			}

			if (FromFrame == DataHistory[FromIndex].LocalFrame)
			{
				if (!bSeeded)
				{
					// Seed the accumulator with the earliest available frame
					AccumulatedOlder = DataHistory[FromIndex];
					bSeeded = true;
				}
				else
				{
					// DataHistory[FromIndex] is newer than AccumulatedOlder.
					// Call with *this = newer, From = accumulated-older so the API contract holds.
					DataType NewerFrameCopy = DataHistory[FromIndex];
					NewerFrameCopy.MergeData(AccumulatedOlder);
					AccumulatedOlder = NewerFrameCopy;
				}
			}
		}

		// MergeToData was never written in the loop (only AccumulatedOlder was), so it
		// still holds the original target data (newest). Merge accumulated older frames in:
		// *this = newest (target), From = accumulated older frames
		if (bSeeded)
		{
			MergeToData.MergeData(AccumulatedOlder);
			MergeToData.bDataAltered = true;
		}
	}

	/** Load the data from the buffer at a specific frame */
	FORCEINLINE bool LoadData(const int32 LoadFrame)
	{
		const int32 LoadIndex = GetFrameIndex(LoadFrame);
		CurrentFrame = LoadFrame;
		CurrentIndex = LoadIndex;
		return true;
	}

	/** Eval the data from the buffer at a specific frame */
	FORCEINLINE bool EvalData(const int32 EvalFrame)
	{
		const int32 EvalIndex = GetFrameIndex(EvalFrame);
		if (EvalFrame == DataHistory[EvalIndex].LocalFrame)
		{
			CurrentFrame = EvalFrame;
			CurrentIndex = EvalIndex;
			return true;
		}
		return false;
	}

	FORCEINLINE virtual bool RecordData(const int32 RecordFrame, const void* HistoryData) override
	{
		LoadData(RecordFrame);

		const DataType* HistoryDataCast = static_cast<const DataType*>(HistoryData);

		// If incremental recording, early out if currently cached data is newer than the data to record
		if (bIncremental && DataHistory[CurrentIndex].LocalFrame >= HistoryDataCast->LocalFrame)
		{
			return false;
		}

		DataHistory[CurrentIndex] = *HistoryDataCast;

		LatestFrame = FMath::Max(LatestFrame, DataHistory[CurrentIndex].LocalFrame);
		return true;
	}

	FORCEINLINE virtual bool RecordDataGrowingOrdered(const void* HistoryData) override
	{
		const DataType* HistoryDataCast = static_cast<const DataType*>(HistoryData);

		int32 InsertAtIdx = 0;
		int32 StoredFrame = 0;

		if (NumFrames > 0)
		{
			for (int32 Idx = CurrentIndex; Idx >= 0; Idx--)
			{
				StoredFrame = DataHistory[Idx].LocalFrame;
				if (StoredFrame == HistoryDataCast->LocalFrame)
				{
					// Data for frame already stored
					return false;
				}
				else if (StoredFrame < HistoryDataCast->LocalFrame)
				{
					// Found index where earlier data is stored, insert newer data on the next index
					InsertAtIdx = Idx + 1;
					break;
				}
			}
		}

		DataHistory.Insert(*HistoryDataCast, InsertAtIdx);

		NumFrames++;
		CurrentIndex = NumFrames - 1;
		LatestFrame = FMath::Max(LatestFrame, HistoryDataCast->LocalFrame);

		return true;
	}

	FORCEINLINE virtual void SetRecordDataIncremental(const bool bInIncremental) override
	{
		bIncremental = bInIncremental;
	}

	virtual bool CopyAllDataGrowingOrdered(Chaos::FBaseRewindHistory& OutHistory) override
	{
		TDataRewindHistory& OutDataHistory = static_cast<TDataRewindHistory&>(OutHistory);
		bool bHasCopiedData = false;

		for (int32 CopyIndex = 0; CopyIndex < NumFrames; ++CopyIndex)
		{
			bHasCopiedData |= OutDataHistory.RecordDataGrowingOrdered(&DataHistory[CopyIndex]);
		}
		return bHasCopiedData;
	}

	/** Current data that is being loaded/recorded*/
	DataType& GetCurrentData() { return DataHistory[CurrentIndex]; }

	const DataType& GetCurrentData() const { return DataHistory[CurrentIndex]; }

	/** Returns the earliest data entry and loads it as the current data of the history */
	DataType& GetAndLoadEarliestData()
	{
		int32 EarliestFrame = INT_MAX;
		for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			DataType& Data = DataHistory[FrameIndex];
			if (Data.LocalFrame > INDEX_NONE && Data.LocalFrame < EarliestFrame)
			{
				CurrentIndex = FrameIndex;
				CurrentFrame = Data.LocalFrame;
				EarliestFrame = CurrentFrame;
			}
		}

		return GetCurrentData();
	}

	/** Returns the next incremental data entry after the CurrentIndex (and loads it as the current data of the history), if there isn't a data entry with a higher LocalFrame value, nullptr is returned */
	DataType* GetAndLoadNextIncrementalData()
	{
		if (NumFrames > 0)
		{
			const int32 NextIndex = GetFrameIndex(CurrentIndex + 1);
			if (DataHistory[NextIndex].LocalFrame > CurrentFrame)
			{
				CurrentIndex = NextIndex;
				CurrentFrame = DataHistory[NextIndex].LocalFrame;

				return &DataHistory[NextIndex];
			}
		}

		return nullptr;
	}

	/** Loads and returns the data on the next index */
	DataType* GetAndLoadNextData()
	{
		if (NumFrames > 0)
		{
			const int32 NextIndex = GetFrameIndex(CurrentIndex + 1);
			CurrentIndex = NextIndex;
			CurrentFrame = DataHistory[NextIndex].LocalFrame;

			return &DataHistory[NextIndex];
		}

		return nullptr;
	}

	/** Get the number of valid data in the buffer index range */
	FORCEINLINE uint32 NumValidData(const uint32 StartFrame, const uint32 EndFrame) const
	{
		uint32 NumData = 0;
		for (uint32 ValidFrame = StartFrame; ValidFrame < EndFrame; ++ValidFrame)
		{
			const int32 ValidIndex = GetFrameIndex(ValidFrame);
			if (ValidFrame == DataHistory[ValidIndex].LocalFrame)
			{
				++NumData;
			}
		}
		return NumData;
	}

	TArray<DataType>& GetDataHistory() { return DataHistory; }
	const TArray<DataType>& GetDataHistoryConst() const { return DataHistory; }

	/** Return the most up to date frame entry in history */
	virtual const int32 GetLatestFrame() const override
	{
		return LatestFrame;
	}

	/** Return the least up to date frame entry in history */
	virtual const int32 GetEarliestFrame() const override
	{
		int32 EarliestFrame = INT_MAX;
		
		for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
		{
			if (DataHistory[FrameIndex].LocalFrame > INDEX_NONE)
			{
				EarliestFrame = FMath::Min(EarliestFrame, DataHistory[FrameIndex].LocalFrame);
			}
		}

		return EarliestFrame;
	}

	/** Return the max size of the history */
	virtual const int32 GetHistorySize() const
	{
		return NumFrames;
	}

	virtual const bool HasDataInHistory() const
	{
		return LatestFrame > INDEX_NONE;
	}

	/** Resize the history */
	FORCEINLINE void ResizeDataHistory(const int32 FrameCount, const EAllowShrinking AllowShrinking = EAllowShrinking::Default) override
	{
		if (NumFrames != FrameCount)
		{
			NumFrames = FrameCount;
			DataHistory.SetNum(NumFrames, AllowShrinking);
			CurrentIndex = GetFrameIndex(CurrentFrame);
		}
	}

	/** Return circular index for given frame value */
	FORCEINLINE const uint32 GetFrameIndex(const int32 Frame) const
	{
		if (NumFrames <= 0)
		{
			return 0u;
		}

		int32 Idx = Frame % NumFrames;
		if (Idx < 0)
		{
			Idx = Idx + NumFrames;
		}

		return static_cast<uint32>(Idx);
	}

	FORCEINLINE virtual void ResetFast()
	{
		LatestFrame = INDEX_NONE;
		CurrentFrame = 0;
		CurrentIndex = 0;
	}

protected : 

	/** Check if the history is on the local/remote client*/
	bool bIsLocalHistory;

	/** If this history only record data that is of a higher frame value than previous recorded frame on the same index */
	bool bIncremental;

	/** Data buffer holding the history */
	TArray<DataType> DataHistory;

	/** The most up to date frame entry in history */
	int32 LatestFrame;

	/** Current frame that is being loaded/recorded */
	int32 CurrentFrame;

	/** Current index that is being loaded/recorded */
	int32 CurrentIndex;

	/** Number of frames in data history */
	int32 NumFrames = 0;
};

struct FFrameAndPhase
{
	enum EParticleHistoryPhase : uint8
	{
		//The particle state before PushData, server state update, or any sim callbacks are processed 
		//This is the results of the previous frame before any GT modifications are made in this frame
		//This is what the server state should be compared against
		//This is what we rewind to before a resim
		PrePushData = 0,

		//The particle state after PushData is applied, but before any server state is applied
		PostPushData,

		//The particle state after sim callbacks are applied.
		//This is used to detect desync of particles before simulation itself is run (these desyncs can come from server state or the sim callback itself)
		PostCallbacks,

		NumPhases
	};

	int32 Frame : 30;
	uint32 Phase : 2;

	bool operator<(const FFrameAndPhase& Other) const
	{
		return Frame < Other.Frame || (Frame == Other.Frame && Phase < Other.Phase);
	}

	bool operator<=(const FFrameAndPhase& Other) const
	{
		return Frame < Other.Frame || (Frame == Other.Frame && Phase <= Other.Phase);
	}

	bool operator==(const FFrameAndPhase& Other) const
	{
		return Frame == Other.Frame && Phase == Other.Phase;
	}
};

template <typename THandle, typename T, bool bNoEntryIsHead>
struct NoEntryInSync
{
	static bool Helper(const THandle& Handle)
	{
		//nothing written so we're pointing to the particle which means it's in sync
		return true;
	}
};

template <typename THandle, typename T>
struct NoEntryInSync<THandle, T, false>
{
	static bool Helper(const THandle& Handle)
	{
		//nothing written so compare to zero
		T HeadVal;
		HeadVal.CopyFrom(Handle);
		return HeadVal == T::ZeroValue();
	}
};

struct FPropertyInterval
{
	FPropertyIdx Ref;
	FFrameAndPhase FrameAndPhase;
};

template <typename TData, typename TObj>
void CopyDataFromObject(TData& Data, const TObj& Obj)
{
	Data.CopyFrom(Obj);
}

inline void CopyDataFromObject(FPBDJointSettings& Data, const FPBDJointConstraintHandle& Joint)
{
	Data = Joint.GetSettings();
}

template <typename T, EChaosProperty PropName, bool bNoEntryIsHead = true>
class TParticlePropertyBuffer
{
public:
	explicit TParticlePropertyBuffer(int32 InCapacity)
	: Next(0)
	, NumValid(0)
	, Capacity(InCapacity)
	{
	}

	TParticlePropertyBuffer(TParticlePropertyBuffer<T, PropName>&& Other)
	: Next(Other.Next)
	, NumValid(Other.NumValid)
	, Capacity(Other.Capacity)
	, Buffer(MoveTemp(Other.Buffer))
	{
		Other.NumValid = 0;
		Other.Next = 0;
	}

	TParticlePropertyBuffer(const TParticlePropertyBuffer<T, PropName>& Other) = delete;

	~TParticlePropertyBuffer()
	{
		//Need to explicitly cleanup before destruction using Release (release back into the pool)
		ensure(Buffer.Num() == 0);
	}

	//Gets access into buffer in monotonically increasing FrameAndPhase order: x_{n+1} > x_n
	T& WriteAccessMonotonic(const FFrameAndPhase FrameAndPhase, FDirtyPropertiesPool& Manager)
	{
		return *WriteAccessImp<true>(FrameAndPhase, Manager);
	}

	//Gets access into buffer in non-decreasing FrameAndPhase order: x_{n+1} >= x_n
	//If x_{n+1} == x_n we return null to inform the user (usefull when a single phase can have multiple writes)
	T* WriteAccessNonDecreasing(const FFrameAndPhase FrameAndPhase, FDirtyPropertiesPool& Manager)
	{
		return WriteAccessImp<false>(FrameAndPhase, Manager);
	}

	//Searches in reverse order for interval that contains FrameAndPhase
	const T* Read(const FFrameAndPhase FrameAndPhase, const FDirtyPropertiesPool& Manager) const
	{
		const int32 Idx = FindIdx(FrameAndPhase);
		return Idx != INDEX_NONE ? &GetPool(Manager).GetElement(Buffer[Idx].Ref) : nullptr;
	}

	//Get the FrameAndPhase of the head / last entry
	const bool GetHeadFrameAndPhase(FFrameAndPhase& OutFrameAndPhase) const
	{
		if (NumValid)
		{
			const int32 Prev = Next == 0 ? Buffer.Num() - 1 : Next - 1;
			OutFrameAndPhase = Buffer[Prev].FrameAndPhase;
			return true;
		}
		return false;
	}

	//Releases data back into the pool
	void Release(FDirtyPropertiesPool& Manager)
	{
		TPropertyPool<T>& Pool = GetPool(Manager);
		for(FPropertyInterval& Interval : Buffer)
		{
			Pool.RemoveElement(Interval.Ref);
		}

		Buffer.Empty();
		NumValid = 0;
	}

	void Reset()
	{
		NumValid = 0;
	}

	bool IsEmpty() const
	{
		return NumValid == 0;
	}

	void ClearEntryAndFuture(const FFrameAndPhase FrameAndPhase)
	{
		//Move next backwards until FrameAndPhase and anything more future than it is gone
		while (NumValid)
		{
			const int32 PotentialNext = Next - 1 >= 0 ? Next - 1 : Buffer.Num() - 1;

			if (Buffer[PotentialNext].FrameAndPhase < FrameAndPhase)
			{
				break;
			}

			Next = PotentialNext;
			--NumValid;
		}
	}

	void ExtractBufferState(int32& ValidCount, int32& NextIterator) const
	{
		ValidCount = NumValid;
		NextIterator = Next;
	}

	void RestoreBufferState(const int32& ValidCount, const int32& NextIterator)
	{
		NumValid = ValidCount;
		Next = NextIterator;
	}

	bool IsClean(const FFrameAndPhase FrameAndPhase) const
	{
		return FindIdx(FrameAndPhase) == INDEX_NONE;
	}

	template <typename THandle>
	bool IsInSync(const THandle& Handle, const FFrameAndPhase FrameAndPhase, const FDirtyPropertiesPool& Pool) const
	{
		if (const T* Val = Read(FrameAndPhase, Pool))
		{
			T HeadVal;
			CopyDataFromObject(HeadVal, Handle);
			return Val->IsNearlyEqual(HeadVal, ResimulationCVars::DesyncLeniencyValue);
		}

		return NoEntryInSync<THandle, T, bNoEntryIsHead>::Helper(Handle);
	}

	T& Insert(const FFrameAndPhase FrameAndPhase, FDirtyPropertiesPool& Manager)
	{
		T* Result = nullptr;

		int32 FrameIndex = FindIdx(FrameAndPhase);
		if (FrameIndex != INDEX_NONE)
		{
			Result = &GetPool(Manager).GetElement(Buffer[FrameIndex].Ref);
		}
		else
		{
			FPropertyIdx ElementRef;
			if (Next >= Buffer.Num())
			{
				GetPool(Manager).AddElement(ElementRef);
				Buffer.Add({ ElementRef, FrameAndPhase });
			}
			else
			{
				ElementRef = Buffer[Next].Ref;
			}
			Result = &GetPool(Manager).GetElement(ElementRef);

			int32 PrevFrame = Next;
			int32 NextFrame = PrevFrame;
			for (int32 Count = 0; Count < NumValid; ++Count)
			{
				NextFrame = PrevFrame;

				--PrevFrame;
				if (PrevFrame < 0) { PrevFrame = Buffer.Num() - 1; }

				const FPropertyInterval& PrevInterval = Buffer[PrevFrame];
				if (PrevInterval.FrameAndPhase < FrameAndPhase)
				{
					Buffer[NextFrame].FrameAndPhase = FrameAndPhase;
					Buffer[NextFrame].Ref = ElementRef;
					break;
				}
				else
				{
					Buffer[NextFrame] = Buffer[PrevFrame];

					if (Count == NumValid - 1)
					{ 
						// If we shift back and reach the end of the buffer, insert here
						Buffer[PrevFrame].FrameAndPhase = FrameAndPhase;
						Buffer[PrevFrame].Ref = ElementRef;
					}
				}
			}

			++Next;
			if (Next == Capacity) { Next = 0; }

			NumValid = FMath::Min(++NumValid, Capacity);
		}
		return *Result;
	}

private:

	const int32 FindIdx(const FFrameAndPhase FrameAndPhase) const
	{
		int32 Cur = Next;	//go in reverse order because hopefully we don't rewind too far back
		int32 Result = INDEX_NONE;
		for (int32 Count = 0; Count < NumValid; ++Count)
		{
			--Cur;
			if (Cur < 0) { Cur = Buffer.Num() - 1; }

			const FPropertyInterval& Interval = Buffer[Cur];
			
			if (Interval.FrameAndPhase < FrameAndPhase)
			{
				//no reason to keep searching, frame is bigger than everything before this
				break;
			}
			else
			{
				Result = Cur;
			}
		}

		if (bNoEntryIsHead || Result == INDEX_NONE)
		{
			//in this mode we consider the entire interval as one entry
			return Result;
		}
		else
		{
			//in this mode each interval just represents the frame the property was dirtied on
			//so in that case we have to check for equality
			return Buffer[Result].FrameAndPhase == FrameAndPhase ? Result : INDEX_NONE;
		}
	}

	TPropertyPool<T>& GetPool(FDirtyPropertiesPool& Manager) { return Manager.GetPool<T, PropName>(); }
	const TPropertyPool<T>& GetPool(const FDirtyPropertiesPool& Manager) const { return Manager.GetPool<T, PropName>(); }

	//Gets access into buffer in FrameAndPhase order.
	//It's assumed FrameAndPhase is monotonically increasing: x_{n+1} > x_n
	//If bEnsureMonotonic is true we will always return a valid access (unless assert fires)
	//If bEnsureMonotonic is false we will ensure x_{n+1} >= x_n. If x_{n+1} == x_n we return null to inform the user (can be useful when multiple writes happen in same phase)
	template <bool bEnsureMonotonic>
	T* WriteAccessImp(const FFrameAndPhase FrameAndPhase, FDirtyPropertiesPool& Manager)
	{
		if (NumValid)
		{
			const int32 Prev = Next == 0 ? Buffer.Num() - 1 : Next - 1;
			const FFrameAndPhase& LatestFrameAndPhase = Buffer[Prev].FrameAndPhase;
			if (bEnsureMonotonic)
			{
				//Must write in monotonic growing order so that x_{n+1} > x_n
				ensureMsgf(LatestFrameAndPhase < FrameAndPhase, TEXT("WriteAccessImp<bEnsureMonotonic = true> trying to write to already written FrameAndPhase: %d/%d >= %d/%d"), LatestFrameAndPhase.Frame, LatestFrameAndPhase.Phase, FrameAndPhase.Frame, FrameAndPhase.Phase);
			}
			else
			{
				//Must write in growing order so that x_{n+1} >= x_n
				ensureMsgf(LatestFrameAndPhase <= FrameAndPhase, TEXT("WriteAccessImp<bEnsureMonotonic = false> trying to write to already written FrameAndPhase: %d/%d > %d/%d"), LatestFrameAndPhase.Frame, LatestFrameAndPhase.Phase, FrameAndPhase.Frame, FrameAndPhase.Phase);
				if (LatestFrameAndPhase == FrameAndPhase)
				{
					//Already wrote once for this FrameAndPhase so skip
					return nullptr;
				}
			}

			ValidateOrder();
		}

		T* Result;

		if (Next < Buffer.Num())
		{
			//reuse
			FPropertyInterval& Interval = Buffer[Next];
			Interval.FrameAndPhase = FrameAndPhase;
			Result = &GetPool(Manager).GetElement(Interval.Ref);
		}
		else
		{
			//no reuse yet so can just push
			FPropertyIdx NewIdx;
			Result = &GetPool(Manager).AddElement(NewIdx);
			Buffer.Add({NewIdx, FrameAndPhase });
		}

		++Next;
		if (Next == Capacity) { Next = 0; }

		NumValid = FMath::Min(++NumValid, Capacity);

		return Result;
	}

	void ValidateOrder()
	{
#if VALIDATE_REWIND_DATA
		int32 Val = Next;
		FFrameAndPhase PrevVal;
		for (int32 Count = 0; Count < NumValid; ++Count)
		{
			--Val;
			if (Val < 0) { Val = Buffer.Num() - 1; }
			if (Count == 0)
			{
				PrevVal = Buffer[Val].FrameAndPhase;
			}
			else
			{
				ensureMsgf(Buffer[Val].FrameAndPhase < PrevVal, TEXT("ValidateOrder Idx: %d TailFrame: %d/%d, HeadFrame: %d/%d"), Val, Buffer[Val].FrameAndPhase.Frame, Buffer[Val].FrameAndPhase.Phase, PrevVal.Frame, PrevVal.Phase);
				PrevVal = Buffer[Val].FrameAndPhase;
			}
		}
#endif
	}

private:
	int32 Next;
	int32 NumValid;
	int32 Capacity;
	TArray<FPropertyInterval> Buffer;
};


enum EDesyncResult
{
	InSync, //both have entries and are identical, or both have no entries
	Desync, //both have entries but they are different
	NeedInfo //one of the entries is missing. Need more context to determine whether desynced
};

// Wraps FDirtyPropertiesManager and its DataIdx to avoid confusion between Source and offset Dest indices
struct FDirtyPropData
{
	FDirtyPropData(FDirtyPropertiesManager* InManager, int32 InDataIdx)
		: Ptr(InManager), DataIdx(InDataIdx) { }

	FDirtyPropertiesManager* Ptr;
	int32 DataIdx;
};

struct FConstDirtyPropData
{
	FConstDirtyPropData(const FDirtyPropertiesManager* InManager, int32 InDataIdx)
		: Ptr(InManager), DataIdx(InDataIdx) { }

	const FDirtyPropertiesManager* Ptr;
	int32 DataIdx;
};

template <typename T, EShapeProperty PropName>
class TPerShapeDataStateProperty
{
public:
	const T& Read() const
	{
		check(bSet);
		return Val;
	}

	void Write(const T& InVal)
	{
		bSet = true;
		Val = InVal;
	}

	bool IsSet() const
	{
		return bSet;
	}

private:
	T Val;
	bool bSet = false;
};

struct FPerShapeDataStateBase
{
	TPerShapeDataStateProperty<FCollisionData, EShapeProperty::CollisionData> CollisionData;
	TPerShapeDataStateProperty<FMaterialData, EShapeProperty::Materials> MaterialData;

	//helper functions for shape API
	template <typename TParticle>
	static Chaos::Filter::FCombinedShapeFilterData GetShapeCombinedFilterData(const FPerShapeDataStateBase* State, const TParticle& Particle, int32 ShapeIdx)
	{
		return State && State->CollisionData.IsSet() ? State->CollisionData.Read().GetCombinedShapeFilterData() : Particle.ShapesArray()[ShapeIdx]->GetCombinedShapeFilterData();
	}

	template <typename TParticle>
	UE_DEPRECATED(5.8, "This is unused and will be removed. Use GetCombinedShapeFilterData instead")
	static FCollisionFilterData GetQueryData(const FPerShapeDataStateBase* State, const TParticle& Particle, int32 ShapeIdx)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return State && State->CollisionData.IsSet() ? State->CollisionData.Read().GetQueryData() : Particle.ShapesArray()[ShapeIdx]->GetQueryData();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
};

class FPerShapeDataState
{
public:
	FPerShapeDataState(const FPerShapeDataStateBase* InState, const FGeometryParticleHandle& InParticle, const int32 InShapeIdx)
	: State(InState)
	, Particle(InParticle)
	, ShapeIdx(InShapeIdx)
	{
	}

	const Chaos::Filter::FCombinedShapeFilterData GetCombinedShapeFilterData() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return FPerShapeDataStateBase::GetShapeCombinedFilterData(State, Particle, ShapeIdx);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "This is unused and will be removed. Use GetCombinedShapeFilterData instead")
	FCollisionFilterData GetQueryData() const { return FPerShapeDataStateBase::GetQueryData(State, Particle, ShapeIdx); }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
private:
	const FPerShapeDataStateBase* State;
	const FGeometryParticleHandle& Particle;
	const int32 ShapeIdx;

};

struct FShapesArrayStateBase
{
	TArray<FPerShapeDataStateBase> PerShapeData;

	FPerShapeDataStateBase& FindOrAdd(const int32 ShapeIdx)
	{
		if(ShapeIdx >= PerShapeData.Num())
		{
			const int32 NumNeededToAdd = ShapeIdx + 1 - PerShapeData.Num();
			PerShapeData.AddDefaulted(NumNeededToAdd);
		}
		return PerShapeData[ShapeIdx];

	}
};

template <typename T>
FString ToStringHelper(const T& Val)
{
	return Val.ToString();
}

template <typename T>
FString ToStringHelper(const TVector<T, 2>& Val)
{
	return FString::Printf(TEXT("(%s, %s)"), *Val[0].ToString(), *Val[1].ToString());
}

inline FString ToStringHelper(void* Val)
{
	// We don't print pointers because they will always be different in diff, need this function so we will compile
	// when using property .inl macros.
	return FString();
}

inline FString ToStringHelper(const FReal Val)
{
	return FString::Printf(TEXT("%f"), Val);
}

inline FString ToStringHelper(const FRealSingle Val)
{
	return FString::Printf(TEXT("%f"), Val);
}

inline FString ToStringHelper(const EObjectStateType Val)
{
	return FString::Printf(TEXT("%d"), EnumToUnderlyingType(Val));
}

inline FString ToStringHelper(const EPlasticityType Val)
{
	return FString::Printf(TEXT("%d"), EnumToUnderlyingType(Val));
}

inline FString ToStringHelper(const EJointForceMode Val)
{
	return FString::Printf(TEXT("%d"), EnumToUnderlyingType(Val));
}

inline FString ToStringHelper(const EJointMotionType Val)
{
	return FString::Printf(TEXT("%d"), EnumToUnderlyingType(Val));
}

inline FString ToStringHelper(const bool Val)
{
	return FString::Printf(TEXT("%d"), Val);
}

inline FString ToStringHelper(const int32 Val)
{
	return FString::Printf(TEXT("%d"), Val);
}


template <typename TParticle>
class TShapesArrayState
{
public:
	TShapesArrayState(const TParticle& InParticle, const FShapesArrayStateBase* InState)
		: Particle(InParticle)
		, State(InState)
	{}

	FPerShapeDataState operator[](const int32 ShapeIdx) const { return FPerShapeDataState{ State && ShapeIdx < State->PerShapeData.Num() ? &State->PerShapeData[ShapeIdx] : nullptr, Particle, ShapeIdx }; }
private:
	const TParticle& Particle;
	const FShapesArrayStateBase* State;
};

#define REWIND_CHAOS_PARTICLE_PROPERTY(PROP, NAME)\
		const auto Data = State ? State->PROP.Read(FrameAndPhase, Pool) : nullptr;\
		return Data ? Data->NAME() : Head.NAME();\

#define REWIND_CHAOS_ZERO_PARTICLE_PROPERTY(PROP, NAME)\
		const auto Data = State ? State->PROP.Read(FrameAndPhase, Pool) : nullptr;\
		return Data ? Data->NAME() : ZeroVector;\

#define REWIND_PARTICLE_STATIC_PROPERTY(PROP, NAME)\
	decltype(auto) NAME() const\
	{\
		auto& Head = Particle;\
		REWIND_CHAOS_PARTICLE_PROPERTY(PROP, NAME);\
	}\

#define REWIND_PARTICLE_KINEMATIC_PROPERTY(PROP, NAME)\
	decltype(auto) NAME() const\
	{\
		auto& Head = *Particle.CastToKinematicParticle();\
		REWIND_CHAOS_PARTICLE_PROPERTY(PROP, NAME);\
	}\

#define REWIND_PARTICLE_RIGID_PROPERTY(PROP, NAME)\
	decltype(auto) NAME() const\
	{\
		auto& Head = *Particle.CastToRigidParticle();\
		REWIND_CHAOS_PARTICLE_PROPERTY(PROP, NAME);\
	}\

#define REWIND_PARTICLE_ZERO_PROPERTY(PROP, NAME)\
	decltype(auto) NAME() const\
	{\
		auto& Head = *Particle.CastToRigidParticle();\
		REWIND_CHAOS_ZERO_PARTICLE_PROPERTY(PROP, NAME);\
	}\

#define REWIND_JOINT_PROPERTY(PROP, FUNC_NAME, NAME)\
	decltype(auto) Get##FUNC_NAME() const\
	{\
		const auto Data = State ? State->PROP.Read(FrameAndPhase, Pool) : nullptr;\
		return Data ? Data->NAME : Head.Get##PROP().NAME;\
	}\

inline int32 ComputeCircularSize(int32 NumFrames) { return NumFrames * FFrameAndPhase::NumPhases; }

struct FGeometryParticleStateBase
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.6, "Use the constructor that takes a @param bCacheOnePhase")
	explicit FGeometryParticleStateBase(int32 NumFrames)
	: ParticlePositionRotation(ComputeCircularSize(NumFrames))
	, NonFrequentData(ComputeCircularSize(NumFrames))
	, Velocities(ComputeCircularSize(NumFrames))
	, Dynamics(ComputeCircularSize(NumFrames))
	, DynamicsMisc(ComputeCircularSize(NumFrames))
	, MassProps(ComputeCircularSize(NumFrames))
	, KinematicTarget(ComputeCircularSize(NumFrames))
	, TargetPositions(ComputeCircularSize(NumFrames))
	, TargetVelocities(ComputeCircularSize(NumFrames))
	, TargetStates(ComputeCircularSize(NumFrames))
	{
	}

	explicit FGeometryParticleStateBase(int32 NumFrames, bool bCacheOnePhase)
		: ParticlePositionRotation(bCacheOnePhase ? NumFrames : ComputeCircularSize(NumFrames))
		, NonFrequentData(bCacheOnePhase ? NumFrames : ComputeCircularSize(NumFrames))
		, Velocities(bCacheOnePhase ? NumFrames : ComputeCircularSize(NumFrames))
		, Dynamics(bCacheOnePhase ? NumFrames : ComputeCircularSize(NumFrames))
		, DynamicsMisc(bCacheOnePhase ? NumFrames : ComputeCircularSize(NumFrames))
		, MassProps(bCacheOnePhase ? NumFrames : ComputeCircularSize(NumFrames))
		, KinematicTarget(bCacheOnePhase ? NumFrames : ComputeCircularSize(NumFrames))
		, TargetPositions(NumFrames)
		, TargetVelocities(NumFrames)
		, TargetStates(NumFrames)
	{
	}

	FGeometryParticleStateBase(const FGeometryParticleStateBase& Other) = delete;
	FGeometryParticleStateBase(FGeometryParticleStateBase&& Other) = default;
	~FGeometryParticleStateBase() = default;

	// Can be removed when Deprecations are removed
	FGeometryParticleStateBase() = delete;
	// Can be removed when Deprecations are removed
	FGeometryParticleStateBase& operator=(const FGeometryParticleStateBase&) = delete;
	// Can be removed when Deprecations are removed
	FGeometryParticleStateBase& operator=(FGeometryParticleStateBase&&) = delete;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void Release(FDirtyPropertiesPool& Manager)
	{
		ParticlePositionRotation.Release(Manager);
		NonFrequentData.Release(Manager);
		Velocities.Release(Manager);
		Dynamics.Release(Manager);
		DynamicsMisc.Release(Manager);
		MassProps.Release(Manager);
		KinematicTarget.Release(Manager);
		TargetPositions.Release(Manager);
		TargetVelocities.Release(Manager);
		TargetStates.Release(Manager);
	}

	void Reset()
	{
		ParticlePositionRotation.Reset();
		NonFrequentData.Reset();
		Velocities.Reset();
		Dynamics.Reset();
		DynamicsMisc.Reset();
		MassProps.Reset();
		KinematicTarget.Reset();
		TargetVelocities.Reset();
		TargetPositions.Reset();
		TargetStates.Reset();
	}

	void ClearEntryAndFuture(const FFrameAndPhase FrameAndPhase)
	{
		ParticlePositionRotation.ClearEntryAndFuture(FrameAndPhase);
		NonFrequentData.ClearEntryAndFuture(FrameAndPhase);
		Velocities.ClearEntryAndFuture(FrameAndPhase);
		Dynamics.ClearEntryAndFuture(FrameAndPhase);
		DynamicsMisc.ClearEntryAndFuture(FrameAndPhase);
		MassProps.ClearEntryAndFuture(FrameAndPhase);
		KinematicTarget.ClearEntryAndFuture(FrameAndPhase);
	}

	void ExtractHistoryState(int32& PositionValidCount, int32& VelocityValidCount, int32& PositionNextIterator, int32& VelocityNextIterator) const
	{
		ParticlePositionRotation.ExtractBufferState(PositionValidCount, PositionNextIterator);
		Velocities.ExtractBufferState(VelocityValidCount, VelocityNextIterator);
	}

	void RestoreHistoryState(const int32& PositionValidCount, const int32& VelocityValidCount, const int32& PositionNextIterator, const int32& VelocityNextIterator)
	{
		ParticlePositionRotation.RestoreBufferState(PositionValidCount, PositionNextIterator);
		Velocities.RestoreBufferState(VelocityValidCount, VelocityNextIterator);
	}

	bool IsClean(const FFrameAndPhase FrameAndPhase) const
	{
		return IsCleanExcludingDynamics(FrameAndPhase) && Dynamics.IsClean(FrameAndPhase);
	}

	bool IsCleanExcludingDynamics(const FFrameAndPhase FrameAndPhase) const
	{
		return ParticlePositionRotation.IsClean(FrameAndPhase) &&
			NonFrequentData.IsClean(FrameAndPhase) &&
			Velocities.IsClean(FrameAndPhase) &&
			DynamicsMisc.IsClean(FrameAndPhase) &&
			MassProps.IsClean(FrameAndPhase) &&
			KinematicTarget.IsClean(FrameAndPhase);
	}

	template <bool bSkipDynamics = false>
	bool IsInSync(const FGeometryParticleHandle& Handle, const FFrameAndPhase FrameAndPhase, const FDirtyPropertiesPool& Pool) const;

	template <typename TParticle>
	static TShapesArrayState<TParticle> ShapesArray(const FGeometryParticleStateBase* State, const TParticle& Particle)
	{
		return TShapesArrayState<TParticle>{ Particle, State ? &State->ShapesArrayState : nullptr };
	}

	void SyncSimWritablePropsFromSim(FDirtyPropData Manager,const TPBDRigidParticleHandle<FReal,3>& Rigid);
	void SyncDirtyDynamics(FDirtyPropData& DestManager,const FDirtyChaosProperties& Dirty,const FConstDirtyPropData& SrcManager);
	
	/** Setting bNoEntryIsHead = false in TParticlePropertyBuffer allows us to not cache the data each frame and it will only return entries for a specific frame when fetched 
	* For example, we don't need to cache a kinematic target that is set to None, instead we only cache when it's not set to None and if we don't have an entry for a specific frame we can expect it to be a None kinematic target.
	* Note that bNoEntryIsHead = true (which is default) will return the latest cached entry closest to the requested frame and phase without returning an earlier entry. */
	TParticlePropertyBuffer<FParticlePositionRotation,EChaosProperty::XR> ParticlePositionRotation;
	TParticlePropertyBuffer<FParticleNonFrequentData,EChaosProperty::NonFrequentData> NonFrequentData;
	TParticlePropertyBuffer<FParticleVelocities,EChaosProperty::Velocities> Velocities;
	TParticlePropertyBuffer<FParticleDynamics,EChaosProperty::Dynamics, /*bNoEntryIsHead=*/false> Dynamics;
	TParticlePropertyBuffer<FParticleDynamicMisc,EChaosProperty::DynamicMisc> DynamicsMisc;
	TParticlePropertyBuffer<FParticleMassProps,EChaosProperty::MassProps> MassProps;
	TParticlePropertyBuffer<FKinematicTarget, EChaosProperty::KinematicTarget, /*bNoEntryIsHead=*/false> KinematicTarget;

	TParticlePropertyBuffer<FParticlePositionRotation, EChaosProperty::XR, /*bNoEntryIsHead=*/false> TargetPositions;
	TParticlePropertyBuffer<FParticleVelocities, EChaosProperty::Velocities, /*bNoEntryIsHead=*/false> TargetVelocities;
	TParticlePropertyBuffer<FParticleDynamicMisc, EChaosProperty::DynamicMisc, /*bNoEntryIsHead=*/false> TargetStates;

	FShapesArrayStateBase ShapesArrayState;
};

struct FParticleSimpleState
{
private:
	FGeometryParticleHandle* HandlePtr;
	FVec3 X = { 0,0,0 };
	FQuat R = FQuat::Identity;
	FVec3 V = { 0,0,0 };
	FVec3 W = { 0,0,0 };

public:
	FParticleSimpleState(FGeometryParticleHandle& InHandle) : HandlePtr(&InHandle)
	{
	}

	void CacheXR(FVec3 InX, FQuat InR)
	{
		X = InX;
		R = InR;
	}
	void CacheVW(FVec3 InV, FVec3 InW)
	{
		V = InV;
		W = InW;
	}

	FGeometryParticleHandle* GetObjectPtr() const { return HandlePtr; }
	FVec3 GetX() const { return X; }
	FQuat GetR() const { return R; }
	FVec3 GetV() const { return V; }
	FVec3 GetW() const { return W; }
};

class FGeometryParticleState
{
public:

	FGeometryParticleState(const FGeometryParticleHandle& InParticle, const FDirtyPropertiesPool& InPool)
	: Particle(InParticle)
	, Pool(InPool)
	, FrameAndPhase{0,0}
	{
	}

	FGeometryParticleState(const FGeometryParticleStateBase* InState, const FGeometryParticleHandle& InParticle, const FDirtyPropertiesPool& InPool, const FFrameAndPhase InFrameAndPhase)
	: Particle(InParticle)
	, Pool(InPool)
	, State(InState)
	, FrameAndPhase(InFrameAndPhase)
	{
	}

	REWIND_PARTICLE_STATIC_PROPERTY(ParticlePositionRotation, GetX)
	REWIND_PARTICLE_STATIC_PROPERTY(ParticlePositionRotation, GetR)

	REWIND_PARTICLE_KINEMATIC_PROPERTY(Velocities, GetV)
	REWIND_PARTICLE_KINEMATIC_PROPERTY(Velocities, GetW)

	REWIND_PARTICLE_RIGID_PROPERTY(DynamicsMisc, LinearEtherDrag)
	REWIND_PARTICLE_RIGID_PROPERTY(DynamicsMisc, AngularEtherDrag)
	REWIND_PARTICLE_RIGID_PROPERTY(DynamicsMisc, MaxLinearSpeedSq)
	REWIND_PARTICLE_RIGID_PROPERTY(DynamicsMisc, MaxAngularSpeedSq)
	REWIND_PARTICLE_RIGID_PROPERTY(DynamicsMisc, InitialOverlapDepenetrationVelocity)
	REWIND_PARTICLE_RIGID_PROPERTY(DynamicsMisc, SleepThresholdMultiplier)
	REWIND_PARTICLE_RIGID_PROPERTY(DynamicsMisc, ObjectState)
	REWIND_PARTICLE_RIGID_PROPERTY(DynamicsMisc, CollisionGroup)
	REWIND_PARTICLE_RIGID_PROPERTY(DynamicsMisc, ControlFlags)

	REWIND_PARTICLE_RIGID_PROPERTY(MassProps, CenterOfMass)
	REWIND_PARTICLE_RIGID_PROPERTY(MassProps, RotationOfMass)
	REWIND_PARTICLE_RIGID_PROPERTY(MassProps, I)
	REWIND_PARTICLE_RIGID_PROPERTY(MassProps, M)
	REWIND_PARTICLE_RIGID_PROPERTY(MassProps, InvM)

	REWIND_PARTICLE_STATIC_PROPERTY(NonFrequentData, GetGeometry)
	REWIND_PARTICLE_STATIC_PROPERTY(NonFrequentData, UniqueIdx)
	REWIND_PARTICLE_STATIC_PROPERTY(NonFrequentData, SpatialIdx)
#if CHAOS_DEBUG_NAME
	REWIND_PARTICLE_STATIC_PROPERTY(NonFrequentData, DebugName)
#endif

	REWIND_PARTICLE_ZERO_PROPERTY(Dynamics, Acceleration)
	REWIND_PARTICLE_ZERO_PROPERTY(Dynamics, AngularAcceleration)
	REWIND_PARTICLE_ZERO_PROPERTY(Dynamics, LinearImpulseVelocity)
	REWIND_PARTICLE_ZERO_PROPERTY(Dynamics, AngularImpulseVelocity)

	TShapesArrayState<FGeometryParticleHandle> ShapesArray() const
	{
		return FGeometryParticleStateBase::ShapesArray(State, Particle);
	}

	const FGeometryParticleHandle& GetHandle() const
	{
		return Particle;
	}

	void SetState(const FGeometryParticleStateBase* InState)
	{
		State = InState;
	}

	FString ToString() const
	{
#undef REWIND_PARTICLE_TO_STR
#define REWIND_PARTICLE_TO_STR(PropName) Out += FString::Printf(TEXT(#PropName":%s\n"), *ToStringHelper(PropName()));
		//TODO: use macro to define api and the to string
		FString Out = FString::Printf(TEXT("ParticleID:[Global: %d Local: %d]\n"), Particle.ParticleID().GlobalID, Particle.ParticleID().LocalID);

		REWIND_PARTICLE_TO_STR(GetX)
		REWIND_PARTICLE_TO_STR(GetR)
		//REWIND_PARTICLE_TO_STR(Geometry)
		//REWIND_PARTICLE_TO_STR(UniqueIdx)
		//REWIND_PARTICLE_TO_STR(SpatialIdx)

		if(Particle.CastToKinematicParticle())
		{
			REWIND_PARTICLE_TO_STR(GetV)
			REWIND_PARTICLE_TO_STR(GetW)
		}

		if(Particle.CastToRigidParticle())
		{
			REWIND_PARTICLE_TO_STR(LinearEtherDrag)
			REWIND_PARTICLE_TO_STR(AngularEtherDrag)
			REWIND_PARTICLE_TO_STR(MaxLinearSpeedSq)
			REWIND_PARTICLE_TO_STR(MaxAngularSpeedSq)
			REWIND_PARTICLE_TO_STR(InitialOverlapDepenetrationVelocity)
			REWIND_PARTICLE_TO_STR(SleepThresholdMultiplier)

			REWIND_PARTICLE_TO_STR(ObjectState)
			REWIND_PARTICLE_TO_STR(CollisionGroup)
			REWIND_PARTICLE_TO_STR(ControlFlags)

			REWIND_PARTICLE_TO_STR(CenterOfMass)
			REWIND_PARTICLE_TO_STR(RotationOfMass)
			REWIND_PARTICLE_TO_STR(I)
			REWIND_PARTICLE_TO_STR(M)
			REWIND_PARTICLE_TO_STR(InvM)

			REWIND_PARTICLE_TO_STR(Acceleration)
			REWIND_PARTICLE_TO_STR(AngularAcceleration)
			REWIND_PARTICLE_TO_STR(LinearImpulseVelocity)
			REWIND_PARTICLE_TO_STR(AngularImpulseVelocity)
		}

		return Out;
	}

private:
	const FGeometryParticleHandle& Particle;
	const FDirtyPropertiesPool& Pool;
	const FGeometryParticleStateBase* State = nullptr;
	const FFrameAndPhase FrameAndPhase;

	CHAOS_API static FVec3 ZeroVector;

	};


struct FJointStateBase
{
	UE_DEPRECATED(5.6, "Use the constructor that takes a @param bCacheOnePhase")
	explicit FJointStateBase(int32 NumFrames)
		: JointSettings(ComputeCircularSize(NumFrames))
		, JointProxies(ComputeCircularSize(NumFrames))
	{
	}

	explicit FJointStateBase(int32 NumFrames, bool bCacheOnePhase)
		: JointSettings(bCacheOnePhase ? NumFrames : ComputeCircularSize(NumFrames))
		, JointProxies(bCacheOnePhase ? NumFrames : ComputeCircularSize(NumFrames))
	{
	}

	FJointStateBase(const FJointStateBase& Other) = delete;
	FJointStateBase(FJointStateBase&& Other) = default;
	~FJointStateBase() = default;

	void Release(FDirtyPropertiesPool& Manager)
	{
		JointSettings.Release(Manager);
		JointProxies.Release(Manager);
	}

	void Reset()
	{
		JointSettings.Reset();
		JointProxies.Reset();
	}

	void ClearEntryAndFuture(const FFrameAndPhase FrameAndPhase)
	{
		JointSettings.ClearEntryAndFuture(FrameAndPhase);
		JointProxies.ClearEntryAndFuture(FrameAndPhase);
	}

	bool IsClean(const FFrameAndPhase FrameAndPhase) const
	{
		return JointSettings.IsClean(FrameAndPhase) && JointProxies.IsClean(FrameAndPhase);
	}

	template <bool bSkipDynamics>
	bool IsInSync(const FPBDJointConstraintHandle& Handle, const FFrameAndPhase FrameAndPhase, const FDirtyPropertiesPool& Pool) const;

	TParticlePropertyBuffer<FPBDJointSettings, EChaosProperty::JointSettings> JointSettings;
	TParticlePropertyBuffer<FProxyBasePairProperty, EChaosProperty::JointParticleProxies> JointProxies;
};

class FJointState
{
public:
	FJointState(const FPBDJointConstraintHandle& InJoint, const FDirtyPropertiesPool& InPool)
	: Head(InJoint)
	, Pool(InPool)
	{
	}

	FJointState(const FJointStateBase* InState, const FPBDJointConstraintHandle& InJoint, const FDirtyPropertiesPool& InPool, const FFrameAndPhase InFrameAndPhase)
	: Head(InJoint)
	, Pool(InPool)
	, State(InState)
	, FrameAndPhase(InFrameAndPhase)
	{
	}

	//See JointProperties for API
	//Each CHAOS_INNER_JOINT_PROPERTY entry will have a Get*
#define CHAOS_INNER_JOINT_PROPERTY(OuterProp, FuncName, Inner, InnerType) REWIND_JOINT_PROPERTY(OuterProp, FuncName, Inner);
#include "Chaos/JointProperties.inl"


	FString ToString() const
	{
		TVector<FGeometryParticleHandle*, 2> Particles = Head.GetConstrainedParticles();
		FString Out = FString::Printf(TEXT("Joint: Particle0 ID:[Global: %d Local: %d] Particle1 ID:[Global: %d Local: %d]\n"), Particles[0]->ParticleID().GlobalID, Particles[0]->ParticleID().LocalID, Particles[1]->ParticleID().GlobalID, Particles[1]->ParticleID().LocalID);

#define CHAOS_INNER_JOINT_PROPERTY(OuterProp, FuncName, Inner, InnerType) Out += FString::Printf(TEXT(#FuncName":%s\n"), *ToStringHelper(Get##FuncName()));
#include "Chaos/JointProperties.inl"
#undef CHAOS_INNER_JOINT_PROPERTY

		return Out;
	}

private:
	const FPBDJointConstraintHandle& Head;
	const FDirtyPropertiesPool& Pool;
	const FJointStateBase* State = nullptr;
	const FFrameAndPhase FrameAndPhase = { 0,0 };
};

template <typename T> 
const T* ConstifyHelper(T* Ptr) { return Ptr; }


template <typename T>
T NoRefHelper(const T& Ref) { return Ref; }

template <typename TVal>
class TDirtyObjects
{
public:
	using TKey = decltype(ConstifyHelper(
		((TVal*)0)->GetObjectPtr()
	));

	TVal& Add(const TKey Key, TVal&& Val)
	{
		if(int32* ExistingIdx = KeyToIdx.Find(Key))
		{
			ensure(false);	//Item alread exists, shouldn't be adding again
			return DenseVals[*ExistingIdx];
		}
		else
		{
			const int32 Idx = DenseVals.Emplace(MoveTemp(Val));
			KeyToIdx.Add(Key, Idx);
			return DenseVals[Idx];
		}
	}

	const TVal& FindChecked(const TKey Key) const
	{
		const int32 Idx = KeyToIdx.FindChecked(Key);
		return DenseVals[Idx];
	}

	TVal& FindChecked(const TKey Key)
	{
		const int32 Idx = KeyToIdx.FindChecked(Key);
		return DenseVals[Idx];
	}

	const TVal* Find(const TKey Key) const
	{
		if (const int32* Idx = KeyToIdx.Find(Key))
		{
			return &DenseVals[*Idx];
		}

		return nullptr;
	}

	TVal* Find(const TKey Key)
	{
		if (const int32* Idx = KeyToIdx.Find(Key))
		{
			return &DenseVals[*Idx];
		}

		return nullptr;
	}

	void Remove(const TKey Key, const EAllowShrinking AllowShrinking)
	{
		if (const int32* Idx = KeyToIdx.Find(Key))
		{
			constexpr int32 Count = 1;
			DenseVals.RemoveAtSwap(*Idx, Count, AllowShrinking);

			if(*Idx < DenseVals.Num())
			{
				const TKey SwappedKey = DenseVals[*Idx].GetObjectPtr();
				KeyToIdx.FindChecked(SwappedKey) = *Idx;
			}

			KeyToIdx.Remove(Key);
		}
	}

	void Shrink()
	{
		DenseVals.Shrink();
	}

	void Reset()
	{
		DenseVals.Reset();
		KeyToIdx.Reset();
	}

	int32 Num() const { return DenseVals.Num(); }

	auto begin() { return DenseVals.begin(); }
	auto end() { return DenseVals.end(); }

	auto cbegin() const { return DenseVals.begin(); }
	auto cend() const { return DenseVals.end(); }

	const TVal& GetDenseAt(const int32 Idx) const { return DenseVals[Idx]; }
	TVal& GetDenseAt(const int32 Idx) { return DenseVals[Idx]; }

private:
	TMap<TKey, int32> KeyToIdx;
	TArray<TVal> DenseVals;
};

extern CHAOS_API int32 SkipDesyncTest;

class FPBDRigidsSolver;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnRewindDataResizeInternal, const int32);

class FRewindData
{
public:
	FRewindData(FPBDRigidsSolver* InSolver, int32 NumFrames, bool InRewindDataOptimization, int32 InCurrentFrame);
	FRewindData(FPBDRigidsSolver* InSolver, int32 NumFrames, int32 InCurrentFrame);

	void Init(FPBDRigidsSolver* InSolver, int32 NumFrames, bool InRewindDataOptimization, int32 InCurrentFrame);
	void Init(FPBDRigidsSolver* InSolver, int32 NumFrames, int32 InCurrentFrame);

private:
	void RegisterEvolutionCallbacks();

public: 
	FOnRewindDataResizeInternal RewindDataResize;

	int32 Capacity() const { return Managers.Capacity(); }
	int32 CurrentFrame() const { return CurFrame; }
	int32 GetLatestFrame() const { return LatestFrame; }
	int32 GetFramesSaved() const { return FramesSaved; }

	/** Check if we need to resize the rewind data */
	void ResizeRewindDataIfNeeded();

	FReal GetDeltaTimeForFrame(int32 Frame) const
	{
		ensure(Managers[Frame].FrameCreatedFor == Frame);
		return Managers[Frame].DeltaTime;
	}

	void RemoveObject(const FGeometryParticleHandle* Particle, const EAllowShrinking AllowShrinking=EAllowShrinking::Default)
	{
		DirtyParticles.Remove(Particle, AllowShrinking);
		DirtyParticlePreResimState.Remove(Particle, AllowShrinking);
		DirtyParticleErrors.Remove(Particle, AllowShrinking);
	}

	void RemoveObject(const FPBDJointConstraintHandle* Joint, const EAllowShrinking AllowShrinking = EAllowShrinking::Default)
	{
		DirtyJoints.Remove(Joint, AllowShrinking);
	}

	int32 CHAOS_API GetEarliestFrame_Internal() const;

	/* Extend the current history size to be sure to include the given frame */
	void CHAOS_API ExtendHistoryWithFrame(const int32 Frame);

	/* Clear all the simulation history after Frame */
	void CHAOS_API ClearPhaseAndFuture(FGeometryParticleHandle& Handle, int32 Frame, FFrameAndPhase::EParticleHistoryPhase Phase);

	/* Push a physics state in the rewind data at specified frame */
	void CHAOS_API PushStateAtFrame(FGeometryParticleHandle& Handle, int32 Frame, FFrameAndPhase::EParticleHistoryPhase Phase, const FVector& Position, const FQuat& Quaternion,
					const FVector& LinVelocity, const FVector& AngVelocity, const bool bShouldSleep);

	void CHAOS_API SetTargetStateAtFrame(FGeometryParticleHandle& Handle, const int32 Frame, FFrameAndPhase::EParticleHistoryPhase Phase,
		const FVector& Position, const FQuat& Quaternion, const FVector& LinVelocity, const FVector& AngVelocity, const bool bShouldSleep);

	void CHAOS_API ForceResimAsFollower(FGeometryParticleHandle* Handle, bool bForceResim);

	int32 CHAOS_API GetLatestReceivedTargetStateFrame(FGeometryParticleHandle& Handle) const;

	/** Extract some history information before cleaning/pushing state*/
	void ExtractHistoryState(FGeometryParticleHandle& Handle, int32& PositionValidCount, int32& VelocityValidCount, int32& PositionNextIterator, int32& VelocityNextIterator)
	{
		FDirtyParticleInfo& Info = FindOrAddDirtyObj(Handle);
		Info.GetHistory().ExtractHistoryState(PositionValidCount, VelocityValidCount, PositionNextIterator, VelocityNextIterator);
	}

	/** Restore some history information after cleaning/pushing state*/
	void RestoreHistoryState(FGeometryParticleHandle& Handle, const int32& PositionValidCount, const int32& VelocityValidCount, const int32& PositionNextIterator, const int32& VelocityNextIterator)
	{
		FDirtyParticleInfo& Info = FindOrAddDirtyObj(Handle);
		Info.GetHistory().RestoreHistoryState(PositionValidCount, VelocityValidCount, PositionNextIterator, VelocityNextIterator);
	}

	/* Query the state of particles from the past. Can only be used when not already resimming*/
	FGeometryParticleState CHAOS_API GetPastStateAtFrame(const FGeometryParticleHandle& Handle, int32 Frame, FFrameAndPhase::EParticleHistoryPhase Phase = FFrameAndPhase::EParticleHistoryPhase::PostPushData) const;

	/* Query the state of joints from the past. Can only be used when not already resimming*/
	FJointState CHAOS_API GetPastJointStateAtFrame(const FPBDJointConstraintHandle& Handle, int32 Frame, FFrameAndPhase::EParticleHistoryPhase Phase = FFrameAndPhase::EParticleHistoryPhase::PostPushData) const;

	IResimCacheBase* GetCurrentStepResimCache() const
	{
		return Managers[CurFrame].ExternalResimCache.Get();
	}

	void CHAOS_API DumpHistory_Internal(const int32 FramePrintOffset, const FString& Filename = FString("Dump"));

	/** Check if a resim cache based on IResimCacheBase (FEvolutionResimCache by default) is being used. Read FPhysicsSolverBase.SetUseCollisionResimCache() for more info. */
	bool GetUseCollisionResimCache() const;

	/** Called just before physics is solved */
	template <typename CreateCache>
	void AdvanceFrame(FReal DeltaTime, const CreateCache& CreateCacheFunc)
	{
		QUICK_SCOPE_CYCLE_COUNTER(RewindDataAdvance);
		Managers[CurFrame].DeltaTime = DeltaTime;
		Managers[CurFrame].FrameCreatedFor = CurFrame;
		TUniquePtr<IResimCacheBase>& ResimCache = Managers[CurFrame].ExternalResimCache;

		if (GetUseCollisionResimCache())
		{
			if (IsResim())
			{
				if (ResimCache)
				{
					ResimCache->SetResimming(true);
				}
			}
			else
			{
				if (ResimCache)
				{
					ResimCache->ResetCache();
				}
				else
				{
					ResimCache = CreateCacheFunc();
				}
				ResimCache->SetResimming(false);
			}
		}
		else
		{
			ResimCache.Reset();
		}

		AdvanceFrameImp(ResimCache.Get());
	}

	void FinishFrame();

	/** Advance the frame counter without running desync checks or other per-frame logic.
	 *  Used during adaptive resim coalescing to skip intermediate frames. */
	void SkipFrames(int32 NumFramesToSkip);

	bool IsResim() const
	{
		return CurFrame < LatestFrame;
	}

	bool IsFinalResim() const
	{
		return (CurFrame + 1) == LatestFrame;
	}

	//Number of particles that we're currently storing history for
	int32 GetNumDirtyParticles() const { return DirtyParticles.Num(); }

	/** Called just before Proxy::PushToPhysicsState is called */
	void PushGTDirtyData(const FDirtyPropertiesManager& SrcManager,const int32 SrcDataIdx,const FDirtyProxy& Dirty, const FShapeDirtyData* ShapeDirtyData);

	/** Called post solve but just before PQ are applied to XR */
	void PushPTDirtyData(TPBDRigidParticleHandle<FReal,3>& Rigid,const int32 SrcDataIdx);

	/** Call this to mark specific particle as dirty and cache its current data, expected to be called at the start of OnPreSimulate_Internal, it caches data in FFrameAndPhase::PostPushData */
	void CHAOS_API MarkDirtyFromPT(FGeometryParticleHandle& Handle);
	
	/** Call this to mark specific joint as dirty and cache its current data, expected to be called at the start of OnPreSimulate_Internal, it caches data in FFrameAndPhase::PostPushData */
	void CHAOS_API MarkDirtyJointFromPT(FPBDJointConstraintHandle& Handle);

	void CHAOS_API SpawnProxyIfNeeded(FSingleParticlePhysicsProxy& Proxy);

	void DisableInHistoryFromFrame(IPhysicsProxyBase* PhysicsProxy, int32 DisableParticleOnFrame = INDEX_NONE);

	/** Add input history to the rewind data for future use while resimulating */
	void AddInputHistory(const TSharedPtr<FBaseRewindHistory>& InputHistory)
	{
		InputHistories.AddUnique(InputHistory.ToWeakPtr());
	}

	/** Remove input history from the rewind data */
	void RemoveInputHistory(const TSharedPtr<FBaseRewindHistory>& InputHistory)
	{
		InputHistories.Remove(InputHistory.ToWeakPtr());
	}

	/** Add input history for particle to the rewind data */
	void AddInputHistory(const TSharedPtr<FBaseRewindHistory>& InputHistory, Chaos::FGeometryParticleHandle* Particle)
	{
		AddInputHistory(InputHistory);
		if (Particle != nullptr)
		{
			InputParticleHistories.Add(Particle, InputHistory.ToWeakPtr());
		}
	}

	/** Remove input history for particle from the rewind data */
	void RemoveInputHistory(const TSharedPtr<FBaseRewindHistory>& InputHistory, Chaos::FGeometryParticleHandle* Particle)
	{
		RemoveInputHistory(InputHistory);
		if (Particle != nullptr)
		{
			InputParticleHistories.Remove(Particle);
		}
	}

	/** Add state history to the rewind data for future use while rewinding */
	void AddStateHistory(const TSharedPtr<FBaseRewindHistory>& StateHistory)
	{
		StateHistories.AddUnique(StateHistory.ToWeakPtr());
	}

	/** Remove state history from the rewind data */
	void RemoveStateHistory(const TSharedPtr<FBaseRewindHistory>& StateHistory)
	{
		StateHistories.Remove(StateHistory.ToWeakPtr());
	}

	/** Add state history for particle to the rewind data */
	void AddStateHistory(const TSharedPtr<FBaseRewindHistory>& StateHistory, Chaos::FGeometryParticleHandle* Particle)
	{
		AddStateHistory(StateHistory);
		if (Particle != nullptr)
		{
			StateParticleHistories.Add(Particle, StateHistory.ToWeakPtr());
		}
	}

	/** Remove state history for particle from the rewind data */
	void RemoveStateHistory(const TSharedPtr<FBaseRewindHistory>& StateHistory, Chaos::FGeometryParticleHandle* Particle)
	{
		RemoveStateHistory(StateHistory);
		if (Particle != nullptr)
		{
			StateParticleHistories.Remove(Particle);
		}
	}

	/** Apply inputs for specified frame from rewind data */
	UE_DEPRECATED(5.6, "Deprecated, ApplyInputs is no longer viable. Any custom states can be applied during IRewindCallback::ProcessInputs_Internal during resimulation. Example FNetworkPhysicsCallback")
		void ApplyInputs(const int32 ApplyFrame, const bool bResetSolver);

	/** Rewind to state for a specified frame from rewind data */
	UE_DEPRECATED(5.6, "Deprecated, RewindStates is no longer viable. Any custom states can be applied during IRewindCallback::ProcessInputs_Internal during resimulation. Example FNetworkPhysicsCallback")
		void RewindStates(const int32 RewindFrame, const bool bResetSolver);

	/** Move post-resim error correction data from RewindData to FPullPhysicsData for marshaling to GT where it can be used in render interpolation */
	void BufferPhysicsResults(TMap<const IPhysicsProxyBase*, struct FDirtyRigidParticleReplicationErrorData>& DirtyRigidErrors);

	/** Return the rewind data solver */
	const FPBDRigidsSolver* GetSolver() const { return Solver; }

	/** Find the first previous valid frame having received physics target from the server */
	int32 CHAOS_API FindValidResimFrame(const int32 RequestedFrame);

	/** Get and set the frame we resimulate from */
	const int32 GetResimFrame() const { return ResimFrame; }
	void SetResimFrame(int32 Frame) { ResimFrame = Frame; }
	const bool IsResimCooldown() const { return (LatestFrame < ResimCooldownUntilFrame); }

	/** Called at the end of resim */
	void EndResim();

	/** Check if a frame number is within the currently allowed rewind span */
	bool IsFrameWithinRewindHistory(int32 Frame) const { return Frame < LatestFrame && Frame >= GetEarliestFrame_Internal(); }

	/** Request a resimulation by setting a requested frame to rewind to
	* @param Particle can be used to mark a specific particle as the one triggering the resimulation */
	void CHAOS_API RequestResimulation(int32 RequestedFrame, Chaos::FGeometryParticleHandle* Particle = nullptr);

	/** This blocks any future resimulation to rewind back past the frame this is called on */
	void BlockResim();

	/** Get the latest frame resim has been blocked from rewinding past */
	const int32 GetBlockedResimFrame() const { return BlockResimFrame; }

	/** Set if RewindData optimizations should be enabled or not. 
	* Effect: Only alter the minimum required properties during a resim for particles not marked for FullResim */
	void SetRewindDataOptimization(bool InRewindDataOptimization) { bRewindDataOptimization = InRewindDataOptimization; }

	/** Check if we have received targets already for the last frame simulated,
	* if so compare those with the result of the simulation and if they desync return the frame value to request a rewind for to correct the desync
	* NOTE: This only happens when the client has desynced and is behind the server, so we receive server states for frames not yet simulated */
	const int32 CHAOS_API CompareTargetsToLastFrame();

	/** Get cached pre resimulation state for particle, if there is one */
	void CHAOS_API GetPreResimState(Chaos::FGeometryParticleHandle* GeometryParticle, FParticleSimpleState*& OutPreResimState);

	/** Remove cached pre resimulation state for particle*/
	void CHAOS_API RemovePreResimState(Chaos::FGeometryParticleHandle* GeometryParticle);

	static bool CHAOS_API CheckVectorThreshold(FVec3 A, FVec3 B, float Threshold);
	static bool CHAOS_API CheckQuaternionThreshold(FQuat A, FQuat B, float ThresholdDegrees);

private:
	friend class FPBDRigidsSolver;

	void CHAOS_API AdvanceFrameImp(IResimCacheBase* ResimCache);

	/** Called post-solve from the evolution with all particles that are dirty after solving physics. Note: Before PQ are applied to XR */
	void ProcessDirtyPTParticles(const TParticleView<TPBDRigidParticles<FReal, 3>>& DirtyPTParticles);

	/** Called during integration just before KinematicTargets are processed with all active kinematic particles */
	void ProcessDirtyKinematicTargets(const TParticleView<TPBDRigidParticles<FReal, 3>>& ActiveKinematicParticles);

	/** Called during integration just before KinematicTargets are processed */
	void CacheKinematicTarget(TPBDRigidParticleHandle<FReal, 3>& Rigid);

	/** Called to cache a particles current state into a specified phase of the current frame
	* @param bDirty will mark the particle as dirty in the rewind history which will continue to cache it for FRewindData::Capacity() amount of frames */
	void CacheDirtyParticleData(FGeometryParticleHandle* Geometry, const FFrameAndPhase::EParticleHistoryPhase& CurrentPhase, const bool& bDirty);

	/** Called to cache a joints current state into a specified phase of the current frame
	* @param bDirty will mark the joint as dirty in the rewind history which will continue to cache it for FRewindData::Capacity() amount of frames */
	void CacheDirtyJointData(FPBDJointConstraintHandle* Joint, const FFrameAndPhase::EParticleHistoryPhase& CurrentPhase, const bool& bDirty);

	/** Caches data for all particles and joints marked dirty in RewindData
	* @param CurrentPhase Data gets cached under this phase for the current frame, be sure to call this from the corresponding phase during physics */
	void CacheCurrentDirtyData(const FFrameAndPhase::EParticleHistoryPhase& CurrentPhase);

	struct FFrameManagerInfo
	{
		TUniquePtr<IResimCacheBase> ExternalResimCache;

		//Note that this is not exactly the same as which frame this manager represents. 
		//A manager can have data for two frames at once, the important part is just knowing which frame it was created on so we know whether the physics data can rely on it
		//Consider the case where nothing is dirty from GT and then an object moves from the simulation, in that case it needs a manager to record the data into
		int32 FrameCreatedFor = INDEX_NONE;
		FReal DeltaTime = 0.0f;
	};

	template <typename THistoryType, typename TObj>
	struct TDirtyObjectInfo
	{
	private:
		THistoryType History;
		TObj* ObjPtr;
		FDirtyPropertiesPool* PropertiesPool;
	public:
		int32 DirtyDynamics = INDEX_NONE;	//Only used by particles, indicates the dirty properties was written to.
		int32 LastDirtyFrame;	//Track how recently this was made dirty
		int32 InitializedOnStep = INDEX_NONE;	//if not INDEX_NONE, it indicates we saw initialization during rewind history window
		bool bResimAsFollower = true;	//Indicates the particle will always resim in the exact same way from game thread data
		bool bNeedsResim = false;	//This particle needs resimulation, should have a higher priority when checking for valid rewind frames

		TDirtyObjectInfo(FDirtyPropertiesPool& InPropertiesPool, TObj& InObj, const int32 CurFrame, const int32 NumFrames, const bool bCacheOnePhase)
			: History(NumFrames, bCacheOnePhase)
			, ObjPtr(&InObj)
			, PropertiesPool(&InPropertiesPool)
			, LastDirtyFrame(CurFrame)
		{
		}

		TDirtyObjectInfo(TDirtyObjectInfo&& Other)
			: History(MoveTemp(Other.History))
			, ObjPtr(Other.ObjPtr)
			, PropertiesPool(Other.PropertiesPool)
			, LastDirtyFrame(Other.LastDirtyFrame)
			, InitializedOnStep(Other.InitializedOnStep)
			, bResimAsFollower(Other.bResimAsFollower)
			, bNeedsResim(Other.bNeedsResim)
		{
			Other.PropertiesPool = nullptr;
		}

		~TDirtyObjectInfo()
		{
			if (PropertiesPool)
			{
				History.Release(*PropertiesPool);
			}
		}

		TDirtyObjectInfo(const TDirtyObjectInfo& Other) = delete;

		TObj* GetObjectPtr() const { return ObjPtr; }

		UE_DEPRECATED(5.7, "Deprecated, Use GetHistory() and MarkDirty() individually instead")
		THistoryType& AddFrame(const int32 Frame)
		{
			LastDirtyFrame = Frame;
			return History;
		}

		void ClearPhaseAndFuture(const FFrameAndPhase FrameAndPhase)
		{
			History.ClearEntryAndFuture(FrameAndPhase);
		}

		const THistoryType& GetHistory() const
		{
			return History;
		}

		THistoryType& GetHistory()
		{
			return History;
		}

		void MarkDirty(const int32 Frame)
		{
			LastDirtyFrame = Frame;
		}
	};

	using FDirtyParticleInfo = TDirtyObjectInfo<FGeometryParticleStateBase, FGeometryParticleHandle>;
	using FDirtyJointInfo = TDirtyObjectInfo<FJointStateBase, FPBDJointConstraintHandle>;

	struct FDirtyParticleErrorInfo
	{
	private:
		FGeometryParticleHandle* HandlePtr;
		FVec3 ErrorX = { 0,0,0 };
		FQuat ErrorR = FQuat::Identity;

	public:
		FDirtyParticleErrorInfo(FGeometryParticleHandle& InHandle) : HandlePtr(&InHandle)
		{ }

		void AccumulateError(FVec3 NewErrorX, FQuat NewErrorR)
		{
			ErrorX += NewErrorX;
			ErrorR *= NewErrorR;
		}

		FGeometryParticleHandle* GetObjectPtr() const { return HandlePtr; }
		FVec3 GetErrorX() const { return ErrorX; }
		FQuat GetErrorR() const { return ErrorR; }
	};

	template <typename TDirtyObjs, typename TObj>
	auto* FindDirtyObjImp(TDirtyObjs& DirtyObjs, TObj& Handle)
	{
		return DirtyObjs.Find(&Handle);
	}

	template <typename TDirtyObjs, typename TObj>
	const auto* FindDirtyObjImp(const TDirtyObjs& DirtyObjs, TObj& Handle) const
	{
		return DirtyObjs.Find(&Handle);
	}

	FDirtyParticleInfo* FindDirtyObj(const FGeometryParticleHandle& Handle)
	{
		return FindDirtyObjImp(DirtyParticles, Handle);
	}

	FDirtyJointInfo* FindDirtyObj(const FPBDJointConstraintHandle& Handle)
	{
		return FindDirtyObjImp(DirtyJoints, Handle);
	}

	const FDirtyParticleInfo* FindDirtyObj(const FGeometryParticleHandle& Handle) const
	{
		return FindDirtyObjImp(DirtyParticles, Handle);
	}

	const FDirtyJointInfo* FindDirtyObj(const FPBDJointConstraintHandle& Handle) const
	{
		return FindDirtyObjImp(DirtyJoints, Handle);
	}

	template <typename TDirtyObjs, typename TObj>
	auto& FindOrAddDirtyObjImp(TDirtyObjs & DirtyObjs, TObj & Handle, const int32 InitializedOnFrame = INDEX_NONE)
	{
		if (auto Info = DirtyObjs.Find(&Handle))
		{
			return *Info;
		}

		using TDirtyObj = decltype(NoRefHelper(DirtyObjs.GetDenseAt(0)));
		TDirtyObj& Info = DirtyObjs.Add(&Handle, TDirtyObj(PropertiesPool, Handle, CurFrame, Managers.Capacity(), bRewindDataOptimization));
		Info.InitializedOnStep = InitializedOnFrame;
		return Info;
	}

	FDirtyParticleInfo& FindOrAddDirtyObj(FGeometryParticleHandle& Handle, const int32 InitializedOnFrame = INDEX_NONE)
	{
		return FindOrAddDirtyObjImp(DirtyParticles, Handle, InitializedOnFrame);
	}

	FDirtyJointInfo& FindOrAddDirtyObj(FPBDJointConstraintHandle& Handle, const int32 InitializedOnFrame = INDEX_NONE)
	{
		return FindOrAddDirtyObjImp(DirtyJoints, Handle, InitializedOnFrame);
	}

	template <typename TObjState, typename TDirtyObjs, typename TObj>
	auto GetPastStateAtFrameImp(const TDirtyObjs& DirtyObjs, const TObj& Handle, int32 Frame, FFrameAndPhase::EParticleHistoryPhase Phase) const
	{
		ensure(!IsResim());
		ensure(Frame >= LatestFrame - FramesSaved);

		const auto* Info = DirtyObjs.Find(&Handle);
		const auto* State = Info ? &Info->GetHistory() : nullptr;
		return TObjState(State, Handle, PropertiesPool, { Frame, Phase });
	}

	/** Returns whether rewinding to RewindFrame is possible (if we have history for the frame etc.) */
	bool CanRewindToFrame(int32 RewindFrame);

	/** Returns whether rewinding to RewindFrame is allowed (if we are currently under cooldown etc.) */
	bool AllowRewindToFrame(int32 RewindFrame) const;

	/** Apply the cached history state for the given frame cached particles and joints */
	void RewindToFrame(int32 RewindFrame);

	/** Apply targets positions and velocities while resimulating */
	void ApplyTargets(const int32 Frame, const bool bResetSimulation);

	/** Apply resim data for objects not simulating during resimulation */
	void StepNonResimParticles(const int32 Frame);

	template <typename TDirtyInfo>
	static void DesyncObject(TDirtyInfo& Info, const FFrameAndPhase FrameAndPhase)
	{
		Info.ClearPhaseAndFuture(FrameAndPhase);
		Info.GetObjectPtr()->SetSyncState(ESyncState::HardDesync);
	}

	TCircularBuffer<FFrameManagerInfo> Managers;
	FDirtyPropertiesPool PropertiesPool;	//must come before DirtyParticles since it relies on it (and used in destruction)

	TDirtyObjects<FDirtyParticleInfo> DirtyParticles;
	TDirtyObjects<FDirtyJointInfo> DirtyJoints;
	TDirtyObjects<FParticleSimpleState> DirtyParticlePreResimState;
	TDirtyObjects<FDirtyParticleErrorInfo> DirtyParticleErrors;

	TArray<TWeakPtr<FBaseRewindHistory>> InputHistories; // Todo, deprecate in favor of Particle->Input map?
	TArray<TWeakPtr<FBaseRewindHistory>> StateHistories; // Todo, deprecate in favor of Particle->State map?

	TMap<Chaos::FGeometryParticleHandle*, TWeakPtr<FBaseRewindHistory>> InputParticleHistories;
	TMap<Chaos::FGeometryParticleHandle*, TWeakPtr<FBaseRewindHistory>> StateParticleHistories;

	FPBDRigidsSolver* Solver;
	int32 CurFrame;
	int32 LatestFrame;
	int32 FramesSaved;
	int32 DataIdxOffset;
	bool bNeedsSave;	//Indicates that some data is pointing at head and requires saving before a rewind
	bool bRewindDataOptimization;
	int32 ResimFrame = INDEX_NONE;
	int32 ResimCooldownUntilFrame = INDEX_NONE;
	int32 ResimClampedFrames = INDEX_NONE;
	int32 LatestTargetFrame;
	bool AllowResizingFromTickTrate = false;
	Chaos::FReal InitializedDeltaTime = 0.0f;

	// Used to measure how long a resim takes in realtime and then wait that much before allowing to trigger a new resim
	uint64 ResimStartCycleCounter = 0;
	uint64 ResimPerformanceCooldownCycleCounter = 0;

	// Used to block rewinding past a physics change we currently don't handle
	int32 BlockResimFrame = INDEX_NONE;

	// Properties for EResimFrameValidation::IslandValidation logic
	TArray<const Private::FPBDIsland*> IslandValidationIslands;
	TArray<const FGeometryParticleHandle*> IslandValidationIslandParticles;
	
	/** Dynamic particle handles to be set in bulk */
	TArray<FPBDRigidParticleHandle*> DynamicHandlesUpdateHolder;
	
	/** Sleeping particle handles to be set in bulk */
	TArray<FPBDRigidParticleHandle*> SleepingHandlesUpdateHolder;
	
	/** Static particle handles to be set in bulk */
	TArray<FPBDRigidParticleHandle*> StaticHandlesUpdateHolder;
	
	/** Kinematic particle handles to be set in bulk */
	TArray<FPBDRigidParticleHandle*> KinematicHandlesUpdateHolder;

	TSet<const FGeometryParticleHandle*> ParticlesToForceResimAsFollower;

	/** Average number of predicted frames, overall estimated value */
	float NumPredictedFramesAverage = 0.0f;

	// BUBBLE RESIM
	// Bubble resim needs to be enabled, see Chaos::ResimConsoleVars::bIsBubbleResimEnabled
	// Bubble resim mitigates the cost of resimulations by restricting the amount of particles being resimulated
	// to those for which resimulation could lead to a change in state compared to the original predictive simulation
public:
	// This can be used to determine if an update that affects physics and restricts its effects to within an AABB can be skipped during a resim
	bool CHAOS_API OverlapsAnyResimBubble(const Chaos::FAABB3& InAABB) const;
	
#if WITH_CHAOS_VISUAL_DEBUGGER
	// Traces the particles that were modified during rewind. in the bubble resim case this also traces particles that were frozen.
	void TraceParticlesModifiedDuringRewind();
#endif
private:
	// Freeze particles outside resim bubbles before rewind/resim
	void FreezeParticlesOutsideResimBubbles();
	// Unfreeze particles that were frozen, this happens at the end of a resim (after all the resim frames have occurred)
	void UnfreezeParticlesOutsideResimBubbles();
	// Util function to get a particle's swept AABB at a given frame and phase of the history
	FAABB3 GetParticleSweptAABB(const FGeometryParticleHandle* ParticleHandle, FDirtyParticleInfo* DirtyParticleInfo, const FFrameAndPhase& AtFrameAndPhase, float HorizonDuration, float ExtraSweptDistance, float ExtraThickness) const;
	// Adds all particles needing resim and particle likely to interact with them during resim to OutParticlesOverlappingResimBubbles
	// This is accomplished by creating resim "bubbles" around particles needing resim and growing them as they overlap other particles
	void FindParticlesOverlappingResimBubbles(const FFrameAndPhase& RewindFrameAndPhase, float TotalResimDuration, TSet<FGeometryParticleHandle*>& OutParticlesOverlappingResimBubbles);
	// Used to keep track of objects we "freeze" during resim (by making them kinematic and marking them FrozenDuringResim)
	// and their object state prior to the freezing so it can be set back to its original value
	TArray<TPair<Chaos::FPhysicsProxyHandle, EObjectStateType>> FrozenParticles;
#if WITH_CHAOS_VISUAL_DEBUGGER
	// Used to keep track of which particles were rewound during the last rewind. Emptied at the end of resim.
	TArray<FGeometryParticleHandle*> ParticlesModifiedDuringRewind;
#endif

	// Resim bubbles start as the swept AABBs of particles needing resim, and grow to encompass nearby kinematic and dynamic particles involved in the resim
	// Their formation happens in FindParticlesOverlappingResimBubbles()
	TArray<Chaos::FAABB3> ResimBubbles;
	// END OF BUBBLE RESIM

	template <typename TObj>
	bool IsResimAndInSync(const TObj& Handle) const { return IsResim() && Handle.SyncState() == ESyncState::InSync; }

	template <bool bSkipDynamics, typename TDirtyInfo>
	void DesyncIfNecessary(TDirtyInfo& Info, const FFrameAndPhase FrameAndPhase);

	void CachePreResimState(FGeometryParticleHandle& Handle);

	template<typename TObj>
	void AccumulateErrorIfNecessary(TObj& Handle, const FFrameAndPhase FrameAndPhase) { }
};

struct FResimDebugInfo
{
	double ResimTime = 0.0;
	int32 FramesCoalesced = 0;
	int32 MaxMultiplierUsed = 1;
};

/** Used by user code to determine when rewind should occur and gives it the opportunity to record any additional data */
class IRewindCallback
{
public:
	virtual ~IRewindCallback() = default;
	
	/** Called just before ProcessInputs_Internal */
	virtual void PreProcessInputs_Internal(int32 PhysicsStep) {}

	/** Called before any sim callbacks are triggered but after physics data has marshaled over
	*   This means brand new physics particles are already created for example, and any pending game thread modifications have happened
	*   See ISimCallbackObject for recording inputs to callbacks associated with this PhysicsStep */
	virtual void ProcessInputs_Internal(int32 PhysicsStep, const TArray<FSimCallbackInputAndObject>& SimCallbackInputs) {}
	
	/** Called just after ProcessInputs_Internal */
	virtual void PostProcessInputs_Internal(int32 PhysicsStep) {}

	/** Called after any presim callbacks are triggered and after physics data has marshaled over in order to modify the sim callback outputs */
	virtual void ApplyCallbacks_Internal(int32 PhysicsStep, const TArray<ISimCallbackObject*>& SimCallbackObjects) {}

	/** Called before any inputs are marshalled over to the physics thread.
	*	The physics state has not been applied yet, and cannot be inspected anyway because this is triggered from the external thread (game thread)
	*	Gives user the ability to modify inputs or record them - this can help with reducing latency if you want to act on inputs immediately
	*/
	virtual void ProcessInputs_External(int32 PhysicsStep, const TArray<FSimCallbackInputAndObject>& SimCallbackInputs) {}

	/** Called before inputs are split into potential sub-steps and marshaled over to the physics thread.
	*	The physics state has not been applied yet, and cannot be inspected anyway because this is triggered from the external thread (game thread)
	*	Gives user the ability to call GetProducerInputData_External one last time.
	*	Input data is shared amongst sub-steps. If NumSteps > 1 it means any input data injected will be shared for all sub-steps generated
	*/
	virtual void InjectInputs_External(int32 PhysicsStep, int32 NumSteps) {}

	/** Called on game thread for each physics thread task that gets queued up, before async inputs are marshaled.
	*	Gives user the ability to call GetProducerInputData_External for each physics step.
	*	Input data is shared amongst sub-steps. If NumSteps > 1 it means any input data injected will be shared for all sub-steps generated
	*/
	virtual void ProduceInputs_External(int32 PhysicsStep) {}

	/** Called after sim step to give the option to rewind. Any pending inputs for the next frame will remain in the queue
	*   Return the PhysicsStep to start resimulating from. Resim will run up until latest step passed into RecordInputs (i.e. latest physics sim simulated so far)
	*   Return INDEX_NONE to indicate no rewind
	*/
	virtual int32 TriggerRewindIfNeeded_Internal(int32 LatestStepCompleted) { return INDEX_NONE; }

	/** Add potential resimulation requests to the rewind data */
	virtual void AddResimulationRequest_Internal(const int32 PhysicsStep, const float DeltaSeconds) {}

	/** Called before each rewind step. This is to give user code the opportunity to trigger other code before each rewind step
	*   Usually to simulate external systems that ran in lock step with the physics sim
	*/
	virtual void PreResimStep_Internal(int32 PhysicsStep, bool bFirstStep) {}

	/** Called after each rewind step. This is to give user code the opportunity to trigger other code after each rewind step
	*   Usually to simulate external systems that ran in lock step with the physics sim
	*/
	virtual void PostResimStep_Internal(int32 PhysicsStep) {}

	/** Register a sim callback onto the rewind callback */
	virtual void RegisterRewindableSimCallback_Internal(ISimCallbackObject* Callback) {}

	/** Unregister a sim callback from the rewind callback */
	virtual void UnregisterRewindableSimCallback_Internal(ISimCallbackObject* Callback) {}

	/** Called When resim is finished with debug information about the resim */
	virtual void SetResimDebugInfo_Internal(const FResimDebugInfo& ResimDebugInfo) {}

	/** Rewind Data holding the callback */
	Chaos::FRewindData* RewindData = nullptr;
};
}
