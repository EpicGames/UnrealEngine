// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Chaos/Real.h"
#include "UObject/ObjectMacros.h"
#include "SolverEventFilters.generated.h"




	USTRUCT(Blueprintable)
	struct FSolverTrailingFilterSettings
	{
		GENERATED_USTRUCT_BODY()

		FSolverTrailingFilterSettings() 
			: FilterEnabled(false)
			, MinMass(0.0f)
			, MinSpeed(0.f)
			, MinVolume(0.f)
		{}

		/** Filter is enabled. */
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|TrailingData Generation")
		bool FilterEnabled;

		/** The minimum mass threshold for the results (compared with min of particle 1 mass and particle 2 mass). */
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|TrailingData Generation", meta = (DisplayName = "Min Mass Threshold"))
		float MinMass;

		/** */
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|TrailingData Generation", meta = (DisplayName = "Min Speed Threshold"))
		float MinSpeed;

		/** */
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|TrailingData Generation", meta = (DisplayName = "Min Volume Threshold"))
		float MinVolume;
	};

	USTRUCT(BlueprintType)
	struct FSolverCollisionFilterSettings
	{
		GENERATED_USTRUCT_BODY()

		FSolverCollisionFilterSettings()
			: FilterEnabled(false)
			, MinMass(0.0f)
			, MinSpeed(0.0f)
			, MinImpulse(0.0f)
		{}

		/** Filter is enabled. */
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|CollisionData Generation")
		bool FilterEnabled;

		/** The minimum mass threshold for the results (compared with min of particle 1 mass and particle 2 mass). */
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|CollisionData Generation", meta = (DisplayName = "Min Mass Threshold"))
		float MinMass;

		/** The min velocity threshold for the results (compared with min of particle 1 speed and particle 2 speed). */
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|CollisionData Generation", meta = (DisplayName = "Min Speed Threshold"))
		float MinSpeed;

		/** The minimum impulse threshold for the results. */
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|CollisionData Generation", meta = (DisplayName = "Min Impulse Threshold"))
		float MinImpulse;

	};

	USTRUCT(BlueprintType)
	struct FSolverBreakingFilterSettings
	{
		GENERATED_USTRUCT_BODY()

			FSolverBreakingFilterSettings()
			: FilterEnabled(false)
			, MinMass(0.0f)
			, MinSpeed(0.0f)
			, MinVolume(0.0f)
		{}

		/** Filter is enabled. */
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|BreakingData Generation")
		bool FilterEnabled;

		/** The minimum mass threshold for the results (compared with min of particle 1 mass and particle 2 mass). */
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|BreakingData Generation", meta = (DisplayName = "Min Mass Threshold"))
		float MinMass;

		/** The min velocity threshold for the results (compared with min of particle 1 speed and particle 2 speed). */
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|BreakingData Generation", meta = (DisplayName = "Min Speed Threshold"))
		float MinSpeed;

		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|BreakingData Generation", meta = (DisplayName = "Min Volume Threshold"))
		float MinVolume;

	};

	USTRUCT(BlueprintType)
	struct FSolverRemovalFilterSettings
	{
		GENERATED_USTRUCT_BODY()

		FSolverRemovalFilterSettings()
			: FilterEnabled(false)
			, MinMass(0.0f)
			, MinVolume(0.0f)
		{}

		/** Filter is enabled. */
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|BreakingData Generation")
		bool FilterEnabled;

		/** The minimum mass threshold for the results (compared with min of particle 1 mass and particle 2 mass). */
		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|BreakingData Generation", meta = (DisplayName = "Min Mass Threshold"))
		float MinMass;

		UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|BreakingData Generation", meta = (DisplayName = "Min Volume Threshold"))
		float MinVolume;

	};


namespace Chaos
{

	struct FCollidingData;

	struct FTrailingData;

	struct FBreakingData;

	struct FRemovalData;

	class FSolverCollisionEventFilter
	{
	public:
		FSolverCollisionEventFilter() {}
		FSolverCollisionEventFilter(const FSolverCollisionFilterSettings& InSettings) : Settings(InSettings) {}

		CHAOS_API bool Pass(const Chaos::FCollidingData& InData) const;
		bool Enabled() const { return Settings.FilterEnabled; }
		void UpdateFilterSettings(const FSolverCollisionFilterSettings& InSettings) { Settings = InSettings; }

		FSolverCollisionFilterSettings Settings;
	};

	class FSolverTrailingEventFilter
	{
	public:
		FSolverTrailingEventFilter() {}
		FSolverTrailingEventFilter(const FSolverTrailingFilterSettings &InSettings) : Settings(InSettings) {}

		CHAOS_API bool Pass(const Chaos::FTrailingData& InData) const;
		bool Enabled() const { return Settings.FilterEnabled; }
		void UpdateFilterSettings(const FSolverTrailingFilterSettings& InSettings) { Settings = InSettings; }

		FSolverTrailingFilterSettings Settings;
	};

	class FSolverBreakingEventFilter
	{
	public:
		FSolverBreakingEventFilter() {}
		FSolverBreakingEventFilter(const FSolverBreakingFilterSettings& InSettings) : Settings(InSettings) {}

		CHAOS_API bool Pass(const Chaos::FBreakingData& InData) const;
		bool Enabled() const { return Settings.FilterEnabled; }
		void UpdateFilterSettings(const FSolverBreakingFilterSettings& InSettings) { Settings = InSettings; }

		FSolverBreakingFilterSettings Settings;
	};

	class FSolverRemovalEventFilter
	{
	public:
		FSolverRemovalEventFilter() {}
		FSolverRemovalEventFilter(const FSolverRemovalFilterSettings& InSettings) : Settings(InSettings) {}

		CHAOS_API bool Pass(const Chaos::FRemovalData& InData) const;
		bool Enabled() const { return Settings.FilterEnabled; }
		void UpdateFilterSettings(const FSolverRemovalFilterSettings& InSettings) { Settings = InSettings; }

		FSolverRemovalFilterSettings Settings;
	};


	/**
	 * Container for the Solver Event Filters that have settings exposed through the Solver Actor
	 */
	class FSolverEventFilters
	{
	public:
		FSolverEventFilters()
			: CollisionFilter(new FSolverCollisionEventFilter())
			, BreakingFilter(new FSolverBreakingEventFilter())
			, TrailingFilter(new FSolverTrailingEventFilter())
			, RemovalFilter(new FSolverRemovalEventFilter())
			, CollisionEventsEnabled(false)
			, BreakingEventsEnabled(false)
			, TrailingEventsEnabled(false)
			, RemovalEventsEnabled(false)
		{}

		void SetGenerateCollisionEvents(bool bDoGenerate) { CollisionEventsEnabled = bDoGenerate; }
		void SetGenerateBreakingEvents(bool bDoGenerate) { BreakingEventsEnabled = bDoGenerate; }
		void SetGenerateTrailingEvents(bool bDoGenerate) { TrailingEventsEnabled = bDoGenerate; }
		void SetGenerateRemovalEvents(bool bDoGenerate) { RemovalEventsEnabled = bDoGenerate; }

		/* Const access */
		FSolverCollisionEventFilter* GetCollisionFilter() const { return CollisionFilter.Get(); }
		FSolverBreakingEventFilter* GetBreakingFilter() const { return BreakingFilter.Get(); }
		FSolverTrailingEventFilter* GetTrailingFilter() const { return TrailingFilter.Get(); }
		FSolverRemovalEventFilter* GetRemovalFilter() const { return RemovalFilter.Get(); }

		/* non-const access */
		FSolverCollisionEventFilter* GetCollisionFilter() { return CollisionFilter.Get(); }
		FSolverBreakingEventFilter* GetBreakingFilter() { return BreakingFilter.Get(); }
		FSolverTrailingEventFilter* GetTrailingFilter() { return TrailingFilter.Get(); }
		FSolverRemovalEventFilter* GetRemovalFilter() { return RemovalFilter.Get(); }

		bool IsCollisionEventEnabled() const { return CollisionEventsEnabled; }
		bool IsBreakingEventEnabled() const { return BreakingEventsEnabled; }
		bool IsTrailingEventEnabled() const { return TrailingEventsEnabled; }
		bool IsRemovalEventEnabled() const { return RemovalEventsEnabled; }

	private:

		TUniquePtr<FSolverCollisionEventFilter> CollisionFilter;
		TUniquePtr<FSolverBreakingEventFilter> BreakingFilter;
		TUniquePtr<FSolverTrailingEventFilter> TrailingFilter;
		TUniquePtr<FSolverRemovalEventFilter> RemovalFilter;

		bool CollisionEventsEnabled;
		bool BreakingEventsEnabled;
		bool TrailingEventsEnabled;
		bool RemovalEventsEnabled;
	};


} // namespace Chaos
