// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class ChaosCore : ModuleRules
    {
        public ChaosCore(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
                new string[] {
                "Core",
                "IntelISPC"
                }
            );

            PublicDefinitions.Add("COMPILE_WITHOUT_UNREAL_SUPPORT=0");

            if (Target.bUseChaosChecked == true)
            {
                PublicDefinitions.Add("CHAOS_CHECKED=1");
            }
            else
            {
                PublicDefinitions.Add("CHAOS_CHECKED=0");
            }

			if(Target.Configuration != UnrealTargetConfiguration.Shipping && Target.Configuration != UnrealTargetConfiguration.Test)
			{
				PublicDefinitions.Add("CHAOS_DEBUG_NAME=1");
			}
			else
			{
				PublicDefinitions.Add("CHAOS_DEBUG_NAME=0");
			}
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
        }
    }
}
