// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/AABB.h"
#include "Chaos/Real.h"
#include "Chaos/CoreSphere.h"
#include "Chaos/CoreCapsule.h"
#include "Chaos/CorePlane.h"
#include "HAL/IConsoleManager.h"
#include "AutoRTFM.h"
#if INTEL_ISPC
#include "AABB.ispc.generated.h"

static_assert(sizeof(ispc::FTransform) == sizeof(FTransform), "sizeof(ispc::FTransform) != sizeof(FTransform)");
static_assert(sizeof(ispc::FVector) == sizeof(Chaos::TVector<Chaos::FReal, 3>), "sizeof(ispc::FVector) != sizeof(Chaos::TVector<Chaos::FReal, 3>)");
#endif

#if !defined(CHAOS_AABB_TRANSFORM_ISPC_ENABLED_DEFAULT)
#define CHAOS_AABB_TRANSFORM_ISPC_ENABLED_DEFAULT 1
#endif

#if !defined(USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
#define USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING 0
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || (UE_BUILD_SHIPPING && !USE_ISPC_KERNEL_CONSOLE_VARIABLES_IN_SHIPPING)
static constexpr bool bChaos_AABBTransform_ISPC_Enabled = INTEL_ISPC && CHAOS_AABB_TRANSFORM_ISPC_ENABLED_DEFAULT;
#else
static bool bChaos_AABBTransform_ISPC_Enabled = CHAOS_AABB_TRANSFORM_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarChaosAABBTransformISPCEnabled(TEXT("p.Chaos.AABBTransform.ISPC"), bChaos_AABBTransform_ISPC_Enabled, TEXT("Whether to use ISPC optimizations when computing AABB transforms"));
#endif

static bool bChaos_AABBTransform_Optimized = false;
static FAutoConsoleVariableRef CVarChaosAABBTransformOptimized(TEXT("p.Chaos.AABBTransform.Optimized"), bChaos_AABBTransform_Optimized, TEXT("Whether to use optimized AABB transform"));

namespace Chaos
{

template <typename T, int d>
TVector<T, d> TAABB<T, d>::FindClosestPoint(const TVector<T, d>& StartPoint, const T Thickness) const
{
	TVector<T, d> Result(0);

	// clamp exterior to surface
	bool bIsExterior = false;
	for (int i = 0; i < 3; i++)
	{
		T v = StartPoint[i];
		if (v < MMin[i])
		{
			v = MMin[i];
			bIsExterior = true;
		}
		if (v > MMax[i])
		{
			v = MMax[i];
			bIsExterior = true;
		}
		Result[i] = v;
	}

	if (!bIsExterior)
	{
		TArray<Pair<T, TVector<T, d>>> Intersections;

		// sum interior direction to surface
		for (int32 i = 0; i < d; ++i)
		{
			auto PlaneIntersection = TCorePlane<T, d>(MMin - Thickness, -TVector<T, d>::AxisVector(i)).FindClosestPoint(Result, 0);
			Intersections.Add(MakePair((T)(PlaneIntersection - Result).Size(), -TVector<T, d>::AxisVector(i)));
			PlaneIntersection = TCorePlane<T, d>(MMax + Thickness, TVector<T, d>::AxisVector(i)).FindClosestPoint(Result, 0);
			Intersections.Add(MakePair((T)(PlaneIntersection - Result).Size(), TVector<T, d>::AxisVector(i)));
		}
		Intersections.Sort([](const Pair<T, TVector<T, d>>& Elem1, const Pair<T, TVector<T, d>>& Elem2) { return Elem1.First < Elem2.First; });

		if (!FMath::IsNearlyEqual(Intersections[0].First, (T)0.))
		{
			T SmallestDistance = Intersections[0].First;
			Result += Intersections[0].Second * Intersections[0].First;
			for (int32 i = 1; i < 3 && FMath::IsNearlyEqual(SmallestDistance, Intersections[i].First); ++i)
			{
				Result += Intersections[i].Second * Intersections[i].First;
			}
		}
	}
	return Result;
}

template <typename T, int d>
Pair<TVector<FReal, d>, bool> TAABB<T, d>::FindClosestIntersectionImp(const TVector<FReal, d>& StartPoint, const TVector<FReal, d>& EndPoint, const FReal Thickness) const
{
	TArray<Pair<FReal, TVector<FReal, d>>> Intersections;
	for (int32 i = 0; i < d; ++i)
	{
		auto PlaneIntersection = TCorePlane<FReal, d>(TVector<FReal, d>(MMin) - Thickness, -TVector<FReal, d>::AxisVector(i)).FindClosestIntersection(StartPoint, EndPoint, 0);
		if (PlaneIntersection.Second)
			Intersections.Add(MakePair((FReal)(PlaneIntersection.First - StartPoint).Size(), PlaneIntersection.First));
		PlaneIntersection = TCorePlane<FReal, d>(TVector<FReal, d>(MMax) + Thickness, TVector<FReal, d>::AxisVector(i)).FindClosestIntersection(StartPoint, EndPoint, 0);
		if (PlaneIntersection.Second)
			Intersections.Add(MakePair((FReal)(PlaneIntersection.First - StartPoint).Size(), PlaneIntersection.First));
	}
	Intersections.Sort([](const Pair<FReal, TVector<FReal, d>>& Elem1, const Pair<FReal, TVector<FReal, d>>& Elem2) { return Elem1.First < Elem2.First; });
	for (const auto& Elem : Intersections)
	{
		if (SignedDistance(Elem.Second) < (Thickness + 1e-4))
		{
			return MakePair(Elem.Second, true);
		}
	}
	return MakePair(TVector<FReal, d>(0), false);
}


template <typename T, int d>
bool TAABB<T, d>::Raycast(const TVector<FReal, d>& StartPoint, const TVector<FReal, d>& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, TVector<FReal, d>& OutPosition, TVector<FReal, d>& OutNormal, int32& OutFaceIndex) const
{
	return Raycasts::RayAabb(StartPoint, Dir, Length, Thickness, MMin, MMax, OutTime, OutPosition, OutNormal, OutFaceIndex);
}

template<typename T, typename U>
inline TAABB<T, 3> TransformedAABBHelper2(const TAABB<T, 3>& AABB, const TRigidTransform<U, 3>& SpaceTransform)
{
	// Full and empty bounds do not transform
	if (AABB.IsFull() || AABB.IsEmpty())
	{
		return AABB;
	}

	// NOTE: This is required for TAABB3<float> and TAABB3<double> with either float or double transform
	const TVec3<T> Translation = TVec3<T>(SpaceTransform.GetTranslation());
	const TRotation3<T> Rotation = TRotation3<T>(SpaceTransform.GetRotation());
	const TVec3<T> Scale = TVec3<T>(SpaceTransform.GetScale3D());

	// Center-relative scaled verts
	// NOTE: Scale may be negative, but it does not impact the bounds calculation
	const TVec3<T> Extent = T(0.5) * Scale * (AABB.Max() - AABB.Min());
	const TVec3<T> Vert0 = TVec3<T>(Extent.X, Extent.Y, Extent.Z);
	const TVec3<T> Vert1 = TVec3<T>(Extent.X, Extent.Y, -Extent.Z);
	const TVec3<T> Vert2 = TVec3<T>(Extent.X, -Extent.Y, Extent.Z);
	const TVec3<T> Vert3 = TVec3<T>(Extent.X, -Extent.Y, -Extent.Z);

	// Rotated center-relative scaled verts
	const TVec3<T> RVert0 = Rotation * Vert0;
	const TVec3<T> RVert1 = Rotation * Vert1;
	const TVec3<T> RVert2 = Rotation * Vert2;
	const TVec3<T> RVert3 = Rotation * Vert3;

	// Max rotated scaled center-relative extent
	TVec3<T> RExtent = TVec3<T>(0);
	RExtent = TVec3<T>::Max(RExtent, RVert0.GetAbs());
	RExtent = TVec3<T>::Max(RExtent, RVert1.GetAbs());
	RExtent = TVec3<T>::Max(RExtent, RVert2.GetAbs());
	RExtent = TVec3<T>::Max(RExtent, RVert3.GetAbs());

	// Transformed center
	// NOTE: This is where positive/negative scales matters
	const TVec3<T> Center = T(0.5) * (AABB.Min() + AABB.Max());
	const TVec3<T> TCenter = Translation + Rotation * (Scale * Center);

	// Transformed bounds
	return TAABB<T, 3>(TCenter - RExtent, TCenter + RExtent);
}

template<typename T, typename U>
inline TAABB<T, 3> InverseTransformedAABBHelper2(const TAABB<T, 3>& AABB, const TRigidTransform<U, 3>& SpaceTransform)
{
	// Full and empty bounds do not transform
	if (AABB.IsFull() || AABB.IsEmpty())
	{
		return AABB;
	}

	// NOTE: This is required for TAABB3<float> and TAABB3<double> with either float or double transform
	const TVec3<T> InvTranslation = -TVec3<T>(SpaceTransform.GetTranslation());
	const TRotation3<T> InvRotation = TRotation3<T>(SpaceTransform.GetRotation()).Inverse();
	const TVec3<T> InvScale = TVec3<T>(1) / TVec3<T>(SpaceTransform.GetScale3D());

	// Center-relative scaled verts
	const TVec3<T> Extent = T(0.5) * (AABB.Max() - AABB.Min());
	const TVec3<T> Vert0 = TVec3<T>(Extent.X, Extent.Y, Extent.Z);
	const TVec3<T> Vert1 = TVec3<T>(Extent.X, Extent.Y, -Extent.Z);
	const TVec3<T> Vert2 = TVec3<T>(Extent.X, -Extent.Y, Extent.Z);
	const TVec3<T> Vert3 = TVec3<T>(Extent.X, -Extent.Y, -Extent.Z);

	// Rotated center-relative scaled verts
	// Note: Scale may be negative but it does not affect the bounds calculation
	const TVec3<T> RVert0 = InvScale * (InvRotation * Vert0);
	const TVec3<T> RVert1 = InvScale * (InvRotation * Vert1);
	const TVec3<T> RVert2 = InvScale * (InvRotation * Vert2);
	const TVec3<T> RVert3 = InvScale * (InvRotation * Vert3);

	// Max rotated scaled center-relative extent
	TVec3<T> RExtent = TVec3<T>(0);
	RExtent = TVec3<T>::Max(RExtent, RVert0.GetAbs());
	RExtent = TVec3<T>::Max(RExtent, RVert1.GetAbs());
	RExtent = TVec3<T>::Max(RExtent, RVert2.GetAbs());
	RExtent = TVec3<T>::Max(RExtent, RVert3.GetAbs());

	// Transformed center
	// NOTE: This is where positive/negative scales matters
	const TVec3<T> Center = T(0.5) * (AABB.Min() + AABB.Max());
	const TVec3<T> TCenter =  InvScale * (InvRotation * (InvTranslation + Center));

	// Transformed bounds
	return TAABB<T, 3>(TCenter - RExtent, TCenter + RExtent);
}

template<typename T>
inline TAABB<T, 3> TransformedAABBHelper(const TAABB<T, 3>& AABB, const FMatrix44& SpaceTransform)
{
	// Full and empty bounds do not transform
	if (AABB.IsFull() || AABB.IsEmpty())
	{
		return AABB;
	}

	// Initialize to center
	FVec3 Translation(SpaceTransform.M[3][0], SpaceTransform.M[3][1], SpaceTransform.M[3][2]);
	FVec3 Min = Translation;
	FVec3 Max = Translation;

	// Compute extents per axis
	for (int32 i = 0; i < 3; ++i)
	{
		for (int32 j = 0; j < 3; ++j)	
		{
			FReal A = SpaceTransform.M[j][i] * AABB.Min()[j];
			FReal B = SpaceTransform.M[j][i] * AABB.Max()[j];
			if (A < B)
			{
				Min[i] += A;
				Max[i] += B;
			}
			else 
			{
				Min[i] += B;
				Max[i] += A;
			}
		}
	}

	return TAABB<T, 3>(Min, Max);
}

inline TAABB<FReal, 3> TransformedAABBHelperISPC(const TAABB<FReal, 3>& AABB, const FTransform& SpaceTransform)
{
	// Full and empty bounds do not transform
	if (AABB.IsFull() || AABB.IsEmpty())
	{
		return AABB;
	}

	check(bRealTypeCompatibleWithISPC);
#if INTEL_ISPC
	TVector<Chaos::FReal, 3> NewMin, NewMax;
	UE_AUTORTFM_OPEN
	{
		ispc::TransformedAABB((const ispc::FTransform&)SpaceTransform, (const ispc::FVector&)AABB.Min(), (const ispc::FVector&)AABB.Max(), (ispc::FVector&)NewMin, (ispc::FVector&)NewMax);
	};

	TAABB<FReal, 3> NewAABB(NewMin, NewMax);
	return NewAABB;
#else
	check(false);
	return TAABB<FReal, 3>::EmptyAABB();
#endif
}

inline TAABB<Chaos::FRealSingle, 3> TransformedAABBHelperISPC(const TAABB<Chaos::FRealSingle, 3>& AABB, const FTransform& SpaceTransform)
{
	// Full and empty bounds do not transform
	if (AABB.IsFull() || AABB.IsEmpty())
	{
		return AABB;
	}

	check(bRealTypeCompatibleWithISPC);
#if INTEL_ISPC
	static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::TVector<Chaos::FRealSingle, 3>), "sizeof(ispc::FVector3f) != sizeof(Chaos::TVector<Chaos::FRealSingle, 3>)");

	TVector<Chaos::FRealSingle, 3> NewMin, NewMax;
	UE_AUTORTFM_OPEN
	{
		ispc::TransformedAABBMixed((const ispc::FTransform&)SpaceTransform, (const ispc::FVector3f&)AABB.Min(), (const ispc::FVector3f&)AABB.Max(), (ispc::FVector3f&)NewMin, (ispc::FVector3f&)NewMax);
	};

	TAABB<Chaos::FRealSingle, 3> NewAABB(NewMin, NewMax);
	return NewAABB;
#else
	check(false);
	return TAABB<Chaos::FRealSingle, 3>::EmptyAABB();
#endif
}

inline TAABB<FReal, 3> TransformedAABBHelperISPC2(const TAABB<FReal, 3>& AABB, const FTransform& SpaceTransform)
{
	// Full and empty bounds do not transform
	if (AABB.IsFull() || AABB.IsEmpty())
	{
		return AABB;
	}

	check(bRealTypeCompatibleWithISPC);
#if INTEL_ISPC
	TVector<Chaos::FReal, 3> NewMin, NewMax;
	UE_AUTORTFM_OPEN
	{
		ispc::TransformedAABB2((const ispc::FTransform&)SpaceTransform, (const ispc::FVector&)AABB.Min(), (const ispc::FVector&)AABB.Max(), (ispc::FVector&)NewMin, (ispc::FVector&)NewMax);
	};

	TAABB<FReal, 3> NewAABB(NewMin, NewMax);
	return NewAABB;
#else
	check(false);
	return TAABB<FReal, 3>::EmptyAABB();
#endif
}

inline TAABB<Chaos::FRealSingle, 3> TransformedAABBHelperISPC2(const TAABB<Chaos::FRealSingle, 3>& AABB, const FTransform& SpaceTransform)
{
	// Full and empty bounds do not transform
	if (AABB.IsFull() || AABB.IsEmpty())
	{
		return AABB;
	}

	check(bRealTypeCompatibleWithISPC);
#if INTEL_ISPC
	static_assert(sizeof(ispc::FVector3f) == sizeof(Chaos::TVector<Chaos::FRealSingle, 3>), "sizeof(ispc::FVector3f) != sizeof(Chaos::TVector<Chaos::FRealSingle, 3>)");

	TVector<Chaos::FRealSingle, 3> NewMin, NewMax;
	UE_AUTORTFM_OPEN
	{
		ispc::TransformedAABBMixed((const ispc::FTransform&)SpaceTransform, (const ispc::FVector3f&)AABB.Min(), (const ispc::FVector3f&)AABB.Max(), (ispc::FVector3f&)NewMin, (ispc::FVector3f&)NewMax);
	};

	TAABB<Chaos::FRealSingle, 3> NewAABB(NewMin, NewMax);
	return NewAABB;
#else
	check(false);
	return TAABB<Chaos::FRealSingle, 3>::EmptyAABB();
#endif
}

template<typename T, int d>
TAABB<T, d> TAABB<T, d>::TransformedAABB(const Chaos::TRigidTransform<FReal, 3>& SpaceTransform) const
{
	if (bRealTypeCompatibleWithISPC && INTEL_ISPC && bChaos_AABBTransform_ISPC_Enabled )
	{
		if (bChaos_AABBTransform_Optimized)
		{
			return TransformedAABBHelperISPC2(*this, SpaceTransform);
		}
		else
		{
			return TransformedAABBHelperISPC(*this, SpaceTransform);
		}
	}
	else 
	{
		if (bChaos_AABBTransform_Optimized)
		{
			return TransformedAABBHelper2<T>(*this, SpaceTransform);
		}
		else
		{
			return TransformedAABBHelper<T>(*this, SpaceTransform.ToMatrixWithScale());
		}
	}
}

template<typename T, int d>
TAABB<T, d> TAABB<T, d>::TransformedAABB(const FMatrix& SpaceTransform) const
{
	return TransformedAABBHelper<T>(*this, SpaceTransform);
}


template<typename T, int d>
TAABB<T, d> TAABB<T, d>::TransformedAABB(const Chaos::PMatrix<FReal, 4, 4>& SpaceTransform) const
{
	return TransformedAABBHelper<T>(*this, SpaceTransform);
}

template<typename T, int d>
TAABB<T, d> TAABB<T, d>::TransformedAABB(const FTransform& SpaceTransform) const
{
	if (bRealTypeCompatibleWithISPC && INTEL_ISPC && bChaos_AABBTransform_ISPC_Enabled)
	{
		if (bChaos_AABBTransform_Optimized)
		{
			return TransformedAABBHelperISPC2(*this, SpaceTransform);
		}
		else
		{
			return TransformedAABBHelperISPC(*this, SpaceTransform);
		}
	}
	else 
	{
		if (bChaos_AABBTransform_Optimized)
		{
			return TransformedAABBHelper2<T>(*this, Chaos::TRigidTransform<FReal, 3>(SpaceTransform));
		}
		else
		{
			return TransformedAABBHelper<T>(*this, SpaceTransform.ToMatrixWithScale());
		}
	}
}

template<typename T, int d>
TAABB<T, d> TAABB<T, d>::InverseTransformedAABB(const FRigidTransform3& SpaceTransform) const
{
	if (bChaos_AABBTransform_Optimized)
	{
		return InverseTransformedAABBHelper2<T>(*this, SpaceTransform);
	}
	else
	{
		// Full and empty bounds do not transform
		if (IsFull() || IsEmpty())
		{
			return *this;
		}

		TVector<T, d> CurrentExtents = Extents();
		int32 Idx = 0;
		const TVector<T, d> MinToNewSpace = SpaceTransform.InverseTransformPosition(FVector(Min()));
		TAABB<T, d> NewAABB(MinToNewSpace, MinToNewSpace);
		NewAABB.GrowToInclude(SpaceTransform.InverseTransformPosition(FVector(Max())));

		for (int32 j = 0; j < d; ++j)
		{
			NewAABB.GrowToInclude(SpaceTransform.InverseTransformPosition(FVector(Min() + TVector<T, d>::AxisVector(j) * CurrentExtents)));
			NewAABB.GrowToInclude(SpaceTransform.InverseTransformPosition(FVector(Max() - TVector<T, d>::AxisVector(j) * CurrentExtents)));
		}

		return NewAABB;
	}
}
}

template class Chaos::TAABB<Chaos::FRealSingle, 3>;
template class Chaos::TAABB<Chaos::FRealDouble, 3>;
