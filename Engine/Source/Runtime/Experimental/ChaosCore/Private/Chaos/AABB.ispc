// Copyright Epic Games, Inc. All Rights Reserved.

#define EXPLICIT_VECTOR4 1

#include "Math/Vector.isph"
#include "Math/Transform.isph"

static const uniform struct FVector4d DoubleNegateXVector = { {-1.0d, 1.0d, 1.0d, 1.0d} };
static const uniform struct FVector4d DoubleNegateYVector = { {1.0d, -1.0d, 1.0d, 1.0d} };
static const uniform struct FVector4d DoubleNegateZVector = { {1.0d, 1.0d, -1.0d, 1.0d} };

#define NegateXVector DoubleNegateXVector
#define NegateYVector DoubleNegateYVector
#define NegateZVector DoubleNegateZVector
#define NegateWVector DoubleNegateWVector


inline static void GrowToInclude(uniform FVector4 &Min, uniform FVector4 &Max, const uniform FVector4& V)
{
	Min = VectorMin(Min, V);
	Max = VectorMax(Max, V);
}

// Two permutations need supporting - FTransform<double>/AABB<double> and FTransform<double>/AABB<float>.
// TransformAABB handles the first. Mixed handles the second.
export void TransformedAABB(const uniform FTransform &SpaceTransform, const uniform FVector &MMin, const uniform FVector &MMax, uniform FVector &NewMin, uniform FVector &NewMax)
{
	const uniform FVector4 Min = SetVector4(MMin.V[0], MMin.V[1], MMin.V[2], ZERO);
	const uniform FVector4 Max = SetVector4(MMax.V[0], MMax.V[1], MMax.V[2], ZERO);
	const uniform FVector4 CurrentExtents = Max - Min;

	uniform FVector4 MinToNewSpace = TransformPosition(SpaceTransform, Min);
	uniform FVector4 MaxToNewSpace = MinToNewSpace;
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Max));

	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Min + ForwardVector4 * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Max - ForwardVector4 * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Min + RightVector4 * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Max - RightVector4 * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Min + UpVector4 * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Max - UpVector4 * CurrentExtents));

	NewMin = SetVector(MinToNewSpace);
	NewMax = SetVector(MaxToNewSpace);
}

export void TransformedAABBMixed(const uniform FTransform &SpaceTransform, const uniform FVector3f &MMin, const uniform FVector3f &MMax, uniform FVector3f &NewMin, uniform FVector3f &NewMax)
{
	const uniform FVector Min3 = ConvertVector3fTo3Native(MMin);
	const uniform FVector Max3 = ConvertVector3fTo3Native(MMax);
	const uniform FVector4 Min = SetVector4(Min3.V[0], Min3.V[1], Min3.V[2], ZERO);
	const uniform FVector4 Max = SetVector4(Max3.V[0], Max3.V[1], Max3.V[2], ZERO);
	const uniform FVector4 CurrentExtents = Max - Min;

	uniform FVector4 MinToNewSpace = TransformPosition(SpaceTransform, Min);
	uniform FVector4 MaxToNewSpace = MinToNewSpace;
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Max));

	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Min + ForwardVector4 * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Max - ForwardVector4 * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Min + RightVector4 * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Max - RightVector4 * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Min + UpVector4 * CurrentExtents));
	GrowToInclude(MinToNewSpace, MaxToNewSpace, TransformPosition(SpaceTransform, Max - UpVector4 * CurrentExtents));

	const uniform FVector4f MinToNewSpace4f = ConvertVector4NativeTo4f(MinToNewSpace);
	const uniform FVector4f MaxToNewSpace4f = ConvertVector4NativeTo4f(MaxToNewSpace);

	NewMin = SetVector(MinToNewSpace4f);
	NewMax = SetVector(MaxToNewSpace4f);
}

export void TransformedAABB2(const uniform FTransform &SpaceTransform, const uniform FVector &MMin, const uniform FVector &MMax, uniform FVector &NewMin, uniform FVector &NewMax)
{
	const uniform FVector4 Min = SetVector4(MMin.V[0], MMin.V[1], MMin.V[2], ZERO);
	const uniform FVector4 Max = SetVector4(MMax.V[0], MMax.V[1], MMax.V[2], ZERO);

	// Center-relative scaled verts
	// NOTE: Scale may be negative, but it does not impact the bounds calculation
	const uniform FVector4 Extent = 0.5 * SpaceTransform.Scale3D * (Max - Min);
	const uniform FVector4 Vert0 = Extent;										// (Extent.V[0],  Extent.V[1],  Extent.V[2])
	const uniform FVector4 Vert1 = Extent * NegateZVector;						// (Extent.V[0],  Extent.V[1], -Extent.V[2])
	const uniform FVector4 Vert2 = Extent * NegateYVector;						// (Extent.V[0], -Extent.V[1],  Extent.V[2])
	const uniform FVector4 Vert3 = Extent * (NegateYVector * NegateZVector);	// (Extent.V[0], -Extent.V[1], -Extent.V[2])

	// Rotated center-relative scaled verts
	const uniform FVector4 RVert0 = VectorQuaternionRotateVector(SpaceTransform.Rotation, Vert0);
	const uniform FVector4 RVert1 = VectorQuaternionRotateVector(SpaceTransform.Rotation, Vert1);
	const uniform FVector4 RVert2 = VectorQuaternionRotateVector(SpaceTransform.Rotation, Vert2);
	const uniform FVector4 RVert3 = VectorQuaternionRotateVector(SpaceTransform.Rotation, Vert3);

	// Max rotated scaled center-relative extent
	uniform FVector4 RExtent = VectorZero;
	RExtent = VectorMax(RExtent, VectorAbs(RVert0));
	RExtent = VectorMax(RExtent, VectorAbs(RVert1));
	RExtent = VectorMax(RExtent, VectorAbs(RVert2));
	RExtent = VectorMax(RExtent, VectorAbs(RVert3));

	// Transformed center
	// NOTE: This is where positive/negative scales matters
	const uniform FVector4 Center = 0.5 * (Min + Max);
	const uniform FVector4 TCenter = SpaceTransform.Translation + VectorQuaternionRotateVector(SpaceTransform.Rotation, (SpaceTransform.Scale3D * Center));

	// Transformed bounds
	NewMin = SetVector(TCenter - RExtent);
	NewMax = SetVector(TCenter + RExtent);
}
