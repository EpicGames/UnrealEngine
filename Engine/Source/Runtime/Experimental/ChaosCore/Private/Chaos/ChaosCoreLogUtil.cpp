// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/ChaosCoreLogUtil.h"
#include "CoreTypes.h"

namespace Chaos
{
	FString ToString(const FRotation3& V)
	{
		return FString::Format(TEXT("{0}, {1}, {2}, {3}"), { V.X, V.Y, V.Z, V.W });
	}

	FString ToString(const FRotation3f& V)
	{
		return FString::Format(TEXT("{0}, {1}, {2}, {3}"), { V.X, V.Y, V.Z, V.W });
	}

	FString ToString(const FVec3& V)
	{
		return FString::Format(TEXT("{0}, {1}, {2}"), { V.X, V.Y, V.Z });
	}

	FString ToString(const FVec3f& V)
	{
		return FString::Format(TEXT("{0}, {1}, {2}"), { V.X, V.Y, V.Z });
	}

	FString ToString(const TBitArray<>& BitArray)
	{
		FString S;
		S.Reserve(BitArray.Num());

		for (int32 BitIndex = 0; BitIndex < BitArray.Num(); ++BitIndex)
		{
			S.Append(BitArray[BitIndex] ? TEXT("1") : TEXT("0"));
		}

		return S;
	}

}