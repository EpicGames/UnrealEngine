// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/CoreCapsule.h"

#include "Chaos/CoreSphere.h"
#include "Chaos/Raycasts.h"

namespace Chaos
{
	FCoreCapsule::FCoreCapsule(const TSegment<FRealSingle>& InSegment, const FRealSingle InRadius)
		: MSegment(InSegment)
		, Radius(InRadius)
	{
	}

	FCoreCapsule::FCoreCapsule(const FVec3f& X1, const FVec3f& X2, const FRealSingle InRadius)
		: MSegment(X1, X2)
		, Radius(InRadius)
	{
	}

	FRealSingle FCoreCapsule::GetRadius() const
	{
		return Radius;
	}

	void FCoreCapsule::SetRadius(FRealSingle InRadius)
	{
		Radius = InRadius;
	}

	const FVec3f FCoreCapsule::GetX1() const
	{
		return MSegment.GetX1();
	}

	const FVec3f FCoreCapsule::GetX2() const
	{
		return MSegment.GetX2();
	}

	const TSegment<FRealSingle>& FCoreCapsule::GetSegment() const
	{
		return MSegment;
	}

	FRealSingle FCoreCapsule::GetHeight() const
	{
		return MSegment.GetLength();
	}

	const FVec3f FCoreCapsule::GetAxis() const
	{
		return MSegment.GetAxis();
	}

	FVec3f FCoreCapsule::GetCenter() const
	{
		return MSegment.GetCenter();
	}

	const FAABB3 FCoreCapsule::BoundingBox() const
	{
		FAABB3 Box = FAABB3(MSegment.BoundingBox());
		Box.Thicken(GetRadius());
		return Box;
	}

	bool FCoreCapsule::Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal) const
	{
		return Raycasts::RayCapsule(StartPoint, Dir, Length, Thickness, GetRadius(), GetHeight(), GetAxis(), GetX1(), GetX2(), OutTime, OutPosition, OutNormal);
	}
} // namespace Chaos
