// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Raycasts.h"

#include "Chaos/AABB.h"
#include "Chaos/CoreSphere.h"
#include "Chaos/CoreCapsule.h"
#include "Chaos/CorePlane.h"

namespace Chaos::Raycasts
{
	template <typename T, int d>
	bool RayAabb(const TVector<FReal, d>& RayStart, const TVector<FReal, d>& RayDir, const FReal RayLength, const FReal RayThickness, const TVector<T, d>& AabbMin, const TVector<T, d>& AabbMax, FReal& OutTime, TVector<FReal, d>& OutPosition, TVector<FReal, d>& OutNormal, int32& OutFaceIndex)
	{
		ensure(RayLength > 0);
		ensure(FMath::IsNearlyEqual(RayDir.SizeSquared(), (FReal)1, (FReal)UE_KINDA_SMALL_NUMBER));

		OutFaceIndex = INDEX_NONE;
		const TVector<FReal, d> MinInflated = TVector<FReal, d>(AabbMin) - RayThickness;
		const TVector<FReal, d> StartToMin = MinInflated - RayStart;

		const TVector<FReal, d> MaxInflated = TVector<FReal, d>(AabbMax) + RayThickness;
		const TVector<FReal, d> StartToMax = MaxInflated - RayStart;

		//For each axis record the start and end time when ray is in the box. If the intervals overlap the ray is inside the box
		FReal LatestStartTime = 0;
		FReal EarliestEndTime = TNumericLimits<FReal>::Max();
		TVector<FReal, d> Normal(0);	//not needed but fixes compiler warning

		for (int Axis = 0; Axis < d; ++Axis)
		{
			const bool bParallel = FMath::IsNearlyZero(RayDir[Axis]);
			FReal Time1, Time2;
			if (bParallel)
			{
				if (StartToMin[Axis] > 0 || StartToMax[Axis] < 0)
				{
					return false;	//parallel and outside
				}
				else
				{
					Time1 = 0;
					Time2 = FLT_MAX;
				}
			}
			else
			{
				const FReal InvRayDir = (FReal)1 / RayDir[Axis];
				Time1 = StartToMin[Axis] * InvRayDir;
				Time2 = StartToMax[Axis] * InvRayDir;
			}

			TVector<FReal, d> CurNormal = TVector<FReal, d>::AxisVector(Axis);

			if (Time1 > Time2)
			{
				//going from max to min direction
				std::swap(Time1, Time2);
			}
			else
			{
				//hit negative plane first
				CurNormal[Axis] = -1;
			}

			if (Time1 > LatestStartTime)
			{
				//last plane to enter so save its normal
				Normal = CurNormal;
			}
			LatestStartTime = FMath::Max(LatestStartTime, Time1);
			EarliestEndTime = FMath::Min(EarliestEndTime, Time2);

			if (LatestStartTime > EarliestEndTime)
			{
				return false;	//Outside of slab before entering another
			}
		}

		//infinite ray intersects with inflated box
		if (LatestStartTime > RayLength || EarliestEndTime < 0)
		{
			//outside of line segment given
			return false;
		}

		const TVector<FReal, d> BoxIntersection = RayStart + LatestStartTime * RayDir;

		//If the box is rounded we have to consider corners and edges.
		//Break the box into voronoi regions based on features (corner, edge, face) and see which region the raycast hit

		if (RayThickness != (FReal)0)
		{
			check(d == 3);
			TVector<FReal, d> GeomStart;
			TVector<FReal, d> GeomEnd;
			int32 NumAxes = 0;

			for (int Axis = 0; Axis < d; ++Axis)
			{
				if (BoxIntersection[Axis] < (FReal)AabbMin[Axis])
				{
					GeomStart[Axis] = (FReal)AabbMin[Axis];
					GeomEnd[Axis] = (FReal)AabbMin[Axis];
					++NumAxes;
				}
				else if (BoxIntersection[Axis] > (FReal)AabbMax[Axis])
				{
					GeomStart[Axis] = (FReal)AabbMax[Axis];
					GeomEnd[Axis] = (FReal)AabbMax[Axis];
					++NumAxes;
				}
				else
				{
					GeomStart[Axis] = (FReal)AabbMin[Axis];
					GeomEnd[Axis] = (FReal)AabbMax[Axis];
				}
			}

			if (NumAxes >= 2)
			{
				bool bHit = false;
				if (NumAxes == 3)
				{
					//hit a corner. For now just use 3 capsules, there's likely a better way to determine which capsule is needed
					FReal CornerTimes[3];
					TVector<FReal, d> CornerPositions[3];
					TVector<FReal, d> CornerNormals[3];
					int32 HitIdx = INDEX_NONE;
					FReal MinTime = 0;	//initialization just here for compiler warning
					for (int CurIdx = 0; CurIdx < 3; ++CurIdx)
					{
						TVector<FReal, d> End = GeomStart;
						End[CurIdx] = End[CurIdx] == AabbMin[CurIdx] ? AabbMax[CurIdx] : AabbMin[CurIdx];
						FCoreCapsule Capsule(GeomStart, End, FRealSingle(RayThickness));
						if (Capsule.Raycast(RayStart, RayDir, RayLength, 0, CornerTimes[CurIdx], CornerPositions[CurIdx], CornerNormals[CurIdx]))
						{
							if (HitIdx == INDEX_NONE || CornerTimes[CurIdx] < MinTime)
							{
								MinTime = CornerTimes[CurIdx];
								HitIdx = CurIdx;

								if (MinTime == 0)
								{
									OutTime = 0;	//initial overlap so just exit
									return true;
								}
							}
						}
					}

					if (HitIdx != INDEX_NONE)
					{
						OutPosition = CornerPositions[HitIdx];
						OutTime = MinTime;
						OutNormal = CornerNormals[HitIdx];
						bHit = true;
					}
				}
				else
				{
					//capsule: todo(use a cylinder which is cheaper. Our current cylinder raycast implementation doesn't quite work for this setup)
					FCoreCapsule CapsuleBorder(GeomStart, GeomEnd, FRealSingle(RayThickness));
					bHit = CapsuleBorder.Raycast(RayStart, RayDir, RayLength, 0, OutTime, OutPosition, OutNormal);
				}

				if (bHit && OutTime > 0)
				{
					OutPosition -= OutNormal * RayThickness;
				}
				return bHit;
			}
		}

		// didn't hit any rounded parts so just use the box intersection
		OutTime = LatestStartTime;
		OutNormal = Normal;
		OutPosition = BoxIntersection - RayThickness * Normal;
		return true;
	}

	bool RayCapsule(const FVec3& RayStart, const FVec3& RayDir, const FReal RayLength, const FReal RayThickness, FReal CapsuleRadius, FReal CapsuleHeight, const FVec3& CapsuleAxis, const FVec3& CapsuleX1, const FVec3& CapsuleX2, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal)
	{
		ensure(FMath::IsNearlyEqual(CapsuleAxis.SizeSquared(), (FReal)1, (FReal)UE_KINDA_SMALL_NUMBER));
		ensure(FMath::IsNearlyEqual(RayDir.SizeSquared(), (FReal)1, (FReal)UE_KINDA_SMALL_NUMBER));
		ensure(RayLength >= 0);

		const FReal R = CapsuleRadius + RayThickness;
		const FReal R2 = R * R;

		// first test, segment to segment distance
		const FVec3 RayEndPoint = RayStart + RayDir * RayLength;
		FVec3 OutP1, OutP2;
		FMath::SegmentDistToSegmentSafe(CapsuleX1, CapsuleX2, RayStart, RayEndPoint, OutP1, OutP2);
		const FReal DistanceSquared = (OutP2 - OutP1).SizeSquared();
		if (DistanceSquared > R2 + DBL_EPSILON)
		{
			return false;
		}

		// Raycast against capsule bounds.
		// We will use the intersection point as our ray start, this prevents precision issues if start is far away.
		// All calculations below should use LocalStart/LocalLength, and convert output time using input length if intersecting.
		FAABB3 CapsuleBounds;
		{
			CapsuleBounds.GrowToInclude(CapsuleX1);
			CapsuleBounds.GrowToInclude(CapsuleX2);
			CapsuleBounds.Thicken(R);
		}

		FVec3 InvRayDir;
		bool bParallel[3];
		for (int32 Axis = 0; Axis < 3; ++Axis)
		{
			bParallel[Axis] = RayDir[Axis] == 0;
			InvRayDir[Axis] = bParallel[Axis] ? 0 : 1 / RayDir[Axis];
		}

		FVec3 LocalStart = RayStart;
		FReal LocalLength = RayLength;
		FReal RemovedLength = 0;
		{
			FReal OutBoundsTime;
			bool bStartHit = CapsuleBounds.RaycastFast(RayStart, RayDir, InvRayDir, bParallel, RayLength, OutBoundsTime, OutPosition);
			if (bStartHit == false)
			{
				return false;
			}

			LocalStart = RayStart + OutBoundsTime * RayDir;
			RemovedLength = OutBoundsTime;
			LocalLength = RayLength - OutBoundsTime; // Note: This could be 0.
		}

		//First check if we are initially overlapping
		//Find closest point to cylinder core and check if it's inside the inflated capsule
		const FVec3 X1ToStart = LocalStart - CapsuleX1;
		const FReal MVectorDotX1ToStart = FVec3::DotProduct(X1ToStart, CapsuleAxis);
		if (MVectorDotX1ToStart >= -R && MVectorDotX1ToStart <= CapsuleHeight + R)
		{
			//projection is somewhere in the capsule. Clamp to cylinder length and check if inside sphere
			const FReal ClampedProjection = FMath::Clamp(MVectorDotX1ToStart, (FReal)0, CapsuleHeight);
			const FVec3 ClampedProjectionPosition = CapsuleAxis * ClampedProjection;
			const FReal Dist2 = (X1ToStart - ClampedProjectionPosition).SizeSquared();
			if (Dist2 <= R2)
			{
				// In this case, clamped project position is either inside capsule or on the surface.

				OutTime = RemovedLength; // We may have shortened our ray, not actually 0 time.

				// We clipped ray against bounds, not a true initial overlap, compute normal/position
				if (RemovedLength > 0.0f)
				{
					// Ray must have started outside capsule bounds, intersected bounds where it is touched capsule surface.
					OutNormal = (X1ToStart - ClampedProjectionPosition) / R;
					OutPosition = LocalStart - OutNormal * RayThickness;
				}
				else
				{
					// Input ray started inside capsule, out time is 0, we are just filling out outputs so they aren't uninitialized.
					OutPosition = LocalStart;
					OutNormal = -RayDir;
				}

				return true;
			}
		}

		if (FMath::IsNearlyEqual(LocalLength, 0., UE_KINDA_SMALL_NUMBER))
		{
			// If LocalLength is 0, this means the ray's endpoint is on the bounding AABB of thickened capsule.
			// At this point we have determined this point is not on surface of capsule, so the ray has missed.
			return false;
		}

		// Raycast against cylinder first

		//let <x,y> denote x \dot y
		//cylinder implicit representation: ||((X - x1) \cross CapsuleAxis)||^2 - R^2 = 0, where X is any point on the cylinder surface (only true because CapsuleAxis is unit)
		//Using Lagrange's identity we get ||X-x1||^2 ||CapsuleAxis||^2 - <CapsuleAxis, X-x1>^2 - R^2 = ||X-x1||^2 - <CapsuleAxis, X-x1>^2 - R^2 = 0
		//Then plugging the ray into X we have: ||RayStart + t RayDir - x1||^2 - <CapsuleAxis, Start + t RayDir - x1>^2 - R^2
		// = ||RayStart-x1||^2 + t^2 + 2t <RayStart-x1, RayDir> - <CapsuleAxis, RayStart-x1>^2 - t^2 <CapsuleAxis,RayDir>^2 - 2t<CapsuleAxis, RayStart -x1><CapsuleAxis, RayDir> - R^2 = 0
		//Solving for the quadratic formula we get:
		//a = 1 - <CapsuleAxis,RayDir>^2	Note a = 0 implies CapsuleAxis and RayDir are parallel
		//b = 2(<RayStart-x1, RayDir> - <CapsuleAxis, RayStart - x1><CapsuleAxis, RayDir>)
		//c = ||RayStart-x1||^2 - <CapsuleAxis, RayStart-x1>^2 - R^2 Note this tells us if start point is inside (c < 0) or outside (c > 0) of cylinder

		const FReal MVectorDotX1ToStart2 = MVectorDotX1ToStart * MVectorDotX1ToStart;
		const FReal MVectorDotDir = FVec3::DotProduct(CapsuleAxis, RayDir);
		const FReal MVectorDotDir2 = MVectorDotDir * MVectorDotDir;
		const FReal X1ToStartDotDir = FVec3::DotProduct(X1ToStart, RayDir);
		const FReal X1ToStart2 = X1ToStart.SizeSquared();
		const FReal A = 1 - MVectorDotDir2;
		const FReal C = X1ToStart2 - MVectorDotX1ToStart2 - R2;

		constexpr FReal Epsilon = (FReal)1e-4;
		bool bCheckCaps = false;

		if (C <= 0.f)
		{
			// We already tested initial overlap of start point, so start must be in cylinder
			// but above/below segment end points.
			bCheckCaps = true;
		}
		else
		{
			const FReal HalfB = (X1ToStartDotDir - MVectorDotX1ToStart * MVectorDotDir);
			const FReal QuarterUnderRoot = HalfB * HalfB - A * C;

			if (QuarterUnderRoot < 0)
			{
				bCheckCaps = true;
			}
			else
			{
				FReal Time;
				const bool bSingleHit = QuarterUnderRoot < Epsilon;
				if (bSingleHit)
				{
					Time = (A == 0) ? 0 : (-HalfB / A);

				}
				else
				{
					Time = (A == 0) ? 0 : ((-HalfB - FMath::Sqrt(QuarterUnderRoot)) / A); //we already checked for initial overlap so just take smallest time
					if (Time < 0)	//we must have passed the cylinder
					{
						return false;
					}
				}

				const FVec3 SpherePosition = LocalStart + Time * RayDir;
				const FVec3 CylinderToSpherePosition = SpherePosition - CapsuleX1;
				const FReal PositionLengthOnCoreCylinder = FVec3::DotProduct(CylinderToSpherePosition, CapsuleAxis);
				if (PositionLengthOnCoreCylinder >= 0 && PositionLengthOnCoreCylinder < CapsuleHeight)
				{
					OutTime = Time + RemovedLength; // Account for ray clipped against bounds
					OutNormal = (CylinderToSpherePosition - CapsuleAxis * PositionLengthOnCoreCylinder) / R;
					OutPosition = SpherePosition - OutNormal * RayThickness;
					return true;
				}
				else
				{
					//if we have a single hit the ray is tangent to the cylinder.
					//the caps are fully contained in the infinite cylinder, so no need to check them
					bCheckCaps = !bSingleHit;
				}
			}
		}

		if (bCheckCaps)
		{
			//can avoid some work here, but good enough for now
			FCoreSphere X1Sphere(CapsuleX1, CapsuleRadius);
			FCoreSphere X2Sphere(CapsuleX2, CapsuleRadius);

			FReal Time1, Time2;
			FVec3 Position1, Position2;
			FVec3 Normal1, Normal2;
			bool bHitX1 = X1Sphere.Raycast(LocalStart, RayDir, LocalLength, RayThickness, Time1, Position1, Normal1);
			bool bHitX2 = X2Sphere.Raycast(LocalStart, RayDir, LocalLength, RayThickness, Time2, Position2, Normal2);

			if (bHitX1 && bHitX2)
			{
				if (Time1 <= Time2)
				{
					OutTime = Time1 + RemovedLength;  // Account for ray clipped against bounds
					OutPosition = Position1;
					OutNormal = Normal1;
				}
				else
				{
					OutTime = Time2 + RemovedLength;  // Account for ray clipped against bounds
					OutPosition = Position2;
					OutNormal = Normal2;
				}

				return true;
			}
			else if (bHitX1)
			{
				OutTime = Time1 + RemovedLength;  // Account for ray clipped against bounds
				OutPosition = Position1;
				OutNormal = Normal1;
				return true;
			}
			else if (bHitX2)
			{
				OutTime = Time2 + RemovedLength;  // Account for ray clipped against bounds
				OutPosition = Position2;
				OutNormal = Normal2;
				return true;
			}
		}

		return false;
	}


	// These methods are in the cpp because of cycles. Explicitly instantiate the used ones (taken from explicit instantiations of AABB)
	template bool RayAabb<Chaos::FRealSingle, 3>(const TVector<FReal, 3>&, const TVector<FReal, 3>&, const FReal, const FReal, const TVector<FRealSingle, 3>&, const TVector<FRealSingle, 3>&, FReal&, TVector<FReal, 3>&, TVector<FReal, 3>&, int32&);
	template bool RayAabb<Chaos::FReal, 3>(const TVector<FReal, 3>&, const TVector<FReal, 3>&, const FReal, const FReal, const TVector<FReal, 3>&, const TVector<FReal, 3>&, FReal&, TVector<FReal, 3>&, TVector<FReal, 3>&, int32&);
} // Chaos::Raycasts
