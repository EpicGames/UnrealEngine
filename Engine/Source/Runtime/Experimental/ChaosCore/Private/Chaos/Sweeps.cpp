// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Sweeps.h"

#include "Chaos/Raycasts.h"
#include "Chaos/SweepsMTD.h"

namespace Chaos::Sweeps
{
	bool SweepSphereVsSphere(const FVec3& SweepStart, const FVec3& SweepDir, const FReal SweepLength, const FReal Sphere0Radius, const FVec3& Sphere1Center, const FReal Sphere1Radius, const ESweepFlags Flags, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal)
	{
		const bool bResult = Raycasts::RaySphere<FReal, 3>(SweepStart, SweepDir, SweepLength, Sphere0Radius, Sphere1Center, FRealSingle(Sphere1Radius), OutTime, OutPosition, OutNormal);
		if (bResult && OutTime <= UE_DOUBLE_SMALL_NUMBER && EnumHasAnyFlags(Flags, ESweepFlags::MTD))
		{
			SphereSphereMTD(Sphere1Center, Sphere1Radius, SweepStart, Sphere0Radius, OutTime, OutPosition, OutNormal);
		}
		return bResult;
	}

	bool SweepSphereVsAabb(const FVec3& SweepStart, const FVec3& SweepDir, const FReal SweepLength, const FReal SphereRadius, const FVec3& AabbMin, const FVec3& AabbMax, const ESweepFlags Flags, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex)
	{
		const bool bResult = Raycasts::RayAabb<FReal, 3>(SweepStart, SweepDir, SweepLength, SphereRadius, AabbMin, AabbMax, OutTime, OutPosition, OutNormal, OutFaceIndex);
		if (bResult && OutTime <= UE_DOUBLE_SMALL_NUMBER && EnumHasAnyFlags(Flags, ESweepFlags::MTD))
		{
			AabbSphereMTD(AabbMin, AabbMax, SweepStart, SphereRadius, OutTime, OutPosition, OutNormal);
		}

		return bResult;
	}

	bool SweepSphereVsCapsule(const FVec3& SweepStart, const FVec3& SweepDir, const FReal SweepLength, const FReal SphereRadius,
		const FReal CapsuleRadius, const FReal CapsuleHeight, const FVec3& CapsuleAxis, const FVec3& CapsuleX1, const FVec3& CapsuleX2,
		const ESweepFlags Flags, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal)
	{
		const bool bResult = Raycasts::RayCapsule(SweepStart, SweepDir, SweepLength, SphereRadius, CapsuleRadius, CapsuleHeight, CapsuleAxis, CapsuleX1, CapsuleX2, OutTime, OutPosition, OutNormal);
		if (bResult && OutTime <= UE_DOUBLE_SMALL_NUMBER && EnumHasAnyFlags(Flags, ESweepFlags::MTD))
		{
			CapsuleSphereMTD(CapsuleX1, CapsuleX2, CapsuleRadius, SweepStart, SphereRadius, OutTime, OutPosition, OutNormal);
		}

		return bResult;
	}
} // namespace Chaos::Sweeps
