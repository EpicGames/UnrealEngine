// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/SweepsMTD.h"

namespace Chaos::Sweeps
{
	void SphereSphereMTD(const FVec3& Sphere0Center, const FReal Sphere0Radius, const FVec3& Sphere1Center, const FReal Sphere1Radius, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal)
	{
		const FReal RadiusSum = Sphere0Radius + Sphere1Radius;
		const FVec3 C01 = Sphere1Center - Sphere0Center;
		const FReal Dist01 = C01.Size();
		if (Dist01 > UE_DOUBLE_SMALL_NUMBER)
		{
			OutNormal = C01 / Dist01;
		}
		else
		{
			OutNormal = FVec3(0, 0, 1);
		}
		OutTime = Dist01 - RadiusSum;
		OutPosition = Sphere0Center + OutNormal * Sphere0Radius;
	}

	void AabbSphereMTD(const FVec3& AabbMin, const FVec3& AabbMax, const FVec3& SphereCenter, const FReal SphereRadius, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal)
	{
		FVec3 ClampedCenter;
		ClampedCenter.X = FMath::Clamp(SphereCenter.X, AabbMin.X, AabbMax.X);
		ClampedCenter.Y = FMath::Clamp(SphereCenter.Y, AabbMin.Y, AabbMax.Y);
		ClampedCenter.Z = FMath::Clamp(SphereCenter.Z, AabbMin.Z, AabbMax.Z);

		const FVec3 DistVec = SphereCenter - ClampedCenter;
		const FReal DistSq = DistVec.SquaredLength();
		if (DistSq > UE_DOUBLE_SMALL_NUMBER)
		{
			// Sphere center was outside the aabb. The MTD is along the dist vec.
			const FReal Dist = FMath::Sqrt(DistSq);
			OutPosition = ClampedCenter;
			OutNormal = DistVec / Dist;
			OutTime = Dist - SphereRadius;
		}
		else
		{
			// Sphere center was inside the aabb. The MTD is along the aabb axis of deepest penetration.
			const FVec3 AabbCenter = (AabbMax + AabbMin) * 0.5;
			const FVec3 AabbHalfExtent = (AabbMax - AabbMin) * 0.5;
			const FVec3 R = SphereCenter - AabbCenter;
			const FVec3 AbsR = R.GetAbs();

			int32 LargestAxis = 0;
			FReal LargestAxisValue = AbsR.X;
			for (int32 I = 1; I < 3; ++I)
			{
				if (AbsR[I] > LargestAxisValue)
				{
					LargestAxis = I;
					LargestAxisValue = AbsR[I];
				}
			}
			OutNormal = FVec3::Zero();
			OutNormal[LargestAxis] = 1 * FMath::Sign(R[LargestAxis]);
			OutPosition = AabbCenter + AabbHalfExtent * OutNormal;
			// OutTime = (AabbR + SphereR) - Penetration
			OutTime = -(AabbHalfExtent[LargestAxis] + SphereRadius - AbsR[LargestAxis]);
		}
	}

	void CapsuleSphereMTD(const FVec3& CapsuleX1, const FVec3& CapsuleX2, const FReal CapsuleRadius, const FVec3& SphereCenter, const FReal SphereRadius, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal)
	{
		const FVec3 Axis = CapsuleX2 - CapsuleX1;
		const FReal AxisLengthSq = Axis.SquaredLength();
		const FReal T = FVec3::DotProduct(SphereCenter - CapsuleX1, Axis) / AxisLengthSq;
		const FReal ClampedT = FMath::Clamp(T, 0, 1);
		const FVec3 ProjectedPoint = CapsuleX1 + Axis * ClampedT;
		return SphereSphereMTD(ProjectedPoint, CapsuleRadius, SphereCenter, SphereRadius, OutTime, OutPosition, OutNormal);
	}
} // namespace Chaos::Sweeps
