// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosCoreModule.h"
#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FChaosCoreEngineModule, ChaosCore);
