// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/Rotation.h"

namespace Chaos
{
	/**
	 * Return the complex conjugate of the rotation
	 */
	TRotation<FReal, 3> TRotation<FReal, 3>::Conjugate(const ::Chaos::TRotation<FReal, 3>& InR)
	{		
#if 0
		// Disabled: Inlining this block can generate bad code in MSVC optimized builds with /arch:AVX, causing NaNs in the output.
		TRotation<FReal, 3> R;
		R.X = -InR.X;
		R.Y = -InR.Y;
		R.Z = -InR.Z;
		R.W = InR.W;
		return R;
#else
		return InR.Inverse();
#endif
	}

	TRotation<FRealSingle, 3> TRotation<FRealSingle, 3>::Conjugate(const ::Chaos::TRotation<FRealSingle, 3>& InR)
	{
		return InR.Inverse();
	}

	/**
	 * Negate all values of the quaternion (note: not the inverse rotation. See Conjugate)
	 */
	TRotation<FReal, 3> TRotation<FReal, 3>::Negate(const ::Chaos::TRotation<FReal, 3>& InR)
	{
		return -InR;
	}

	/**
	 * Create a rotation about an axis V/|V| by angle |V| in radians
	 */
	TRotation<FReal, 3> TRotation<FReal, 3>::FromVector(const ::Chaos::TVector<FReal, 3>& V)
	{
		using FQuatReal = BaseQuat::FReal;
		TRotation<FReal, 3> Rot;
		FReal HalfSize = 0.5f * V.Size();
		FReal sinc = (FMath::Abs(HalfSize) > 1e-8) ? FMath::Sin(HalfSize) / HalfSize : 1;
		auto RotV = 0.5f * sinc * V;
		Rot.X = (FQuatReal)RotV.X;
		Rot.Y = (FQuatReal)RotV.Y;
		Rot.Z = (FQuatReal)RotV.Z;
		Rot.W = (FQuatReal)FMath::Cos(HalfSize);
		return Rot;
	}

	/**
	 * Generate a Rotation that would rotate vector InitialVector to FinalVector
	 */
	TRotation<FReal, 3> TRotation<FReal, 3>::FromRotatedVector(
		const ::Chaos::TVector<FReal, 3>& InitialVector,
		const ::Chaos::TVector<FReal, 3>& FinalVector)
	{
		typedef Chaos::TVector<FReal, 3> TV;
		checkSlow(FMath::Abs(InitialVector.Size() - 1.0) < UE_KINDA_SMALL_NUMBER);
		checkSlow(FMath::Abs(FinalVector.Size() - 1.0) < UE_KINDA_SMALL_NUMBER);

		const double CosTheta = FMath::Clamp<FReal>(TV::DotProduct(InitialVector, FinalVector), -1., 1.);

		TV V = TV::CrossProduct(InitialVector, FinalVector);
		const FReal VMagnitude = V.Size();
		if (VMagnitude == 0)
		{
			return TRotation<FReal, 3>::FromElements(InitialVector, 0.f);
		}

		const FRealDouble SSquared = .5 * (1.0 + CosTheta); // Uses the half angle formula
		const FRealDouble VMagnitudeDesired = sqrt(1.0 - SSquared);
		V *= static_cast<FReal>(VMagnitudeDesired / VMagnitude);

		return TRotation<FReal, 3>::FromElements(V, static_cast<FReal>(sqrt(SSquared)));
	}

	//PRAGMA_DISABLE_OPTIMIZATION

	/**
	 * Calculate the angular velocity required to take an object with orientation R0 to orientation R1 in time Dt.
	 *
	 * Uses the relation: DQ/DT = (W * Q)/2
	 */
	TVector<FReal, 3> TRotation<FReal, 3>::CalculateAngularVelocity1(const TRotation<FReal, 3>& InR0, const TRotation<FReal, 3>& InR1, const FReal InDt)
	{
		if (!ensure(InDt > UE_SMALL_NUMBER))
		{
			return TVector<FReal, 3>(0);
		}

		const TRotation<FReal, 3> R0 = InR0;
		TRotation<FReal, 3> R1 = InR1;
		R1.EnforceShortestArcWith(R0);

		// W = 2 * dQ/dT * Qinv
		const TRotation<FReal, 3> DRDt = (R1 - R0) / InDt;
		const TRotation<FReal, 3> RInv = Conjugate(R0);
		const TRotation<FReal, 3> W = (DRDt * RInv) * 2.0f;

		return TVector<FReal, 3>(W.X, W.Y, W.Z);
	}

	TVector<FRealSingle, 3> TRotation<FRealSingle, 3>::CalculateAngularVelocity1(const TRotation<FRealSingle, 3>& InR0, const TRotation<FRealSingle, 3>& InR1, const FRealSingle InDt)
	{
		if (!ensure(InDt > UE_SMALL_NUMBER))
		{
			return TVector<FRealSingle, 3>(0);
		}

		const TRotation<FRealSingle, 3> R0 = InR0;
		TRotation<FRealSingle, 3> R1 = InR1;
		R1.EnforceShortestArcWith(R0);

		// W = 2 * dQ/dT * Qinv
		const TRotation<FRealSingle, 3> DRDt = (R1 - R0) / InDt;
		const TRotation<FRealSingle, 3> RInv = Conjugate(R0);
		const TRotation<FRealSingle, 3> W = (DRDt * RInv) * 2.0f;

		return TVector<FRealSingle, 3>(W.X, W.Y, W.Z);
	}

	//PRAGMA_ENABLE_OPTIMIZATION


	/**
	 * Calculate the angular velocity required to take an object with orientation R0 to orientation R1 in time Dt.
	 *
	 * Uses the Quaternion to Axis/Angle method.
	 */
	TVector<FReal, 3> TRotation<FReal, 3>::CalculateAngularVelocity2(const TRotation<FReal, 3>& InR0, const TRotation<FReal, 3>& InR1, const FReal InDt)
	{
		// @todo(ccaulfield): ToAxisAndAngle starts to return increasingly random, non-normalized axes for very small angles. This 
		// underestimates the angular velocity magnitude and randomizes direction.
		if (!ensure(InDt > UE_SMALL_NUMBER))
		{
			return TVector<FReal, 3>(0);
		}

		const TRotation<FReal, 3> R0 = InR0;
		TRotation<FReal, 3> R1 = InR1;
		R1.EnforceShortestArcWith(R0);

		const TRotation<FReal, 3> DR = R1 * Conjugate(R0);
		FReal Angle = DR.GetAngle();
		TVector<FReal, 3> Axis = DR.GetRotationAxis();
		return Axis * (Angle / InDt);
	}

	/**
	 * Return a new rotation equal to the input rotation with angular velocity W applied over time Dt.
	 *
	 * Uses the relation: DQ/DT = (W * Q)/2
	 */
	TRotation<FReal, 3> TRotation<FReal, 3>::IntegrateRotationWithAngularVelocity(const TRotation<FReal, 3>& InR0, const TVector<FReal, 3>& InW, const FReal InDt)
	{
		TRotation<FReal, 3> R1 = InR0 + (TRotation<FReal, 3>::FromElements(InW.X, InW.Y, InW.Z, 0.f) * InR0) * decltype(InR0.X)(InDt * 0.5f);
		return R1.GetNormalized();
	}

	TRotation<FRealSingle, 3> TRotation<FRealSingle, 3>::IntegrateRotationWithAngularVelocity(const TRotation<FRealSingle, 3>& InR0, const TVector<FRealSingle, 3>& InW, const FRealSingle InDt)
	{
		TRotation<FRealSingle, 3> R1 = InR0 + (TRotation<FRealSingle, 3>::FromElements(InW.X, InW.Y, InW.Z, 0.f) * InR0) * decltype(InR0.X)(InDt * 0.5f);
		return R1.GetNormalized();
	}

}