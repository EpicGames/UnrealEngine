// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/Transform.h"
#include "Chaos/Real.h"
#include "Chaos/Matrix.h"

namespace Chaos
{
	PMatrix<FRealSingle, 4, 4> TRigidTransform<FRealSingle, 3>::operator*(const PMatrix<FRealSingle, 4, 4>& Matrix) const
	{
		// LWC_TODO: Perf pessimization
		return ToMatrixNoScale() * static_cast<const UE::Math::TMatrix<FRealSingle>&>(Matrix);
	}

	PMatrix<FRealDouble, 4, 4> TRigidTransform<FRealDouble, 3>::operator*(const PMatrix<FRealDouble, 4, 4>& Matrix) const
	{
		// LWC_TODO: Perf pessimization
		return ToMatrixNoScale() * static_cast<const UE::Math::TMatrix<FRealDouble>&>(Matrix);
	}

	const PMatrix<FRealSingle, 3, 3> PMatrix<FRealSingle, 3, 3>::Zero = PMatrix<FRealSingle, 3, 3>(0, 0, 0);
	const PMatrix<FRealSingle, 3, 3> PMatrix<FRealSingle, 3, 3>::Identity = PMatrix<FRealSingle, 3, 3>(1, 1, 1);

	const PMatrix<FRealDouble, 3, 3> PMatrix<FRealDouble, 3, 3>::Zero = PMatrix<FRealDouble, 3, 3>(0, 0, 0);
	const PMatrix<FRealDouble, 3, 3> PMatrix<FRealDouble, 3, 3>::Identity = PMatrix<FRealDouble, 3, 3>(1, 1, 1);
}
