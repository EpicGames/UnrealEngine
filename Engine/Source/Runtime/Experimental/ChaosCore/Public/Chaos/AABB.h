// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Serialization/Archive.h"
#include "Chaos/Raycasts.h"

namespace Chaos
{ 
	template<class T, int d>
	class TAABB;

	template<typename T, int d>
	struct TAABBSpecializeSamplingHelper
	{
		static FORCEINLINE TArray<TVector<T, d>> ComputeLocalSamplePoints(const class TAABB<T, d>& AABB)
		{
			check(false);
			return TArray<TVector<T, d>>();
		}
	};

	struct FAABBEdge
	{
		int8 VertexIndex0;
		int8 VertexIndex1;
	};

	struct FAABBFace
	{
		int8 VertexIndex[4];
		int8 EdgeIndex[4];
	};

	template<class T, int d>
	class TAABB
	{
	public:
		using TType = T;
		static constexpr int D = d;

		FORCEINLINE TAABB()
			: MMin(TVector<T, d>(TNumericLimits<T>::Max()))
			, MMax(TVector<T, d>(-TNumericLimits<T>::Max()))
		{
		}

		FORCEINLINE TAABB(const TVector<T, d>& Min, const TVector<T, d>&Max)
			: MMin(Min)
			, MMax(Max)
		{
		}

		FORCEINLINE TAABB(const TAABB<T, d>& Other)
			: MMin(Other.MMin)
			, MMax(Other.MMax)
		{
		}

		template<typename OtherType> 
		explicit TAABB(const TAABB<OtherType, d>& Other)
			: MMin(TVector<T, d>(Other.Min()))
			, MMax(TVector<T, d>(Other.Max()))
		{
		}

		FORCEINLINE TAABB(TAABB<T, d>&& Other)
			: MMin(MoveTemp(Other.MMin))
			, MMax(MoveTemp(Other.MMax))
		{
		}

		FORCEINLINE TAABB<T, d>& operator=(const TAABB<T, d>& Other)
		{
			MMin = Other.MMin;
			MMax = Other.MMax;
			return *this;
		}

		FORCEINLINE TAABB<T, d>& operator=(TAABB<T, d>&& Other)
		{
			MMin = MoveTemp(Other.MMin);
			MMax = MoveTemp(Other.MMax);
			return *this;
		}

		/**
		 * Returns sample points centered about the origin.
		 */
		FORCEINLINE TArray<TVector<T, d>> ComputeLocalSamplePoints() const
		{
			const TVector<T, d> Mid = Center();
			return TAABBSpecializeSamplingHelper<T, d>::ComputeSamplePoints(TAABB<T, d>(Min() - Mid, Max() - Mid));
		}

		/**
		 * Returns sample points at the current location of the box.
		 */
		FORCEINLINE TArray<TVector<T, d>> ComputeSamplePoints() const
		{
			return TAABBSpecializeSamplingHelper<T, d>::ComputeSamplePoints(*this);
		}

		CHAOSCORE_API TAABB<T, d> TransformedAABB(const FTransform&) const;
		CHAOSCORE_API TAABB<T, d> TransformedAABB(const Chaos::TRigidTransform<FReal, 3>&) const;
		CHAOSCORE_API TAABB<T, d> TransformedAABB(const FMatrix&) const;
		CHAOSCORE_API TAABB<T, d> TransformedAABB(const Chaos::PMatrix<FReal, 4, 4>&) const;

		CHAOSCORE_API TAABB<T, d> InverseTransformedAABB(const Chaos::FRigidTransform3&) const;

		template <typename TReal>
		FORCEINLINE bool Intersects(const TAABB<TReal, d>& Other) const
		{
			for (int32 i = 0; i < d; ++i)
			{
				if (Other.Max()[i] < MMin[i] || Other.Min()[i] > MMax[i])
					return false;
			}
			return true;
		}

		FORCEINLINE TAABB<T, d> GetIntersection(const TAABB<T, d>& Other) const
		{
			return TAABB<T, d>(MMin.ComponentwiseMax(Other.MMin), MMax.ComponentwiseMin(Other.MMax));
		}

		FORCEINLINE bool Contains(const TVector<T, d>& Point) const
		{
			for (int i = 0; i < d; i++)
			{
				if (Point[i] < MMin[i] || Point[i] > MMax[i])
				{
					return false;
				}
			}
			return true;
		}

		FORCEINLINE bool Contains(const TVector<T, d>& Point, const T Tolerance) const
		{
			for (int i = 0; i < d; i++)
			{
				if (Point[i] < MMin[i] - Tolerance || Point[i] > MMax[i] + Tolerance)
				{
					return false;
				}
			}
			return true;
		}

		FORCEINLINE bool Contains(const TAABB<T, d>& Other) const
		{
			return Contains(Other.Min()) && Contains(Other.Max());
		}

		FORCEINLINE const TAABB<T, d>& BoundingBox() const { return *this; }

		FORCEINLINE uint16 GetMaterialIndex(uint32 HintIndex) const { return 0; }

		FORCEINLINE FReal SignedDistance(const TVector<FReal, d>& x) const
		{
			TVector<FReal, d> Normal;
			return PhiWithNormal(x, Normal);
		}

		FORCEINLINE FReal PhiWithNormal(const TVector<FReal, d>& X, TVector<FReal, d>& Normal) const
		{
			const TVector<FReal, d> MaxDists = X - MMax;
			const TVector<FReal, d> MinDists = MMin - X;
			const TVector<FReal, d> MinAsTVecReal(MMin);
			const TVector<FReal, d> MaxAsTVecReal(MMax);
			if (X <= MaxAsTVecReal && X >= MinAsTVecReal)
			{
				const Pair<FReal, int32> MaxAndAxis = TVector<FReal, d>::MaxAndAxis(MinDists, MaxDists);
				Normal = MaxDists[MaxAndAxis.Second] > MinDists[MaxAndAxis.Second] ? TVector<FReal, d>::AxisVector(MaxAndAxis.Second) : -TVector<FReal, d>::AxisVector(MaxAndAxis.Second);
				return MaxAndAxis.First;
			}
			else
			{
				for (int i = 0; i < d; ++i)
				{
					if (MaxDists[i] > 0)
					{
						Normal[i] = MaxDists[i];
					}
					else if (MinDists[i] > 0)
					{
						Normal[i] = -MinDists[i];
					}
					else
					{
						Normal[i] = 0;
					}
				}
				FReal Phi = Normal.SafeNormalize();
				if (Phi < UE_KINDA_SMALL_NUMBER)
				{
					for (int i = 0; i < d; ++i)
					{
						if (Normal[i] > 0)
						{
							Normal[i] = 1;
						}
						else if (Normal[i] < 0)
						{
							Normal[i] = -1;
						}
					}
					Normal.Normalize();
				}
				return Phi;
			}
		}

		CHAOSCORE_API bool Raycast(const TVector<FReal, d>& StartPoint, const TVector<FReal, d>& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, TVector<FReal, d>& OutPosition, TVector<FReal, d>& OutNormal, int32& OutFaceIndex) const;

		FORCEINLINE bool RaycastFast(const TVector<FReal, d>& StartPoint, const TVector<FReal, d>& Dir, const TVector<FReal, d>& InvDir, const bool* bParallel, const FReal Length, FReal& OutEntryTime, FReal& OutExitTime) const
		{
			return Raycasts::RayAabb(StartPoint, Dir, InvDir, bParallel, Length, MMin, MMax, OutEntryTime, OutExitTime);
		}

		FORCEINLINE bool RaycastFast(const TVector<FReal, d>& StartPoint, const TVector<FReal, d>& Dir, const TVector<FReal, d>& InvDir, const bool* bParallel, const FReal Length, const FReal InvLength, FReal& OutEntryTime, FReal& OutExitTime) const
		{
			return RaycastFast(StartPoint, Dir, InvDir, bParallel, Length, OutEntryTime, OutExitTime);
		}

		FORCEINLINE bool RaycastFast(const TVector<FReal, d>& StartPoint, const TVector<FReal, d>& Dir, const TVector<FReal, d>& InvDir, const bool* bParallel, const FReal Length, FReal& OutTime, TVector<FReal, d>& OutPosition) const
		{
			FReal RayEntryTime;
			FReal RayExitTime;
			if (RaycastFast(StartPoint, Dir, InvDir, bParallel, Length, RayEntryTime, RayExitTime))
			{
				OutTime = RayEntryTime;
				OutPosition = StartPoint + RayEntryTime * Dir;
				return true;
			}
			return false;
		}

		FORCEINLINE bool RaycastFast(const TVector<FReal, d>& StartPoint, const TVector<FReal, d>& Dir, const TVector<FReal, d>& InvDir, const bool* bParallel, const FReal Length, const FReal InvLength, FReal& OutTime, TVector<FReal, d>& OutPosition) const
		{
			return RaycastFast(StartPoint, Dir, InvDir, bParallel, Length, OutTime, OutPosition);
		}

		CHAOSCORE_API TVector<T, d> FindClosestPoint(const TVector<T, d>& StartPoint, const T Thickness = (T)0) const;

		CHAOSCORE_API Pair<TVector<FReal, d>, bool> FindClosestIntersectionImp(const TVector<FReal, d>& StartPoint, const TVector<FReal, d>& EndPoint, const FReal Thickness) const;

		FORCEINLINE TVector<T, d> FindGeometryOpposingNormal(const TVector<T, d>& DenormDir, int32 FaceIndex, const TVector<T, d>& OriginalNormal) const 
		{
			// Find which faces were included in the contact normal, and for multiple faces, use the one most opposing the sweep direction.
			TVector<T, d> BestNormal(OriginalNormal);
			T BestOpposingDot = TNumericLimits<T>::Max();

			for (int32 Axis = 0; Axis < d; Axis++)
			{
				// Select axis of face to compare to, based on normal.
				if (OriginalNormal[Axis] > UE_KINDA_SMALL_NUMBER)
				{
					const T TraceDotFaceNormal = DenormDir[Axis]; // TraceDirDenormLocal.dot(BoxFaceNormal)
					if (TraceDotFaceNormal < BestOpposingDot)
					{
						BestOpposingDot = TraceDotFaceNormal;
						BestNormal = TVector<T, d>(0);
						BestNormal[Axis] = 1;
					}
				}
				else if (OriginalNormal[Axis] < -UE_KINDA_SMALL_NUMBER)
				{
					const T TraceDotFaceNormal = -DenormDir[Axis]; // TraceDirDenormLocal.dot(BoxFaceNormal)
					if (TraceDotFaceNormal < BestOpposingDot)
					{
						BestOpposingDot = TraceDotFaceNormal;
						BestNormal = FVector(0.f);
						BestNormal[Axis] = -1.f;
					}
				}
			}

			return BestNormal;
		}
		
		FORCEINLINE_DEBUGGABLE TVector<T, d> Support(const TVector<T, d>& Direction, const T Thickness, int32& OutVertexIndex) const
		{
			TVector<T, d> ChosenPt;
			FIntVector ChosenAxis;
			for (int Axis = 0; Axis < d; ++Axis)
			{
				if(Direction[Axis] < 0)
				{
					ChosenPt[Axis] = MMin[Axis];
					ChosenAxis[Axis] = 0;
				}
				else
				{
					ChosenPt[Axis] = MMax[Axis];
					ChosenAxis[Axis] = 1;
				}
			}

			OutVertexIndex = GetIndex(ChosenAxis);

			if (Thickness != (T)0)
			{
				//We want N / ||N|| and to avoid inf
				//So we want N / ||N|| < 1 / eps => N eps < ||N||, but this is clearly true for all eps < 1 and N > 0
				T SizeSqr = Direction.SizeSquared();
				if (SizeSqr <= TNumericLimits<T>::Min())
				{
					return ChosenPt;
				}
				const TVector<T, d> Normalized = Direction / FMath::Sqrt(SizeSqr);

				const TVector<T, d> InflatedPt = ChosenPt + Normalized.GetSafeNormal() * Thickness;
				return InflatedPt;
			}

			return ChosenPt;
		}

		// Support vertex in the specified direction, assuming each face has been moved inwards by InMargin
		FORCEINLINE_DEBUGGABLE FVec3 SupportCore(const FVec3& Direction, const FReal InMargin, FReal* OutSupportDelta, int32& OutVertexIndex) const
		{
			FVec3 ChosenPt;
			FIntVector ChosenAxis;
			for (int Axis = 0; Axis < d; ++Axis)
			{
				if(Direction[Axis] < 0)
				{
					ChosenPt[Axis] = MMin[Axis] + InMargin;
					ChosenAxis[Axis] = 0;
				}
				else
				{
					ChosenPt[Axis] = MMax[Axis] - InMargin;
					ChosenAxis[Axis] = 1;
				}
			}

			OutVertexIndex = GetIndex(ChosenAxis);

			// Maximum distance between the Core+Margin position and the original outer vertex
			constexpr FReal RootThreeMinusOne = FReal(1.7320508075688772935274463415059 - 1.0);
			if (OutSupportDelta != nullptr)
			{
				*OutSupportDelta = RootThreeMinusOne * InMargin;
			}

			return ChosenPt;
		}

		FORCEINLINE_DEBUGGABLE VectorRegister4Float SupportCoreSimd(const VectorRegister4Float& Direction, const FReal InMargin) const
		{
			FVec3 DirectionVec3;
			VectorStoreFloat3(Direction, &DirectionVec3);
			int32 VertexIndex = INDEX_NONE;
			FVec3 SupportVert = SupportCore(DirectionVec3, InMargin, nullptr, VertexIndex);
			return MakeVectorRegisterFloatFromDouble(MakeVectorRegister(SupportVert.X, SupportVert.Y, SupportVert.Z, 0.0));
		}

		FORCEINLINE_DEBUGGABLE TVector<T, d> SupportCoreScaled(const TVector<T, d>& Direction, const T InMargin, const TVector<T, d>& Scale, T* OutSupportDelta, int32& OutVertexIndex) const
		{
			const TVector<T, d> ScaledDirection = Direction * Scale; // Compensate for Negative scales, only the signs really matter here

			TVector<T, d> ChosenPt;
			FIntVector ChosenAxis;
			for (int Axis = 0; Axis < d; ++Axis)
			{
				if(ScaledDirection[Axis] < 0)
				{
					ChosenPt[Axis] = Scale[Axis] * MMin[Axis] + InMargin;
					ChosenAxis[Axis] = 0;
				}
				else
				{
					ChosenPt[Axis] = Scale[Axis] *  MMax[Axis] - InMargin;
					ChosenAxis[Axis] = 1;
				}
			}

			OutVertexIndex = GetIndex(ChosenAxis);

			constexpr T RootThreeMinusOne = T(1.7320508075688772935274463415059 - 1.0);
			if (OutSupportDelta != nullptr)
			{
				*OutSupportDelta = RootThreeMinusOne * InMargin;
			}

			return ChosenPt;
		}


		FORCEINLINE void GrowToInclude(const TVector<T, d>& V)
		{
			MMin = TVector<T, d>(FGenericPlatformMath::Min(MMin[0], V[0]), FGenericPlatformMath::Min(MMin[1], V[1]), FGenericPlatformMath::Min(MMin[2], V[2]));
			MMax = TVector<T, d>(FGenericPlatformMath::Max(MMax[0], V[0]), FGenericPlatformMath::Max(MMax[1], V[1]), FGenericPlatformMath::Max(MMax[2], V[2]));
		}

		FORCEINLINE void GrowToInclude(const TAABB<T, d>& Other)
		{
			MMin = TVector<T, d>(FGenericPlatformMath::Min(MMin[0], Other.MMin[0]), FGenericPlatformMath::Min(MMin[1], Other.MMin[1]), FGenericPlatformMath::Min(MMin[2], Other.MMin[2]));
			MMax = TVector<T, d>(FGenericPlatformMath::Max(MMax[0], Other.MMax[0]), FGenericPlatformMath::Max(MMax[1], Other.MMax[1]), FGenericPlatformMath::Max(MMax[2], Other.MMax[2]));
		}

		FORCEINLINE void ShrinkToInclude(const TAABB<T, d>& Other)
		{
			MMin = TVector<T, d>(FGenericPlatformMath::Max(MMin[0], Other.MMin[0]), FGenericPlatformMath::Max(MMin[1], Other.MMin[1]), FGenericPlatformMath::Max(MMin[2], Other.MMin[2]));
			MMax = TVector<T, d>(FGenericPlatformMath::Min(MMax[0], Other.MMax[0]), FGenericPlatformMath::Min(MMax[1], Other.MMax[1]), FGenericPlatformMath::Min(MMax[2], Other.MMax[2]));
		}

		FORCEINLINE TAABB<T, d>& Thicken(const T Thickness)
		{
			MMin -= TVector<T, d>(Thickness);
			MMax += TVector<T, d>(Thickness);
			return *this;
		}

		//Grows the box by this vector symmetrically - Changed name because previous Thicken had different semantics which caused several bugs
		FORCEINLINE TAABB<T, d>& ThickenSymmetrically(const TVector<T, d>& Thickness)
		{
			const TVector<T, d> AbsThickness = TVector<T, d>(FGenericPlatformMath::Abs(Thickness.X), FGenericPlatformMath::Abs(Thickness.Y), FGenericPlatformMath::Abs(Thickness.Z));
			MMin -= AbsThickness;
			MMax += AbsThickness;
			return *this;
		}

		FORCEINLINE TAABB<T, d>& ShrinkSymmetrically(const TVector<T, d>& Thickness)
		{
			const TVector<T, d> AbsThickness = TVector<T, d>(FGenericPlatformMath::Abs(Thickness.X), FGenericPlatformMath::Abs(Thickness.Y), FGenericPlatformMath::Abs(Thickness.Z));
			MMin += AbsThickness;
			MMax -= AbsThickness;
			return *this;
		}

		/** Grow along a vector (as if swept by the vector's direction and magnitude) */
		FORCEINLINE TAABB<T, d>& GrowByVector(const TVector<T, d>& V)
		{
			MMin += V.ComponentwiseMin(TVector<T, d>(0));
			MMax += V.ComponentwiseMax(TVector<T, d>(0));
			return *this;
		}

		// Move the bounding box along a vector
		FORCEINLINE void MoveByVector(TVector<T, d> InVector)
		{
			MMin += InVector;
			MMax += InVector;
		}

		FORCEINLINE TVector<T, d> Center() const { return (MMax - MMin) * T(0.5) + MMin; }
		FORCEINLINE TVector<T, d> GetCenter() const { return Center(); }
		FORCEINLINE TVector<T, d> GetCenterOfMass() const { return GetCenter(); }
		FORCEINLINE TVector<T, d> Extents() const { return MMax - MMin; }

		/**
		*	Get a vector one of the eight corners of the box.
		*	
		*	Each of the first three bits in an index are used to pick
		*	an axis value from either the min or max vector of the AABB.
		*	0 = min, 1 = max.
		*	
		*	This algorithm produces the following index scheme, where
		*	the vertex at 0 is the "min" vertex and 7 is the "max":
		*	
		*	   6---------7
		*	  /|        /|
		*	 / |       / |
		*	4---------5  |
		*	|  |      |  |
		*	|  2------|--3
		*	| /       | /
		*	|/        |/
		*	0---------1
		*/
		FORCEINLINE TVector<T, d> GetVertex(const int32 Index) const
		{
			check(0 <= Index && Index < 8);

			// See GetIndex() for reverse logic
			return TVector<T, d>(
				(Index & (1 << 0)) == 0 ? MMin.X : MMax.X,
				(Index & (1 << 1)) == 0 ? MMin.Y : MMax.Y,
				(Index & (1 << 2)) == 0 ? MMin.Z : MMax.Z);
		}

		/**
		* Given a point on a unit cube expressed as an IntVector of 0s and 1s, get the vertex index of that point.
		* 0 represents a negative axis and 1 represents a positive axis. All other values are invalid, but we do 
		* not check this as it would add overhead to support functions. This is used by the support functions and 
		* must invert the index logic in GetVertex().
		*/
		FORCEINLINE int32 GetIndex(const FIntVector& AxisSelector) const
		{
			// See GetVertex() for reverse logic
			return AxisSelector[0] + AxisSelector[1] * 2 + AxisSelector[2] * 4;
		}

		/**
		*	Get an array of two indices into the vertex list for one of the
		*	twelve edges of the box. Edges are ordered by increasing vertex
		*	indices.
		*	
		*	This algorithm produces the following index scheme, where the
		*	bottom left vertex is the "min" vertex and the top right is "max":
		*	
		*	        6-----11------7
		*	       /|            /|
		*	      9 |          10 |
		*	     /  6          /  7
		*	    4------8------5   |
		*	    |   |         |   |
		*	    |   2------5--|---3
		*	    2  /          4  /
		*	    | 1           | 3
		*	    |/            |/
		*	    0------0------1
		*/
		FORCEINLINE FAABBEdge GetEdge(const int32 Index) const
		{
			// See "GetVertex(int32)"
			check(0 <= Index && Index < 12);
			static constexpr FAABBEdge Edges[]
			{
				{ 0, 1 }, { 0, 2 }, { 0, 4 },
				{ 1, 3 }, { 1, 5 }, { 2, 3 },
				{ 2, 6 }, { 3, 7 }, { 4, 5 },
				{ 4, 6 }, { 5, 7 }, { 6, 7 }
			};
			return Edges[Index];
		}

		FORCEINLINE FAABBFace GetFace(const int32 Index) const
		{
			// See "GetVertex(int32)"
			check(0 <= Index && Index < 6);
			static constexpr FAABBFace Faces[]
			{
				{{ 0, 1, 5, 4 }, {0, 4, 8, 2}},
				{{ 1, 3, 7, 5 }, {3, 7, 10, 4}},
				{{ 4, 5, 7, 6 }, {8, 10, 11, 9}},
				{{ 0, 2, 3, 1 }, {1, 5, 3, 0}},
				{{ 0, 4, 6, 2 }, {2, 9, 6, 1}},
				{{ 2, 6, 7, 3 }, {6, 11, 7, 5}}
			};
			return Faces[Index];
		}		

		FORCEINLINE int LargestAxis() const
		{
			const auto Extents = this->Extents();
			if (Extents[0] > Extents[1] && Extents[0] > Extents[2])
			{
				return 0;
			}
			else if (Extents[1] > Extents[2])
			{
				return 1;
			}
			else
			{
				return 2;
			}
		}

		FORCEINLINE TAABB<T, d>& ScaleWithNegative(const TVector<T, d>& InScale)
		{
			*this = FromPoints(MMin * InScale, MMax * InScale);
			return *this;
		}

		/**
		* Scale the AABB relative to the origin
		* IMPORTANT : this does not support negative scale
		*/
		FORCEINLINE TAABB<T, d>& Scale(const TVector<T, d>& InScale)
		{
			MMin *= InScale;
			MMax *= InScale;
			return *this;
		}

		/**
		* Scale the AABB relative to its center
		* IMPORTANT : this does not support negative scale 
		*/
		FORCEINLINE TAABB<T, d>& LocalScale(const TVector<T, d>& InScale)
		{
			const TVector<T, d> BoxCenter = Center();
			const TVector<T, d> ScaledHalfExtents = Extents() * InScale * FReal(0.5);
			MMin = BoxCenter - ScaledHalfExtents;
			MMax = BoxCenter + ScaledHalfExtents;
			return *this;
		}

		FORCEINLINE const TVector<T, d>& Min() const { return MMin; }
		FORCEINLINE const TVector<T, d>& Max() const { return MMax; }

		// The radius of the sphere centered on the AABB origin (not the AABB center) which would encompass this AABB
		FORCEINLINE T OriginRadius() const
		{
			const TVector<T, d> MaxAbs = TVector<T, d>::Max(MMin.GetAbs(), MMax.GetAbs());
			return MaxAbs.Size();
		}

		FORCEINLINE T CenterRadius() const
		{
			return (T(0.5) * Extents()).Size();
		}

		FORCEINLINE T GetArea() const { return GetArea(Extents()); }
		FORCEINLINE static T GetArea(const TVector<T, d>& Dim) { return d == 2 ? Dim.Product() : (T)2. * (Dim[0] * Dim[1] + Dim[0] * Dim[2] + Dim[1] * Dim[2]); }

		FORCEINLINE T GetVolume() const { return GetVolume(Extents()); }
		FORCEINLINE static T GetVolume(const TVector<T, 3>& Dim) { return Dim.Product(); }

		FORCEINLINE T GetMargin() const { return 0; }
		FORCEINLINE T GetRadius() const { return 0; }
		FORCEINLINE FRealSingle GetMarginf() const { return 0.0f; }
		FORCEINLINE FRealSingle GetRadiusf() const { return 0.0f; }

		// A bounding box that will fail all overlap tests.
		// NOTE: this bounds cannot be transformed (all transform return EmptyAABB)
		FORCEINLINE static TAABB<T, d> EmptyAABB() { return TAABB<T, d>(TVector<T, d>(TNumericLimits<T>::Max()), TVector<T, d>(-TNumericLimits<T>::Max())); }

		// A bounding box that overlaps all space.
		// NOTE: this bounds cannot be transformed (all transforms return FullAABB)
		FORCEINLINE static TAABB<T, d> FullAABB() { return TAABB<T, d>(TVector<T, d>(-TNumericLimits<T>::Max()), TVector<T, d>(TNumericLimits<T>::Max())); }

		// A single-point bounds at the origin
		FORCEINLINE static TAABB<T, d> ZeroAABB() { return TAABB<T, d>(TVector<T, d>((T)0), TVector<T, d>((T)0)); }


		// If this AABB covering all of space (all overlap tests will succeed)
		// NOTE: This just checks if Max.X is Real::Max. The assumption is that max numeric values 
		// are only used when explicitly set in EmptyAABB() and FullAABB().
		FORCEINLINE bool IsFull() const
		{
			return (MMax.X == TNumericLimits<T>::Max());
		}

		// If this AABB empty (all overlap tests will fail)
		// NOTE: This just checks if Min.X is Real::Max. The assumption is that max numeric values 
		// are only used when explicitly set in EmptyAABB() and FullAABB().
		FORCEINLINE bool IsEmpty() const
		{
			return (MMin.X == TNumericLimits<T>::Max());
		}

		// AABB is valid if it doesn't contain nan and it's non empty (min <= max)
		FORCEINLINE_DEBUGGABLE bool IsValid() const
		{
			return MMin <= MMax && !MMin.ContainsNaN() && !MMax.ContainsNaN();
		}

		// Shrink this AABB to being empty
		FORCEINLINE void Clear()
		{
			MMin.X = TNumericLimits<T>::Max();
			MMin.Y = TNumericLimits<T>::Max();
			MMin.Z = TNumericLimits<T>::Max();
			MMax.X = -TNumericLimits<T>::Max();
			MMax.Y = -TNumericLimits<T>::Max();
			MMax.Z = -TNumericLimits<T>::Max();
		}

		FORCEINLINE void Serialize(FArchive &Ar) 
		{
			Ar << MMin << MMax;
		}

		FORCEINLINE uint32 GetTypeHash() const
		{
			return HashCombine(UE::Math::GetTypeHash(MMin), UE::Math::GetTypeHash(MMax));
		}

		FString ToString() const
		{
			return FString::Printf(TEXT("AABB: Min: [%s], Max: [%s]"), *MMin.ToString(), *MMax.ToString());
		}

		FORCEINLINE PMatrix<FReal, d, d> GetInertiaTensor(const FReal Mass) const { return GetInertiaTensor(Mass, Extents()); }
		FORCEINLINE static PMatrix<FReal, 3, 3> GetInertiaTensor(const FReal Mass, const TVector<FReal, 3>& Dim)
		{
			// https://www.wolframalpha.com/input/?i=cuboid
			const FReal M = Mass / 12;
			const FReal WW = Dim[0] * Dim[0];
			const FReal HH = Dim[1] * Dim[1];
			const FReal DD = Dim[2] * Dim[2];
			return PMatrix<FReal, 3, 3>(M * (HH + DD), M * (WW + DD), M * (WW + HH));
		}

		FORCEINLINE static TRotation<FReal, d> GetRotationOfMass()
		{
			return TRotation<FReal, d>::FromIdentity();
		}

		FORCEINLINE constexpr bool IsConvex() const { return true; }

		/**
		 * Given a set of points, wrap an AABB around them
		 * @param P0 The first of the points to wrap
		 * @param InPoints Parameter pack of all subsequent points
		 */
		template<typename... Points>
		static TAABB<T, d> FromPoints(const TVector<T, d>& P0, const Points&... InPoints)
		{
			static_assert(sizeof...(InPoints) > 0);
			static_assert(std::is_same_v<std::common_type_t<Points...>, TVector<T, d>>);

			TAABB<T, d> Result(P0, P0);
			(Result.GrowToInclude(InPoints), ...);
			return Result;
		}

	private:
		TVector<T, d> MMin, MMax;
	};

	template<class T, int d>
	FORCEINLINE FArchive& operator<<(FArchive& Ar, TAABB<T, d>& AABB)
	{
		AABB.Serialize(Ar);
		return Ar;
	}

	template<typename T>
	struct TAABBSpecializeSamplingHelper<T, 2>
	{
		static FORCEINLINE TArray<TVector<T, 2>> ComputeSamplePoints(const TAABB<T, 2>& AABB)
		{
			const TVector<T, 2>& Min = AABB.Min();
			const TVector<T, 2>& Max = AABB.Max();
			const TVector<T, 2> Mid = AABB.Center();

			TArray<TVector<T, 2>> SamplePoints;
			SamplePoints.SetNum(8);
			//top line (min y)
			SamplePoints[0] = TVector<T, 2>{Min.X, Min.Y};
			SamplePoints[1] = TVector<T, 2>{Mid.X, Min.Y};
			SamplePoints[2] = TVector<T, 2>{Max.X, Min.Y};

			//mid line (y=0) (mid point removed because internal)
			SamplePoints[3] = TVector<T, 2>{Min.X, Mid.Y};
			SamplePoints[4] = TVector<T, 2>{Max.X, Mid.Y};

			//bottom line (max y)
			SamplePoints[5] = TVector<T, 2>{Min.X, Max.Y};
			SamplePoints[6] = TVector<T, 2>{Mid.X, Max.Y};
			SamplePoints[7] = TVector<T, 2>{Max.X, Max.Y};

			return SamplePoints;
		}
	};

	template<typename T>
	struct TAABBSpecializeSamplingHelper<T, 3>
	{
		static FORCEINLINE TArray<TVector<T, 3>> ComputeSamplePoints(const TAABB<T, 3>& AABB)
		{
			const TVector<T, 3>& Min = AABB.Min();
			const TVector<T, 3>& Max = AABB.Max();
			const TVector<T, 3> Mid = AABB.Center();

			//todo(ocohen): should order these for best levelset cache traversal
			TArray<TVector<T, 3>> SamplePoints;
			SamplePoints.SetNum(26);
			{
				//xy plane for Min Z
				SamplePoints[0] = TVector<T, 3>{Min.X, Min.Y, Min.Z};
				SamplePoints[1] = TVector<T, 3>{Mid.X, Min.Y, Min.Z};
				SamplePoints[2] = TVector<T, 3>{Max.X, Min.Y, Min.Z};

				SamplePoints[3] = TVector<T, 3>{Min.X, Mid.Y, Min.Z};
				SamplePoints[4] = TVector<T, 3>{Mid.X, Mid.Y, Min.Z};
				SamplePoints[5] = TVector<T, 3>{Max.X, Mid.Y, Min.Z};

				SamplePoints[6] = TVector<T, 3>{Min.X, Max.Y, Min.Z};
				SamplePoints[7] = TVector<T, 3>{Mid.X, Max.Y, Min.Z};
				SamplePoints[8] = TVector<T, 3>{Max.X, Max.Y, Min.Z};
			}

			{
				//xy plane for z = 0 (skip mid point since inside)
				SamplePoints[9] = TVector<T, 3>{Min.X, Min.Y, Mid.Z};
				SamplePoints[10] = TVector<T, 3>{Mid.X, Min.Y, Mid.Z};
				SamplePoints[11] = TVector<T, 3>{Max.X, Min.Y, Mid.Z};

				SamplePoints[12] = TVector<T, 3>{Min.X, Mid.Y, Mid.Z};
				SamplePoints[13] = TVector<T, 3>{Max.X, Mid.Y, Mid.Z};

				SamplePoints[14] = TVector<T, 3>{Min.X, Max.Y, Mid.Z};
				SamplePoints[15] = TVector<T, 3>{Mid.X, Max.Y, Mid.Z};
				SamplePoints[16] = TVector<T, 3>{Max.X, Max.Y, Mid.Z};
			}

			{
				//xy plane for Max Z
				SamplePoints[17] = TVector<T, 3>{Min.X, Min.Y, Max.Z};
				SamplePoints[18] = TVector<T, 3>{Mid.X, Min.Y, Max.Z};
				SamplePoints[19] = TVector<T, 3>{Max.X, Min.Y, Max.Z};

				SamplePoints[20] = TVector<T, 3>{Min.X, Mid.Y, Max.Z};
				SamplePoints[21] = TVector<T, 3>{Mid.X, Mid.Y, Max.Z};
				SamplePoints[22] = TVector<T, 3>{Max.X, Mid.Y, Max.Z};

				SamplePoints[23] = TVector<T, 3>{Min.X, Max.Y, Max.Z};
				SamplePoints[24] = TVector<T, 3>{Mid.X, Max.Y, Max.Z};
				SamplePoints[25] = TVector<T, 3>{Max.X, Max.Y, Max.Z};
			}

			return SamplePoints;
		}
	};

	using FAABB3 = TAABB<FReal, 3>;
}
