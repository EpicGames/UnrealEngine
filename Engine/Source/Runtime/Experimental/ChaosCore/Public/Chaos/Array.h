// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Templates/TypeHash.h"

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Containers/Array.h"
#else
#include <vector>

template<class T>
class TArray : public std::vector<T>
{
  public:
	TArray()
	    : std::vector<T>() {}
	TArray(TArray<T>&& Other)
	    : std::vector<T>(std::move(Other)) {}
	TArray(const TArray<T>& Other)
	    : std::vector<T>(Other) {}
	TArray(std::initializer_list<T> l)
	    : std::vector<T>(l) {}
	void operator=(TArray<T>&& Other)
	{
		std::vector<T>::operator=(std::move(Other));
	}
	void SetNum(const int32 Size)
	{
		std::vector<T>::resize(Size);
	}
	int32 Num() const
	{
		return static_cast<int32>(std::vector<T>::size());
	}
	void Add(const T& Elem)
	{
		std::vector<T>::push_back(Elem);
	}
	void Sort()
	{
		std::sort(begin(), end());
	}

	friend uint32 GetTypeHash(const TArray<int32>& Array)
	{
		uint32 Seed = 0;
		for (const auto& Elem : Array)
			Seed ^= GetTypeHash(Elem) + 0x9e3779b9 + (Seed << 6) + (Seed >> 2);
		return Seed;
	}
};

namespace std
{
template<>
struct hash<TArray<int32>>
{
	std::size_t operator()(const TArray<int32>& Array) const
	{
		size_t Seed = 0;
		for (const auto& Elem : Array)
			Seed ^= std::hash<int>()(Elem) + 0x9e3779b9 + (Seed << 6) + (Seed >> 2);
		return Seed;
	}
};
}
#endif
