// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"
#include "Chaos/ArrayCollectionArrayBase.h"
#include "Chaos/ArrayCollectionArray.h"

namespace Chaos
{
class TArrayCollection
{
public:
	TArrayCollection()
	    : MSize(0) {}
	TArrayCollection(const TArrayCollection& Other) = delete;
	TArrayCollection(TArrayCollection&& Other) = delete;
	virtual ~TArrayCollection() 
	{
		// Null out to find dangling pointers
		for (int32 Index = 0; Index < MArrays.Num(); Index++)
		{
			MArrays[Index] = nullptr;
		}
	}

	void ShrinkArrays(const float MaxSlackFraction, const int32 MinSlack)
	{
		for (int32 Index = 0; Index < MArrays.Num(); Index++)
		{
			if (MArrays[Index] != nullptr)
			{
				MArrays[Index]->ApplyShrinkPolicy(MaxSlackFraction, MinSlack);
			}
		}
	}

	int32 AddArray(TArrayCollectionArrayBase* Array)
	{
		int32 Index = MArrays.Find(nullptr);
		if(Index == INDEX_NONE)
		{
			Index = MArrays.Num();
			MArrays.Add(Array);
		}
		else
		{
			MArrays[Index] = Array;
		}
		MArrays[Index]->Resize(MSize);
		return Index;
	}

	void RemoveArray(TArrayCollectionArrayBase* Array)
	{
		const int32 Idx = MArrays.Find(Array);
		if(Idx != INDEX_NONE)
		{
			MArrays[Idx] = nullptr;
		}
	}

	void RemoveAt(int32 Index, int32 Count)
	{
		RemoveAtHelper(Index, Count);
	}

	uint32 Size() const 
	{ return MSize; }

	uint64 ComputeColumnSize() const
	{
		uint64 Size = 0;
		for (TArrayCollectionArrayBase* Array : MArrays)
		{
			if (Array)
			{
				Size += Array->SizeOfElem();
			}
		}

		return Size;
	}

protected:
	void AddElementsHelper(const int32 Num)
	{
		if(Num == 0)
		{
			return;
		}
		ResizeHelper(MSize + Num);
	}

	void ResizeHelper(const int32 Num)
	{
		check(Num >= 0);
		MSize = Num;
		for (TArrayCollectionArrayBase* Array : MArrays)
		{
			if(Array)
			{
				Array->Resize(Num);
			}
		}
	}

	void RemoveAtHelper(const int32 Index, const int32 Count)
	{
		for (TArrayCollectionArrayBase* Array : MArrays)
		{
			if (Array)
			{
				Array->RemoveAt(Index, Count);
			}
		}
		const int32 AvailableToRemove = MSize - Index;
		MSize -= FMath::Min(AvailableToRemove, Count);
	}

	void RemoveAtSwapHelper(const int32 Index)
	{
		check(static_cast<uint32>(Index) < MSize);
		for (TArrayCollectionArrayBase* Array : MArrays)
		{
			if (Array)
			{
				Array->RemoveAtSwap(Index);
			}
		}
		MSize--;
	}

	void MoveToOtherArrayCollection(const int32 Index, TArrayCollection& Other)
	{
		check(MArrays.Num() == Other.MArrays.Num());
		check(static_cast<uint32>(Index) < MSize);

		for (int32 ArrayIdx = 0; ArrayIdx < MArrays.Num(); ++ArrayIdx)
		{
			if (TArrayCollectionArrayBase* Array = MArrays[ArrayIdx])
			{
				Array->MoveToOtherArray(Index, *Other.MArrays[ArrayIdx]);
			}
		}
		++Other.MSize;
		--MSize;
	}
	/** Move collection data from one array to another one */
	void MoveToOtherArrayCollection(const TArrayView<int32>& Indices, TArrayCollection& Other)
	{
		if (Indices.Num() <= static_cast<int32>(MSize))
		{
			for(int32 Index : Indices)
			{
				if (static_cast<uint32>(Index) >= MSize)
				{
					return;
				}
			}
			if(MArrays.Num() == Other.MArrays.Num())
			{ 
				bool bCanAddIndices = false;

				// For each collection arrays add data to the new array
				for (int32 ArrayIdx = 0; ArrayIdx < MArrays.Num(); ++ArrayIdx)
				{
					if(MArrays[ArrayIdx] && Other.MArrays[ArrayIdx])
					{ 
						MArrays[ArrayIdx]->MoveToOtherArray(Indices, *Other.MArrays[ArrayIdx], false);
						bCanAddIndices = true;
					}
				}
				// Update the old collection size if at least one array has been moved
				if(bCanAddIndices)
				{
					Other.MSize += Indices.Num();
				}
			}
				
			// For each collection arrays remove data from the old array
			RemoveAtSwapLambda(static_cast<uint32>(MSize), Indices, [this](const int32 RemovedIndex)
			{
				// Lambda function to remove at swap the data for a given index
				for (int32 ArrayIdx = 0; ArrayIdx < MArrays.Num(); ++ArrayIdx)
				{
					if(MArrays[ArrayIdx])
					{ 
						// The remove at swap from the TArrayCollectionArray is not shrinking the array which is 
						// necessary for the algorithm to work. Array could potentially be shrinked just after.
						MArrays[ArrayIdx]->RemoveAtSwap(RemovedIndex);
							
					}
				}
			});

			bool bCanRemoveIndices = false;

			// Lambda function to shrink the collection arrays a the end of removal
			for (int32 ArrayIdx = 0; ArrayIdx < MArrays.Num(); ++ArrayIdx)
			{
				if(MArrays[ArrayIdx])
				{ 
					MArrays[ArrayIdx]->Shrink();
					bCanRemoveIndices = true;
				}
			}
			// Update the new collection size if at least one array has been moved
			if(bCanRemoveIndices)
			{
				MSize -= Indices.Num();
			}
		}
	}

private:
	TArray<TArrayCollectionArrayBase*> MArrays;

protected:
	uint32 MSize;
};
}
