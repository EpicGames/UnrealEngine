// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/ArrayView.h"

#include "Chaos/Array.h"
#include "Chaos/ArrayCollectionArrayBase.h"
#include "Containers/BitArray.h"
#include <algorithm>

namespace Chaos
{

/** Remove at swap function working on an unsorted array of indices to remove. Sorting the 
 * indices in place would be an option but the complexity being O(Nlog(N)), it is probably too slow to be used for 
 * a lot of indices to remove. This algorithm requires allocating a mapping buffer to keep track of where the moved elements 
 * are and that the RemoveFunc is not modifying the original memory order */
template<typename RemoveIndexFunc>
FORCEINLINE void RemoveAtSwapLambda(const int32 ArraySize, const TArrayView<int32>& Indices, const RemoveIndexFunc& RemoveFunc)
{
	if (Indices.IsEmpty() || (Indices.Num() > ArraySize))
	{
		return;
	}
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	TBitArray<> ValidIndices;
	ValidIndices.Init(false,ArraySize);
	for (int32 RemovedIndex : Indices)
	{
		if ((RemovedIndex < ArraySize) && (RemovedIndex >= 0))
		{
			// Check if the list of indices to remove is unique or not
			check(!ValidIndices[RemovedIndex]);
			ValidIndices[RemovedIndex] = true;
		}
	}
#endif
	// The mapping array is twice the size M of the indices to be removed among the array of size N elements
	// [forward indices : array index position of the last M element position | reverse indices : element index value of the last M array positions]
	TArray<int32> IndexMapping;
	IndexMapping.Init(INDEX_NONE, 2 * Indices.Num());

	// Since we are storing only that last M elements we need to store the forward/reverse offset to come back to global indices
	const int32 ForwardOffset = ArraySize - Indices.Num();
	const int32 ReverseOffset = ForwardOffset - Indices.Num();

	// The goal here is to perform successive RemoveAtSwap on a list of indices without sorting them
	// since doing naively sequential calls could leads to wrong results if not sorted correctly
	int32 EndIndex = ArraySize - 1;
	for (int32 RemovedIndex : Indices)
	{
		if ((RemovedIndex < ArraySize) && (RemovedIndex >= 0) && IndexMapping.IsValidIndex(EndIndex - ReverseOffset))
		{
			const int32 ForwardMapping = (RemovedIndex < ForwardOffset) ? RemovedIndex : IndexMapping[RemovedIndex - ForwardOffset];
			const int32 ReverseMapping = IndexMapping[EndIndex - ReverseOffset];

			const int32 ForwardIndex = (ForwardMapping != INDEX_NONE) ? ForwardMapping : RemovedIndex;
			const int32 ReverseIndex = (ReverseMapping != INDEX_NONE) ? ReverseMapping : EndIndex;

			if (ReverseIndex >= ForwardOffset)
			{
				IndexMapping[ReverseIndex - ForwardOffset] = ForwardIndex;
			}
		
			// If ForwardIndex >= ForwardOffset then ForwardIndex >= ReverseOffset and the IndexMapping access is valid
			if (ForwardIndex >= ForwardOffset)
			{
				IndexMapping[ForwardIndex - ReverseOffset] = ReverseIndex;
			}
			RemoveFunc(ForwardIndex);
			--EndIndex;
		}
	}
}
	
template<class T>
class TArrayCollectionArray : public TArrayCollectionArrayBase, public TArray<T>
{
	using TArray<T>::SetNum;
	using TArray<T>::RemoveAt;
	using TArray<T>::RemoveAtSwap;
	using TArray<T>::Emplace;
	using TArray<T>::Shrink;
	using TArray<T>::Max;

public:
	constexpr static EAllowShrinking AllowShrinkOnRemove = EAllowShrinking::No;

	using TArray<T>::Num;

	TArrayCollectionArray()
	    : TArray<T>() {}
	TArrayCollectionArray(const TArrayCollectionArray<T>& Other) = delete;
	explicit TArrayCollectionArray(TArrayCollectionArray<T>&& Other)
	    : TArray<T>(MoveTemp(Other)) {}
	TArrayCollectionArray& operator=(TArrayCollectionArray<T>&& Other)
	{
		TArray<T>::operator=(MoveTemp(Other));
		return *this;
	}

	TArrayCollectionArray(TArray<T>&& Other)
	: TArray<T>(MoveTemp(Other))
	{
	}

	virtual ~TArrayCollectionArray() = default;

	void Fill(const T& Value)
	{
		for (int32 Idx = 0; Idx < TArray<T>::Num(); ++Idx)
		{
			TArray<T>::operator[](Idx) = Value;
		}
	}

	TArrayCollectionArray<T> Clone()
	{
		TArrayCollectionArray<T> NewArray;
		static_cast<TArray<T>>(NewArray) = static_cast<TArray<T>>(*this);
		return NewArray;
	}

	// If we have more slack space than MaxSlackFraction x Num(), run the default Shrink policy
	void ApplyShrinkPolicy(const float MaxSlackFraction, const int32 MinSlack) override
	{
		// Never shrink below this size
		const int32 Slack = Max() - Num();
		if (Slack <= MinSlack)
		{
			return;
		}

		// Shrink if we exceed the maximum allowed slack
		const int32 MaxSlack = FMath::Max(MinSlack, FMath::FloorToInt(MaxSlackFraction * float(Num())));
		if (Slack > MaxSlack)
		{
			Shrink();
		}
	}

	void Resize(const int Num) override
	{
		SetNum(Num, AllowShrinkOnRemove);
	}

	FORCEINLINE void RemoveAt(const int Idx, const int Count) override
	{
		TArray<T>::RemoveAt(Idx, Count, AllowShrinkOnRemove);
	}

	FORCEINLINE void RemoveAtSwap(const int Idx) override
	{
		TArray<T>::RemoveAtSwap(Idx, 1, AllowShrinkOnRemove);
	}

	/** Remove a list of elements from the array */
	FORCEINLINE void RemoveAtSwap(const TArrayView<int32>& Indices) override
	{
		RemoveAtSwapLambda(Num(), Indices, [this](const int32 RemovedIndex)
		{
			TArray<T>::RemoveAtSwap(RemovedIndex, 1, EAllowShrinking::No);
		});
		
		if (AllowShrinkOnRemove == EAllowShrinking::Yes)
		{
			TArray<T>::Shrink();
		}
	}

	/** Shrink the array size once the removal has been done. */
	FORCEINLINE void Shrink() override
	{
		TArray<T>::Shrink();
	}

	/** Move only one element from a collection to another one */
	FORCEINLINE void MoveToOtherArray(const int Idx, TArrayCollectionArrayBase& Other)
	{
		//todo: add developer check to make sure this is ok?
		auto& OtherTArray = static_cast<TArrayCollectionArray<T>&>(Other);
		OtherTArray.Emplace(MoveTemp(TArray<T>::operator [](Idx)));
		TArray<T>::RemoveAtSwap(Idx, 1, AllowShrinkOnRemove);
	}

	/** Move a list of elements from a collection to another one */
	FORCEINLINE void MoveToOtherArray(const TArrayView<int32>& Indices, TArrayCollectionArrayBase& Other, const bool bRemoveElements)
	{
		TArrayCollectionArray<T>& OtherTArray = static_cast<TArrayCollectionArray<T>&>(Other);
		OtherTArray.Reserve(OtherTArray.Num() + Indices.Num());
	
		// Add elements to the new collection arrays 
		for(int32 Idx : Indices)
		{
			if (TArray<T>::IsValidIndex(Idx))
			{
				OtherTArray.Emplace(MoveTemp(TArray<T>::operator [](Idx)));
			}
		}

		// Remove elements from the old collection arrays
		if(bRemoveElements)
		{ 
			RemoveAtSwap(Indices);
		}
	}

	FORCEINLINE uint64 SizeOfElem() const override
	{
		return sizeof(T);
	}
};
}

template<class T>
struct TIsContiguousContainer<Chaos::TArrayCollectionArray<T>>
{
	static constexpr bool Value = TIsContiguousContainer<TArrayView<T>>::Value;
};
