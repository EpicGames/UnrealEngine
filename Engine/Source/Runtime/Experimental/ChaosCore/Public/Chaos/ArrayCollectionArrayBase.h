// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Containers/ArrayView.h"

namespace Chaos
{
class TArrayCollectionArrayBase
{
  public:
	virtual void ApplyShrinkPolicy(const float MaxSlackFraction, const int32 MinSlack) = 0;
	virtual void Resize(const int Num) = 0;
	virtual void RemoveAt(const int Idx, const int Count) = 0;
	virtual void RemoveAtSwap(const int Idx) = 0;
	virtual void RemoveAtSwap(const TArrayView<int32>& Indices) = 0;
	virtual uint64 SizeOfElem() const = 0;
	virtual void MoveToOtherArray(const int Idx, TArrayCollectionArrayBase& Other) = 0;
	virtual void MoveToOtherArray(const TArrayView<int32>& Indices, TArrayCollectionArrayBase& Other, const bool bRemoveElements) = 0;
	virtual void Shrink() = 0;
};
}
