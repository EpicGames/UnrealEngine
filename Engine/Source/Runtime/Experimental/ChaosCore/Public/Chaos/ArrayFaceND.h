// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Bad include. Some headers are in Chaos while this is in ChaosCore

#include "Chaos/ArrayND.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Vector.h"

namespace Chaos
{
template<class T, int d>
class TArrayFaceND
{
	TArrayFaceND(const TUniformGrid<FReal, d>& Grid)
	{
		MArray.SetNum(d); // @todo(mlentine): This should not be needed
		const auto& Counts = Grid.Counts();
		for (int32 i = 0; i < d; ++i)
		{
			MArray[i] = TArrayND<T, 3>(Counts + TVec3<int32>::AxisVector(i));
		}
	}
	TArrayFaceND(const TVec3<int32>& Counts)
	{
		MArray.SetNum(d); // @todo(mlentine): This should not be needed
		for (int32 i = 0; i < d; ++i)
		{
			MArray[i] = TArrayND<T, 3>(Counts + TVec3<int32>::AxisVector(i));
		}
	}
	TArrayFaceND(const TArrayFaceND<T, d>& Other) = delete;
	TArrayFaceND(TArrayFaceND<T, d>&& Other)
	    : MArray(MoveTemp(Other.MArray)) {}

  private:
	TArray<TArrayND<T, d>, TFixedAllocator<d>> MArray;
};

template<class T>
class TArrayFaceND<T, 3>
{
  public:
	TArrayFaceND(const TUniformGrid<FReal, 3>& Grid)
	{
		MArray.SetNum(3); // @todo(mlentine): This should not be needed
		const auto& Counts = Grid.Counts();
		MArray[0] = TArrayND<T, 3>(Counts + TVec3<int32>::AxisVector(0));
		MArray[1] = TArrayND<T, 3>(Counts + TVec3<int32>::AxisVector(1));
		MArray[2] = TArrayND<T, 3>(Counts + TVec3<int32>::AxisVector(2));
	}
	TArrayFaceND(const TVec3<int32>& Counts)
	{
		MArray.SetNum(3); // @todo(mlentine): This should not be needed
		MArray[0] = TArrayND<T, 3>(Counts + TVec3<int32>::AxisVector(0));
		MArray[1] = TArrayND<T, 3>(Counts + TVec3<int32>::AxisVector(1));
		MArray[2] = TArrayND<T, 3>(Counts + TVec3<int32>::AxisVector(2));
	}
	TArrayFaceND(const TArrayFaceND<T, 3>& Other) = delete;
	TArrayFaceND(TArrayFaceND<T, 3>&& Other)
	    : MArray(MoveTemp(Other.MArray)) {}
	TArrayFaceND<T, 3> Copy()
	{
		return TArrayFaceND(MArray);
	}
	T& operator()(const Pair<int32, TVec3<int32>>& Index) { return MArray[Index.First](Index.Second[0], Index.Second[1], Index.Second[2]); }
	const T& operator()(const Pair<int32, TVec3<int32>>& Index) const { return MArray[Index.First](Index.Second[0], Index.Second[1], Index.Second[2]); }
	T& operator()(const int32 Axis, const int32& x, const int32& y, const int32& z) { return MArray[Axis](x, y, z); }
	const T& operator()(const int32 Axis, const int32& x, const int32& y, const int32& z) const { return MArray[Axis](x, y, z); }
	void Fill(const T& Scalar)
	{
		MArray[0].Fill(Scalar);
		MArray[1].Fill(Scalar);
		MArray[2].Fill(Scalar);
	}
	const TArrayND<T, 3>& GetComponent(const int32 Axis) const { return MArray[Axis]; }
	TArrayND<T, 3>& GetComponent(const int32 Axis) { return MArray[Axis]; }

  private:
	TArrayFaceND(const TArray<TArrayND<T, 3>, TFixedAllocator<3>>& Array)
	{
		MArray.SetNum(3); // @todo(mlentine): This should not be needed
		MArray[0] = Array[0].Copy();
		MArray[1] = Array[1].Copy();
		MArray[2] = Array[2].Copy();
	}
	TArray<TArrayND<T, 3>, TFixedAllocator<3>> MArray;
};
}
