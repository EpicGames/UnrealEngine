// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Bad include. Some headers are in Chaos while this is in ChaosCore

#include "Chaos/Core.h"
#include "Chaos/Array.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Vector.h"
#include "UObject/DestructionObjectVersion.h"
#include "Chaos/ChaosArchive.h"

namespace Chaos
{

template <typename T>
void TryBulkSerializeArrayNDBase(FArchive& Ar, TArray<T>& Array)
{
	Ar << Array;
}

inline void TryBulkSerializeArrayNDBase(FArchive& Ar, TArray<float>& Array)
{
	Array.BulkSerialize(Ar);
}

inline void TryBulkSerializeArrayNDBase(FArchive& Ar, TArray<TVec3<FRealSingle>>& Array)
{
	Array.BulkSerialize(Ar); 
}

inline float ConvertDoubleToFloat(double DoubleValue)
{
	return (float)DoubleValue; // LWC_TODO : Perf pessimization 
}

inline TVec3<float> ConvertDoubleToFloat(TVec3<double> DoubleValue)
{
	return TVec3<float>((float)DoubleValue.X, (float)DoubleValue.Y, (float)DoubleValue.Z); // LWC_TODO : Perf pessimization 
}

inline double ConvertFloatToDouble(float FloatValue)
{
	return (double)FloatValue;
}

inline TVec3<double> ConvertFloatToDouble(TVec3<float> FloatValue)
{
	return TVec3<double>((double)FloatValue.X, (double)FloatValue.Y, (double)FloatValue.Z); 
}

// LWC_TODO : Perf pessimization : this is sub-optimal but will do until we sort the serialization out
template<typename DOUBLE_T, typename FLOAT_T>
inline void TryBulkSerializeArrayNDBaseForDoubles(FArchive& Ar, TArray<DOUBLE_T>& DoubleTypedArray)
{
	TArray<FLOAT_T> FloatTypedArray;
	if (Ar.IsSaving())
	{
		FloatTypedArray.SetNumUninitialized(DoubleTypedArray.Num());
		for (int i = 0; i < DoubleTypedArray.Num(); ++i)
		{
			FloatTypedArray[i] = ConvertDoubleToFloat(DoubleTypedArray[i]);
		}
	}

	TryBulkSerializeArrayNDBase(Ar, FloatTypedArray);

	if (Ar.IsLoading())
	{
		DoubleTypedArray.SetNumUninitialized(FloatTypedArray.Num());
		for (int i = 0; i < FloatTypedArray.Num(); ++i)
		{
			DoubleTypedArray[i] = ConvertFloatToDouble(FloatTypedArray[i]);
		}
	}
}

// LWC_TODO : Perf pessimization : this is sub-optimal but will do until we sort the serialization out
inline void TryBulkSerializeArrayNDBase(FArchive& Ar, TArray<double>& Array)
{
	TryBulkSerializeArrayNDBaseForDoubles<double, float>(Ar, Array);
}

// LWC_TODO : Perf pessimization : this is sub-optimal but will do until we sort the serialization out
inline void TryBulkSerializeArrayNDBase(FArchive& Ar, TArray<TVec3<FRealDouble>>& Array)
{
	TryBulkSerializeArrayNDBaseForDoubles<TVec3<FRealDouble>, TVec3<FRealSingle>>(Ar, Array);
}

template<class T_DERIVED, class T, int d>
class TArrayNDBase
{
  public:

	FORCEINLINE TArrayNDBase() { MCounts = TVec3<int32>(0); }

	FORCEINLINE TArrayNDBase(const TVector<int32, d>& Counts, const TArray<T>& Array)
	    : MCounts(Counts), MArray(Array) {}
	FORCEINLINE TArrayNDBase(const TArrayNDBase<T_DERIVED, T, d>& Other) = delete;
	FORCEINLINE TArrayNDBase(TArrayNDBase<T_DERIVED, T, d>&& Other)
	    : MCounts(Other.MCounts), MArray(MoveTemp(Other.MArray)) {}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
	FORCEINLINE TArrayNDBase(std::istream& Stream)
	    : MCounts(Stream)
	{
		MArray.SetNum(MCounts[0] * MCounts[1] * MCounts[2]);
		Stream.read(reinterpret_cast<char*>(MArray.GetData()), sizeof(T) * MArray.Num());
	}
	FORCEINLINE void Write(std::ostream& Stream) const
	{
		MCounts.Write(Stream);
		Stream.write(reinterpret_cast<const char*>(MArray.GetData()), sizeof(T) * MArray.Num());
	}
#endif
	void Serialize(FArchive& Ar)
	{
		Ar.UsingCustomVersion(FDestructionObjectVersion::GUID);
		Ar << MCounts;

		if (Ar.CustomVer(FDestructionObjectVersion::GUID) < FDestructionObjectVersion::BulkSerializeArrays)
		{
			Ar << MArray;
		}
		else
		{
			TryBulkSerializeArrayNDBase(Ar, MArray);
		}
	}

	void Serialize(FChaosArchive& Ar)
	{
		Ar << MCounts;
		Ar << MArray;
	}

	FORCEINLINE TArrayNDBase<T_DERIVED, T, d>& operator=(const TArrayNDBase<T_DERIVED, T, d>& Other)
	{
		MCounts = Other.MCounts;
		MArray = Other.MArray;
		return *this;
	}

	FORCEINLINE TArrayNDBase<T_DERIVED, T, d>& operator=(TArrayNDBase<T_DERIVED, T, d>&& Other)
	{
		MCounts = Other.MCounts;
		MArray = MoveTemp(Other.MArray);
		return *this;
	}
	FORCEINLINE T_DERIVED Copy() const { return T_DERIVED(MCounts, MArray); }
	FORCEINLINE void Copy(const TArrayNDBase<T_DERIVED, T, d>& Source) { MCounts = Source.MCounts; MArray = Source.MArray; }
	FORCEINLINE void Fill(const T& Value)
	{
		for (auto& Elem : MArray)
		{
			Elem = Value;
		}
	}
	FORCEINLINE const T& operator[](const int32 i) const { return MArray[i]; }
	FORCEINLINE T& operator[](const int32 i) { return MArray[i]; }

	FORCEINLINE int32 Num() const { return MArray.Num(); }
	FORCEINLINE TVector<int32, d> Counts() const { return MCounts; }

	FORCEINLINE void Reset()
	{
		MCounts = TVector<int32, d>(0);
		MArray.Reset();
	}

	FORCEINLINE T* GetData() { return MArray.GetData(); }
	FORCEINLINE const T* GetData() const { return MArray.GetData(); }

  protected:
	TVector<int32, d> MCounts;
	TArray<T> MArray;
};

template <typename Derived, typename T, int d>
FArchive& operator<<(FArchive& Ar, TArrayNDBase<Derived, T,d>& ValueIn)
{
	ValueIn.Serialize(Ar);
	return Ar;
}

template <typename Derived, typename T, int d>
FArchive& operator<<(FChaosArchive& Ar, TArrayNDBase<Derived, T, d>& ValueIn)
{
	ValueIn.Serialize(Ar);
	return Ar;
}

template<class T, int d>
class TArrayND : public TArrayNDBase<TArrayND<T, d>, T, d>
{
	typedef TArrayNDBase<TArrayND<T, d>, T, d> Base;
	using Base::MArray;
	using Base::MCounts;

  public:
	FORCEINLINE TArrayND(const TVector<int32, d>& Counts) { MArray.SetNum(Counts.Product()); }
	FORCEINLINE TArrayND(const TVector<int32, d>& Counts, const TArray<T>& Array)
	    : Base(Counts, Array) {}
	FORCEINLINE TArrayND(const TArrayND<T, d>& Other) = delete;
	FORCEINLINE TArrayND(TArrayND<T, d>&& Other)
	    : Base(MoveTemp(Other)) {}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
	FORCEINLINE TArrayND(std::istream& Stream)
	    : Base(Stream) {}
#endif
	FORCEINLINE TArrayND<T, d>& operator=(const TArrayND<T, d>& Other)
	{
		Base::operator=(Other);
		return *this;
	}
	FORCEINLINE TArrayND<T, d>& operator=(TArrayND<T, d>&& Other)
	{
		Base::operator=(MoveTemp(Other));
		return *this;
	}
	FORCEINLINE T& operator()(const TVector<int32, d>& Index)
	{
		int32 SingleIndex = 0;
		int32 count = 1;
		for (int32 i = d - 1; i >= 0; ++i)
		{
			SingleIndex += count * Index[i];
			count *= MCounts[i];
		}
		return MArray[SingleIndex];
	}
};

template<class T>
class TArrayND<T, 3> : public TArrayNDBase<TArrayND<T, 3>, T, 3>
{
	typedef TArrayNDBase<TArrayND<T, 3>, T, 3> Base;
	using Base::MArray;
	using Base::MCounts;

  public:
#if UE_BUILD_SHIPPING || UE_BUILD_TEST
	FORCEINLINE TArrayND() {}
#else
	FORCEINLINE TArrayND() { MCounts = TVec3<int32>(0); }
#endif
	template<typename U>
	FORCEINLINE TArrayND(const TUniformGrid<U, 3>& Grid, bool NodeValues = false)
	{
		SetCounts(Grid, NodeValues);
	}
	FORCEINLINE TArrayND(const TVec3<int32>& Counts)
	{
		SetCounts(Counts);
	}
	FORCEINLINE TArrayND(const TVec3<int32>& Counts, const TArray<T>& Array)
	    : Base(Counts, Array) { check(Counts.Product() == Array.Num()); }
	FORCEINLINE TArrayND(const TArrayND<T, 3>& Other) = delete;
	FORCEINLINE TArrayND(TArrayND<T, 3>&& Other)
	    : Base(MoveTemp(Other)) {}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
	FORCEINLINE TArrayND(std::istream& Stream)
	    : Base(Stream) {}
#endif
	FORCEINLINE TArrayND<T, 3>& operator=(const TArrayND<T, 3>& Other)
	{
		Base::operator=(Other);
		return *this;
	}
	FORCEINLINE TArrayND<T, 3>& operator=(TArrayND<T, 3>&& Other)
	{
		Base::operator=(MoveTemp(Other));
		return *this;
	}
	FORCEINLINE T& operator()(const TVec3<int32>& Index) { return (*this)(Index[0], Index[1], Index[2]); }
	FORCEINLINE const T& operator()(const TVec3<int32>& Index) const { return (*this)(Index[0], Index[1], Index[2]); }
	FORCEINLINE T& operator()(const int32& x, const int32& y, const int32& z)
	{
		return MArray[(x * MCounts[1] + y) * MCounts[2] + z];
	}
	FORCEINLINE const T& operator()(const int32& x, const int32& y, const int32& z) const
	{
		return MArray[(x * MCounts[1] + y) * MCounts[2] + z];
	}

	FORCEINLINE void SetCounts(const TVector<int32, 3>& Counts)
	{
		MCounts = Counts;
		MArray.SetNum(MCounts[0] * MCounts[1] * MCounts[2]);
	}

	template<typename U>
	FORCEINLINE void SetCounts(const TUniformGrid<U, 3>& Grid, bool NodeValues = false)
	{
		MCounts = NodeValues ? Grid.NodeCounts() : Grid.Counts();
		MArray.SetNum(MCounts[0] * MCounts[1] * MCounts[2]);
	}
	
};

#if COMPILE_WITHOUT_UNREAL_SUPPORT
template<>
class TArrayND<bool, 3> : public TArrayNDBase<TArrayND<bool, 3>, char, 3>
{
	typedef bool T;
	typedef TArrayNDBase<TArrayND<T, 3>, char, 3> Base;
	using Base::MArray;
	using Base::MCounts;

  public:
	FORCEINLINE TArrayND() {}
	FORCEINLINE TArrayND(const TUniformGrid<float, 3>& grid)
	{
		MCounts = grid.Counts();
		MArray.SetNum(MCounts[0] * MCounts[1] * MCounts[2]);
	}
	FORCEINLINE TArrayND(const Vector<int32, 3>& Counts)
	{
		MCounts = Counts;
		MArray.SetNum(MCounts[0] * MCounts[1] * MCounts[2]);
	}
	FORCEINLINE TArrayND(const Vector<int32, 3>& Counts, const TArray<char>& Array)
	    : Base(Counts, Array) {}
	FORCEINLINE TArrayND(const TArrayND<T, 3>& Other) = delete;
	FORCEINLINE TArrayND(TArrayND<T, 3>&& Other)
	    : Base(std::move(Other)) {}
	FORCEINLINE char& operator()(const Vector<int32, 3>& Index) { return (*this)(Index[0], Index[1], Index[2]); }
	FORCEINLINE const T& operator()(const Vector<int32, 3>& Index) const { return (*this)(Index[0], Index[1], Index[2]); }
	FORCEINLINE char& operator()(const int32& x, const int32& y, const int32& z)
	{
		return MArray[(x * MCounts[1] + y) * MCounts[2] + z];
	}
	FORCEINLINE const T& operator()(const int32& x, const int32& y, const int32& z) const
	{
		return MArray[(x * MCounts[1] + y) * MCounts[2] + z];
	}
};
#endif
}
