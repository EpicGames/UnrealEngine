// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Containers/UnrealString.h"

class FString;

namespace Chaos
{
	CHAOSCORE_API FString ToString(const FRotation3& V);
	CHAOSCORE_API FString ToString(const FRotation3f& V);
	CHAOSCORE_API FString ToString(const FVec3& V);
	CHAOSCORE_API FString ToString(const FVec3f& V);
	CHAOSCORE_API FString ToString(const TBitArray<>& BitArray);
}