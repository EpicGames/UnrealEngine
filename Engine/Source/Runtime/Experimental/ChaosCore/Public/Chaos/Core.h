// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Real.h"
#include "Chaos/Vector.h"
#include "Chaos/Matrix.h"
#include "Chaos/Rotation.h"
#include "Chaos/Transform.h"
#include "Containers/UnrealString.h"

namespace Chaos
{
	template<class T, int d> class TAABB;

	using FVec2 = TVector<FReal, 2>;
	using FVec3 = TVector<FReal, 3>;
	using FVec4 = TVector<FReal, 4>;
	using FRotation3 = TRotation<FReal, 3>;
	using FMatrix33 = PMatrix<FReal, 3, 3>;
	using FMatrix44 = PMatrix<FReal, 4, 4>;
	using FRigidTransform3 = TRigidTransform<FReal, 3>;

	using FAABB3 = TAABB<FReal, 3>;

	using FVec2f = TVector<FRealSingle, 2>;
	using FVec3f = TVector<FRealSingle, 3>;
	using FRotation3f = TRotation<FRealSingle, 3>;
	using FRigidTransform3f = TRigidTransform<FRealSingle, 3>;
	using FTransformPair = TVector<FRigidTransform3, 2>;

	using FAABB3f = TAABB<FRealSingle, 3>;

	// @todo(chaos): deprecate this and use FRigidTransform3f
	using FRigidTransformRealSingle3 = TRigidTransform<FRealSingle, 3>;

	template <typename T>
	using TVec2 = TVector<T, 2>;

	template <typename T>
	using TVec3 = TVector<T, 3>;

	template <typename T>
	using TVec4 = TVector<T, 4>;

	template<typename T>
	using TRotation3 = TRotation<T, 3>;

	template<typename T>
	using TMatrix33 = PMatrix<T, 3, 3>;

	// NOTE: if you get a merge conflict on the GUID, you must replace it with a new GUID - do not accept the source or target
	// or you will likely get DDC version conflicts resulting in crashes during load.
	// Core version string for Chaos data. Any DDC builder dependent on Chaos for serialization should depend on this version
	inline const TCHAR* const ChaosVersionGUID = TEXT("5E07152B-EF71-47B8-B477-0CE28888D459");

	inline FString GetChaosVersionStringInner()
	{
		return FString(ChaosVersionGUID);
	}

#define ChaosVersionString \
	UE_DEPRECATED_MACRO(5.1, "ChaosVersionString is deprecated, please use ChaosVersionGUID instead") \
	GetChaosVersionStringInner()
}
