// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosCheck.h"
#include "Chaos/Core.h"
#include "Chaos/AABB.h"
#include "Chaos/CoreSegment.h"

#include "Serialization/Archive.h"

namespace Chaos
{
	class FCoreCapsule
	{
	public:
		FCoreCapsule() = default;
		FCoreCapsule(const FCoreCapsule&) = default;
		FCoreCapsule(FCoreCapsule&&) = default;
		CHAOSCORE_API FCoreCapsule(const TSegment<FRealSingle>& InSegment, const FRealSingle InRadius);
		CHAOSCORE_API FCoreCapsule(const FVec3f& X1, const FVec3f& X2, const FRealSingle InRadius);

		FCoreCapsule& operator=(const FCoreCapsule&) = default;
		FCoreCapsule& operator=(FCoreCapsule&&) = default;

		CHAOSCORE_API FRealSingle GetRadius() const;
		CHAOSCORE_API void SetRadius(FRealSingle InRadius);

		CHAOSCORE_API const FVec3f GetX1() const;
		CHAOSCORE_API const FVec3f GetX2() const;
		CHAOSCORE_API const TSegment<FRealSingle>& GetSegment() const;

		CHAOSCORE_API FRealSingle GetHeight() const;
		CHAOSCORE_API const FVec3f GetAxis() const;
		CHAOSCORE_API FVec3f GetCenter() const;

		CHAOSCORE_API const FAABB3 BoundingBox() const;

		CHAOSCORE_API bool Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal) const;

	private:

		TSegment<FRealSingle> MSegment;
		FRealSingle Radius = 0.0f;
	};
} // namespace Chaos
