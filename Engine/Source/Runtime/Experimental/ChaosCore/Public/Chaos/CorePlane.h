// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosCheck.h"
#include "Chaos/Core.h"
#include "Serialization/Archive.h"

namespace Chaos
{
	template <typename T, int d = 3>
	class TCorePlane
	{
	public:

		// Scale the plane and assume that any of the scale components could be zero
		static TCorePlane<T> MakeScaledSafe(const TCorePlane<T>& Plane, const TVec3<T>& Scale)
		{
			const TVec3<T> ScaledX = Plane.MX * Scale;

			// If all 3 scale components are non-zero we can just inverse-scale the normal
			// If 1 scale component is zero, the normal will point in that direction of the zero scale
			// If 2 scale components are zero, the normal will be zero along the non-zero scale direction
			// If 3 scale components are zero, the normal will be unchanged
			const int32 ZeroX = FMath::IsNearlyZero(Scale.X) ? 1 : 0;
			const int32 ZeroY = FMath::IsNearlyZero(Scale.Y) ? 1 : 0;
			const int32 ZeroZ = FMath::IsNearlyZero(Scale.Z) ? 1 : 0;
			const int32 NumZeros = ZeroX + ZeroY + ZeroZ;
			TVec3<T> ScaledN;
			if (NumZeros == 0)
			{
				// All 3 scale components non-zero
				ScaledN = TVec3<T>(Plane.MNormal.X / Scale.X, Plane.MNormal.Y / Scale.Y, Plane.MNormal.Z / Scale.Z);
			}
			else if (NumZeros == 1)
			{
				// Exactly one Scale component is zero
				ScaledN = TVec3<T>(
					(ZeroX) ? 1.0f : 0.0f,
					(ZeroY) ? 1.0f : 0.0f,
					(ZeroZ) ? 1.0f : 0.0f);
			}
			else if (NumZeros == 2)
			{
				// Exactly two Scale components is zero
				ScaledN = TVec3<T>(
					(ZeroX) ? Plane.MNormal.X : 0.0f,
					(ZeroY) ? Plane.MNormal.Y : 0.0f,
					(ZeroZ) ? Plane.MNormal.Z : 0.0f);
			}
			else // (NumZeros == 3)
			{
				// All 3 scale components are zero
				ScaledN = Plane.MNormal;
			}

			// Even after all the above, we may still get a zero normal (e.g., we scale N=(1,0,0) by S=(0,1,0))
			const T ScaleN2 = ScaledN.SizeSquared();
			if (ScaleN2 > UE_SMALL_NUMBER)
			{
				ScaledN = ScaledN * FMath::InvSqrt(ScaleN2);
			}
			else
			{
				ScaledN = Plane.MNormal;
			}

			return TCorePlane<T>(ScaledX, ScaledN);
		}

		// Scale the plane and assume that none of the scale components are zero
		template <typename U>
		static FORCEINLINE TCorePlane<T> MakeScaledUnsafe(const TCorePlane<U>& Plane, const TVec3<T>& Scale, const TVec3<T>& InvScale)
		{
			const TVec3<T> ScaledX = TVec3<T>(Plane.X()) * Scale;
			TVec3<T> ScaledN = TVec3<T>(Plane.Normal()) * InvScale;

			// We don't handle zero scales, but we could still end up with a small normal
			const T ScaleN2 = ScaledN.SizeSquared();
			if (ScaleN2 > UE_SMALL_NUMBER)
			{
				ScaledN = ScaledN * FMath::InvSqrt(ScaleN2);
			}
			else
			{
				ScaledN = TVec3<T>(Plane.Normal());
			}

			return TCorePlane<T>(ScaledX, ScaledN);
		}

		template <typename U>
		static FORCEINLINE void MakeScaledUnsafe(const TVec3<U>& PlaneN, const TVec3<U>& PlaneX, const TVec3<T>& Scale, const TVec3<T>& InvScale, TVec3<T>& OutN, TVec3<T>& OutX)
		{
			const TVec3<T> ScaledX = TVec3<T>(PlaneX * Scale);
			TVec3<T> ScaledN = TVec3<T>(PlaneN * InvScale);

			// We don't handle zero scales, but we could still end up with a small normal
			const T ScaleN2 = ScaledN.SizeSquared();
			if (ScaleN2 > UE_SMALL_NUMBER)
			{
				ScaledN = ScaledN * FMath::InvSqrt(ScaleN2);
			}
			else
			{
				ScaledN = TVec3<T>(PlaneN);
			}

			OutN = ScaledN;
			OutX = ScaledX;
		}


		template <typename U>
		static FORCEINLINE TCorePlane<T> MakeFrom(const TCorePlane<U>& Plane)
		{
			return TCorePlane<T>(TVec3<T>(Plane.X()), TVec3<T>(Plane.Normal()));
		}


		TCorePlane() = default;
		TCorePlane(const TVec3<T>& InX, const TVec3<T>& InNormal)
			: MX(InX)
			, MNormal(InNormal)
		{
			static_assert(d == 3, "Only dimension 3 is supported");
		}

		/**
		 * Phi is positive on the side of the normal, and negative otherwise.
		 */
		template <typename U>
		U SignedDistance(const TVec3<U>& x) const
		{
			return TVec3<U>::DotProduct(x - TVec3<U>(MX), TVec3<U>(MNormal));
		}

		/**
		 * Phi is positive on the side of the normal, and negative otherwise.
		 */
		FReal PhiWithNormal(const FVec3& x, FVec3& Normal) const
		{
			Normal = MNormal;
			return FVec3::DotProduct(x - (FVec3)MX, (FVec3)MNormal);
		}

		FVec3 FindClosestPoint(const FVec3& x, const FReal Thickness = (FReal)0) const
		{
			auto Dist = FVec3::DotProduct(x - (FVec3)MX, (FVec3)MNormal) - Thickness;
			return x - FVec3(Dist * MNormal);
		}

		bool Raycast(const FVec3& StartPoint, const FVec3& Dir, const FReal Length, const FReal Thickness, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex) const
		{
			ensure(FMath::IsNearlyEqual(Dir.SizeSquared(), (FReal)1, (FReal)UE_KINDA_SMALL_NUMBER));
			CHAOS_ENSURE(Length > 0);
			OutFaceIndex = INDEX_NONE;
			OutTime = 0;

			// This is mainly to fix static analysis warnings
			OutPosition = FVec3(0);
			OutNormal = FVec3(0);

			const FReal SignedDist = FVec3::DotProduct(StartPoint - (FVec3)MX, (FVec3)MNormal);
			if (FMath::Abs(SignedDist) < Thickness)
			{
				//initial overlap so stop
				//const FReal DirDotNormal = FVec3::DotProduct(Dir, (FVec3)MNormal);
				//OutPosition = StartPoint;
				//OutNormal = DirDotNormal < 0 ? MNormal : -MNormal;
				//OutTime = 0;
				return true;
			}

			const FVec3 DirTowardsPlane = SignedDist < 0 ? MNormal : -MNormal;
			const FReal RayProjectedTowardsPlane = FVec3::DotProduct(Dir, DirTowardsPlane);
			const FReal Epsilon = 1e-7f;
			if (RayProjectedTowardsPlane < Epsilon)	//moving parallel or away
			{
				return false;
			}

			//No initial overlap so we are outside the thickness band of the plane. So translate the plane to account for thickness	
			const FVec3 TranslatedPlaneX = (FVec3)MX - Thickness * DirTowardsPlane;
			const FVec3 StartToTranslatedPlaneX = TranslatedPlaneX - StartPoint;
			const FReal LengthTowardsPlane = FVec3::DotProduct(StartToTranslatedPlaneX, DirTowardsPlane);
			const FReal LengthAlongRay = LengthTowardsPlane / RayProjectedTowardsPlane;

			if (LengthAlongRay > Length)
			{
				return false;	//never reach
			}

			OutTime = LengthAlongRay;
			OutPosition = StartPoint + (LengthAlongRay + Thickness) * Dir;
			OutNormal = -DirTowardsPlane;
			return true;
		}

		Pair<FVec3, bool> FindClosestIntersection(const FVec3& StartPoint, const FVec3& EndPoint, const FReal Thickness) const
		{
			FVec3 Direction = EndPoint - StartPoint;
			FReal Length = Direction.Size();
			Direction = Direction.GetSafeNormal();
			FVec3 XPos = (FVec3)MX + (FVec3)MNormal * Thickness;
			FVec3 XNeg = (FVec3)MX - (FVec3)MNormal * Thickness;
			FVec3 EffectiveX = ((XNeg - StartPoint).Size() < (XPos - StartPoint).Size()) ? XNeg : XPos;
			FVec3 PlaneToStart = EffectiveX - StartPoint;
			FReal Denominator = FVec3::DotProduct(Direction, MNormal);
			if (Denominator == 0)
			{
				if (FVec3::DotProduct(PlaneToStart, MNormal) == 0)
				{
					return MakePair(EndPoint, true);
				}
				return MakePair(FVec3(0), false);
			}
			FReal Root = FVec3::DotProduct(PlaneToStart, MNormal) / Denominator;
			if (Root < 0 || Root > Length)
			{
				return MakePair(FVec3(0), false);
			}
			return MakePair(FVec3(Root * Direction + StartPoint), true);
		}

		const TVec3<T>& X() const { return MX; }
		const TVec3<T>& Normal() const { return MNormal; }
		const TVec3<T>& Normal(const TVec3<T>&) const { return MNormal; }

		FORCEINLINE void Serialize(FArchive& Ar)
		{
			Ar << MX << MNormal;
		}

		uint32 GetTypeHash() const
		{
			return HashCombine(UE::Math::GetTypeHash(MX), UE::Math::GetTypeHash(MNormal));
		}

	private:
		
		TVec3<T> MX;
		TVec3<T> MNormal;
	};

	template <typename T>
	FArchive& operator<<(FArchive& Ar, TCorePlane<T>& PlaneConcrete)
	{
		PlaneConcrete.Serialize(Ar);
		return Ar;
	}

	template <typename T, int d = 3>
	using TPlaneConcrete = TCorePlane<T, d>;
} // namespace Chaos
