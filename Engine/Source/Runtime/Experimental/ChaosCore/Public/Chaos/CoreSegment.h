// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos
{
	template<class T>
	class TCoreSegment
	{
	public:
		TCoreSegment() {}
		TCoreSegment(const TVec3<T>& X1, const TVec3<T>& X2)
			: MPoint(X1)
			, MAxis(X2 - X1)
			, MLength(MAxis.SafeNormalize())
		{ }
		TCoreSegment(const TVec3<T>& X1, const TVec3<T>& Axis, const T Length)
			: MPoint(X1)
			, MAxis(Axis)
			, MLength(Length)
		{ }

		FORCEINLINE bool IsConvex() const { return true; }

		FORCEINLINE const TVec3<T> GetCenter() const { return MPoint + (.5f * MLength * MAxis); }

		FORCEINLINE const TVec3<T>& GetX1() const { return MPoint; }

		FORCEINLINE TVec3<T> GetX2() const { return MPoint + MAxis * MLength; }

		FORCEINLINE const TVec3<T>& GetAxis() const { return MAxis; }

		FORCEINLINE T GetLength() const { return MLength; }

		TVec3<T> Support(const TVec3<T>& Direction, const T Thickness, int32& VertexIndex) const
		{
			const T Dot = TVec3<T>::DotProduct(Direction, MAxis);
			const bool bIsSecond = Dot >= 0;
			const TVec3<T> FarthestCap = bIsSecond? GetX2() : GetX1();	//orthogonal we choose either
			VertexIndex = bIsSecond ? 1 : 0;
			//We want N / ||N|| and to avoid inf
			//So we want N / ||N|| < 1 / eps => N eps < ||N||, but this is clearly true for all eps < 1 and N > 0
			T SizeSqr = Direction.SizeSquared();
			if (SizeSqr <= TNumericLimits<T>::Min())
			{
				return FarthestCap;
			}
			const TVec3<T> NormalizedDirection = Direction / FMath::Sqrt(SizeSqr);
			return FarthestCap + (NormalizedDirection * Thickness);
		}

		FORCEINLINE_DEBUGGABLE TVec3<T> SupportCore(const TVec3<T>& Direction, int32& VertexIndex) const
		{
			const T Dot = TVec3<T>::DotProduct(Direction, MAxis);
			const bool bIsSecond = Dot >= 0;
			const TVec3<T> FarthestCap = bIsSecond ? GetX2() : GetX1();	//orthogonal we choose either
			VertexIndex = bIsSecond  ? 1 : 0;
			return FarthestCap;
		}

		FORCEINLINE void Serialize(FArchive &Ar) 
		{
			Ar << MPoint << MAxis;
			
			FRealSingle LengthFloat = (FRealSingle)MLength;
			Ar << LengthFloat;
			MLength = (T)LengthFloat;
		}

		FString ToString() const
		{
			return FString::Printf(TEXT("Segment: Point: [%f, %f, %f], Axis: [%f, %f, %f], Length: %f"), MPoint.X, MPoint.Y, MPoint.Z, MAxis.X, MAxis.Y, MAxis.Z, MLength);
		}

		FORCEINLINE TAABB<T, 3> BoundingBox() const
		{
			TAABB<T,3> Box(MPoint,MPoint);
			Box.GrowToInclude(GetX2());
			return Box;
		}

	private:
		TVec3<T> MPoint;
		TVec3<T> MAxis;
		T MLength;
	};

	template<class T>
	using TSegment = TCoreSegment<T>;
} // namespace Chaos
