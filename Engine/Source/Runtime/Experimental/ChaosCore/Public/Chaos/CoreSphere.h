// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/Raycasts.h"

namespace Chaos
{
	template<class T, int d>
	class TCoreSphere
	{
	public:
		TCoreSphere() = default;
		TCoreSphere(const TCoreSphere&) = default;
		TCoreSphere(TCoreSphere&&) = default;
		TCoreSphere(const TVector<T, d>& InCenter, const T InRadius)
			: Center(InCenter)
			, Radius(FRealSingle(InRadius))
		{
		}

		TCoreSphere& operator=(const TCoreSphere&) = default;
		TCoreSphere& operator=(TCoreSphere&&) = default;

		FRealSingle GetRadius() const
		{
			return Radius;
		}

		void SetRadius(const FRealSingle InRadius)
		{
			Radius = InRadius;
		}

		const FVec3f& GetCenter() const
		{
			return Center;
		}

		void SetCenter(const FVec3f& InCenter)
		{
			Center = InCenter;
		}

		bool Raycast(const TVector<T, d>& StartPoint, const TVector<T, d>& Dir, const T Length, const T Thickness, T& OutTime, TVector<T, d>& OutPosition, TVector<T, d>& OutNormal) const
		{
			return Raycasts::RaySphere(StartPoint, Dir, Length, Thickness, Center, GetRadius(), OutTime, OutPosition, OutNormal);
		}

	private:
		FVec3f Center;
		FRealSingle Radius;
	};
	
	using FCoreSphere = TCoreSphere<FReal, 3>;
	using FCoreSpheref = TCoreSphere<FRealSingle, 3>;
} // namespace Chaos
