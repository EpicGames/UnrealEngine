// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "ChaosCheck.h"
#include "Serialization/Archive.h"

namespace Chaos
{
	template<typename ElementType, uint32 IndexWidth, uint32 GenerationWidth>
	class THandleArray;

	template<typename ElementType, uint32 IndexWidth, uint32 GenerationWidth>
	class THandleHeap;

	template<typename ElementType, uint32 IndexWidth, uint32 GenerationWidth>
	class TConstHandle;

	template<uint32 IndexWidth, uint32 GenerationWidth>
	class THandleData
	{
	public:

		static_assert((IndexWidth + GenerationWidth) / 8 == sizeof(uint32), "Sum of widths does not match data type");

		THandleData()
			: Index(0)
			, Generation(0)
		{

		}

		THandleData(uint32 InIndex, uint32 InGeneration)
			: Index(InIndex)
			, Generation(InGeneration)
		{

		}

		// Mainly for serialization of the handle container
		THandleData(uint32 FullHandle)
			: Index(FullHandle >> GenerationWidth)
			, Generation(FullHandle)
		{

		}

		bool IsValid() const
		{
			// Default construction is the invalid state
			return Generation != 0;
		}

		uint32 AsUint() const
		{
			// Shift the index up to fit the generation
			return Index << GenerationWidth | Generation;
		}

		uint32 GetIndex() const
		{
			return Index;
		}

		void FromUint(uint32 InUint)
		{
			Index = InUint >> GenerationWidth;
			constexpr uint32 GenerationMask = (1 << GenerationWidth) - 1;
			Generation = InUint & GenerationMask;
		}

		friend FArchive& operator <<(FArchive& Ar, THandleData<IndexWidth, GenerationWidth>& InHandle)
		{
			uint32 Packed = InHandle.AsUint();
			Ar << Packed;
			if(Ar.IsLoading())
			{
				InHandle.FromUint(Packed);
			}

			return Ar;
		}

	protected:
		uint32 Index : IndexWidth;
		uint32 Generation : GenerationWidth;
	};

	template<typename ElementType, uint32 IndexWidth, uint32 GenerationWidth>
	class THandle : public THandleData<IndexWidth, GenerationWidth>
	{
	public:
		friend THandleArray<ElementType, IndexWidth, GenerationWidth>;
		friend THandleHeap<ElementType, IndexWidth, GenerationWidth>;
		friend TConstHandle<ElementType, IndexWidth, GenerationWidth>;

		THandle() = default;
		THandle(uint32 InIndex, uint32 InGeneration)
			: THandleData<IndexWidth, GenerationWidth>(InIndex, InGeneration)
		{

		}

		friend bool operator ==(const THandle<ElementType, IndexWidth, GenerationWidth>& A, const THandle<ElementType, IndexWidth, GenerationWidth>& B)
		{
			return A.AsUint() == B.AsUint();
		}

		friend FArchive& operator <<(FArchive& Ar, THandle<ElementType, IndexWidth, GenerationWidth>& InHandle)
		{
			Ar << *((THandleData<IndexWidth, GenerationWidth>*)&InHandle);
			return Ar;
		}
	};

	template<typename ElementType, uint32 IndexWidth, uint32 GenerationWidth>
	class TConstHandle : public THandleData<IndexWidth, GenerationWidth>
	{
	public:
		friend THandleArray<ElementType, IndexWidth, GenerationWidth>;
		friend THandleHeap<ElementType, IndexWidth, GenerationWidth>;

		TConstHandle() = default;
		TConstHandle(uint32 InIndex, uint32 InGeneration)
			: THandleData<IndexWidth, GenerationWidth>(InIndex, InGeneration)
		{

		}

		TConstHandle(const THandle<ElementType, IndexWidth, GenerationWidth>& InNonConstHandle)
			: THandleData<IndexWidth, GenerationWidth>(InNonConstHandle.Index, InNonConstHandle.Generation)
		{

		}

		friend bool operator ==(const TConstHandle<ElementType, IndexWidth, GenerationWidth>& A, const TConstHandle<ElementType, IndexWidth, GenerationWidth>& B)
		{
			return A.AsUint() == B.AsUint();
		}

		friend FArchive& operator <<(FArchive& Ar, TConstHandle<ElementType, IndexWidth, GenerationWidth>& InHandle)
		{
			Ar << *((THandleData<IndexWidth, GenerationWidth>*)&InHandle);
			return Ar;
		}
	};

	template<typename ElementType, uint32 IndexWidth = 24, uint32 GenerationWidth = 8>
	class THandleArray
	{
	public:

		using FHandle = THandle<ElementType, IndexWidth, GenerationWidth>;
		using FConstHandle = TConstHandle<ElementType, IndexWidth, GenerationWidth>;
		
		static constexpr uint32 ElementAlign = alignof(ElementType);
		static constexpr SIZE_T ElementSize = sizeof(ElementType);
		static constexpr uint32 ElementGrowth = 4;

		static constexpr uint32 InvalidFreeIndex = TNumericLimits<uint32>::Max() >> GenerationWidth;

		THandleArray(const THandleArray<ElementType, IndexWidth, GenerationWidth>& Other)
			: THandleArray(0)
		{
			CopyFrom(Other);
		}

		THandleArray(THandleArray<ElementType, IndexWidth, GenerationWidth>&& Other)
			: THandleArray(0)
		{
			MoveFrom(Other);
		}

		THandleArray& operator =(const THandleArray<ElementType, IndexWidth, GenerationWidth>& Other)
		{
			CopyFrom(Other);
			return *this;
		}

		THandleArray& operator =(THandleArray<ElementType, IndexWidth, GenerationWidth>&& Other)
		{
			MoveFrom(Other);
			return *this;
		}

		THandleArray()
			: THandleArray(0)
		{}

		explicit THandleArray(int32 InitialNum)
			: FreeList(InvalidFreeIndex)
			, Data(nullptr)
			, NumData(0)
			, Capacity(0)
			, NumActive(0)
		{
			if(InitialNum > 0)
			{
				Data = (ElementType*)FMemory::Malloc(InitialNum * ElementSize, ElementAlign);
				NumData = Capacity = InitialNum;

				// Set up handle entries
				HandleEntries.AddDefaulted(NumData);
				for(int32 Index = 1; Index < NumData; ++Index)
				{
					HandleEntries[Index - 1].NextFree = Index;
				}
				FreeList = 0;

				Validity.Add(false, NumData);
			}
		}

		~THandleArray()
		{
			DestroyItems();
		}

		ElementType* Get(FHandle InHandle) const
		{
			if(InHandle.IsValid())
			{
				if(Validity.IsValidIndex(InHandle.Index))
				{
					if(Validity[InHandle.Index])
					{
						const FHandleEntry& Entry = HandleEntries[InHandle.Index];
						return Entry.Generation == InHandle.Generation ? &(Data[InHandle.Index]) : nullptr;
					}
				}
				else
				{
					CHAOS_ENSURE_MSG(false, TEXT("Failed to access handle (%u, %u). NumEntries = %d, NumFlags = %d"), InHandle.Index, InHandle.Generation, HandleEntries.Num(), Validity.Num());
				}
			}
			return nullptr;
		}

		const ElementType* Get(FConstHandle InHandle) const
		{
			if (InHandle.IsValid())
			{
				if (Validity.IsValidIndex(InHandle.Index))
				{
					if (Validity[InHandle.Index])
					{
						const FHandleEntry& Entry = HandleEntries[InHandle.Index];
						return Entry.Generation == InHandle.Generation ? &(Data[InHandle.Index]) : nullptr;
					}
				}
				else
				{
					ensureMsgf(false, TEXT("Failed to access handle (%u, %u). NumEntries = %d, NumFlags = %d"), InHandle.Index, InHandle.Generation, HandleEntries.Num(), Validity.Num());
				}
			}
			return nullptr;
		}

		FHandle GetHandle(uint32 InIndex) const
		{
			// Only return a handle if we have an entry and it hasn't been retired
			if(HandleEntries.IsValidIndex(InIndex) && HandleEntries[InIndex].Generation != 0)
			{
				return FHandle(InIndex, HandleEntries[InIndex].Generation);
			}

			return FHandle();
		}

		FConstHandle GetConstHandle(uint32 InIndex) const
		{
			return FConstHandle(GetHandle(InIndex));
		}

		template<typename... ConstructionArgs>
		FHandle Create(ConstructionArgs&&... InConstructionArgs)
		{
			// Look for free slots
			int32 NewIndex = INDEX_NONE;
			if(FreeList != InvalidFreeIndex)
			{
				NewIndex = FreeList;
				FHandleEntry& ReusedEntry = HandleEntries[FreeList];
				Validity[FreeList] = true;
				FreeList = ReusedEntry.NextFree;
				ReusedEntry.NextFree = InvalidFreeIndex;
			}
			else
			{
				// If nothing in the freelist then proceed to adding new elements
				if(NumData == Capacity)
				{
					// Realloc
					Data = (ElementType*)FMemory::Realloc(Data, ElementSize * (Capacity + ElementGrowth), ElementAlign);
					Capacity += ElementGrowth;
				}

				NewIndex = NumData;
				HandleEntries.AddDefaulted();
				Validity.Add(true);
				++NumData;
			}

			new (Data + NewIndex) ElementType(Forward<ConstructionArgs>(InConstructionArgs)...);

			NumActive++;
			return GetHandle(NewIndex);
		}

		void Destroy(FHandle InHandle)
		{
			if(InHandle.IsValid())
			{
				// Get the entry
				FHandleEntry& Entry = HandleEntries[InHandle.Index];

				if(Entry.Generation != InHandle.Generation)
				{
					// Invalid handle no longer points to this resource
					ensureMsgf(false, TEXT("Attempting to destroy with an invalid handle."));
					return;
				}

				// Invalidate it
				Entry.Generation++;

				// Destruct the element and set the valid flag
				DestructItem(Data + InHandle.Index);
				Validity[InHandle.Index] = false;
				NumActive--;

				if(Entry.Generation == 0)
				{
					// If we overflow we have to retire this array slot otherwise there could technically
					// be collisions with old handles.
					return;
				}

				// If we should reuse this slot, handle the freelist
				if(FreeList != InvalidFreeIndex)
				{
					HandleEntries[InHandle.Index].NextFree = FreeList;
					FreeList = InHandle.Index;
				}
				else
				{
					FreeList = InHandle.Index;
				}
			}
		}

		uint32 Num() const
		{
			return NumData;
		}

		uint32 GetCapacity() const
		{
			return Capacity;
		}

		uint32 GetNumActive() const
		{
			return NumActive;
		}

		uint32 GetNumFree() const
		{
			return Num() - NumActive;
		}

		friend FArchive& operator <<(FArchive& Ar, THandleArray& Array)
		{
			Ar << Array.FreeList << Array.HandleEntries << Array.Validity << Array.NumData << Array.Capacity << Array.NumActive;

			// Need to serialize per element as T.<< could serialize out-of-order
			if(Ar.IsLoading())
			{
				Array.Data = (ElementType*)FMemory::Realloc(Array.Data, Array.Capacity * ElementSize, ElementAlign);
				for(int32 Index = 0; Index < Array.NumData; ++Index)
				{
					Array.Data[Index] = ElementType();
					Ar << Array.Data[Index];
				}
			}
			else
			{
				for(int32 Index = 0; Index < Array.NumData; ++Index)
				{
					Ar << Array.Data[Index];
				}
			}

			return Ar;
		}

		template<bool bConst>
		struct TIterator
		{
			typedef std::conditional_t<bConst, const THandleArray, THandleArray> ArrayType;
			typedef std::conditional_t<bConst, const ElementType, ElementType> ItElementType;

			explicit TIterator(ArrayType& InArray, int32 InIndex)
				: Array(InArray)
				, It(InArray.Validity, InIndex)
			{
			}

			FORCEINLINE TIterator& operator++()
			{
				// Iterate to the next set allocation flag.
				++It;
				return *this;
			}

			[[nodiscard]] FORCEINLINE bool operator==(const TIterator& Rhs) const
			{
				return It == Rhs.It && &Array == &Rhs.Array;
			}

			[[nodiscard]] FORCEINLINE bool operator!=(const TIterator& Rhs) const
			{
				return It != Rhs.It || &Array != &Rhs.Array;
			}

			FConstHandle GetConstHandle() const
			{
				const int32 Index = It.GetIndex();
				return Array.GetConstHandle(Index);
			}

			FHandle GetHandle() const
			{
				const int32 Index = It.GetIndex();
				return Array.GetHandle(Index);
			}

			[[nodiscard]] FORCEINLINE ItElementType& operator*() const
			{
				const FHandle Handle = GetHandle();
				ItElementType* Element = Array.Get(Handle);
				return *Element;
			}

			[[nodiscard]] FORCEINLINE ItElementType* operator->() const
			{
				const FHandle Handle = GetHandle();
				return Array.Get(Handle);
			}

			ArrayType& Array;
			TConstSetBitIterator<> It;
		};

		using TRangedForIterator = TIterator<false>;
		using TRangedForConstIterator = TIterator<true>;

		TRangedForIterator begin()
		{
			return TRangedForIterator(*this, 0);
		}

		TRangedForConstIterator begin() const
		{
			return TRangedForConstIterator(*this, 0);
		}

		TRangedForIterator end()
		{
			return TRangedForIterator(*this, (int32)Num());
		}

		TRangedForConstIterator end() const
		{
			return TRangedForConstIterator(*this, (int32)Num());
		}

	private:

		void DestroyItems()
		{
			if(Data)
			{
				for(int32 Index = 0; Index < NumData; ++Index)
				{
					if(Validity[Index])
					{
						DestructItem(Data + Index);
					}
				}

				FMemory::Free(Data);
			}

			FreeList = InvalidFreeIndex;
			Capacity = 0;
			NumData = 0;
			NumActive = 0;
			Data = nullptr;
			Validity.Reset();
			HandleEntries.Reset();
		}

		void CopyFrom(const THandleArray& Other)
		{
			DestroyItems();

			Capacity = Other.Capacity;
			HandleEntries = Other.HandleEntries;
			Validity = Other.Validity;
			FreeList = Other.FreeList;

			if(Capacity > 0)
			{
				Data = (ElementType*)FMemory::Malloc(Capacity * ElementSize, ElementAlign);

				NumData = Other.NumData;
				NumActive = Other.NumActive;

				for(int32 Index = 0; Index < NumData; ++Index)
				{
					if(Validity[Index])
					{
						ConstructItems<ElementType>(Data + Index, Other.Data + Index, 1);
					}
				}
			}
		}

		void MoveFrom(THandleArray& Other)
		{
			DestroyItems();

			Capacity = Other.Capacity;

			if(Capacity > 0)
			{
				Data = MoveTemp(Other.Data);
				
				NumData = Other.NumData;
				NumActive = Other.NumActive;
			}

			HandleEntries = MoveTemp(Other.HandleEntries);
			Validity = MoveTemp(Other.Validity);

			FreeList = Other.FreeList;
			
			// This container now owns the allocation from Other
			Other.FreeList = InvalidFreeIndex;
			Other.Capacity = 0;
			Other.NumData = 0;
			Other.NumActive = 0;
			Other.Data = nullptr;
		}

		class FHandleEntry
		{
		public:

			FHandleEntry()
				: NextFree(InvalidFreeIndex)
				, Generation(1)
			{

			}

			uint32 NextFree : IndexWidth;
			uint32 Generation : GenerationWidth;

			friend FArchive& operator << (FArchive& Ar, FHandleEntry& InHandleEntry)
			{
				uint32 Packed = InHandleEntry.NextFree << GenerationWidth | InHandleEntry.Generation;
				Ar << Packed;

				if(Ar.IsLoading())
				{
					InHandleEntry.NextFree = Packed >> GenerationWidth;
					InHandleEntry.Generation = Packed & ((1 << IndexWidth) - 1);
				}

				return Ar;
			}
		};
		
		uint32 FreeList;
		TArray<FHandleEntry> HandleEntries;
		TBitArray<> Validity;
		ElementType* Data;
		int32 NumData;
		int32 Capacity;
		int32 NumActive;
	};

	template<typename ElementType, uint32 IndexWidth = 24, uint32 GenerationWidth = 8>
	class THandleHeap
	{
	public:

		static_assert(TIsTrivial<ElementType>::Value, "Handle managed data must be trivial");

		using FHandle = THandle<ElementType, IndexWidth, GenerationWidth>;
		using FConstHandle = TConstHandle < ElementType, IndexWidth, GenerationWidth>;

		static constexpr uint32 InvalidFreeIndex = TNumericLimits<uint32>::Max() >> GenerationWidth;

		explicit THandleHeap(int32 ReserveNum)
			: FreeList(InvalidFreeIndex)
			, NumActive(0)
		{
			HandleEntries.Reserve(ReserveNum);
		}

		THandleHeap(const THandleHeap<ElementType, IndexWidth, GenerationWidth>& Other)
			: THandleHeap(0)
		{
			CopyFrom(Other);
		}

		THandleHeap(THandleHeap<ElementType, IndexWidth, GenerationWidth>&& Other)
		{
			MoveFrom(Other);
		}

		THandleHeap& operator =(const THandleHeap<ElementType, IndexWidth, GenerationWidth>& Other)
		{
			CopyFrom(Other);
			return *this;
		}

		THandleHeap& operator =(THandleHeap<ElementType, IndexWidth, GenerationWidth>&& Other)
		{
			MoveFrom(Other);
			return *this;
		}

		THandleHeap()
			: THandleHeap(0)
		{}

		~THandleHeap()
		{
			Empty();
		}

		ElementType* Get(FHandle InHandle)
		{
			FHandleEntry& Entry = HandleEntries[InHandle.Index];
			return (Entry.Generation == InHandle.Generation && Validity[InHandle.Index]) ? Entry.Ptr : nullptr;
		}

		const ElementType* Get(FConstHandle InHandle) const
		{
			const FHandleEntry& Entry = HandleEntries[InHandle.Index];
			return (Entry.Generation == InHandle.Generation && Validity[InHandle.Index]) ? Entry.Ptr : nullptr;
		}

		FHandle GetHandle(uint32 InIndex)
		{
			// Only return a handle if we have an entry and it hasn't been retired
			if(HandleEntries.IsValidIndex(InIndex) && HandleEntries[InIndex].Generation != 0)
			{
				return FHandle(InIndex, HandleEntries[InIndex].Generation);
			}

			return FHandle();
		}

		FConstHandle GetConstHandle(uint32 InIndex)
		{
			return FConstHandle(GetHandle(InIndex));
		}

		template<typename... ConstructionArgs>
		FHandle Create(ConstructionArgs... Args)
		{
			int32 NewIndex = INDEX_NONE;
			if(FreeList != InvalidFreeIndex)
			{
				NewIndex = FreeList;
				FHandleEntry& ReusedEntry = HandleEntries[FreeList];
				ReusedEntry.Ptr = new ElementType(Forward<ConstructionArgs>(Args)...);
				Validity[FreeList] = true;
				FreeList = ReusedEntry.NextFree;
				ReusedEntry.NextFree = InvalidFreeIndex;
			}
			else
			{
				HandleEntries.AddDefaulted();
				Validity.Add(true);

				FHandleEntry& NewEntry = HandleEntries.Last();
				NewEntry.Ptr = new ElementType(Forward<ConstructionArgs>(Args)...);
				NewIndex = HandleEntries.Num() - 1;
			}

			NumActive++;
			return GetHandle(NewIndex);
		}

		void Destroy(FHandle InHandle)
		{
			if(InHandle.IsValid())
			{
				FHandleEntry& Entry = HandleEntries[InHandle.Index];

				if(Entry.Generation != InHandle.Generation)
				{
					// Bail if this handle is out of date - it doesn't point to this resource
					ensureMsgf(false, TEXT("Attempting to destroy with an invalid handle."));
					return;
				}

				// Invalidate
				Entry.Generation++;
				Validity[InHandle.Index] = false;

				if(Entry.Ptr)
				{
					// If the state of this container has been moved to another then
					// this ptr will be nullptr
					delete Entry.Ptr;
					Entry.Ptr = nullptr;
					NumActive--;
				}

				if(Entry.Generation == 0)
				{
					// If we overflow we have to retire this array slot otherwise there could technically
					// be collisions with old handles.
					return;
				}

				// If we should reuse this slot, handle the freelist
				if(FreeList != InvalidFreeIndex)
				{
					HandleEntries[InHandle.Index].NextFree = FreeList;
					FreeList = InHandle.Index;
				}
				else
				{
					FreeList = InHandle.Index;
				}
			}
		}

		int32 Num() const
		{
			return HandleEntries.Num();
		}

		int32 GetNumActive() const
		{
			return NumActive;
		}

		void Empty(bool bShrink = true)
		{
			// Destruct all the elements (every existing entry will be added to the freelist)
			const int32 NumToRemove = Num();
			for(int32 Index = 0; Index < NumToRemove; ++Index)
			{
				Destroy(GetHandle(Index));
			}

			HandleEntries.Empty(bShrink ? 0 : HandleEntries.GetAllocatedSize());
			Validity.Empty();
			FreeList = InvalidFreeIndex;
		}

		friend FArchive& operator <<(FArchive& Ar, THandleHeap& Array)
		{
			Ar << Array.FreeList << Array.HandleEntries << Array.Validity << Array.NumActive;

			// Need to serialize per element as T.<< could serialize out-of-order
			const int32 NumEntries = Array.HandleEntries.Num();
			if(Ar.IsLoading())
			{
				for(int32 Index = 0; Index < NumEntries; ++Index)
				{
					if(!Array.Validity[Index])
					{
						continue;
					}

					FHandleEntry& Entry = Array.HandleEntries[Index];
					Entry.Ptr = new ElementType();
					Ar << *Entry.Ptr;
				}
			}
			else
			{
				for(int32 Index = 0; Index < NumEntries; ++Index)
				{
					if(!Array.Validity[Index])
					{
						continue;
					}

					FHandleEntry& Entry = Array.HandleEntries[Index];
					Ar << *Entry.Ptr;
				}
			}

			return Ar;
		}

	private:

		void CopyFrom(const THandleHeap<ElementType, IndexWidth, GenerationWidth>& Other)
		{
			Empty();

			FreeList = Other.FreeList;
			NumActive = Other.NumActive;
			Validity = Other.Validity;

			HandleEntries.Reserve(Other.Num());
			for(const FHandleEntry& Entry : Other.HandleEntries)
			{
				HandleEntries.AddDefaulted();
				FHandleEntry& NewEntry = HandleEntries.Last();
				NewEntry.NextFree = Entry.NextFree;
				NewEntry.Generation = Entry.Generation;
				NewEntry.Ptr = Entry.Ptr ? new ElementType(*Entry.Ptr) : nullptr;
			}
		}

		void MoveFrom(THandleHeap<ElementType, IndexWidth, GenerationWidth>& Other)
		{
			Empty();

			FreeList = Other.FreeList;
			NumActive = Other.NumActive;

			Validity = MoveTemp(Other.Validity);
			HandleEntries = MoveTemp(Other.HandleEntries);
			
			// Clear out all the handle entries in the other container.
			Other.Empty();
			Other.NumActive = 0;
		}

		class FHandleEntry
		{
		public:

			FHandleEntry()
				: NextFree(InvalidFreeIndex)
				, Generation(1)
				, Ptr(nullptr)
			{

			}

			uint32 NextFree : IndexWidth;
			uint32 Generation : GenerationWidth;
			ElementType* Ptr;

			friend FArchive& operator << (FArchive& Ar, FHandleEntry& InHandleEntry)
			{
				uint32 Packed = InHandleEntry.NextFree << GenerationWidth | InHandleEntry.Generation;
				Ar << Packed;

				if(Ar.IsLoading())
				{
					InHandleEntry.NextFree = Packed >> GenerationWidth;
					InHandleEntry.Generation = Packed & ((1 << IndexWidth) - 1);
				}

				return Ar;
			}
		};

		uint32 FreeList;
		TArray<FHandleEntry> HandleEntries;
		TBitArray<> Validity;
		int32 NumActive;
	};
}
