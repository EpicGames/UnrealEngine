// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Bad include VectorTypes (living in GeometryCore)

#include "Chaos/Matrix.h"
#include "Chaos/Vector.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "Math/NumericLimits.h"


namespace Chaos {
	
	/**
		Class for givens rotation.
		Row rotation G*A corresponds to something like
		c -s  0
		( s  c  0 ) A
		0  0  1
		Column rotation A G' corresponds to something like
		c -s  0
		A ( s  c  0 )
		0  0  1

		c and s are always Computed so that
		( c -s ) ( a )  =  ( * )
		s  c     b       ( 0 )

		Assume rowi<rowk.
		*/
	template <class T>
	class GivensRotation {
	public:
		int rowi;
		int rowk;
		T c;
		T s;

		inline GivensRotation() : rowi(0), rowk(0), c((T)0.), s((T)0.) {}

		inline GivensRotation(int rowi_in, int rowk_in) : rowi(rowi_in), rowk(rowk_in), c(1), s(0) {}

		inline GivensRotation(T a, T b, int rowi_in, int rowk_in) : rowi(rowi_in), rowk(rowk_in) {
			Compute(a, b);
		}

		~GivensRotation() = default;

		inline void TransposeInPlace() {
			s = -s;
		}

		/**
			Compute c and s from a and b so that
			( c -s ) ( a )  =  ( * )
			s  c     b       ( 0 )
			*/
		inline void Compute(const T a, const T b) {

			T d = a * a + b * b;
			c = 1;
			s = 0;
			if (d != 0) {
				T t = T(1)/FMath::Sqrt(d);
				c = a * t;
				s = -b * t;
			}
		}

		/**
			This function Computes c and s so that
			( c -s ) ( a )  =  ( 0 )
			s  c     b       ( * )
			*/
		inline void ComputeUnconventional(const T a, const T b) {
			T d = a * a + b * b;
			c = 0;
			s = 1;
			if (d != 0) {
				T t = T(1)/FMath::Sqrt(d);
				s = a * t;
				c = b * t;
			}
		}
		/**
		  Fill the R with the entries of this rotation
			*/
		//template <class T>
		inline void Fill(const PMatrix<T, 2, 2>& R) const {
			PMatrix<T, 2, 2>& A = const_cast<PMatrix<T, 2, 2>&>(R);
			A = PMatrix<T, 2, 2>((FReal)1., (FReal)0., (FReal)0., (FReal)1.);
			A.M[rowi*2+rowi] = c;
			A.M[rowi * 2 + rowk] = -s;
			A.M[rowk * 2 + rowi] = s;
			A.M[rowk * 2 + rowk] = c;
		}

		/**
			This function does something like
			c -s  0
			( s  c  0 ) A -> A
			0  0  1
			It only affects row i and row k of A.
			*/
		//template <class T>
		inline void RowRotation(PMatrix<T, 2, 2>& A) const {
			for (int j = 0; j < 2; j++) {
				T tau1 = A.M[j*2+rowi];
				T tau2 = A.M[j*2+rowk];
				A.M[j*2+rowi] = c * tau1 - s * tau2;
				A.M[j*2+rowk] = s * tau1 + c * tau2;
			}
		}

		//template <class T>
		inline void RowRotation(PMatrix<T, 3, 3>& A) const {
			for (int j = 0; j < 3; j++) {
				T tau1 = A.M[j][rowi];
				T tau2 = A.M[j][rowk];
				A.M[j][rowi] = c * tau1 - s * tau2;
				A.M[j][rowk] = s * tau1 + c * tau2;
			}
		}

		/**
			This function does something like
			c  s  0
			A ( -s  c  0 )  -> A
			0  0  1
			It only affects column i and column k of A.
			*/
		//template <class T>
		inline void ColumnRotation(PMatrix<T, 2, 2>& A) const {
			for (int j = 0; j < 2; j++) {
				T tau1 = A.M[rowi*2+j];
				T tau2 = A.M[rowk*2+j];
				A.M[rowi*2+j] = c * tau1 - s * tau2;
				A.M[rowk*2+j]= s * tau1 + c * tau2;
			}
		}

		//template <class T>
		inline void ColumnRotation(PMatrix<T, 3, 3>& A) const {
			for (int j = 0; j < 3; j++) {
				T tau1 = A.M[rowi][j];
				T tau2 = A.M[rowk][j];
				A.M[rowi][j] = c * tau1 - s * tau2;
				A.M[rowk][j] = s * tau1 + c * tau2;
			}
		}

		/**
		  Multiply givens must be for same row and column
		  **/
		inline void operator*=(const GivensRotation<T>& A) {
			T new_c = c * A.c - s * A.s;
			T new_s = s * A.c + c * A.s;
			c = new_c;
			s = new_s;
		}

		/**
		  Multiply givens must be for same row and column
		  **/
		inline GivensRotation<T> operator*(const GivensRotation<T>& A) const {
			GivensRotation<T> r(*this);
			r *= A;
			return r;
		}
	};

	/**
		\brief zero chasing the 3X3 matrix to bidiagonal form
		original form of H:   x x 0
		x x x
		0 0 x
		after zero chase:
		x x 0
		0 x x
		0 0 x
		*/
	template <class T>
	inline void ZeroChase(PMatrix<T, 3, 3>& H, PMatrix<T, 3, 3>& U, PMatrix<T, 3, 3>& V) {
		/**
			Reduce H to of form
			x x +
			0 x x
			0 0 x
			*/
		GivensRotation<T> r1(H.M[0][0], H.M[0][1], 0, 1);
		/**
			Reduce H to of form
			x x 0
			0 x x
			0 + x
			Can calculate r2 without multiplying by r1 since both entries are in first two
			rows thus no need to divide by sqrt(a^2+b^2)
			*/
		GivensRotation<T> r2(1, 2);
		if (H.M[0][1] != (T)0.)
			r2.Compute(H.M[0][0] * H.M[1][0] + H.M[0][1] * H.M[1][1], H.M[0][0] * H.M[2][0] + H.M[0][1] * H.M[2][1]);
		else
			r2.Compute(H.M[1][0], H.M[2][0]);

		r1.RowRotation(H);

		r2.ColumnRotation(H);
		r2.ColumnRotation(V);

		/**
			Reduce H to of form
			x x 0
			0 x x
			0 0 x
			*/
		GivensRotation<T> r3(H.M[1][1], H.M[1][2], 1, 2);
		r3.RowRotation(H);


		r1.ColumnRotation(U);
		r3.ColumnRotation(U);
	}

	/**
		 \brief make a 3X3 matrix to upper bidiagonal form
		 original form of H:   x x x
							   x x x
							   x x x
		 after zero chase:
							   x x 0
							   0 x x
							   0 0 x
	  */
	template <class T>
	inline void MakeUpperBidiag(PMatrix<T, 3, 3>& H, PMatrix<T, 3, 3>& U, PMatrix<T, 3, 3>& V) {
		U = PMatrix<T, 3, 3>(1, 1, 1);
		V = PMatrix<T, 3, 3>(1, 1, 1);

		/**
		  Reduce H to of form
							  x x x
							  x x x
							  0 x x
		*/

		GivensRotation<T> r(H.M[0][1], H.M[0][2], 1, 2);
		r.RowRotation(H);
		r.ColumnRotation(U);
		ZeroChase(H, U, V);
	}

	/**
		 \brief make a 3X3 matrix to lambda shape
		 original form of H:   x x x
		 *                     x x x
		 *                     x x x
		 after :
		 *                     x 0 0
		 *                     x x 0
		 *                     x 0 x
	  */
	template <class T>
	inline void MakeLambdaShape(PMatrix<T, 3, 3>& H, PMatrix<T, 3, 3>& U, PMatrix<T, 3, 3>& V) {
		U = PMatrix<T, 3, 3>(1, 1, 1);
		V = PMatrix<T, 3, 3>(1, 1, 1);

		/**
		  Reduce H to of form
		  *                    x x 0
		  *                    x x x
		  *                    x x x
		  */

		GivensRotation<T> r1(H.M[1][0], H.M[2][0], 1, 2);
		r1.ColumnRotation(H);
		r1.ColumnRotation(V);

		/**
		  Reduce H to of form
		  *                    x x 0
		  *                    x x 0
		  *                    x x x
		  */

		r1.ComputeUnconventional(H.M[2][1], H.M[2][2]);
		r1.RowRotation(H);
		r1.ColumnRotation(U);

		/**
		  Reduce H to of form
		  *                    x x 0
		  *                    x x 0
		  *                    x 0 x
		  */

		GivensRotation<T> r2(H.M[0][2], H.M[1][2], 0, 1);
		r2.ColumnRotation(H);
		r2.ColumnRotation(V);

		/**
		  Reduce H to of form
		  *                    x 0 0
		  *                    x x 0
		  *                    x 0 x
		  */
		r2.ComputeUnconventional(H.M[1][0], H.M[1][1]);
		r2.RowRotation(H);
		r2.ColumnRotation(U);
	}

	/**
	   \brief 2x2 polar decomposition.
	   \param[in] A matrix.
	   \param[out] R Robustly a rotation matrix in givens form
	   \param[out] S_Sym Symmetric. Whole matrix is stored

	   Whole matrix S is stored since its faster to calculate due to simd vectorization
	   Polar guarantees negative sign is on the small magnitude singular value.
	   S is guaranteed to be the closest one to identity.
	   R is guaranteed to be the closest rotation to A.
	*/
	template <class T>
	inline void PolarDecomposition(const PMatrix<T, 2, 2>& A, GivensRotation<T>& R, PMatrix<T, 2, 2>& S_Sym) {
		PMatrix<T, 2, 2>& A_copy = const_cast<PMatrix<T, 2, 2>&>(A);
		TVector<T, 2> x(A_copy.M[0*2+0] + A_copy.M[1*2+1], A_copy.M[0*2+1] - A_copy.M[1*2+0]);
		T denominator = x.Size();
		R.c = (T)1.;
		R.s = (T)0.;
		if (denominator != 0) {
			/*
			  No need to use a tolerance here because x(0) and x(1) always have
			  smaller magnitude then denominator, therefore overflow never happens.
			*/
			R.c = x[0] / denominator;
			R.s = -x[1] / denominator;
		}
		S_Sym = A;
		R.RowRotation(S_Sym);
	}

	/**
	   \brief 2x2 polar decomposition.
	   \param[in] A matrix.
	   \param[out] R Robustly a rotation matrix.
	   \param[out] S_Sym Symmetric. Whole matrix is stored

	   Whole matrix S is stored since its faster to calculate due to simd vectorization
	   Polar guarantees negative sign is on the small magnitude singular value.
	   S is guaranteed to be the closest one to identity.
	   R is guaranteed to be the closest rotation to A.
	*/
	template <class T>
	inline void PolarDecomposition(const PMatrix<T, 2, 2>& A, PMatrix<T, 2, 2>& R, PMatrix<T, 2, 2>& S_Sym) {
		GivensRotation<T> r(0, 1);
		PolarDecomposition(A, r, S_Sym);
		r.Fill(R);
	}

	/**
	   \brief 2x2 SVD (singular value decomposition) A=USV'
	   \param[in] A Input matrix.
	   \param[out] U Robustly a rotation matrix in Givens form
	   \param[out] Sigma Vector of singular values sorted with decreasing magnitude. The second one can be negative.
	   \param[out] V Robustly a rotation matrix in Givens form
	*/
	template < class T>
	inline void SingularValueDecomposition(const PMatrix<T, 2, 2>& A, GivensRotation<T>& U, const TVector<T, 2>& Sigma, GivensRotation<T>& V,
		const T tol = std::numeric_limits<T>::epsilon()) {

		TVector<T, 2>& sigma = const_cast<TVector<T, 2>&>(Sigma);

		PMatrix<T, 2, 2> S_Sym((T)0., (T)0., (T)0.);
		PolarDecomposition(A, U, S_Sym);
		T cosine, sine;
		T x = S_Sym.M[0*2+0];
		T y = S_Sym.M[1*2+0];
		T z = S_Sym.M[3];
		if (FGenericPlatformMath::Abs(y) < tol) {
			// S is already diagonal
			cosine = (T)1.;
			sine = (T)0.;
			sigma[0] = x;
			sigma[1] = z;
		}
		else {
			T tau = (T)0.5 * (x - z);
			T w = FMath::Sqrt(tau * tau + y * y);
			// w > y > 0
			T t;
			if (tau > 0) {
				// tau + w > w > y > 0 ==> division is safe
				t = y / (tau + w);
			}
			else {
				// tau - w < -w < -y < 0 ==> division is safe
				t = y / (tau - w);
			}
			cosine = (T)1. / FMath::Sqrt(t * t + (T)1.);
			sine = -t * cosine;
			/*
			  V = [cosine -sine; sine cosine]
			  Sigma = V'SV. Only Compute the diagonals for efficiency.
			  Also utilize symmetry of S and don't form V yet.
			*/
			T c2 = cosine * cosine;
			T csy = 2 * cosine * sine * y;
			T s2 = sine * sine;
			sigma[0] = c2 * x - csy + s2 * z;
			sigma[1] = s2 * x + csy + c2 * z;
		}

		// Sorting
		// Polar already guarantees negative sign is on the small magnitude singular value.
		if (sigma[0] < sigma[1]) {
			Swap(sigma[0], sigma[1]);
			V.c = -sine;
			V.s = cosine;
		}
		else {
			V.c = cosine;
			V.s = sine;
		}
		U *= V;
	}
	/**
	   \brief 2x2 SVD (singular value decomposition) A=USV'
	   \param[in] A Input matrix.
	   \param[out] U Robustly a rotation matrix.
	   \param[out] Sigma Vector of singular values sorted with decreasing magnitude. The second one can be negative.
	   \param[out] V Robustly a rotation matrix.
	*/
	template <class T>
	inline void SingularValueDecomposition(
		const PMatrix<T, 2, 2>& A, const PMatrix<T, 2, 2>& U, const TVector<T, 2>& Sigma, const PMatrix<T, 2, 2>& V,
		const T tol = std::numeric_limits<T>::epsilon()) {
		//using T = ScalarType<TA>;
		GivensRotation<T> gv(0, 1);
		GivensRotation<T> gu(0, 1);
		SingularValueDecomposition(A, gu, Sigma, gv);

		gu.Fill(U);
		gv.Fill(V);
	}

	/**
	  \brief Compute WilkinsonShift of the block
	  a1     b1
	  b1     a2
	  based on the WilkinsonShift formula
	  mu = c + d - sign (d) \ sqrt (d*d + b*b), where d = (a-c)/2

	  */
	template <class T>
	T WilkinsonShift(const T a1, const T b1, const T a2) {

		T d = (T)0.5 * (a1 - a2);
		T bs = b1 * b1;

		//T mu = a2 - copysign(bs / (FGenericPlatformMath::Abs(d) + FMath::Sqrt(d * d + bs)), d);
		T mu = a2 - (T)FMath::Sign(d) * bs / ((FGenericPlatformMath::Abs(d) + FMath::Sqrt(d * d + bs)));
		return mu;
	}

	/**
	  \brief Helper function of 3X3 SVD for Processing 2X2 SVD
	  */
	template <int t, class T>
	inline void Process(PMatrix<T, 3, 3>& B, PMatrix<T, 3, 3>& U, TVector<T, 3>& sigma, PMatrix<T, 3, 3>& V) {
		int other = (t == 1) ? 0 : 2;
		GivensRotation<T> u(0, 1);
		GivensRotation<T> v(0, 1);
		sigma[other] = B.M[other][other];
		const PMatrix<T, 2, 2> B_sub(B.M[t][t], B.M[t][t+1], B.M[t+1][t], B.M[t+1][t+1]);
		const TVector<T, 2> sigma_sub(sigma[t], sigma[t + 1]);
		SingularValueDecomposition(B_sub, u, sigma_sub, v);
		sigma[t] = sigma_sub[0];
		sigma[t + 1] = sigma_sub[1];
		B.M[t][t] = B_sub.M[0];
		B.M[t][t + 1] = B_sub.M[1];
		B.M[t + 1][t] = B_sub.M[2];
		B.M[t + 1][t + 1] = B_sub.M[3];
		u.rowi += t;
		u.rowk += t;
		v.rowi += t;
		v.rowk += t;
		u.ColumnRotation(U);
		v.ColumnRotation(V);
	}

	/**
	  \brief Helper function of 3X3 SVD for flipping signs due to flipping signs of sigma
	  */
	template <class T>
	inline void FlipSign(int i, PMatrix<T, 3, 3>& U, TVector<T, 3>& sigma) {
		sigma[i] = -sigma[i];
		U.SetColumn(i, -U.GetColumn(i));
	}

	//IMPORTANT: test this function before testing polar:
	template <class T>
	inline void SwapCols(PMatrix<T, 3, 3>& A, const int i1, const int i2) {
		auto OtherCol = A.GetColumn(i1);
		A.SetColumn(i1, A.GetColumn(i2));
		A.SetColumn(i2, OtherCol);
	}

	
	/**
	  \brief Helper function of 3X3 SVD for sorting singular values
	  */
	template <class T>
	void Sort0(PMatrix<T, 3, 3>& U, TVector<T, 3>& sigma, PMatrix<T, 3, 3>& V) {

		// Case: sigma(0) > |sigma(1)| >= |sigma(2)|
		if (FGenericPlatformMath::Abs(sigma[1]) >= FGenericPlatformMath::Abs(sigma[2])) {
			if (sigma[1] < (T)0.) {
				FlipSign(1, U, sigma);
				FlipSign(2, U, sigma);
			}
			return;
		}

		// fix sign of sigma for both cases
		if (sigma[2] < (T)0.) {
			FlipSign(1, U, sigma);
			FlipSign(2, U, sigma);
		}

		// swap sigma(1) and sigma(2) for both cases
		Swap(sigma[1], sigma[2]);
		SwapCols(U, 1, 2);
		SwapCols(V, 1, 2);

		// Case: |sigma(2)| >= sigma(0) > |simga(1)|
		if (sigma[1] > sigma[0]) {
			Swap(sigma[0], sigma[1]);
			SwapCols(U, 0, 1);
			SwapCols(V, 0, 1);
		}

		// Case: sigma(0) >= |sigma(2)| > |simga(1)|
		else {
			U.SetColumn(2, -U.GetColumn(2));
			V.SetColumn(2, -V.GetColumn(2));
			
		}
	}

	/**
	  \brief Helper function of 3X3 SVD for Sorting singular values
	  */
	template <class T>
	void Sort1(PMatrix<T, 3, 3>& U, TVector<T, 3>& sigma, PMatrix<T, 3, 3>& V) {

		// Case: |sigma(0)| >= sigma(1) > |sigma(2)|
		if (FGenericPlatformMath::Abs(sigma[0]) >= sigma[1]) {
			if (sigma[0] < (T)0.) {
				FlipSign(0, U, sigma);
				FlipSign(2, U, sigma);
			}
			return;
		}

		// swap sigma(0) and sigma(1) for both cases
		Swap(sigma[0], sigma[1]);
		SwapCols(U, 0, 1);
		SwapCols(V, 0, 1);

		// Case: sigma(1) > |sigma(2)| >= |sigma(0)|
		if (FGenericPlatformMath::Abs(sigma[1]) < FGenericPlatformMath::Abs(sigma[2])) {
			Swap(sigma[1], sigma[2]);
			SwapCols(U, 1, 2);
			SwapCols(V, 1, 2);
		}

		// Case: sigma(1) >= |sigma(0)| > |sigma(2)|
		else {
			U.SetColumn(1, -U.GetColumn(1));
			V.SetColumn(1, -V.GetColumn(1));
		}

		// fix sign for both cases
		if (sigma[1] < (T)0.) {
			FlipSign(1, U, sigma);
			FlipSign(2, U, sigma);
		}
	}

	/**
	  \brief 3X3 SVD (singular value decomposition) A=USV'
	  \param[in] A Input matrix.
	  \param[out] U is a rotation matrix.
	  \param[out] sigma Diagonal matrix, sorted with decreasing magnitude. The third one can be negative.
	  \param[out] V is a rotation matrix.
	  */
	template <class T>
	inline int SingularValueDecomposition(const PMatrix<T, 3, 3>& A, PMatrix<T, 3, 3>& U, TVector<T, 3>& sigma, PMatrix<T, 3, 3>& V,
		T tol = std::numeric_limits<T>::epsilon()) 
	{
		sigma[0] = T(0.);
		sigma[1] = T(0.);
		sigma[2] = T(0.);

		PMatrix<T, 3, 3> B = A;
		U = PMatrix<T, 3, 3>(1, 1, 1);
		V = PMatrix<T, 3, 3>(1, 1, 1);

		MakeUpperBidiag(B, U, V);

		int count = 0;
		T mu = (T)0;
		GivensRotation<T> r(0, 1);

		T alpha_1 = B.M[0][0];
		T beta_1 = B.M[1][0];
		T alpha_2 = B.M[1][1];
		T alpha_3 = B.M[2][2];
		T beta_2 = B.M[2][1];
		T gamma_1 = alpha_1 * beta_1;
		T gamma_2 = alpha_2 * beta_2;
		tol *= FMath::Max((T)0.5 * FMath::Sqrt(alpha_1 * alpha_1 + alpha_2 * alpha_2 + alpha_3 * alpha_3 + beta_1 * beta_1 + beta_2 * beta_2), (T)1);

		/**
		  Do implicit shift QR until A^T A is block diagonal
		  */

		while (FGenericPlatformMath::Abs(beta_2) > tol && FGenericPlatformMath::Abs(beta_1) > tol && FGenericPlatformMath::Abs(alpha_1) > tol && FGenericPlatformMath::Abs(alpha_2) > tol && FGenericPlatformMath::Abs(alpha_3) > tol) {
			mu = WilkinsonShift(alpha_2 * alpha_2 + beta_1 * beta_1, gamma_2, alpha_3 * alpha_3 + beta_2 * beta_2);

			r.Compute(alpha_1 * alpha_1 - mu, gamma_1);
			r.ColumnRotation(B);

			r.ColumnRotation(V);
			ZeroChase(B, U, V);

			alpha_1 = B.M[0][0];
			beta_1 = B.M[1][0];
			alpha_2 = B.M[1][1];
			alpha_3 = B.M[2][2];
			beta_2 = B.M[2][1];
			gamma_1 = alpha_1 * beta_1;
			gamma_2 = alpha_2 * beta_2;
			count++;
		}
		/**
		  Handle the cases of one of the alphas and betas being 0
		  Sorted by ease of handling and then frequency
		  of occurrence

		  If B is of form
		  x x 0
		  0 x 0
		  0 0 x
		  */
		if (FGenericPlatformMath::Abs(beta_2) <= tol) {
			Process<0>(B, U, sigma, V);
			Sort0(U, sigma, V);
		}
		/**
		  If B is of form
		  x 0 0
		  0 x x
		  0 0 x
		  */
		else if (FGenericPlatformMath::Abs(beta_1) <= tol) {
			Process<1>(B, U, sigma, V);
			Sort1(U, sigma, V);
		}
		/**
		  If B is of form
		  x x 0
		  0 0 x
		  0 0 x
		  */
		else if (FGenericPlatformMath::Abs(alpha_2) <= tol) {
			/**
			Reduce B to
			x x 0
			0 0 0
			0 0 x
			*/
			GivensRotation<T> r1(1, 2);
			r1.ComputeUnconventional(B.M[2][1], B.M[2][2]);
			r1.RowRotation(B);
			r1.ColumnRotation(U);

			Process<0>(B, U, sigma, V);
			Sort0(U, sigma, V);
		}
		/**
		  If B is of form
		  x x 0
		  0 x x
		  0 0 0
		  */
		else if (FGenericPlatformMath::Abs(alpha_3) <= tol) {
			/**
			Reduce B to
			x x +
			0 x 0
			0 0 0
			*/
			GivensRotation<T> r1(1, 2);
			r1.Compute(B.M[1][1], B.M[2][1]);
			r1.ColumnRotation(B);
			r1.ColumnRotation(V);
			/**
			Reduce B to
			x x 0
			+ x 0
			0 0 0
			*/
			GivensRotation<T> r2(0, 2);
			r2.Compute(B.M[0][0], B.M[2][0]);
			r2.ColumnRotation(B);
			r2.ColumnRotation(V);

			Process<0>(B, U, sigma, V);
			Sort0(U, sigma, V);
		}
		/**
		  If B is of form
		  0 x 0
		  0 x x
		  0 0 x
		  */
		else if (FGenericPlatformMath::Abs(alpha_1) <= tol) {
			/**
			Reduce B to
			0 0 +
			0 x x
			0 0 x
			*/
			GivensRotation<T> r1(0, 1);
			r1.ComputeUnconventional(B.M[1][0], B.M[1][1]);
			r1.RowRotation(B);
			r1.ColumnRotation(U);

			/**
			Reduce B to
			0 0 0
			0 x x
			0 + x
			*/
			GivensRotation<T> r2(0, 2);
			r2.ComputeUnconventional(B.M[2][0], B.M[2][2]);
			r2.RowRotation(B);
			r2.ColumnRotation(U);

			Process<1>(B, U, sigma, V);
			Sort1(U, sigma, V);
		}

		return count;
	}

	/**
		   \brief 3X3 polar decomposition.
		   \param[in] A matrix.
		   \param[out] R Robustly a rotation matrix.
		   \param[out] S_Sym Symmetric. Whole matrix is stored

		   Whole matrix S is stored
		   Polar guarantees negative sign is on the small magnitude singular value.
		   S is guaranteed to be the closest one to identity.
		   R is guaranteed to be the closest rotation to A.
		*/
	template <class T>
	inline void PolarDecomposition(const PMatrix<T, 3, 3>& A, PMatrix<T, 3, 3>& R, PMatrix<T, 3, 3>& S_Sym) {
		PMatrix<T, 3, 3> U;
		TVector<T, 3> sigma;
		PMatrix<T, 3, 3> V;

		SingularValueDecomposition(A, U, sigma, V);
		R = V.GetTransposed() * U;
		S_Sym = V.GetTransposed() * PMatrix<T, 3, 3>(sigma) * V;
	}

	// 3X3 version of dRdF
	// indexing of diff: dRdF[9*(i*3+j)+(m*3+n)] = dR.GetAt(m, n) /dF.GetAt (i, j):
	template <class T>
	void dRdFCorotated(const PMatrix<T, 3, 3>& F, TVector<T, 81>& dRdF) {
		PMatrix<T, 3, 3> R, S, Dinv;
		PolarDecomposition(F, R, S);
		PMatrix<T, 3, 3> D = (S.M[0][0] + S.M[1][1] + S.M[2][2]) * PMatrix<T, 3, 3>(1, 1, 1) - S;
		Dinv = D.Inverse();

		dRdF[0 * 9 + 0] = (-R.GetAt(0, 1) * Dinv.GetAt(2, 1) * R.GetAt(0, 2) + R.GetAt(0, 1) * Dinv.GetAt(2, 2) * R.GetAt(0, 1) + R.GetAt(0, 2) * Dinv.GetAt(1, 1) * R.GetAt(0, 2) - R.GetAt(0, 2) * Dinv.GetAt(1, 2) * R.GetAt(0, 1));
		dRdF[1 * 9 + 0] = (R.GetAt(0, 1) * Dinv.GetAt(2, 0) * R.GetAt(0, 2) - R.GetAt(0, 1) * Dinv.GetAt(2, 2) * R.GetAt(0, 0) - R.GetAt(0, 2) * Dinv.GetAt(1, 0) * R.GetAt(0, 2) + R.GetAt(0, 2) * Dinv.GetAt(1, 2) * R.GetAt(0, 0));
		dRdF[2 * 9 + 0] = (-R.GetAt(0, 1) * Dinv.GetAt(2, 0) * R.GetAt(0, 1) + R.GetAt(0, 1) * Dinv.GetAt(2, 1) * R.GetAt(0, 0) + R.GetAt(0, 2) * Dinv.GetAt(1, 0) * R.GetAt(0, 1) - R.GetAt(0, 2) * Dinv.GetAt(1, 1) * R.GetAt(0, 0));
		dRdF[3 * 9 + 0] = (-R.GetAt(0, 1) * Dinv.GetAt(2, 1) * R.GetAt(1, 2) + R.GetAt(0, 1) * Dinv.GetAt(2, 2) * R.GetAt(1, 1) + R.GetAt(0, 2) * Dinv.GetAt(1, 1) * R.GetAt(1, 2) - R.GetAt(0, 2) * Dinv.GetAt(1, 2) * R.GetAt(1, 1));
		dRdF[4 * 9 + 0] = (R.GetAt(0, 1) * Dinv.GetAt(2, 0) * R.GetAt(1, 2) - R.GetAt(0, 1) * Dinv.GetAt(2, 2) * R.GetAt(1, 0) - R.GetAt(0, 2) * Dinv.GetAt(1, 0) * R.GetAt(1, 2) + R.GetAt(0, 2) * Dinv.GetAt(1, 2) * R.GetAt(1, 0));
		dRdF[5 * 9 + 0] = (-R.GetAt(0, 1) * Dinv.GetAt(2, 0) * R.GetAt(1, 1) + R.GetAt(0, 1) *Dinv.GetAt(2, 1) * R.GetAt(1, 0) + R.GetAt(0, 2) * Dinv.GetAt(1, 0) * R.GetAt(1, 1) - R.GetAt(0, 2) * Dinv.GetAt(1, 1) * R.GetAt(1, 0));
		dRdF[6 * 9 + 0] = (-R.GetAt(0, 1) * Dinv.GetAt(2, 1) * R.GetAt(2, 2) + R.GetAt(0, 1) * Dinv.GetAt(2, 2) * R.GetAt(2, 1) + R.GetAt(0, 2) * Dinv.GetAt(1, 1) * R.GetAt(2, 2) - R.GetAt(0, 2) * Dinv.GetAt(1, 2) * R.GetAt(2, 1));
		dRdF[7 * 9 + 0] = (R.GetAt(0, 1) * Dinv.GetAt(2, 0) * R.GetAt(2, 2) - R.GetAt(0, 1) * Dinv.GetAt(2, 2) * R.GetAt(2, 0) - R.GetAt(0, 2) * Dinv.GetAt(1, 0) * R.GetAt(2, 2) + R.GetAt(0, 2) * Dinv.GetAt(1, 2) * R.GetAt(2, 0));
		dRdF[8 * 9 + 0] = (-R.GetAt(0, 1) * Dinv.GetAt(2, 0) * R.GetAt(2, 1) + R.GetAt(0, 1) * Dinv.GetAt(2, 1) * R.GetAt(2, 0) + R.GetAt(0, 2) * Dinv.GetAt(1, 0) * R.GetAt(2, 1) - R.GetAt(0, 2) * Dinv.GetAt(1, 1) * R.GetAt(2, 0));
		dRdF[0 * 9 + 1] = (R.GetAt(0, 0) * Dinv.GetAt(2, 1) * R.GetAt(0, 2) - R.GetAt(0, 0) * Dinv.GetAt(2, 2) * R.GetAt(0, 1) - R.GetAt(0, 2) * Dinv.GetAt(0, 1) * R.GetAt(0, 2) + R.GetAt(0, 2) * Dinv.GetAt(0, 2) * R.GetAt(0, 1));
		dRdF[1 * 9 + 1] = (-R.GetAt(0, 0) * Dinv.GetAt(2, 0) * R.GetAt(0, 2) + R.GetAt(0, 0) * Dinv.GetAt(2, 2) * R.GetAt(0, 0) + R.GetAt(0, 2) * Dinv.GetAt(0, 0) * R.GetAt(0, 2) - R.GetAt(0, 2) * Dinv.GetAt(0, 2) * R.GetAt(0, 0));
		dRdF[2 * 9 + 1] = (R.GetAt(0, 0) * Dinv.GetAt(2, 0) * R.GetAt(0, 1) - R.GetAt(0, 0) * Dinv.GetAt(2, 1) * R.GetAt(0, 0) - R.GetAt(0, 2) * Dinv.GetAt(0, 0) * R.GetAt(0, 1) + R.GetAt(0, 2) * Dinv.GetAt(0, 1) * R.GetAt(0, 0));
		dRdF[3 * 9 + 1] = (R.GetAt(0, 0) * Dinv.GetAt(2, 1) * R.GetAt(1, 2) - R.GetAt(0, 0) * Dinv.GetAt(2, 2) * R.GetAt(1, 1) - R.GetAt(0, 2) * Dinv.GetAt(0, 1) * R.GetAt(1, 2) + R.GetAt(0, 2) * Dinv.GetAt(0, 2) * R.GetAt(1, 1));
		dRdF[4 * 9 + 1] = (-R.GetAt(0, 0) * Dinv.GetAt(2, 0) * R.GetAt(1, 2) + R.GetAt(0, 0) * Dinv.GetAt(2, 2) * R.GetAt(1, 0) + R.GetAt(0, 2) * Dinv.GetAt(0, 0) * R.GetAt(1, 2) - R.GetAt(0, 2) * Dinv.GetAt(0, 2) * R.GetAt(1, 0));
		dRdF[5 * 9 + 1] = (R.GetAt(0, 0) * Dinv.GetAt(2, 0) * R.GetAt(1, 1) - R.GetAt(0, 0) * Dinv.GetAt(2, 1) * R.GetAt(1, 0) - R.GetAt(0, 2) * Dinv.GetAt(0, 0) * R.GetAt(1, 1) + R.GetAt(0, 2) * Dinv.GetAt(0, 1) * R.GetAt(1, 0));
		dRdF[6 * 9 + 1] = (R.GetAt(0, 0) * Dinv.GetAt(2, 1) * R.GetAt(2, 2) - R.GetAt(0, 0) * Dinv.GetAt(2, 2) * R.GetAt(2, 1) - R.GetAt(0, 2) * Dinv.GetAt(0, 1) * R.GetAt(2, 2) + R.GetAt(0, 2) * Dinv.GetAt(0, 2) * R.GetAt(2, 1));
		dRdF[7 * 9 + 1] = (-R.GetAt(0, 0) * Dinv.GetAt(2, 0) * R.GetAt(2, 2) + R.GetAt(0, 0) * Dinv.GetAt(2, 2) * R.GetAt(2, 0) + R.GetAt(0, 2) * Dinv.GetAt(0, 0) * R.GetAt(2, 2) - R.GetAt(0, 2) * Dinv.GetAt(0, 2) * R.GetAt(2, 0));
		dRdF[8 * 9 + 1] = (R.GetAt(0, 0) * Dinv.GetAt(2, 0) * R.GetAt(2, 1) - R.GetAt(0, 0) * Dinv.GetAt(2, 1) * R.GetAt(2, 0) - R.GetAt(0, 2) * Dinv.GetAt(0, 0) * R.GetAt(2, 1) + R.GetAt(0, 2) * Dinv.GetAt(0, 1) * R.GetAt(2, 0));
		dRdF[0 * 9 + 2] = (-R.GetAt(0, 0) * Dinv.GetAt(1, 1) * R.GetAt(0, 2) + R.GetAt(0, 0) * Dinv.GetAt(1, 2) * R.GetAt(0, 1) + R.GetAt(0, 1) * Dinv.GetAt(0, 1) * R.GetAt(0, 2) - R.GetAt(0, 1) * Dinv.GetAt(0, 2) * R.GetAt(0, 1));
		dRdF[1 * 9 + 2] = (R.GetAt(0, 0) * Dinv.GetAt(1, 0) * R.GetAt(0, 2) - R.GetAt(0, 0) * Dinv.GetAt(1, 2) * R.GetAt(0, 0) - R.GetAt(0, 1) * Dinv.GetAt(0, 0) * R.GetAt(0, 2) + R.GetAt(0, 1) * Dinv.GetAt(0, 2) * R.GetAt(0, 0));
		dRdF[2 * 9 + 2] = (-R.GetAt(0, 0) * Dinv.GetAt(1, 0) * R.GetAt(0, 1) + R.GetAt(0, 0) * Dinv.GetAt(1, 1) * R.GetAt(0, 0) + R.GetAt(0, 1) * Dinv.GetAt(0, 0) * R.GetAt(0, 1) - R.GetAt(0, 1) * Dinv.GetAt(0, 1) * R.GetAt(0, 0));
		dRdF[3 * 9 + 2] = (-R.GetAt(0, 0) * Dinv.GetAt(1, 1) * R.GetAt(1, 2) + R.GetAt(0, 0) * Dinv.GetAt(1, 2) * R.GetAt(1, 1) + R.GetAt(0, 1) * Dinv.GetAt(0, 1) * R.GetAt(1, 2) - R.GetAt(0, 1) * Dinv.GetAt(0, 2) * R.GetAt(1, 1));
		dRdF[4 * 9 + 2] = (R.GetAt(0, 0) * Dinv.GetAt(1, 0) * R.GetAt(1, 2) - R.GetAt(0, 0) * Dinv.GetAt(1, 2) * R.GetAt(1, 0) - R.GetAt(0, 1) * Dinv.GetAt(0, 0) * R.GetAt(1, 2) + R.GetAt(0, 1) * Dinv.GetAt(0, 2) * R.GetAt(1, 0));
		dRdF[5 * 9 + 2] = (-R.GetAt(0, 0) * Dinv.GetAt(1, 0) * R.GetAt(1, 1) + R.GetAt(0, 0) * Dinv.GetAt(1, 1) * R.GetAt(1, 0) + R.GetAt(0, 1) * Dinv.GetAt(0, 0) * R.GetAt(1, 1) - R.GetAt(0, 1) * Dinv.GetAt(0, 1) * R.GetAt(1, 0));
		dRdF[6 * 9 + 2] = (-R.GetAt(0, 0) * Dinv.GetAt(1, 1) * R.GetAt(2, 2) + R.GetAt(0, 0) * Dinv.GetAt(1, 2) * R.GetAt(2, 1) + R.GetAt(0, 1) * Dinv.GetAt(0, 1) * R.GetAt(2, 2) - R.GetAt(0, 1) * Dinv.GetAt(0, 2) * R.GetAt(2, 1));
		dRdF[7 * 9 + 2] = (R.GetAt(0, 0) * Dinv.GetAt(1, 0) * R.GetAt(2, 2) - R.GetAt(0, 0) * Dinv.GetAt(1, 2) * R.GetAt(2, 0) - R.GetAt(0, 1) * Dinv.GetAt(0, 0) * R.GetAt(2, 2) + R.GetAt(0, 1) * Dinv.GetAt(0, 2) * R.GetAt(2, 0));
		dRdF[8 * 9 + 2] = (-R.GetAt(0, 0) * Dinv.GetAt(1, 0) * R.GetAt(2, 1) + R.GetAt(0, 0) * Dinv.GetAt(1, 1) * R.GetAt(2, 0) + R.GetAt(0, 1) * Dinv.GetAt(0, 0) * R.GetAt(2, 1) - R.GetAt(0, 1) * Dinv.GetAt(0, 1) * R.GetAt(2, 0));
		dRdF[0 * 9 + 3] = (-R.GetAt(1, 1) * Dinv.GetAt(2, 1) * R.GetAt(0, 2) + R.GetAt(1, 1) * Dinv.GetAt(2, 2) * R.GetAt(0, 1) + R.GetAt(1, 2) * Dinv.GetAt(1, 1) * R.GetAt(0, 2) - R.GetAt(1, 2) * Dinv.GetAt(1, 2) * R.GetAt(0, 1));
		dRdF[1 * 9 + 3] = (R.GetAt(1, 1) * Dinv.GetAt(2, 0) * R.GetAt(0, 2) - R.GetAt(1, 1) * Dinv.GetAt(2, 2) * R.GetAt(0, 0) - R.GetAt(1, 2) * Dinv.GetAt(1, 0) * R.GetAt(0, 2) + R.GetAt(1, 2) * Dinv.GetAt(1, 2) * R.GetAt(0, 0));
		dRdF[2 * 9 + 3] = (-R.GetAt(1, 1) * Dinv.GetAt(2, 0) * R.GetAt(0, 1) + R.GetAt(1, 1) * Dinv.GetAt(2, 1) * R.GetAt(0, 0) + R.GetAt(1, 2) * Dinv.GetAt(1, 0) * R.GetAt(0, 1) - R.GetAt(1, 2) * Dinv.GetAt(1, 1) * R.GetAt(0, 0));
		dRdF[3 * 9 + 3] = (-R.GetAt(1, 1) * Dinv.GetAt(2, 1) * R.GetAt(1, 2) + R.GetAt(1, 1) * Dinv.GetAt(2, 2) * R.GetAt(1, 1) + R.GetAt(1, 2) * Dinv.GetAt(1, 1) * R.GetAt(1, 2) - R.GetAt(1, 2) * Dinv.GetAt(1, 2) * R.GetAt(1, 1));
		dRdF[4 * 9 + 3] = (R.GetAt(1, 1) * Dinv.GetAt(2, 0) * R.GetAt(1, 2) - R.GetAt(1, 1) * Dinv.GetAt(2, 2) * R.GetAt(1, 0) - R.GetAt(1, 2) * Dinv.GetAt(1, 0) * R.GetAt(1, 2) + R.GetAt(1, 2) * Dinv.GetAt(1, 2) * R.GetAt(1, 0));
		dRdF[5 * 9 + 3] = (-R.GetAt(1, 1) * Dinv.GetAt(2, 0) * R.GetAt(1, 1) + R.GetAt(1, 1) * Dinv.GetAt(2, 1) * R.GetAt(1, 0) + R.GetAt(1, 2) * Dinv.GetAt(1, 0) * R.GetAt(1, 1) - R.GetAt(1, 2) * Dinv.GetAt(1, 1) * R.GetAt(1, 0));
		dRdF[6 * 9 + 3] = (-R.GetAt(1, 1) * Dinv.GetAt(2, 1) * R.GetAt(2, 2) + R.GetAt(1, 1) * Dinv.GetAt(2, 2) * R.GetAt(2, 1) + R.GetAt(1, 2) * Dinv.GetAt(1, 1) * R.GetAt(2, 2) - R.GetAt(1, 2) * Dinv.GetAt(1, 2) * R.GetAt(2, 1));
		dRdF[7 * 9 + 3] = (R.GetAt(1, 1) * Dinv.GetAt(2, 0) * R.GetAt(2, 2) - R.GetAt(1, 1) * Dinv.GetAt(2, 2) * R.GetAt(2, 0) - R.GetAt(1, 2) * Dinv.GetAt(1, 0) * R.GetAt(2, 2) + R.GetAt(1, 2) * Dinv.GetAt(1, 2) * R.GetAt(2, 0));
		dRdF[8 * 9 + 3] = (-R.GetAt(1, 1) * Dinv.GetAt(2, 0) * R.GetAt(2, 1) + R.GetAt(1, 1) * Dinv.GetAt(2, 1) * R.GetAt(2, 0) + R.GetAt(1, 2) * Dinv.GetAt(1, 0) * R.GetAt(2, 1) - R.GetAt(1, 2) * Dinv.GetAt(1, 1) * R.GetAt(2, 0));
		dRdF[0 * 9 + 4] = (R.GetAt(1, 0) * Dinv.GetAt(2, 1) * R.GetAt(0, 2) - R.GetAt(1, 0) * Dinv.GetAt(2, 2) * R.GetAt(0, 1) - R.GetAt(1, 2) * Dinv.GetAt(0, 1) * R.GetAt(0, 2) + R.GetAt(1, 2) * Dinv.GetAt(0, 2) * R.GetAt(0, 1));
		dRdF[1 * 9 + 4] = (-R.GetAt(1, 0) * Dinv.GetAt(2, 0) * R.GetAt(0, 2) + R.GetAt(1, 0) * Dinv.GetAt(2, 2) * R.GetAt(0, 0) + R.GetAt(1, 2) * Dinv.GetAt(0, 0) * R.GetAt(0, 2) - R.GetAt(1, 2) * Dinv.GetAt(0, 2) * R.GetAt(0, 0));
		dRdF[2 * 9 + 4] = (R.GetAt(1, 0) * Dinv.GetAt(2, 0) * R.GetAt(0, 1) - R.GetAt(1, 0) * Dinv.GetAt(2, 1) * R.GetAt(0, 0) - R.GetAt(1, 2) * Dinv.GetAt(0, 0) * R.GetAt(0, 1) + R.GetAt(1, 2) * Dinv.GetAt(0, 1) * R.GetAt(0, 0));
		dRdF[3 * 9 + 4] = (R.GetAt(1, 0) * Dinv.GetAt(2, 1) * R.GetAt(1, 2) - R.GetAt(1, 0) * Dinv.GetAt(2, 2) * R.GetAt(1, 1) - R.GetAt(1, 2) * Dinv.GetAt(0, 1) * R.GetAt(1, 2) + R.GetAt(1, 2) * Dinv.GetAt(0, 2) * R.GetAt(1, 1));
		dRdF[4 * 9 + 4] = (-R.GetAt(1, 0) * Dinv.GetAt(2, 0) * R.GetAt(1, 2) + R.GetAt(1, 0) * Dinv.GetAt(2, 2) * R.GetAt(1, 0) + R.GetAt(1, 2) * Dinv.GetAt(0, 0) * R.GetAt(1, 2) - R.GetAt(1, 2) * Dinv.GetAt(0, 2) * R.GetAt(1, 0));
		dRdF[5 * 9 + 4] = (R.GetAt(1, 0) * Dinv.GetAt(2, 0) * R.GetAt(1, 1) - R.GetAt(1, 0) * Dinv.GetAt(2, 1) * R.GetAt(1, 0) - R.GetAt(1, 2) * Dinv.GetAt(0, 0) * R.GetAt(1, 1) + R.GetAt(1, 2) * Dinv.GetAt(0, 1) * R.GetAt(1, 0));
		dRdF[6 * 9 + 4] = (R.GetAt(1, 0) * Dinv.GetAt(2, 1) * R.GetAt(2, 2) - R.GetAt(1, 0) * Dinv.GetAt(2, 2) * R.GetAt(2, 1) - R.GetAt(1, 2) * Dinv.GetAt(0, 1) * R.GetAt(2, 2) + R.GetAt(1, 2) * Dinv.GetAt(0, 2) * R.GetAt(2, 1));
		dRdF[7 * 9 + 4] = (-R.GetAt(1, 0) * Dinv.GetAt(2, 0) * R.GetAt(2, 2) + R.GetAt(1, 0) * Dinv.GetAt(2, 2) * R.GetAt(2, 0) + R.GetAt(1, 2) * Dinv.GetAt(0, 0) * R.GetAt(2, 2) - R.GetAt(1, 2) * Dinv.GetAt(0, 2) * R.GetAt(2, 0));
		dRdF[8 * 9 + 4] = (R.GetAt(1, 0) * Dinv.GetAt(2, 0) * R.GetAt(2, 1) - R.GetAt(1, 0) * Dinv.GetAt(2, 1) * R.GetAt(2, 0) - R.GetAt(1, 2) * Dinv.GetAt(0, 0) * R.GetAt(2, 1) + R.GetAt(1, 2) * Dinv.GetAt(0, 1) * R.GetAt(2, 0));
		dRdF[0 * 9 + 5] = (-R.GetAt(1, 0) * Dinv.GetAt(1, 1) * R.GetAt(0, 2) + R.GetAt(1, 0) * Dinv.GetAt(1, 2) * R.GetAt(0, 1) + R.GetAt(1, 1) * Dinv.GetAt(0, 1) * R.GetAt(0, 2) - R.GetAt(1, 1) * Dinv.GetAt(0, 2) * R.GetAt(0, 1));
		dRdF[1 * 9 + 5] = (R.GetAt(1, 0) * Dinv.GetAt(1, 0) * R.GetAt(0, 2) - R.GetAt(1, 0) * Dinv.GetAt(1, 2) * R.GetAt(0, 0) - R.GetAt(1, 1) * Dinv.GetAt(0, 0) * R.GetAt(0, 2) + R.GetAt(1, 1) * Dinv.GetAt(0, 2) * R.GetAt(0, 0));
		dRdF[2 * 9 + 5] = (-R.GetAt(1, 0) * Dinv.GetAt(1, 0) * R.GetAt(0, 1) + R.GetAt(1, 0) * Dinv.GetAt(1, 1) * R.GetAt(0, 0) + R.GetAt(1, 1) * Dinv.GetAt(0, 0) * R.GetAt(0, 1) - R.GetAt(1, 1) * Dinv.GetAt(0, 1) * R.GetAt(0, 0));
		dRdF[3 * 9 + 5] = (-R.GetAt(1, 0) * Dinv.GetAt(1, 1) * R.GetAt(1, 2) + R.GetAt(1, 0) * Dinv.GetAt(1, 2) * R.GetAt(1, 1) + R.GetAt(1, 1) * Dinv.GetAt(0, 1) * R.GetAt(1, 2) - R.GetAt(1, 1) * Dinv.GetAt(0, 2) * R.GetAt(1, 1));
		dRdF[4 * 9 + 5] = (R.GetAt(1, 0) * Dinv.GetAt(1, 0) * R.GetAt(1, 2) - R.GetAt(1, 0) * Dinv.GetAt(1, 2) * R.GetAt(1, 0) - R.GetAt(1, 1) * Dinv.GetAt(0, 0) * R.GetAt(1, 2) + R.GetAt(1, 1) * Dinv.GetAt(0, 2) * R.GetAt(1, 0));
		dRdF[5 * 9 + 5] = (-R.GetAt(1, 0) * Dinv.GetAt(1, 0) * R.GetAt(1, 1) + R.GetAt(1, 0) * Dinv.GetAt(1, 1) * R.GetAt(1, 0) + R.GetAt(1, 1) * Dinv.GetAt(0, 0) * R.GetAt(1, 1) - R.GetAt(1, 1) * Dinv.GetAt(0, 1) * R.GetAt(1, 0));
		dRdF[6 * 9 + 5] = (-R.GetAt(1, 0) * Dinv.GetAt(1, 1) * R.GetAt(2, 2) + R.GetAt(1, 0) * Dinv.GetAt(1, 2) * R.GetAt(2, 1) + R.GetAt(1, 1) * Dinv.GetAt(0, 1) * R.GetAt(2, 2) - R.GetAt(1, 1) * Dinv.GetAt(0, 2) * R.GetAt(2, 1));
		dRdF[7 * 9 + 5] = (R.GetAt(1, 0) * Dinv.GetAt(1, 0) * R.GetAt(2, 2) - R.GetAt(1, 0) * Dinv.GetAt(1, 2) * R.GetAt(2, 0) - R.GetAt(1, 1) * Dinv.GetAt(0, 0) * R.GetAt(2, 2) + R.GetAt(1, 1) * Dinv.GetAt(0, 2) * R.GetAt(2, 0));
		dRdF[8 * 9 + 5] = (-R.GetAt(1, 0) * Dinv.GetAt(1, 0) * R.GetAt(2, 1) + R.GetAt(1, 0) * Dinv.GetAt(1, 1) * R.GetAt(2, 0) + R.GetAt(1, 1) * Dinv.GetAt(0, 0) * R.GetAt(2, 1) - R.GetAt(1, 1) * Dinv.GetAt(0, 1) * R.GetAt(2, 0));
		dRdF[0 * 9 + 6] = (-R.GetAt(2, 1) * Dinv.GetAt(2, 1) * R.GetAt(0, 2) + R.GetAt(2, 1) * Dinv.GetAt(2, 2) * R.GetAt(0, 1) + R.GetAt(2, 2) * Dinv.GetAt(1, 1) * R.GetAt(0, 2) - R.GetAt(2, 2) * Dinv.GetAt(1, 2) * R.GetAt(0, 1));
		dRdF[1 * 9 + 6] = (R.GetAt(2, 1) * Dinv.GetAt(2, 0) * R.GetAt(0, 2) - R.GetAt(2, 1) * Dinv.GetAt(2, 2) * R.GetAt(0, 0) - R.GetAt(2, 2) * Dinv.GetAt(1, 0) * R.GetAt(0, 2) + R.GetAt(2, 2) * Dinv.GetAt(1, 2) * R.GetAt(0, 0));
		dRdF[2 * 9 + 6] = (-R.GetAt(2, 1) * Dinv.GetAt(2, 0) * R.GetAt(0, 1) + R.GetAt(2, 1) * Dinv.GetAt(2, 1) * R.GetAt(0, 0) + R.GetAt(2, 2) * Dinv.GetAt(1, 0) * R.GetAt(0, 1) - R.GetAt(2, 2) * Dinv.GetAt(1, 1) * R.GetAt(0, 0));
		dRdF[3 * 9 + 6] = (-R.GetAt(2, 1) * Dinv.GetAt(2, 1) * R.GetAt(1, 2) + R.GetAt(2, 1) * Dinv.GetAt(2, 2) * R.GetAt(1, 1) + R.GetAt(2, 2) * Dinv.GetAt(1, 1) * R.GetAt(1, 2) - R.GetAt(2, 2) * Dinv.GetAt(1, 2) * R.GetAt(1, 1));
		dRdF[4 * 9 + 6] = (R.GetAt(2, 1) * Dinv.GetAt(2, 0) * R.GetAt(1, 2) - R.GetAt(2, 1) * Dinv.GetAt(2, 2) * R.GetAt(1, 0) - R.GetAt(2, 2) * Dinv.GetAt(1, 0) * R.GetAt(1, 2) + R.GetAt(2, 2) * Dinv.GetAt(1, 2) * R.GetAt(1, 0));
		dRdF[5 * 9 + 6] = (-R.GetAt(2, 1) * Dinv.GetAt(2, 0) * R.GetAt(1, 1) + R.GetAt(2, 1) * Dinv.GetAt(2, 1) * R.GetAt(1, 0) + R.GetAt(2, 2) * Dinv.GetAt(1, 0) * R.GetAt(1, 1) - R.GetAt(2, 2) * Dinv.GetAt(1, 1) * R.GetAt(1, 0));
		dRdF[6 * 9 + 6] = (-R.GetAt(2, 1) * Dinv.GetAt(2, 1) * R.GetAt(2, 2) + R.GetAt(2, 1) * Dinv.GetAt(2, 2) * R.GetAt(2, 1) + R.GetAt(2, 2) * Dinv.GetAt(1, 1) * R.GetAt(2, 2) - R.GetAt(2, 2) * Dinv.GetAt(1, 2) * R.GetAt(2, 1));
		dRdF[7 * 9 + 6] = (R.GetAt(2, 1) * Dinv.GetAt(2, 0) * R.GetAt(2, 2) - R.GetAt(2, 1) * Dinv.GetAt(2, 2) * R.GetAt(2, 0) - R.GetAt(2, 2) * Dinv.GetAt(1, 0) * R.GetAt(2, 2) + R.GetAt(2, 2) * Dinv.GetAt(1, 2) * R.GetAt(2, 0));
		dRdF[8 * 9 + 6] = (-R.GetAt(2, 1) * Dinv.GetAt(2, 0) * R.GetAt(2, 1) + R.GetAt(2, 1) * Dinv.GetAt(2, 1) * R.GetAt(2, 0) + R.GetAt(2, 2) * Dinv.GetAt(1, 0) * R.GetAt(2, 1) - R.GetAt(2, 2) * Dinv.GetAt(1, 1) * R.GetAt(2, 0));
		dRdF[0 * 9 + 7] = (R.GetAt(2, 0) * Dinv.GetAt(2, 1) * R.GetAt(0, 2) - R.GetAt(2, 0) * Dinv.GetAt(2, 2) * R.GetAt(0, 1) - R.GetAt(2, 2) * Dinv.GetAt(0, 1) * R.GetAt(0, 2) + R.GetAt(2, 2) * Dinv.GetAt(0, 2) * R.GetAt(0, 1));
		dRdF[1 * 9 + 7] = (-R.GetAt(2, 0) * Dinv.GetAt(2, 0) * R.GetAt(0, 2) + R.GetAt(2, 0) * Dinv.GetAt(2, 2) * R.GetAt(0, 0) + R.GetAt(2, 2) * Dinv.GetAt(0, 0) * R.GetAt(0, 2) - R.GetAt(2, 2) * Dinv.GetAt(0, 2) * R.GetAt(0, 0));
		dRdF[2 * 9 + 7] = (R.GetAt(2, 0) * Dinv.GetAt(2, 0) * R.GetAt(0, 1) - R.GetAt(2, 0) * Dinv.GetAt(2, 1) * R.GetAt(0, 0) - R.GetAt(2, 2) * Dinv.GetAt(0, 0) * R.GetAt(0, 1) + R.GetAt(2, 2) * Dinv.GetAt(0, 1) * R.GetAt(0, 0));
		dRdF[3 * 9 + 7] = (R.GetAt(2, 0) * Dinv.GetAt(2, 1) * R.GetAt(1, 2) - R.GetAt(2, 0) * Dinv.GetAt(2, 2) * R.GetAt(1, 1) - R.GetAt(2, 2) * Dinv.GetAt(0, 1) * R.GetAt(1, 2) + R.GetAt(2, 2) * Dinv.GetAt(0, 2) * R.GetAt(1, 1));
		dRdF[4 * 9 + 7] = (-R.GetAt(2, 0) * Dinv.GetAt(2, 0) * R.GetAt(1, 2) + R.GetAt(2, 0) * Dinv.GetAt(2, 2) * R.GetAt(1, 0) + R.GetAt(2, 2) * Dinv.GetAt(0, 0) * R.GetAt(1, 2) - R.GetAt(2, 2) * Dinv.GetAt(0, 2) * R.GetAt(1, 0));
		dRdF[5 * 9 + 7] = (R.GetAt(2, 0) * Dinv.GetAt(2, 0) * R.GetAt(1, 1) - R.GetAt(2, 0) * Dinv.GetAt(2, 1) * R.GetAt(1, 0) - R.GetAt(2, 2) * Dinv.GetAt(0, 0) * R.GetAt(1, 1) + R.GetAt(2, 2) * Dinv.GetAt(0, 1) * R.GetAt(1, 0));
		dRdF[6 * 9 + 7] = (R.GetAt(2, 0) * Dinv.GetAt(2, 1) * R.GetAt(2, 2) - R.GetAt(2, 0) * Dinv.GetAt(2, 2) * R.GetAt(2, 1) - R.GetAt(2, 2) * Dinv.GetAt(0, 1) * R.GetAt(2, 2) + R.GetAt(2, 2) * Dinv.GetAt(0, 2) * R.GetAt(2, 1));
		dRdF[7 * 9 + 7] = (-R.GetAt(2, 0) * Dinv.GetAt(2, 0) * R.GetAt(2, 2) + R.GetAt(2, 0) * Dinv.GetAt(2, 2) * R.GetAt(2, 0) + R.GetAt(2, 2) * Dinv.GetAt(0, 0) * R.GetAt(2, 2) - R.GetAt(2, 2) * Dinv.GetAt(0, 2) * R.GetAt(2, 0));
		dRdF[8 * 9 + 7] = (R.GetAt(2, 0) * Dinv.GetAt(2, 0) * R.GetAt(2, 1) - R.GetAt(2, 0) * Dinv.GetAt(2, 1) * R.GetAt(2, 0) - R.GetAt(2, 2) * Dinv.GetAt(0, 0) * R.GetAt(2, 1) + R.GetAt(2, 2) * Dinv.GetAt(0, 1) * R.GetAt(2, 0));
		dRdF[0 * 9 + 8] = (-R.GetAt(2, 0) * Dinv.GetAt(1, 1) * R.GetAt(0, 2) + R.GetAt(2, 0) * Dinv.GetAt(1, 2) * R.GetAt(0, 1) + R.GetAt(2, 1) * Dinv.GetAt(0, 1) * R.GetAt(0, 2) - R.GetAt(2, 1) * Dinv.GetAt(0, 2) * R.GetAt(0, 1));
		dRdF[1 * 9 + 8] = (R.GetAt(2, 0) * Dinv.GetAt(1, 0) * R.GetAt(0, 2) - R.GetAt(2, 0) * Dinv.GetAt(1, 2) * R.GetAt(0, 0) - R.GetAt(2, 1) * Dinv.GetAt(0, 0) * R.GetAt(0, 2) + R.GetAt(2, 1) * Dinv.GetAt(0, 2) * R.GetAt(0, 0));
		dRdF[2 * 9 + 8] = (-R.GetAt(2, 0) * Dinv.GetAt(1, 0) * R.GetAt(0, 1) + R.GetAt(2, 0) * Dinv.GetAt(1, 1) * R.GetAt(0, 0) + R.GetAt(2, 1) * Dinv.GetAt(0, 0) * R.GetAt(0, 1) - R.GetAt(2, 1) * Dinv.GetAt(0, 1) * R.GetAt(0, 0));
		dRdF[3 * 9 + 8] = (-R.GetAt(2, 0) * Dinv.GetAt(1, 1) * R.GetAt(1, 2) + R.GetAt(2, 0) * Dinv.GetAt(1, 2) * R.GetAt(1, 1) + R.GetAt(2, 1) * Dinv.GetAt(0, 1) * R.GetAt(1, 2) - R.GetAt(2, 1) * Dinv.GetAt(0, 2) * R.GetAt(1, 1));
		dRdF[4 * 9 + 8] = (R.GetAt(2, 0) * Dinv.GetAt(1, 0) * R.GetAt(1, 2) - R.GetAt(2, 0) * Dinv.GetAt(1, 2) * R.GetAt(1, 0) - R.GetAt(2, 1) * Dinv.GetAt(0, 0) * R.GetAt(1, 2) + R.GetAt(2, 1) * Dinv.GetAt(0, 2) * R.GetAt(1, 0));
		dRdF[5 * 9 + 8] = (-R.GetAt(2, 0) * Dinv.GetAt(1, 0) * R.GetAt(1, 1) + R.GetAt(2, 0) * Dinv.GetAt(1, 1) * R.GetAt(1, 0) + R.GetAt(2, 1) * Dinv.GetAt(0, 0) * R.GetAt(1, 1) - R.GetAt(2, 1) * Dinv.GetAt(0, 1) * R.GetAt(1, 0));
		dRdF[6 * 9 + 8] = (-R.GetAt(2, 0) * Dinv.GetAt(1, 1) * R.GetAt(2, 2) + R.GetAt(2, 0) * Dinv.GetAt(1, 2) * R.GetAt(2, 1) + R.GetAt(2, 1) * Dinv.GetAt(0, 1) * R.GetAt(2, 2) - R.GetAt(2, 1) * Dinv.GetAt(0, 2) * R.GetAt(2, 1));
		dRdF[7 * 9 + 8] = (R.GetAt(2, 0) * Dinv.GetAt(1, 0) * R.GetAt(2, 2) - R.GetAt(2, 0) * Dinv.GetAt(1, 2) * R.GetAt(2, 0) - R.GetAt(2, 1) * Dinv.GetAt(0, 0) * R.GetAt(2, 2) + R.GetAt(2, 1) * Dinv.GetAt(0, 2) * R.GetAt(2, 0));
		dRdF[8 * 9 + 8] = (-R.GetAt(2, 0) * Dinv.GetAt(1, 0) * R.GetAt(2, 1) + R.GetAt(2, 0) * Dinv.GetAt(1, 1) * R.GetAt(2, 0) + R.GetAt(2, 1) * Dinv.GetAt(0, 0) * R.GetAt(2, 1) - R.GetAt(2, 1) * Dinv.GetAt(0, 1) * R.GetAt(2, 0));
	}

}  // namespace Chaos

