// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Math/Matrix.h"
#include "Chaos/Vector.h"
#else
#include <array>

struct UE::Math::TMatrix<FReal>
{
public:
	std::array<std::array<Chaos::FReal, 4>, 4> M;
};
#endif

//-V:TMatrix:546 PVS 7.41 bug
namespace Chaos
{
	template<class T, int m, int n>
	class PMatrix
	{
	private:
		PMatrix() {}
		~PMatrix() = default;
	};

	template<>
	class PMatrix<FRealDouble, 3, 2>
	{
	public:
		FRealDouble M[6];

		PMatrix(const TVector<FRealDouble, 3>& C1, const TVector<FRealDouble, 3>& C2)
		{
			M[0] = C1.X;
			M[1] = C1.Y;
			M[2] = C1.Z;
			M[3] = C2.X;
			M[4] = C2.Y;
			M[5] = C2.Z;
		}

		PMatrix(const FRealDouble x00, const FRealDouble x10, const FRealDouble x20, const FRealDouble x01, const FRealDouble x11, const FRealDouble x21)
		{
			M[0] = x00;
			M[1] = x10;
			M[2] = x20;
			M[3] = x01;
			M[4] = x11;
			M[5] = x21;
		}

		TVector<FRealDouble, 3> operator*(const TVector<FRealDouble, 2>& Other)
		{
			return TVector<FRealDouble, 3>(
			    M[0] * Other[0] + M[3] * Other[1],
			    M[1] * Other[0] + M[4] * Other[1],
			    M[2] * Other[0] + M[5] * Other[1]);
		}
	};

	template<>
	class PMatrix<FRealSingle, 3, 2>
	{
	public:
		FRealSingle M[6];

		PMatrix(const TVector<FRealSingle, 3>& C1, const TVector<FRealSingle, 3>& C2)
		{
			M[0] = C1.X;
			M[1] = C1.Y;
			M[2] = C1.Z;
			M[3] = C2.X;
			M[4] = C2.Y;
			M[5] = C2.Z;
		}

		PMatrix(const FRealSingle x00, const FRealSingle x10, const FRealSingle x20, const FRealSingle x01, const FRealSingle x11, const FRealSingle x21)
		{
			M[0] = x00;
			M[1] = x10;
			M[2] = x20;
			M[3] = x01;
			M[4] = x11;
			M[5] = x21;
		}

		PMatrix(const FRealSingle x00)
		{
			M[0] = x00;
			M[1] = x00;
			M[2] = x00;
			M[3] = x00;
			M[4] = x00;
			M[5] = x00;
		}

		TVector<FRealSingle, 3> operator*(const TVector<FRealSingle, 2>& Other)
		{
			return TVector<FRealSingle, 3>(
				M[0] * Other[0] + M[3] * Other[1],
				M[1] * Other[0] + M[4] * Other[1],
				M[2] * Other[0] + M[5] * Other[1]);
		}

		PMatrix<FRealSingle, 3,2> operator-(const PMatrix<FRealSingle, 3, 2>& Other) const
		{
			return PMatrix<FRealSingle, 3, 2>(
				M[0] - Other.M[0],
				M[1] - Other.M[1],
				M[2] - Other.M[2],
				M[3] - Other.M[3],
				M[4] - Other.M[4],
				M[5] - Other.M[5]);
		}

		PMatrix<FRealSingle, 3, 2> operator+(const PMatrix<FRealSingle, 3, 2>& Other) const
		{
			return PMatrix<FRealSingle, 3, 2>(
				M[0] + Other.M[0],
				M[1] + Other.M[1],
				M[2] + Other.M[2],
				M[3] + Other.M[3],
				M[4] + Other.M[4],
				M[5] + Other.M[5]);
		}

		friend PMatrix<FRealSingle, 3, 2> operator*(const FRealSingle OtherF, const PMatrix<FRealSingle, 3, 2>& OtherM)
		{
			return PMatrix<FRealSingle, 3, 2>(
				OtherF * OtherM.M[0],
				OtherF * OtherM.M[1],
				OtherF * OtherM.M[2],
				OtherF * OtherM.M[3],
				OtherF * OtherM.M[4],
				OtherF * OtherM.M[5]);
		}

		
	};

	template<>
	class PMatrix<FReal, 2, 2>
	{
	public:
		FReal M[4];

		PMatrix(const FReal x00, const FReal x10, const FReal x01, const FReal x11)
		{
			M[0] = x00;
			M[1] = x10;
			M[2] = x01;
			M[3] = x11;
		}

		PMatrix(const FReal x00, const FReal x10, const FReal x11)
		{
			M[0] = x00;
			M[1] = x10;
			M[2] = x10;
			M[3] = x11;
		}

		PMatrix<FReal, 2, 2> SubtractDiagonal(const FReal Scalar) const
		{
			return PMatrix<FReal, 2, 2>(
			    M[0] - Scalar,
			    M[1],
			    M[2],
			    M[3] - Scalar);
		}

		TVector<FReal, 2> TransformPosition(const TVector<FReal, 2>& Other) const
		{
			return TVector<FReal, 2>(
			    M[0] * Other.X + M[2] * Other.Y,
			    M[1] * Other.X + M[3] * Other.Y);
		}

		PMatrix<FReal, 2, 2> Inverse() const
		{
			const FReal OneOverDeterminant = static_cast<FReal>(1.0) / (M[0] * M[3] - M[1] * M[2]);
			return PMatrix<FReal, 2, 2>(
			    OneOverDeterminant * M[3],
			    -OneOverDeterminant * M[1],
			    -OneOverDeterminant * M[2],
			    OneOverDeterminant * M[0]);
		}

		PMatrix<FReal, 2, 2> GetTransposed() const
		{
			return PMatrix<FReal, 2, 2>(M[0], M[2], M[1], M[3]);
		}
	};

	template<>
	class PMatrix<FRealSingle, 2, 2>
	{
	public:
		FRealSingle M[4];

		PMatrix(const FRealSingle x00, const FRealSingle x10, const FRealSingle x01, const FRealSingle x11)
		{
			M[0] = x00;
			M[1] = x10;
			M[2] = x01;
			M[3] = x11;
		}

		PMatrix(const FRealSingle x00, const FRealSingle x10, const FRealSingle x11)
		{
			M[0] = x00;
			M[1] = x10;
			M[2] = x10;
			M[3] = x11;
		}

		PMatrix<FRealSingle, 2, 2> SubtractDiagonal(const FRealSingle Scalar) const
		{
			return PMatrix<FRealSingle, 2, 2>(
				M[0] - Scalar,
				M[1],
				M[2],
				M[3] - Scalar);
		}

		TVector<FRealSingle, 2> TransformPosition(const TVector<FRealSingle, 2>& Other) const
		{
			return TVector<FRealSingle, 2>(
				M[0] * Other.X + M[2] * Other.Y,
				M[1] * Other.X + M[3] * Other.Y);
		}

		PMatrix<FRealSingle, 2, 2> Inverse() const
		{
			const FRealSingle OneOverDeterminant = static_cast<FRealSingle>(1.0) / (M[0] * M[3] - M[1] * M[2]);
			return PMatrix<FRealSingle, 2, 2>(
				OneOverDeterminant * M[3],
				-OneOverDeterminant * M[1],
				-OneOverDeterminant * M[2],
				OneOverDeterminant * M[0]);
		}

		PMatrix<FRealSingle, 2, 2> GetTransposed() const
		{
			return PMatrix<FRealSingle, 2, 2>(M[0], M[2], M[1], M[3]);
		}

		FRealSingle Determinant() const
		{
			return M[0] * M[3] - M[1] * M[2];
		}

		FORCEINLINE FRealSingle GetAt(int32 RowIndex, int32 ColIndex) const
		{
			return M[ColIndex * 2 + RowIndex];
		}

		//This is not column major so no need to invert matrix order when multiply them
		PMatrix<FRealSingle, 2, 2> operator*(const PMatrix<FRealSingle, 2, 2>& Other) const
		{
			return PMatrix<FRealSingle, 2, 2>(
				Other.M[0] * M[0] + Other.M[1] * M[2],
				M[1] * Other.M[0] + M[3] * Other.M[1],
				M[0] * Other.M[2] + M[2] * Other.M[3],
				M[1] * Other.M[2] + M[3] * Other.M[3]);
		}

		friend PMatrix<FRealSingle, 3, 2> operator*(const PMatrix<FRealSingle, 3, 2>& First, const PMatrix<FRealSingle, 2, 2>& Other)
		{
			return PMatrix<FRealSingle, 3, 2>(
				First.M[0] * Other.M[0] + First.M[3] * Other.M[1],
				First.M[1] * Other.M[0] + First.M[4] * Other.M[1],
				First.M[2] * Other.M[0] + First.M[5] * Other.M[1],
				First.M[0] * Other.M[2] + First.M[3] * Other.M[3],
				First.M[1] * Other.M[2] + First.M[4] * Other.M[3],
				First.M[2] * Other.M[2] + First.M[5] * Other.M[3]);
		}
	};

	template<>
	class PMatrix<FRealSingle, 4, 4> : public UE::Math::TMatrix<FRealSingle>
	{
	public:
		PMatrix()
		    : UE::Math::TMatrix<FRealSingle>() {}
		PMatrix(const FRealSingle x00, const FRealSingle x10, const FRealSingle x20, const FRealSingle x30, const FRealSingle x01, const FRealSingle x11, const FRealSingle x21, const FRealSingle x31, const FRealSingle x02, const FRealSingle x12, const FRealSingle x22, const FRealSingle x32, const FRealSingle x03, const FRealSingle x13, const FRealSingle x23, const FRealSingle x33)
		    : UE::Math::TMatrix<FRealSingle>()
		{
			M[0][0] = x00;
			M[1][0] = x10;
			M[2][0] = x20;
			M[3][0] = x30;
			M[0][1] = x01;
			M[1][1] = x11;
			M[2][1] = x21;
			M[3][1] = x31;
			M[0][2] = x02;
			M[1][2] = x12;
			M[2][2] = x22;
			M[3][2] = x32;
			M[0][3] = x03;
			M[1][3] = x13;
			M[2][3] = x23;
			M[3][3] = x33;
		}
		PMatrix(const UE::Math::TMatrix<FRealSingle>& Matrix)
		    : UE::Math::TMatrix<FRealSingle>(Matrix)
		{
		}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
		Vector<FRealSingle, 4> operator*(const Vector<Chaos::FRealSingle, 4>& Other)
		{
			return Vector<Chaos::FRealSingle, 4>(
			    M[0][0] * Other[0] + M[0][1] * Other[1] + M[0][2] * Other[2] + M[0][3] * Other[3],
			    M[1][0] * Other[0] + M[1][1] * Other[1] + M[1][2] * Other[2] + M[1][3] * Other[3],
			    M[2][0] * Other[0] + M[2][1] * Other[1] + M[2][2] * Other[2] + M[2][3] * Other[3],
			    M[3][0] * Other[0] + M[3][1] * Other[1] + M[3][2] * Other[2] + M[3][3] * Other[3]);
		}
#endif

		PMatrix<FRealSingle, 4, 4> operator*(const PMatrix<FRealSingle, 4, 4>& Other) const
		{
			return static_cast<const UE::Math::TMatrix<FRealSingle>*>(this)->operator*(static_cast<const UE::Math::TMatrix<FRealSingle>&>(Other));
		}
	};

	template<>
	class PMatrix<FRealDouble, 4, 4> : public UE::Math::TMatrix<FRealDouble>
	{
	public:
		PMatrix()
		    : UE::Math::TMatrix<FRealDouble>() {}
		PMatrix(const FRealDouble x00, const FRealDouble x10, const FRealDouble x20, const FRealDouble x30, const FRealDouble x01, const FRealDouble x11, const FRealDouble x21, const FRealDouble x31, const FRealDouble x02, const FRealDouble x12, const FRealDouble x22, const FRealDouble x32, const FRealDouble x03, const FRealDouble x13, const FRealDouble x23, const FRealDouble x33)
		    : UE::Math::TMatrix<FRealDouble>()
		{
			M[0][0] = x00;
			M[1][0] = x10;
			M[2][0] = x20;
			M[3][0] = x30;
			M[0][1] = x01;
			M[1][1] = x11;
			M[2][1] = x21;
			M[3][1] = x31;
			M[0][2] = x02;
			M[1][2] = x12;
			M[2][2] = x22;
			M[3][2] = x32;
			M[0][3] = x03;
			M[1][3] = x13;
			M[2][3] = x23;
			M[3][3] = x33;
		}
		PMatrix(const UE::Math::TMatrix<FRealDouble>& Matrix)
		    : UE::Math::TMatrix<FRealDouble>(Matrix)
		{
		}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
		Vector<FRealDouble, 4> operator*(const Vector<Chaos::FRealDouble, 4>& Other)
		{
			return Vector<Chaos::FRealDouble, 4>(
			    M[0][0] * Other[0] + M[0][1] * Other[1] + M[0][2] * Other[2] + M[0][3] * Other[3],
			    M[1][0] * Other[0] + M[1][1] * Other[1] + M[1][2] * Other[2] + M[1][3] * Other[3],
			    M[2][0] * Other[0] + M[2][1] * Other[1] + M[2][2] * Other[2] + M[2][3] * Other[3],
			    M[3][0] * Other[0] + M[3][1] * Other[1] + M[3][2] * Other[2] + M[3][3] * Other[3]);
		}
#endif

		PMatrix<FRealDouble, 4, 4> operator*(const PMatrix<FRealDouble, 4, 4>& Other) const
		{
			return static_cast<const UE::Math::TMatrix<FRealDouble>*>(this)->operator*(static_cast<const UE::Math::TMatrix<FRealDouble>&>(Other));
		}
	};

	// TODO(mlentine): Do not use 4x4 matrix for 3x3 implementation
	template<>
	class PMatrix<FRealDouble, 3, 3> : public UE::Math::TMatrix<FRealDouble>
	{
	public:
		PMatrix()
		    : UE::Math::TMatrix<FRealDouble>() {}
		PMatrix(UE::Math::TMatrix<FRealDouble>&& Other)
		    : UE::Math::TMatrix<FRealDouble>(MoveTemp(Other)) {}
		PMatrix(const UE::Math::TMatrix<FRealSingle>& Other)
		    : UE::Math::TMatrix<FRealDouble>((UE::Math::TMatrix<FRealDouble>)Other) {}
		PMatrix(const UE::Math::TMatrix<FRealDouble>& Other)
			: UE::Math::TMatrix<FRealDouble>((UE::Math::TMatrix<FRealDouble>)Other) {}
		PMatrix(const FRealDouble x00, const FRealDouble x11, const FRealDouble x22)
		    : UE::Math::TMatrix<FRealDouble>()
		{
			M[0][0] = x00;
			M[1][0] = 0;
			M[2][0] = 0;
			M[0][1] = 0;
			M[1][1] = x11;
			M[2][1] = 0;
			M[0][2] = 0;
			M[1][2] = 0;
			M[2][2] = x22;
			// Fill in the remainder with reasonable values.
			M[3][0] = 0;
			M[3][1] = 0;
			M[3][2] = 0;
			M[3][3] = 1;
			M[0][3] = 0;
			M[1][3] = 0;
			M[2][3] = 0;
		}
		explicit PMatrix(const TVector<FRealDouble, 3>& Vector)
			: UE::Math::TMatrix<FReal>()
		{
			M[0][0] = Vector[0];
			M[1][0] = 0;
			M[2][0] = 0;
			M[0][1] = 0;
			M[1][1] = Vector[1];
			M[2][1] = 0;
			M[0][2] = 0;
			M[1][2] = 0;
			M[2][2] = Vector[2];
			M[3][0] = 0;
			M[3][1] = 0;
			M[3][2] = 0;
			M[3][3] = 1;
			M[0][3] = 0;
			M[1][3] = 0;
			M[2][3] = 0;
		}
		PMatrix(const FRealDouble x00, const FRealDouble x10, const FRealDouble x20, const FRealDouble x11, const FRealDouble x21, const FRealDouble x22)
		    : UE::Math::TMatrix<FRealDouble>()
		{
			M[0][0] = x00;
			M[1][0] = x10;
			M[2][0] = x20;
			M[0][1] = x10;
			M[1][1] = x11;
			M[2][1] = x21;
			M[0][2] = x20;
			M[1][2] = x21;
			M[2][2] = x22;
			// Fill in the remainder with reasonable values.
			M[3][0] = 0;
			M[3][1] = 0;
			M[3][2] = 0;
			M[3][3] = 1;
			M[0][3] = 0;
			M[1][3] = 0;
			M[2][3] = 0;
		}
		PMatrix(const FRealDouble x00, const FRealDouble x10, const FRealDouble x20, const FRealDouble x01, const FRealDouble x11, const FRealDouble x21, const FRealDouble x02, const FRealDouble x12, const FRealDouble x22)
		    : UE::Math::TMatrix<FRealDouble>()
		{
			M[0][0] = x00;
			M[1][0] = x10;
			M[2][0] = x20;
			M[0][1] = x01;
			M[1][1] = x11;
			M[2][1] = x21;
			M[0][2] = x02;
			M[1][2] = x12;
			M[2][2] = x22;
			// Fill in the remainder with reasonable values.
			M[3][0] = 0;
			M[3][1] = 0;
			M[3][2] = 0;
			M[3][3] = 1;
			M[0][3] = 0;
			M[1][3] = 0;
			M[2][3] = 0;
		}
		PMatrix(const FRealDouble x)
		    : UE::Math::TMatrix<FRealDouble>()
		{
			M[0][0] = x;
			M[1][0] = x;
			M[2][0] = x;
			M[0][1] = x;
			M[1][1] = x;
			M[2][1] = x;
			M[0][2] = x;
			M[1][2] = x;
			M[2][2] = x;
			// Fill in the remainder with reasonable values.
			M[3][0] = 0;
			M[3][1] = 0;
			M[3][2] = 0;
			M[3][3] = 1;
			M[0][3] = 0;
			M[1][3] = 0;
			M[2][3] = 0;
		}
		PMatrix(const TVector<FRealDouble, 3>& C1, const TVector<FRealDouble, 3>& C2, const TVector<FRealDouble, 3>& C3)
		{
			M[0][0] = C1.X;
			M[1][0] = C1.Y;
			M[2][0] = C1.Z;
			M[0][1] = C2.X;
			M[1][1] = C2.Y;
			M[2][1] = C2.Z;
			M[0][2] = C3.X;
			M[1][2] = C3.Y;
			M[2][2] = C3.Z;
			// Fill in the remainder with reasonable values.
			M[3][0] = 0;
			M[3][1] = 0;
			M[3][2] = 0;
			M[3][3] = 1;
			M[0][3] = 0;
			M[1][3] = 0;
			M[2][3] = 0;
		}

		PMatrix<FRealDouble, 3, 3> GetTransposed() const
		{
			return PMatrix<FRealDouble, 3, 3>(M[0][0], M[0][1], M[0][2], M[1][0], M[1][1], M[1][2], M[2][0], M[2][1], M[2][2]);
		}
		FRealDouble Determinant() const
		{
			return M[0][0] * (M[1][1] * M[2][2] - M[1][2] * M[2][1]) - M[0][1] * (M[1][0] * M[2][2] - M[1][2] * M[2][0]) + M[0][2] * (M[1][0] * M[2][1] - M[1][1] * M[2][0]);
		}
		PMatrix<FRealDouble, 3, 3>& operator+=(const PMatrix<FRealDouble, 3, 3>& Other)
		{
			M[0][0] += Other.M[0][0];
			M[0][1] += Other.M[0][1];
			M[0][2] += Other.M[0][2];
			M[1][0] += Other.M[1][0];
			M[1][1] += Other.M[1][1];
			M[1][2] += Other.M[1][2];
			M[2][0] += Other.M[2][0];
			M[2][1] += Other.M[2][1];
			M[2][2] += Other.M[2][2];
			return *this;
		}

		// TDOD(mlentine): This should really be a vector multiply and sum for each entry using sse
		TVector<FRealDouble, 3> operator*(const TVector<FRealDouble, 3>& Other) const
		{
			return TVector<FRealDouble, 3>(
			    M[0][0] * Other[0] + M[0][1] * Other[1] + M[0][2] * Other[2],
			    M[1][0] * Other[0] + M[1][1] * Other[1] + M[1][2] * Other[2],
			    M[2][0] * Other[0] + M[2][1] * Other[1] + M[2][2] * Other[2]);
		}
		PMatrix<FRealDouble, 3, 3> operator+(const PMatrix<FRealDouble, 3, 3>& Other) const
		{
			return PMatrix<FRealDouble, 3, 3>(
				M[0][0] + Other.M[0][0],
				M[1][0] + Other.M[1][0],
				M[2][0] + Other.M[2][0],
				M[0][1] + Other.M[0][1],
				M[1][1] + Other.M[1][1],
				M[2][1] + Other.M[2][1],
				M[0][2] + Other.M[0][2],
				M[1][2] + Other.M[1][2],
				M[2][2] + Other.M[2][2]);
		}
		friend PMatrix<FRealDouble, 3, 3> operator+(const PMatrix<FRealDouble, 3, 3>& Other)
		{
			return Other;
		}
		PMatrix<FRealDouble, 3, 3> operator-(const PMatrix<FRealDouble, 3, 3>& Other) const
		{
			return PMatrix<FRealDouble, 3, 3>(
				M[0][0] - Other.M[0][0],
				M[1][0] - Other.M[1][0],
				M[2][0] - Other.M[2][0],
				M[0][1] - Other.M[0][1],
				M[1][1] - Other.M[1][1],
				M[2][1] - Other.M[2][1],
				M[0][2] - Other.M[0][2],
				M[1][2] - Other.M[1][2],
				M[2][2] - Other.M[2][2]);
		}
		friend PMatrix<FRealDouble, 3, 3> operator-(const PMatrix<FRealDouble, 3, 3>& Other)
		{
			return PMatrix<FRealDouble, 3, 3>(
				-Other.M[0][0],
				-Other.M[1][0],
				-Other.M[2][0],
				-Other.M[0][1],
				-Other.M[1][1],
				-Other.M[2][1],
				-Other.M[0][2],
				-Other.M[1][2],
				-Other.M[2][2]);
		}
		PMatrix<FRealDouble, 3, 3> operator*(const PMatrix<FRealDouble, 3, 3>& Other) const
		{
			return static_cast<const UE::Math::TMatrix<FRealDouble>*>(this)->operator*(static_cast<const UE::Math::TMatrix<FRealDouble>&>(Other));
		}
		// Needs to be overridden because base version multiplies M[3][3]
		PMatrix<FRealDouble, 3, 3> operator*(const FRealDouble Other) const
		{
			return PMatrix<FRealDouble, 3, 3>(
			    M[0][0] * Other,
			    M[1][0] * Other,
			    M[2][0] * Other,
			    M[0][1] * Other,
			    M[1][1] * Other,
			    M[2][1] * Other,
			    M[0][2] * Other,
			    M[1][2] * Other,
			    M[2][2] * Other);
		}
		// Needs to be overridden because base version multiplies M[3][3]
		PMatrix<FRealDouble, 3, 3> operator*=(const FRealDouble Other)
		{
			M[0][0] *= Other;
			M[0][1] *= Other;
			M[0][2] *= Other;
			M[1][0] *= Other;
			M[1][1] *= Other;
			M[1][2] *= Other;
			M[2][0] *= Other;
			M[2][1] *= Other;
			M[2][2] *= Other;
			return *this;
		}
		friend PMatrix<FRealDouble, 3, 3> operator*(const FRealDouble OtherF, const PMatrix<FRealDouble, 3, 3>& OtherM)
		{
			return OtherM * OtherF;
		}
		PMatrix<FRealDouble, 3, 2> operator*(const PMatrix<FRealDouble, 3, 2>& Other) const
		{
			return PMatrix<FRealDouble, 3, 2>(
			    M[0][0] * Other.M[0] + M[0][1] * Other.M[1] + M[0][2] * Other.M[2],
			    M[1][0] * Other.M[0] + M[1][1] * Other.M[1] + M[1][2] * Other.M[2],
			    M[2][0] * Other.M[0] + M[2][1] * Other.M[1] + M[2][2] * Other.M[2],
			    M[0][0] * Other.M[3] + M[0][1] * Other.M[4] + M[0][2] * Other.M[5],
			    M[1][0] * Other.M[3] + M[1][1] * Other.M[4] + M[1][2] * Other.M[5],
			    M[2][0] * Other.M[3] + M[2][1] * Other.M[4] + M[2][2] * Other.M[5]);
		}
		PMatrix<FRealDouble, 3, 3> SubtractDiagonal(const FRealDouble Scalar) const
		{
			return PMatrix<FRealDouble, 3, 3>(
			    M[0][0] - Scalar,
			    M[1][0],
			    M[2][0],
			    M[0][1],
			    M[1][1] - Scalar,
			    M[2][1],
			    M[0][2],
			    M[1][2],
			    M[2][2] - Scalar);
		}
		PMatrix<FRealDouble, 3, 3> SymmetricCofactorMatrix() const
		{
			return PMatrix<FRealDouble, 3, 3>(
			    M[1][1] * M[2][2] - M[2][1] * M[2][1],
			    M[2][1] * M[2][0] - M[1][0] * M[2][2],
			    M[1][0] * M[2][1] - M[1][1] * M[2][0],
			    M[0][0] * M[2][2] - M[2][0] * M[2][0],
			    M[1][0] * M[2][0] - M[0][0] * M[2][1],
			    M[0][0] * M[1][1] - M[1][0] * M[1][0]);
		}
		TVector<FRealDouble, 3> LargestColumnNormalized() const
		{
			FRealDouble m10 = M[1][0] * M[1][0];
			FRealDouble m20 = M[2][0] * M[2][0];
			FRealDouble m21 = M[2][1] * M[2][1];
			FRealDouble c0 = M[0][0] * M[0][0] + m10 + m20;
			FRealDouble c1 = m10 + M[1][1] * M[1][1] + m21;
			FRealDouble c2 = m20 + m21 + M[2][2] * M[2][2];
			if (c0 > c1 && c0 > c2)
			{
				return TVector<FRealDouble, 3>(M[0][0], M[1][0], M[2][0]) / FMath::Sqrt(c0);
			}
			if (c1 > c2)
			{
				return TVector<FRealDouble, 3>(M[1][0], M[1][1], M[2][1]) / FMath::Sqrt(c1);
			}
			if (c2 > 0)
			{
				return TVector<FRealDouble, 3>(M[2][0], M[2][1], M[2][2]) / FMath::Sqrt(c2);
			}
			return TVector<FRealDouble, 3>(1, 0, 0);
		}

		/**
		 * Get the specified axis (0-indexed, X,Y,Z).
		 * @note: we are treating matrices as column major, so axis elements are sequential in memory
		 */
		FORCEINLINE TVector<FRealDouble, 3> GetAxis(int32 AxisIndex) const
		{
			return TVector<FRealDouble, 3>(M[AxisIndex][0], M[AxisIndex][1], M[AxisIndex][2]);
		}

		/**
		 * Set the specified axis (0-indexed, X,Y,Z).
		 * @note: we are treating matrices as column major, so axis elements are sequential in memory
		 */
		FORCEINLINE void SetAxis(int32 AxisIndex, const TVector<FRealDouble, 3>& Axis)
		{
			M[AxisIndex][0] = Axis.X;
			M[AxisIndex][1] = Axis.Y;
			M[AxisIndex][2] = Axis.Z;
			M[AxisIndex][3] = 0;
		}

		/**
		 * Get the specified row (0-indexed, X,Y,Z).
		 * @note: we are treating matrices as column major, so rows are not sequential in memory
		 * @seealso GetAxis, GetColumn
		 */
		FORCEINLINE TVector<FRealDouble, 3> GetRow(int32 RowIndex) const
		{
			return TVector<FRealDouble, 3>(M[0][RowIndex], M[1][RowIndex], M[2][RowIndex]);
		}

		/**
		 * Set the specified row.
		 * @note: we are treating matrices as column major, so axis elements are sequential in memory
		 * @seealso SetAxis, SetColumn
		 */
		FORCEINLINE void SetRow(int32 RowIndex, const TVector<FRealDouble, 3>& V)
		{
			M[0][RowIndex] = V.X;
			M[1][RowIndex] = V.Y;
			M[2][RowIndex] = V.Z;
			M[3][RowIndex] = 0;
		}

		/**
		 * Get the specified column (0-indexed, X,Y,Z). Equivalent to GetAxis.
		 * @note: we are treating matrices as column major, so columns are sequential in memory
		 * @seealso GetAxis, GetRow
		 */
		FORCEINLINE TVector<FRealDouble, 3> GetColumn(int32 ColumnIndex) const
		{
			return GetAxis(ColumnIndex);
		}

		/**
		 * Set the specified column. Equivalent to SetAxis.
		 * @note: we are treating matrices as column major, so axis elements are sequential in memory
		 * @seealso SetAxis, SetRow
		 */
		FORCEINLINE void SetColumn(int32 ColumnIndex, const TVector<FRealDouble, 3>& V)
		{
			SetAxis(ColumnIndex, V);
		}

		/**
		 * Get the diagonal elements as a vector.
		 */
		FORCEINLINE TVector<FRealDouble, 3> GetDiagonal() const
		{
			return TVector<FRealDouble, 3>(M[0][0], M[1][1], M[2][2]);
		}

		FORCEINLINE FRealDouble GetAt(int32 RowIndex, int32 ColIndex) const
		{
			return M[ColIndex][RowIndex];
		}

		FORCEINLINE void SetAt(int32 RowIndex, int32 ColIndex, FRealDouble V)
		{
			M[ColIndex][RowIndex] = V;
		}

		/**
		 * Return a diagonal matrix with the specified elements
		 */
		static PMatrix<FRealDouble, 3, 3> FromDiagonal(const TVector<FRealDouble, 3>& D)
		{
			return PMatrix<FRealDouble, 3, 3>(D.X, D.Y, D.Z);
		}

#if COMPILE_WITHOUT_UNREAL_SUPPORT
		// TODO(mlentine): Document which one is row and which one is column
		PMatrix<FRealDouble, 3, 3> operator*(const PMatrix<FRealDouble, 3, 3>& Other)
		{
			return PMatrix<FRealDouble, 3, 3>(
			    M[0][0] * Other.M[0][0] + M[0][1] * Other.M[1][0] + M[0][2] * Other.M[2][0],
			    M[1][0] * Other.M[0][0] + M[1][1] * Other.M[1][0] + M[1][2] * Other.M[2][0],
			    M[2][0] * Other.M[0][0] + M[2][1] * Other.M[1][0] + M[2][2] * Other.M[2][0],
			    M[0][0] * Other.M[0][1] + M[0][1] * Other.M[1][1] + M[0][2] * Other.M[2][1],
			    M[1][0] * Other.M[0][1] + M[1][1] * Other.M[1][1] + M[1][2] * Other.M[2][1],
			    M[2][0] * Other.M[0][1] + M[2][1] * Other.M[1][1] + M[2][2] * Other.M[2][1],
			    M[0][0] * Other.M[0][2] + M[0][1] * Other.M[1][2] + M[0][2] * Other.M[2][2],
			    M[1][0] * Other.M[0][2] + M[1][1] * Other.M[1][2] + M[1][2] * Other.M[2][2],
			    M[2][0] * Other.M[0][2] + M[2][1] * Other.M[1][2] + M[2][2] * Other.M[2][2]);
		}
#endif
		inline bool Equals(const PMatrix<FRealDouble, 3, 3>& Other, FRealDouble Tolerance = UE_KINDA_SMALL_NUMBER) const
		{
			return true
				&& (FMath::Abs(Other.M[0][0] - M[0][0]) <= Tolerance)
				&& (FMath::Abs(Other.M[0][1] - M[0][1]) <= Tolerance)
				&& (FMath::Abs(Other.M[0][2] - M[0][2]) <= Tolerance)
				&& (FMath::Abs(Other.M[1][0] - M[1][0]) <= Tolerance)
				&& (FMath::Abs(Other.M[1][1] - M[1][1]) <= Tolerance)
				&& (FMath::Abs(Other.M[1][2] - M[1][2]) <= Tolerance)
				&& (FMath::Abs(Other.M[2][0] - M[2][0]) <= Tolerance)
				&& (FMath::Abs(Other.M[2][1] - M[2][1]) <= Tolerance)
				&& (FMath::Abs(Other.M[2][2] - M[2][2]) <= Tolerance);
		}

		// M[i][j] = x[i] * y[j]
		static PMatrix<FRealDouble, 3, 3> OuterProduct(const TVector<FRealDouble, 3>& X, const TVector<FRealDouble, 3>& Y)
		{
			return PMatrix<FRealDouble, 3, 3>(
				X[0] * Y[0],
				X[1] * Y[0],
				X[2] * Y[0],
				X[0] * Y[1],
				X[1] * Y[1],
				X[2] * Y[1],
				X[0] * Y[2],
				X[1] * Y[2],
				X[2] * Y[2]);
		}

		static CHAOSCORE_API const PMatrix<FRealDouble, 3, 3> Zero;
		static CHAOSCORE_API const PMatrix<FRealDouble, 3, 3> Identity;
	};

	template<>
	class PMatrix<FRealSingle, 3, 3> : public UE::Math::TMatrix<FRealSingle>
	{
	public:
		PMatrix()
			: UE::Math::TMatrix<FRealSingle>() {}
		PMatrix(UE::Math::TMatrix<FRealSingle>&& Other)
			: UE::Math::TMatrix<FRealSingle>(MoveTemp(Other)) {}
		PMatrix(const UE::Math::TMatrix<FRealSingle>& Other)
			: UE::Math::TMatrix<FRealSingle>((UE::Math::TMatrix<FRealSingle>)Other) {}
		PMatrix(const UE::Math::TMatrix<FRealDouble>& Other)
			: UE::Math::TMatrix<FRealSingle>((UE::Math::TMatrix<FRealSingle>)Other) {}
		PMatrix(const FRealSingle x00, const FRealSingle x11, const FRealSingle x22)
			: UE::Math::TMatrix<FRealSingle>()
		{
			M[0][0] = x00;
			M[1][0] = 0;
			M[2][0] = 0;
			M[0][1] = 0;
			M[1][1] = x11;
			M[2][1] = 0;
			M[0][2] = 0;
			M[1][2] = 0;
			M[2][2] = x22;
			// Fill in the remainder with reasonable values.
			M[3][0] = 0;
			M[3][1] = 0;
			M[3][2] = 0;
			M[3][3] = 1;
			M[0][3] = 0;
			M[1][3] = 0;
			M[2][3] = 0;
		}
		explicit PMatrix(const TVector<FRealSingle, 3>& Vector)
			: UE::Math::TMatrix<FReal>()
		{
			M[0][0] = Vector[0];
			M[1][0] = 0;
			M[2][0] = 0;
			M[0][1] = 0;
			M[1][1] = Vector[1];
			M[2][1] = 0;
			M[0][2] = 0;
			M[1][2] = 0;
			M[2][2] = Vector[2];
			M[3][0] = 0;
			M[3][1] = 0;
			M[3][2] = 0;
			M[3][3] = 1;
			M[0][3] = 0;
			M[1][3] = 0;
			M[2][3] = 0;
		}
		PMatrix(const FRealSingle x00, const FRealSingle x10, const FRealSingle x20, const FRealSingle x11, const FRealSingle x21, const FRealSingle x22)
			: UE::Math::TMatrix<FRealSingle>()
		{
			M[0][0] = x00;
			M[1][0] = x10;
			M[2][0] = x20;
			M[0][1] = x10;
			M[1][1] = x11;
			M[2][1] = x21;
			M[0][2] = x20;
			M[1][2] = x21;
			M[2][2] = x22;
			// Fill in the remainder with reasonable values.
			M[3][0] = 0;
			M[3][1] = 0;
			M[3][2] = 0;
			M[3][3] = 1;
			M[0][3] = 0;
			M[1][3] = 0;
			M[2][3] = 0;
		}
		PMatrix(const FRealSingle x00, const FRealSingle x10, const FRealSingle x20, const FRealSingle x01, const FRealSingle x11, const FRealSingle x21, const FRealSingle x02, const FRealSingle x12, const FRealSingle x22)
			: UE::Math::TMatrix<FRealSingle>()
		{
			M[0][0] = x00;
			M[1][0] = x10;
			M[2][0] = x20;
			M[0][1] = x01;
			M[1][1] = x11;
			M[2][1] = x21;
			M[0][2] = x02;
			M[1][2] = x12;
			M[2][2] = x22;
			// Fill in the remainder with reasonable values.
			M[3][0] = 0;
			M[3][1] = 0;
			M[3][2] = 0;
			M[3][3] = 1;
			M[0][3] = 0;
			M[1][3] = 0;
			M[2][3] = 0;
		}
		PMatrix(const FRealSingle x)
			: UE::Math::TMatrix<FRealSingle>()
		{
			M[0][0] = x;
			M[1][0] = x;
			M[2][0] = x;
			M[0][1] = x;
			M[1][1] = x;
			M[2][1] = x;
			M[0][2] = x;
			M[1][2] = x;
			M[2][2] = x;
			// Fill in the remainder with reasonable values.
			M[3][0] = 0;
			M[3][1] = 0;
			M[3][2] = 0;
			M[3][3] = 1;
			M[0][3] = 0;
			M[1][3] = 0;
			M[2][3] = 0;
		}
		PMatrix(const TVector<FRealSingle, 3>& C1, const TVector<FRealSingle, 3>& C2, const TVector<FRealSingle, 3>& C3)
		{
			M[0][0] = C1.X;
			M[1][0] = C1.Y;
			M[2][0] = C1.Z;
			M[0][1] = C2.X;
			M[1][1] = C2.Y;
			M[2][1] = C2.Z;
			M[0][2] = C3.X;
			M[1][2] = C3.Y;
			M[2][2] = C3.Z;
			// Fill in the remainder with reasonable values.
			M[3][0] = 0;
			M[3][1] = 0;
			M[3][2] = 0;
			M[3][3] = 1;
			M[0][3] = 0;
			M[1][3] = 0;
			M[2][3] = 0;
		}

		PMatrix<FRealSingle, 3, 3> GetTransposed() const
		{
			return PMatrix<FRealSingle, 3, 3>(M[0][0], M[0][1], M[0][2], M[1][0], M[1][1], M[1][2], M[2][0], M[2][1], M[2][2]);
		}
		FRealSingle Determinant() const
		{
			return M[0][0] * (M[1][1] * M[2][2] - M[1][2] * M[2][1]) - M[0][1] * (M[1][0] * M[2][2] - M[1][2] * M[2][0]) + M[0][2] * (M[1][0] * M[2][1] - M[1][1] * M[2][0]);
		}
		PMatrix<FRealSingle, 3, 3>& operator+=(const PMatrix<FRealSingle, 3, 3>& Other)
		{
			M[0][0] += Other.M[0][0];
			M[0][1] += Other.M[0][1];
			M[0][2] += Other.M[0][2];
			M[1][0] += Other.M[1][0];
			M[1][1] += Other.M[1][1];
			M[1][2] += Other.M[1][2];
			M[2][0] += Other.M[2][0];
			M[2][1] += Other.M[2][1];
			M[2][2] += Other.M[2][2];
			return *this;
		}

		// TDOD(mlentine): This should really be a vector multiply and sum for each entry using sse
		TVector<FRealSingle, 3> operator*(const TVector<FRealSingle, 3>& Other) const
		{
			return TVector<FRealSingle, 3>(
				M[0][0] * Other[0] + M[0][1] * Other[1] + M[0][2] * Other[2],
				M[1][0] * Other[0] + M[1][1] * Other[1] + M[1][2] * Other[2],
				M[2][0] * Other[0] + M[2][1] * Other[1] + M[2][2] * Other[2]);
		}
		PMatrix<FRealSingle, 3, 3> operator+(const PMatrix<FRealSingle, 3, 3>& Other) const
		{
			return PMatrix<FRealSingle, 3, 3>(
				M[0][0] + Other.M[0][0],
				M[1][0] + Other.M[1][0],
				M[2][0] + Other.M[2][0],
				M[0][1] + Other.M[0][1],
				M[1][1] + Other.M[1][1],
				M[2][1] + Other.M[2][1],
				M[0][2] + Other.M[0][2],
				M[1][2] + Other.M[1][2],
				M[2][2] + Other.M[2][2]);
		}
		friend PMatrix<FRealSingle, 3, 3> operator+(const PMatrix<FRealSingle, 3, 3>& Other)
		{
			return Other;
		}
		PMatrix<FRealSingle, 3, 3> operator-(const PMatrix<FRealSingle, 3, 3>& Other) const
		{
			return PMatrix<FRealSingle, 3, 3>(
				M[0][0] - Other.M[0][0],
				M[1][0] - Other.M[1][0],
				M[2][0] - Other.M[2][0],
				M[0][1] - Other.M[0][1],
				M[1][1] - Other.M[1][1],
				M[2][1] - Other.M[2][1],
				M[0][2] - Other.M[0][2],
				M[1][2] - Other.M[1][2],
				M[2][2] - Other.M[2][2]);
		}
		friend PMatrix<FRealSingle, 3, 3> operator-(const PMatrix<FRealSingle, 3, 3>& Other)
		{
			return PMatrix<FRealSingle, 3, 3>(
				-Other.M[0][0],
				-Other.M[1][0],
				-Other.M[2][0],
				-Other.M[0][1],
				-Other.M[1][1],
				-Other.M[2][1],
				-Other.M[0][2],
				-Other.M[1][2],
				-Other.M[2][2]);
		}
		PMatrix<FRealSingle, 3, 3> operator*(const PMatrix<FRealSingle, 3, 3>& Other) const
		{
			return static_cast<const UE::Math::TMatrix<FRealSingle>*>(this)->operator*(static_cast<const UE::Math::TMatrix<FRealSingle>&>(Other));
		}
		// Needs to be overridden because base version multiplies M[3][3]
		PMatrix<FRealSingle, 3, 3> operator*(const FRealSingle Other) const
		{
			return PMatrix<FRealSingle, 3, 3>(
				M[0][0] * Other,
				M[1][0] * Other,
				M[2][0] * Other,
				M[0][1] * Other,
				M[1][1] * Other,
				M[2][1] * Other,
				M[0][2] * Other,
				M[1][2] * Other,
				M[2][2] * Other);
		}
		// Needs to be overridden because base version multiplies M[3][3]
		PMatrix<FRealSingle, 3, 3> operator*=(const FRealSingle Other)
		{
			M[0][0] *= Other;
			M[0][1] *= Other;
			M[0][2] *= Other;
			M[1][0] *= Other;
			M[1][1] *= Other;
			M[1][2] *= Other;
			M[2][0] *= Other;
			M[2][1] *= Other;
			M[2][2] *= Other;
			return *this;
		}
		friend PMatrix<FRealSingle, 3, 3> operator*(const FRealSingle OtherF, const PMatrix<FRealSingle, 3, 3>& OtherM)
		{
			return OtherM * OtherF;
		}
		PMatrix<FRealSingle, 3, 2> operator*(const PMatrix<FRealSingle, 3, 2>& Other) const
		{
			return PMatrix<FRealSingle, 3, 2>(
				M[0][0] * Other.M[0] + M[0][1] * Other.M[1] + M[0][2] * Other.M[2],
				M[1][0] * Other.M[0] + M[1][1] * Other.M[1] + M[1][2] * Other.M[2],
				M[2][0] * Other.M[0] + M[2][1] * Other.M[1] + M[2][2] * Other.M[2],
				M[0][0] * Other.M[3] + M[0][1] * Other.M[4] + M[0][2] * Other.M[5],
				M[1][0] * Other.M[3] + M[1][1] * Other.M[4] + M[1][2] * Other.M[5],
				M[2][0] * Other.M[3] + M[2][1] * Other.M[4] + M[2][2] * Other.M[5]);
		}
		PMatrix<FRealSingle, 3, 3> SubtractDiagonal(const FRealSingle Scalar) const
		{
			return PMatrix<FRealSingle, 3, 3>(
				M[0][0] - Scalar,
				M[1][0],
				M[2][0],
				M[0][1],
				M[1][1] - Scalar,
				M[2][1],
				M[0][2],
				M[1][2],
				M[2][2] - Scalar);
		}
		PMatrix<FRealSingle, 3, 3> SymmetricCofactorMatrix() const
		{
			return PMatrix<FRealSingle, 3, 3>(
				M[1][1] * M[2][2] - M[2][1] * M[2][1],
				M[2][1] * M[2][0] - M[1][0] * M[2][2],
				M[1][0] * M[2][1] - M[1][1] * M[2][0],
				M[0][0] * M[2][2] - M[2][0] * M[2][0],
				M[1][0] * M[2][0] - M[0][0] * M[2][1],
				M[0][0] * M[1][1] - M[1][0] * M[1][0]);
		}
		TVector<FRealSingle, 3> LargestColumnNormalized() const
		{
			FRealSingle m10 = M[1][0] * M[1][0];
			FRealSingle m20 = M[2][0] * M[2][0];
			FRealSingle m21 = M[2][1] * M[2][1];
			FRealSingle c0 = M[0][0] * M[0][0] + m10 + m20;
			FRealSingle c1 = m10 + M[1][1] * M[1][1] + m21;
			FRealSingle c2 = m20 + m21 + M[2][2] * M[2][2];
			if (c0 > c1 && c0 > c2)
			{
				return TVector<FRealSingle, 3>(M[0][0], M[1][0], M[2][0]) / FMath::Sqrt(c0);
			}
			if (c1 > c2)
			{
				return TVector<FRealSingle, 3>(M[1][0], M[1][1], M[2][1]) / FMath::Sqrt(c1);
			}
			if (c2 > 0)
			{
				return TVector<FRealSingle, 3>(M[2][0], M[2][1], M[2][2]) / FMath::Sqrt(c2);
			}
			return TVector<FRealSingle, 3>(1, 0, 0);
		}

		/**
		 * Get the specified axis (0-indexed, X,Y,Z).
		 * @note: we are treating matrices as column major, so axis elements are sequential in memory
		 */
		FORCEINLINE TVector<FRealSingle, 3> GetAxis(int32 AxisIndex) const
		{
			return TVector<FRealSingle, 3>(M[AxisIndex][0], M[AxisIndex][1], M[AxisIndex][2]);
		}

		/**
		 * Set the specified axis (0-indexed, X,Y,Z).
		 * @note: we are treating matrices as column major, so axis elements are sequential in memory
		 */
		FORCEINLINE void SetAxis(int32 AxisIndex, const TVector<FRealSingle, 3>& Axis)
		{
			M[AxisIndex][0] = Axis.X;
			M[AxisIndex][1] = Axis.Y;
			M[AxisIndex][2] = Axis.Z;
			M[AxisIndex][3] = 0;
		}

		/**
		 * Get the specified row (0-indexed, X,Y,Z).
		 * @note: we are treating matrices as column major, so rows are not sequential in memory
		 * @seealso GetAxis, GetColumn
		 */
		FORCEINLINE TVector<FRealSingle, 3> GetRow(int32 RowIndex) const
		{
			return TVector<FRealSingle, 3>(M[0][RowIndex], M[1][RowIndex], M[2][RowIndex]);
		}

		/**
		 * Set the specified row.
		 * @note: we are treating matrices as column major, so axis elements are sequential in memory
		 * @seealso SetAxis, SetColumn
		 */
		FORCEINLINE void SetRow(int32 RowIndex, const TVector<FRealSingle, 3>& V)
		{
			M[0][RowIndex] = V.X;
			M[1][RowIndex] = V.Y;
			M[2][RowIndex] = V.Z;
			M[3][RowIndex] = 0;
		}

		/**
		 * Get the specified column (0-indexed, X,Y,Z). Equivalent to GetAxis.
		 * @note: we are treating matrices as column major, so columns are sequential in memory
		 * @seealso GetAxis, GetRow
		 */
		FORCEINLINE TVector<FRealSingle, 3> GetColumn(int32 ColumnIndex) const
		{
			return GetAxis(ColumnIndex);
		}

		/**
		 * Set the specified column. Equivalent to SetAxis.
		 * @note: we are treating matrices as column major, so axis elements are sequential in memory
		 * @seealso SetAxis, SetRow
		 */
		FORCEINLINE void SetColumn(int32 ColumnIndex, const TVector<FRealSingle, 3>& V)
		{
			SetAxis(ColumnIndex, V);
		}

		/**
		 * Get the diagonal elements as a vector.
		 */
		FORCEINLINE TVector<FRealSingle, 3> GetDiagonal() const
		{
			return TVector<FRealSingle, 3>(M[0][0], M[1][1], M[2][2]);
		}

		FORCEINLINE FRealSingle GetAt(int32 RowIndex, int32 ColIndex) const
		{
			return M[ColIndex][RowIndex];
		}

		FORCEINLINE void SetAt(int32 RowIndex, int32 ColIndex, FRealSingle V)
		{
			M[ColIndex][RowIndex] = V;
		}

		/**
		 * Return a diagonal matrix with the specified elements
		 */
		static PMatrix<FRealSingle, 3, 3> FromDiagonal(const TVector<FRealSingle, 3>& D)
		{
			return PMatrix<FRealSingle, 3, 3>(D.X, D.Y, D.Z);
		}

#if COMPILE_WITHOUT_UNREAL_SUPPORT
		// TODO(mlentine): Document which one is row and which one is column
		PMatrix<FRealSingle, 3, 3> operator*(const PMatrix<FRealSingle, 3, 3>& Other)
		{
			return PMatrix<FRealSingle, 3, 3>(
				M[0][0] * Other.M[0][0] + M[0][1] * Other.M[1][0] + M[0][2] * Other.M[2][0],
				M[1][0] * Other.M[0][0] + M[1][1] * Other.M[1][0] + M[1][2] * Other.M[2][0],
				M[2][0] * Other.M[0][0] + M[2][1] * Other.M[1][0] + M[2][2] * Other.M[2][0],
				M[0][0] * Other.M[0][1] + M[0][1] * Other.M[1][1] + M[0][2] * Other.M[2][1],
				M[1][0] * Other.M[0][1] + M[1][1] * Other.M[1][1] + M[1][2] * Other.M[2][1],
				M[2][0] * Other.M[0][1] + M[2][1] * Other.M[1][1] + M[2][2] * Other.M[2][1],
				M[0][0] * Other.M[0][2] + M[0][1] * Other.M[1][2] + M[0][2] * Other.M[2][2],
				M[1][0] * Other.M[0][2] + M[1][1] * Other.M[1][2] + M[1][2] * Other.M[2][2],
				M[2][0] * Other.M[0][2] + M[2][1] * Other.M[1][2] + M[2][2] * Other.M[2][2]);
		}
#endif
		inline bool Equals(const PMatrix<FRealSingle, 3, 3>& Other, FRealSingle Tolerance = UE_KINDA_SMALL_NUMBER) const
		{
			return true
				&& (FMath::Abs(Other.M[0][0] - M[0][0]) <= Tolerance)
				&& (FMath::Abs(Other.M[0][1] - M[0][1]) <= Tolerance)
				&& (FMath::Abs(Other.M[0][2] - M[0][2]) <= Tolerance)
				&& (FMath::Abs(Other.M[1][0] - M[1][0]) <= Tolerance)
				&& (FMath::Abs(Other.M[1][1] - M[1][1]) <= Tolerance)
				&& (FMath::Abs(Other.M[1][2] - M[1][2]) <= Tolerance)
				&& (FMath::Abs(Other.M[2][0] - M[2][0]) <= Tolerance)
				&& (FMath::Abs(Other.M[2][1] - M[2][1]) <= Tolerance)
				&& (FMath::Abs(Other.M[2][2] - M[2][2]) <= Tolerance);
		}

		// M[i][j] = x[i] * y[j]
		static PMatrix<FRealSingle, 3, 3> OuterProduct(const TVector<FRealSingle, 3>& X, const TVector<FRealSingle, 3>& Y)
		{
			return PMatrix<FRealSingle, 3, 3>(
				X[0] * Y[0],
				X[1] * Y[0],
				X[2] * Y[0],
				X[0] * Y[1],
				X[1] * Y[1],
				X[2] * Y[1],
				X[0] * Y[2],
				X[1] * Y[2],
				X[2] * Y[2]);
		}


		static CHAOSCORE_API const PMatrix<FRealSingle, 3, 3> Zero;
		static CHAOSCORE_API const PMatrix<FRealSingle, 3, 3> Identity;
	};

	typedef PMatrix<double, 3, 3> PMatrix33d;
}
