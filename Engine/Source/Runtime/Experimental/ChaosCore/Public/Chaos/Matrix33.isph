// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef __MATRIX33_ISPH__
#define __MATRIX33_ISPH__

#include "Math/Vector.isph"
#include "Math/Matrix.isph"

typedef FMatrix44d FMatrix33d;
typedef FMatrix44f FMatrix33f;
typedef FMatrix44d FMatrix33;

static const uniform struct FMatrix33d Matrix33DoubleZero = {{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 1}};
static const uniform struct FMatrix33f Matrix33FloatZero = {{ 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 0, 1}};

#define FMatrix33Zero Matrix33DoubleZero

inline uniform FMatrix33 SetMatrix33(
	const uniform FReal x00, const uniform FReal x10, const uniform FReal x20,
	const uniform FReal x01, const uniform FReal x11, const uniform FReal x21,
	const uniform FReal x02, const uniform FReal x12, const uniform FReal x22)
{
	return SetMatrix(SetVector4(x00, x01, x02, ZERO), SetVector4(x10, x11, x12, ZERO), SetVector4(x20, x21, x22, ZERO), Vector0001);
}

inline FMatrix33d SetMatrix33(
	const double x00, const double x10, const double x20,
	const double x01, const double x11, const double x21,
	const double x02, const double x12, const double x22)
{
	return SetMatrix(SetVector4(x00, x01, x02, DOUBLE_ZERO), SetVector4(x10, x11, x12, DOUBLE_ZERO), SetVector4(x20, x21, x22, DOUBLE_ZERO), Double0001);
}

inline FMatrix33f SetMatrix33(
	const float x00, const float x10, const float x20,
	const float x01, const float x11, const float x21,
	const float x02, const float x12, const float x22)
{
	return SetMatrix(SetVector4(x00, x01, x02, FLOAT_ZERO), SetVector4(x10, x11, x12, FLOAT_ZERO), SetVector4(x20, x21, x22, FLOAT_ZERO), Float0001);
}

inline uniform FMatrix33 SetMatrix33(
	const uniform FReal x00, const uniform FReal x10, const uniform FReal x20,
	const uniform FReal x11, const uniform FReal x21, const uniform FReal x22)
{
	return SetMatrix(SetVector4(x00, x10, x20, ZERO), SetVector4(x10, x11, x21, ZERO), SetVector4(x20, x21, x22, ZERO), Vector0001);
}

inline uniform FMatrix33 SetMatrix33(
	const uniform FReal x00, const uniform FReal x11, const uniform FReal x22)
{
	return SetMatrix(SetVector4(x00, ZERO, ZERO, ZERO), SetVector4(ZERO, x11, ZERO, ZERO), SetVector4(ZERO, ZERO, x22, ZERO), Vector0001);
}

inline uniform FMatrix33d SetMatrix33(const uniform FVector4d& x0, const uniform FVector4d& x1, const uniform FVector4d& x2)
{
	return SetMatrix(x0, x1, x2, Double0001);
}

inline uniform FMatrix33f SetMatrix33(const uniform FVector4f& x0, const uniform FVector4f& x1, const uniform FVector4f& x2)
{
	return SetMatrix(x0, x1, x2, Float0001);
}

inline FMatrix33d SetMatrix33(const FVector4d& x0, const FVector4d& x1, const FVector4d& x2)
{
	return SetMatrix(x0, x1, x2, Double0001);
}

inline FMatrix33f SetMatrix33(const FVector4f& x0, const FVector4f& x1, const FVector4f& x2)
{
	return SetMatrix(x0, x1, x2, Float0001);
}

inline uniform FMatrix33 MultiplyAB(const uniform FMatrix33& L, const uniform FMatrix33& R)
{
	const uniform FVector4 L0 = SetVector4(L.M[0], L.M[1], L.M[2], L.M[3]);
	const uniform FVector4 L1 = SetVector4(L.M[4], L.M[5], L.M[6], L.M[7]);
	const uniform FVector4 L2 = SetVector4(L.M[8], L.M[9], L.M[10], L.M[11]);

	uniform FVector4 x0 = SetVector4(R.M[0]) * L0 + SetVector4(R.M[1]) * L1 + SetVector4(R.M[2]) * L2;
	uniform FVector4 x1 = SetVector4(R.M[4]) * L0 + SetVector4(R.M[5]) * L1 + SetVector4(R.M[6]) * L2;
	uniform FVector4 x2 = SetVector4(R.M[8]) * L0 + SetVector4(R.M[9]) * L1 + SetVector4(R.M[10]) * L2;

	x0.V[3] = 0;
	x1.V[3] = 0;
	x2.V[3] = 0;

	return SetMatrix33(x0, x1, x2);
}

inline FMatrix33 MultiplyAB(const FMatrix33& L, const FMatrix33& R)
{
	const FVector4 L0 = SetVector4(L.M[0], L.M[1], L.M[2], L.M[3]);
	const FVector4 L1 = SetVector4(L.M[4], L.M[5], L.M[6], L.M[7]);
	const FVector4 L2 = SetVector4(L.M[8], L.M[9], L.M[10], L.M[11]);

	FVector4 x0 = SetVector4(R.M[0]) * L0 + SetVector4(R.M[1]) * L1 + SetVector4(R.M[2]) * L2;
	FVector4 x1 = SetVector4(R.M[4]) * L0 + SetVector4(R.M[5]) * L1 + SetVector4(R.M[6]) * L2;
	FVector4 x2 = SetVector4(R.M[8]) * L0 + SetVector4(R.M[9]) * L1 + SetVector4(R.M[10]) * L2;

	x0.V[3] = 0;
	x1.V[3] = 0;
	x2.V[3] = 0;

	return SetMatrix33(x0, x1, x2);
}

inline uniform FMatrix33d MultiplyABt(const uniform FMatrix33d& L, const uniform FMatrix33d& R)
{
	const uniform FVector4d L0 = SetVector4(L.M[0], L.M[1], L.M[2], L.M[3]);
	const uniform FVector4d L1 = SetVector4(L.M[4], L.M[5], L.M[6], L.M[7]);
	const uniform FVector4d L2 = SetVector4(L.M[8], L.M[9], L.M[10], L.M[11]);

	uniform FVector4d x0 = SetVector4(R.M[0]) * L0 + SetVector4(R.M[4]) * L1 + SetVector4(R.M[8]) * L2;
	uniform FVector4d x1 = SetVector4(R.M[1]) * L0 + SetVector4(R.M[5]) * L1 + SetVector4(R.M[9]) * L2;
	uniform FVector4d x2 = SetVector4(R.M[2]) * L0 + SetVector4(R.M[6]) * L1 + SetVector4(R.M[10]) * L2;

	x0.V[3] = 0;
	x1.V[3] = 0;
	x2.V[3] = 0;

	return SetMatrix33(x0, x1, x2);
}

inline uniform FMatrix33f MultiplyABt(const uniform FMatrix33f& L, const uniform FMatrix33f& R)
{
	const uniform FVector4f L0 = SetVector4(L.M[0], L.M[1], L.M[2], L.M[3]);
	const uniform FVector4f L1 = SetVector4(L.M[4], L.M[5], L.M[6], L.M[7]);
	const uniform FVector4f L2 = SetVector4(L.M[8], L.M[9], L.M[10], L.M[11]);

	uniform FVector4f x0 = SetVector4(R.M[0]) * L0 + SetVector4(R.M[4]) * L1 + SetVector4(R.M[8]) * L2;
	uniform FVector4f x1 = SetVector4(R.M[1]) * L0 + SetVector4(R.M[5]) * L1 + SetVector4(R.M[9]) * L2;
	uniform FVector4f x2 = SetVector4(R.M[2]) * L0 + SetVector4(R.M[6]) * L1 + SetVector4(R.M[10]) * L2;

	x0.V[3] = 0;
	x1.V[3] = 0;
	x2.V[3] = 0;

	return SetMatrix33(x0, x1, x2);
}

inline FMatrix33d MultiplyABt(const FMatrix33d& L, const FMatrix33d& R)
{
	const FVector4d L0 = SetVector4(L.M[0], L.M[1], L.M[2], L.M[3]);
	const FVector4d L1 = SetVector4(L.M[4], L.M[5], L.M[6], L.M[7]);
	const FVector4d L2 = SetVector4(L.M[8], L.M[9], L.M[10], L.M[11]);

	FVector4d x0 = SetVector4(R.M[0]) * L0 + SetVector4(R.M[4]) * L1 + SetVector4(R.M[8]) * L2;
	FVector4d x1 = SetVector4(R.M[1]) * L0 + SetVector4(R.M[5]) * L1 + SetVector4(R.M[9]) * L2;
	FVector4d x2 = SetVector4(R.M[2]) * L0 + SetVector4(R.M[6]) * L1 + SetVector4(R.M[10]) * L2;

	x0.V[3] = 0;
	x1.V[3] = 0;
	x2.V[3] = 0;

	return SetMatrix33(x0, x1, x2);
}

inline FMatrix33f MultiplyABt(const FMatrix33f& L, const FMatrix33f& R)
{
	const FVector4f L0 = SetVector4(L.M[0], L.M[1], L.M[2], L.M[3]);
	const FVector4f L1 = SetVector4(L.M[4], L.M[5], L.M[6], L.M[7]);
	const FVector4f L2 = SetVector4(L.M[8], L.M[9], L.M[10], L.M[11]);

	FVector4f x0 = SetVector4(R.M[0]) * L0 + SetVector4(R.M[4]) * L1 + SetVector4(R.M[8]) * L2;
	FVector4f x1 = SetVector4(R.M[1]) * L0 + SetVector4(R.M[5]) * L1 + SetVector4(R.M[9]) * L2;
	FVector4f x2 = SetVector4(R.M[2]) * L0 + SetVector4(R.M[6]) * L1 + SetVector4(R.M[10]) * L2;

	x0.V[3] = 0;
	x1.V[3] = 0;
	x2.V[3] = 0;

	return SetMatrix33(x0, x1, x2);
}

inline uniform FVector Multiply(const uniform FMatrix33& L, const uniform FVector& R)
{
	const uniform FVector4 L0 = SetVector4(L.M[0], L.M[1], L.M[2], L.M[3]);
	const uniform FVector4 L1 = SetVector4(L.M[4], L.M[5], L.M[6], L.M[7]);
	const uniform FVector4 L2 = SetVector4(L.M[8], L.M[9], L.M[10], L.M[11]);

	const uniform FVector4 Result = L0 * SetVector4(R.V[0]) + L1 * SetVector4(R.V[1]) + L2 * SetVector4(R.V[2]);

	return SetVector(Result.V[0], Result.V[1], Result.V[2]);
}

inline varying FVector Multiply(const uniform FMatrix33& L, const varying FVector& R)
{
	const uniform FVector4 L0 = SetVector4(L.M[0], L.M[1], L.M[2], L.M[3]);
	const uniform FVector4 L1 = SetVector4(L.M[4], L.M[5], L.M[6], L.M[7]);
	const uniform FVector4 L2 = SetVector4(L.M[8], L.M[9], L.M[10], L.M[11]);

	const varying FVector4 Result = L0 * SetVector4(R.V[0]) + L1 * SetVector4(R.V[1]) + L2 * SetVector4(R.V[2]);

	return SetVector(Result.V[0], Result.V[1], Result.V[2]);
}

inline uniform FMatrix33 Multiply(const uniform FMatrix33& L, const uniform FReal R)
{
	return SetMatrix33(
		L.M[0] * R, L.M[1] * R, L.M[2] * R,
		L.M[4] * R, L.M[5] * R, L.M[6] * R,
		L.M[8] * R, L.M[9] * R, L.M[10] * R
	);
}

inline FMatrix33d Multiply(const FMatrix33d& L, const double R)
{
	return SetMatrix33(
		L.M[0] * R, L.M[1] * R, L.M[2] * R,
		L.M[4] * R, L.M[5] * R, L.M[6] * R,
		L.M[8] * R, L.M[9] * R, L.M[10] * R
	);
}

inline FMatrix33f Multiply(const FMatrix33f& L, const float R)
{
	return SetMatrix33(
		L.M[0] * R, L.M[1] * R, L.M[2] * R,
		L.M[4] * R, L.M[5] * R, L.M[6] * R,
		L.M[8] * R, L.M[9] * R, L.M[10] * R
	);
}

inline uniform FMatrix33 AddAB(const uniform FMatrix33& L, const uniform FMatrix33& R)
{
	const uniform FVector4 L0 = SetVector4(L.M[0], L.M[1], L.M[2], L.M[3]);
	const uniform FVector4 L1 = SetVector4(L.M[4], L.M[5], L.M[6], L.M[7]);
	const uniform FVector4 L2 = SetVector4(L.M[8], L.M[9], L.M[10], L.M[11]);

	const uniform FVector4 R0 = SetVector4(R.M[0], R.M[1], R.M[2], R.M[3]);
	const uniform FVector4 R1 = SetVector4(R.M[4], R.M[5], R.M[6], R.M[7]);
	const uniform FVector4 R2 = SetVector4(R.M[8], R.M[9], R.M[10], R.M[11]);

	uniform FVector4 x0 = R0 + L0;
	uniform FVector4 x1 = R1 + L1;
	uniform FVector4 x2 = R2 + L2;

	x0.V[3] = 0;
	x1.V[3] = 0;
	x2.V[3] = 0;

	return SetMatrix33(x0, x1, x2);
}

inline FMatrix33d AddAB(const FMatrix33d& L, const FMatrix33d& R)
{
	const FVector4d L0 = SetVector4(L.M[0], L.M[1], L.M[2], L.M[3]);
	const FVector4d L1 = SetVector4(L.M[4], L.M[5], L.M[6], L.M[7]);
	const FVector4d L2 = SetVector4(L.M[8], L.M[9], L.M[10], L.M[11]);

	const FVector4d R0 = SetVector4(R.M[0], R.M[1], R.M[2], R.M[3]);
	const FVector4d R1 = SetVector4(R.M[4], R.M[5], R.M[6], R.M[7]);
	const FVector4d R2 = SetVector4(R.M[8], R.M[9], R.M[10], R.M[11]);

	FVector4d x0 = R0 + L0;
	FVector4d x1 = R1 + L1;
	FVector4d x2 = R2 + L2;

	x0.V[3] = 0;
	x1.V[3] = 0;
	x2.V[3] = 0;

	return SetMatrix33(x0, x1, x2);
}

inline FMatrix33f AddAB(const FMatrix33f& L, const FMatrix33f& R)
{
	const FVector4f L0 = SetVector4(L.M[0], L.M[1], L.M[2], L.M[3]);
	const FVector4f L1 = SetVector4(L.M[4], L.M[5], L.M[6], L.M[7]);
	const FVector4f L2 = SetVector4(L.M[8], L.M[9], L.M[10], L.M[11]);

	const FVector4f R0 = SetVector4(R.M[0], R.M[1], R.M[2], R.M[3]);
	const FVector4f R1 = SetVector4(R.M[4], R.M[5], R.M[6], R.M[7]);
	const FVector4f R2 = SetVector4(R.M[8], R.M[9], R.M[10], R.M[11]);

	FVector4f x0 = R0 + L0;
	FVector4f x1 = R1 + L1;
	FVector4f x2 = R2 + L2;

	x0.V[3] = 0;
	x1.V[3] = 0;
	x2.V[3] = 0;

	return SetMatrix33(x0, x1, x2);
}

inline uniform FMatrix33 MakeQuatRotationTranslationMatrix(const uniform FVector4& Q)
{
	// Assumes origin is 0,0,0
	const uniform FVector4 xyz2 = Q + Q; // x2, y2, z2
	const uniform FVector4 x = xyz2 * Q.V[0]; // xx, xy, xz
	const uniform FVector4 y = xyz2 * Q.V[1]; // yx, yy, yz
	const uniform FVector4 w = xyz2 * Q.V[3]; // wx, wy, wz
	const uniform FReal zz = Q.V[2] * xyz2.V[2];

	const uniform FVector4 yx = SetVector4(y.V[2], x.V[2], x.V[1], 0); // yz, xz, xy
	const uniform FVector4 Adds = yx + w; // M[6], M[8], M[1]
	const uniform FVector4 Minuses = yx - w; // M[9], M[2], M[4]

	const uniform FVector4 OneMinus0 = SetVector4(y.V[1], x.V[0], x.V[0], 0); // yy, xx, xx
	const uniform FVector4 Oneminus1 = SetVector4(zz, zz, y.V[1], 0); // zz, zz, yy
	const uniform FVector4 OneMinuses = VectorOne - (OneMinus0 + Oneminus1); // M[0], M[5], M[10]

	return SetMatrix33(
		SetVector4(OneMinuses.V[0], Adds.V[2], Minuses.V[1], 0),
		SetVector4(Minuses.V[2], OneMinuses.V[1], Adds.V[0], 0),
		SetVector4(Adds.V[1], Minuses.V[0], OneMinuses.V[2], 0)
	);
}

#endif
