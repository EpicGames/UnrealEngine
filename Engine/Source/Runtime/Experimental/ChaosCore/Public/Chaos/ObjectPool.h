// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/AlignmentTemplates.h"
#include "Templates/UnrealTemplate.h"

namespace Chaos
{
	// SFINAE helpers to clean up the appearance in the class definition.

	// The type needs to have a destructor run when the pool pointer is freed
	template<typename T>
	using TRequiresDestructor = std::enable_if_t<!std::is_trivially_destructible_v<T>>;

	// The type needs no destruction and can just be abandoned on free
	template<typename T>
	using TTrivialDestruct = std::enable_if_t<std::is_trivially_destructible_v<T>>;

	template<typename ObjectType>
	class TObjectPool
	{
	public:

		// Alignment of the stored type
		constexpr static int32 ItemAlign = alignof(ObjectType);

		using FObject = ObjectType;
		using FPtr = ObjectType*;

		explicit TObjectPool(int32 InNumPerBlock, int32 InitialBlocks = 1)
			: NumPerBlock(InNumPerBlock)
		{
			ReserveBlocks(InitialBlocks);
		}

		~TObjectPool()
		{
			Reset();
		}

		// No copy, no move, no default
		// #TODO possibly add move and copy if the internal type is copy constructible but likely better
		// to discourage copying. Moving to another pool should be fine.
		TObjectPool() = delete;
		TObjectPool(const TObjectPool&) = delete;
		TObjectPool(TObjectPool&&) = delete;
		TObjectPool& operator=(const TObjectPool&) = delete;
		TObjectPool& operator=(TObjectPool&&) = delete;

		/**
		 * Allocate an object from the pool
		 * Returns an instance of the object type specified for this pool, either from the next available on the
		 * final block, the free list or by allocating a new block to continue providing objects. As this method
		 * may defer to the runtime allocator it may sometimes run significantly slower when the pool is exhausted
		 * Requires that the object is default constructible
		 * @param Args - ObjectType constructor arguments
		 * @return The newly allocated object
		 */
		template<typename... TArgs>
		FPtr Alloc(TArgs&&... Args)
		{
			// Need a new Block
			if(FreeCount == 0)
			{
				// Need a new block
				ObjectType* NewPtr = AddBlock().GetNextFree();
				Construct<ObjectType>(NewPtr, Forward<TArgs>(Args)...);
				--FreeCount;
				return NewPtr;
			}

			// We know there's a free item somewhere, find one - shuffling full blocks to the end of the list
			check(Blocks.Num() > 0);

			if (Blocks[0].IsFull())
			{
				// Find the fullest block that is not completely full
				int32 SelectedIndex = INDEX_NONE;
				int32 SelectedNumFree = TNumericLimits<int32>::Max();
				for (int32 Index = 0; Index < Blocks.Num(); ++Index)
				{
					const int32 BlockNumFree = Blocks[Index].NumFree;
					if ((BlockNumFree > 0) && (BlockNumFree < SelectedNumFree))
					{
						SelectedNumFree = BlockNumFree;
						SelectedIndex = Index;
					}
				}

				// Move the selected block to the front
				checkf(SelectedIndex != INDEX_NONE, TEXT("Could not find an empty block"));
				Swap(Blocks[0], Blocks[SelectedIndex]);
			}

			// Ensure any writes to Blocks[0] cannot be reordered so that the GetNextFree
			// call is definitely handled by the right block
			FPlatformMisc::MemoryBarrier();

			// Blocks[0] is now a block with at least one free element
			ObjectType* NewPtr = Blocks[0].GetNextFree();
			Construct<ObjectType>(NewPtr, Forward<TArgs>(Args)...);
			--FreeCount;
			return NewPtr;
		}

		/**
		 * Free a single object provided by this pool
		 * Takes an object previously allocated in this pool and frees it, accessing the pointer beyond a call
		 * to free is an undefined operation and could return garbage or an entirely different object. Avoiding this
		 * case is the responsibility of the caller
		 * Behavior is also undefined if an object not owned by this pool is passed to Free. Outside of debug
		 * this will not be asserted
		 * @param Object The object to free
		 */
		void Free(FPtr Object)
		{
			int32 BlockIndex = FindBlock(Object);
			Blocks[BlockIndex].Free(Object);
			++FreeCount;
		}

		/**
		 * Reset the whole pool
		 * This will invalidate every currently live object from the pool. If a pointer from this pool is
		 * dereferenced beyond a call to Reset then it may return garbage, or point to an entirely different
		 * object. Avoiding this case is the responsibility of the caller.
		 */
		void Reset()
		{
			for(FBlock& B : Blocks)
			{
				B.Reset();
			}

			FreeCount = Blocks.Num() * NumPerBlock;
		}

		/**
		 * Get the number of blocks currently allocated in the pool
		 */
		int32 GetNumAllocatedBlocks() const
		{
			return Blocks.Num();
		}

		/**
		 * Get the max number of items per block
		*/
		int32 GetNumPerBlock() const
		{
			return NumPerBlock;
		}

		/**
		 * Get the allocated (total) size of all blocks in the pool
		 */
		int32 GetAllocatedSize() const
		{
			return (PaddedItemSize * NumPerBlock) * Blocks.Num();
		}

		/**
		 * Shrinks the number of blocks
		 * All blocks that are not empty are kept, plus a number of empty blocks specified by the caller
		 * Passing zero will remove all empty blocks, leaving any non-empty blocks behind
		 * @param NumDesiredEmptyBlocks - number of empty blocks to keep
		 */
		void ShrinkTo(int32 NumDesiredEmptyBlocks)
		{
			int32 CurrentNumEmpty = 0;
			int32 Index = 0;
			while(Index < Blocks.Num())
			{
				if(Blocks[Index].IsEmpty())
				{
					if(CurrentNumEmpty < NumDesiredEmptyBlocks)
					{
						++CurrentNumEmpty;
					}
					else
					{
						Blocks.RemoveAtSwap(Index);
						continue;
					}
				}
				++Index;
			}
		}

		/**
		 * Makes sure the pool has at least NumBlocks blocks allocated
		 * Note: This will never reduce the number of blocks, it will only make sure the pool has at least
		 * NumBlocks block allocated, empty or not.
		 * @param NumBlocks - Number of blocks to reserve
		 */
		void ReserveBlocks(int32 NumBlocks)
		{
			while(NumBlocks > Blocks.Num())
			{
				AddBlock();
			}
		}

		/**
		 * Makes sure the pool has at least enough blocks allocated to store NumItems
		 * Not: This isn't a number of free items, but total items. If a pool has 100 items in it and 
		 * ReserveItems(100); is called - no action will be taken.
		 * This function will also never remove blocks from the pool - only ever increase. To reduce the number of
		 * blocks see ShrinkTo
		 * @see ShrinkTo
		 * @Param NumItems - Number of items to reserve
		 */
		void ReserveItems(int32 NumItems)
		{
			const int32 ItemsSize = NumItems * PaddedItemSize;
			const int32 BlockSize = NumPerBlock * PaddedItemSize;
			const int32 NumBlocks = int32(FMath::CeilToInt(static_cast<double>(ItemsSize) / BlockSize));
			ReserveBlocks(NumBlocks);
		}

		/**
		 * Calculates the storage/overhead ratio of the pool. Larger items that have a size close to the alignment
		 * boundary will be more efficient on space
		 * @return Ratio of storage to overhead
		 */
		float GetRatio() const
		{
			return static_cast<float>(sizeof(ObjectType)) / PaddedItemSize;
		}

		/**
		 * Get the number of allocated items.
		*/
		int32 GetNumAllocated() const
		{
			return GetCapacity() - GetNumFree();
		}

		/**
		 * Gets the number of free items the pool has remaining
		 * @return Number of free items
		 */
		int32 GetNumFree() const
		{
			return FreeCount;
		}

		/**
		 * Gets the total number of items the pool can currently stor
		 * @return The pool capacity
		 */
		int32 GetCapacity() const
		{
			return Capacity;
		}

	private:

		// Storage for an item, and an index of the next free item if it's currently on the free list
		template<typename T_ = ObjectType, typename Destructible = void>
		struct alignas(ItemAlign) TItem
		{
			TItem()
				: NextFree(INDEX_NONE)
			{}

			// Object must always remain as the first item to enable casting between Item and Object
			ObjectType Object;

			// Free-list tracking, index of the next free item
			int32 NextFree;

			// Set whether the object is live - encapsulated in a function as the base template performs no action
			void SetLive(bool)
			{}
		};

		// Specialized type for types that require a destructor as we need to add an extra
		// flag for live/dead objects (calling Free will destruct the object but Reset/DestroyAll
		// will need to destruct any non-free objects). This is specialized so that if the type
		// is trivially destructible we can avoid adding the extra member.
		template<typename T_>
		struct alignas(ItemAlign) TItem<T_, TRequiresDestructor<T_>>
		{
			TItem()
				: NextFree(INDEX_NONE)
			{}

			// Object must always remain as the first item to enable casting between Item and Object
			ObjectType Object;

			// Free-list tracking, index of the next free item
			int32 NextFree;

			// Whether the object is live (constructed) and will require destruction
			bool bLive : 1;

			// Set whether the object is live - encapsulated in a function as the base template performs no action
			// @param bInLive - new live flag value
			void SetLive(bool bInLive)
			{
				bLive = bInLive;
			}
		};

		// Helper alias for the item in this pool
		using FItem = TItem<ObjectType>;

		// Size of the item, plus padding up to the correct alignment so we can allocate the whole
		// block with the correct size.
		constexpr static int32 PaddedItemSize = Align(int32(sizeof(FItem)), ItemAlign);

		// A block is an area of memory large enough to hold NumInBlock items, correctly aligned and
		// provide items on demand to the pool
		struct FBlock
		{
			FBlock() = delete;

			explicit FBlock(int32 InNum)
				: Begin(static_cast<FItem*>(FMemory::Malloc(PaddedItemSize * InNum, ItemAlign)))
				, Next(Begin)
				, NumInBlock(InNum)
				, NumFree(NumInBlock)
			{
			}

			~FBlock()
			{
				check(Begin);

				// Return our memory block
				FMemory::Free(Begin);
			}

			FPtr GetNextFree()
			{
				if(FreeList != INDEX_NONE)
				{
					// Something in the free list, prefer this
					FItem* NewItem = Begin + FreeList;
					FreeList = NewItem->NextFree;
					NewItem->NextFree = INDEX_NONE;
					--NumFree;

					return &NewItem->Object;
				}

				if(NumValid < NumInBlock)
				{
					FItem* NewItem = Next++;
					NewItem->NextFree = INDEX_NONE;
					NewItem->SetLive(false);

					--NumFree;
					++NumValid;

					return &NewItem->Object;
				}

				checkf(false, TEXT("Attempt to request a free item from a full block (Freelist is empty, NumValid is %d, NumInBlock is %d, NumFree is %d)"), NumValid, NumInBlock, NumFree);

				return nullptr;
			}

			void Free(ObjectType* Object)
			{
				checkf(Owns(Object), TEXT("A pointer was passed to an object pool block that it does not own."));

				// If required, call the object destructor
				ConditionalDestruct<ObjectType>(Object);

				// Get the offset into this block (check above ensures this is correct)
				FItem* const AsItem = reinterpret_cast<FItem*>(Object);
				const UPTRINT Offset = reinterpret_cast<UPTRINT>(AsItem) - reinterpret_cast<UPTRINT>(Begin);
				const UPTRINT IndexInBlock = Offset / PaddedItemSize;

				// If there's already a freelist, add it to our next item
				if(FreeList != INDEX_NONE)
				{
					AsItem->NextFree = FreeList;
				}

				FreeList = (int32)IndexInBlock;
				++NumFree;
			}

			void Reset()
			{
				DestructAll();

				Next = Begin;
				NumFree = NumInBlock;
				FreeList = INDEX_NONE;
				NumValid = 0;
			}

			void DestructAll()
			{
				for(int32 i = 0; i < NumValid; ++i)
				{
					FItem* ToDestroy = Begin + i;
					ConditionalDestruct<ObjectType>(&ToDestroy->Object);
				}
			}

			bool Owns(ObjectType* InPtr) const
			{
				FItem* AsItem = reinterpret_cast<FItem*>(InPtr);

				return AsItem >= Begin && AsItem < Begin + NumInBlock;
			}

			bool IsEmpty() const
			{
				return NumFree == NumInBlock;
			}

			bool IsFull() const
			{
				return NumFree == 0;
			}

			// Ptr to the beginning of the block
			FItem* Begin = nullptr;
			// Ptr to the next free item when block is filling initially
			FItem* Next = nullptr;

			// Number of items the block can hold
			const int32 NumInBlock;
			// Count of free items in the block
			int32 NumFree = 0;
			// The maximum number of valid items in the block (items are valid if they were ever allocated / constructed)
			int32 NumValid = 0;
			// Head of the free list for this block
			int32 FreeList = INDEX_NONE;
		};

		/**
		 * Adds a block to the pool, updating internal tracking state as appropriate
		 * @return Reference to the new block
		 */
		FBlock& AddBlock()
		{
			Blocks.Emplace(NumPerBlock);

			// Update pool capacity and free count
			Capacity += NumPerBlock;
			FreeCount += NumPerBlock;

			return Blocks.Last();
		}

		/**
		 * Find the index of the owning block for the provided pointer
		 * The provided pointer must have been provided from this pool otherwise an assert will fire.
		 * @param InPtr the pointer to search for
		 * @return Index for the found block
		 */
		int32 FindBlock(ObjectType* InPtr)
		{
			for(int32 i = 0; i < Blocks.Num(); ++i)
			{
				if(Blocks[i].Owns(InPtr))
				{
					return i;
				}
			}

			checkf(false, TEXT("A pointer was passed to an object pool method that it does not own."));
			return INDEX_NONE;
		}

		/**
		 * Construct helper
		 * this specialization is called for objects that are trivially destructible as they require
		 * no tracking of their construction state
		 * @note internal use only - At *MUST* be castable to FItem*
		 * @param At - the pointer to construct at
		 * @param Args - ObjectType constructor args
		 */
		template<
			typename ObjectType_ = ObjectType, 
			TTrivialDestruct<ObjectType_>* = nullptr, 
			typename... TArgs>
		static ObjectType* Construct(ObjectType* At, TArgs&&... Args)
		{
			new (At) ObjectType(Forward<TArgs>(Args)...);

			return At;
		}

		/**
		 * Construct helper
		 * this specialization is called for objects that are not trivially destructible as they
		 * require tracking of their construction state.
		 * @note internal use only - At *MUST* be castable to FItem*
		 * @param At - the pointer to construct at
		 * @param Args - ObjectType constructor args
		 */
		template<
			typename ObjectType_ = ObjectType, 
			TRequiresDestructor<ObjectType_>* = nullptr, 
			typename... TArgs>
		static ObjectType* Construct(ObjectType* At, TArgs&&... Args)
		{
			new (At) ObjectType(Forward<TArgs>(Args)...);

			FItem* AsItem = reinterpret_cast<FItem*>(At);
			AsItem->bLive = true;

			return At;
		}

		/**
		 * Destruct helper
		 * This specialization is called for objects that actually require a destructor to be called
		 * (non trivial destruction). It will also set the bLive flag on the object in the pool so
		 * Reset() calls can skip destruction
		 * @param At - The pointer to destruct
		 */
		template<
			typename ObjectType_ = ObjectType, 
			TRequiresDestructor<ObjectType_>* = nullptr>
		static void ConditionalDestruct(ObjectType* At)
		{
			checkSlow(At);

			// Check the live flag, items on the free list will already have been destructed
			// and we should avoid calling the destructor twice
			FItem* AsItem = reinterpret_cast<FItem*>(At);
			if(AsItem->bLive)
			{
				At->~ObjectType();
				AsItem->bLive = false;
			}
		}

		/**
		 * Destruct helper
		 * This specialization is called for trivially destructible types (with no destructor).
		 * This is essentially a no-op and just makes the calling code cleaner for generic types
		 */
		template<
			typename ObjectType_ = ObjectType, 
			TTrivialDestruct<ObjectType_>* = nullptr>
		static void ConditionalDestruct(ObjectType*)
		{}

		TArray<FBlock> Blocks;

		int32 NumPerBlock = 0;
		int32 Capacity = 0;
		int32 FreeCount = 0;
	};


	/**
	 * A deleter for use with TUniquePtr and a TObjectPool item
	 */
	template<typename ObjectPoolType>
	class TObjectPoolDeleter
	{
	public:
		using FObjectPool = ObjectPoolType;
		using FObject = typename FObjectPool::FObject;

		TObjectPoolDeleter()
			: Pool(nullptr)
		{
		}

		TObjectPoolDeleter(FObjectPool& InPool)
			: Pool(&InPool)
		{
		}

		void operator()(FObject* Object)
		{
			if (Object != nullptr)
			{
				check(Pool != nullptr);
				Pool->Free(Object);
			}
		}

	private:
		FObjectPool* Pool;
	};
}
