// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

namespace Chaos
{
template<class T1, class T2>
struct Pair
{
	Pair()
	{}
	Pair(const T1& First, const T2& Second)
		: First(First), Second(Second)
	{}
	T1 First;
	T2 Second;
};

template<class T1, class T2>
bool operator==(const Pair<T1, T2>& First, const Pair<T1, T2>& Second)
{
	return First.First == Second.First && First.Second == Second.Second;
}

template<class T1, class T2>
bool operator<(const Pair<T1, T2>& First, const Pair<T1, T2>& Second)
{
	if (First.First != Second.First)
	{
		return First.First < Second.First;
	}
	return First.Second < Second.Second;
}

template<class T1, class T2>
bool operator>(const Pair<T1, T2>& First, const Pair<T1, T2>& Second)
{
	if (First.First != Second.First)
	{
		return First.First > Second.First;
	}
	return First.Second > Second.Second;
}

template<class T1, class T2>
Pair<T1, T2> MakePair(const T1& First, const T2& Second)
{
	Pair<T1, T2> NewPair;
	NewPair.First = First;
	NewPair.Second = Second;
	return NewPair;
}
}
