// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"

namespace Chaos
{
struct FRange
{
	FRange(int32 InStart, int32 InCount) : Start(InStart), Count(InCount) {}
	int32 Start;
	int32 Count;
};

inline bool operator==(const FRange& First, const FRange& Second)
{
	return First.Start == Second.Start && First.Count == Second.Count;
}
}
