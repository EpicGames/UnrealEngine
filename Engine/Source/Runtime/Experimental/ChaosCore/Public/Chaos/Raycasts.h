// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos::Raycasts
{ 
	template <typename T, int d>
	CHAOSCORE_API bool RayAabb(const TVector<FReal, d>& RayStart, const TVector<FReal, d>& RayDir, const FReal RayLength, const FReal RayThickness, const TVector<T, d>& AabbMin, const TVector<T, d>& AabbMax, FReal& OutTime, TVector<FReal, d>& OutPosition, TVector<FReal, d>& OutNormal, int32& OutFaceIndex);

	template <typename T, int d>
	FORCEINLINE bool RayAabb(const TVector<FReal, d>& RayStart, const TVector<FReal, d>& RayDir, const TVector<FReal, d>& InvRayDir, const bool* bRayParallel, const FReal RayLength, const TVector<T, d>& AabbMin, const TVector<T, d>& AabbMax, FReal& OutEntryTime, FReal& OutExitTime)
	{
		const TVector<FReal, d> StartToMin = TVector<FReal, d>(AabbMin) - RayStart;
		const TVector<FReal, d> StartToMax = TVector<FReal, d>(AabbMax) - RayStart;

		//For each axis record the start and end time when ray is in the box. If the intervals overlap the ray is inside the box
		FReal LatestStartTime = 0;
		FReal EarliestEndTime = TNumericLimits<FReal>::Max();

		for (int Axis = 0; Axis < d; ++Axis)
		{
			FReal Time1, Time2;
			if (bRayParallel[Axis])
			{
				if (StartToMin[Axis] > 0 || StartToMax[Axis] < 0)
				{
					return false;	//parallel and outside
				}
				else
				{
					Time1 = 0;
					Time2 = TNumericLimits<FReal>::Max();
				}
			}
			else
			{
				Time1 = StartToMin[Axis] * InvRayDir[Axis];
				Time2 = StartToMax[Axis] * InvRayDir[Axis];
			}

			if (Time1 > Time2)
			{
				//going from max to min direction
				Swap(Time1, Time2);
			}

			LatestStartTime = FMath::Max(LatestStartTime, Time1);
			EarliestEndTime = FMath::Min(EarliestEndTime, Time2);

			if (LatestStartTime > EarliestEndTime)
			{
				return false;	//Outside of slab before entering another
			}
		}

		//infinite ray intersects with inflated box
		if (LatestStartTime > RayLength || EarliestEndTime < 0)
		{
			//outside of line segment given
			return false;
		}

		OutEntryTime = LatestStartTime;
		OutExitTime = EarliestEndTime;
		return true;
	}

	CHAOSCORE_API bool RayCapsule(const FVec3& RayStart, const FVec3& RayDir, const FReal RayLength, const FReal RayThickness, FReal CapsuleRadius, FReal CapsuleHeight, const FVec3& CapsuleAxis, const FVec3& CapsuleX1, const FVec3& CapsuleX2, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal);

	template <typename T, int d>
	bool RaySphere(const TVector<T, d>& RayStart, const TVector<T, d>& RayDir, const T RayLength, const T RayThickness, const FVec3f& SphereCenter, const FRealSingle SphereRadius, T& OutTime, TVector<T, d>& OutPosition, TVector<T, d>& OutNormal)
	{
		ensure(FMath::IsNearlyEqual(RayDir.SizeSquared(), (FReal)1, (FReal)UE_KINDA_SMALL_NUMBER));
		ensure(RayLength >= 0);

		const T EffectiveRadius = RayThickness + SphereRadius;
		const T EffectiveRadius2 = EffectiveRadius * EffectiveRadius;
		const TVector<T, d> Offset = SphereCenter - RayStart;
		const T OffsetSize2 = Offset.SizeSquared();
		if (OffsetSize2 < EffectiveRadius2)
		{
			//initial overlap
			OutTime = 0;	//no position or normal since initial overlap
			return true;
		}

		//(SphereCenter-X) \dot (SphereCenter-X) = EffectiveRadius^2
		//Let X be on ray, then (SphereCenter - RayStart - t RayDir) \dot (SphereCenter - RayStart - t RayDir) = EffectiveRadius^2
		//Let Offset = (SphereCenter - RayStart), then reduces to quadratic: t^2 - 2t*(Offset \dot RayDir) + Offset^2 - EffectiveRadius^2 = 0
		//const T A = 1;
		const T HalfB = -TVector<T, d>::DotProduct(Offset, RayDir);
		const T C = OffsetSize2 - EffectiveRadius2;
		//time = (-b +- sqrt(b^2 - 4ac)) / 2a
		//2 from the B cancels because of 2a and 4ac
		const T QuarterUnderRoot = HalfB * HalfB - C;
		if (QuarterUnderRoot < 0)
		{
			return false;
		}

		constexpr T Epsilon = 1e-4f;
		//we early out if starting in sphere, so using first time is always acceptable
		T FirstTime = QuarterUnderRoot < Epsilon ? -HalfB : -HalfB - FMath::Sqrt(QuarterUnderRoot);
		if (FirstTime >= 0 && FirstTime <= RayLength)
		{
			const TVector<T, d> FinalSpherePosition = RayStart + FirstTime * RayDir;
			const TVector<T, d> FinalNormal = (FinalSpherePosition - SphereCenter) / EffectiveRadius;
			const TVector<T, d> IntersectionPosition = FinalSpherePosition - FinalNormal * RayThickness;

			OutTime = FirstTime;
			OutPosition = IntersectionPosition;
			OutNormal = FinalNormal;
			return true;
		}

		return false;
	}
} // namespace Chaos::Raycasts
