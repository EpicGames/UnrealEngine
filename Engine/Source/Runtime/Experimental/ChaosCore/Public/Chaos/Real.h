// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

namespace Chaos
{
	/**
	* Specific precision types, this should be used in very targeted places
	* where the use of a specific type is necessary over using FReal
	*/
	using FRealDouble = double;
	using FRealSingle = float;

	/**
	* Common data types for the Chaos physics engine. Unless a specific
	* precision of type is required most code should use these existing types
	* (e.g. FVec3) to adapt to global changes in precision.
	*/

	using FReal = FRealDouble;

	/**
	* ISPC optimization supports float and double, this allows classes that uses ISPC to branch to the right implementation 
	* without having to check the actual underlying type of FReal
	*/
	inline constexpr bool bRealTypeCompatibleWithISPC = std::is_same_v<FReal, float> || std::is_same_v<FReal, double>;
}
