// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Real.h"
#include "Chaos/Vector.h"
#include "Chaos/Matrix.h"

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Math/Quat.h"
#include "Math/RotationMatrix.h"
#else
#include <array>
#include <cmath>

struct _FQuat
{
public:
	const Chaos::FReal operator[](const int32 i) const
	{
		return angles[i];
	}
	Chaos::FReal& operator[](const int32 i)
	{
		return angles[i];
	}
	std::array<Chaos::FReal, 3> angles;
	static MakeFromEuler(const Vector<Chaos::FReal, 3>& InAngles)
	{
		_FQuat Quat;
		Quat.angles = InAngles;
		return Quat;
	}
};
using FQuat = _FQuat;	// Work around include tool not understanding that this won't be compiled alongside MathFwd.h
#endif

namespace Chaos
{
	template<class T, int d>
	class TRotation
	{
	private:
		TRotation() {}
		~TRotation() = default;
	};

	template<>
	class TRotation<FRealSingle, 3> : public UE::Math::TQuat<FRealSingle>
	{
		using BaseQuat = UE::Math::TQuat<FRealSingle>;
	public:
		TRotation()
		    : BaseQuat() {}
		TRotation(const BaseQuat& Quat)
		    : BaseQuat(Quat) {}
		TRotation(const FMatrix44f& Matrix)
		    : BaseQuat(Matrix) {}
		TRotation(const FMatrix44d& Matrix)
		    : BaseQuat(FMatrix44f(Matrix)) {}
		template<typename OtherType>
		TRotation(const UE::Math::TQuat<OtherType>& Other)
		    : BaseQuat((FRealSingle)Other.X, (FRealSingle)Other.Y, (FRealSingle)Other.Z, (FRealSingle)Other.W) {}

		inline PMatrix<FRealSingle, 3, 3> ToMatrix() const
		{
			PMatrix<FRealSingle, 3, 3> R;
			BaseQuat::ToMatrix(R);
			return R;
		}

		/**
		 * Extract the axis and angle from the Quaternion.
		 * @param OutAxis The axis of rotation.
		 * @param OutAngle The angle of rotation about the axis (radians).
		 * @param DefaultAxis The axis to set when the angle is too small to accurately calculate the axis.
		 * @param EpsilonSq The squared tolerance used to check for small angles.
		 * @return Whether the axis was successfully calculated (true except for very small angles around or less than Epsilon).
		 * @warning The axis calculation cannot succeed for small angles due to numerical error. In this case, the function will return false, but set the axis to DefaultAxis.
		 * @note EpsilonSq is approximately the square of the angle below which we cannot calculate the axis. It needs to be "much greater" than square of the error in the
		 * quaternion values which is usually ~1e-4, so values around 1e-3^2 = 1e-6 or greater are about right.
		 */
		inline bool ToAxisAndAngleSafe(TVector<FRealSingle, 3>& OutAxis, FRealSingle& OutAngle, const TVector<FRealSingle, 3>& DefaultAxis, FRealSingle EpsilionSq = 1e-6f) const
		{
			OutAngle = GetAngle();
			return GetRotationAxisSafe(OutAxis, DefaultAxis, EpsilionSq);
		}

		/**
		 * Extract the axis from the Quaternion.
		 * @param OutAxis The axis of rotation.
		 * @param DefaultAxis The axis to set when the angle is too small to accurately calculate the axis.
		 * @param EpsilonSq The squared tolerance used to check for small angles.
		 * @return Whether the axis was successfully calculated (true except for very small angles around or less than Epsilon).
		 * @warning The axis calculation cannot succeed for small angles due to numerical error. In this case, the function will return false, but set the axis to DefaultAxis.
		 * @note EpsilonSq is approximately the square of the angle below which we cannot calculate the axis. It needs to be "much greater" than square of the error in the
		 * quaternion values which is usually ~1e-4, so values around 1e-3^2 = 1e-6 or greater are about right.
		 */
		inline bool GetRotationAxisSafe(TVector<FRealSingle, 3>& OutAxis, const TVector<FRealSingle, 3>& DefaultAxis, FRealSingle EpsilionSq = 1e-6f) const
		{
			// Tolerance must be much larger than error in normalized vector (usually ~1e-4) for the 
			// axis calculation to succeed for small angles. For small angles, W ~= 1, and
			// X, Y, Z ~= 0. If the values of X, Y, Z are around 1e-4 we are just normalizing error.
			const FRealSingle LenSq = X * X + Y * Y + Z * Z;
			if (LenSq > EpsilionSq)
			{
				FRealSingle InvLen = FMath::InvSqrt(LenSq);
				OutAxis = TVector<FRealSingle, 3>(X * InvLen, Y * InvLen, Z * InvLen);
				return true;
			}

			OutAxis = DefaultAxis;
			return false;
		}

		/**
		* Convert to a matrix and return as the 3 matrix axes.
		*/
		inline void ToMatrixAxes(TVector<FRealSingle, 3>& OutX, TVector<FRealSingle, 3>& OutY, TVector<FRealSingle, 3>& OutZ)
		{
			const FRealSingle x2 = X + X;    const FRealSingle y2 = Y + Y;    const FRealSingle z2 = Z + Z;
			const FRealSingle xx = X * x2;   const FRealSingle xy = X * y2;   const FRealSingle xz = X * z2;
			const FRealSingle yy = Y * y2;   const FRealSingle yz = Y * z2;   const FRealSingle zz = Z * z2;
			const FRealSingle wx = W * x2;   const FRealSingle wy = W * y2;   const FRealSingle wz = W * z2;

			OutX = TVector<FRealSingle, 3>(1.0f - (yy + zz), xy + wz, xz - wy);
			OutY = TVector<FRealSingle, 3>(xy - wz, 1.0f - (xx + zz), yz + wx);
			OutZ = TVector<FRealSingle, 3>(xz + wy, yz - wx, 1.0f - (xx + yy));
		}

		/**
		 * Extract the Swing and Twist rotations, assuming that the Twist Axis is (1,0,0).
		 * /see ToSwingTwist
		 */
		inline void ToSwingTwistX(BaseQuat& OutSwing, BaseQuat& OutTwist) const
		{
			OutTwist = (X != 0.0f)? BaseQuat(X, 0, 0, W).GetNormalized() : BaseQuat::Identity;
			OutSwing = *this * OutTwist.Inverse();
		}

		/**
		 * @brief Return the large absolute element value
		*/
		FORCEINLINE FRealSingle GetAbsMax() const
		{
			return FMath::Max(FMath::Max(FMath::Abs(X), FMath::Abs(Y)), FMath::Max(FMath::Abs(Z), FMath::Abs(W)));
		}

		/**
		 * @brief Return the dot product of two quaternions
		*/
		static inline FRealSingle DotProduct(const TRotation<FRealSingle, 3>& L, const TRotation<FRealSingle, 3>& R)
		{
			return (L | R);
		}

		/**
		 * Return the complex conjugate of the rotation
		 */
		CHAOSCORE_API static TRotation<FRealSingle, 3> Conjugate(const ::Chaos::TRotation<FRealSingle, 3>& InR);

		/**
		 * Negate all values of the quaternion (note: not the inverse rotation. See Conjugate)
		 */
		CHAOSCORE_API static TRotation<FRealSingle, 3> Negate(const ::Chaos::TRotation<FRealSingle, 3>& InR);

		/**
		 * Create an identity rotation
		 */
		static inline TRotation<FRealSingle, 3> FromIdentity()
		{
			return BaseQuat(0, 0, 0, 1);
		}

		/**
		 * Create a rotation by explicitly specifying all elements
		 */
		static inline TRotation<FRealSingle, 3> FromElements(const FRealSingle X, const FRealSingle Y, const FRealSingle Z, const FRealSingle W)
		{
			using FQuatReal = BaseQuat::FReal;
			return BaseQuat((FQuatReal)X, (FQuatReal)Y, (FQuatReal)Z, (FQuatReal)W);
		}

		/**
		 * Create a rotation by explicitly specifying all elements
		 */
		static inline TRotation<FRealSingle, 3> FromElements(const ::Chaos::TVector<FRealSingle, 3>& V, const FRealSingle W)
		{
			return FromElements(V.X, V.Y, V.Z, W);
		}

		/**
		 * Create a rotation about an axis by an angle specified in radians
		 */
		static inline TRotation<FRealSingle, 3> FromAxisAngle(const ::Chaos::TVector<FRealSingle, 3>& Axis, const FRealSingle AngleRad)
		{
			return BaseQuat(UE::Math::TVector<FRealSingle>(Axis.X, Axis.Y, Axis.Z), (BaseQuat::FReal)AngleRad);
		}

		/**
		 * Create a rotation about an axis V/|V| by angle |V| in radians
		 */
		CHAOSCORE_API static TRotation<FRealSingle, 3> FromVector(const ::Chaos::TVector<FRealSingle, 3>& V);

		/**
		 * Generate a Rotation that would rotate vector InitialVector to FinalVector
		 */
		CHAOSCORE_API static TRotation<FRealSingle, 3> FromRotatedVector(
			const ::Chaos::TVector<FRealSingle, 3>& InitialVector,
			const ::Chaos::TVector<FRealSingle, 3>& FinalVector);

		/**
		 * Calculate the angular velocity required to take an object with orientation R0 to orientation R1 in time Dt.
		 *
		 * Uses the relation: DQ/DT = (W * Q)/2
		 */
		CHAOSCORE_API static TVector<FRealSingle, 3> CalculateAngularVelocity1(const TRotation<FRealSingle, 3>& R0, const TRotation<FRealSingle, 3>& InR1, const FRealSingle InDt);

		/**
		 * Calculate the angular velocity required to take an object with orientation R0 to orientation R1 in time Dt.
		 *
		 * Uses the Quaternion to Axis/Angle method.
		 */
		CHAOSCORE_API static TVector<FRealSingle, 3> CalculateAngularVelocity2(const TRotation<FRealSingle, 3>& R0, const TRotation<FRealSingle, 3>& InR1, const FRealSingle InDt);

		/**
		 * Calculate the angular velocity required to take an object with orientation R0 to orientation R1 in time Dt.
		 *
		 * This should match the algorithm used in PerParticleUpdateFromDeltaPosition rule.
		 */
		static inline TVector<FRealSingle, 3> CalculateAngularVelocity(const TRotation<FRealSingle, 3>& InR0, const TRotation<FRealSingle, 3>& InR1, const FRealSingle InDt)
		{
			return CalculateAngularVelocity1(InR0, InR1, InDt);
		}

		/**
		 * Calculate the axis-angle delta (angular velocity * dt) required to take an object with orientation R0 to orientation R1.
		 *
		 * This should match the algorithm used in PerParticleUpdateFromDeltaPosition rule.
		 */
		static inline TVector<FRealSingle, 3> CalculateAngularDelta(const TRotation<FRealSingle, 3>& InR0, const TRotation<FRealSingle, 3>& InR1)
		{
			return CalculateAngularVelocity(InR0, InR1, 1.0f);
		}

		/**
		 * Return a new rotation equal to the input rotation with angular velocity W applied over time Dt.
		 *
		 * Uses the relation: DQ/DT = (W * Q)/2
		 */
		CHAOSCORE_API static TRotation<FRealSingle, 3> IntegrateRotationWithAngularVelocity(const TRotation<FRealSingle, 3>& InR0, const TVector<FRealSingle, 3>& InW, const FRealSingle InDt);

		/**
		 * Check that two rotations are approximately equal. Assumes the quaternions are normalized and in the same hemisphere.
		 * For small values of Epsilon, this is approximately equivalent to checking that the rotations are within 2*Epsilon
		 * radians of each other.
		 */
		static inline bool IsNearlyEqual(const TRotation<FRealSingle, 3>& A, const TRotation<FRealSingle, 3>& B, const FRealSingle Epsilon)
		{
			// Only check imaginary part. This is comparing Epsilon to 2*AngleDelta for small angle deltas
			return FMath::IsNearlyEqual(A.X, B.X, Epsilon)
				&& FMath::IsNearlyEqual(A.Y, B.Y, Epsilon)
				&& FMath::IsNearlyEqual(A.Z, B.Z, Epsilon);
		}
	};

	template<>
	class TRotation<FRealDouble, 3> : public UE::Math::TQuat<FRealDouble>
	{
		using BaseQuat = UE::Math::TQuat<FRealDouble>;
	public:
		TRotation()
		    : BaseQuat() {}
		TRotation(const BaseQuat& Quat)
		    : BaseQuat(Quat) {}
		TRotation(const FMatrix44d& Matrix)
		    : BaseQuat(Matrix) {}
		TRotation(const FMatrix44f& Matrix)
		    : BaseQuat(FMatrix44d(Matrix)) {}
		template<typename OtherType>
		TRotation(const UE::Math::TQuat<OtherType>& Other)
		    : BaseQuat((FRealDouble)Other.X, (FRealDouble)Other.Y, (FRealDouble)Other.Z, (FRealDouble)Other.W) {}

		inline PMatrix<FRealDouble, 3, 3> ToMatrix() const
		{
			PMatrix<FRealDouble, 3, 3> R;
			BaseQuat::ToMatrix(R);
			return R;
		}

		/**
		 * Extract the axis and angle from the Quaternion.
		 * @param OutAxis The axis of rotation.
		 * @param OutAngle The angle of rotation about the axis (radians).
		 * @param DefaultAxis The axis to set when the angle is too small to accurately calculate the axis.
		 * @param EpsilonSq The squared tolerance used to check for small angles.
		 * @return Whether the axis was successfully calculated (true except for very small angles around or less than Epsilon).
		 * @warning The axis calculation cannot succeed for small angles due to numerical error. In this case, the function will return false, but set the axis to DefaultAxis.
		 * @note EpsilonSq is approximately the square of the angle below which we cannot calculate the axis. It needs to be "much greater" than square of the error in the
		 * quaternion values which is usually ~1e-4, so values around 1e-3^2 = 1e-6 or greater are about right.
		 */
		inline bool ToAxisAndAngleSafe(TVector<FRealDouble, 3>& OutAxis, FRealDouble& OutAngle, const TVector<FRealDouble, 3>& DefaultAxis, FRealDouble EpsilionSq = 1e-6f) const
		{
			OutAngle = GetAngle();
			return GetRotationAxisSafe(OutAxis, DefaultAxis, EpsilionSq);
		}

		/**
		 * Extract the axis from the Quaternion.
		 * @param OutAxis The axis of rotation.
		 * @param DefaultAxis The axis to set when the angle is too small to accurately calculate the axis.
		 * @param EpsilonSq The squared tolerance used to check for small angles.
		 * @return Whether the axis was successfully calculated (true except for very small angles around or less than Epsilon).
		 * @warning The axis calculation cannot succeed for small angles due to numerical error. In this case, the function will return false, but set the axis to DefaultAxis.
		 * @note EpsilonSq is approximately the square of the angle below which we cannot calculate the axis. It needs to be "much greater" than square of the error in the
		 * quaternion values which is usually ~1e-4, so values around 1e-3^2 = 1e-6 or greater are about right.
		 */
		inline bool GetRotationAxisSafe(TVector<FRealDouble, 3>& OutAxis, const TVector<FRealDouble, 3>& DefaultAxis, FRealDouble EpsilionSq = 1e-6f) const
		{
			// Tolerance must be much larger than error in normalized vector (usually ~1e-4) for the 
			// axis calculation to succeed for small angles. For small angles, W ~= 1, and
			// X, Y, Z ~= 0. If the values of X, Y, Z are around 1e-4 we are just normalizing error.
			const FRealDouble LenSq = X * X + Y * Y + Z * Z;
			if (LenSq > EpsilionSq)
			{
				FRealDouble InvLen = FMath::InvSqrt(LenSq);
				OutAxis = TVector<FRealDouble, 3>(X * InvLen, Y * InvLen, Z * InvLen);
				return true;
			}

			OutAxis = DefaultAxis;
			return false;
		}

		/**
		* Convert to a matrix and return as the 3 matrix axes.
		*/
		inline void ToMatrixAxes(TVector<FRealDouble, 3>& OutX, TVector<FRealDouble, 3>& OutY, TVector<FRealDouble, 3>& OutZ)
		{
			const FRealDouble x2 = X + X;    const FRealDouble y2 = Y + Y;    const FRealDouble z2 = Z + Z;
			const FRealDouble xx = X * x2;   const FRealDouble xy = X * y2;   const FRealDouble xz = X * z2;
			const FRealDouble yy = Y * y2;   const FRealDouble yz = Y * z2;   const FRealDouble zz = Z * z2;
			const FRealDouble wx = W * x2;   const FRealDouble wy = W * y2;   const FRealDouble wz = W * z2;

			OutX = TVector<FRealDouble, 3>(1.0f - (yy + zz), xy + wz, xz - wy);
			OutY = TVector<FRealDouble, 3>(xy - wz, 1.0f - (xx + zz), yz + wx);
			OutZ = TVector<FRealDouble, 3>(xz + wy, yz - wx, 1.0f - (xx + yy));
		}

		/**
		 * Extract the Swing and Twist rotations, assuming that the Twist Axis is (1,0,0).
		 * /see ToSwingTwist
		 */
		inline void ToSwingTwistX(BaseQuat& OutSwing, BaseQuat& OutTwist) const
		{
			OutTwist = (X != 0.0f)? BaseQuat(X, 0, 0, W).GetNormalized() : BaseQuat::Identity;
			OutSwing = *this * OutTwist.Inverse();
		}

		/**
		 * @brief Return the large absolute element value
		*/
		FORCEINLINE FRealDouble GetAbsMax() const
		{
			return FMath::Max(FMath::Max(FMath::Abs(X), FMath::Abs(Y)), FMath::Max(FMath::Abs(Z), FMath::Abs(W)));
		}

		/**
		 * @brief Return the dot product of two quaternions
		*/
		static inline FRealDouble DotProduct(const TRotation<FRealDouble, 3>& L, const TRotation<FRealDouble, 3>& R)
		{
			return (L | R);
		}

		/**
		 * Return the complex conjugate of the rotation
		 */
		CHAOSCORE_API static TRotation<FRealDouble, 3> Conjugate(const ::Chaos::TRotation<FRealDouble, 3>& InR);

		/**
		 * Negate all values of the quaternion (note: not the inverse rotation. See Conjugate)
		 */
		CHAOSCORE_API static TRotation<FRealDouble, 3> Negate(const ::Chaos::TRotation<FRealDouble, 3>& InR);

		/**
		 * Create an identity rotation
		 */
		static inline TRotation<FRealDouble, 3> FromIdentity()
		{
			return BaseQuat(0, 0, 0, 1);
		}

		/**
		 * Create a rotation by explicitly specifying all elements
		 */
		static inline TRotation<FRealDouble, 3> FromElements(const FRealDouble X, const FRealDouble Y, const FRealDouble Z, const FRealDouble W)
		{
			using FQuatReal = BaseQuat::FReal;
			return BaseQuat((FQuatReal)X, (FQuatReal)Y, (FQuatReal)Z, (FQuatReal)W);
		}

		/**
		 * Create a rotation by explicitly specifying all elements
		 */
		static inline TRotation<FRealDouble, 3> FromElements(const ::Chaos::TVector<FRealDouble, 3>& V, const FRealDouble W)
		{
			return FromElements(V.X, V.Y, V.Z, W);
		}

		/**
		 * Create a rotation about an axis by an angle specified in radians
		 */
		static inline TRotation<FRealDouble, 3> FromAxisAngle(const ::Chaos::TVector<FRealDouble, 3>& Axis, const FRealDouble AngleRad)
		{
			return BaseQuat(UE::Math::TVector<FRealDouble>(Axis.X, Axis.Y, Axis.Z), (BaseQuat::FReal)AngleRad);
		}

		/**
		 * Create a rotation about an axis V/|V| by angle |V| in radians
		 */
		CHAOSCORE_API static TRotation<FRealDouble, 3> FromVector(const ::Chaos::TVector<FRealDouble, 3>& V);

		/**
		 * Generate a Rotation that would rotate vector InitialVector to FinalVector
		 */
		CHAOSCORE_API static TRotation<FRealDouble, 3> FromRotatedVector(
			const ::Chaos::TVector<FRealDouble, 3>& InitialVector,
			const ::Chaos::TVector<FRealDouble, 3>& FinalVector);

		/**
		 * Calculate the angular velocity required to take an object with orientation R0 to orientation R1 in time Dt.
		 *
		 * Uses the relation: DQ/DT = (W * Q)/2
		 */
		CHAOSCORE_API static TVector<FRealDouble, 3> CalculateAngularVelocity1(const TRotation<FRealDouble, 3>& R0, const TRotation<FRealDouble, 3>& InR1, const FRealDouble InDt);

		/**
		 * Calculate the angular velocity required to take an object with orientation R0 to orientation R1 in time Dt.
		 *
		 * Uses the Quaternion to Axis/Angle method.
		 */
		CHAOSCORE_API static TVector<FRealDouble, 3> CalculateAngularVelocity2(const TRotation<FRealDouble, 3>& R0, const TRotation<FRealDouble, 3>& InR1, const FRealDouble InDt);

		/**
		 * Calculate the angular velocity required to take an object with orientation R0 to orientation R1 in time Dt.
		 *
		 * This should match the algorithm used in PerParticleUpdateFromDeltaPosition rule.
		 */
		static inline TVector<FRealDouble, 3> CalculateAngularVelocity(const TRotation<FRealDouble, 3>& InR0, const TRotation<FRealDouble, 3>& InR1, const FRealDouble InDt)
		{
			return CalculateAngularVelocity1(InR0, InR1, InDt);
		}

		/**
		 * Calculate the axis-angle delta (angular velocity * dt) required to take an object with orientation R0 to orientation R1.
		 *
		 * This should match the algorithm used in PerParticleUpdateFromDeltaPosition rule.
		 */
		static inline TVector<FRealDouble, 3> CalculateAngularDelta(const TRotation<FRealDouble, 3>& InR0, const TRotation<FRealDouble, 3>& InR1)
		{
			return CalculateAngularVelocity(InR0, InR1, 1.0f);
		}

		/**
		 * Return a new rotation equal to the input rotation with angular velocity W applied over time Dt.
		 *
		 * Uses the relation: DQ/DT = (W * Q)/2
		 */
		CHAOSCORE_API static TRotation<FRealDouble, 3> IntegrateRotationWithAngularVelocity(const TRotation<FRealDouble, 3>& InR0, const TVector<FRealDouble, 3>& InW, const FRealDouble InDt);

		/**
		 * Check that two rotations are approximately equal. Assumes the quaternions are normalized and in the same hemisphere.
		 * For small values of Epsilon, this is approximately equivalent to checking that the rotations are within 2*Epsilon
		 * radians of each other.
		 */
		static inline bool IsNearlyEqual(const TRotation<FRealDouble, 3>& A, const TRotation<FRealDouble, 3>& B, const FRealDouble Epsilon)
		{
			// Only check imaginary part. This is comparing Epsilon to 2*AngleDelta for small angle deltas
			return FMath::IsNearlyEqual(A.X, B.X, Epsilon)
				&& FMath::IsNearlyEqual(A.Y, B.Y, Epsilon)
				&& FMath::IsNearlyEqual(A.Z, B.Z, Epsilon);
		}
	};
}
