// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/SimdTypes.h"

namespace Chaos
{
	namespace Private
	{
		// WIP: Not ready for public use. Keep in the Private namespace for now...
		// NOTE: the below only work with TNumLanes == 4 for now

		///////////////////////////////////////////////////////////////////////
		//
		// 4-wide float typedefs
		//
		///////////////////////////////////////////////////////////////////////

		using FSimd4Realf = TSimdRealf<4>;
		using FSimd4Vec3f = TSimdVec3f<4>;
		using FSimd4Int32 = TSimdInt32<4>;
		using FSimd4Selector = TSimdSelector<4>;

		///////////////////////////////////////////////////////////////////////
		//
		// 4-wide template specializations
		//
		///////////////////////////////////////////////////////////////////////

		template<>
		FORCEINLINE TSimdSelector<4> TSimdSelector<4>::True()
		{
			TSimdSelector<4> Selector;
			VectorIntStoreAligned(GlobalVectorConstants::IntAllMask, Selector.V);
			return Selector;
		}

		template<>
		FORCEINLINE TSimdSelector<4> TSimdSelector<4>::False()
		{
			TSimdSelector<4> Selector;
			VectorIntStoreAligned(GlobalVectorConstants::IntZero, Selector.V);
			return Selector;
		}

		template<>
		FORCEINLINE TSimdInt32<4> TSimdInt32<4>::Make(const int32 I)
		{
			TSimdInt32<4> R;
			VectorIntStoreAligned(MakeVectorRegisterInt(I, I, I, I), R.V);
			return R;
		}

		template<>
		FORCEINLINE TSimdInt32<4> TSimdInt32<4>::Zero()
		{
			TSimdInt32<4> I;
			VectorIntStoreAligned(GlobalVectorConstants::IntZero, I.V);
			return I;
		}

		template<>
		FORCEINLINE TSimdRealf<4> TSimdRealf<4>::Make(const float F)
		{
			TSimdRealf<4> Out;
			VectorStoreAligned(VectorSetFloat1(F), Out.V);
			return Out;
		}

		template<>
		FORCEINLINE TSimdRealf<4> TSimdRealf<4>::Zero()
		{
			TSimdRealf<4> Out;
			VectorStoreAligned(VectorZeroFloat(), Out.V);
			return Out;
		}

		template<>
		FORCEINLINE TSimdRealf<4> TSimdRealf<4>::One()
		{
			TSimdRealf<4> Out;
			VectorStoreAligned(VectorOneFloat(), Out.V);
			return Out;
		}

		
#if 0
		// Failed experiment!
		template<>
		FORCEINLINE TSimdVec3f<4> TSimdVec3f<4>::Make(const FVec3f* InV0, const FVec3f* InV1, const FVec3f* InV2, const FVec3f* InV3)
		{
			VectorRegister4f V0 = VectorLoadFloat3(InV0);
			VectorRegister4f V1 = VectorLoadFloat3(InV1);
			VectorRegister4f V2 = VectorLoadFloat3(InV2);
			VectorRegister4f V3 = VectorLoadFloat3(InV3);

			// Load V0 and V1 into the component vectors in first and second lanes
			VectorRegister4f Mask0100 = MakeVectorRegister((uint32)-1, (uint32)0, (uint32)-1, (uint32)-1);
			VectorRegister4f X = VectorSelect(Mask0100, VectorReplicate(V0, 0), VectorReplicate(V1, 0));
			VectorRegister4f Y = VectorSelect(Mask0100, VectorReplicate(V0, 1), VectorReplicate(V1, 1));
			VectorRegister4f Z = VectorSelect(Mask0100, VectorReplicate(V0, 2), VectorReplicate(V1, 2));

			// Load V2 into the component vectors in the third lane
			VectorRegister4f Mask0010 = MakeVectorRegister((uint32)-1, (uint32)-1, (uint32)0, (uint32)-1);
			X = VectorSelect(Mask0010, X, VectorReplicate(V2, 0));
			Y = VectorSelect(Mask0010, Y, VectorReplicate(V2, 1));
			Z = VectorSelect(Mask0010, Z, VectorReplicate(V2, 2));

			// Load V3 into the component vectors in the fourth lane
			VectorRegister4f Mask0001 = MakeVectorRegister((uint32)-1, (uint32)-1, (uint32)-1, (uint32)0);
			X = VectorSelect(Mask0001, X, VectorReplicate(V3, 0));
			Y = VectorSelect(Mask0001, Y, VectorReplicate(V3, 1));
			Z = VectorSelect(Mask0001, Z, VectorReplicate(V3, 2));

			TSimdVec3f<4> Out;
			VectorStoreAligned(X, Out.VX);
			VectorStoreAligned(Y, Out.VY);
			VectorStoreAligned(Z, Out.VZ);
			return Out;
		}
#endif

		///////////////////////////////////////////////////////////////////////
		//
		// 4-wide Logical operations
		//
		///////////////////////////////////////////////////////////////////////

		FORCEINLINE bool SimdAnyTrue(const FSimd4Selector& InL)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			return VectorMaskBits(L) != 0;
		}

		FORCEINLINE bool SimdAllTrue(const FSimd4Selector& InL)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			return ((VectorMaskBits(L) & 0xF) == 0xF);
		}

		FORCEINLINE FSimd4Selector SimdNot(const FSimd4Selector& InL)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
		
			VectorRegister4f Mask = VectorBitwiseNotAnd(L, GlobalVectorConstants::AllMask());

			FSimd4Selector Out;
			VectorStoreAligned(Mask, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Selector SimdOr(const FSimd4Selector& InL, const FSimd4Selector& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			VectorRegister4f Mask = VectorBitwiseOr(L, R);

			FSimd4Selector Out;
			VectorStoreAligned(Mask, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Selector SimdAnd(const FSimd4Selector& InL, const FSimd4Selector& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			VectorRegister4f Mask = VectorBitwiseAnd(L, R);

			FSimd4Selector Out;
			VectorStoreAligned(Mask, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Selector SimdEqual(const FSimd4Realf& InL, const FSimd4Realf& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			VectorRegister4f Mask = VectorCompareEQ(L, R);

			FSimd4Selector Out;
			VectorStoreAligned(Mask, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Selector SimdNotEqual(const FSimd4Realf& InL, const FSimd4Realf& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			VectorRegister4f Mask = VectorCompareNE(L, R);

			FSimd4Selector Out;
			VectorStoreAligned(Mask, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Selector SimdGreaterEqual(const FSimd4Int32& InL, const FSimd4Int32& InR)
		{
			VectorRegister4i L = VectorIntLoadAligned(InL.V);
			VectorRegister4i R = VectorIntLoadAligned(InR.V);

			union { VectorRegister4Float V; VectorRegister4Int I; } Mask;
			Mask.I = VectorIntCompareGE(L, R);

			FSimd4Selector Out;
			VectorStoreAligned(Mask.V, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Selector SimdGreaterEqual(const FSimd4Realf& InL, const FSimd4Realf& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			VectorRegister4f Mask = VectorCompareGE(L, R);

			FSimd4Selector Out;
			VectorStoreAligned(Mask, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Selector SimdGreater(const FSimd4Int32& InL, const FSimd4Int32& InR)
		{
			VectorRegister4i L = VectorIntLoadAligned(InL.V);
			VectorRegister4i R = VectorIntLoadAligned(InR.V);

			union { VectorRegister4Float V; VectorRegister4Int I; } Mask;
			Mask.I = VectorIntCompareGT(L, R);

			FSimd4Selector Out;
			VectorStoreAligned(Mask.V, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Selector SimdGreater(const FSimd4Realf& InL, const FSimd4Realf& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			VectorRegister4f Mask = VectorCompareGT(L, R);

			FSimd4Selector Out;
			VectorStoreAligned(Mask, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Selector SimdLess(const FSimd4Int32& InL, const FSimd4Int32& InR)
		{
			VectorRegister4i L = VectorIntLoadAligned(InL.V);
			VectorRegister4i R = VectorIntLoadAligned(InR.V);

			VectorRegister4i Mask = VectorIntCompareLT(L, R);

			FSimd4Selector Out;
			VectorIntStoreAligned(Mask, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Selector SimdLess(const FSimd4Realf& InL, const FSimd4Realf& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			VectorRegister4f Mask = VectorCompareLT(L, R);

			FSimd4Selector Out;
			VectorStoreAligned(Mask, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Realf SimdSelect(const FSimd4Selector& InSelector, const FSimd4Realf& InL, const FSimd4Realf& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);
			VectorRegister4f Mask = VectorLoadAligned(InSelector.V);

			VectorRegister4f Selected = VectorSelect(Mask, L, R);

			FSimd4Realf Out;
			VectorStoreAligned(Selected, Out.V);
			return Out;
		}

		///////////////////////////////////////////////////////////////////////
		//
		// 4-wide Math operations
		//
		///////////////////////////////////////////////////////////////////////

		FORCEINLINE FSimd4Realf SimdNegate(const FSimd4Realf& InL)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);

			VectorRegister4f Neg = VectorNegate(L);

			FSimd4Realf Out;
			VectorStoreAligned(Neg, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Realf SimdAdd(const FSimd4Realf& InL, const FSimd4Realf& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			VectorRegister4f Sum = VectorAdd(L, R);

			FSimd4Realf Out;
			VectorStoreAligned(Sum, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Vec3f SimdAdd(const FSimd4Vec3f& L, const FSimd4Vec3f& R)
		{
			VectorRegister4f LX = VectorLoadAligned(L.VX);
			VectorRegister4f LY = VectorLoadAligned(L.VY);
			VectorRegister4f LZ = VectorLoadAligned(L.VZ);

			VectorRegister4f RX = VectorLoadAligned(R.VX);
			VectorRegister4f RY = VectorLoadAligned(R.VY);
			VectorRegister4f RZ = VectorLoadAligned(R.VZ);

			FSimd4Vec3f Out;
			VectorStoreAligned(VectorAdd(LX, RX), Out.VX);
			VectorStoreAligned(VectorAdd(LY, RY), Out.VY);
			VectorStoreAligned(VectorAdd(LZ, RZ), Out.VZ);
			return Out;
		}

		FORCEINLINE FSimd4Realf SimdSubtract(const FSimd4Realf& InL, const FSimd4Realf& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			VectorRegister4f Diff = VectorSubtract(L, R);

			FSimd4Realf Out;
			VectorStoreAligned(Diff, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Vec3f SimdSubtract(const FSimd4Vec3f& L, const FSimd4Vec3f& R)
		{
			VectorRegister4f LX = VectorLoadAligned(L.VX);
			VectorRegister4f LY = VectorLoadAligned(L.VY);
			VectorRegister4f LZ = VectorLoadAligned(L.VZ);

			VectorRegister4f RX = VectorLoadAligned(R.VX);
			VectorRegister4f RY = VectorLoadAligned(R.VY);
			VectorRegister4f RZ = VectorLoadAligned(R.VZ);

			FSimd4Vec3f Out;
			VectorStoreAligned(VectorSubtract(LX, RX), Out.VX);
			VectorStoreAligned(VectorSubtract(LY, RY), Out.VY);
			VectorStoreAligned(VectorSubtract(LZ, RZ), Out.VZ);
			return Out;
		}

		FORCEINLINE FSimd4Realf SimdMultiply(const FSimd4Realf& InL, const FSimd4Realf& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			VectorRegister4f Product = VectorMultiply(L, R);

			FSimd4Realf Out;
			VectorStoreAligned(Product, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Vec3f SimdMultiply(const FSimd4Vec3f& L, const FSimd4Realf& InR)
		{
			VectorRegister4f LX = VectorLoadAligned(L.VX);
			VectorRegister4f LY = VectorLoadAligned(L.VY);
			VectorRegister4f LZ = VectorLoadAligned(L.VZ);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			FSimd4Vec3f Out;
			VectorStoreAligned(VectorMultiply(LX, R), Out.VX);
			VectorStoreAligned(VectorMultiply(LY, R), Out.VY);
			VectorStoreAligned(VectorMultiply(LZ, R), Out.VZ);
			return Out;
		}

		FORCEINLINE FSimd4Vec3f SimdMultiply(const FSimd4Realf& L, const FSimd4Vec3f& R)
		{
			return SimdMultiply(R, L);
		}

		FORCEINLINE FSimd4Vec3f SimdMultiply(const FSimd4Vec3f& L, const FSimd4Vec3f& R)
		{
			VectorRegister4f LX = VectorLoadAligned(L.VX);
			VectorRegister4f LY = VectorLoadAligned(L.VY);
			VectorRegister4f LZ = VectorLoadAligned(L.VZ);

			VectorRegister4f RX = VectorLoadAligned(R.VX);
			VectorRegister4f RY = VectorLoadAligned(R.VY);
			VectorRegister4f RZ = VectorLoadAligned(R.VZ);

			FSimd4Vec3f Out;
			VectorStoreAligned(VectorMultiply(LX, RX), Out.VX);
			VectorStoreAligned(VectorMultiply(LY, RY), Out.VY);
			VectorStoreAligned(VectorMultiply(LZ, RZ), Out.VZ);
			return Out;
		}

		FORCEINLINE FSimd4Vec3f SimdMultiplyAdd(const FSimd4Vec3f& L, const FSimd4Vec3f& R, const FSimd4Vec3f& Acc)
		{
			VectorRegister4f LX = VectorLoadAligned(L.VX);
			VectorRegister4f LY = VectorLoadAligned(L.VY);
			VectorRegister4f LZ = VectorLoadAligned(L.VZ);

			VectorRegister4f RX = VectorLoadAligned(R.VX);
			VectorRegister4f RY = VectorLoadAligned(R.VY);
			VectorRegister4f RZ = VectorLoadAligned(R.VZ);

			VectorRegister4f AX = VectorLoadAligned(Acc.VX);
			VectorRegister4f AY = VectorLoadAligned(Acc.VY);
			VectorRegister4f AZ = VectorLoadAligned(Acc.VZ);

			FSimd4Vec3f Out;
			VectorStoreAligned(VectorMultiplyAdd(LX, RX, AX), Out.VX);
			VectorStoreAligned(VectorMultiplyAdd(LY, RY, AY), Out.VY);
			VectorStoreAligned(VectorMultiplyAdd(LZ, RZ, AZ), Out.VZ);
			return Out;
		}

		FORCEINLINE FSimd4Realf SimdDivide(const FSimd4Realf& InL, const FSimd4Realf& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			VectorRegister4f Ratio = VectorDivide(L, R);

			FSimd4Realf Out;
			VectorStoreAligned(Ratio, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Vec3f SimdCrossProduct(const FSimd4Vec3f& L, const FSimd4Vec3f& R)
		{
			VectorRegister4f LX = VectorLoadAligned(L.VX);
			VectorRegister4f LY = VectorLoadAligned(L.VY);
			VectorRegister4f LZ = VectorLoadAligned(L.VZ);

			VectorRegister4f RX = VectorLoadAligned(R.VX);
			VectorRegister4f RY = VectorLoadAligned(R.VY);
			VectorRegister4f RZ = VectorLoadAligned(R.VZ);

			FSimd4Vec3f Out;
			VectorStoreAligned(VectorSubtract(VectorMultiply(LY, RZ), VectorMultiply(LZ, RY)), Out.VX);
			VectorStoreAligned(VectorSubtract(VectorMultiply(LZ, RX), VectorMultiply(LX, RZ)), Out.VY);
			VectorStoreAligned(VectorSubtract(VectorMultiply(LX, RY), VectorMultiply(LY, RX)), Out.VZ);
			return Out;
		}

		FORCEINLINE FSimd4Realf SimdDotProduct(const FSimd4Vec3f& L, const FSimd4Vec3f& R)
		{
			VectorRegister4f LX = VectorLoadAligned(L.VX);
			VectorRegister4f LY = VectorLoadAligned(L.VY);
			VectorRegister4f LZ = VectorLoadAligned(L.VZ);

			VectorRegister4f RX = VectorLoadAligned(R.VX);
			VectorRegister4f RY = VectorLoadAligned(R.VY);
			VectorRegister4f RZ = VectorLoadAligned(R.VZ);

			VectorRegister4f Dot = VectorMultiply(LX, RX);
			Dot = VectorMultiplyAdd(LY, RY, Dot);
			Dot = VectorMultiplyAdd(LZ, RZ, Dot);

			FSimd4Realf Out;
			VectorStoreAligned(Dot, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Realf SimdSquare(const FSimd4Realf& InV)
		{
			VectorRegister4f V = VectorLoadAligned(InV.V);

			VectorRegister4f Product = VectorMultiply(V, V);

			FSimd4Realf Out;
			VectorStoreAligned(Product, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Realf SimdSqrt(const FSimd4Realf& InV)
		{
			VectorRegister4f V = VectorLoadAligned(InV.V);

			VectorRegister4f Sqrt = VectorSqrt(V);

			FSimd4Realf Out;
			VectorStoreAligned(Sqrt, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Realf SimdInvSqrt(const FSimd4Realf& InV)
		{
			VectorRegister4f V = VectorLoadAligned(InV.V);

			VectorRegister4f Sqrt = VectorReciprocalSqrt(V);

			FSimd4Realf Out;
			VectorStoreAligned(Sqrt, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Realf SimdMin(const FSimd4Realf& InL, const FSimd4Realf& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			VectorRegister4f Min = VectorMin(L, R);

			FSimd4Realf Out;
			VectorStoreAligned(Min, Out.V);
			return Out;
		}

		FORCEINLINE FSimd4Realf SimdMax(const FSimd4Realf& InL, const FSimd4Realf& InR)
		{
			VectorRegister4f L = VectorLoadAligned(InL.V);
			VectorRegister4f R = VectorLoadAligned(InR.V);

			VectorRegister4f Max = VectorMax(L, R);

			FSimd4Realf Out;
			VectorStoreAligned(Max, Out.V);
			return Out;
		}

		///////////////////////////////////////////////////////////////////////
		//
		// 4-wide Gather/Scatter operations
		//
		///////////////////////////////////////////////////////////////////////

		// Convert 4 row-vectors into 3 column-vectors
		// NOTE: The input vectors must be 16-byte aligned and padded to 16 bytes to avoid reading past valid memory
		FORCEINLINE FSimd4Vec3f SimdGatherAligned(const FVec3f& InA, const FVec3f& InB, const FVec3f& InC, const FVec3f& InD)
		{
			VectorRegister4f A = VectorLoadAligned(&InA.X);	// Ax Ay Az Aw
			VectorRegister4f B = VectorLoadAligned(&InB.X);	// Bx By Bz Bw
			VectorRegister4f C = VectorLoadAligned(&InC.X);	// Cx Cy Cz Cw
			VectorRegister4f D = VectorLoadAligned(&InD.X);	// Dx Dy Dz Dw

			// This can be done with fewer registers, but the compiler should figure that out
			// and its much easier to follow when left like this...
			VectorRegister4f P = VectorUnpackLo(A, C);		// Ax Cx Ay Cy
			VectorRegister4f Q = VectorUnpackLo(B, D);		// Bx Dx By Dy
			VectorRegister4f R = VectorUnpackHi(A, C);		// Az Cz Aw Cw
			VectorRegister4f S = VectorUnpackHi(B, D);		// Bz Dz Bw Dw
			
			VectorRegister4f X = VectorUnpackLo(P, Q);		// Ax Bx Cx Dx
			VectorRegister4f Y = VectorUnpackHi(P, Q);		// Ay By Cy Dy 
			VectorRegister4f Z = VectorUnpackLo(R, S);		// Az Bz Cz Dz

			FSimd4Vec3f Out;
			VectorStoreAligned(X, Out.VX);
			VectorStoreAligned(Y, Out.VY);
			VectorStoreAligned(Z, Out.VZ);
			return Out;
		}

	}
}