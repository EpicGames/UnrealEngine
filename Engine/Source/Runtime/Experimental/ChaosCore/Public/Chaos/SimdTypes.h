// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"
#include "Chaos/VectorUtility.h"
#include "Math/VectorRegister.h"

namespace Chaos
{
	namespace Private
	{
		//
		// WIP: Not ready for public use. Keep in the Private namespace for now...
		// 
		// NOTE: Only TNumLanes == 4 is supported for now
		//

		static const int SimdAlignment = 16;

		/**
		* \file SimdTypes.h
		* 
		* This file contains a set of classes to help with SIMD data storage and 
		* operations on that data.
		* E.g., sets of N 3-vectors are stored as 3 arrays of N floats (one for
		* the X elements, one for Y, one for Z).
		* 
		* Arranging data this way allows us to process N datasets for the cost of 1,
		* and eliminates the need to call any horizontal vector operations (such as
		* is required for the summation in a dot product of two vectors).
		* 
		* Examples:
		*
		*	For a real-world example see methods in FPBDCollisionSolverSimd like
		*	SolvePositionNoFriction, SolvePositionWithFriction, and SolveVelocity.
		* 
		*	// Hopefully your data is already stored in SIMD lanes, but if not you
		*	// can load it into SIMD registers with the factory functions and SetValue 
		*	// methods. This should be avoided if possible.
		* 
		*	FSimd4Realf R0;
		*	SimdR0.SetValue(0, -1);
		*	SimdR0.SetValue(1, 0);
		*	SimdR0.SetValue(2, 1);
		*	SimdR0.SetValue(3, 2);
		*	FSimd4Realf R1;
		*	SimdR1.SetValue(0, 3);
		*	SimdR1.SetValue(1, 4);
		*	SimdR1.SetValue(2, 5);
		*	SimdR1.SetValue(3, 6);
		*
		*	FSim4Selector IsR0GreaterThanZero = SimdGreater(R0, FSimd4Realf::Zero());
		* 
		*	// If it's likely that all lanes will agree on IsGreaterZero, it may be worth
		*	// skipping the code block when possible. This will depend how expensive the
		*	// branch is vs the code we skip and the probability of doing so.
		*	FSimd4Realf R2 = FSimd4Realf::Zero();
		*	if (SimdAnyTrue(IsR0GreaterThanZero))
		*	{
		*		R2 = SimdSelect(IsR0GreaterThanZero, R0, R1);	//	-> (3,4,1,2)
		*	}
		*/


		/**
		* The result of Simd comparison operations and used in SimdSelect.
		* 
		*	// Replace any non-positive values in R0 with the value from R1
		*	FSim4Selector IsR0GreaterThanZero = SimdGreater(R0, FSimd4Realf::Zero());
		*	R0 = SimdSelect(IsV0GreaterThanZero, R0, R1);
		* 
		*/
		template<int TNumLanes>
		struct TSimdSelector
		{
			alignas(SimdAlignment) float V[TNumLanes];

			FORCEINLINE void SetValue(const int LaneIndex, bool B)
			{
				reinterpret_cast<uint32*>(V)[LaneIndex] = B ? 0xFFFFFFFF : 0;
			}

			FORCEINLINE bool GetValue(const int LaneIndex) const
			{
				// Can't do float comparison because its a NaN for true
				return (reinterpret_cast<const uint32*>(V)[LaneIndex] != 0);
			}

			static TSimdSelector<TNumLanes> True();
			static TSimdSelector<TNumLanes> False();
		};

		/**
		* Used to store any value type in a TNumLanes-wide set. This is used primarily whenyou have 
		* non-numeric data that varies per lane. E.g., a pointer to some per-lane shared data.
		*/
		template<typename T, int TNumLanes>
		struct TSimdValue
		{
			using ValueType = T;
			alignas(SimdAlignment) ValueType V[TNumLanes];

			FORCEINLINE void SetValue(const int32 LaneIndex, const ValueType F)
			{
				V[LaneIndex] = F;
			}

			FORCEINLINE ValueType GetValue(const int32 LaneIndex) const
			{
				return V[LaneIndex];
			}

			FORCEINLINE void SetValues(const ValueType F)
			{
				for (int32 LaneIndex = 0; LaneIndex < TNumLanes; ++LaneIndex)
				{
					SetValue(LaneIndex, F);
				}
			}
		};

		/**
		* A TNumLanes-wide set of integers.
		*/
		template<int TNumLanes>
		struct TSimdInt32
		{
			alignas(SimdAlignment) int32 V[TNumLanes];

			FORCEINLINE TSimdInt32()
			{
			}

			FORCEINLINE void SetValue(const int32 LaneIndex, const int32 I)
			{
				V[LaneIndex] = I;
			}

			FORCEINLINE int32 GetValue(const int32 LaneIndex) const
			{
				return V[LaneIndex];
			}

			FORCEINLINE void SetValues(const int32 I)
			{
				*this = Make(I);
			}

			FORCEINLINE int32 GetMaxValue() const
			{
				int32 MaxValue = V[0];
				for (int32 LaneIndex = 1; LaneIndex < TNumLanes; ++LaneIndex)
				{
					MaxValue = FMath::Max(MaxValue, V[LaneIndex]);
				}
				return MaxValue;
			}

			static TSimdInt32<TNumLanes> Make(const int32 I);
			static TSimdInt32<TNumLanes> Zero();
		};

		/**
		* A TNumLanes-wide set of single-precision floats.
		*/
		template<int TNumLanes>
		struct TSimdRealf
		{
			alignas(SimdAlignment) float V[TNumLanes];

			FORCEINLINE TSimdRealf()
			{
			}

			FORCEINLINE void SetValue(const int32 LaneIndex, const float F)
			{
				V[LaneIndex] = F;
			}

			FORCEINLINE float GetValue(const int32 LaneIndex) const
			{
				return V[LaneIndex];
			}

			FORCEINLINE void SetValues(const float F)
			{
				*this = Make(F);
			}

			static TSimdRealf Make(const float F);
			static TSimdRealf Zero();
			static TSimdRealf One();
		};

		/**
		* A TNumLanes-wide set of floating point 3-vectors. The vector elements
		* are stored as 3 vectors of Xs, Ys and Zs.
		*
		*	FSimd4Vec3f C = SimdCrossProduct(A, B);
		* 
		*/
		template<int TNumLanes>
		struct TSimdVec3f
		{
			static_assert(TNumLanes == 4, "Other sizes not yet supported");

			alignas(SimdAlignment) float VX[TNumLanes];
			alignas(SimdAlignment) float VY[TNumLanes];
			alignas(SimdAlignment) float VZ[TNumLanes];

			FORCEINLINE TSimdVec3f()
			{
			}

			FORCEINLINE TSimdVec3f(const FVec3f& V)
			{
				SetValues(V);
			}

			FORCEINLINE void SetValue(const int32 LaneIndex, const FVec3f& V)
			{
				VX[LaneIndex] = V.X;
				VY[LaneIndex] = V.Y;
				VZ[LaneIndex] = V.Z;
			};

			FORCEINLINE FVec3f GetValue(const int32 LaneIndex) const
			{
				return FVec3f(VX[LaneIndex], VY[LaneIndex], VZ[LaneIndex]);
			}

			FORCEINLINE void SetValues(const FVec3f& V)
			{
				*this = Make(V);
			}

			FORCEINLINE static TSimdVec3f Make(const FVec3f& V)
			{
				TSimdVec3f Out;
				for (int32 LaneIndex = 0; LaneIndex < TNumLanes; ++LaneIndex)
				{
					Out.SetValue(LaneIndex, V);
				}
				return Out;
			}

			FORCEINLINE static TSimdVec3f Make(const FVec3f& V0, const FVec3f& V1, const FVec3f& V2, const FVec3f& V3)
			{
				TSimdVec3f Out;
				if (TNumLanes == 4)
				{
					Out.SetValue(0, V0);
					Out.SetValue(1, V1);
					Out.SetValue(2, V2);
					Out.SetValue(3, V3);

				}
				return Out;
			}
		};
	}
}