// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos::Sweeps
{
	enum class ESweepFlags
	{
		None = 0,
		MTD = (1 << 9),
	};
	ENUM_CLASS_FLAGS(ESweepFlags);

	CHAOSCORE_API bool SweepSphereVsSphere(const FVec3& SweepStart, const FVec3& SweepDir, const FReal SweepLength, const FReal Sphere0Radius, const FVec3& Sphere1Center, const FReal Sphere1Radius, const ESweepFlags Flags, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal);

	CHAOSCORE_API bool SweepSphereVsAabb(const FVec3& SweepStart, const FVec3& SweepDir, const FReal SweepLength, const FReal SphereRadius, const FVec3& AabbMin, const FVec3& AabbMax, const ESweepFlags Flags, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal, int32& OutFaceIndex);

	CHAOSCORE_API bool SweepSphereVsCapsule(const FVec3& SweepStart, const FVec3& SweepDir, const FReal SweepLength, const FReal SphereRadius,
		const FReal CapsuleRadius, const FReal CapsuleHeight, const FVec3& CapsuleAxis, const FVec3& CapsuleX1, const FVec3& CapsuleX2,
		const ESweepFlags Flags, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal);
} // namespace Chaos::Sweeps
