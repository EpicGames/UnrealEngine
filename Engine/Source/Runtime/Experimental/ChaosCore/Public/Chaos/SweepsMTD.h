// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Core.h"

namespace Chaos::Sweeps
{
	void SphereSphereMTD(const FVec3& Sphere0Center, const FReal Sphere0Radius, const FVec3& Sphere1Center, const FReal Sphere1Radius, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal);

	void AabbSphereMTD(const FVec3& AabbMin, const FVec3& AabbMax, const FVec3& SphereCenter, const FReal SphereRadius, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal);

	void CapsuleSphereMTD(const FVec3& CapsuleX1, const FVec3& CapsuleX2, const FReal CapsuleRadius, const FVec3& SphereCenter, const FReal SphereRadius, FReal& OutTime, FVec3& OutPosition, FVec3& OutNormal);
} // namespace Chaos::Sweeps
