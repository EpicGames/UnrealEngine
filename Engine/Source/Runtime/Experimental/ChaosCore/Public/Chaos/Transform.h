// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Real.h"
#include "Chaos/Matrix.h"
#include "Chaos/Rotation.h"
#include "Chaos/Vector.h"

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Math/Transform.h"
#else
//TODO(mlentine): If we use this class in engine we need to make it more efficient.
struct _FTransform
{
public:
	_FTransform() {}
	_FTransform(const Chaos::TRotation<Chaos::FReal, 3>& Rotation, const Chaos::TVector<Chaos::FReal, 3>& Translation)
		: MRotation(Rotation), MTranslation(Translation)
	{
	}
	_FTransform(const FMatrix& Matrix)
	{
		MTranslation[0] = Matrix.M[0][3];
		MTranslation[1] = Matrix.M[1][3];
		MTranslation[2] = Matrix.M[2][3];

		Chaos::FReal angle = sqrt(Matrix.M[0][0] * Matrix.M[0][0] + Matrix.M[1][0] * Matrix.M[1][0]);
		if (angle > 1e-6)
		{
			MRotation[0] = atan2(Matrix.M[2][1], Matrix.M[2][2]);
			MRotation[1] = atan2(-Matrix.M[2][0], angle);
			MRotation[2] = atan2(Matrix.M[1][0], Matrix.M[0][0]);
		}
		else
		{
			MRotation[0] = atan2(-Matrix.M[1][2], Matrix.M[1][1]);
			MRotation[1] = atan2(-Matrix.M[2][0], angle);
			MRotation[2] = 0;
		}
	}
	_FTransform(const _FTransform& Transform)
		: MRotation(Transform.MRotation), MTranslation(Transform.MTranslation)
	{
	}
	Chaos::TVector<Chaos::FReal, 3> InverseTransformPosition(const Chaos::TVector<Chaos::FReal, 3>& Position)
	{
		Chaos::TVector<Chaos::FReal, 4> Position4(Position[0], Position[1], Position[2], 1);
		Chaos::TVector<Chaos::FReal, 4> NewPosition = ToInverseMatrix() * Position4;
		return Chaos::TVector<Chaos::FReal, 3>(NewPosition[0], NewPosition[1], NewPosition[2]);
	}
	Chaos::TVector<Chaos::FReal, 3> TransformVector(const Chaos::TVector<Chaos::FReal, 3>& Vector)
	{
		Chaos::TVector<Chaos::FReal, 4> Vector4(Vector[0], Vector[1], Vector[2], 0);
		Chaos::TVector<Chaos::FReal, 4> NewVector = ToMatrix() * Vector4;
		return Chaos::TVector<Chaos::FReal, 3>(NewVector[0], NewVector[1], NewVector[2]);
	}
	Chaos::TVector<Chaos::FReal, 3> InverseTransformVector(const Chaos::TVector<Chaos::FReal, 3>& Vector)
	{
		Chaos::TVector<Chaos::FReal, 4> Vector4(Vector[0], Vector[1], Vector[2], 0);
		Chaos::TVector<Chaos::FReal, 4> NewVector = ToInverseMatrix() * Vector4;
		return Chaos::TVector<Chaos::FReal, 3>(NewVector[0], NewVector[1], NewVector[2]);
	}
	Chaos::TVector<float, 3> InverseTransformVector(const Chaos::TVector<float, 3>& Vector)
	{
		Chaos::TVector<float, 4> Vector4(Vector[0], Vector[1], Vector[2], 0);
		Chaos::TVector<float, 4> NewVector = ToInverseMatrix() * Vector4;
		return Chaos::TVector<float, 3>(NewVector[0], NewVector[1], NewVector[2]);
	}
	Chaos::PMatrix<Chaos::FReal, 3, 3> ToRotationMatrix()
	{
		return Chaos::PMatrix<Chaos::FReal, 3, 3>(
			cos(MRotation[0]), sin(MRotation[0]), 0,
			-sin(MRotation[0]), cos(MRotation[0]), 0,
			0, 0, 1) *
			Chaos::PMatrix<Chaos::FReal, 3, 3>(
				cos(MRotation[1]), 0, -sin(MRotation[1]),
				0, 1, 0,
				sin(MRotation[1]), 0, cos(MRotation[1])) *
			Chaos::PMatrix<Chaos::FReal, 3, 3>(
				1, 0, 0,
				0, cos(MRotation[2]), sin(MRotation[2]),
				0, -sin(MRotation[2]), cos(MRotation[2]));
	}
	Chaos::PMatrix<Chaos::FReal, 4, 4> ToMatrix()
	{
		auto RotationMatrix = ToRotationMatrix();
		return Chaos::PMatrix<Chaos::FReal, 4, 4>(
			RotationMatrix.M[0][0], RotationMatrix.M[1][0], RotationMatrix.M[2][0], 0,
			RotationMatrix.M[0][1], RotationMatrix.M[1][1], RotationMatrix.M[2][1], 0,
			RotationMatrix.M[0][2], RotationMatrix.M[1][2], RotationMatrix.M[2][2], 0,
			MTranslation[0], MTranslation[1], MTranslation[2], 1);
	}
	Chaos::PMatrix<Chaos::FReal, 4, 4> ToInverseMatrix()
	{
		auto RotationMatrix = ToRotationMatrix().GetTransposed();
		auto Vector = (RotationMatrix * MTranslation) * -1;
		return Chaos::PMatrix<Chaos::FReal, 4, 4>(
			RotationMatrix.M[0][0], RotationMatrix.M[1][0], RotationMatrix.M[2][0], 0,
			RotationMatrix.M[0][1], RotationMatrix.M[1][1], RotationMatrix.M[2][1], 0,
			RotationMatrix.M[0][2], RotationMatrix.M[1][2], RotationMatrix.M[2][2], 0,
			Vector[0], Vector[1], Vector[2], 1);
	}

private:
	Chaos::TRotation<Chaos::FReal, 3> MRotation;
	Chaos::TVector<Chaos::FReal, 3> MTranslation;
};
using FTransform = _FTransform;	// Work around include tool not understanding that this can't be compiled alongside MathFwd.h
#endif

namespace Chaos
{
	template<class T, int d>
	class TRigidTransform
	{
	private:
		TRigidTransform() {}
		~TRigidTransform() = default;
	};

	template<>
	class TRigidTransform<FReal, 2> : public UE::Math::TTransform<FReal>
	{
		using BaseTransform = UE::Math::TTransform<FReal>;
	public:
		TRigidTransform()
			: BaseTransform() {}
		TRigidTransform(const TVector<FReal, 3>& Translation, const TRotation<FReal, 3>& Rotation)
			: BaseTransform(Rotation, Translation) {}
		TRigidTransform(const FMatrix44d& Matrix)
			: BaseTransform(Matrix) {}
		TRigidTransform(const FMatrix44f& Matrix)
			: BaseTransform(FMatrix(Matrix)) {}
		TRigidTransform(const BaseTransform& Transform)
			: BaseTransform(Transform) {}
		TRigidTransform<FReal, 2> Inverse() const
		{
			return BaseTransform::Inverse();
		}

		inline TRigidTransform<FReal, 2> operator*(const TRigidTransform<FReal, 2>& Other) const
		{
			return BaseTransform::operator*(Other);
		}
	};

	template<>
	class TRigidTransform<FRealSingle, 3> : public UE::Math::TTransform<FRealSingle>
	{
		using BaseTransform = UE::Math::TTransform<FRealSingle>;
	public:
		TRigidTransform()
			: BaseTransform() {}
		TRigidTransform(const TVector<FRealSingle, 3>& Translation, const TRotation<FRealSingle, 3>& Rotation)
			: BaseTransform(Rotation, Translation) {}
		TRigidTransform(const TVector<FRealSingle, 3>& Translation, const TRotation<FRealSingle, 3>& Rotation, const TVector<FRealSingle, 3>& Scale)
			: BaseTransform(Rotation, Translation, Scale) {}
		TRigidTransform(const FMatrix44f& Matrix)
			: BaseTransform(Matrix) {}
		TRigidTransform(const FMatrix44d& Matrix)
			: BaseTransform(FMatrix44f(Matrix)) {}
		TRigidTransform(TransformVectorRegister InRotation, TransformVectorRegister InTranslation)
			: BaseTransform(InRotation, InTranslation, GlobalVectorConstants::Float1110) {}
		TRigidTransform(const BaseTransform& Transform)
			: BaseTransform(Transform) {}
		template<typename OtherType>
		explicit TRigidTransform(const OtherType& Other)
			: BaseTransform(TRotation<FRealSingle, 3>(Other.GetRotation()), TVector<FRealSingle, 3>(Other.GetTranslation())) {}

		TRigidTransform<FRealSingle, 3> Inverse() const
		{
			return BaseTransform::Inverse();
		}

		PMatrix<FRealSingle, 4, 4> ToMatrixWithScale() const
		{
			return BaseTransform::ToMatrixWithScale();
		}

		PMatrix<FRealSingle, 4, 4> ToMatrixNoScale() const
		{
			return BaseTransform::ToMatrixNoScale();
		}

		CHAOSCORE_API PMatrix<FRealSingle, 4, 4> operator*(const PMatrix<FRealSingle, 4, 4>& Matrix) const;
		
		inline TRigidTransform<FRealSingle, 3> operator*(const TRigidTransform<FRealSingle, 3>& Other) const
		{
			return BaseTransform::operator*(Other);
		}

		// Get the transform which maps from Other to This, ignoring the scale on both.
		TRigidTransform<FRealSingle, 3> GetRelativeTransformNoScale(const TRigidTransform<FRealSingle, 3>& Other) const
		{
#if ENABLE_VECTORIZED_TRANSFORM
			// A * B(-1) = VQS(B)(-1) (VQS (A))
			// 
			// Rotation = Q(B)(-1) * Q(A)
			// Translation = [Q(B)(-1)*(T(A)-T(B))*Q(B)]
			// where A = this, B = Other

			//VQTranslation = (  ( T(A).X - T(B).X ),  ( T(A).Y - T(B).Y ), ( T(A).Z - T(B).Z), 0.f );
			TransformVectorRegister VQTranslation = VectorSet_W0(VectorSubtract(Translation, Other.Translation));

			// Inverse RotatedTranslation
			TransformVectorRegister VInverseRot = VectorQuaternionInverse(Other.Rotation);
			TransformVectorRegister VTranslation = VectorQuaternionRotateVector(VInverseRot, VQTranslation);

			// Rotation = Q(B)(-1) * Q(A)	
			TransformVectorRegister VRotation = VectorQuaternionMultiply2(VInverseRot, Rotation);

			return TRigidTransform<FRealSingle, 3>(VRotation, VTranslation);
#else
			// @todo(chaos): optimize
			TRotation<FRealSingle, 3> OtherInverse = Other.GetRotation().Inverse();
			return TRigidTransform<FRealSingle, 3>(
				(OtherInverse * (GetTranslation() - Other.GetTranslation())),
				OtherInverse * GetRotation());
#endif
		}

		TVector<FRealSingle, 3> TransformNormalNoScale(const TVector<FRealSingle, 3>& Normal) const
		{
			return TransformVectorNoScale(Normal);
		}

		// Transform the normal when scale may be non-unitary. Assumes no scale components are zero.
		TVector<FRealSingle, 3> TransformNormalUnsafe(const TVector<FRealSingle, 3>& Normal) const
		{
			const TVector<FRealSingle, 3> ScaledNormal = Normal / GetScale3D();
			const FRealSingle ScaledNormal2 = ScaledNormal.SizeSquared();
			if (ScaledNormal2 > UE_SMALL_NUMBER)
			{
				return TransformNormalNoScale(ScaledNormal * FMath::InvSqrt(ScaledNormal2));
			}
			else
			{
				return TransformNormalNoScale(Normal);
			}
		}

		// Transform the normal when scale may be non-unitary.
		TVector<FRealSingle, 3> TransformNormal(const TVector<FRealSingle, 3>& Normal) const
		{
			// Apply inverse scaling without a per-component divide (conceptually, by scaling the normal by an extra det*sign(det))
			const TVector<FRealSingle, 3>& S = GetScale3D();
			FRealSingle DetSign = (S.X * S.Y * S.Z) < 0 ? (FRealSingle)-1 : (FRealSingle)1;
			const TVector<FRealSingle, 3> SafeInvS(S.Y * S.Z * DetSign, S.X * S.Z * DetSign, S.X * S.Y * DetSign);
			const TVector<FRealSingle, 3> ScaledNormal = SafeInvS * Normal;
			const FRealSingle ScaledNormalLengthSq = ScaledNormal.SizeSquared();
			// If inverse scaling would not zero the normal, normalize it and rotate
			if (ScaledNormalLengthSq > UE_SMALL_NUMBER)
			{
				return TransformVectorNoScale(ScaledNormal * FMath::InvSqrt(ScaledNormalLengthSq));
			}
			// Otherwise just rotate without scaling
			else
			{
				return TransformVectorNoScale(Normal);
			}
		}

		/**
		 * @brief Equivalent to (A * B) but assuming both have unit scale
		*/
		static TRigidTransform<FRealSingle, 3> MultiplyNoScale(const TRigidTransform<FRealSingle, 3>& A, const TRigidTransform<FRealSingle, 3>& B)
		{
			TRigidTransform<FRealSingle, 3> Result;

#if ENABLE_VECTORIZED_TRANSFORM
			const TransformVectorRegister QuatA = A.Rotation;
			const TransformVectorRegister QuatB = B.Rotation;
			const TransformVectorRegister TranslateA = A.Translation;
			const TransformVectorRegister TranslateB = B.Translation;

			const TransformVectorRegister Rotation = VectorQuaternionMultiply2(QuatB, QuatA);
			const TransformVectorRegister RotatedTranslate = VectorQuaternionRotateVector(QuatB, TranslateA);
			const TransformVectorRegister Translation = VectorAdd(RotatedTranslate, TranslateB);

			Result.Rotation = Rotation;
			Result.Translation = Translation;
			Result.Scale3D = VectorOne();
#else
			Result.Rotation = B.Rotation * A.Rotation;
			Result.Translation = B.Rotation * A.Translation + B.Translation;
			Result.Scale3D = FVector3f::OneVector;
#endif

			return Result;
		}

		friend inline uint32 GetTypeHash(const TRigidTransform<FRealSingle, 3>& InTransform)
		{
			return HashCombine(GetTypeHash(InTransform.GetTranslation()), HashCombine(GetTypeHash(InTransform.GetRotation().Euler()), GetTypeHash(InTransform.GetScale3D())));
		}
	};

	template<>
	class TRigidTransform<FRealDouble, 3> : public UE::Math::TTransform<FRealDouble>
	{
		using BaseTransform = UE::Math::TTransform<FRealDouble>;
	public:
		TRigidTransform()
			: BaseTransform() {}
		TRigidTransform(const TVector<FRealDouble, 3>& Translation, const TRotation<FRealDouble, 3>& Rotation)
			: BaseTransform(Rotation, Translation) {}
		TRigidTransform(const TVector<FRealDouble, 3>& Translation, const TRotation<FRealDouble, 3>& Rotation, const TVector<FRealDouble, 3>& Scale)
			: BaseTransform(Rotation, Translation, Scale) {}
		TRigidTransform(const FMatrix44d& Matrix)
			: BaseTransform(Matrix) {}
		TRigidTransform(const FMatrix44f& Matrix)
			: BaseTransform(FMatrix44d(Matrix)) {}
		TRigidTransform(TransformVectorRegister InRotation, TransformVectorRegister InTranslation)
			: BaseTransform(InRotation, InTranslation, GlobalVectorConstants::Double1110) {}
		TRigidTransform(const BaseTransform& Transform)
			: BaseTransform(Transform) {}
		template<typename OtherType>
		explicit TRigidTransform(const OtherType& Other)
			: BaseTransform(TRotation<FRealDouble, 3>(Other.GetRotation()), TVector<FRealDouble, 3>(Other.GetTranslation())) {}

		TRigidTransform<FRealDouble, 3> Inverse() const
		{
			return BaseTransform::Inverse();
		}

		PMatrix<FRealDouble, 4, 4> ToMatrixWithScale() const
		{
			return BaseTransform::ToMatrixWithScale();
		}

		PMatrix<FRealDouble, 4, 4> ToMatrixNoScale() const
		{
			return BaseTransform::ToMatrixNoScale();
		}

		CHAOSCORE_API PMatrix<FRealDouble, 4, 4> operator*(const Chaos::PMatrix<FRealDouble, 4, 4>& Matrix) const;
		
		inline TRigidTransform<FRealDouble, 3> operator*(const TRigidTransform<FRealDouble, 3>& Other) const
		{
			return BaseTransform::operator*(Other);
		}

		// Get the transform which maps from Other to This, ignoring the scale on both.
		TRigidTransform<FRealDouble, 3> GetRelativeTransformNoScale(const TRigidTransform<FRealDouble, 3>& Other) const
		{
#if ENABLE_VECTORIZED_TRANSFORM
			// A * B(-1) = VQS(B)(-1) (VQS (A))
			// 
			// Rotation = Q(B)(-1) * Q(A)
			// Translation = [Q(B)(-1)*(T(A)-T(B))*Q(B)]
			// where A = this, B = Other

			//VQTranslation = (  ( T(A).X - T(B).X ),  ( T(A).Y - T(B).Y ), ( T(A).Z - T(B).Z), 0.f );
			TransformVectorRegister VQTranslation = VectorSet_W0(VectorSubtract(Translation, Other.Translation));

			// Inverse RotatedTranslation
			TransformVectorRegister VInverseRot = VectorQuaternionInverse(Other.Rotation);
			TransformVectorRegister VTranslation = VectorQuaternionRotateVector(VInverseRot, VQTranslation);

			// Rotation = Q(B)(-1) * Q(A)	
			TransformVectorRegister VRotation = VectorQuaternionMultiply2(VInverseRot, Rotation);

			return TRigidTransform<FRealDouble, 3>(VRotation, VTranslation);
#else
			// @todo(chaos): optimize
			TRotation<FRealDouble, 3> OtherInverse = Other.GetRotation().Inverse();
			return TRigidTransform<FRealDouble, 3>(
				(OtherInverse * (GetTranslation() - Other.GetTranslation())),
				OtherInverse * GetRotation());
#endif
		}

		TVector<FRealDouble, 3> TransformNormalNoScale(const TVector<FRealDouble, 3>& Normal) const
		{
			return TransformVectorNoScale(Normal);
		}

		// Transform the normal when scale may be non-unitary. Assumes no scale components are zero.
		TVector<FRealDouble, 3> TransformNormalUnsafe(const TVector<FRealDouble, 3>& Normal) const
		{
			const TVector<FRealDouble, 3> ScaledNormal = Normal / GetScale3D();
			const FRealDouble ScaledNormal2 = ScaledNormal.SizeSquared();
			if (ScaledNormal2 > UE_SMALL_NUMBER)
			{
				return TransformNormalNoScale(ScaledNormal * FMath::InvSqrt(ScaledNormal2));
			}
			else
			{
				return TransformNormalNoScale(Normal);
			}
		}

		// Transform the normal when scale may be non-unitary.
		TVector<FRealDouble, 3> TransformNormal(const TVector<FRealDouble, 3>& Normal) const
		{
			// Apply inverse scaling without a per-component divide (conceptually, by scaling the normal by an extra det*sign(det))
			const TVector<FRealDouble, 3>& S = GetScale3D();
			FRealDouble DetSign = (S.X * S.Y * S.Z) < 0 ? (FRealDouble)-1 : (FRealDouble)1;
			const TVector<FRealDouble, 3> SafeInvS(S.Y * S.Z * DetSign, S.X * S.Z * DetSign, S.X * S.Y * DetSign);
			const TVector<FRealDouble, 3> ScaledNormal = SafeInvS * Normal;
			const FRealDouble ScaledNormalLengthSq = ScaledNormal.SizeSquared();
			// If inverse scaling would not zero the normal, normalize it and rotate
			if (ScaledNormalLengthSq > UE_SMALL_NUMBER)
			{
				return TransformVectorNoScale(ScaledNormal * FMath::InvSqrt(ScaledNormalLengthSq));
			}
			// Otherwise just rotate without scaling
			else
			{
				return TransformVectorNoScale(Normal);
			}
		}

		/**
		 * @brief Equivalent to (A * B) but assuming both have unit scale
		*/
		static TRigidTransform<FRealDouble, 3> MultiplyNoScale(const TRigidTransform<FRealDouble, 3>& A, const TRigidTransform<FRealDouble, 3>& B)
		{
			TRigidTransform<FRealDouble, 3> Result;

#if ENABLE_VECTORIZED_TRANSFORM
			const TransformVectorRegister QuatA = A.Rotation;
			const TransformVectorRegister QuatB = B.Rotation;
			const TransformVectorRegister TranslateA = A.Translation;
			const TransformVectorRegister TranslateB = B.Translation;

			const TransformVectorRegister Rotation = VectorQuaternionMultiply2(QuatB, QuatA);
			const TransformVectorRegister RotatedTranslate = VectorQuaternionRotateVector(QuatB, TranslateA);
			const TransformVectorRegister Translation = VectorAdd(RotatedTranslate, TranslateB);

			Result.Rotation = Rotation;
			Result.Translation = Translation;
			Result.Scale3D = VectorOne();
#else
			Result.Rotation = B.Rotation * A.Rotation;
			Result.Translation = B.Rotation * A.Translation + B.Translation;
			Result.Scale3D = FVector3d::OneVector;
#endif

			return Result;
		}

		friend inline uint32 GetTypeHash(const TRigidTransform<FRealDouble, 3>& InTransform)
		{
			return HashCombine(GetTypeHash(InTransform.GetTranslation()), HashCombine(GetTypeHash(InTransform.GetRotation().Euler()), GetTypeHash(InTransform.GetScale3D())));
		}
	};
}
