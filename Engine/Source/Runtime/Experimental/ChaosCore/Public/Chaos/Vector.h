// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Real.h"
#include "Chaos/Array.h"
#include "Chaos/Pair.h"

#include "Containers/StaticArray.h"

#include <initializer_list>

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Vector4.h"
#else
#include <cmath>
#include <iostream>
#include <utility>
#include <limits>
namespace FMath
{
	float Sqrt(float v) { return sqrt(v); }
	double Sqrt(double v) { return sqrt(v); }
	float Atan2(float x, float y) { return atan2(x, y); }
	double Atan2(double x, double y) { return atan2(x, y); }
}
#endif

//-V:TVector:546 PVS 7.41 bug
namespace Chaos
{
	template<class T, int d>
	struct TVectorTraits
	{
		static const bool RequireRangeCheck = true;
	};


	template<class T, int d>
	class TVector
	{
	public:
		using FElement = T;
		static const int NumElements = d;
		using FTraits = TVectorTraits<T, d>;

		TVector(const TVector&) = default;
		TVector& operator=(const TVector&) = default;

		TVector() = default;

		explicit TVector(const FElement& Element)
		{
			for (int32 i = 0; i < NumElements; ++i)
			{
				V[i] = Element;
			}
		}

		TVector(std::initializer_list<T> InElements)
		{
			check(InElements.size() == NumElements);
			FElement* Element = &V[0];
			for (auto InIt = InElements.begin(); InIt != InElements.end(); ++InIt)
			{
				*Element++ = *InIt;
			}
		}

		template <int N=d, typename std::enable_if<N==2, int>::type = 0>
		TVector(const T& V0, const T& V1)
		{
			V[0] = V0;
			V[1] = V1;
		}

		template <int N=d, typename std::enable_if<N==3, int>::type = 0>
		TVector(const T& V0, const T& V1, const T& V2)
		{
			V[0] = V0;
			V[1] = V1;
			V[2] = V2; //-V557
		}

		template <int N=d, typename std::enable_if<N==4, int>::type = 0>
		TVector(const T& V0, const T& V1, const T& V2, const T& V3)
		{
			V[0] = V0;
			V[1] = V1;
			V[2] = V2; //-V557
			V[3] = V3; //-V557
		}

#if !COMPILE_WITHOUT_UNREAL_SUPPORT
		template <int N=d, typename std::enable_if<N==3, int>::type = 0>
		TVector(const FVector3f& Other)	// LWC_TODO: Make this explicit for FReal = double
		{
			V[0] = static_cast<FElement>(Other.X);
			V[1] = static_cast<FElement>(Other.Y);
			V[2] = static_cast<FElement>(Other.Z); //-V557
		}
		template <int N = d, typename std::enable_if<N == 3, int>::type = 0>
		TVector(const FVector3d& Other)
		{
			V[0] = static_cast<FElement>(Other.X);
			V[1] = static_cast<FElement>(Other.Y);
			V[2] = static_cast<FElement>(Other.Z); //-V557
		}
#endif

		template<class T2>
		TVector(const TVector<T2, d>& Other)
		{
			for (int32 i = 0; i < NumElements; ++i)
			{
				V[i] = static_cast<T>(Other[i]);
			}
		}

		template<class T2>
		TVector<T, d>& operator=(const TVector<T2, d>& Other)
		{
			for (int32 i = 0; i < NumElements; ++i)
			{
				(*this)[i] = Other[i];
			}
			return *this;
		}

		FElement& operator[](int Index)
		{
			if (FTraits::RequireRangeCheck)
			{
				check(Index >= 0);
				check(Index < NumElements);
			}
			return V[Index];
		}

		const FElement& operator[](int Index) const
		{
			if (FTraits::RequireRangeCheck)
			{
				check(Index >= 0);
				check(Index < NumElements);
			}
			return V[Index];
		}

		int32 Num() const
		{
			return NumElements;
		}

#if COMPILE_WITHOUT_UNREAL_SUPPORT
		TVector(std::istream& Stream)
		{
			for (int32 i = 0; i < NumElements; ++i)
			{
				Stream.read(reinterpret_cast<char*>(&V[i]), sizeof(FElement));
			}
		}
		void Write(std::ostream& Stream) const
		{
			for (int32 i = 0; i < NumElements; ++i)
			{
				Stream.write(reinterpret_cast<const char*>(&V[i]), sizeof(FElement));
			}
		}
#endif
		friend bool operator==(const TVector<T, d>& L, const TVector<T, d>& R)
		{
			for (int32 i = 0; i < NumElements; ++i)
			{
				if (L.V[i] != R.V[i])
				{
					return false;
				}
			}
			return true;
		}

		friend bool operator!=(const TVector<T, d>& L, const TVector<T, d>& R)
		{
			return !(L == R);
		}

		bool ContainsNaN() const
		{
			for (int32 i = 0; i < NumElements; ++i)
			{
				if (!FMath::IsFinite(V[i]))
				{
					return true;
				}
			}

			return false;
		}


		// @todo(ccaulfield): the following should only be available for TVector of numeric types
//		T Size() const
//		{
//			T SquaredSum = 0;
//			for (int32 i = 0; i < NumElements; ++i)
//			{
//				SquaredSum += ((*this)[i] * (*this)[i]);
//			}
//			return FMath::Sqrt(SquaredSum);
//		}
//		T Product() const
//		{
//			T Result = 1;
//			for (int32 i = 0; i < NumElements; ++i)
//			{
//				Result *= (*this)[i];
//			}
//			return Result;
//		}
//		static TVector<T, d> AxisVector(const int32 Axis)
//		{
//			check(Axis >= 0 && Axis < NumElements);
//			TVector<T, d> Result(0);
//			Result[Axis] = (T)1;
//			return Result;
//		}
//		T SizeSquared() const
//		{
//			T Result = 0;
//			for (int32 i = 0; i < d; ++i)
//			{
//				Result += ((*this)[i] * (*this)[i]);
//			}
//			return Result;
//		}
//		TVector<T, d> GetSafeNormal() const
//		{
//			//We want N / ||N|| and to avoid inf
//			//So we want N / ||N|| < 1 / eps => N eps < ||N||, but this is clearly true for all eps < 1 and N > 0
//			T SizeSqr = SizeSquared();
//			if (SizeSqr <= TNumericLimits<T>::Min())
//				return AxisVector(0);
//			return (*this) / FMath::Sqrt(SizeSqr);
//		}
//		T SafeNormalize()
//		{
//			T Size = SizeSquared();
//			if (Size < (T)1e-4)
//			{
//				*this = AxisVector(0);
//				return (T)0.;
//			}
//			Size = FMath::Sqrt(Size);
//			*this = (*this) / Size;
//			return Size;
//		}
//		TVector<T, d> operator-() const
//		{
//			TVector<T, d> Result;
//			for (int32 i = 0; i < Num; ++i)
//			{
//				Result[i] = -(*this)[i];
//			}
//			return Result;
//		}
//		TVector<T, d> operator*(const TVector<T, d>& Other) const
//		{
//			TVector<T, d> Result;
//			for (int32 i = 0; i < d; ++i)
//			{
//				Result[i] = (*this)[i] * Other[i];
//			}
//			return Result;
//		}
//		TVector<T, d> operator/(const TVector<T, d>& Other) const
//		{
//			TVector<T, d> Result;
//			for (int32 i = 0; i < d; ++i)
//			{
//				Result[i] = (*this)[i] / Other[i];
//			}
//			return Result;
//		}
//		TVector<T, d> operator+(const TVector<T, d>& Other) const
//		{
//			TVector<T, d> Result;
//			for (int32 i = 0; i < d; ++i)
//			{
//				Result[i] = (*this)[i] + Other[i];
//			}
//			return Result;
//		}
//		TVector<T, d> operator-(const TVector<T, d>& Other) const
//		{
//			TVector<T, d> Result;
//			for (int32 i = 0; i < d; ++i)
//			{
//				Result[i] = (*this)[i] - Other[i];
//			}
//			return Result;
//		}
//		TVector<T, d>& operator+=(const TVector<T, d>& Other)
//		{
//			for (int32 i = 0; i < d; ++i)
//			{
//				(*this)[i] += Other[i];
//			}
//			return *this;
//		}
//		TVector<T, d>& operator-=(const TVector<T, d>& Other)
//		{
//			for (int32 i = 0; i < d; ++i)
//			{
//				(*this)[i] -= Other[i];
//			}
//			return *this;
//		}
//		TVector<T, d>& operator/=(const TVector<T, d>& Other)
//		{
//			for (int32 i = 0; i < d; ++i)
//			{
//				(*this)[i] /= Other[i];
//			}
//			return *this;
//		}
//		TVector<T, d> operator*(const T& S) const
//		{
//			TVector<T, d> Result;
//			for (int32 i = 0; i < d; ++i)
//			{
//				Result[i] = (*this)[i] * S;
//			}
//			return Result;
//		}
//		friend TVector<T, d> operator*(const T S, const TVector<T, d>& V)
//		{
//			TVector<T, d> Ret;
//			for (int32 i = 0; i < d; ++i)
//			{
//				Ret[i] = S * V[i];
//			}
//			return Ret;
//		}
//		TVector<T, d>& operator*=(const T& S)
//		{
//			for (int32 i = 0; i < d; ++i)
//			{
//				(*this)[i] *= S;
//			}
//			return *this;
//		}
//		friend TVector<T, d> operator/(const T S, const TVector<T, d>& V)
//		{
//			TVector<T, d> Ret;
//			for (int32 i = 0; i < d; ++i)
//			{
//				Ret = S / V[i];
//			}
//			return Ret;
//		}
//
//#if COMPILE_WITHOUT_UNREAL_SUPPORT
//		static inline FReal DotProduct(const Vector<FReal, 3>& V1, const Vector<FReal, 3>& V2)
//		{
//			return V1[0] * V2[0] + V1[1] * V2[1] + V1[2] * V2[2];
//		}
//		static inline Vector<FReal, 3> CrossProduct(const Vector<FReal, 3>& V1, const Vector<FReal, 3>& V2)
//		{
//			Vector<FReal, 3> Result;
//			Result[0] = V1[1] * V2[2] - V1[2] * V2[1];
//			Result[1] = V1[2] * V2[0] - V1[0] * V2[2];
//			Result[2] = V1[0] * V2[1] - V1[1] * V2[0];
//			return Result;
//		}
//#endif

	private:
		FElement V[NumElements];
	};


#if !COMPILE_WITHOUT_UNREAL_SUPPORT
	template<>
	class TVector<FReal, 4> : public UE::Math::TVector4<FReal>
	{
	public:
		using FElement = FReal;
		using BaseType = UE::Math::TVector4<FReal>;
		using BaseType::X;
		using BaseType::Y;
		using BaseType::Z;
		using BaseType::W;

		TVector() = default;

		explicit TVector(const FReal x)
		    : BaseType(x, x, x, x) {}
		TVector(const FReal x, const FReal y, const FReal z, const FReal w)
		    : BaseType(x, y, z, w) {}
		TVector(const BaseType& vec)
		    : BaseType(vec) {}
	};

	template<>
	class TVector<FRealSingle, 3> : public UE::Math::TVector<FRealSingle>
	{
	public:
		using FElement = FRealSingle;
		using UE::Math::TVector<FRealSingle>::X;
		using UE::Math::TVector<FRealSingle>::Y;
		using UE::Math::TVector<FRealSingle>::Z;

		TVector() = default;

		explicit TVector(const FRealSingle x)
		    : UE::Math::TVector<FRealSingle>(x, x, x) {}
		TVector(const FRealSingle x, const FRealSingle y, const FRealSingle z)
		    : UE::Math::TVector<FRealSingle>(x, y, z) {}
		TVector(const UE::Math::TVector<FRealSingle>& vec)
		    : UE::Math::TVector<FRealSingle>((UE::Math::TVector<FRealSingle>)vec) {}
		TVector(const UE::Math::TVector<FRealDouble>& vec)					// LWC_TODO: Precision loss. Make explicit for FRealSingle = FRealSingle?
			: UE::Math::TVector<FRealSingle>((UE::Math::TVector<FRealSingle>)vec) {}
		TVector(const UE::Math::TVector4<FRealSingle>& vec)
		    : UE::Math::TVector<FRealSingle>(vec.X, vec.Y, vec.Z) {}
		TVector(const UE::Math::TVector4<FRealDouble>& vec)					// LWC_TODO: Precision loss. Make explicit for FRealSingle = FRealSingle?
			: UE::Math::TVector<FRealSingle>((UE::Math::TVector4<FRealSingle>)vec) {}

#if COMPILE_WITHOUT_UNREAL_SUPPORT
		TVector(std::istream& Stream)
		{
			Stream.read(reinterpret_cast<char*>(&X), sizeof(X));
			Stream.read(reinterpret_cast<char*>(&Y), sizeof(Y));
			Stream.read(reinterpret_cast<char*>(&Z), sizeof(Z));
		}
		//operator UE::Math::TVector<FRealDouble>() const { return UE::Math::TVector<FRealDouble>((FRealDouble)X, (FRealDouble)Y, (FRealDouble)Z); }
		void Write(std::ostream& Stream) const
		{
			Stream.write(reinterpret_cast<const char*>(&X), sizeof(X));
			Stream.write(reinterpret_cast<const char*>(&Y), sizeof(Y));
			Stream.write(reinterpret_cast<const char*>(&Z), sizeof(Z));
		}
#endif
		static inline TVector<FRealSingle, 3> Lerp(const TVector<FRealSingle, 3>& V1, const TVector<FRealSingle, 3>& V2, const FRealSingle F) { return FMath::Lerp<UE::Math::TVector<FRealSingle>, FRealSingle>(V1, V2, F); }
		static inline TVector<FRealSingle, 3> CrossProduct(const TVector<FRealSingle, 3>& V1, const TVector<FRealSingle, 3>& V2) { return UE::Math::TVector<FRealSingle>::CrossProduct(V1, V2); }
		static inline FRealSingle DotProduct(const TVector<FRealSingle, 3>& V1, const TVector<FRealSingle, 3>& V2) { return UE::Math::TVector<FRealSingle>::DotProduct(V1, V2); }
		bool operator<=(const TVector<FRealSingle, 3>& V) const
		{
			return X <= V.X && Y <= V.Y && Z <= V.Z;
		}
		bool operator>=(const TVector<FRealSingle, 3>& V) const
		{
			return X >= V.X && Y >= V.Y && Z >= V.Z;
		}
		TVector<FRealSingle, 3> operator-() const
		{
			return TVector<FRealSingle, 3>(-X, -Y, -Z);
		}
		TVector<FRealSingle, 3> operator+(const FRealSingle Other) const
		{
			return TVector<FRealSingle, 3>(X + Other, Y + Other, Z + Other);
		}
		TVector<FRealSingle, 3> operator-(const FRealSingle Other) const
		{
			return TVector<FRealSingle, 3>(X - Other, Y - Other, Z - Other);
		}
		TVector<FRealSingle, 3> operator*(const FRealSingle Other) const
		{
			return TVector<FRealSingle, 3>(X * Other, Y * Other, Z * Other);
		}
		TVector<FRealSingle, 3> operator/(const FRealSingle Other) const
		{
			return TVector<FRealSingle, 3>(X / Other, Y / Other, Z / Other);
		}
		friend TVector<FRealSingle, 3> operator/(const FRealSingle S, const TVector<FRealSingle, 3>& V)
		{
			return TVector<FRealSingle, 3>(S / V.X, S / V.Y, S / V.Z);
		}
		TVector<FRealSingle, 3> operator+(const TVector<FRealSingle, 3>& Other) const
		{
			return TVector<FRealSingle, 3>(X + Other[0], Y + Other[1], Z + Other[2]);
		}
		TVector<FRealSingle, 3> operator-(const TVector<FRealSingle, 3>& Other) const
		{
			return TVector<FRealSingle, 3>(X - Other[0], Y - Other[1], Z - Other[2]);
		}
		TVector<FRealSingle, 3> operator*(const TVector<FRealSingle, 3>& Other) const
		{
			return TVector<FRealSingle, 3>(X * Other[0], Y * Other[1], Z * Other[2]);
		}
		TVector<FRealSingle, 3> operator/(const TVector<FRealSingle, 3>& Other) const
		{
			return TVector<FRealSingle, 3>(X / Other[0], Y / Other[1], Z / Other[2]);
		}
		template<class T2>
		TVector<FRealSingle, 3> operator+(const TVector<T2, 3>& Other) const
		{
			return TVector<FRealSingle, 3>(X + static_cast<FRealSingle>(Other[0]), Y + static_cast<FRealSingle>(Other[1]), Z + static_cast<FRealSingle>(Other[2]));
		}
		template<class T2>
		TVector<FRealSingle, 3> operator-(const TVector<T2, 3>& Other) const
		{
			return TVector<FRealSingle, 3>(X - static_cast<FRealSingle>(Other[0]), Y - static_cast<FRealSingle>(Other[1]), Z - static_cast<FRealSingle>(Other[2]));
		}
		template<class T2>
		TVector<FRealSingle, 3> operator*(const TVector<T2, 3>& Other) const
		{
			return TVector<FRealSingle, 3>(X * static_cast<FRealSingle>(Other[0]), Y * static_cast<FRealSingle>(Other[1]), Z * static_cast<FRealSingle>(Other[2]));
		}
		template<class T2>
		TVector<FRealSingle, 3> operator/(const TVector<T2, 3>& Other) const
		{
			return TVector<FRealSingle, 3>(X / static_cast<FRealSingle>(Other[0]), Y / static_cast<FRealSingle>(Other[1]), Z / static_cast<FRealSingle>(Other[2]));
		}
		FRealSingle Product() const
		{
			return X * Y * Z;
		}
		FRealSingle Max() const
		{
			return (X > Y && X > Z) ? X : (Y > Z ? Y : Z);
		}
		FRealSingle Min() const
		{
			return (X < Y && X < Z) ? X : (Y < Z ? Y : Z);
		}
		int32 MaxAxis() const
		{
			return (X > Y && X > Z) ? 0 : (Y > Z ? 1 : 2);
		}
		FRealSingle Mid() const
		{
			return (X == Y || !((Y < X) ^ (X < Z))) ? X : !((X < Y) ^ (Y < Z)) ? Y : Z;
		}
		TVector<FRealSingle, 3> ComponentwiseMin(const TVector<FRealSingle, 3>& Other) const { return {FMath::Min(X,Other.X), FMath::Min(Y,Other.Y), FMath::Min(Z,Other.Z)}; }
		TVector<FRealSingle, 3> ComponentwiseMax(const TVector<FRealSingle, 3>& Other) const { return {FMath::Max(X,Other.X), FMath::Max(Y,Other.Y), FMath::Max(Z,Other.Z)}; }
		static TVector<FRealSingle, 3> Max(const TVector<FRealSingle, 3>& V1, const TVector<FRealSingle, 3>& V2)
		{
			return TVector<FRealSingle, 3>(V1.X > V2.X ? V1.X : V2.X, V1.Y > V2.Y ? V1.Y : V2.Y, V1.Z > V2.Z ? V1.Z : V2.Z);
		}
		inline void FlushToZero(FRealSingle Epsilon)
		{
			if ((X >= -Epsilon) && (X <= Epsilon)) { X = 0.0f; }
			if ((Y >= -Epsilon) && (Y <= Epsilon)) { Y = 0.0f; }
			if ((Z >= -Epsilon) && (Z <= Epsilon)) { Z = 0.0f; }
		}
		static TVector<FRealSingle, 3> AxisVector(const int32 Axis)
		{ return Axis == 0 ? TVector<FRealSingle, 3>(1.f, 0.f, 0.f) : (Axis == 1 ? TVector<FRealSingle, 3>(0.f, 1.f, 0.f) : TVector<FRealSingle, 3>(0.f, 0.f, 1.f)); }
		static Pair<FRealSingle, int32> MaxAndAxis(const TVector<FRealSingle, 3>& V1, const TVector<FRealSingle, 3>& V2)
		{
			const TVector<FRealSingle, 3> max = Max(V1, V2);
			if (max.X > max.Y)
			{
				if (max.X > max.Z)
					return MakePair(max.X, 0);
				else
					return MakePair(max.Z, 2);
			}
			else
			{
				if (max.Y > max.Z)
					return MakePair(max.Y, 1);
				else
					return MakePair(max.Z, 2);
			}
		}
		FRealSingle SafeNormalize(FRealSingle Epsilon = 1e-4f)
		{
			FRealSingle Size = SizeSquared();
			if (Size < Epsilon)
			{
				*this = AxisVector(0);
				return 0.f;
			}
			Size = FMath::Sqrt(Size);
			*this = (*this) / Size;
			return Size;
		}
		TVector<FRealSingle, 3> GetOrthogonalVector() const
		{
			TVector<FRealSingle, 3> AbsVector(FMath::Abs(X), FMath::Abs(Y), FMath::Abs(Z));
			if ((AbsVector.X <= AbsVector.Y) && (AbsVector.X <= AbsVector.Z))
			{
				// X is the smallest component
				return TVector<FRealSingle, 3>(0, Z, -Y);
			}
			if ((AbsVector.Z <= AbsVector.X) && (AbsVector.Z <= AbsVector.Y))
			{
				// Z is the smallest component
				return TVector<FRealSingle, 3>(Y, -X, 0);
			}
			// Y is the smallest component
			return TVector<FRealSingle, 3>(-Z, 0, X);
		}
		static FRealSingle AngleBetween(const TVector<FRealSingle, 3>& V1, const TVector<FRealSingle, 3>& V2)
		{
			FRealSingle s = CrossProduct(V1, V2).Size();
			FRealSingle c = DotProduct(V1, V2);
			return FMath::Atan2(s, c);
		}
		/** Calculate the velocity to move from P0 to P1 in time Dt. Exists just for symmetry with TRotation::CalculateAngularVelocity! */
		static TVector<FRealSingle, 3> CalculateVelocity(const TVector<FRealSingle, 3>& P0, const TVector<FRealSingle, 3>& P1, const FRealSingle Dt)
		{
			return (P1 - P0) / Dt;
		}

		static bool IsNearlyEqual(const TVector<FRealSingle, 3>& A, const TVector<FRealSingle, 3>& B, const FRealSingle Epsilon)
		{
			return (B - A).IsNearlyZero(Epsilon);
		}
	};


	template<>
	class TVector<FRealDouble, 3> : public UE::Math::TVector<FRealDouble>
	{
	public:
		using FElement = FRealDouble;
		using UE::Math::TVector<FRealDouble>::X;
		using UE::Math::TVector<FRealDouble>::Y;
		using UE::Math::TVector<FRealDouble>::Z;

		TVector() = default;

		explicit TVector(const FRealDouble x)
			: UE::Math::TVector<FRealDouble>(x, x, x) {}
		TVector(const FRealDouble x, const FRealDouble y, const FRealDouble z)
			: UE::Math::TVector<FRealDouble>(x, y, z) {}
		TVector(const UE::Math::TVector<FRealSingle>& vec)
			: UE::Math::TVector<FRealDouble>((UE::Math::TVector<FRealDouble>)vec) {}
		TVector(const UE::Math::TVector<FRealDouble>& vec)					// LWC_TODO: Precision loss. Make explicit for FRealDouble = FRealSingle?
			: UE::Math::TVector<FRealDouble>((UE::Math::TVector<FRealDouble>)vec) {}
		TVector(const FVector4& vec)
			: UE::Math::TVector<FRealDouble>(vec.X, vec.Y, vec.Z) {}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
		TVector(std::istream& Stream)
		{
			Stream.read(reinterpret_cast<char*>(&X), sizeof(X));
			Stream.read(reinterpret_cast<char*>(&Y), sizeof(Y));
			Stream.read(reinterpret_cast<char*>(&Z), sizeof(Z));
		}
		//operator UE::Math::TVector<FRealDouble>() const { return UE::Math::TVector<FRealDouble>((FRealDouble)X, (FRealDouble)Y, (FRealDouble)Z); }
		void Write(std::ostream& Stream) const
		{
			Stream.write(reinterpret_cast<const char*>(&X), sizeof(X));
			Stream.write(reinterpret_cast<const char*>(&Y), sizeof(Y));
			Stream.write(reinterpret_cast<const char*>(&Z), sizeof(Z));
		}
#endif
		static inline TVector<FRealDouble, 3> Lerp(const TVector<FRealDouble, 3>& V1, const TVector<FRealDouble, 3>& V2, const FRealDouble F) { return FMath::Lerp<UE::Math::TVector<FRealDouble>, FRealDouble>(V1, V2, F); }
		static inline TVector<FRealDouble, 3> CrossProduct(const TVector<FRealDouble, 3>& V1, const TVector<FRealDouble, 3>& V2) { return UE::Math::TVector<FRealDouble>::CrossProduct(V1, V2); }
		static inline FRealDouble DotProduct(const TVector<FRealDouble, 3>& V1, const TVector<FRealDouble, 3>& V2) { return UE::Math::TVector<FRealDouble>::DotProduct(V1, V2); }
		bool operator<=(const TVector<FRealDouble, 3>& V) const
		{
			return X <= V.X && Y <= V.Y && Z <= V.Z;
		}
		bool operator>=(const TVector<FRealDouble, 3>& V) const
		{
			return X >= V.X && Y >= V.Y && Z >= V.Z;
		}
		TVector<FRealDouble, 3> operator-() const
		{
			return TVector<FRealDouble, 3>(-X, -Y, -Z);
		}
		TVector<FRealDouble, 3> operator+(const FRealDouble Other) const
		{
			return TVector<FRealDouble, 3>(X + Other, Y + Other, Z + Other);
		}
		TVector<FRealDouble, 3> operator-(const FRealDouble Other) const
		{
			return TVector<FRealDouble, 3>(X - Other, Y - Other, Z - Other);
		}
		TVector<FRealDouble, 3> operator*(const FRealDouble Other) const
		{
			return TVector<FRealDouble, 3>(X * Other, Y * Other, Z * Other);
		}
		TVector<FRealDouble, 3> operator/(const FRealDouble Other) const
		{
			return TVector<FRealDouble, 3>(X / Other, Y / Other, Z / Other);
		}
		friend TVector<FRealDouble, 3> operator/(const FRealDouble S, const TVector<FRealDouble, 3>& V)
		{
			return TVector<FRealDouble, 3>(S / V.X, S / V.Y, S / V.Z);
		}
		TVector<FRealDouble, 3> operator+(const TVector<FRealDouble, 3>& Other) const
		{
			return TVector<FRealDouble, 3>(X + Other[0], Y + Other[1], Z + Other[2]);
		}
		TVector<FRealDouble, 3> operator-(const TVector<FRealDouble, 3>& Other) const
		{
			return TVector<FRealDouble, 3>(X - Other[0], Y - Other[1], Z - Other[2]);
		}
		TVector<FRealDouble, 3> operator*(const TVector<FRealDouble, 3>& Other) const
		{
			return TVector<FRealDouble, 3>(X * Other[0], Y * Other[1], Z * Other[2]);
		}
		TVector<FRealDouble, 3> operator/(const TVector<FRealDouble, 3>& Other) const
		{
			return TVector<FRealDouble, 3>(X / Other[0], Y / Other[1], Z / Other[2]);
		}
		template<class T2>
		TVector<FRealDouble, 3> operator+(const TVector<T2, 3>& Other) const
		{
			return TVector<FRealDouble, 3>(X + static_cast<FRealDouble>(Other[0]), Y + static_cast<FRealDouble>(Other[1]), Z + static_cast<FRealDouble>(Other[2]));
		}
		template<class T2>
		TVector<FRealDouble, 3> operator-(const TVector<T2, 3>& Other) const
		{
			return TVector<FRealDouble, 3>(X - static_cast<FRealDouble>(Other[0]), Y - static_cast<FRealDouble>(Other[1]), Z - static_cast<FRealDouble>(Other[2]));
		}
		template<class T2>
		TVector<FRealDouble, 3> operator*(const TVector<T2, 3>& Other) const
		{
			return TVector<FRealDouble, 3>(X * static_cast<FRealDouble>(Other[0]), Y * static_cast<FRealDouble>(Other[1]), Z * static_cast<FRealDouble>(Other[2]));
		}
		template<class T2>
		TVector<FRealDouble, 3> operator/(const TVector<T2, 3>& Other) const
		{
			return TVector<FRealDouble, 3>(X / static_cast<FRealDouble>(Other[0]), Y / static_cast<FRealDouble>(Other[1]), Z / static_cast<FRealDouble>(Other[2]));
		}
		FRealDouble Product() const
		{
			return X * Y * Z;
		}
		FRealDouble Max() const
		{
			return (X > Y && X > Z) ? X : (Y > Z ? Y : Z);
		}
		FRealDouble Min() const
		{
			return (X < Y&& X < Z) ? X : (Y < Z ? Y : Z);
		}
		int32 MaxAxis() const
		{
			return (X > Y && X > Z) ? 0 : (Y > Z ? 1 : 2);
		}
		FRealDouble Mid() const
		{
			return (X == Y || !((Y < X) ^ (X < Z))) ? X : !((X < Y) ^ (Y < Z)) ? Y : Z;
		}
		TVector<FRealDouble, 3> ComponentwiseMin(const TVector<FRealDouble, 3>& Other) const { return { FMath::Min(X,Other.X), FMath::Min(Y,Other.Y), FMath::Min(Z,Other.Z) }; }
		TVector<FRealDouble, 3> ComponentwiseMax(const TVector<FRealDouble, 3>& Other) const { return { FMath::Max(X,Other.X), FMath::Max(Y,Other.Y), FMath::Max(Z,Other.Z) }; }
		static TVector<FRealDouble, 3> Max(const TVector<FRealDouble, 3>& V1, const TVector<FRealDouble, 3>& V2)
		{
			return TVector<FRealDouble, 3>(V1.X > V2.X ? V1.X : V2.X, V1.Y > V2.Y ? V1.Y : V2.Y, V1.Z > V2.Z ? V1.Z : V2.Z);
		}
		inline void FlushToZero(FRealDouble Epsilon)
		{
			if ((X >= -Epsilon) && (X <= Epsilon)) { X = 0.0; }
			if ((Y >= -Epsilon) && (Y <= Epsilon)) { Y = 0.0; }
			if ((Z >= -Epsilon) && (Z <= Epsilon)) { Z = 0.0; }
		}
		static TVector<FRealDouble, 3> AxisVector(const int32 Axis)
		{
			return Axis == 0 ? TVector<FRealDouble, 3>(1.f, 0.f, 0.f) : (Axis == 1 ? TVector<FRealDouble, 3>(0.f, 1.f, 0.f) : TVector<FRealDouble, 3>(0.f, 0.f, 1.f));
		}
		static Pair<FRealDouble, int32> MaxAndAxis(const TVector<FRealDouble, 3>& V1, const TVector<FRealDouble, 3>& V2)
		{
			const TVector<FRealDouble, 3> max = Max(V1, V2);
			if (max.X > max.Y)
			{
				if (max.X > max.Z)
					return MakePair(max.X, 0);
				else
					return MakePair(max.Z, 2);
			}
			else
			{
				if (max.Y > max.Z)
					return MakePair(max.Y, 1);
				else
					return MakePair(max.Z, 2);
			}
		}
		FRealDouble SafeNormalize(FRealDouble Epsilon = 1e-4f)
		{
			FRealDouble Size = SizeSquared();
			if (Size < Epsilon)
			{
				*this = AxisVector(0);
				return 0.f;
			}
			Size = FMath::Sqrt(Size);
			*this = (*this) / Size;
			return Size;
		}
		TVector<FRealDouble, 3> GetOrthogonalVector() const
		{
			TVector<FRealDouble, 3> AbsVector(FMath::Abs(X), FMath::Abs(Y), FMath::Abs(Z));
			if ((AbsVector.X <= AbsVector.Y) && (AbsVector.X <= AbsVector.Z))
			{
				// X is the smallest component
				return TVector<FRealDouble, 3>(0, Z, -Y);
			}
			if ((AbsVector.Z <= AbsVector.X) && (AbsVector.Z <= AbsVector.Y))
			{
				// Z is the smallest component
				return TVector<FRealDouble, 3>(Y, -X, 0);
			}
			// Y is the smallest component
			return TVector<FRealDouble, 3>(-Z, 0, X);
		}
		static FRealDouble AngleBetween(const TVector<FRealDouble, 3>& V1, const TVector<FRealDouble, 3>& V2)
		{
			FRealDouble s = CrossProduct(V1, V2).Size();
			FRealDouble c = DotProduct(V1, V2);
			return FMath::Atan2(s, c);
		}
		/** Calculate the velocity to move from P0 to P1 in time Dt. Exists just for symmetry with TRotation::CalculateAngularVelocity! */
		static TVector<FRealDouble, 3> CalculateVelocity(const TVector<FRealDouble, 3>& P0, const TVector<FRealDouble, 3>& P1, const FRealDouble Dt)
		{
			return (P1 - P0) / Dt;
		}

		static bool IsNearlyEqual(const TVector<FRealDouble, 3>& A, const TVector<FRealDouble, 3>& B, const FRealDouble Epsilon)
		{
			return (B - A).IsNearlyZero(Epsilon);
		}
	};

	template<>
	class TVector<FRealSingle, 2> : public FVector2f
	{
	public:
		using FElement = decltype(FVector2f::X);
		using FVector2f::X;
		using FVector2f::Y;

		TVector() = default;

		TVector(const FRealSingle x)
		    : FVector2f((decltype(FVector2f::X))x, (decltype(FVector2f::X))x) {}	// LWC_TODO: Remove casts once FVector2f supports variants
		TVector(const FRealSingle x, const FRealSingle y)
		    : FVector2f((decltype(FVector2f::X))x, (decltype(FVector2f::X))y) {}
		TVector(const FVector2f& vec)
		    : FVector2f(vec) {}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
		TVector(std::istream& Stream)
		{
			Stream.read(reinterpret_cast<char*>(&X), sizeof(X));
			Stream.read(reinterpret_cast<char*>(&Y), sizeof(Y));
		}
#endif
		template <typename OtherT>
		TVector(const TVector<OtherT, 2>& InVector)
		{
			X = ((decltype(X))InVector[0]);	// LWC_TODO: Remove casts once FVector2f supports variants
			Y = ((decltype(Y))InVector[1]);
		}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
		void Write(std::ostream& Stream) const
		{
			Stream.write(reinterpret_cast<const char*>(&X), sizeof(X));
			Stream.write(reinterpret_cast<const char*>(&Y), sizeof(Y));
		}
#endif
		static TVector<FRealSingle, 2> AxisVector(const int32 Axis)
		{
			check(Axis >= 0 && Axis <= 1);
			return Axis == 0 ? TVector<FRealSingle, 2>(1.f, 0.f) : TVector<FRealSingle, 2>(0.f, 1.f);
		}
		FRealSingle Product() const
		{
			return X * Y;
		}
		FRealSingle Max() const
		{
			return X > Y ? X : Y;
		}
		FRealSingle Min() const
		{
			return X < Y ? X : Y;
		}
		static TVector<FRealSingle, 2> Max(const TVector<FRealSingle, 2>& V1, const TVector<FRealSingle, 2>& V2)
		{
			return TVector<FRealSingle, 2>(V1.X > V2.X ? V1.X : V2.X, V1.Y > V2.Y ? V1.Y : V2.Y);
		}
		static Pair<FRealSingle, int32> MaxAndAxis(const TVector<FRealSingle, 2>& V1, const TVector<FRealSingle, 2>& V2)
		{
			const TVector<FRealSingle, 2> max = Max(V1, V2);
			if (max.X > max.Y)
			{
				return MakePair((FRealSingle)max.X, 0);
			}
			else
			{
				return MakePair((FRealSingle)max.Y, 1);
			}
		}
		template<class T2>
		TVector<FRealSingle, 2> operator/(const TVector<T2, 2>& Other) const
		{
			return TVector<FRealSingle, 2>(X / static_cast<FRealSingle>(Other[0]), Y / static_cast<FRealSingle>(Other[1]));
		}
		TVector<FRealSingle, 2> operator/(const FRealSingle Other) const
		{
			return TVector<FRealSingle, 2>(X / Other, Y / Other);
		}
		TVector<FRealSingle, 2> operator*(const FRealSingle Other) const
		{
			return TVector<FRealSingle, 2>(X * Other, Y * Other);
		}
		TVector<FRealSingle, 2> operator*(const TVector<FRealSingle, 2>& Other) const
		{
			return TVector<FRealSingle, 2>(X * Other[0], Y * Other[1]);
		}
	};

	template<>
	class TVector<FRealDouble, 2> : public FVector2d
	{
	public:
		using FElement = decltype(FVector2d::X);
		using FVector2d::X;
		using FVector2d::Y;

		TVector() = default;

		TVector(const FRealDouble x)
		    : FVector2d((decltype(FVector2d::X))x, (decltype(FVector2d::X))x) {}	// LWC_TODO: Remove casts once FVector2d supports variants
		TVector(const FRealDouble x, const FRealDouble y)
		    : FVector2d((decltype(FVector2d::X))x, (decltype(FVector2d::X))y) {}
		TVector(const FVector2d& vec)
		    : FVector2d(vec) {}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
		TVector(std::istream& Stream)
		{
			Stream.read(reinterpret_cast<char*>(&X), sizeof(X));
			Stream.read(reinterpret_cast<char*>(&Y), sizeof(Y));
		}
#endif
		template <typename OtherT>
		TVector(const TVector<OtherT, 2>& InVector)
		{
			X = ((decltype(X))InVector[0]);	// LWC_TODO: Remove casts once FVector2d supports variants
			Y = ((decltype(Y))InVector[1]);
		}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
		void Write(std::ostream& Stream) const
		{
			Stream.write(reinterpret_cast<const char*>(&X), sizeof(X));
			Stream.write(reinterpret_cast<const char*>(&Y), sizeof(Y));
		}
#endif
		static TVector<FRealDouble, 2> AxisVector(const int32 Axis)
		{
			check(Axis >= 0 && Axis <= 1);
			return Axis == 0 ? TVector<FRealDouble, 2>(1.f, 0.f) : TVector<FRealDouble, 2>(0.f, 1.f);
		}
		FRealDouble Product() const
		{
			return X * Y;
		}
		FRealDouble Max() const
		{
			return X > Y ? X : Y;
		}
		FRealDouble Min() const
		{
			return X < Y ? X : Y;
		}
		static TVector<FRealDouble, 2> Max(const TVector<FRealDouble, 2>& V1, const TVector<FRealDouble, 2>& V2)
		{
			return TVector<FRealDouble, 2>(V1.X > V2.X ? V1.X : V2.X, V1.Y > V2.Y ? V1.Y : V2.Y);
		}
		static Pair<FRealDouble, int32> MaxAndAxis(const TVector<FRealDouble, 2>& V1, const TVector<FRealDouble, 2>& V2)
		{
			const TVector<FRealDouble, 2> max = Max(V1, V2);
			if (max.X > max.Y)
			{
				return MakePair((FRealDouble)max.X, 0);
			}
			else
			{
				return MakePair((FRealDouble)max.Y, 1);
			}
		}
		template<class T2>
		TVector<FRealDouble, 2> operator/(const TVector<T2, 2>& Other) const
		{
			return TVector<FRealDouble, 2>(X / static_cast<FRealDouble>(Other[0]), Y / static_cast<FRealDouble>(Other[1]));
		}
		TVector<FRealDouble, 2> operator/(const FRealDouble Other) const
		{
			return TVector<FRealDouble, 2>(X / Other, Y / Other);
		}
		TVector<FRealDouble, 2> operator*(const FRealDouble Other) const
		{
			return TVector<FRealDouble, 2>(X * Other, Y * Other);
		}
		TVector<FRealDouble, 2> operator*(const TVector<FRealDouble, 2>& Other) const
		{
			return TVector<FRealDouble, 2>(X * Other[0], Y * Other[1]);
		}
	};
#endif // !COMPILE_WITHOUT_UNREAL_SUPPORT

	template<class T>
	class TVector<T, 3>
	{
	public:
		using FElement = T;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TVector(const TVector&) = default;
		TVector& operator=(const TVector&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FORCEINLINE TVector() = default;

		FORCEINLINE explicit TVector(T InX)
		    : X(InX), Y(InX), Z(InX) {}
		FORCEINLINE TVector(T InX, T InY, T InZ)
		    : X(InX), Y(InY), Z(InZ) {}

		FORCEINLINE int32 Num() const { return 3; }
		FORCEINLINE bool operator==(const TVector<T, 3>& Other) const { return X == Other.X && Y == Other.Y && Z == Other.Z; }
#if !COMPILE_WITHOUT_UNREAL_SUPPORT
		FORCEINLINE TVector(const UE::Math::TVector<FReal>& Other)
		{
			X = static_cast<T>(Other.X);
			Y = static_cast<T>(Other.Y);
			Z = static_cast<T>(Other.Z);
		}
#endif
		template<class T2>
		FORCEINLINE TVector(const TVector<T2, 3>& Other)
		    : X(static_cast<T>(Other.X))
		    , Y(static_cast<T>(Other.Y))
		    , Z(static_cast<T>(Other.Z))
		{}

#if COMPILE_WITHOUT_UNREAL_SUPPORT
		FORCEINLINE TVector(std::istream& Stream)
		{
			Stream.read(reinterpret_cast<char*>(&X), sizeof(T));
			Stream.read(reinterpret_cast<char*>(&Y), sizeof(T));
			Stream.read(reinterpret_cast<char*>(&Z), sizeof(T));
		}
		FORCEINLINE void Write(std::ostream& Stream) const
		{
			Stream.write(reinterpret_cast<const char*>(&X), sizeof(T));
			Stream.write(reinterpret_cast<const char*>(&Y), sizeof(T));
			Stream.write(reinterpret_cast<const char*>(&Z), sizeof(T));
		}
#endif
		template<class T2>
		FORCEINLINE TVector<T, 3>& operator=(const TVector<T2, 3>& Other)
		{
			X = Other.X;
			Y = Other.Y;
			Z = Other.Z;
			return *this;
		}
		FORCEINLINE T Size() const
		{
			const T SquaredSum = X * X + Y * Y + Z * Z;
			return FMath::Sqrt(SquaredSum);
		}
		FORCEINLINE T Product() const { return X * Y * Z; }
		FORCEINLINE static TVector<T, 3> AxisVector(const int32 Axis)
		{
			TVector<T, 3> Result(0);
			Result[Axis] = (T)1;
			return Result;
		}
		FORCEINLINE T SizeSquared() const { return X * X + Y * Y + Z * Z; }

		FORCEINLINE T Min() const { return FMath::Min3(X, Y, Z); }
		FORCEINLINE T Max() const { return FMath::Max3(X, Y, Z); }
		T Mid() const
		{
			return (X == Y || !((Y < X) ^ (X < Z))) ? X : !((X < Y) ^ (Y < Z)) ? Y : Z;
		}

		FORCEINLINE TVector<T, 3> ComponentwiseMin(const TVector<T, 3>& Other) const { return {FMath::Min(X,Other.X), FMath::Min(Y,Other.Y), FMath::Min(Z,Other.Z)}; }
		FORCEINLINE TVector<T, 3> ComponentwiseMax(const TVector<T, 3>& Other) const { return {FMath::Max(X,Other.X), FMath::Max(Y,Other.Y), FMath::Max(Z,Other.Z)}; }

		TVector<T, 3> GetSafeNormal() const
		{
			//We want N / ||N|| and to avoid inf
			//So we want N / ||N|| < 1 / eps => N eps < ||N||, but this is clearly true for all eps < 1 and N > 0
			T SizeSqr = SizeSquared();
			if (SizeSqr <= TNumericLimits<T>::Min())
				return AxisVector(0);
			return (*this) / FMath::Sqrt(SizeSqr);
		}
		FORCEINLINE T SafeNormalize()
		{
			T Size = SizeSquared();
			if (Size < (T)1e-4)
			{
				*this = AxisVector(0);
				return (T)0.;
			}
			Size = FMath::Sqrt(Size);
			*this = (*this) / Size;
			return Size;
		}

		FORCEINLINE bool ContainsNaN() const
		{
			return !FMath::IsFinite(X) || !FMath::IsFinite(Y) || !FMath::IsFinite(Z);
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FORCEINLINE T operator[](int32 Idx) const { return XYZ[Idx]; }
		FORCEINLINE T& operator[](int32 Idx) { return XYZ[Idx]; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FORCEINLINE TVector<T, 3> operator-() const { return {-X, -Y, -Z}; }
		FORCEINLINE TVector<T, 3> operator*(const TVector<T, 3>& Other) const { return {X * Other.X, Y * Other.Y, Z * Other.Z}; }
		FORCEINLINE TVector<T, 3> operator/(const TVector<T, 3>& Other) const { return {X / Other.X, Y / Other.Y, Z / Other.Z}; }
		FORCEINLINE TVector<T, 3> operator/(const T Scalar) const { return {X / Scalar, Y / Scalar, Z / Scalar}; }
		FORCEINLINE TVector<T, 3> operator+(const TVector<T, 3>& Other) const { return {X + Other.X, Y + Other.Y, Z + Other.Z}; }
		FORCEINLINE TVector<T, 3> operator+(const T Scalar) const { return {X + Scalar, Y + Scalar, Z + Scalar}; }
		FORCEINLINE TVector<T, 3> operator-(const TVector<T, 3>& Other) const { return {X - Other.X, Y - Other.Y, Z - Other.Z}; }
		FORCEINLINE TVector<T, 3> operator-(const T Scalar) const { return {X - Scalar, Y - Scalar, Z - Scalar}; }

		FORCEINLINE TVector<T, 3>& operator+=(const TVector<T, 3>& Other)
		{
			X += Other.X;
			Y += Other.Y;
			Z += Other.Z;
			return *this;
		}
		FORCEINLINE TVector<T, 3>& operator-=(const TVector<T, 3>& Other)
		{
			X -= Other.X;
			Y -= Other.Y;
			Z -= Other.Z;
			return *this;
		}
		FORCEINLINE TVector<T, 3>& operator/=(const TVector<T, 3>& Other)
		{
			X /= Other.X;
			Y /= Other.Y;
			Z /= Other.Z;
			return *this;
		}
		FORCEINLINE TVector<T, 3> operator*(const T S) const { return {X * S, Y * S, Z * S}; }
		FORCEINLINE TVector<T, 3>& operator*=(const T S)
		{
			X *= S;
			Y *= S;
			Z *= S;
			return *this;
		}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
		FORCEINLINE static inline FReal DotProduct(const Vector<FReal, 3>& V1, const Vector<FReal, 3>& V2)
		{
			return V1[0] * V2[0] + V1[1] * V2[1] + V1[2] * V2[2];
		}
		FORCEINLINE static inline Vector<FReal, 3> CrossProduct(const Vector<FReal, 3>& V1, const Vector<FReal, 3>& V2)
		{
			Vector<FReal, 3> Result;
			Result[0] = V1[1] * V2[2] - V1[2] * V2[1];
			Result[1] = V1[2] * V2[0] - V1[0] * V2[2];
			Result[2] = V1[0] * V2[1] - V1[1] * V2[0];
			return Result;
		}
#endif

		union
		{
			struct
			{
				T X;
				T Y;
				T Z;
			};

			UE_DEPRECATED(all, "For internal use only")
			T XYZ[3];
		};
	};
	template<class T>
	inline TVector<T, 3> operator*(const T S, const TVector<T, 3>& V)
	{
		return TVector<T, 3>{V.X * S, V.Y * S, V.Z * S};
	}
	template<class T>
	inline TVector<T, 3> operator/(const T S, const TVector<T, 3>& V)
	{
		return TVector<T, 3>{V.X / S, V.Y / S, V.Z / S};
	}

	template<>
	class TVector<int32, 2>
	{
	public:
		using FElement = int32;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TVector(const TVector&) = default;
		TVector& operator=(const TVector&) = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FORCEINLINE TVector() = default;

		FORCEINLINE explicit TVector(const FElement InX)
		    : X(InX), Y(InX)
		{}
		FORCEINLINE TVector(const FElement InX, const FElement InY)
		    : X(InX), Y(InY)
		{}
		template<typename OtherT>
		FORCEINLINE TVector(const TVector<OtherT, 2>& InVector)
			: X((int32)InVector.X)
			, Y((int32)InVector.Y)
		{}
#if COMPILE_WITHOUT_UNREAL_SUPPORT
		FORCEINLINE TVector(std::istream& Stream)
		{
			Stream.read(reinterpret_cast<char*>(&X), sizeof(int32));
			Stream.read(reinterpret_cast<char*>(&Y), sizeof(int32));
		}
#endif
		FORCEINLINE int32 Num() const { return 2; }

		FORCEINLINE FElement Product() const
		{
			return X * Y;
		}

		FORCEINLINE static TVector<FElement, 2> AxisVector(const int32 Axis)
		{
			TVector<FElement, 2> Result(0);
			Result[Axis] = 1;
			return Result;
		}

#if COMPILE_WITHOUT_UNREAL_SUPPORT
		FORCEINLINE void Write(std::ostream& Stream) const
		{
			Stream.write(reinterpret_cast<const char*>(&X), sizeof(FElement));
			Stream.write(reinterpret_cast<const char*>(&Y), sizeof(FElement));
		}
#endif

		template<typename OtherT>
		FORCEINLINE TVector<int32, 2>& operator=(const TVector<OtherT, 2>& Other)
		{
			X = Other.X;
			Y = Other.Y;
			return *this;
		}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FORCEINLINE FElement operator[](const int32 Idx) const { return XY[Idx]; }
		FORCEINLINE FElement& operator[](const int32 Idx) { return XY[Idx]; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		FORCEINLINE TVector<FElement, 2> operator-() const { return {-X, -Y}; }
		FORCEINLINE TVector<FElement, 2> operator*(const TVector<FElement, 2>& Other) const { return {X * Other.X, Y * Other.Y}; }
		FORCEINLINE TVector<FElement, 2> operator/(const TVector<FElement, 2>& Other) const { return {X / Other.X, Y / Other.Y}; }
		FORCEINLINE TVector<FElement, 2> operator+(const TVector<FElement, 2>& Other) const { return {X + Other.X, Y + Other.Y}; }
		FORCEINLINE TVector<FElement, 2> operator-(const TVector<FElement, 2>& Other) const { return {X - Other.X, Y - Other.Y}; }

		FORCEINLINE TVector<FElement, 2>& operator+=(const TVector<FElement, 2>& Other)
		{
			X += Other.X;
			Y += Other.Y;
			return *this;
		}
		FORCEINLINE TVector<FElement, 2>& operator-=(const TVector<FElement, 2>& Other)
		{
			X -= Other.X;
			Y -= Other.Y;
			return *this;
		}
		FORCEINLINE TVector<FElement, 2>& operator/=(const TVector<FElement, 2>& Other)
		{
			X /= Other.X;
			Y /= Other.Y;
			return *this;
		}
		FORCEINLINE TVector<FElement, 2> operator*(const FElement& S) const
		{
			return {X * S, Y * S};
		}
		FORCEINLINE TVector<FElement, 2>& operator*=(const FElement& S)
		{
			X *= S;
			Y *= S;
			return *this;
		}

		FORCEINLINE friend bool operator==(const TVector<FElement, 2>& L, const TVector<FElement, 2>& R)
		{
			return (L.X == R.X) && (L.Y == R.Y);
		}

		FORCEINLINE friend bool operator!=(const TVector<FElement, 2>& L, const TVector<FElement, 2>& R)
		{
			return !(L == R);
		}


	private:
		union
		{
			struct
			{
				FElement X;
				FElement Y;
			};

			UE_DEPRECATED(all, "For internal use only")
			FElement XY[2];
		};
	};

	template<class T>
	inline uint32 GetTypeHash(const Chaos::TVector<T, 2>& V)
	{
		uint32 Seed = ::GetTypeHash(V[0]);
		Seed ^= ::GetTypeHash(V[1]) + 0x9e3779b9 + (Seed << 6) + (Seed >> 2);
		return Seed;
	}

	template<class T>
	inline uint32 GetTypeHash(const Chaos::TVector<T, 3>& V)
	{
		uint32 Seed = ::GetTypeHash(V[0]);
		Seed ^= ::GetTypeHash(V[1]) + 0x9e3779b9 + (Seed << 6) + (Seed >> 2);
		Seed ^= ::GetTypeHash(V[2]) + 0x9e3779b9 + (Seed << 6) + (Seed >> 2);
		return Seed;
	}

	// this is used as 
	template<typename T, int d>
	inline FArchive& SerializeReal(FArchive& Ar, TVector<T, d>& ValueIn)
	{
		static_assert(std::is_same_v<T, float> || std::is_same_v<T, double>, "only float or double are supported by this function");
		for (int32 Idx = 0; Idx < d; ++Idx)
		{
			FRealSingle RealSingle = (FRealSingle)ValueIn[Idx];
			Ar << RealSingle;
			ValueIn[Idx] = (typename TVector<T, d>::FElement)RealSingle;
		}
		return Ar;
	}

	template<int d>
	FArchive& operator<<(FArchive& Ar, TVector<FRealSingle, d>& ValueIn) 
	{
		return SerializeReal(Ar, ValueIn);
	}

	template<int d>
	FArchive& operator<<(FArchive& Ar, TVector<FRealDouble, d>& ValueIn)
	{
		return SerializeReal(Ar, ValueIn);
	}

	// general for for all other vectors
	template<typename T, int d>
	FArchive& operator<<(FArchive& Ar, TVector<T, d>& ValueIn)
	{
		// unchanged type code path 
		for (int32 Idx = 0; Idx < d; ++Idx)
		{
			Ar << ValueIn[Idx];
		}
		return Ar;
	}

} // namespace Chaos

//template<>
//uint32 GetTypeHash(const Chaos::TVector<int32, 2>& V)
//{
//	uint32 Seed = GetTypeHash(V[0]);
//	Seed ^= GetTypeHash(V[1]) + 0x9e3779b9 + (Seed << 6) + (Seed >> 2);
//	return Seed;
//}

// LWC_TODO: UE::Math::TVector<FReal> construction from a chaos float vec3
//inline UE::Math::TVector<FReal>::UE::Math::TVector<FReal>(const Chaos::TVector<float, 3>& ChaosVector) : X(ChaosVector.X), Y(ChaosVector.Y), Z(ChaosVector.Z) {}

