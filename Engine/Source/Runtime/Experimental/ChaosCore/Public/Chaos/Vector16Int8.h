// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/VectorRegister.h"

// Platform specific vector intrinsics include.
#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON
#include "Chaos/Vector16Int8Neon.h"
#elif defined(__cplusplus_cli)
#include "Chaos/Vector16Int8FPU.h"
#elif PLATFORM_ENABLE_VECTORINTRINSICS
#include "Chaos/Vector16Int8SSE.h"
#else
#include "Chaos/Vector16Int8FPU.h"
#endif
