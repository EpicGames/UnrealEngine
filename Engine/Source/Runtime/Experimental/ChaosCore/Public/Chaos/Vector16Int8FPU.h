// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "Math/VectorRegister.h"

struct UE_INTERNAL VectorRegister16Int8
{
	int8 Value[16];
};

UE_INTERNAL FORCEINLINE VectorRegister16Int8 Vector16Int8Set1(int8 Value)
{
	VectorRegister16Int8 Result;
	for (uint32 I = 0; I < 16; ++I)
	{
		Result.Value[I] = Value;
	}
	return Result;
}

UE_INTERNAL FORCEINLINE VectorRegister16Int8 VectorLoad16Int8(const int8* Ptr)
{
	VectorRegister16Int8 Result;
	for (uint32 I = 0; I < 16; ++I)
	{
		Result.Value[I] = Ptr[I];
	}
	return Result;
}

UE_INTERNAL FORCEINLINE void VectorStore(VectorRegister16Int8 Vec, int8* Ptr)
{
	for (uint32 I = 0; I < 16; ++I)
	{
		Ptr[I] = Vec.Value[I];
	}
}

UE_INTERNAL FORCEINLINE VectorRegister16Int8 VectorCompareEQ(VectorRegister16Int8 Vec1, VectorRegister16Int8 Vec2)
{
	VectorRegister16Int8 Result;
	for (uint32 I = 0; I < 16; ++I)
	{
		Result.Value[I] = (Vec1.Value[I] == Vec2.Value[I]) ? 0xFF : 0;
	}
	return Result;
}

UE_INTERNAL FORCEINLINE uint16 VectorMaskBits(VectorRegister16Int8 VecMask)
{
	constexpr uint8 SignMask = (uint16)(1 << 7);
	uint16 Result = 0;
	for (uint16 I = 0; I < 16; ++I)
	{
		const uint8 Sign = VecMask.Value[I] & SignMask;
		if (Sign != 0)
		{
			Result |= 1 << I;
		}
	}
	return Result;
}
