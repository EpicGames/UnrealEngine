// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Not included directly

#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON

#include "Math/VectorRegister.h"

struct UE_INTERNAL VectorRegister16Int8
{
	int8x16_t Value;
};

UE_INTERNAL FORCEINLINE VectorRegister16Int8 Vector16Int8Set1(int8 Value)
{
	VectorRegister16Int8 Result;
	Result.Value = vdupq_n_s8(Value);
	return Result;
}

UE_INTERNAL FORCEINLINE VectorRegister16Int8 VectorLoad16Int8(const int8* Ptr)
{
	VectorRegister16Int8 Result;
	Result.Value = vld1q_s8(Ptr);
	return Result;
}

UE_INTERNAL FORCEINLINE void VectorStore(VectorRegister16Int8 Vec, int8* Ptr)
{
	vst1q_s8(Ptr, Vec.Value);
}

UE_INTERNAL FORCEINLINE VectorRegister16Int8 VectorCompareEQ(VectorRegister16Int8 Vec1, VectorRegister16Int8 Vec2)
{
	VectorRegister16Int8 Result;
	Result.Value = vceqq_s8(Vec1.Value, Vec2.Value);
	return Result;
}

UE_INTERNAL FORCEINLINE uint16 VectorMaskBits(VectorRegister16Int8 VecMask)
{
	const uint8 Weights[16]{ 1, 2, 4, 8, 16, 32, 64, 128, 1, 2, 4, 8, 16, 32, 64, 128 };

	int32x4_t Signs = vshrq_n_s8(VecMask.Value, 7); // sign bit of each lane replicated 32x
	uint8x16_t Weighted = vandq_u8(Signs, vld1q_u8(Weights)); // pick bit for lane position

	// Split into low and high and sum across the vectors
	const uint16 Low = vaddv_u8(vget_low_u8(Weighted));
	const uint16 High = vaddv_u8(vget_high_u8(Weighted));
	const uint16 Result = static_cast<uint16>(High << 8u) | Low;
	return Result;
}

#endif // PLATFORM_ENABLE_VECTORINTRINSICS_NEON
