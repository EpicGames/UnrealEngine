// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Not included directly

#if !PLATFORM_ENABLE_VECTORINTRINSICS_NEON && !(defined(__cplusplus_cli)) && PLATFORM_ENABLE_VECTORINTRINSICS

#include "Math/VectorRegister.h"

struct UE_INTERNAL VectorRegister16Int8
{
	VectorRegister4Int Value;
};

UE_INTERNAL FORCEINLINE VectorRegister16Int8 Vector16Int8Set1(int8 Value)
{
	VectorRegister16Int8 Result;
	Result.Value = _mm_set1_epi8(Value);
	return Result;
}

UE_INTERNAL FORCEINLINE VectorRegister16Int8 VectorLoad16Int8(const int8* Ptr)
{
	VectorRegister16Int8 Result;
	Result.Value = _mm_loadu_si128((const __m128i*)(Ptr));
	return Result;
}

UE_INTERNAL FORCEINLINE void VectorStore(VectorRegister16Int8 Vec, int8* Ptr)
{
	_mm_storeu_si128((__m128i*)(Ptr), Vec.Value);
}

UE_INTERNAL FORCEINLINE VectorRegister16Int8 VectorCompareEQ(VectorRegister16Int8 Vec1, VectorRegister16Int8 Vec2)
{
	VectorRegister16Int8 Result;
	Result.Value = _mm_cmpeq_epi8(Vec1.Value, Vec2.Value);
	return Result;
}

UE_INTERNAL FORCEINLINE uint16 VectorMaskBits(VectorRegister16Int8 VecMask)
{
	uint16 Result = (uint16)_mm_movemask_epi8(VecMask.Value);
	return Result;
}

#endif // PLATFORM_ENABLE_VECTORINTRINSICS
