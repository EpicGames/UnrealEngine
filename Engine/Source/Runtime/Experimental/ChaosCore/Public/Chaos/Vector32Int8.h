// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/VectorRegister.h"

// Platform specific vector intrinsics include.
#if !PLATFORM_ENABLE_VECTORINTRINSICS_NEON && !(defined(__cplusplus_cli)) && PLATFORM_ENABLE_VECTORINTRINSICS && UE_PLATFORM_MATH_USE_AVX
#include "Chaos/Vector32Int8AVX.h"
#else
#include "Chaos/Vector32Int8Common.h"
#endif
