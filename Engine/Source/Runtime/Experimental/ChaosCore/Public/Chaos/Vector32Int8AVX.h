// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Not included directly

#if !PLATFORM_ENABLE_VECTORINTRINSICS_NEON && !(defined(__cplusplus_cli)) && PLATFORM_ENABLE_VECTORINTRINSICS && UE_PLATFORM_MATH_USE_AVX

#include "Chaos/Vector16Int8.h"

struct UE_INTERNAL VectorRegister32Int8
{
	__m256i V;
};

UE_INTERNAL FORCEINLINE VectorRegister32Int8 Vector32Int8Set1(int8 Value)
{
	VectorRegister32Int8 Result;
	Result.V = _mm256_set1_epi8(Value);
	return Result;
}

UE_INTERNAL FORCEINLINE VectorRegister32Int8 VectorLoad32Int8(const int8* Ptr)
{
	VectorRegister32Int8 Result;
	Result.V = _mm256_loadu_si256((const __m256i*)Ptr);
	return Result;
}

UE_INTERNAL FORCEINLINE void VectorStore(VectorRegister32Int8 Vec, int8* Ptr)
{
	_mm256_storeu_si256((__m256i*)Ptr, Vec.V);
}

UE_INTERNAL FORCEINLINE VectorRegister32Int8 VectorCompareEQ(VectorRegister32Int8 Vec1, VectorRegister32Int8 Vec2)
{
	VectorRegister32Int8 Result;
#if UE_PLATFORM_MATH_USE_AVX_2
	Result.V = _mm256_cmpeq_epi8(Vec1.V, Vec2.V);
#else
	VectorRegister16Int8 V1Low = { _mm256_extractf128_si256(Vec1.V, 0) };
	VectorRegister16Int8 V1High = { _mm256_extractf128_si256(Vec1.V, 1) };
	VectorRegister16Int8 V2Low = { _mm256_extractf128_si256(Vec2.V, 0) };
	VectorRegister16Int8 V2High = { _mm256_extractf128_si256(Vec2.V, 1) };

	VectorRegister16Int8 CmpLow = VectorCompareEQ(V1Low, V2Low);
	VectorRegister16Int8 CmpHigh = VectorCompareEQ(V1High, V2High);

	Result.V = _mm256_castsi128_si256(CmpLow.Value);
	Result.V = _mm256_insertf128_si256(Result.V, CmpHigh.Value, 1);
#endif
	return Result;
}

UE_INTERNAL FORCEINLINE uint32 VectorMaskBits(VectorRegister32Int8 VecMask)
{
	uint32 Result = 0;
#if UE_PLATFORM_MATH_USE_AVX_2
	Result = _mm256_movemask_epi8(VecMask.V);
#else
	VectorRegister16Int8 VecMaskLow = { _mm256_extractf128_si256(VecMask.V, 0) };
	VectorRegister16Int8 VecMaskHigh = { _mm256_extractf128_si256(VecMask.V, 1) };
	uint32 Low = (uint32)VectorMaskBits(VecMaskLow);
	uint32 High = (uint32)VectorMaskBits(VecMaskHigh);
	Result = (High << 16) | Low;
#endif
	return Result;
}

#endif
