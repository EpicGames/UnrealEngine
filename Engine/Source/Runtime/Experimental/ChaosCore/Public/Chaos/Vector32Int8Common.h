// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// HEADER_UNIT_SKIP - Not included directly

#include "Chaos/Vector16Int8.h"

struct UE_INTERNAL VectorRegister32Int8
{
	VectorRegister16Int8 V[2];
};

UE_INTERNAL FORCEINLINE VectorRegister32Int8 Vector32Int8Set1(int8 Value)
{
	VectorRegister32Int8 Result;
	Result.V[1] = Result.V[0] = Vector16Int8Set1(Value);
	return Result;
}

UE_INTERNAL FORCEINLINE VectorRegister32Int8 VectorLoad32Int8(const int8* Ptr)
{
	VectorRegister32Int8 Result;
	Result.V[0] = VectorLoad16Int8(Ptr);
	Result.V[1] = VectorLoad16Int8(Ptr + 16);
	return Result;
}

UE_INTERNAL FORCEINLINE void VectorStore(VectorRegister32Int8 Vec, int8* Ptr)
{
	VectorStore(Vec.V[0], Ptr);
	VectorStore(Vec.V[1], Ptr + 16);
}

UE_INTERNAL FORCEINLINE VectorRegister32Int8 VectorCompareEQ(VectorRegister32Int8 Vec1, VectorRegister32Int8 Vec2)
{
	VectorRegister32Int8 Result;
	Result.V[0] = VectorCompareEQ(Vec1.V[0], Vec2.V[0]);
	Result.V[1] = VectorCompareEQ(Vec1.V[1], Vec2.V[1]);
	return Result;
}

UE_INTERNAL FORCEINLINE uint32 VectorMaskBits(VectorRegister32Int8 VecMask)
{
	uint32 Result = 0;
	uint32 Low = (uint32)VectorMaskBits(VecMask.V[0]);
	uint32 High = (uint32)VectorMaskBits(VecMask.V[1]);
	Result = (High << 16) | Low;
	return Result;
}
