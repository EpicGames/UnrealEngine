// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/Array.h"

namespace Chaos
{
template<class T>
class VectorND : public TArray<T>
{
  public:
	using TArray<T>::SetNum;

	VectorND(const int32 Size)
	{
		SetNum(Size);
	}
};
}
