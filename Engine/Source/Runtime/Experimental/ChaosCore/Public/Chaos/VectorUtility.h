// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Math/VectorRegister.h"
#include "Chaos/Core.h"


template<typename T>
T TVectorZero();

template<>
inline VectorRegister4Float TVectorZero<VectorRegister4Float>()
{
	return VectorZeroFloat();
}

template<>
inline VectorRegister4Double TVectorZero<VectorRegister4Double>()
{
	return VectorZeroDouble();
}

template<typename T>
T TMakeVectorRegister(float X, float Y, float Z, float W);

template<>
inline VectorRegister4Float TMakeVectorRegister<VectorRegister4Float>(float X, float Y, float Z, float W)
{
	return MakeVectorRegisterFloat(X, Y, Z, W);
}

template<>
inline VectorRegister4Double TMakeVectorRegister<VectorRegister4Double>(float X, float Y, float Z, float W)
{
	return MakeVectorRegisterDouble(X, Y, Z, W);
}

template<typename T>
constexpr T TMakeVectorRegisterConstant(float X, float Y, float Z, float W);

template<>
constexpr VectorRegister4Float TMakeVectorRegisterConstant<VectorRegister4Float>(float X, float Y, float Z, float W)
{
	return MakeVectorRegisterFloatConstant(X, Y, Z, W);
}

template<>
constexpr VectorRegister4Double TMakeVectorRegisterConstant<VectorRegister4Double>(float X, float Y, float Z, float W)
{
	return MakeVectorRegisterDoubleConstant(X, Y, Z, W);
}

template<typename T>
VectorRegister4Float TMakeVectorRegisterFloatFromDouble(const T& V);

template<>
inline VectorRegister4Float TMakeVectorRegisterFloatFromDouble<VectorRegister4Double>(const VectorRegister4Double& V)
{
	return MakeVectorRegisterFloatFromDouble(V);
}

// Should generate no op
template<>
constexpr VectorRegister4Float TMakeVectorRegisterFloatFromDouble<VectorRegister4Float>(const VectorRegister4Float& V)
{
	return V;
}

/**
 * Cast VectorRegister4Int in VectorRegister4Float
 *
 * @param V	vector
 * @return		VectorRegister4Float( B.x, A.y, A.z, A.w)
 */
FORCEINLINE VectorRegister4Float VectorCast4IntTo4Float(const VectorRegister4Int& V)
{
#if (!defined(_MSC_VER) || PLATFORM_ENABLE_VECTORINTRINSICS_NEON) && PLATFORM_ENABLE_VECTORINTRINSICS
	return VectorRegister4Float(V);
#elif PLATFORM_ENABLE_VECTORINTRINSICS
	return _mm_castsi128_ps(V);
#else
	return VectorCastIntToFloat(Vec);
#endif
}


/**
 * Cast VectorRegister4Float in VectorRegister4Int
 *
 * @param V	vector
 * @return		VectorCast4FloatTo4Int( B.x, A.y, A.z, A.w)
 */
FORCEINLINE VectorRegister4Int VectorCast4FloatTo4Int(const VectorRegister4Float& V)
{
#if (!defined(_MSC_VER) || PLATFORM_ENABLE_VECTORINTRINSICS_NEON) && PLATFORM_ENABLE_VECTORINTRINSICS
	return VectorRegister4Int(V);
#elif PLATFORM_ENABLE_VECTORINTRINSICS
	return _mm_castps_si128(V);
#else
	return VectorCastFloatToInt(Vec);
#endif

}

/**
 * Selects and interleaves the lower two SP FP values from A and B.
 *
 * @param A	1st vector
 * @param B	2nd vector
 * @return		VectorRegister4Float( A.x, B.x, A.y, B.y)
 */
FORCEINLINE VectorRegister4Float VectorUnpackLo(const VectorRegister4Float& A, const VectorRegister4Float& B)
{
#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON
	return vzip1q_f32(A, B);
#elif PLATFORM_ENABLE_VECTORINTRINSICS
	return _mm_unpacklo_ps(A, B);
#else
	return MakeVectorRegisterFloat(A.V[0], B.V[0], A.V[1], B.V[1]);
#endif
}

/**
 * Selects and interleaves the lower two DP FP values from A and B.
 *
 * @param A	1st vector
 * @param B	2nd vector
 * @return		VectorRegister4Float( A.x, B.x, A.y, B.y)
 */
FORCEINLINE VectorRegister4Double VectorUnpackLo(const VectorRegister4Double& A, const VectorRegister4Double& B)
{
#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON
	VectorRegister4Double Result;
	Result.XY = vzip1q_f64(A.XY, B.XY); 
	Result.ZW = vzip2q_f64(A.XY, B.XY);
	return Result;
#elif PLATFORM_ENABLE_VECTORINTRINSICS
	#if UE_PLATFORM_MATH_USE_AVX
		return _mm256_permute2f128_pd(_mm256_unpackhi_pd(A, B), _mm256_unpacklo_pd(A, B), 0x02);
	#else
	VectorRegister4Double Result;
	Result.XY = _mm_unpacklo_pd(A.XY, B.XY);
	Result.ZW = _mm_unpackhi_pd(A.XY, B.XY);
	return Result;
	#endif
#else
	return MakeVectorRegisterFloat(A.V[0], B.V[0], A.V[1], B.V[1]);
#endif
}


/**
 * Selects and interleaves the higher two SP FP values from A and B.
 *
 * @param A	1st vector
 * @param B	2nd vector
 * @return		VectorRegister4Float( A.z, B.z, A.w, B.w)
 */
FORCEINLINE VectorRegister4Float VectorUnpackHi(const VectorRegister4Float& A, const VectorRegister4Float& B)
{
#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON
	return vzip2q_f32(A, B);
#elif PLATFORM_ENABLE_VECTORINTRINSICS
	return _mm_unpackhi_ps(A, B);
#else
	return MakeVectorRegisterFloat(A.V[2], B.V[2], A.V[3], B.V[3]);
#endif
}

/**
 * Moves the lower 2 SP FP values of b to the upper 2 SP FP values of the result. The lower 2 SP FP values of a are passed through to the result.
 *
 * @param A	1st vector
 * @param B	2nd vector
 * @return		VectorRegister4Float( A.x, A.y, B.x, B.y)
  */
FORCEINLINE VectorRegister4Float VectorMoveLh(const VectorRegister4Float& A, const VectorRegister4Float& B)
{
#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON
	return vzip1q_f64(A, B);
#else
	return VectorCombineLow(A, B);
#endif
}


namespace Chaos::Private
{
	/**
	 * Calculates the dot product of two vectors and returns a vector with the result in the first component.
	 * W values should be set to 0 on the vector input. This function should be used with caution. Only the returned X should be read.
	 *
	 * @param Vec1	1st vector set with W equal to 0
	 * @param Vec2	2nd vector set with W equal to 0
	 * @return		d = dot3X(Vec1.xyz0, Vec2.xyz0), VectorRegister4Float( d, ?, ?, ? )
	 */
	FORCEINLINE VectorRegister4Float VectorDot3FastX(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
	{
#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON
		VectorRegister4Float Temp = VectorMultiply(Vec1, Vec2); // Multiply 2 vector
		float32x2_t sum = vpadd_f32(vget_low_f32(Temp), vget_high_f32(Temp));
		sum = vpadd_f32(sum, sum);
		return vcombine_f32(sum, sum);
#elif PLATFORM_ENABLE_VECTORINTRINSICS
		return _mm_dp_ps(Vec1, Vec2, 0xFF);
#else
		return VectorDot3(Vec1, Vec2);
#endif
	}

	FORCEINLINE VectorRegister4Float VectorMatrixMultiply(const VectorRegister4Float& Vec, const FMatrix33& M)
	{
		const VectorRegister4Float VecX = VectorReplicate(Vec, 0);
		const VectorRegister4Float VecY = VectorReplicate(Vec, 1);
		const VectorRegister4Float VecZ = VectorReplicate(Vec, 2);

		const VectorRegister4Float R0 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(M.M[0][0], M.M[0][1], M.M[0][2], 0.0));
		const VectorRegister4Float R1 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(M.M[1][0], M.M[1][1], M.M[1][2], 0.0));
		const VectorRegister4Float R2 = MakeVectorRegisterFloatFromDouble(MakeVectorRegister(M.M[2][0], M.M[2][1], M.M[2][2], 0.0));
		return VectorMultiplyAdd(R0, VecX, VectorMultiplyAdd(R1, VecY, VectorMultiply(R2, VecZ)));
	}

	/**
	 * Calculates the cross product of two vectors (XYZ components). W of the input should be 0, and will remain 0.
	 * This function is not using FMA for stability reason, rounding with FMA could cause numerical instability.
	 *
	 * @param Vec1	1st vector
	 * @param Vec2	2nd vector
	 * @return		cross(Vec1.xyz, Vec2.xyz). W of the input should be 0, and will remain 0.
	 */
	FORCEINLINE VectorRegister4Float VectorCrossNoFMA(const VectorRegister4Float& Vec1, const VectorRegister4Float& Vec2)
	{
#if PLATFORM_ENABLE_VECTORINTRINSICS
		// YZX
		VectorRegister4Float A = VectorSwizzle(Vec2, 1, 2, 0, 3);
		VectorRegister4Float B = VectorSwizzle(Vec1, 1, 2, 0, 3);
		// XY, YZ, ZX
		// This is the only way found to avoid the compiler on XSX using FMA.
		// By forcing two FMA in a row, the subtract cannot be a FMA at the end.
		// This allow to have a symmetric and reliable cross product. 
		A = VectorMultiplyAdd(A, Vec1, VectorZero());
		B = VectorMultiplyAdd(B, Vec2, VectorZero());
		// XY-YX, YZ-ZY, ZX-XZ
		A = VectorSubtract(A, B);
		// YZ-ZY, ZX-XZ, XY-YX
		return VectorSwizzle(A, 1, 2, 0, 3);
#else
		return VectorCross(Vec1, Vec2);
#endif
	}

	/**
		 * Calculates the cross product of two vectors (XYZ components). W of the input should be 0, and will remain 0.
		 * This function is not using FMA for stability reason, rounding with FMA could cause numerical instability.
		 *
		 * @param Vec1	1st vector
		 * @param Vec2	2nd vector
		 * @return		cross(Vec1.xyz, Vec2.xyz). W of the input should be 0, and will remain 0.
		 */
	FORCEINLINE VectorRegister4Double VectorCrossNoFMA(const VectorRegister4Double& Vec1, const VectorRegister4Double& Vec2)
	{
#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON
		VectorRegister4Double C = VectorMultiply(Vec1, VectorSwizzle(Vec2, 1, 2, 0, 3));
		C = VectorSubtract(C, VectorMultiply(VectorSwizzle(Vec1, 1, 2, 0, 3), Vec2));
		return VectorSwizzle(C, 1, 2, 0, 3);
#elif PLATFORM_ENABLE_VECTORINTRINSICS
		// YZX
		VectorRegister4Double A = VectorSwizzle(Vec2, 1, 2, 0, 3);
		VectorRegister4Double B = VectorSwizzle(Vec1, 1, 2, 0, 3);
		// XY, YZ, ZX
		A = VectorMultiply(A, Vec1);
		// XY-YX, YZ-ZY, ZX-XZ
		A = VectorSubtract(A, VectorMultiply(B, Vec2));
		// YZ-ZY, ZX-XZ, XY-YX
		return VectorSwizzle(A, 1, 2, 0, 3);
#else
		return VectorCross(Vec1, Vec2);
#endif
	}
}

/**
 * Combines two vectors using bitwise NOT AND (treating each vector as a 128 bit field)
 *
 * @param Vec1	1st vector
 * @param Vec2	2nd vector
 * @return		VectorRegister4Float( for each bit i: !Vec1[i] & Vec2[i] )
 */
FORCEINLINE VectorRegister4Float VectorBitwiseNotAnd(const VectorRegister4Float& A, const VectorRegister4Float& B)
{
#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON
	return (VectorRegister4Float)vandq_u32(vmvnq_u32((VectorRegister4Int)A), (VectorRegister4Int)B);
#elif PLATFORM_ENABLE_VECTORINTRINSICS
	return _mm_andnot_ps(A, B);
#else
	return MakeVectorRegisterFloat(
		uint32(~((uint32*)(A.V))[0] & ((uint32*)(B.V))[0]),
		uint32(~((uint32*)(A.V))[1] & ((uint32*)(B.V))[1]),
		uint32(~((uint32*)(A.V))[2] & ((uint32*)(B.V))[2]),
		uint32(~((uint32*)(A.V))[3] & ((uint32*)(B.V))[3]));
#endif
}

FORCEINLINE VectorRegister4Double VectorBitwiseNotAnd(const VectorRegister4Double& A, const VectorRegister4Double& B)
{
	VectorRegister4Double Result;
#if PLATFORM_ENABLE_VECTORINTRINSICS_NEON	
	Result.XY = (VectorRegister2Double)vandq_u32(vmvnq_u32((VectorRegister2Double)A.XY), (VectorRegister2Double)B.XY);
	Result.ZW = (VectorRegister2Double)vandq_u32(vmvnq_u32((VectorRegister2Double)A.ZW), (VectorRegister2Double)B.ZW);
#elif PLATFORM_ENABLE_VECTORINTRINSICS
#if !UE_PLATFORM_MATH_USE_AVX
	Result.XY = _mm_cvtps_pd(_mm_andnot_ps(_mm_cvtpd_ps(A.XY), _mm_cvtpd_ps(B.XY)));
	Result.ZW = _mm_cvtps_pd(_mm_andnot_ps(_mm_cvtpd_ps(A.ZW), _mm_cvtpd_ps(B.ZW)));
#else
	Result = _mm256_andnot_pd(A, B);
#endif
#else
	Result = MakeVectorRegisterDouble(
		uint64(~((uint64*)(A.V))[0] & ((uint64*)(B.V))[0]),
		uint64(~((uint64*)(A.V))[1] & ((uint64*)(B.V))[1]),
		uint64(~((uint64*)(A.V))[2] & ((uint64*)(B.V))[2]),
		uint64(~((uint64*)(A.V))[3] & ((uint64*)(B.V))[3]));
#endif
	return Result;
}

