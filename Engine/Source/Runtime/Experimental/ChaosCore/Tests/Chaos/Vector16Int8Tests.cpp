// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_LOW_LEVEL_TESTS

#include "CoreMinimal.h"
#include "TestHarness.h"

#include "Chaos/Vector16Int8.h"

#include <catch2/generators/catch_generators.hpp>

namespace Chaos
{
	TEST_CASE("VectorRegister16Int8::LoadStore", "[Chaos][VectorRegister16Int8]")
	{
		const int8 Input[16]{ -128, -7, -6, -5, -4, -3, -2, -1, 0, 1, 2, 3, 4, 5, 6, 127 };

		const VectorRegister16Int8 Register = VectorLoad16Int8(Input);
		int8 Output[16]{ -1 };
		VectorStore(Register, Output);

		for (uint8 I = 0; I < 16; ++I)
		{
			CAPTURE(I);
			CHECK(Output[I] == Input[I]);
		}
	}

	TEST_CASE("VectorRegister16Int8::Set1", "[Chaos][VectorRegister16Int8]")
	{
		const int8 Input = (int8)GENERATE(-1, 0, 1);

		const VectorRegister16Int8 Register = Vector16Int8Set1(Input);
		int8 Output[16]{ -2 };
		VectorStore(Register, Output);

		for (uint8 I = 0; I < 16; ++I)
		{
			CAPTURE(I);
			CHECK(Output[I] == Input);
		}
	}

	TEST_CASE("VectorRegister16Int8::CompareEQ", "[Chaos][VectorRegister16Int8]")
	{
		int8 Output[16]{ -1 };

		SECTION("All Equal")
		{
			const int8 Input[16]{ -128, -7, -6, -5, -4, -3, -2, -1, 0, 1, 2, 3, 4, 5, 6, 127 };

			const VectorRegister16Int8 Register = VectorLoad16Int8(Input);
			const VectorRegister16Int8 Result = VectorCompareEQ(Register, Register);
			VectorStore(Result, Output);

			for (uint8 I = 0; I < 16; ++I)
			{
				CAPTURE(I);
				CHECK(Output[I] != 0);
			}
		}
		SECTION("Some Equal")
		{
			const int8 InputA[16]{ 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16 };
			const int8 InputB[16]{ 1, 0, 3, 0, 5, 0, 7, 0, 9,  0, 11,  0, 13,  0, 15,  0 };

			const VectorRegister16Int8 RegisterA = VectorLoad16Int8(InputA);
			const VectorRegister16Int8 RegisterB = VectorLoad16Int8(InputB);
			const VectorRegister16Int8 Result = VectorCompareEQ(RegisterA, RegisterB);
			VectorStore(Result, Output);

			for (uint8 I = 0; I < 16; ++I)
			{
				CAPTURE(I);
				const bool bExpected = (I % 2) == 0 ? true : false;
				const bool bActual = Output[I] != 0;
				CHECK(bExpected == bActual);
			}
		}
		SECTION("None Equal")
		{
			const int8 InputA[16]{ 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16 };
			const int8 InputB[16]{ -1, -2, -3, -4, -5, -6, -7, -8, -9, -10, -11, -12, -13, -14, -15, -16 };
			const VectorRegister16Int8 RegisterA = VectorLoad16Int8(InputA);
			const VectorRegister16Int8 RegisterB = VectorLoad16Int8(InputB);
			const VectorRegister16Int8 Result = VectorCompareEQ(RegisterA, RegisterB);
			VectorStore(Result, Output);

			for (uint8 I = 0; I < 16; ++I)
			{
				CAPTURE(I);
				CHECK(Output[I] == 0);
			}
		}
	}

	TEST_CASE("VectorRegister16Int8::VectorMaskBits", "[Chaos][VectorRegister16Int8]")
	{
		const int8 InputA[16]{ 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16 };
		const VectorRegister16Int8 RegisterA = VectorLoad16Int8(InputA);
		SECTION("Test Each Bit")
		{
			for (uint8 I = 0; I < 16; ++I)
			{
				CAPTURE(I);
				const VectorRegister16Int8 RegisterB = Vector16Int8Set1(InputA[I]);

				const VectorRegister16Int8 Result = VectorCompareEQ(RegisterA, RegisterB);
				const uint16 Actual = VectorMaskBits(Result);

				const uint16 Expected = (uint16)(1 << I);
				CHECK(Expected == Actual);
			}
		}
		SECTION("All Bits")
		{
			const VectorRegister16Int8 Result = VectorCompareEQ(RegisterA, RegisterA);
			const uint16 Actual = VectorMaskBits(Result);

			CHECK(Actual == 0xFFFF);
		}
		SECTION("No Bits")
		{
			const VectorRegister16Int8 RegisterB = Vector16Int8Set1(-1);

			const VectorRegister16Int8 Result = VectorCompareEQ(RegisterA, RegisterB);
			const uint16 Actual = VectorMaskBits(Result);

			CHECK(Actual == 0);
		}
	}
} // namespace Chaos

#endif // WITH_LOW_LEVEL_TESTS
