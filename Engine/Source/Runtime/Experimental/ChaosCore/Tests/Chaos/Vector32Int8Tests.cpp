// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_LOW_LEVEL_TESTS

#include "CoreMinimal.h"
#include "TestHarness.h"

#include "Chaos/Vector32Int8.h"

#include <catch2/generators/catch_generators.hpp>

namespace Chaos
{
	TEST_CASE("VectorRegister32Int8::LoadStore", "[Chaos][VectorRegister32Int8]")
	{
		const int8 Input[32]{ -128, -15, -14, -13, -12, -11, -10, -9, -8, -7, -6, -5, -4, -3, -2, -1, 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 127 };

		const VectorRegister32Int8 Register = VectorLoad32Int8(Input);
		int8 Output[32]{ -1 };
		VectorStore(Register, Output);

		for (uint8 I = 0; I < 32; ++I)
		{
			CAPTURE(I);
			CHECK(Output[I] == Input[I]);
		}
	}

	TEST_CASE("VectorRegister32Int8::Set1", "[Chaos][VectorRegister32Int8]")
	{
		const int8 Input = (int8)GENERATE(-1, 0, 1);

		const VectorRegister32Int8 Register = Vector32Int8Set1(Input);
		int8 Output[32]{ -2 };
		VectorStore(Register, Output);

		for (uint8 I = 0; I < 32; ++I)
		{
			CAPTURE(I);
			CHECK(Output[I] == Input);
		}
	}

	TEST_CASE("VectorRegister32Int8::CompareEQ", "[Chaos][VectorRegister32Int8]")
	{
		int8 Output[32]{ -1 };

		SECTION("All Equal")
		{
			const int8 Input[32]{ -128, -15, -14, -13, -12, -11, -10, -9, -8, -7, -6, -5, -4, -3, -2, -1, 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 127 };

			const VectorRegister32Int8 Register = VectorLoad32Int8(Input);
			const VectorRegister32Int8 Result = VectorCompareEQ(Register, Register);
			VectorStore(Result, Output);

			for (uint8 I = 0; I < 32; ++I)
			{
				CAPTURE(I);
				CHECK(Output[I] != 0);
			}
		}
		SECTION("Some Equal")
		{
			const int8 InputA[32]{ 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20, 21, 22, 23, 24, 25, 26, 27, 28, 29, 30, 31, 32 };
			const int8 InputB[32]{ 1, 0, 3, 0, 5, 0, 7, 0, 9, 00, 11, 00, 13, 00, 15, 00, 17, 00, 19, 00, 21, 00, 23, 00, 25, 00, 27, 00, 29, 00, 31, 00 };

			const VectorRegister32Int8 RegisterA = VectorLoad32Int8(InputA);
			const VectorRegister32Int8 RegisterB = VectorLoad32Int8(InputB);
			const VectorRegister32Int8 Result = VectorCompareEQ(RegisterA, RegisterB);
			VectorStore(Result, Output);

			for (uint8 I = 0; I < 32; ++I)
			{
				CAPTURE(I);
				const bool bExpected = (I % 2) == 0 ? true : false;
				const bool bActual = Output[I] != 0;
				CHECK(bExpected == bActual);
			}
		}
		SECTION("None Equal")
		{
			const int8 InputA[32]{ 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20, 21, 22, 23, 24, 25, 26, 27, 28, 29, 30, 31, 32 };
			const int8 InputB[32]{ -1, -2, -3, -4, -5, -6, -7, -8, -9, -10, -11, -12, -13, -14, -15, -16, -17, -18, -19, -20, -21, -22, -23, -24, -25, -26, -27, -28, -29, -30, -31, -32 };
			const VectorRegister32Int8 RegisterA = VectorLoad32Int8(InputA);
			const VectorRegister32Int8 RegisterB = VectorLoad32Int8(InputB);
			const VectorRegister32Int8 Result = VectorCompareEQ(RegisterA, RegisterB);
			VectorStore(Result, Output);

			for (uint8 I = 0; I < 32; ++I)
			{
				CAPTURE(I);
				CHECK(Output[I] == 0);
			}
		}
	}

	TEST_CASE("VectorRegister32Int8::VectorMaskBits", "[Chaos][VectorRegister32Int8]")
	{
		const int8 InputA[32]{ 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17, 18, 19, 20, 21, 22, 23, 24, 25, 26, 27, 28, 29, 30, 31, 32 };
		const VectorRegister32Int8 RegisterA = VectorLoad32Int8(InputA);
		SECTION("Test Each Bit")
		{
			for (uint8 I = 0; I < 32; ++I)
			{
				CAPTURE(I);
				const VectorRegister32Int8 RegisterB = Vector32Int8Set1(InputA[I]);

				const VectorRegister32Int8 Result = VectorCompareEQ(RegisterA, RegisterB);
				const uint32 Actual = VectorMaskBits(Result);

				const uint32 Expected = 1u << I;
				CHECK(Expected == Actual);
			}
		}
		SECTION("All Bits")
		{
			const VectorRegister32Int8 Result = VectorCompareEQ(RegisterA, RegisterA);
			const uint32 Actual = VectorMaskBits(Result);

			CHECK(Actual == 0xFFFFFFFF);
		}
		SECTION("No Bits")
		{
			const VectorRegister32Int8 RegisterB = Vector32Int8Set1(-1);

			const VectorRegister32Int8 Result = VectorCompareEQ(RegisterA, RegisterB);
			const uint32 Actual = VectorMaskBits(Result);

			CHECK(Actual == 0);
		}
	}
} // namespace Chaos

#endif // WITH_LOW_LEVEL_TESTS
