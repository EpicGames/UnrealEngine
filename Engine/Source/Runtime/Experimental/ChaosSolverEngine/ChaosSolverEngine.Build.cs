// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChaosSolverEngine : ModuleRules
	{
		public ChaosSolverEngine(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"RenderCore",
					"RHI",
					"DeveloperSettings",
					"DataflowCore",
					"DataflowEngine",
					"DataflowSimulation",
					"ChaosVDRuntime", // If CVD is compiled out, this only provides access to its structs data types
					"TraceBasedDebuggers"
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Messaging",
					"RigidPhysics",
				}
				);

			SetupModulePhysicsSupport(Target);
			SetupModuleChaosVisualDebuggerSupport(Target);
			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");

			bAllowUETypesInNamespaces = true;
		}
	}
}
