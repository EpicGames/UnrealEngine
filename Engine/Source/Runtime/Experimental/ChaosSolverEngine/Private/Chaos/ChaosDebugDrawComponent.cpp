// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosDebugDrawComponent.h"
#include "Chaos/DebugDrawQueue.h"
#include "ChaosLog.h"
#include "Debug/DebugDrawService.h"
#include "DrawDebugHelpers.h"
#if WITH_EDITOR
#include "UObject/UObjectIterator.h"
#endif
#include "Engine/Canvas.h"
#include "Engine/Engine.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosDebugDrawComponent)

#if CHAOS_DEBUG_DRAW
void ChaosDebugDraw_Enabled_Changed(IConsoleVariable* CVar)
{
	Chaos::FDebugDrawQueue::GetInstance().SetEnabled(CVar->GetBool());
}

bool bChaosDebugDraw_Enabled = false;
FAutoConsoleVariableRef CVarChaos_DebugDraw_Enabled(TEXT("p.Chaos.DebugDraw.Enabled"), bChaosDebugDraw_Enabled, TEXT("Whether to debug draw low level physics solver information"), FConsoleVariableDelegate::CreateStatic(ChaosDebugDraw_Enabled_Changed));

int ChaosDebugDraw_MaxElements = 20000;
FAutoConsoleVariableRef CVarChaos_DebugDraw_MaxElements(TEXT("p.Chaos.DebugDraw.MaxLines"), ChaosDebugDraw_MaxElements, TEXT("Set the maximum number of debug draw lines that can be rendered (to limit perf drops)"));

float ChaosDebugDraw_Radius = 3000.0f;
FAutoConsoleVariableRef CVarChaos_DebugDraw_Radius(TEXT("p.Chaos.DebugDraw.Radius"), ChaosDebugDraw_Radius, TEXT("Set the radius from the camera where debug draw capture stops (0 means infinite)"));

bool ChaosDebugDraw_SeeThrough = true;
FAutoConsoleVariableRef CVarChaos_DebugDraw_SeeThrough(TEXT("p.Chaos.DebugDraw.SeeThrough"), ChaosDebugDraw_SeeThrough, TEXT("When enabled , lines will be drawn on top of all geometry"));

bool bChaosDebugDraw_SingleActor = false;
FAutoConsoleVariableRef CVarChaosDebugDraw_SingleActor(TEXT("p.Chaos.DebugDraw.SingleActor"), bChaosDebugDraw_SingleActor, TEXT("If true, then we draw for the actor the camera is looking at."));

float ChaosDebugDraw_SingleActorTraceLength = 2000.0f;
FAutoConsoleVariableRef CVarChaosDebugDraw_SingleActorTraceLength(TEXT("p.Chaos.DebugDraw.SingleActorTraceLength"), ChaosDebugDraw_SingleActorTraceLength, TEXT("Set the trace length from the camera that is used to select the single actor."));

float ChaosDebugDraw_SingleActorMaxRadius = 1000.0f;
FAutoConsoleVariableRef CVarChaosDebugDraw_SingleActorMaxRadius(TEXT("p.Chaos.DebugDraw.SingleActorMaxRadius"), ChaosDebugDraw_SingleActorMaxRadius, TEXT("Set the max radius to draw around the single actor."));

float ChaosDebugDraw_ShowPIEServer = false;
FAutoConsoleVariableRef CVarChaos_DebugDraw_ShowPIEServer(TEXT("p.Chaos.DebugDraw.ShowPIEServer"), ChaosDebugDraw_ShowPIEServer, TEXT("When running in PIE mode, show the server debug draw"));

float ChaosDebugDraw_ShowPIEClient = true;
FAutoConsoleVariableRef CVarChaos_DebugDraw_ShowPIEClient(TEXT("p.Chaos.DebugDraw.ShowPIEClient"), ChaosDebugDraw_ShowPIEClient, TEXT("When running in PIE mode, show the client debug draw"));

int bChaosDebugDraw_DrawMode = 0;
FAutoConsoleVariableRef CVarArrowSize(TEXT("p.Chaos.DebugDraw.Mode"), bChaosDebugDraw_DrawMode, TEXT("Where to send debug draw commands. 0 = UE Debug Draw; 1 = VisLog; 2 = Both"));


float CommandLifeTime(const float LifeTime)
{
	// The linebatch time handling is a bit awkward and we need to translate
	// Linebatcher Lifetime < 0: eternal (regardless of bPersistent flag)
	// Linebatcher Lifetime = 0: default lifetime (which is usually 1 second)
	// Linebatcher Lifetime > 0: specified duration
	// Whereas in our Command, 
	// Command lifetime <= 0: 1 frame
	// Command lifetime > 0: specified duration
	if (LifeTime <= 0)
	{
		// One frame - must be non-zero but also less than the next frame time which we don't know
		// NOTE: this only works because UChaosDebugDrawComponent ticks after the line batcher
		return UE_SMALL_NUMBER;
	}
	
	return LifeTime;
}

void DebugDrawChaosCommand(const UWorld* World, const Chaos::FLatentDrawCommand& Command)
{
	using namespace Chaos;

	const uint8 DepthPriority = ChaosDebugDraw_SeeThrough ? Command.DepthPriority : 0;
	switch (Command.Type)
	{
	case FLatentDrawCommand::EDrawType::Point:
		DrawDebugPoint(World, Command.LineStart, Command.Thickness, Command.Color, Command.bPersistentLines, CommandLifeTime(Command.LifeTime), DepthPriority);
		break;
	case FLatentDrawCommand::EDrawType::Line:
		DrawDebugLine(World, Command.LineStart, Command.LineEnd, Command.Color, Command.bPersistentLines, CommandLifeTime(Command.LifeTime), DepthPriority, Command.Thickness);
		break;
	case FLatentDrawCommand::EDrawType::DirectionalArrow:
		DrawDebugDirectionalArrow(World, Command.LineStart, Command.LineEnd, Command.ArrowSize, Command.Color, Command.bPersistentLines, CommandLifeTime(Command.LifeTime), DepthPriority, Command.Thickness);
		break;
	case FLatentDrawCommand::EDrawType::Sphere:
		DrawDebugSphere(World, Command.LineStart, Command.Radius, Command.Segments, Command.Color, Command.bPersistentLines, CommandLifeTime(Command.LifeTime), DepthPriority, Command.Thickness);
		break;
	case FLatentDrawCommand::EDrawType::Box:
		DrawDebugBox(World, Command.Center, Command.Extent, Command.Rotation, Command.Color, Command.bPersistentLines, CommandLifeTime(Command.LifeTime), DepthPriority, Command.Thickness);
		break;
	case FLatentDrawCommand::EDrawType::String:
		DrawDebugString(World, Command.TextLocation, Command.Text, Command.TestBaseActor, Command.Color, CommandLifeTime(Command.LifeTime), Command.bDrawShadow, Command.FontScale);
		break;
	case FLatentDrawCommand::EDrawType::Circle:
	{
		FMatrix M = FRotationMatrix::MakeFromYZ(Command.YAxis, Command.ZAxis);
		M.SetOrigin(Command.Center);
		DrawDebugCircle(World, M, Command.Radius, Command.Segments, Command.Color, Command.bPersistentLines, CommandLifeTime(Command.LifeTime), DepthPriority, Command.Thickness, Command.bDrawAxis);
		break;
	}
	case FLatentDrawCommand::EDrawType::Capsule:
		DrawDebugCapsule(World, Command.Center, Command.HalfHeight, Command.Radius, Command.Rotation, Command.Color, Command.bPersistentLines, CommandLifeTime(Command.LifeTime), DepthPriority, Command.Thickness);
	default:
		break;
	}
}

void VisLogChaosCommand(const AActor* Actor, const Chaos::FLatentDrawCommand& Command)
{
	using namespace Chaos;

	switch (Command.Type)
	{
	case FLatentDrawCommand::EDrawType::Point:
		UE_VLOG_SEGMENT_THICK(Actor, LogChaos, Log, Command.LineStart, Command.LineStart, Command.Color, Command.Thickness, TEXT_EMPTY);
		break;
	case FLatentDrawCommand::EDrawType::Line:
		UE_VLOG_SEGMENT(Actor, LogChaos, Log, Command.LineStart, Command.LineEnd, Command.Color, TEXT_EMPTY);
		break;
	case FLatentDrawCommand::EDrawType::DirectionalArrow:
		UE_VLOG_SEGMENT(Actor, LogChaos, Log, Command.LineStart, Command.LineEnd, Command.Color, TEXT_EMPTY);
		break;
	case FLatentDrawCommand::EDrawType::Sphere:
	{
		// VLOG Capsule uses the bottom end as the origin (though the variable is named Center)
		FVector Base = Command.LineStart - Command.Radius * FVector::UpVector;
		UE_VLOG_CAPSULE(Actor, LogChaos, Log, Base, Command.Radius + KINDA_SMALL_NUMBER, Command.Radius, FQuat::Identity, Command.Color, TEXT_EMPTY);
		break;
	}
	case FLatentDrawCommand::EDrawType::Box:
		UE_VLOG_OBOX(Actor, LogChaos, Log, FBox(-Command.Extent, Command.Extent), FQuatRotationTranslationMatrix::Make(Command.Rotation, Command.Center), Command.Color, TEXT_EMPTY);
		break;
	case FLatentDrawCommand::EDrawType::String:
		UE_VLOG(Command.TestBaseActor, LogChaos, Log, TEXT("%s"), *Command.Text);
		break;
	case FLatentDrawCommand::EDrawType::Circle:
	{
		const FMatrix M = FRotationMatrix::MakeFromYZ(Command.YAxis, Command.ZAxis);
		UE_VLOG_CIRCLE(Actor, LogChaos, Log, Command.Center, M.GetUnitAxis(EAxis::X), Command.Radius, Command.Color, TEXT_EMPTY);
		break;
	}
	case FLatentDrawCommand::EDrawType::Capsule:
	{
		// VLOG Capsule uses the bottom end as the origin (though the variable is named Center)
		FVector Base = Command.Center - Command.HalfHeight * (Command.Rotation * FVector::UpVector);
		UE_VLOG_CAPSULE(Actor, LogChaos, Log, Base, Command.HalfHeight, Command.Radius, Command.Rotation, Command.Color, TEXT_EMPTY);
		break;
	}
	default:
		break;
	}
}

void DebugDrawChaos(const AActor* DebugDrawActor, const TArray<Chaos::FLatentDrawCommand>& DrawCommands, const bool bIsPaused)
{
	using namespace Chaos;

	if (DebugDrawActor == nullptr)
	{
		return;
	}

	const UWorld* World = DebugDrawActor->GetWorld();
	if (World == nullptr)
	{
		return;
	}

	if (World->IsPreviewWorld())
	{
		return;
	}

	if ((World->GetNetMode() == NM_DedicatedServer) && !ChaosDebugDraw_ShowPIEServer)
	{
		return;
	}

	if ((World->GetNetMode() != NM_DedicatedServer) && !ChaosDebugDraw_ShowPIEClient)
	{
		return;
	}

	// Draw all the captured elements in the viewport
	const bool bDrawUe = bChaosDebugDraw_DrawMode != 1;
	if (bDrawUe)
	{
		for (const FLatentDrawCommand& Command : DrawCommands)
		{
			DebugDrawChaosCommand(World, Command);
		}
	}

	// Draw all the captured elements in the VisLog
	const bool bDrawVisLog = bChaosDebugDraw_DrawMode != 0;
	if (bDrawVisLog)
	{
		for (const FLatentDrawCommand& Command : DrawCommands)
		{
			const AActor* Actor = (Command.TestBaseActor) ? Command.TestBaseActor : DebugDrawActor;
			VisLogChaosCommand(Actor, Command);
		}
	}
}
#endif



UChaosDebugDrawComponent::UChaosDebugDrawComponent()
	: bInPlay(false)
{
#if CHAOS_DEBUG_DRAW
	// We must tick after anything that uses Chaos Debug Draw and also after the Line Batcher Component
	PrimaryComponentTick.bAllowTickOnDedicatedServer = false;
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.SetTickFunctionEnable(true);
	PrimaryComponentTick.TickGroup = TG_PostUpdateWork;

	bTickInEditor = true;
#endif
}

void UChaosDebugDrawComponent::BeginDestroy()
{
	Super::BeginDestroy();

#if CHAOS_DEBUG_DRAW
	// Don't allow new commands to be enqueued when we are paused
	Chaos::FDebugDrawQueue::GetInstance().SetConsumerActive(this, false);
#endif
}

void UChaosDebugDrawComponent::BeginPlay()
{
	Super::BeginPlay();

#if CHAOS_DEBUG_DRAW
	SetTickableWhenPaused(true);
#endif

	bInPlay = true;

#if CHAOS_DEBUG_DRAW
	Chaos::FDebugDrawQueue::GetInstance().SetConsumerActive(this, bInPlay);

	ChaosDebugDraw_Enabled_Changed(&(*CVarChaos_DebugDraw_Enabled));
#endif
}

void UChaosDebugDrawComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);

#if CHAOS_DEBUG_DRAW
	SetTickableWhenPaused(false);
#endif

	bInPlay = false;

#if CHAOS_DEBUG_DRAW
	Chaos::FDebugDrawQueue::GetInstance().SetConsumerActive(this, bInPlay);
#endif
}

#if WITH_EDITOR && CHAOS_DEBUG_DRAW
void UChaosDebugDrawComponent::OnRegister()
{
	Super::OnRegister();
	const UWorld* World = GetWorld();
	check(World);
	// GameWorld is handled by BeginPlay/EndPlay
	if (!World->IsGameWorld() && !World->IsPreviewWorld())
	{
		Chaos::FDebugDrawQueue::GetInstance().SetConsumerActive(this, /*bConsumerActive*/true);
	}
}
void UChaosDebugDrawComponent::OnUnregister()
{
	const UWorld* World = GetWorld();
	check(World);
	// GameWorld is handled by BeginPlay/EndPlay
	if (!World->IsGameWorld() && !World->IsPreviewWorld())
	{
		Chaos::FDebugDrawQueue::GetInstance().SetConsumerActive(this, /*bConsumerActive*/false);
	}
	Super::OnUnregister();
}
#endif // WITH_EDITOR && CHAOS_DEBUG_DRAW

void UChaosDebugDrawComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	using namespace Chaos;

	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

#if CHAOS_DEBUG_DRAW
	// Update the region of interest based on camera location
	// @todo(chaos): this should use the view location of the primary viewport, but not sure how to get that
	// We're not handling multiple worlds or viewports anyway, so this is as good as it gets.
	UWorld* World = GetWorld();
	if (World != nullptr)
	{
		FVector RegionOfInterestOrigin = FVec3(0);
		float RegionOfInterestRadius = ChaosDebugDraw_Radius;

		if (World->ViewLocationsRenderedLastFrame.Num() > 0)
		{
			RegionOfInterestOrigin = World->ViewLocationsRenderedLastFrame[0];

			if (bChaosDebugDraw_SingleActor)
			{
				if (const APlayerController* Controller = GEngine->GetFirstLocalPlayerController(World))
				{
					FVector CamLoc;
					FRotator CamRot;
					Controller->GetPlayerViewPoint(CamLoc, CamRot);
					FVector CamForward = CamRot.Vector();
					CamForward *= ChaosDebugDraw_SingleActorTraceLength;

					FVector TraceStart = CamLoc;
					FVector TraceEnd = TraceStart + CamForward;

					FHitResult HitResult(ForceInit);
					FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(ChaosDebugVisibilityTrace), true, Controller->GetPawn());
					bool bHit = World->LineTraceSingleByChannel(HitResult, TraceStart, TraceEnd, ECC_Visibility, TraceParams);
					if (bHit && HitResult.GetActor() != nullptr)
					{
						FVector Origin, BoxExtent;
						HitResult.GetActor()->GetActorBounds(true, Origin, BoxExtent);
						const float Radius = FMath::Min(BoxExtent.Size(), ChaosDebugDraw_SingleActorMaxRadius);
						RegionOfInterestOrigin = Origin;
						RegionOfInterestRadius = Radius;
					}
				}
			}

		}

		FDebugDrawQueue::GetInstance().SetRegionOfInterest(RegionOfInterestOrigin, RegionOfInterestRadius);
		FDebugDrawQueue::GetInstance().SetMaxCost(ChaosDebugDraw_MaxElements);

		// Draw any commands from threads that did not have the debug draw context initialized
		// @todo(chaos): we should make sure all threads initialize the context...
		const bool bIsPaused = World->IsPaused();
		if (!bIsPaused)
		{
			FDebugDrawQueue::GetInstance().ExtractAllElements(DrawCommands);

			DebugDrawChaos(GetOwner(), DrawCommands, World->IsPaused());
		}
	}
#endif
}

void UChaosDebugDrawComponent::BindWorldDelegates()
{
#if CHAOS_DEBUG_DRAW
	FWorldDelegates::OnPostWorldInitialization.AddStatic(&HandlePostWorldInitialization);
#endif
}

void UChaosDebugDrawComponent::HandlePostWorldInitialization(UWorld* World, const UWorld::InitializationValues)
{
#if CHAOS_DEBUG_DRAW
	if ((World != nullptr) && !IsRunningCommandlet() && !World->IsPreviewWorld())
	{
		CreateDebugDrawActor(World);
	}
#endif
}

void UChaosDebugDrawComponent::CreateDebugDrawActor(UWorld* World)
{
#if CHAOS_DEBUG_DRAW
	if ((World->GetNetMode() != ENetMode::NM_Client) && (World->GetNetMode() != ENetMode::NM_Standalone))
		return;

	static FName NAME_ChaosDebugDrawActor = "ChaosDebugDrawActor";

	FActorSpawnParameters Params;
	Params.Name = NAME_ChaosDebugDrawActor;
	Params.ObjectFlags = Params.ObjectFlags | RF_Transient;

#if WITH_EDITOR
	Params.bHideFromSceneOutliner = true;

	// Make sure to not create more than one actor for non game worlds.
	// Those can get reinitialized and OnPostWorldInitialization is called more than once
	if (!World->IsGameWorld())
	{
		for (TObjectIterator<UChaosDebugDrawComponent> It; It; ++It)
		{
			const AActor* Actor = It->GetOwner();
			if (Actor != nullptr && Actor->GetFName()== NAME_ChaosDebugDrawActor && Actor->GetWorld() == World)
			{
				return;
			}
		}
	}	
#endif // WITH_EDITOR
	
	AActor* Actor = World->SpawnActor<AActor>(FVector::ZeroVector, FRotator::ZeroRotator, Params);
	
	UChaosDebugDrawComponent* Comp = NewObject<UChaosDebugDrawComponent>(Actor);
	Actor->AddInstanceComponent(Comp);
	Comp->RegisterComponent();

	// SetMaxCommands and SetRegionOfInterest here so that the first frame gets whatever cvar value we have set.
	// We also call it every tick (which is at the end of each frame)
	Chaos::FDebugDrawQueue::GetInstance().SetMaxCost(ChaosDebugDraw_MaxElements);
	Chaos::FDebugDrawQueue::GetInstance().SetRegionOfInterest(FVector::ZeroVector, ChaosDebugDraw_Radius);
#endif
}




