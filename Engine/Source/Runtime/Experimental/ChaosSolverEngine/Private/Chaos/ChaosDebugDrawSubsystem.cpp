// Copyright Epic Games, Inc. All Rights Reserved.
#include "Chaos/ChaosDebugDrawSubsystem.h"
#include "Chaos/DebugDrawQueue.h"
#include "ChaosDebugDraw/ChaosDDContext.h"
#include "ChaosDebugDraw/ChaosDDLog.h"
#include "ChaosDebugDraw/ChaosDDRenderer.h"
#include "ChaosDebugDraw/ChaosDDScene.h"
#include "ChaosDebugDraw/ChaosDDTimeline.h"
#include "DrawDebugHelpers.h"
#include "Engine/Engine.h"
#include "Engine/HitResult.h"
#include "Engine/World.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "HAL/CriticalSection.h"
#include "Logging/StructuredLog.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosDebugDrawSubsystem)

#if CHAOS_DEBUG_DRAW

FDelegateHandle UChaosDebugDrawSubsystem::OnTickWorldStartDelegate;
FDelegateHandle UChaosDebugDrawSubsystem::OnTickWorldEndDelegate;

// @todo(chaos): Stuff in ChaosDebugDrawComponent that should move here when we drop that file
extern bool bChaosDebugDraw_Enabled;
extern float ChaosDebugDraw_Radius;
extern int32 bChaosDebugDraw_DrawMode;
extern int32 ChaosDebugDraw_MaxElements;
extern bool bChaosDebugDraw_SingleActor;
extern float ChaosDebugDraw_SingleActorTraceLength;
extern float ChaosDebugDraw_SingleActorMaxRadius;

extern float CommandLifeTime(const float LifeTime);
extern void DebugDrawChaosCommand(const UWorld* World, const Chaos::FLatentDrawCommand& Command);
extern void VisLogChaosCommand(const AActor* Actor, const Chaos::FLatentDrawCommand& Command);

namespace Chaos::CVars
{
	extern int32 ChaosSolverDebugDrawShowServer;
	extern int32 ChaosSolverDebugDrawShowClient;

	// Temp cvar to allow debug draw in Asset Editors until we can implement a better per world (and per asset editor) solution - UE-229329
	bool bChaosDebugDraw_PreviewWorldsEnabled = false;
	FAutoConsoleVariableRef CVarbChaosDebugDraw_SupportPreviewWorlds(TEXT("p.Chaos.PreviewWorld.DebugDraw.Enabled"), bChaosDebugDraw_PreviewWorldsEnabled, TEXT("Enables/Disables Chaos debug Draw support in Preview worlds. Mostly used by Asset Editors."));
}

class FChaosDDRenderer : public ChaosDD::Private::IChaosDDRenderer
{
public:
	FChaosDDRenderer(UWorld* InWorld, const FSphere3d& InDrawRegion, const int32 InRenderBudget)
		: World(InWorld)
		, DrawRegion(InDrawRegion)
		, RenderBudget(InRenderBudget)
		, RenderCost(0)
		, SphereSegments(8)
		, DepthPriority(10)
		, bIsServer(false)
	{
	}

	virtual bool IsServer() const override final
	{
		return bIsServer;
	}

	void SetIsServer(const bool bInIsServer)
	{
		bIsServer = bInIsServer;
	}

	virtual FSphere3d GetDrawRegion() const override final
	{
		return DrawRegion;
	}

	int32 GetRenderCost() const
	{
		return RenderCost;
	}

	int32 GetRenderBudget() const
	{
		return RenderBudget;
	}

	bool WasRenderBudgetExceeded() const
	{
		return (RenderCost > RenderBudget) && (RenderBudget > 0);
	}

	bool IsInDrawRegion(const FBox3d& Bounds) const
	{
		const double DistanceSq = Bounds.ComputeSquaredDistanceToPoint(DrawRegion.Center);
		return DistanceSq <= FMath::Square(DrawRegion.W);
	}

	virtual void RenderPoint(const FVector3d& Position, const FColor& Color, float PointSize, float Lifetime) override final
	{
		constexpr int32 Cost = 1;
		const FBox3d Bounds = FBox3d(Position, Position);

		if (IsInDrawRegion(Bounds) && TryAddToCost(Cost))
		{
			DrawDebugPoint(World, Position, PointSize, Color, false, CommandLifeTime(Lifetime), DepthPriority);
		}
	}

	virtual void RenderLine(const FVector3d& A, const FVector3d& B, const FColor& Color, float LineThickness, float Lifetime) override final
	{
		constexpr int32 Cost = 1;
		const FBox3d Bounds = FBox3d(FVector3d::Min(A, B), FVector3d::Max(A, B));

		if (IsInDrawRegion(Bounds) && TryAddToCost(Cost))
		{
			DrawDebugLine(World, A, B, Color, false, CommandLifeTime(Lifetime), DepthPriority, LineThickness);
		}
	}

	virtual void RenderArrow(const FVector3d& A, const FVector3d& B, float ArrowSize, const FColor& Color, float LineThickness, float Lifetime) override final
	{
		constexpr int32 Cost = 3;
		const FBox3d Bounds = FBox3d(FVector3d::Min(A, B), FVector3d::Max(A, B));

		if (IsInDrawRegion(Bounds) && TryAddToCost(Cost))
		{
			DrawDebugDirectionalArrow(World, A, B, ArrowSize, Color, false, CommandLifeTime(Lifetime), DepthPriority, LineThickness);
		}
	}

	virtual void RenderCircle(const FVector3d& Center, const FMatrix& Axes, float Radius, const FColor& Color, float LineThickness, float Lifetime) override final
	{
		constexpr int32 Cost = 8;
		const FBox3d Bounds = FBox3d(Center - FVector3d(Radius), Center + FVector3d(Radius));

		if (IsInDrawRegion(Bounds) && TryAddToCost(Cost))
		{
			FMatrix M = Axes;
			M.SetOrigin(Center);
			DrawDebugCircle(World, M, Radius, SphereSegments, Color, false, CommandLifeTime(Lifetime), DepthPriority, LineThickness, false);
		}
	}

	virtual void RenderSphere(const FVector3d& Center, float Radius, const FColor& Color, float LineThickness, float Lifetime) override final
	{
		constexpr int32 Cost = 16;
		const FBox3d Bounds = FBox3d(Center - FVector3d(Radius), Center + FVector3d(Radius));

		if (IsInDrawRegion(Bounds) && TryAddToCost(Cost))
		{
			DrawDebugSphere(World, Center, Radius, SphereSegments, Color, false, CommandLifeTime(Lifetime), DepthPriority, LineThickness);
		}
	}

	virtual void RenderCapsule(const FVector3d& Center, const FQuat4d& Rotation, float HalfHeight, float Radius, const FColor& Color, float LineThickness, float Lifetime) override final
	{
		constexpr int32 Cost = 16;
		const FVector3d EndOffset = HalfHeight * (Rotation * FVector3d::UnitZ());
		const FVector3d A = Center - EndOffset;
		const FVector3d B = Center + EndOffset;
		const FBox3d Bounds = FBox3d(FVector3d::Min(A, B), FVector3d::Max(A, B));

		if (IsInDrawRegion(Bounds) && TryAddToCost(Cost))
		{
			DrawDebugCapsule(World, Center, HalfHeight, Radius, Rotation, Color, false, CommandLifeTime(Lifetime), DepthPriority, LineThickness);
		}
	}

	virtual void RenderBox(const FVector3d& Position, const FQuat4d& Rotation, const FVector3d& Size, const FColor& Color, float LineThickness, float Lifetime) override final
	{
		constexpr int32 Cost = 12;
		const FBox3d Bounds = FBox3d(-0.5 * Size, 0.5 * Size).TransformBy(FTransform(Rotation, Position));

		if (IsInDrawRegion(Bounds) && TryAddToCost(Cost))
		{
			DrawDebugBox(World, Position, Size, Rotation, Color, false, CommandLifeTime(Lifetime), DepthPriority, LineThickness);
		}
	}

	virtual void RenderTriangle(const FVector3d& A, const FVector3d& B, const FVector3d& C, const FColor& Color, float LineThickness, float Lifetime) override final
	{
		constexpr int32 Cost = 3;
		const FBox3d Bounds = FBox3d(FVector3d::Min3(A, B, C), FVector3d::Max3(A, B, C));

		if (IsInDrawRegion(Bounds) && TryAddToCost(Cost))
		{
			DrawDebugLine(World, A, B, Color, false, CommandLifeTime(Lifetime), DepthPriority, LineThickness);
			DrawDebugLine(World, B, C, Color, false, CommandLifeTime(Lifetime), DepthPriority, LineThickness);
			DrawDebugLine(World, C, A, Color, false, CommandLifeTime(Lifetime), DepthPriority, LineThickness);
		}
	}

	virtual void RenderString(const FVector3d& TextLocation, const FString& Text, const FColor& Color, float FontScale, bool bDrawShadow, float Lifetime) override final
	{
		constexpr int32 Cost = 10;
		const FBox3d Bounds = FBox3d(TextLocation, TextLocation);

		if (IsInDrawRegion(Bounds) && TryAddToCost(Cost))
		{
			DrawDebugString(World, TextLocation, Text, nullptr, Color, CommandLifeTime(Lifetime), bDrawShadow, FontScale);
		}
	}

	virtual void RenderLatentCommand(const Chaos::FLatentDrawCommand& Command) override final
	{
		// We don't check the cost of the legacy commands because the budget is applied at capture time for them
		const bool bDrawUe = bChaosDebugDraw_DrawMode != 1;
		if (bDrawUe)
		{
			DebugDrawChaosCommand(World, Command);
		}

		const bool bDrawVisLog = bChaosDebugDraw_DrawMode != 0;
		if (bDrawVisLog)
		{
			VisLogChaosCommand(Command.TestBaseActor, Command);
		}
	}

private:
	bool TryAddToCost(int32 InCost)
	{
		RenderCost += InCost;

		// A budget of zero means infinity
		if ((RenderCost <= RenderBudget) || (RenderBudget == 0))
		{
			RenderCost += InCost;
			return true;
		}

		return false;
	}

	UWorld* World;
	FSphere3d DrawRegion;
	int32 RenderBudget;
	int32 RenderCost;

	// Draw Settings
	int32 SphereSegments;
	int8 DepthPriority;
	bool bIsServer;
};

class FChaosDDWorldManager
{
public:
	static FChaosDDWorldManager& Get()
	{
		static FChaosDDWorldManager SManager;
		return SManager;
	}

	void SetServerDebugDrawScene(const ChaosDD::Private::FChaosDDScenePtr& InServerScene)
	{
		FScopeLock Lock(&SceneCS);
		CDDServerScene = InServerScene;
	}

	const ChaosDD::Private::FChaosDDScenePtr& GetServerDebugDrawScene() const
	{
		FScopeLock Lock(&SceneCS);
		return CDDServerScene;
	}

private:

	mutable FCriticalSection SceneCS;

	ChaosDD::Private::FChaosDDScenePtr CDDServerScene;
};

#endif //CHAOS_DEBUG_DRAW


bool UChaosDebugDrawSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
#if CHAOS_DEBUG_DRAW
	if (UWorld* World = Cast<UWorld>(Outer))
	{
		const bool bIsWorldTypeSupported = World->IsPreviewWorld() ? Chaos::CVars::bChaosDebugDraw_PreviewWorldsEnabled : true;
		const bool bCreateDebugDraw = !IsRunningCommandlet() && bIsWorldTypeSupported;

		if (bCreateDebugDraw)
		{
			UE_LOGFMT(LogChaosDD, Log, "Creating Chaos Debug Draw Scene for world {0}",World->GetName());
		}
		else
		{
			UE_LOGFMT(LogChaosDD, Verbose, "Not creating Chaos Debug Draw Scene for world {0}", World->GetName());
		}
		
		return bCreateDebugDraw;
	}
	return Super::ShouldCreateSubsystem(Outer);
#else
	return false;
#endif
}

void UChaosDebugDrawSubsystem::PostInitialize()
{
	Super::PostInitialize();

#if CHAOS_DEBUG_DRAW
	if (UWorld* World = GetWorld())
	{
		const bool bIsServer = World->IsNetMode(NM_DedicatedServer);

		// Create the debug draw scene which holds all debug draw data for this world
		const FString DDSceneName = FString::Format(TEXT("{0} {1}"), { World->GetName(), bIsServer ? TEXT("Server") : TEXT("Client") });
		CDDScene = MakeShared<ChaosDD::Private::FChaosDDScene>(DDSceneName, bIsServer);

		// Create the debug draw timeline for the game thread
		const FString DDTimelineName = FString::Format(TEXT("{0} {1} {2}"), { World->GetName(), bIsServer ? TEXT("Server") : TEXT("Client"), TEXT("Game Frame") });
		CDDWorldTimeline = CDDScene->CreateTimeline(DDTimelineName);

		// Tell the physics scene about the debug draw - it is async and will create its own timeline(s)
		if (World->GetPhysicsScene() != nullptr)
		{
			World->GetPhysicsScene()->SetDebugDrawScene(CDDScene);
		}

		if (bIsServer)
		{
			FChaosDDWorldManager::Get().SetServerDebugDrawScene(CDDScene);
		}
	}
#endif
}

void UChaosDebugDrawSubsystem::Deinitialize()
{
#if CHAOS_DEBUG_DRAW
	if (UWorld* World = GetWorld())
	{
		if (CDDScene.IsValid() && CDDScene->IsServer())
		{
			FChaosDDWorldManager::Get().SetServerDebugDrawScene({});
		}
	}
#endif

	Super::Deinitialize();
}

void UChaosDebugDrawSubsystem::Startup()
{
#if CHAOS_DEBUG_DRAW
	UE_LOGFMT(LogChaosDD, Log, "Chaos Debug Draw Startup");

	OnTickWorldStartDelegate = FWorldDelegates::OnWorldPreActorTick.AddStatic(&StaticOnWorldTickStart);
	OnTickWorldEndDelegate = FWorldDelegates::OnWorldPostActorTick.AddStatic(&StaticOnWorldTickEnd);
#endif
}

void UChaosDebugDrawSubsystem::Shutdown()
{
#if CHAOS_DEBUG_DRAW
	UE_LOGFMT(LogChaosDD, Log, "Chaos Debug Draw Shutdown");

	FWorldDelegates::OnWorldPreActorTick.Remove(OnTickWorldStartDelegate);
	FWorldDelegates::OnWorldPostActorTick.Remove(OnTickWorldEndDelegate);
#endif
}


#if CHAOS_DEBUG_DRAW

void UChaosDebugDrawSubsystem::OnWorldTickStart(ELevelTick TickType, float Dt)
{
	if (UWorld* World = GetWorld())
	{
		// Enable or disable the debug draw system
		ChaosDD::Private::FChaosDDContext::SetIsDebugDrawEnabled(bChaosDebugDraw_Enabled);

		CDDWorldTimelineContext.BeginFrame(CDDWorldTimeline, World->GetTimeSeconds(), Dt);
	}
}

void UChaosDebugDrawSubsystem::OnWorldTickEnd(ELevelTick TickType, float Dt)
{
	if (UWorld* World = GetWorld())
	{
		CDDWorldTimelineContext.EndFrame();

		if (CDDScene.IsValid())
		{
			CDDScene->SetCommandBudget(ChaosDebugDraw_MaxElements);
		}

		UpdateDrawRegion();

		RenderScene();
	}
}

void UChaosDebugDrawSubsystem::UpdateDrawRegion()
{
	if (UWorld* World = GetWorld())
	{
		CDDWorldTimelineContext.EndFrame();

		if (CDDScene.IsValid())
		{
			FSphere3d DrawRegion = CDDScene->GetDrawRegion();

			// By default we center the draw region on wherever the world was rendered from,
			// or optionally center the draw region around where the player is looking.
			if (!bChaosDebugDraw_SingleActor)
			{
				DrawRegion.W = ChaosDebugDraw_Radius;
				if (World->ViewLocationsRenderedLastFrame.Num() > 0)
				{
					DrawRegion.Center = World->ViewLocationsRenderedLastFrame[0];
				}
			}
			else
			{
				FVector CamLoc = FVector::ZeroVector;
				FVector CamLook = FVector::XAxisVector;
				AActor* PlayerPawnActor = nullptr;
				bool bHaveCamera = false;

				// If we have a player, use their camera for the raycast as a fallback, but this does
				// not work in PIE Simulate mode
				if (const APlayerController* Controller = GEngine->GetFirstLocalPlayerController(World))
				{
					FRotator CamRot;
					Controller->GetPlayerViewPoint(CamLoc, CamRot);
					CamLook = CamRot.Vector();
					PlayerPawnActor = Controller->GetPawn<AActor>();
					bHaveCamera = true;
				}

				// Use the last rendered transform for the raycast, which works in PIE Simulate mode
				if (World->CachedViewInfoRenderedLastFrame.Num() > 0)
				{
					// NOTE: ViewToWorld has Look=Z-Axis
					FMatrix ViewToWorld = World->CachedViewInfoRenderedLastFrame[0].ViewToWorld;
					FMatrix ViewRot = FRotationMatrix::MakeFromX(ViewToWorld.GetUnitAxis(EAxis::Z));
					CamLoc = ViewToWorld.GetOrigin();
					CamLook = ViewToWorld.GetUnitAxis(EAxis::Z);
					bHaveCamera = true;
				}

				if (bHaveCamera)
				{
					FVector TraceStart = CamLoc;
					FVector TraceEnd = CamLoc + CamLook * ChaosDebugDraw_SingleActorTraceLength;

					FHitResult HitResult(ForceInit);
					FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(ChaosDebugVisibilityTrace), true, PlayerPawnActor);
					bool bHit = World->LineTraceSingleByChannel(HitResult, TraceStart, TraceEnd, ECC_Visibility, TraceParams);
					if (bHit && HitResult.GetActor() != nullptr)
					{
						FVector Origin, BoxExtent;
						HitResult.GetActor()->GetActorBounds(true, Origin, BoxExtent);
						const float Radius = FMath::Min(BoxExtent.Size(), ChaosDebugDraw_SingleActorMaxRadius);
						DrawRegion.Center = Origin;
						DrawRegion.W = Radius;
					}
				}
			}

			CDDScene->SetDrawRegion(DrawRegion);

			// @todo(chaos): This is a problem in multi-client PIE - the draw region on the server can flipflop between
			// the client positions depending on which one renders last. Could support multiple draw regions maybe...
			ChaosDD::Private::FChaosDDScenePtr ServerScene = FChaosDDWorldManager::Get().GetServerDebugDrawScene();
			if (ServerScene.IsValid())
			{
				ServerScene->SetDrawRegion(DrawRegion);
				ServerScene->SetCommandBudget(ChaosDebugDraw_MaxElements);
			}
		}
	}
}

void UChaosDebugDrawSubsystem::RenderScene()
{
	if (UWorld* World = GetWorld())
	{
		if (CDDScene.IsValid() && !CDDScene->IsServer())
		{
			if (!World->IsPaused())
			{
				// @todo(chaos): command budget and render budget should be two different values
				FChaosDDRenderer Renderer = FChaosDDRenderer(World, CDDScene->GetDrawRegion(), CDDScene->GetCommandBudget());

				// Render all of the out-of frame commands
				// @todo(chaos): extracting the global frame hereis a problem if we have multiple 
				// PIE clients, but the global scene is a hack anyway so won't worry about that
				RenderFrame(Renderer, ChaosDD::Private::FChaosDDContext::ExtractGlobalFrame());

				// Render the commands from this world
				RenderScene(Renderer, CDDScene);

				// Render the commands from the server on every client (in PIE)
				// Render the server last so that the client uses the command and render budgets first
				Renderer.SetIsServer(true);
				RenderScene(Renderer, FChaosDDWorldManager::Get().GetServerDebugDrawScene());
				Renderer.SetIsServer(false);

				if (Renderer.WasRenderBudgetExceeded())
				{
					constexpr int32 MsgId = 86421358;
					const FString Msg = FString::Format(TEXT("Debug Draw Render Budget Exceeded for {0} [{1} / {2}]"), { CDDScene->GetName(), Renderer.GetRenderCost(), Renderer.GetRenderBudget()});
					GEngine->AddOnScreenDebugMessage(MsgId, 1.0f, FColor::Red, *Msg);
				}
			}
		}
	}
}

void UChaosDebugDrawSubsystem::RenderScene(FChaosDDRenderer& Renderer, const ChaosDD::Private::FChaosDDScenePtr& Scene)
{
	if (Scene.IsValid() && Scene->IsRenderEnabled())
	{
		TArray<ChaosDD::Private::FChaosDDFramePtr> Frames = Scene->GetLatestFrames();
		for (const ChaosDD::Private::FChaosDDFramePtr& Frame : Frames)
		{
			RenderFrame(Renderer, Frame);

			if (Frame->WasCommandBudgetExceeded())
			{
				constexpr int32 MsgId = 86421357;
				const FString Msg = FString::Format(TEXT("Debug Draw Capture Budget Exceeded for {0} [{1} / {2}]"), { Frame->GetTimeline()->GetName(), Frame->GetCommandCost(), Frame->GetCommandBudget() });
				GEngine->AddOnScreenDebugMessage(MsgId, 1.0f, FColor::Red, *Msg);
			}
		}
	}
}


void UChaosDebugDrawSubsystem::RenderFrame(FChaosDDRenderer& Renderer, const ChaosDD::Private::FChaosDDFramePtr& Frame)
{
	const bool bDebugDrawEnabled = ChaosDD::Private::FChaosDDContext::IsDebugDrawEnabled();

	if (bDebugDrawEnabled && Frame.IsValid() && (Frame->GetNumCommands() + Frame->GetNumLatentCommands() > 0))
	{
		UE_LOGFMT(LogChaosDD, VeryVerbose, "Render {0} {1}+{2} Commands", Frame->GetName(), Frame->GetNumCommands(), Frame->GetNumLatentCommands());

		// Render the legacy commands
		Frame->VisitLatentCommands(
			[&Renderer](const Chaos::FLatentDrawCommand& Command)
			{
				Renderer.RenderLatentCommand(Command);
			});

		// Render the commands
		Frame->VisitCommands(
			[&Renderer](const ChaosDD::Private::FChaosDDCommand& Command)
			{
				Command(Renderer);
			});
	}
}


void UChaosDebugDrawSubsystem::StaticOnWorldTickStart(UWorld* World, ELevelTick TickType, float Dt)
{
	if (UChaosDebugDrawSubsystem* CDDSystem = World->GetSubsystem<UChaosDebugDrawSubsystem>())
	{
		CDDSystem->OnWorldTickStart(TickType, Dt);
	}
}

void UChaosDebugDrawSubsystem::StaticOnWorldTickEnd(UWorld* World, ELevelTick TickType, float Dt)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UChaosDebugDrawSubsystem::StaticOnWorldTickEnd)

	if (UChaosDebugDrawSubsystem* CDDSystem = World->GetSubsystem<UChaosDebugDrawSubsystem>())
	{
		CDDSystem->OnWorldTickEnd(TickType, Dt);
	}
}

#endif // CHAOS_DEBUG_DRAW
