// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosEventListenerComponent.h"
#include "PhysicsSolver.h"
#include "Chaos/ChaosSolverActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosEventListenerComponent)

UChaosEventListenerComponent::UChaosEventListenerComponent()
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.SetTickFunctionEnable(true);
	PrimaryComponentTick.TickGroup = TG_PostPhysics;
}

void UChaosEventListenerComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
}


