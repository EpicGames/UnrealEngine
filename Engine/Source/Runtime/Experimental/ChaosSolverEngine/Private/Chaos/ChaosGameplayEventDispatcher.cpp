// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosGameplayEventDispatcher.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "PhysicsEngine/BodyInstance.h"
#include "Chaos/Framework/PhysicsProxy.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Engine/World.h"
#include "PhysicsEngine/PhysicsCollisionHandler.h"
#include "ChaosStats.h"
#include "Chaos/ChaosNotifyHandlerInterface.h"
#include "EventsData.h"
#include "PhysicsEngine/BodySetup.h"
#include "EventManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosGameplayEventDispatcher)



UChaosGameplayEventDispatcher::UChaosGameplayEventDispatcher()
{
	PrimaryComponentTick.bCanEverTick = false;
	PrimaryComponentTick.bStartWithTickEnabled = false;
	PrimaryComponentTick.bAllowTickOnDedicatedServer = false;
	PrimaryComponentTick.SetTickFunctionEnable(false);
}

void UChaosGameplayEventDispatcher::OnRegister()
{
	Super::OnRegister();
	// make sure we unregister the events before register them again , because the geometry collection component may call RegisterChaosEvents before OnRegister is called on the dispatcher
	UnregisterChaosEvents();
	RegisterChaosEvents();
}


void UChaosGameplayEventDispatcher::OnUnregister()
{
	UnregisterChaosEvents();
	Super::OnUnregister();
}

// internal
static void DispatchPendingBreakEvents(TArray<FChaosBreakEvent> const& Events, TMap<TObjectPtr<UPrimitiveComponent>, FBreakEventCallbackWrapper> const& Registrations)
{
	for (FChaosBreakEvent const& E : Events)
	{
		if (E.Component)
		{
			const FBreakEventCallbackWrapper* const Callback = Registrations.Find(E.Component);
			if (Callback)
			{
				Callback->BreakEventCallback(E);
			}
		}
	}
}

static void DispatchPendingRemovalEvents(TArray<FChaosRemovalEvent> const& Events, TMap<TObjectPtr<UPrimitiveComponent>, FRemovalEventCallbackWrapper> const& Registrations)
{
	for (FChaosRemovalEvent const& E : Events)
	{
		if (E.Component)
		{
			const FRemovalEventCallbackWrapper* const Callback = Registrations.Find(E.Component);
			if (Callback)
			{
				Callback->RemovalEventCallback(E);
			}
		}
	}
}

static void DispatchPendingCrumblingEvents(TArray<FChaosCrumblingEvent> const& Events, TMap<TObjectPtr<UPrimitiveComponent>, FCrumblingEventCallbackWrapper> const& Registrations)
{
	for (FChaosCrumblingEvent const& E : Events)
	{
		if (E.Component)
		{
			if (const FCrumblingEventCallbackWrapper* const Callback = Registrations.Find(E.Component))
			{
				Callback->CrumblingEventCallback(E);
			}
		}
	}
}

static void SetCollisionInfoFromComp(FRigidBodyCollisionInfo& Info, UPrimitiveComponent* Comp)
{
	if (Comp)
	{
		Info.Component = Comp;
		Info.Actor = Comp->GetOwner();

		const FBodyInstance* const BodyInst = Comp->GetBodyInstance();
		Info.BodyIndex = BodyInst ? BodyInst->InstanceBodyIndex : INDEX_NONE;
		Info.BoneName = BodyInst && BodyInst->BodySetup.IsValid() ? BodyInst->BodySetup->BoneName : NAME_None;
	}
	else
	{
		Info.Component = nullptr;
		Info.Actor = nullptr;
		Info.BodyIndex = INDEX_NONE;
		Info.BoneName = NAME_None;
	}
}

FCollisionNotifyInfo& UChaosGameplayEventDispatcher::GetPendingCollisionForContactPair(const void* P0, const void* P1, bool& bNewEntry)
{
	const FUniqueContactPairKey Key = { P0, P1 };
	const int32* PendingNotifyIdx = ContactPairToPendingNotifyMap.Find(Key);
	if (PendingNotifyIdx)
	{
		// we already have one for this pair
		bNewEntry = false;
		return PendingCollisionNotifies[*PendingNotifyIdx];
	}

	// make a new entry
	bNewEntry = true;
	int32 NewIdx = PendingCollisionNotifies.AddZeroed();
	return PendingCollisionNotifies[NewIdx];
}

FChaosPendingCollisionNotify& UChaosGameplayEventDispatcher::GetPendingChaosCollisionForContactPair(const void* P0, const void* P1, bool& bNewEntry)
{
	const FUniqueContactPairKey Key = { P0, P1 };
	const int32* PendingNotifyIdx = ContactPairToPendingChaosNotifyMap.Find(Key);
	if (PendingNotifyIdx)
	{
		// we already have one for this pair
		bNewEntry = false;
		return PendingChaosCollisionNotifies[*PendingNotifyIdx];
	}

	// make a new entry
	bNewEntry = true;
	int32 NewIdx = PendingChaosCollisionNotifies.AddZeroed();
	return PendingChaosCollisionNotifies[NewIdx];
}

void UChaosGameplayEventDispatcher::DispatchPendingCollisionNotifies()
{
	UWorld const* const OwningWorld = GetWorld();

	// Let the game-specific PhysicsCollisionHandler process any physics collisions that took place
	if (OwningWorld != nullptr && OwningWorld->PhysicsCollisionHandler != nullptr)
	{
		OwningWorld->PhysicsCollisionHandler->HandlePhysicsCollisions_AssumesLocked(PendingCollisionNotifies);
	}

	// Fire any collision notifies in the queue.
	for (FCollisionNotifyInfo& NotifyInfo : PendingCollisionNotifies)
	{
//		if (NotifyInfo.RigidCollisionData.ContactInfos.Num() > 0)
		{
			if (NotifyInfo.bCallEvent0 && /*NotifyInfo.IsValidForNotify() && */ NotifyInfo.Info0.Actor.IsValid())
			{
				NotifyInfo.Info0.Actor->DispatchPhysicsCollisionHit(NotifyInfo.Info0, NotifyInfo.Info1, NotifyInfo.RigidCollisionData);
			}

			// CHAOS: don't call event 1, because the code below will generate the reflexive hit data as separate entries
		}
	}
	for (FChaosPendingCollisionNotify& NotifyInfo : PendingChaosCollisionNotifies)
	{
		for (UObject* Obj : NotifyInfo.NotifyRecipients)
		{
			IChaosNotifyHandlerInterface* const Handler = Cast< IChaosNotifyHandlerInterface>(Obj);
			ensure(Handler);
			if (Handler)
			{
				Handler->HandlePhysicsCollision(NotifyInfo.CollisionInfo);
			}
		}
	}

	PendingCollisionNotifies.Reset();
	PendingChaosCollisionNotifies.Reset();
}

void UChaosGameplayEventDispatcher::RegisterForCollisionEvents(UPrimitiveComponent* ComponentToListenTo, UObject* ObjectToNotify)
{
	FChaosHandlerSet& HandlerSet = CollisionEventRegistrations.FindOrAdd(ComponentToListenTo);

	if (IChaosNotifyHandlerInterface* ChaosHandler = Cast<IChaosNotifyHandlerInterface>(ObjectToNotify))
	{
		HandlerSet.ChaosHandlers.Add(ObjectToNotify);
	}
	
	// a component can also implement the handler interface to get both types of events, so these aren't mutually exclusive
	if (ObjectToNotify == ComponentToListenTo)
	{
		HandlerSet.bLegacyComponentNotify = true;
	}

	// note: theoretically supportable to have external listeners to the legacy-style notifies, but will take more plumbing
}

void UChaosGameplayEventDispatcher::UnRegisterForCollisionEvents(UPrimitiveComponent* ComponentToListenTo, UObject* ObjectToNotify)
{
	FChaosHandlerSet* HandlerSet = CollisionEventRegistrations.Find(ComponentToListenTo);
	if (HandlerSet)
	{
		HandlerSet->ChaosHandlers.Remove(ObjectToNotify);

		if (ObjectToNotify == ComponentToListenTo)
		{
			HandlerSet->bLegacyComponentNotify = false;
		}

		if ((HandlerSet->ChaosHandlers.Num() == 0) && (HandlerSet->bLegacyComponentNotify == false))
		{
			// no one listening to this component any more, remove it entirely
			CollisionEventRegistrations.Remove(ComponentToListenTo);
		}
	}
}

void UChaosGameplayEventDispatcher::RegisterForBreakEvents(UPrimitiveComponent* Component, FOnBreakEventCallback InFunc)
{
	if (Component)
	{
		FBreakEventCallbackWrapper F = { InFunc };
		BreakEventRegistrations.Add(Component, F);
	}
}

void UChaosGameplayEventDispatcher::UnRegisterForBreakEvents(UPrimitiveComponent* Component)
{
	if (Component)
	{
		BreakEventRegistrations.Remove(Component);
	}
}

void UChaosGameplayEventDispatcher::RegisterForRemovalEvents(UPrimitiveComponent* Component, FOnRemovalEventCallback InFunc)
{
	if (Component)
	{
		FRemovalEventCallbackWrapper F = { InFunc };
		RemovalEventRegistrations.Add(Component, F);
	}
}

void UChaosGameplayEventDispatcher::UnRegisterForRemovalEvents(UPrimitiveComponent* Component)
{
	if (Component)
	{
		RemovalEventRegistrations.Remove(Component);
	}
}

void UChaosGameplayEventDispatcher::RegisterForCrumblingEvents(UPrimitiveComponent* Component, FOnCrumblingEventCallback InFunc)
{
	if (Component)
	{
		FCrumblingEventCallbackWrapper F = { InFunc };
		CrumblingEventRegistrations.Add(Component, F);
	}
}

void UChaosGameplayEventDispatcher::UnRegisterForCrumblingEvents(UPrimitiveComponent* Component)
{
	if (Component)
	{
		CrumblingEventRegistrations.Remove(Component);
	}
}

void UChaosGameplayEventDispatcher::RegisterChaosEvents()
{
	if (FPhysScene* Scene = GetWorld()->GetPhysicsScene())
	{
		if (Chaos::FPhysicsSolver* Solver = Scene->GetSolver())
		{
			Chaos::FEventManager* EventManager = Solver->GetEventManager();
			EventManager->RegisterHandler<Chaos::FCollisionEventData>(Chaos::EEventType::Collision, this, &UChaosGameplayEventDispatcher::HandleCollisionEvents, &UChaosGameplayEventDispatcher::GetInterestedProxyOwnersForCollisionEvents);
			EventManager->RegisterHandler<Chaos::FBreakingEventData>(Chaos::EEventType::Breaking, this, &UChaosGameplayEventDispatcher::HandleBreakingEvents, &UChaosGameplayEventDispatcher::GetInterestedProxyOwnersForBreakingEvents);
			EventManager->RegisterHandler<Chaos::FRemovalEventData>(Chaos::EEventType::Removal, this, &UChaosGameplayEventDispatcher::HandleRemovalEvents, &UChaosGameplayEventDispatcher::GetInterestedProxyOwnersForRemovalEvents);
			EventManager->RegisterHandler<Chaos::FCrumblingEventData>(Chaos::EEventType::Crumbling, this, &UChaosGameplayEventDispatcher::HandleCrumblingEvents, &UChaosGameplayEventDispatcher::GetInterestedProxyOwnersForCrumblingEvents);
		}
	}
}

void UChaosGameplayEventDispatcher::UnregisterChaosEvents()
{
	if (GetWorld())
	{
		if (FPhysScene* Scene = GetWorld()->GetPhysicsScene())
		{
			if (Chaos::FPhysicsSolver* Solver = Scene->GetSolver())
			{
				Chaos::FEventManager* EventManager = Solver->GetEventManager();
				EventManager->UnregisterHandler(Chaos::EEventType::Collision, this);
				EventManager->UnregisterHandler(Chaos::EEventType::Breaking, this);
				EventManager->UnregisterHandler(Chaos::EEventType::Sleeping, this);
				EventManager->UnregisterHandler(Chaos::EEventType::Removal, this);
				EventManager->UnregisterHandler(Chaos::EEventType::Crumbling, this);
			}
		}
	}
}

template <typename EventIterator>
void UChaosGameplayEventDispatcher::FillPhysicsProxy(FPhysScene_Chaos& Scene, TArray<UObject*>& Result, EventIterator& It)
{
	UPrimitiveComponent* const Comp0 = Cast<UPrimitiveComponent>(It.Key());
	const TArray<IPhysicsProxyBase*>* PhysicsProxyArray = Scene.GetOwnedPhysicsProxies(Comp0);

	if (PhysicsProxyArray)
	{
		for (IPhysicsProxyBase* PhysicsProxy0 : *PhysicsProxyArray)
		{
			Result.AddUnique(PhysicsProxy0->GetOwner());
		}
	}
}


TArray<UObject*> UChaosGameplayEventDispatcher::GetInterestedProxyOwnersForCollisionEvents()
{
	if ((GetWorld() == nullptr) || (GetWorld()->GetPhysicsScene() == nullptr))
	{
		return {};
	}

	TArray<UObject*> Result;
	FPhysScene_Chaos& Scene = *(GetWorld()->GetPhysicsScene());
		
	// look through all the components that someone is interested in and get all the proxies
	for (decltype(CollisionEventRegistrations)::TIterator It(CollisionEventRegistrations); It; ++It)
	{
		FillPhysicsProxy(Scene, Result, It);
	}

	return Result;
}

TArray<UObject*> UChaosGameplayEventDispatcher::GetInterestedProxyOwnersForBreakingEvents()
{
	if ((GetWorld() == nullptr) || (GetWorld()->GetPhysicsScene() == nullptr))
	{
		return {};
	}

	TArray<UObject*> Result;

	FPhysScene_Chaos& Scene = *(GetWorld()->GetPhysicsScene());

	// look through all the components that someone is interested in and get all the proxies
	for (decltype(BreakEventRegistrations)::TIterator It(BreakEventRegistrations); It; ++It)
	{
		FillPhysicsProxy(Scene, Result, It);
	}

	return Result;
}

TArray<UObject*> UChaosGameplayEventDispatcher::GetInterestedProxyOwnersForRemovalEvents()
{
	if ((GetWorld() == nullptr) || (GetWorld()->GetPhysicsScene() == nullptr))
	{
		return {};
	}

	TArray<UObject*> Result;
	FPhysScene_Chaos& Scene = *(GetWorld()->GetPhysicsScene());

	// look through all the components that someone is interested in and get all the proxies
	for (decltype(RemovalEventRegistrations)::TIterator It(RemovalEventRegistrations); It; ++It)
	{
		FillPhysicsProxy(Scene, Result, It);
	}

	return Result;
}

TArray<UObject*> UChaosGameplayEventDispatcher::GetInterestedProxyOwnersForCrumblingEvents()
{
	if ((GetWorld() == nullptr) || (GetWorld()->GetPhysicsScene() == nullptr))
	{
		return {};
	}

	TArray<UObject*> Result;
	FPhysScene_Chaos& Scene = *(GetWorld()->GetPhysicsScene());

	// look through all the components that someone is interested in and get all the proxies
	for (decltype(CrumblingEventRegistrations)::TIterator It(CrumblingEventRegistrations); It; ++It)
	{
		FillPhysicsProxy(Scene, Result, It);
	}

	return Result;
}

void UChaosGameplayEventDispatcher::HandleCollisionEvents(const Chaos::FCollisionEventData& Event)
{
	// todo(chaos) : this code is very similar to FPhysScene_Chaos::HandleCollisionEvents, we should propably consolidate if possible or share as much code as possible 
	SCOPE_CYCLE_COUNTER(STAT_DispatchCollisionEvents);

	if ((GetWorld() == nullptr) || (GetWorld()->GetPhysicsScene() == nullptr))
	{
		return;
	}

	FPhysScene_Chaos& Scene = *(GetWorld()->GetPhysicsScene());

	PendingChaosCollisionNotifies.Reset();
	ContactPairToPendingNotifyMap.Reset();

	{
		TMap<IPhysicsProxyBase*, TArray<int32>> const& PhysicsProxyToCollisionIndicesMap = Event.PhysicsProxyToCollisionIndices.PhysicsProxyToIndicesMap;
		Chaos::FCollisionDataArray const& CollisionData = Event.CollisionData.AllCollisionsArray;

		int32 NumCollisions = CollisionData.Num();
		if (NumCollisions > 0)
		{
			// look through all the components that someone is interested in, and see if they had a collision
			// note that we only need to care about the interaction from the POV of the registered component,
			// since if anyone wants notifications for the other component it hit, it's also registered and we'll get to that elsewhere in the list
			for (decltype(CollisionEventRegistrations)::TIterator It(CollisionEventRegistrations); It; ++It)
			{
				const FChaosHandlerSet& HandlerSet = It.Value();

				UPrimitiveComponent* Comp0 = Cast<UPrimitiveComponent>(It.Key());
				const TArray<IPhysicsProxyBase*>* PhysicsProxyArray = Scene.GetOwnedPhysicsProxies(Comp0);

				if (PhysicsProxyArray)
				{
					for (IPhysicsProxyBase* PhysicsProxy0 : *PhysicsProxyArray)
					{
						TArray<int32> const* const CollisionIndices = PhysicsProxyToCollisionIndicesMap.Find(PhysicsProxy0);
						if (CollisionIndices)
						{
							const int32 NumCollisionIndices = CollisionIndices->Num();

							if(NumCollisionIndices == 0)
							{
								continue;
							}

							for(int32 Index = 0; Index < NumCollisionIndices; ++Index)
							{
								int32 EncodedCollisionIdx = (*CollisionIndices)[Index];

								bool bSwapOrder;
								int32 CollisionIdx = Chaos::FEventManager::DecodeCollisionIndex(EncodedCollisionIdx, bSwapOrder);

								Chaos::FCollidingData const& CollisionDataItem = CollisionData[CollisionIdx];

								IPhysicsProxyBase* PhysicsProxy1 = CollisionDataItem.Proxy2 ? CollisionDataItem.Proxy2: PhysicsProxy0;

								if (bSwapOrder)
								{
									PhysicsProxy1 = CollisionDataItem.Proxy1;
									PhysicsProxy0 = CollisionDataItem.Proxy2;
								}

								// Are the proxies pending destruction? If they are no longer tracked by the PhysScene, the proxy is deleted or pending deletion.
								if (Scene.GetOwningComponent<UPrimitiveComponent>(PhysicsProxy0) == nullptr || Scene.GetOwningComponent<UPrimitiveComponent>(PhysicsProxy1) == nullptr)
								{
									continue;
								}

								{
									bool bNewEntry = false;
									FCollisionNotifyInfo& NotifyInfo = GetPendingCollisionForContactPair(PhysicsProxy0, PhysicsProxy1, bNewEntry);

									// #note: we only notify on the first contact, though we will still accumulate the impulse data from subsequent contacts
									const FVector NormalImpulse = FVector::DotProduct(CollisionDataItem.AccumulatedImpulse, CollisionDataItem.Normal) * CollisionDataItem.Normal;	// project impulse along normal
									const FVector FrictionImpulse = FVector(CollisionDataItem.AccumulatedImpulse) - NormalImpulse; // friction is component not along contact normal
									NotifyInfo.RigidCollisionData.TotalNormalImpulse += NormalImpulse;
									NotifyInfo.RigidCollisionData.TotalFrictionImpulse += FrictionImpulse;

									if (bNewEntry)
									{
										UPrimitiveComponent* const Comp1 = Scene.GetOwningComponent<UPrimitiveComponent>(PhysicsProxy1);
										if (bSwapOrder)
										{
											Comp0 = Scene.GetOwningComponent<UPrimitiveComponent>(PhysicsProxy0);
										}

										// fill in legacy contact data
										NotifyInfo.bCallEvent0 = true;
										// if Comp1 wants this event too, it will get its own pending collision entry, so we leave it false

										// @todo(chaos) this may not handle welded objects properly as the component returned may be thewrong one ( see FPhysScene_Chaos::HandleCollisionEvents ) 
										SetCollisionInfoFromComp(NotifyInfo.Info0, Comp0);
										SetCollisionInfoFromComp(NotifyInfo.Info1, Comp1);

										FRigidBodyContactInfo& NewContact = NotifyInfo.RigidCollisionData.ContactInfos.AddZeroed_GetRef();
										NewContact.ContactNormal = CollisionDataItem.Normal;
										NewContact.ContactPosition = CollisionDataItem.Location;
										NewContact.ContactPenetration = CollisionDataItem.PenetrationDepth;
										// NewContact.PhysMaterial[1] UPhysicalMaterial required here

										if (bSwapOrder)
										{
											NotifyInfo.RigidCollisionData.SwapContactOrders();
										}
									}
								}

								if (HandlerSet.ChaosHandlers.Num() > 0)
								{
									bool bNewEntry = false;
									FChaosPendingCollisionNotify& ChaosNotifyInfo = GetPendingChaosCollisionForContactPair(PhysicsProxy0, PhysicsProxy1, bNewEntry);

									// #note: we only notify on the first contact, though we will still accumulate the impulse data from subsequent contacts
									ChaosNotifyInfo.CollisionInfo.AccumulatedImpulse += CollisionDataItem.AccumulatedImpulse;

									if (bNewEntry)
									{
										UPrimitiveComponent* const Comp1 = Scene.GetOwningComponent<UPrimitiveComponent>(PhysicsProxy1);
										if (bSwapOrder)
										{
											Comp0 = Scene.GetOwningComponent<UPrimitiveComponent>(PhysicsProxy0);
										}

										// fill in Chaos contact data
										ChaosNotifyInfo.CollisionInfo.Component = Comp0;
										ChaosNotifyInfo.CollisionInfo.OtherComponent = Comp1;
										ChaosNotifyInfo.CollisionInfo.Location = CollisionDataItem.Location;
										ChaosNotifyInfo.NotifyRecipients = HandlerSet.ChaosHandlers;

										if (bSwapOrder)
										{
											ChaosNotifyInfo.CollisionInfo.AccumulatedImpulse = -CollisionDataItem.AccumulatedImpulse;
											ChaosNotifyInfo.CollisionInfo.Normal = -CollisionDataItem.Normal;

											ChaosNotifyInfo.CollisionInfo.Velocity = CollisionDataItem.Velocity2;
											ChaosNotifyInfo.CollisionInfo.OtherVelocity = CollisionDataItem.Velocity1;
											ChaosNotifyInfo.CollisionInfo.AngularVelocity = CollisionDataItem.AngularVelocity2;
											ChaosNotifyInfo.CollisionInfo.OtherAngularVelocity = CollisionDataItem.AngularVelocity1;
											ChaosNotifyInfo.CollisionInfo.Mass = CollisionDataItem.Mass2;
											ChaosNotifyInfo.CollisionInfo.OtherMass = CollisionDataItem.Mass1;
										}
										else
										{
											ChaosNotifyInfo.CollisionInfo.AccumulatedImpulse = CollisionDataItem.AccumulatedImpulse;
											ChaosNotifyInfo.CollisionInfo.Normal = CollisionDataItem.Normal;

											ChaosNotifyInfo.CollisionInfo.Velocity = CollisionDataItem.Velocity1;
											ChaosNotifyInfo.CollisionInfo.OtherVelocity = CollisionDataItem.Velocity2;
											ChaosNotifyInfo.CollisionInfo.AngularVelocity = CollisionDataItem.AngularVelocity1;
											ChaosNotifyInfo.CollisionInfo.OtherAngularVelocity = CollisionDataItem.AngularVelocity2;
											ChaosNotifyInfo.CollisionInfo.Mass = CollisionDataItem.Mass1;
											ChaosNotifyInfo.CollisionInfo.OtherMass = CollisionDataItem.Mass2;
										}
									}
								}
							}
						}
					}
				}
			}
		}
	}

	// Tell the world and actors about the collisions
	DispatchPendingCollisionNotifies();
}

void UChaosGameplayEventDispatcher::HandleBreakingEvents(const Chaos::FBreakingEventData& Event)
{
	SCOPE_CYCLE_COUNTER(STAT_DispatchBreakEvents);

	// BREAK EVENTS

	TArray<FChaosBreakEvent> PendingBreakEvents;

	const float BreakingDataTimestamp = Event.BreakingData.TimeCreated;
	if (BreakingDataTimestamp > LastBreakingDataTime)
	{
		LastBreakingDataTime = BreakingDataTimestamp;

		const Chaos::FBreakingDataArray& BreakingDataArray = Event.BreakingData.AllBreakingsArray;

		// let's assume breaks are very rare, so we will iterate breaks instead of registered components for now
		const int32 NumBreaks = BreakingDataArray.Num();
		if (NumBreaks > 0)
		{
			const FPhysScene& Scene = *(GetWorld()->GetPhysicsScene());

			for (const Chaos::FBreakingData& BreakingData : BreakingDataArray)
			{	
				if ((BreakingData.EmitterFlag & Chaos::EventEmitterFlag::OwnDispatcher) && BreakingData.Proxy)
				{
					UPrimitiveComponent* const PrimComp = Scene.GetOwningComponent<UPrimitiveComponent>(BreakingData.Proxy);
					if (PrimComp)
					{
						// queue them up so we can release the physics data before trigging BP events
						FChaosBreakEvent& BreakEvent = PendingBreakEvents.Emplace_GetRef(BreakingData);
						BreakEvent.Component = PrimComp;
					}
				}
			}

			DispatchPendingBreakEvents(PendingBreakEvents, BreakEventRegistrations);
		}
	}

}

void UChaosGameplayEventDispatcher::HandleSleepingEvents(const Chaos::FSleepingEventData& SleepingData)
{
	const FPhysScene& Scene = *(GetWorld()->GetPhysicsScene());

	const Chaos::FSleepingDataArray& SleepingArray = SleepingData.SleepingData;

	for (const Chaos::FSleepingData& SleepData : SleepingArray)
	{
		ESleepEvent WakeSleepEvent = SleepData.Sleeping ? ESleepEvent::SET_Sleep : ESleepEvent::SET_Wakeup;
		if (UPrimitiveComponent* PrimitiveComponent = Scene.GetOwningComponent<UPrimitiveComponent>(SleepData.Proxy))
		{
			FName BoneName = NAME_None;
			if (FBodyInstance* BodyInstance = Scene.GetBodyInstanceFromProxy(SleepData.Proxy))
			{
				BoneName = BodyInstance->BodySetup->BoneName;
			}

			if (PrimitiveComponent->ShouldDispatchWakeEvents(BoneName))
			{
				PrimitiveComponent->DispatchWakeEvents(WakeSleepEvent, BoneName);
			}
		}
	}
}

void UChaosGameplayEventDispatcher::HandleRemovalEvents(const Chaos::FRemovalEventData& Event)
{
	// REMOVAL EVENTS

	TArray<FChaosRemovalEvent> PendingRemovalEvents;

	const float RemovalDataTimestamp = Event.RemovalData.TimeCreated;
	if (RemovalDataTimestamp > LastRemovalDataTime)
	{
		LastRemovalDataTime = RemovalDataTimestamp;

		Chaos::FRemovalDataArray const& RemovalData = Event.RemovalData.AllRemovalArray;

		const int32 NumRemovals = RemovalData.Num();
		if (NumRemovals > 0)
		{
			const FPhysScene& Scene = *(GetWorld()->GetPhysicsScene());

			for (Chaos::FRemovalData const& RemovalDataItem : RemovalData)
			{
				if (RemovalDataItem.Proxy)
				{
					UPrimitiveComponent* const PrimComp = Scene.GetOwningComponent<UPrimitiveComponent>(RemovalDataItem.Proxy);
					if (PrimComp && RemovalEventRegistrations.Contains(PrimComp))
					{
						// queue them up so we can release the physics data before trigging BP events
						FChaosRemovalEvent& RemovalEvent = PendingRemovalEvents.AddZeroed_GetRef();
						RemovalEvent.Component = PrimComp;
						RemovalEvent.Location = RemovalDataItem.Location;
						RemovalEvent.Mass = RemovalDataItem.Mass;
					}
				}
			}

			DispatchPendingRemovalEvents(PendingRemovalEvents, RemovalEventRegistrations);
		}
	}

}

void UChaosGameplayEventDispatcher::HandleCrumblingEvents(const Chaos::FCrumblingEventData& Event)
{
	SCOPE_CYCLE_COUNTER(STAT_DispatchCrumblingEvents);

	// CRUMBLING EVENTS

	const float CrumblingDataTimestamp = Event.CrumblingData.TimeCreated;
	if (CrumblingDataTimestamp > LastCrumblingDataTime)
	{
		LastCrumblingDataTime = CrumblingDataTimestamp;

		Chaos::FCrumblingDataArray const& BreakingData = Event.CrumblingData.AllCrumblingsArray;

		const FPhysScene& Scene = *(GetWorld()->GetPhysicsScene());

		// let's assume crumbles are rare, so we will iterate breaks instead of registered components for now
		TArray<FChaosCrumblingEvent> PendingCrumblingEvent;
		for (const Chaos::FCrumblingData& CrumblingDataItem : Event.CrumblingData.AllCrumblingsArray)
		{	
			if ((CrumblingDataItem.EmitterFlag & Chaos::EventEmitterFlag::OwnDispatcher) && CrumblingDataItem.Proxy)
			{
				if (UPrimitiveComponent* const PrimComp = Scene.GetOwningComponent<UPrimitiveComponent>(CrumblingDataItem.Proxy))
				{
					// queue them up so we can release the physics data before triggering BP events
					FChaosCrumblingEvent& CrumblingEvent = PendingCrumblingEvent.AddZeroed_GetRef();
					CrumblingEvent.Component = PrimComp;
					CrumblingEvent.Location = CrumblingDataItem.Location;
					CrumblingEvent.Orientation = CrumblingDataItem.Orientation;
					CrumblingEvent.LinearVelocity = CrumblingDataItem.LinearVelocity;
					CrumblingEvent.AngularVelocity = CrumblingDataItem.AngularVelocity;
					CrumblingEvent.Mass = static_cast<float>(CrumblingDataItem.Mass);
					CrumblingEvent.LocalBounds = FBox(CrumblingDataItem.LocalBounds.Min(), CrumblingDataItem.LocalBounds.Max());
					CrumblingEvent.Children = CrumblingDataItem.Children;
				}
			}
		}
		if (PendingCrumblingEvent.Num() > 0)
		{
			DispatchPendingCrumblingEvents(PendingCrumblingEvent, CrumblingEventRegistrations);
		}
	}

}

