// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosNotifyHandlerInterface.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosNotifyHandlerInterface)

UChaosNotifyHandlerInterface::UChaosNotifyHandlerInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void IChaosNotifyHandlerInterface::HandlePhysicsCollision(const FChaosPhysicsCollisionInfo& CollisionInfo)
{
	// native
	NotifyPhysicsCollision(CollisionInfo);

	// bp
	DispatchChaosPhysicsCollisionBlueprintEvents(CollisionInfo);
}

FHitResult UChaosSolverEngineBlueprintLibrary::ConvertPhysicsCollisionToHitResult(const FChaosPhysicsCollisionInfo& PhysicsCollision)
{
	FHitResult Hit(0.f);
	
	Hit.Component = PhysicsCollision.OtherComponent;
	Hit.HitObjectHandle = FActorInstanceHandle(Hit.Component.IsValid() ? Hit.Component->GetOwner() : nullptr);
	Hit.bBlockingHit = true;
	Hit.Normal = PhysicsCollision.Normal;
	Hit.ImpactNormal = PhysicsCollision.Normal;
	Hit.Location = PhysicsCollision.Location;
	Hit.ImpactPoint = PhysicsCollision.Location;
	//Hit.PhysMaterial = 

	return Hit;
}

FChaosPhysicsCollisionInfo::FChaosPhysicsCollisionInfo()
	: Component(nullptr)
	, OtherComponent(nullptr)
	, Location(FVector::ZeroVector)
	, Normal(FVector::ZeroVector)
	, AccumulatedImpulse(FVector::ZeroVector)
	, Velocity(FVector::ZeroVector)
	, OtherVelocity(FVector::ZeroVector)
	, AngularVelocity(FVector::ZeroVector)
	, OtherAngularVelocity(FVector::ZeroVector)
	, Mass(0.0f)
	, OtherMass(0.0f)
{

}

