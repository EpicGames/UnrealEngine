// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosSolverActor.h"
#include "UObject/ConstructorHelpers.h"
#include "PhysicsSolver.h"
#include "ChaosModule.h"

#include "Components/BillboardComponent.h"
#include "EngineUtils.h"
#include "ChaosSolversModule.h"
#include "Chaos/ChaosGameplayEventDispatcher.h"
#include "Chaos/Framework/DebugSubstep.h"
#include "Engine/Texture2D.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Dataflow/DataflowSimulationManager.h"
#include "PhysicsEngine/PhysicsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosSolverActor)

//DEFINE_LOG_CATEGORY_STATIC(AFA_Log, NoLogging, All);

#ifndef CHAOS_WITH_PAUSABLE_SOLVER
#define CHAOS_WITH_PAUSABLE_SOLVER 1
#endif

#if CHAOS_DEBUG_SUBSTEP
#include "HAL/IConsoleManager.h"
#include "Chaos/Utilities.h"

class FChaosSolverActorConsoleObjects final
{
public:
	FChaosSolverActorConsoleObjects()
		: ConsoleCommands()
	{
		// Register console command
		
		ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("p.Chaos.Solver.Pause"),
			TEXT("Debug pause the specified solver."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FChaosSolverActorConsoleObjects::Pause),
			ECVF_Cheat));

		ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("p.Chaos.Solver.Step"),
			TEXT("Debug step the specified solver."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FChaosSolverActorConsoleObjects::Step),
			ECVF_Cheat));

		ConsoleCommands.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("p.Chaos.Solver.Substep"),
			TEXT("Debug substep the specified solver."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FChaosSolverActorConsoleObjects::Substep),
			ECVF_Cheat));
	}

	~FChaosSolverActorConsoleObjects()
	{
		for (IConsoleObject* ConsoleCommand: ConsoleCommands)
		{
			IConsoleManager::Get().UnregisterConsoleObject(ConsoleCommand);
		}
	}

	void AddSolver(const FString& Name, AChaosSolverActor* SolverActor) { SolverActors.Add(Name, SolverActor); }
	void RemoveSolver(const FString& Name) { SolverActors.Remove(Name); }

private:

	void Pause(const TArray<FString>& Args)
	{
		AChaosSolverActor* const* SolverActor;
		Chaos::FPhysicsSolver* Solver;
		switch (Args.Num())
		{
		default:
			break;  // Invalid arguments
		case 1:
			if ((SolverActor = SolverActors.Find(Args[0])) != nullptr &&
				(Solver = (*SolverActor)->GetSolver()) != nullptr)
			{
				UE_LOGF(LogChaosDebug, Display, "%d", (*SolverActor)->ChaosDebugSubstepControl.bPause);
				return;
			}
			break;  // Invalid arguments
		case 2:
			if ((SolverActor = SolverActors.Find(Args[0])) != nullptr &&
				(Solver = (*SolverActor)->GetSolver()) != nullptr)
			{
#if TODO_REIMPLEMENT_DEBUG_SUBSTEP
				if (Args[1] == TEXT("0"))
				{
					Solver->GetDebugSubstep().Enable(false);
					(*SolverActor)->ChaosDebugSubstepControl.bPause = false;
#if WITH_EDITOR
					(*SolverActor)->ChaosDebugSubstepControl.OnPauseChanged.ExecuteIfBound();
#endif
					return;
				}
				else if (Args[1] == TEXT("1"))
				{
					Solver->GetDebugSubstep().Enable(true);
					(*SolverActor)->ChaosDebugSubstepControl.bPause = true;
#if WITH_EDITOR
					(*SolverActor)->ChaosDebugSubstepControl.OnPauseChanged.ExecuteIfBound();
#endif
					return;
				}
#endif
			}
			break;  // Invalid arguments
		}
		UE_LOGF(LogChaosDebug, Display, "Invalid arguments.");
		UE_LOGF(LogChaosDebug, Display, "Usage:");
		UE_LOGF(LogChaosDebug, Display, "  p.Chaos.Solver.Pause [SolverName] [0|1|]");
		UE_LOGF(LogChaosDebug, Display, "  SolverName  The Id name of the solver as shown by p.Chaos.Solver.List");
		UE_LOGF(LogChaosDebug, Display, "  0|1|        Use either 0 to unpause, 1 to pause, or nothing to query");
		UE_LOGF(LogChaosDebug, Display, "Example: p.Chaos.Solver.Pause ChaosSolverActor_3 1");
	}

	void Step(const TArray<FString>& Args)
	{
#if TODO_REIMPLEMENT_DEBUG_SUBSTEP
		AChaosSolverActor* const* SolverActor;
		Chaos::FPhysicsSolver* Solver;
		switch (Args.Num())
		{
		default:
			break;  // Invalid arguments
		case 1:
			if ((SolverActor = SolverActors.Find(Args[0])) != nullptr &&
				(Solver = (*SolverActor)->GetSolver()) != nullptr)
			{
				Solver->GetDebugSubstep().ProgressToStep();
				return;
			}
			break;  // Invalid arguments
		}
		UE_LOGF(LogChaosDebug, Display, "Invalid arguments.");
		UE_LOGF(LogChaosDebug, Display, "Usage:");
		UE_LOGF(LogChaosDebug, Display, "  p.Chaos.Solver.Step [SolverName]");
		UE_LOGF(LogChaosDebug, Display, "  SolverName  The Id name of the solver as shown by p.Chaos.Solver.List");
		UE_LOGF(LogChaosDebug, Display, "Example: p.Chaos.Solver.Step ChaosSolverActor_3");
#endif
	}

	void Substep(const TArray<FString>& Args)
	{
#if TODO_REIMPLEMENT_DEBUG_SUBSTEP
		AChaosSolverActor* const* SolverActor;
		Chaos::FPhysicsSolver* Solver;
		switch (Args.Num())
		{
		default:
			break;  // Invalid arguments
		case 1:
			if ((SolverActor = SolverActors.Find(Args[0])) != nullptr &&
				(Solver = (*SolverActor)->GetSolver()) != nullptr)
			{
				Solver->GetDebugSubstep().ProgressToSubstep();
				return;
			}
			break;  // Invalid arguments
		}
		UE_LOGF(LogChaosDebug, Display, "Invalid arguments.");
		UE_LOGF(LogChaosDebug, Display, "Usage:");
		UE_LOGF(LogChaosDebug, Display, "  p.Chaos.Solver.Substep [SolverName]");
		UE_LOGF(LogChaosDebug, Display, "  SolverName  The Id name of the solver as shown by p.Chaos.Solver.List");
		UE_LOGF(LogChaosDebug, Display, "Example: p.Chaos.Solver.Substep ChaosSolverActor_3");
#endif
	}

private:
	TArray<IConsoleObject*> ConsoleCommands;
	TMap<FString, AChaosSolverActor*> SolverActors;
};
static TUniquePtr<FChaosSolverActorConsoleObjects> ChaosSolverActorConsoleObjects;
#endif  // #if CHAOS_DEBUG_SUBSTEP

void FDataflowRigidSolverProxy::AdvanceSolverDatas(const float DeltaTime)
{
	if(Solver != nullptr)
	{
		for(auto* PushData : PushDatas)
		{
			Chaos::FSolverTasksPTOnly  SolverTask(*Solver, PushData);
			SolverTask.AdvanceSolver();
		}
	}
}

AChaosSolverActor::AChaosSolverActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TimeStepMultiplier_DEPRECATED(1.f)
	, CollisionIterations_DEPRECATED(1)
	, PushOutIterations_DEPRECATED(3)
	, PushOutPairIterations_DEPRECATED(2)
	, ClusterConnectionFactor_DEPRECATED(1.0)
	, ClusterUnionConnectionType_DEPRECATED(EClusterConnectionTypeEnum::Chaos_DelaunayTriangulation)
	, DoGenerateCollisionData_DEPRECATED(true)
	, DoGenerateBreakingData_DEPRECATED(true)
	, DoGenerateTrailingData_DEPRECATED(true)
	, MassScale_DEPRECATED(1.f)
	, bHasFloor(true)
	, FloorHeight(0.f)
	, ChaosDebugSubstepControl()
	, PhysScene(nullptr)
	, RigidSolverProxy()
	, Proxy(nullptr)
{
	if(!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Ticking setup for collision/breaking notifies
		PrimaryActorTick.TickGroup = TG_PostPhysics;
		PrimaryActorTick.bCanEverTick = true;
		PrimaryActorTick.bStartWithTickEnabled = true;
	}

	/*
	* Display icon in the editor
	*/
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		// A helper class object we use to find target UTexture2D object in resource package
		ConstructorHelpers::FObjectFinderOptional<UTexture2D> SolverTextureObject;

		// Icon sprite category name
		FName ID_Solver;

		// Icon sprite display name
		FText NAME_Solver;

		FConstructorStatics()
			// Use helper class object to find the texture
			// "/Engine/EditorResources/S_Solver" is resource path
			: SolverTextureObject(TEXT("/Engine/EditorResources/S_Solver.S_Solver"))
			, ID_Solver(TEXT("Solver"))
			, NAME_Solver(NSLOCTEXT("SpriteCategory", "Solver", "Solver"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	// We need a scene component to attach Icon sprite
	USceneComponent* SceneComponent = ObjectInitializer.CreateDefaultSubobject<USceneComponent>(this, TEXT("SceneComp"));
	RootComponent = SceneComponent;
	RootComponent->Mobility = EComponentMobility::Static;

#if WITH_EDITORONLY_DATA
	SpriteComponent = ObjectInitializer.CreateEditorOnlyDefaultSubobject<UBillboardComponent>(this, TEXT("Sprite"));
	if (SpriteComponent)
	{
		SpriteComponent->Sprite = ConstructorStatics.SolverTextureObject.Get();		// Get the sprite texture from helper class object
		SpriteComponent->SpriteInfo.Category = ConstructorStatics.ID_Solver;		// Assign sprite category name
		SpriteComponent->SpriteInfo.DisplayName = ConstructorStatics.NAME_Solver;	// Assign sprite display name
		SpriteComponent->AttachToComponent(RootComponent, FAttachmentTransformRules::KeepRelativeTransform);
		SpriteComponent->Mobility = EComponentMobility::Static;
	}
#endif // WITH_EDITORONLY_DATA

	GameplayEventDispatcherComponent = ObjectInitializer.CreateDefaultSubobject<UChaosGameplayEventDispatcher>(this, TEXT("GameplayEventDispatcher"));
}

void AChaosSolverActor::PreInitializeComponents()
{
	Super::PreInitializeComponents();
}

void AChaosSolverActor::ConditionalCreateSolver()
{
	if(PhysScene)
	{
		return;
	}

	if(!HasAnyFlags(RF_DefaultSubObject | RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		PhysScene = MakeShareable(new FPhysScene_Chaos(this
#if CHAOS_DEBUG_NAME
			, TEXT("Solver Actor Physics")
#endif
		));
		RigidSolverProxy.Solver = PhysScene->GetSolver();
	}
}

void AChaosSolverActor::MigrateSolver() const
{
	if(GetSolver())
	{
		if(FChaosSolversModule* Module = FChaosSolversModule::GetModule())
		{
			if(SimulationAsset.DataflowAsset)
			{
				Module->MigrateSolver(GetSolver(), this);
				GetSolver()->SetStandaloneSolver(true);
			}
			else
			{
				Module->MigrateSolver(GetSolver(), GetWorld());
				GetSolver()->SetStandaloneSolver(false);
			}
		}
	}
	// Make sure that the owning world is correct
	PhysScene->SetOwningWorld(GetWorld());
}

void AChaosSolverActor::BeginPlay()
{
	Super::BeginPlay();

	if(!RigidSolverProxy.Solver)
	{
		return;
	}

	MigrateSolver();
	
	RigidSolverProxy.Solver->EnqueueCommandImmediate(
		[InSolver = RigidSolverProxy.Solver, InProps = Properties]()
		{
			InSolver->ApplyConfig(InProps);
		});

	MakeFloor();

#if TODO_REIMPLEMENT_DEBUG_SUBSTEP
#if CHAOS_DEBUG_SUBSTEP
	if (!ChaosSolverActorConsoleObjects)
	{
		ChaosSolverActorConsoleObjects = MakeUnique<FChaosSolverActorConsoleObjects>();
	}
	ChaosSolverActorConsoleObjects->AddSolver(GetName(), this);
#if WITH_EDITOR
	if (ChaosDebugSubstepControl.bPause)
	{
		Solver->GetDebugSubstep().Enable(true);
	}
#endif  // #if WITH_EDITOR
#endif  // #if CHAOS_DEBUG_SUBSTEP
#endif  // #if TODO_REIMPLEMENT_DEBUG_SUBSTEP
}

void AChaosSolverActor::EndPlay(const EEndPlayReason::Type ReasonEnd)
{
	Super::EndPlay(ReasonEnd);

	if(!RigidSolverProxy.Solver)
	{
		return;
	}

	if(Proxy)
	{
		RigidSolverProxy.Solver->UnregisterObject(Proxy);
		Proxy = nullptr;
	}

	RigidSolverProxy.Solver->EnqueueCommandImmediate([InSolver=RigidSolverProxy.Solver]()
		{
			// #TODO BG - We should really reset the solver here but the current reset function
			// is really heavy handed and clears out absolutely everything. Ideally we want to keep
			// all of the solver physics proxies and revert to a state before the very first tick
			//InSolver->SetEnabled(false);
		});
#if TODO_REIMPLEMENT_DEBUG_SUBSTEP
#if CHAOS_DEBUG_SUBSTEP
	ChaosSolverActorConsoleObjects->RemoveSolver(GetName());
#endif  // #if CHAOS_DEBUG_SUBSTEP
#endif  // #if TODO_REIMPLEMENT_DEBUG_SUBSTEP
}

void AChaosSolverActor::PostLoad()
{
	Super::PostLoad();
	
	if(GetLinkerCustomVersion(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::ChaosSolverPropertiesMoved)
	{
		auto ConvertDeprecatedConnectionType = [](EClusterConnectionTypeEnum LegacyType) -> EClusterUnionMethod
		{
			switch(LegacyType)
			{
			case EClusterConnectionTypeEnum::Chaos_PointImplicit:
				return EClusterUnionMethod::PointImplicit;
			case EClusterConnectionTypeEnum::Chaos_DelaunayTriangulation:
				return EClusterUnionMethod::DelaunayTriangulation;
			case EClusterConnectionTypeEnum::Chaos_MinimalSpanningSubsetDelaunayTriangulation:
				return EClusterUnionMethod::MinimalSpanningSubsetDelaunayTriangulation;
			case EClusterConnectionTypeEnum::Chaos_PointImplicitAugmentedWithMinimalDelaunay:
				return EClusterUnionMethod::PointImplicitAugmentedWithMinimalDelaunay;
			case EClusterConnectionTypeEnum::Chaos_BoundsOverlapFilteredDelaunayTriangulation:
				return EClusterUnionMethod::BoundsOverlapFilteredDelaunayTriangulation;
			default:
				break;
			}

			return EClusterUnionMethod::None;
		};

		Properties.PositionIterations = CollisionIterations_DEPRECATED;
		Properties.VelocityIterations = PushOutIterations_DEPRECATED;
		Properties.ClusterConnectionFactor = ClusterConnectionFactor_DEPRECATED;
		Properties.ClusterUnionConnectionType = ConvertDeprecatedConnectionType(ClusterUnionConnectionType_DEPRECATED);
		Properties.bGenerateBreakData = DoGenerateBreakingData_DEPRECATED;
		Properties.bGenerateCollisionData = DoGenerateCollisionData_DEPRECATED;
		Properties.bGenerateTrailingData = DoGenerateTrailingData_DEPRECATED;
		Properties.BreakingFilterSettings = BreakingFilterSettings_DEPRECATED;
		Properties.CollisionFilterSettings = CollisionFilterSettings_DEPRECATED;
		Properties.TrailingFilterSettings = TrailingFilterSettings_DEPRECATED;
	}
}

void AChaosSolverActor::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Attach custom version
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
}

void AChaosSolverActor::PreRegisterAllComponents()
{
	Super::PreRegisterAllComponents();

	ConditionalCreateSolver();
}

void AChaosSolverActor::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	MigrateSolver();

	UWorld* const W = GetWorld(); 
	if (W && !W->PhysicsScene_Chaos)
	{
		SetAsCurrentWorldSolver();
	}

	// Register the dataflow simulation interface
	UE::Dataflow::RegisterSimulationInterface(this);
}

void AChaosSolverActor::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();
	
	// Unregister the dataflow simulation interface
	UE::Dataflow::UnregisterSimulationInterface(this);

	// At this point all components/actors that connect to the solver actor via register/beginplay
	// should now no longer be resident within the physics scene so we can remove our reference
	PhysScene = nullptr;
	RigidSolverProxy.Solver = nullptr;
}

void AChaosSolverActor::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	ConditionalCreateSolver();
	MigrateSolver();
}

void AChaosSolverActor::MakeFloor()
{
	if(bHasFloor)
	{
		TUniquePtr<Chaos::FGeometryParticle> FloorParticle = Chaos::FGeometryParticle::CreateParticle();
		// todo(chaos) : Changing the floor to be a box for now because there's few cases where this may fail to collide with geometry collection
		//				 since the box is finite, let's center it at the actor position for better user control
		//FloorParticle->SetGeometry(TUniquePtr<Chaos::TPlane<Chaos::FReal, 3>>(new Chaos::TPlane<Chaos::FReal, 3>(FVector(0), FVector(0, 0, 1))));
		const FVector SolverLocation = GetActorLocation();
		const FVector BoxMin(-100000, -100000, -1000);
		const FVector BoxMax(100000, 100000, 0);
		FloorParticle->SetGeometry(MakeImplicitObjectPtr<Chaos::TBox<Chaos::FReal, 3>>(BoxMin, BoxMax));
		FloorParticle->SetX(Chaos::FVec3(SolverLocation.X, SolverLocation.Y, FloorHeight));

		const Chaos::Filter::FShapeFilterData ShapeFilterData = Chaos::Filter::FShapeFilterBuilder::BuildBlockAll(Chaos::EFilterFlags::All);
		FloorParticle->SetShapeFilterData(0, ShapeFilterData);
		FloorParticle->SetShapeQueryCollisionEnabled(0, false);
		Proxy = Chaos::FSingleParticlePhysicsProxy::Create(MoveTemp(FloorParticle));
		RigidSolverProxy.Solver->RegisterObject(Proxy);
	}
}

void AChaosSolverActor::SetAsCurrentWorldSolver()
{
	UWorld* const W = GetWorld();
	if (W && (GetSolver() && !GetSolver()->IsStandaloneSolver()))
	{
		W->PhysicsScene_Chaos = PhysScene;
	}
}

void AChaosSolverActor::SetSolverActive(bool bActive)
{
	if(RigidSolverProxy.Solver && PhysScene)
	{
		RigidSolverProxy.Solver->SetIsPaused_External(!bActive);
	}
}

void AChaosSolverActor::BuildSimulationProxy()
{
	RigidSolverProxy.Solver = PhysScene->GetSolver();
	
	// Ticking setup for collision/breaking notifies
	PrimaryActorTick.TickGroup = TG_PostPhysics;
	PrimaryActorTick.bCanEverTick = true;
	PrimaryActorTick.bStartWithTickEnabled = true;
}

void AChaosSolverActor::ResetSimulationProxy()
{
	RigidSolverProxy.Solver = nullptr;
}

void AChaosSolverActor::WriteToSimulation(const float DeltaTime, const bool bAsyncTask)
{
	if (RigidSolverProxy.IsValid())
	{
		if(!bAsyncTask)
		{
			if(!Proxy && bHasFloor)
			{
				MakeFloor();
			} 
		
			// Update gravity in case it changed
			const FVector DefaultGravity( 0.f, 0.f, GetWorld()->GetGravityZ() );
		
			PhysScene->SetUpForFrame(&DefaultGravity, DeltaTime, UPhysicsSettings::Get()->MinPhysicsDeltaTime, UPhysicsSettings::Get()->MaxPhysicsDeltaTime,
				UPhysicsSettings::Get()->MaxSubstepDeltaTime, UPhysicsSettings::Get()->MaxSubsteps, UPhysicsSettings::Get()->bSubstepping);
		
			PhysScene->StartFrame();
		}
		else
		{
			GetSolver()->AdvanceAndDispatch_External(DeltaTime);
		}

		RigidSolverProxy.PushDatas.Reset();
		while(Chaos::FPushPhysicsData* PushData = RigidSolverProxy.Solver->GetMarshallingManager().StepInternalTime_External())
		{
			RigidSolverProxy.PushDatas.Add(PushData);
		}
	}
}

void AChaosSolverActor::ReadFromSimulation(const float DeltaTime, const bool bAsyncTask)
{
	if(!bAsyncTask)
	{
		if (RigidSolverProxy.IsValid())
		{
			PhysScene->EndFrame();
		}
	}
}

#if WITH_EDITOR
void AChaosSolverActor::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if(RigidSolverProxy.Solver && PropertyChangedEvent.Property)
	{
		if(PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(AChaosSolverActor, Properties))
		{
			RigidSolverProxy.Solver->EnqueueCommandImmediate([InSolver = RigidSolverProxy.Solver, InConfig = Properties]()
			{
				InSolver->ApplyConfig(InConfig);
			});
		}
		else if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(AChaosSolverActor, bHasFloor))
		{
			if(Proxy)
			{
				RigidSolverProxy.Solver->UnregisterObject(Proxy);
				Proxy = nullptr;
			}

			MakeFloor();
		}
		else if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(AChaosSolverActor, FloorHeight))
		{
			if(Proxy)
			{
				Proxy->GetGameThreadAPI().SetX(FVector(0.0f, 0.0f, FloorHeight));
			}
		}
		else if(PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(AChaosSolverActor, SimulationAsset))
		{
			MigrateSolver();
		}
	}
}

bool AChaosSolverActor::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	const FName& Name = InProperty->GetFName();

	if (Name == GET_MEMBER_NAME_CHECKED(ThisClass, SimulationAsset))
	{
		static const auto CVarEnableSimulationDataflow = IConsoleManager::Get().FindConsoleVariable(TEXT("p.Dataflow.EnableSimulation"));
		return CVarEnableSimulationDataflow->GetBool();
	}

	return true;
}

#if TODO_REIMPLEMENT_SERIALIZATION_FOR_PERF_TEST
#if !UE_BUILD_SHIPPING
void SerializeForPerfTest(const TArray< FString >&, UWorld* World, FOutputDevice&)
{
	UE_LOGF(LogChaos, Log, "Serializing for perf test:");
	
	const FString FileName("ChaosPerf");
	//todo(mlentine): use this once chaos solver actors are in
	for (TActorIterator<AChaosSolverActor> Itr(World); Itr; ++Itr)
	{
		Chaos::FPhysicsSolver* Solver = Itr->GetSolver();
		Solver->EnqueueCommandImmediate([FileName, InSolver=Solver]() { InSolver->SerializeForPerfTest(FileName); });
	}
}

FAutoConsoleCommand SerializeForPerfTestCommand(TEXT("p.SerializeForPerfTest"), TEXT(""), FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(&SerializeForPerfTest));
#endif
#endif

#endif




