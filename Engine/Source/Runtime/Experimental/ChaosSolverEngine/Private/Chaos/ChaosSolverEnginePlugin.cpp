// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosSolverEnginePlugin.h"

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Chaos/ChaosDebugDrawComponent.h"
#include "Chaos/ChaosDebugDrawSubsystem.h"
#include "Chaos/ChaosSolverActor.h"
#include "ChaosSolversModule.h"
#include "Chaos/ChaosVDEngineEditorBridge.h"

class FChaosSolverEnginePlugin : public IChaosSolverEnginePlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FChaosSolverEnginePlugin, ChaosSolverEngine )

void FChaosSolverEnginePlugin::StartupModule()
{
	FChaosSolversModule* const ChaosModule = FChaosSolversModule::GetModule();
	check(ChaosModule);
	ChaosModule->SetSolverActorClass(AChaosSolverActor::StaticClass(), AChaosSolverActor::StaticClass());

	UChaosDebugDrawComponent::BindWorldDelegates();

#if CHAOS_DEBUG_DRAW
	UChaosDebugDrawSubsystem::Startup();
#endif

#if WITH_CHAOS_VISUAL_DEBUGGER
	FCoreDelegates::OnFEngineLoopInitComplete.AddLambda([]()
	{
		FChaosVDEngineEditorBridge::Get().Initialize();
	});
#endif // WITH_CHAOS_VISUAL_DEBUGGER

}

void FChaosSolverEnginePlugin::ShutdownModule()
{
#if CHAOS_DEBUG_DRAW
	UChaosDebugDrawSubsystem::Shutdown();
#endif

#if WITH_CHAOS_VISUAL_DEBUGGER
	// Calling Teardown only if the bridge was instantiated since we don't
	// want calling 'Get()' which will create the global instance during the shutdown.
	if (FChaosVDEngineEditorBridge::IsInstantiated())
	{
		FChaosVDEngineEditorBridge::Get().TearDown();
	}
#endif // WITH_CHAOS_VISUAL_DEBUGGER
}
