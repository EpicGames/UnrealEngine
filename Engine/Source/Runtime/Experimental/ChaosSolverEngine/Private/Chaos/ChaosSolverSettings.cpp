// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosSolverSettings.h"
#include "Chaos/ChaosSolverActor.h"
#include "ChaosSolversModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosSolverSettings)

UChaosSolverSettings::UChaosSolverSettings()
{
}

UClass* UChaosSolverSettings::GetSolverActorClass() const
{
	UClass* const SolverActorClass = DefaultChaosSolverActorClass.IsValid() ? LoadObject<UClass>(NULL, *DefaultChaosSolverActorClass.ToString()) : nullptr;
	return (SolverActorClass != nullptr) ? SolverActorClass : AChaosSolverActor::StaticClass();
}

void UChaosSolverSettings::UpdateProperty(FProperty* InProperty)
{
	UpdateAllProperties();
}

void UChaosSolverSettings::UpdateAllProperties()
{
}

// internal
void UChaosSolverSettings::RegisterSolverActorProvider()
{
 	FChaosSolversModule* const ChaosModule = FChaosSolversModule::GetModule();
 	check(ChaosModule);
	ChaosModule->RegisterSolverActorClassProvider(this);
}

#if WITH_EDITOR

void UChaosSolverSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UpdateProperty(PropertyChangedEvent.Property);
}
#endif

void UChaosSolverSettings::PostInitProperties()
{
	Super::PostInitProperties();

	RegisterSolverActorProvider();

	UpdateAllProperties();
}

void UChaosSolverSettings::PostReloadConfig(class FProperty* PropertyThatWasLoaded)
{
	Super::PostReloadConfig(PropertyThatWasLoaded);

	UpdateProperty(PropertyThatWasLoaded);
}

