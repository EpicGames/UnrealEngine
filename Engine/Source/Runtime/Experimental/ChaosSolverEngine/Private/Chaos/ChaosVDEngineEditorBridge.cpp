// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosVDEngineEditorBridge.h"

#if WITH_CHAOS_VISUAL_DEBUGGER

#include "Chaos/ChaosVDRemoteSessionsManager.h"
#include "ChaosVDRuntimeModule.h"
#include "ChaosVisualDebugger/ChaosVDTraceMacros.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "DataWrappers/ChaosVDCollisionDataWrappers.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "Engine/Engine.h"
#include "PhysicsEngine/BodySetup.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Runtime/Experimental/Chaos/Private/Chaos/PhysicsObjectInternal.h"
#include "Serialization/MemoryWriter.h"
#include "TraceBasedDebuggerRuntime.h"
#include "TraceDataRelayTransport.h"
#include "UObject/Package.h"


bool FChaosVDEngineEditorBridge::bIsInstantiated = false;

bool FChaosVDEngineEditorBridge::AddOnScreenRecordingMessage(float DummyDeltaTime)
{
	constexpr bool bContinueLooping = false;
	if (!GEngine)
	{
		return bContinueLooping;
	}

	if (!IsInGameThread())
	{
		if (!DeferredShowMessageOnScreenHandle.IsValid())
		{
			DeferredShowMessageOnScreenHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FChaosVDEngineEditorBridge::AddOnScreenRecordingMessage));
		}

		return bContinueLooping;
	}

	DeferredShowMessageOnScreenHandle.Reset();

	static FText ChaosVDRecordingStartedMessage = NSLOCTEXT("ChaosVisualDebugger", "OnScreenChaosVDRecordingStartedMessage", "Chaos Visual Debugger recording in progress...");

	if (CVDRecordingMessageKey == 0)
	{
		CVDRecordingMessageKey = GetTypeHash(ChaosVDRecordingStartedMessage.ToString());
	}

	// Add a long duration value, we will remove the message manually when the recording stops
	constexpr float MessageDurationSeconds = 3600.0f;
	GEngine->AddOnScreenDebugMessage(CVDRecordingMessageKey, MessageDurationSeconds, FColor::Red, ChaosVDRecordingStartedMessage.ToString());

	return bContinueLooping;
}

void FChaosVDEngineEditorBridge::RemoveOnScreenRecordingMessage()
{
	ensure(IsInGameThread());
	
	if (!GEngine)
	{
		return;
	}

	if (DeferredShowMessageOnScreenHandle.IsValid())
	{
		FTSTicker::RemoveTicker(DeferredShowMessageOnScreenHandle);
		DeferredShowMessageOnScreenHandle.Reset();
	}
	else if (CVDRecordingMessageKey != 0)
	{
		GEngine->RemoveOnScreenDebugMessage(CVDRecordingMessageKey);
	}
}

void FChaosVDEngineEditorBridge::HandleCVDRecordingStarted()
{
	// Call base class to handle external communication
	// and will call back with OnRecordingStartedInternal.
	HandleRecordingStarted();
}

void FChaosVDEngineEditorBridge::OnRecordingStartedInternal()
{
	AddOnScreenRecordingMessage();
}

void FChaosVDEngineEditorBridge::HandleCVDPostRecordingStarted()
{
	SerializeCollisionChannelsNames();
}

void FChaosVDEngineEditorBridge::HandleCVDRecordingStopped()
{
	// Call base class to handle external communication
	// and will call back with OnRecordingStoppedInternal.
	HandleRecordingStopped();
}

void FChaosVDEngineEditorBridge::OnRecordingStoppedInternal()
{
	RemoveOnScreenRecordingMessage();
}

void FChaosVDEngineEditorBridge::HandleTraceConnectionDetailsUpdated() const
{
	UE::TraceBasedDebuggers::FTraceConnectionDetailsMessage ConnectionDetailsMessage;
	ConnectionDetailsMessage.InstanceId = FApp::GetInstanceId();
	ConnectionDetailsMessage.TraceDetails = FChaosVDRuntimeModule::Get().GetCurrentTraceConnectionDetails();

	GetSessionsManager()->PublishMessage(ConnectionDetailsMessage);
}

void FChaosVDEngineEditorBridge::HandleCVDRecordingStartFailed(const FText& InFailureReason) const
{
#if !WITH_EDITOR
	if (GEngine)
	{
		// In non-editor builds we don't have an error pop-up, therefore we want to show the error message on screen
		FText ErrorMessage = FText::FormatOrdered(NSLOCTEXT("ChaosVisualDebugger", "StartRecordingFailedOnScreenMessage", "Failed to start CVD recording. {0}"), InFailureReason);

		constexpr float MessageDurationSeconds = 4.0f;
		GEngine->AddOnScreenDebugMessage(CVDRecordingMessageKey, MessageDurationSeconds, FColor::Red, ErrorMessage.ToString());
	}
#endif
}

void FChaosVDEngineEditorBridge::HandlePIEStarted(UGameInstance* GameInstance)
{
	// If we were already recording, show the message
	if (FChaosVDRuntimeModule::Get().IsRecording())
	{
		AddOnScreenRecordingMessage();
	}
}

void FChaosVDEngineEditorBridge::HandleDataChannelChanged(TWeakPtr<FChaosVDDataDataChannel> ChannelWeakPtr)
{
	if (const TSharedPtr<FChaosVDDataDataChannel> DataChannelPtr = ChannelWeakPtr.Pin())
	{
		FChaosVDChannelStateChangeResponseMessage NewChannelState;
		NewChannelState.InstanceID = FApp::GetInstanceId();
		NewChannelState.NewState.bIsEnabled = DataChannelPtr->IsChannelEnabled();
		NewChannelState.NewState.ChannelName = DataChannelPtr->GetId().ToString();
		NewChannelState.NewState.bCanChangeChannelState = DataChannelPtr->CanChangeEnabledState();

		GetSessionsManager()->PublishMessage(NewChannelState);
	}
}

void FChaosVDEngineEditorBridge::SerializeCollisionChannelsNames()
{
	TArray<uint8> CollisionChannelsDataBuffer;
	FMemoryWriter MemWriterAr(CollisionChannelsDataBuffer);

	FChaosVDCollisionChannelsInfoContainer CollisionChannelInfoContainer;

	if (UCollisionProfile* CollisionProfileData = UCollisionProfile::Get())
	{
		for (int32 ChannelIndex = 0; ChannelIndex < FChaosVDCollisionChannelsInfoContainer::MaxSupportedChannels; ++ChannelIndex)
		{
			FChaosVDCollisionChannelInfo Info;
			Info.DisplayName = CollisionProfileData->ReturnChannelNameFromContainerIndex(ChannelIndex).ToString();
			Info.CollisionChannel = ChannelIndex;
			Info.bIsTraceType = CollisionProfileData->ConvertToTraceType(static_cast<ECollisionChannel>(ChannelIndex)) != TraceTypeQuery_MAX;
			CollisionChannelInfoContainer.CollisionChannelInfos.Add(Info);
		}
	}

	Chaos::VisualDebugger::WriteDataToBuffer(CollisionChannelsDataBuffer, CollisionChannelInfoContainer);

	CVD_TRACE_BINARY_DATA(CollisionChannelsDataBuffer, FChaosVDCollisionChannelsInfoContainer::WrapperTypeName);
}

FChaosVDEngineEditorBridge::FChaosVDEngineEditorBridge(): FEngineEditorBridge(LogChaos)
{
	RegisterRemoteSessionsHandler(MakeShared<FChaosVDRemoteSessionsHandler>());
}

FChaosVDEngineEditorBridge& FChaosVDEngineEditorBridge::Get()
{
	static FChaosVDEngineEditorBridge CVDEngineEditorBridge;
	bIsInstantiated = true;
	return CVDEngineEditorBridge;
}

bool FChaosVDEngineEditorBridge::IsInstantiated()
{
	return bIsInstantiated;
}

void FChaosVDEngineEditorBridge::BuildRecordingStatusInternal(UE::TraceBasedDebuggers::FRecordingStatusMessage& OutStatusMessage) const
{
	const FChaosVDRuntimeModule& RuntimeModule = FChaosVDRuntimeModule::Get();

	OutStatusMessage.DebuggerId = RuntimeModule.GetDebuggerId();
	OutStatusMessage.RequesterId = RuntimeModule.GetRecordingRequesterId();
	OutStatusMessage.ElapsedTime = RuntimeModule.GetAccumulatedRecordingTime();
	OutStatusMessage.BufferedDataBytesSize = RuntimeModule.GetBufferedDataBytesSize();
}

FChaosVDParticleMetadata FChaosVDEngineEditorBridge::GenerateParticleMetadata(const IPhysicsProxyBase* ParticleProxy, const Chaos::FGeometryParticleHandle* ParticleHandle)
{
	FChaosVDParticleMetadata Metadata;

	if (!ParticleProxy)
	{
		return Metadata;
	}

	// This method is expected to be called from worker threads as we discover new particles that need to be traced.
	// As we need to access their uobject owners to generate the name, we need to make sure these are not GC'd or wait until GC is done.
	FGCScopeGuard GCLock;

	UObject* ProxyOwner = ParticleProxy->GetOwner();
	if (IsValid(ProxyOwner))
	{
		if (UPrimitiveComponent* OwnerAsPrimitiveComponent = Cast<UPrimitiveComponent>(ProxyOwner))
		{
			Metadata.ComponentName = ProxyOwner->GetFName();
			Metadata.MapAssetPath = FTopLevelAssetPath(ProxyOwner->GetWorld());
			Metadata.OwnerName = GetFNameSafe(OwnerAsPrimitiveComponent->GetOwner());

			if (AActor* OwningActor = OwnerAsPrimitiveComponent->GetOwner())
			{
				Metadata.OwnerName = OwningActor->GetFName();
				FSoftObjectPath ObjectPath = FSoftObjectPath(OwningActor->GetClass());
				Metadata.OwnerAssetPath = ObjectPath.GetAssetPath();
			}
		}
		else
		{
			Metadata.OwnerName = ProxyOwner->GetFName();
		}

		if (!Metadata.OwnerAssetPath.IsValid())
		{
			FSoftObjectPath ObjectPath = FSoftObjectPath(ProxyOwner);
			Metadata.OwnerAssetPath = ObjectPath.GetAssetPath();
		}
	}

	switch (ParticleProxy->GetType())
	{
	case EPhysicsProxyType::SingleParticleProxy:
		{
			const Chaos::FSingleParticlePhysicsProxy* AsSingleParticleProxy = static_cast<const Chaos::FSingleParticlePhysicsProxy*>(ParticleProxy);
			if (Chaos::FPhysicsObjectHandle PhysicsObjectHandle = AsSingleParticleProxy->GetPhysicsObject())
			{
				// Currently only bodies corresponding to the skeletal mesh bones bodies set this value
				Metadata.BoneName = PhysicsObjectHandle->GetBodyName();
			}

			break;
		}
	case EPhysicsProxyType::GeometryCollectionType:
		{
			if (ParticleHandle)
			{
				const FGeometryCollectionPhysicsProxy* AsGeometryCollectionProxy = static_cast<const FGeometryCollectionPhysicsProxy*>(ParticleProxy);
				Metadata.Index = AsGeometryCollectionProxy->GetTransformGroupIndexFromHandle(ParticleHandle->CastToRigidParticle());
			}
			break;
		}
	case EPhysicsProxyType::ClusterUnionProxy: // These follow the same rules as Single particle proxies
	case EPhysicsProxyType::NoneType:
	case EPhysicsProxyType::StaticMeshType:
	case EPhysicsProxyType::FieldType:
	case EPhysicsProxyType::JointConstraintType:
	case EPhysicsProxyType::SuspensionConstraintType:
	case EPhysicsProxyType::CharacterGroundConstraintType:
	case EPhysicsProxyType::Count:
	default:
		ensureMsgf(false, TEXT("Unsupported Proxy type"));
		break;
	}

	return Metadata;
}

void FChaosVDEngineEditorBridge::OnInitializeInternal()
{
	using namespace UE::TraceBasedDebuggers;
	FChaosVDRuntimeModule& CVDRuntimeModule = FChaosVDRuntimeModule::Get();

	RecordingStartedHandle = CVDRuntimeModule.RegisterRecordingStartedCallback(FRecordingStateChangedDelegate::FDelegate::CreateRaw(this, &FChaosVDEngineEditorBridge::HandleCVDRecordingStarted));
	PostRecordingStartedHandle = CVDRuntimeModule.RegisterPostRecordingStartedCallback(FRecordingStateChangedDelegate::FDelegate::CreateRaw(this, &FChaosVDEngineEditorBridge::HandleCVDPostRecordingStarted));
	RecordingStoppedHandle = CVDRuntimeModule.RegisterRecordingStopCallback(FRecordingStateChangedDelegate::FDelegate::CreateRaw(this, &FChaosVDEngineEditorBridge::HandleCVDRecordingStopped));
	RecordingStartFailedHandle = CVDRuntimeModule.RegisterRecordingStartFailedCallback(FRecordingStartFailedDelegate::FDelegate::CreateRaw(this, &FChaosVDEngineEditorBridge::HandleCVDRecordingStartFailed));

	CVDRuntimeModule.OnTraceConnectionDetailsUpdated().AddRaw(this, &FChaosVDEngineEditorBridge::HandleTraceConnectionDetailsUpdated);

	FChaosVisualDebuggerTrace::RegisterExternalParticleDebugNameGenerator(Chaos::VD::FRecordingSessionState::FParticleMetaDataGeneratorDelegate::CreateStatic(&FChaosVDEngineEditorBridge::GenerateParticleMetadata));

	Chaos::VisualDebugger::FChaosVDDataChannelsManager::Get().OnChannelStateChanged().AddRaw(this, &FChaosVDEngineEditorBridge::HandleDataChannelChanged);

#if WITH_EDITOR
	PIEStartedHandle = FWorldDelegates::OnPIEStarted.AddRaw(this, &FChaosVDEngineEditorBridge::HandlePIEStarted);
#endif

	// If we were already recording, we need to make sure we run the initialization step to set up the session broadcast ticker
	// and the collision channel serialization
	if (CVDRuntimeModule.IsRecording())
	{
		HandleCVDRecordingStarted();
		HandleCVDPostRecordingStarted();
	}
}

void FChaosVDEngineEditorBridge::OnTearDownInternal()
{
#if WITH_EDITOR
	FWorldDelegates::OnPIEStarted.Remove(PIEStartedHandle);
	PIEStartedHandle.Reset();
#endif
	FTSTicker::RemoveTicker(DeferredShowMessageOnScreenHandle);
	DeferredShowMessageOnScreenHandle.Reset();

	if (FModuleManager::Get().IsModuleLoaded(TEXT("ChaosVDRuntime")))
	{
		FChaosVDRuntimeModule& CVDRuntimeModule = FChaosVDRuntimeModule::Get();
		CVDRuntimeModule.RemoveRecordingStartedCallback(RecordingStartedHandle);
		RecordingStartedHandle.Reset();
		CVDRuntimeModule.RemovePostRecordingStartedCallback(PostRecordingStartedHandle);
		PostRecordingStartedHandle.Reset();
		CVDRuntimeModule.RemoveRecordingStopCallback(RecordingStoppedHandle);
		RecordingStoppedHandle.Reset();
		CVDRuntimeModule.RemoveRecordingStartFailedCallback(RecordingStartFailedHandle);
		RecordingStartFailedHandle.Reset();

		CVDRuntimeModule.OnTraceConnectionDetailsUpdated().RemoveAll(this);

		// Make sure of removing the message from the screen in case the recording didn't quite stop yet
		if (CVDRuntimeModule.IsRecording())
		{
			HandleCVDRecordingStopped();
		}
	}
}
#else

FChaosVDEngineEditorBridge& FChaosVDEngineEditorBridge::Get()
{
	static FChaosVDEngineEditorBridge CVDEngineEditorBridge;
	return CVDEngineEditorBridge;
}

#endif // WITH_CHAOS_VISUAL_DEBUGGER
