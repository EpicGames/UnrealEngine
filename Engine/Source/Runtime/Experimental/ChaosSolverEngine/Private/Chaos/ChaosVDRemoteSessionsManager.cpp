// Copyright Epic Games, Inc. All Rights Reserved.

#include "Chaos/ChaosVDRemoteSessionsManager.h"

#if WITH_TRACE_BASED_DEBUGGERS
#include "ChaosVDRuntimeModule.h"
#include "ChaosVisualDebugger/ChaosVDOptionalDataChannel.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "MessageEndpointBuilder.h"
#include "SessionInfo.h"

void FChaosVDRemoteSessionsHandler::OnCreatingMessageEndpoint(
	const TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager>& SessionsManager
	, FMessageEndpointBuilder& InEndPointBuilder)
{
	InEndPointBuilder
		.Handling<FChaosVDChannelStateChangeCommandMessage>(
			[](const FChaosVDChannelStateChangeCommandMessage& InMessage, const TSharedRef<IMessageContext>&)
			{
				UE_AUTORTFM_ONCOMMIT(InMessage)
				{
					using namespace Chaos::VisualDebugger;
					TWeakPtr<FChaosVDOptionalDataChannel> ChannelInstance = FChaosVDDataChannelsManager::Get().GetChannelById(FName(InMessage.NewState.ChannelName));
					if (TSharedPtr<FChaosVDOptionalDataChannel> LockedChannelInstance = ChannelInstance.Pin())
					{
						LockedChannelInstance->SetChannelEnabled(InMessage.NewState.bIsEnabled);
					}
				};
			})
		.Handling<FChaosVDStartRecordingCommandMessage>(
			[](const FChaosVDStartRecordingCommandMessage& InMessage, const TSharedRef<IMessageContext>& InContext)
			{
				UE_AUTORTFM_ONCOMMIT(InMessage, InContext)
				{
					FChaosVisualDebuggerTrace::OverrideDefaultEnabledDataChannels(InMessage.DataChannelsEnabledOverrideList);
					FChaosVDRuntimeModule::Get().StartRecording(InMessage);
				};
			})
		.Handling<FChaosVDStopRecordingCommandMessage>(
			[](const FChaosVDStopRecordingCommandMessage&, const TSharedRef<IMessageContext>&)
			{
				UE_AUTORTFM_ONCOMMIT()
				{
					FChaosVDRuntimeModule::Get().StopRecording();
				};
			});

	using namespace UE::TraceBasedDebuggers;
	if (SessionsManager && SessionsManager->IsController())
	{
		InEndPointBuilder
			.Handling<FChaosVDChannelStateChangeResponseMessage>(
				[WeakSessionsManager = SessionsManager.ToWeakPtr()](const FChaosVDChannelStateChangeResponseMessage& InMessage, const TSharedRef<IMessageContext>&)
				{
					if (const TSharedPtr<FRemoteSessionsManager> SessionManager = WeakSessionsManager.Pin())
					{
						if (const TSharedPtr<FSessionInfo> SessionInfoPtr = SessionManager->GetSessionInfo(InMessage.InstanceID).Pin())
						{
							if (FChaosVDSessionData* ChaosData = SessionInfoPtr->GetDebuggerData<FChaosVDSessionData>())
							{
								if (FChaosVDDataChannelState* FoundChannelState = ChaosData->DataChannelsStatesByName.Find(InMessage.NewState.ChannelName))
								{
									*FoundChannelState = InMessage.NewState;
								}
							}
						}
					}
				});
	}
}

void FChaosVDRemoteSessionsHandler::OnSubscribingMessageTypes(
	const TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager>& InSessionsManager
	, FMessageEndpoint& InMessageEndpoint)
{
	if (InSessionsManager)
	{
		InSessionsManager->RegisterExternalSupportedMessageType<FChaosVDStartRecordingCommandMessage>();
		InSessionsManager->RegisterExternalSupportedMessageType<FChaosVDStopRecordingCommandMessage>();
		InSessionsManager->RegisterExternalSupportedMessageType<FChaosVDChannelStateChangeCommandMessage>();
		InSessionsManager->RegisterExternalSupportedMessageType<FChaosVDChannelStateChangeResponseMessage>();

		if (InSessionsManager->IsController())
		{
			InMessageEndpoint.Subscribe<FChaosVDChannelStateChangeResponseMessage>();
		}
	}

	InMessageEndpoint.Subscribe<FChaosVDStartRecordingCommandMessage>();
	InMessageEndpoint.Subscribe<FChaosVDStopRecordingCommandMessage>();
	InMessageEndpoint.Subscribe<FChaosVDChannelStateChangeCommandMessage>();
}

void FChaosVDRemoteSessionsHandler::OnBuildingFullSessionInfoResponseMessage(
	UE::TraceBasedDebuggers::FFullSessionInfoResponseMessage& InMessage)
{
	using namespace Chaos::VisualDebugger;

	InMessage.RecordingRequesterId = FChaosVDRuntimeModule::Get().GetRecordingRequesterId();
	InMessage.DebuggerId = Chaos::VD::DebuggerGuid;

	FChaosVDFullSessionInfoResponseData MessageData;
	FChaosVDDataChannelsManager::Get().EnumerateDataChannels([&MessageData](const TWeakPtr<FChaosVDOptionalDataChannel>& Channel)
		{
			if (const TSharedPtr<FChaosVDOptionalDataChannel> LockedChannel = Channel.Pin())
			{
				MessageData.DataChannelsStates.Emplace(
					FChaosVDDataChannelState
					{
						LockedChannel->GetId().ToString()
						, LockedChannel->IsChannelEnabled()
						, LockedChannel->CanChangeEnabledState()
					});
			}
			return true;
		});

	InMessage.DebuggerSpecificData.Emplace(FInstancedStruct::Make(MoveTemp(MessageData)));
}

void FChaosVDRemoteSessionsHandler::OnHandlingFullSessionInfoResponseMessage(
	const UE::TraceBasedDebuggers::FFullSessionInfoResponseMessage& InMessage
	, const TSharedPtr<UE::TraceBasedDebuggers::FSessionInfo>& InSessionInfo)
{
	if (!InSessionInfo.IsValid())
	{
		return;
	}

	if (const FChaosVDFullSessionInfoResponseData* MessageData = InMessage.GetDebuggerData<FChaosVDFullSessionInfoResponseData>())
	{
		FChaosVDSessionData NewSessionData;
		NewSessionData.DataChannelsStatesByName.Reserve(MessageData->DataChannelsStates.Num());
		for (const FChaosVDDataChannelState& ChannelState : MessageData->DataChannelsStates)
		{
			NewSessionData.DataChannelsStatesByName.Emplace(ChannelState.ChannelName, ChannelState);
		};

		InSessionInfo->SetDebuggerData<FChaosVDSessionData>(MoveTemp(NewSessionData));
	}
}

#endif // WITH_TRACE_BASED_DEBUGGERS

PRAGMA_DISABLE_DEPRECATION_WARNINGS

const FChaosVDTraceDetails& FChaosVDSessionInfo::GetConnectionDetails()
{
	return LastKnownConnectionDetails;
}

EChaosVDRecordingMode FChaosVDSessionInfo::GetRecordingMode() const
{
	return LastKnownConnectionDetails.Mode;
}

EChaosVDRecordingMode FChaosVDSessionInfo::GetLastRequestedRecordingMode() const
{
	return EChaosVDRecordingMode::Invalid;
}

void FChaosVDSessionInfo::SetLastRequestedRecordingMode(EChaosVDRecordingMode NewRecordingMode)
{
	LastRequestedRecordingMode = NewRecordingMode;
}

bool FChaosVDSessionInfo::IsConnected() const
{
	return false;
}

EChaosVDRecordingMode FChaosVDMultiSessionInfo::GetRecordingMode() const
{
	EChaosVDRecordingMode FirstValidInstanceRecordingMode = EChaosVDRecordingMode::Invalid;
	EnumerateInnerSessions([&FirstValidInstanceRecordingMode](const TSharedRef<FChaosVDSessionInfo>& InSessionRef)
		{
			EChaosVDRecordingMode RecordingMode = InSessionRef->GetRecordingMode();
			if (RecordingMode == EChaosVDRecordingMode::Invalid)
			{
				// In multi-session, all recordings will have the same recording mode, but not of them might report connected state at the same time
				// Therefore we need to continue searching until one of the session has a valid state before giving up.
				return true;
			}

			FirstValidInstanceRecordingMode = RecordingMode;
			return false;
		});

	return FirstValidInstanceRecordingMode;	
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS