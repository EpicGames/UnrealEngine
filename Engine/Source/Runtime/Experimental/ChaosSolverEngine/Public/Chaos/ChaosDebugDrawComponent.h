// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "Chaos/ChaosDebugDrawDeclares.h"
#include "Chaos/Declares.h"
#include "Chaos/DebugDrawQueue.h"
#include "Engine/World.h"
#include "ChaosDebugDrawComponent.generated.h"


UCLASS(BlueprintType, ClassGroup = Chaos, meta = (BlueprintSpawnableComponent), MinimalAPI)
class UChaosDebugDrawComponent : public UActorComponent
{
	GENERATED_BODY()

public:

	CHAOSSOLVERENGINE_API UChaosDebugDrawComponent();

	//~ Begin UActorComponent interface
	CHAOSSOLVERENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	CHAOSSOLVERENGINE_API virtual void BeginPlay() override;
	CHAOSSOLVERENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	CHAOSSOLVERENGINE_API virtual void BeginDestroy() override;
	//~ End UActorComponent interface

	static CHAOSSOLVERENGINE_API void BindWorldDelegates();

protected:
#if WITH_EDITOR && CHAOS_DEBUG_DRAW
	virtual void OnRegister() override;
	virtual void OnUnregister() override;
#endif // WITH_EDITOR && CHAOS_DEBUG_DRAW

private:
	static void HandlePostWorldInitialization(UWorld* World, const UWorld::InitializationValues IVS);
	static void CreateDebugDrawActor(UWorld* World);

	bool bInPlay;

#if CHAOS_DEBUG_DRAW
	TArray<Chaos::FLatentDrawCommand> DrawCommands;
#endif
};

