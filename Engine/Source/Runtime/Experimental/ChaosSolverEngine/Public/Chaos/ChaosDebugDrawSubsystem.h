// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosDebugDraw/ChaosDDContext.h"
#include "ChaosDebugDraw/ChaosDDTypes.h"
#include "Engine/EngineBaseTypes.h"
#include "Subsystems/WorldSubsystem.h"

#include "ChaosDebugDrawSubsystem.generated.h"

class UWorld;
class FChaosDDRenderer;

UCLASS(MinimalAPI)
class UChaosDebugDrawSubsystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:

	virtual bool ShouldCreateSubsystem(UObject* Outer) const override final;
	virtual void PostInitialize() override final;
	virtual void Deinitialize() override final;

	static void Startup();
	static void Shutdown();

#if CHAOS_DEBUG_DRAW
public:

	const ChaosDD::Private::FChaosDDScenePtr& GetDebugDrawScene() const { return CDDScene; }

protected:

	void RenderScene(FChaosDDRenderer& Renderer, const ChaosDD::Private::FChaosDDScenePtr& Scene);
	void RenderFrame(FChaosDDRenderer& Renderer, const ChaosDD::Private::FChaosDDFramePtr& Frame);
	void OnWorldTickStart(ELevelTick TickType, float Dt);
	void OnWorldTickEnd(ELevelTick TickType, float Dt);
	void UpdateDrawRegion();
	void RenderScene();
	static void StaticOnWorldTickStart(UWorld* World, ELevelTick TickType, float Dt);
	static void StaticOnWorldTickEnd(UWorld* World, ELevelTick TickType, float Dt);

	ChaosDD::Private::FChaosDDScenePtr CDDScene;
	ChaosDD::Private::FChaosDDTimelinePtr CDDWorldTimeline;
	ChaosDD::Private::FChaosDDTimelineContext CDDWorldTimelineContext;

	static FDelegateHandle OnTickWorldStartDelegate;
	static FDelegateHandle OnTickWorldEndDelegate;
#endif
};
