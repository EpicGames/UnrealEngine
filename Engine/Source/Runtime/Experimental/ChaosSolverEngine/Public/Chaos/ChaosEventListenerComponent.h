// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


// #include "Chaos/ChaosSolver.h"
// #include "CoreMinimal.h"
// #include "GameFramework/Actor.h"
// #include "Physics/Experimental/PhysScene_Chaos.h"
// #include "UObject/ObjectMacros.h"

#include "Components/ActorComponent.h"
#include "Chaos/Declares.h"
#include "ChaosEventListenerComponent.generated.h"

class FPhysScene_Chaos;
class AChaosSolverActor;

/** 
 * Base class for listeners that query and respond to a frame's physics data (collision events, break events, etc).
 */

UCLASS(BlueprintType, ClassGroup = Chaos, meta = (BlueprintSpawnableComponent), MinimalAPI)
class UChaosEventListenerComponent : public UActorComponent
{
	GENERATED_BODY()

public:

	CHAOSSOLVERENGINE_API UChaosEventListenerComponent();

	//~ Begin UActorComponent interface
	CHAOSSOLVERENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//~ End UActorComponent interface


protected:

	/** Used to know when the physics thread has updated the collision info for processing on the game thread */
	float LastCollisionTickTime;
};

