// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosEventListenerComponent.h"
#include "PhysicsPublic.h"
#include "Physics/Experimental/ChaosEventType.h"
#include "ChaosNotifyHandlerInterface.h"
#include "ChaosGameplayEventDispatcher.generated.h"

struct FBodyInstance;

namespace Chaos
{
	struct FCollisionEventData;
	struct FBreakingEventData;
	struct FSleepingEventData;
	struct FRemovalEventData;
	struct FCrumblingEventData;
	struct FBreakingData;
}

typedef TFunction<void(const FChaosBreakEvent&)> FOnBreakEventCallback;

/** UStruct wrapper so we can store the TFunction in a TMap */
USTRUCT()
struct FBreakEventCallbackWrapper
{
	GENERATED_BODY()

public:
	FOnBreakEventCallback BreakEventCallback;
};

typedef TFunction<void(const FChaosRemovalEvent&)> FOnRemovalEventCallback;

/** UStruct wrapper so we can store the TFunction in a TMap */
USTRUCT()
struct FRemovalEventCallbackWrapper
{
	GENERATED_BODY()

public:
	FOnRemovalEventCallback RemovalEventCallback;
};

typedef TFunction<void(const FChaosCrumblingEvent&)> FOnCrumblingEventCallback;

/** UStruct wrapper so we can store the TFunction in a TMap */
USTRUCT()
struct FCrumblingEventCallbackWrapper
{
	GENERATED_BODY()

public:
	FOnCrumblingEventCallback CrumblingEventCallback;
};

/** UStruct wrapper so we can store the TSet in a TMap */
USTRUCT()
struct FChaosHandlerSet
{
	GENERATED_BODY()

	bool bLegacyComponentNotify;
		
	/** These should be IChaosNotifyHandlerInterface refs, but we can't store those here */
	UPROPERTY()
	TSet<TObjectPtr<UObject>> ChaosHandlers;
};

struct FChaosPendingCollisionNotify
{
	FChaosPhysicsCollisionInfo CollisionInfo;
	TSet<TObjectPtr<UObject>> NotifyRecipients;
};


UCLASS(MinimalAPI)
class UChaosGameplayEventDispatcher : public UChaosEventListenerComponent
{
	GENERATED_BODY()

public:

	CHAOSSOLVERENGINE_API UChaosGameplayEventDispatcher();

	CHAOSSOLVERENGINE_API virtual void OnRegister() override;
	CHAOSSOLVERENGINE_API virtual void OnUnregister() override;

	CHAOSSOLVERENGINE_API void RegisterChaosEvents();
	CHAOSSOLVERENGINE_API void UnregisterChaosEvents();

private:

	// contains the set of properties that uniquely identifies a reported collision
	// Note that order matters, { Body0, Body1 } is not the same as { Body1, Body0 }
	struct FUniqueContactPairKey
	{
		const void* Body0;
		const void* Body1;

		friend bool operator==(const FUniqueContactPairKey& Lhs, const FUniqueContactPairKey& Rhs) 
		{ 
			return Lhs.Body0 == Rhs.Body0 && Lhs.Body1 == Rhs.Body1;
		}

		friend inline uint32 GetTypeHash(FUniqueContactPairKey const& P)
		{
			return uint32((PTRINT)P.Body0 ^ ((PTRINT)P.Body1 << 18));
		}
	};

	FCollisionNotifyInfo& GetPendingCollisionForContactPair(const void* P0, const void* P1, bool& bNewEntry);
	/** Key is the unique pair, value is index into PendingNotifies array */
	TMap<FUniqueContactPairKey, int32> ContactPairToPendingNotifyMap;

	FChaosPendingCollisionNotify& GetPendingChaosCollisionForContactPair(const void* P0, const void* P1, bool& bNewEntry);
	/** Key is the unique pair, value is index into PendingChaosCollisionNotifies array */
	TMap<FUniqueContactPairKey, int32> ContactPairToPendingChaosNotifyMap;

	/** Holds the list of pending Chaos notifies that are to be processed */
	TArray<FChaosPendingCollisionNotify> PendingChaosCollisionNotifies;

	/** Holds the list of pending legacy notifies that are to be processed */
	TArray<FCollisionNotifyInfo> PendingCollisionNotifies;

public:
	/** 
	 * Use to subscribe to collision events. 
	 * @param ComponentToListenTo	The component whose collisions will be reported
	 * @param ObjectToNotify		The object that will receive the notifications. Should be a PrimitiveComponent or implement IChaosNotifyHandlerInterface, or both.
	 */
	CHAOSSOLVERENGINE_API void RegisterForCollisionEvents(UPrimitiveComponent* ComponentToListenTo, UObject* ObjectToNotify);
	CHAOSSOLVERENGINE_API void UnRegisterForCollisionEvents(UPrimitiveComponent* ComponentToListenTo, UObject* ObjectToNotify);

	CHAOSSOLVERENGINE_API void RegisterForBreakEvents(UPrimitiveComponent* Component, FOnBreakEventCallback InFunc);
	CHAOSSOLVERENGINE_API void UnRegisterForBreakEvents(UPrimitiveComponent* Component);

	CHAOSSOLVERENGINE_API void RegisterForRemovalEvents(UPrimitiveComponent* Component, FOnRemovalEventCallback InFunc);
	CHAOSSOLVERENGINE_API void UnRegisterForRemovalEvents(UPrimitiveComponent* Component);

	CHAOSSOLVERENGINE_API void RegisterForCrumblingEvents(UPrimitiveComponent* Component, FOnCrumblingEventCallback InFunc);
	CHAOSSOLVERENGINE_API void UnRegisterForCrumblingEvents(UPrimitiveComponent* Component);
	
private:

 	UPROPERTY()
 	TMap<TObjectPtr<UPrimitiveComponent>, FChaosHandlerSet> CollisionEventRegistrations;

	UPROPERTY()
	TMap<TObjectPtr<UPrimitiveComponent>, FBreakEventCallbackWrapper> BreakEventRegistrations;

	UPROPERTY()
	TMap<TObjectPtr<UPrimitiveComponent>, FRemovalEventCallbackWrapper> RemovalEventRegistrations;

	UPROPERTY()
	TMap<TObjectPtr<UPrimitiveComponent>, FCrumblingEventCallbackWrapper> CrumblingEventRegistrations;
	

	float LastCollisionDataTime = -1.f;
	float LastBreakingDataTime = -1.f;
	float LastRemovalDataTime = -1.f;
	float LastCrumblingDataTime = -1.f;

	void DispatchPendingCollisionNotifies();

	template <typename EventIterator>
	void FillPhysicsProxy(FPhysScene_Chaos& Scene, TArray<UObject*>& Result, EventIterator& It);

	TArray<UObject*> GetInterestedProxyOwnersForCollisionEvents();
	TArray<UObject*> GetInterestedProxyOwnersForBreakingEvents();
	TArray<UObject*> GetInterestedProxyOwnersForRemovalEvents();
	TArray<UObject*> GetInterestedProxyOwnersForCrumblingEvents();

	// Chaos Event Handlers
	void HandleCollisionEvents(const Chaos::FCollisionEventData& CollisionData);
	void HandleBreakingEvents(const Chaos::FBreakingEventData& BreakingData);
	void HandleSleepingEvents(const Chaos::FSleepingEventData& SleepingData);
	void HandleRemovalEvents(const Chaos::FRemovalEventData& RemovalData);
	void HandleCrumblingEvents(const Chaos::FCrumblingEventData& CrumblingData);

};


