// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "UObject/Interface.h"
#include "ChaosNotifyHandlerInterface.generated.h"

struct FHitResult;

USTRUCT(BlueprintType)
struct FChaosPhysicsCollisionInfo
{
	GENERATED_BODY()
public:

	CHAOSSOLVERENGINE_API FChaosPhysicsCollisionInfo();

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Chaos")
	TObjectPtr<UPrimitiveComponent> Component = nullptr;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chaos")
	TObjectPtr<UPrimitiveComponent> OtherComponent = nullptr;

	/** Location of the impact */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chaos")
	FVector Location;
	
	/** Normal at the impact */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chaos")
	FVector Normal;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chaos")
	FVector AccumulatedImpulse;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chaos")
	FVector Velocity;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chaos")
	FVector OtherVelocity;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chaos")
	FVector AngularVelocity;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chaos")
	FVector OtherAngularVelocity;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chaos")
	float Mass;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Chaos")
	float OtherMass;
};

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnChaosPhysicsCollision, const FChaosPhysicsCollisionInfo&, CollisionInfo);

/** Interface for objects that want collision and trailing notifies from the Chaos solver */
UINTERFACE(BlueprintType, MinimalAPI)
class UChaosNotifyHandlerInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IChaosNotifyHandlerInterface : public IInterface
{
	GENERATED_IINTERFACE_BODY()

public:
	/** Override for native handling of a physics collision */
	virtual void NotifyPhysicsCollision(const FChaosPhysicsCollisionInfo& CollisionInfo) {};

	/** Implementing classes should override to dispatch whatever blueprint events they choose to offer */
	virtual void DispatchChaosPhysicsCollisionBlueprintEvents(const FChaosPhysicsCollisionInfo& CollisionInfo) {};

	/** Entry point for collision notifications, called by the underlying system. Not for overriding. */
	CHAOSSOLVERENGINE_API void HandlePhysicsCollision(const FChaosPhysicsCollisionInfo& CollisionInfo);

};



/**
 * 
 */
UCLASS(MinimalAPI)
class UChaosSolverEngineBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
public:

	UFUNCTION(BlueprintPure, Category = "Chaos", meta = (WorldContext = "WorldContextObject"))
	static CHAOSSOLVERENGINE_API FHitResult ConvertPhysicsCollisionToHitResult(const FChaosPhysicsCollisionInfo& PhysicsCollision);
};
