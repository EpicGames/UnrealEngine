// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Physics/Experimental/PhysScene_Chaos.h"

#include "ChaosSolver.generated.h"

class UChaosSolver;

/**
* UChaosSolver (UObject)
*
*/
UCLASS(customconstructor, MinimalAPI)
class UChaosSolver : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	CHAOSSOLVERENGINE_API UChaosSolver(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	bool IsVisible() { return true; }
};



