// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** This class represents an ChaosSolver Actor. */

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "Components/BillboardComponent.h"
#include "UObject/ObjectMacros.h"
#include "Chaos/ChaosSolver.h"
#include "Chaos/ChaosSolverComponentTypes.h"
#include "Chaos/ClusterCreationParameters.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "ChaosSolverConfiguration.h"
#include "SolverEventFilters.h"
#include "Dataflow/Interfaces/DataflowPhysicsSolver.h"
#include "PhysicsProxy/SingleParticlePhysicsProxyFwd.h"

#include "ChaosSolverActor.generated.h"

class UChaosGameplayEventDispatcher;

namespace Chaos
{
	template<typename, int>
	class TGeometryParticle;
}

/** Legacy enum for old deprecated configuration properties. To be removed when those properties are */
UENUM()
enum class EClusterConnectionTypeEnum : uint8
{
	Chaos_PointImplicit = Chaos::FClusterCreationParameters::PointImplicit ,
	Chaos_DelaunayTriangulation = Chaos::FClusterCreationParameters::DelaunayTriangulation UMETA(Hidden),
	Chaos_MinimalSpanningSubsetDelaunayTriangulation = Chaos::FClusterCreationParameters::MinimalSpanningSubsetDelaunayTriangulation,
	Chaos_PointImplicitAugmentedWithMinimalDelaunay = Chaos::FClusterCreationParameters::PointImplicitAugmentedWithMinimalDelaunay,
	Chaos_BoundsOverlapFilteredDelaunayTriangulation = Chaos::FClusterCreationParameters::BoundsOverlapFilteredDelaunayTriangulation,
	Chaos_None = Chaos::FClusterCreationParameters::None UMETA(Hidden),
	//
	Chaos_EClsuterCreationParameters_Max UMETA(Hidden)
};

USTRUCT()
struct FChaosDebugSubstepControl
{
	GENERATED_USTRUCT_BODY()

	FChaosDebugSubstepControl() : bPause(false), bSubstep(false), bStep(false) {}
	
	/*
	* Pause the solver at the next synchronization point.
	*/
	UPROPERTY(EditAnywhere, Category = "ChaosPhysics|Debug")
	bool bPause;

	/*
	* Substep the solver to the next synchronization point.
	*/
	UPROPERTY(EditAnywhere, Category = "ChaosPhysics|Debug", meta = (EditCondition = "bPause"))
	bool bSubstep;

	/*
	* Step the solver to the next synchronization point.
	*/
	UPROPERTY(EditAnywhere, Category = "ChaosPhysics|Debug", meta = (EditCondition = "bPause"))
	bool bStep;

#if WITH_EDITOR
	FSimpleDelegate OnPauseChanged;  // Delegate used to refresh the Editor's details customization when the pause value changed.
#endif
};

USTRUCT(BlueprintType)
struct FDataflowRigidSolverProxy : public FDataflowPhysicsSolverProxy
{
	GENERATED_USTRUCT_BODY()

	FDataflowRigidSolverProxy() : FDataflowPhysicsSolverProxy()
	{}
	virtual ~FDataflowRigidSolverProxy() override = default;

	//~ Begin FPhysicsSolverInterface interface
	virtual void AdvanceSolverDatas(const float DeltaTime) override;
	virtual bool IsValid() const override { return Solver != nullptr;}
	virtual const UScriptStruct* GetScriptStruct() const override
	{
		return StaticStruct();
	}
	//~ End FPhysicsSolverInterface interface

	/** Chaos deformable solver that will be used in the component */
	Chaos::FPhysicsSolver* Solver = nullptr;

	/** List of push datas that will be used to advance the solver */
	TArray<Chaos::FPushPhysicsData*> PushDatas = {};
};

template <>
struct TStructOpsTypeTraits<FDataflowRigidSolverProxy> : public TStructOpsTypeTraitsBase2<FDataflowRigidSolverProxy>
{
	enum { WithCopy = false };
};

UCLASS(MinimalAPI)
class AChaosSolverActor : public AActor, public IDataflowPhysicsSolverInterface
{
	GENERATED_UCLASS_BODY()

public:
	
	UPROPERTY(EditAnywhere, Category = "Chaos", meta=(ShowOnlyInnerProperties))
	FChaosSolverConfiguration Properties;

	/** Deprecated solver properties (moved to FChaosSolverConfiguration)*/
	UPROPERTY()
	float TimeStepMultiplier_DEPRECATED;
	UPROPERTY()
	int32 CollisionIterations_DEPRECATED;
	UPROPERTY()
	int32 PushOutIterations_DEPRECATED;
	UPROPERTY()
	int32 PushOutPairIterations_DEPRECATED;
	UPROPERTY()
	float ClusterConnectionFactor_DEPRECATED;
	UPROPERTY()
	EClusterConnectionTypeEnum ClusterUnionConnectionType_DEPRECATED;
	UPROPERTY()
	bool DoGenerateCollisionData_DEPRECATED;
	UPROPERTY()
	FSolverCollisionFilterSettings CollisionFilterSettings_DEPRECATED;
	UPROPERTY()
	bool DoGenerateBreakingData_DEPRECATED;
	UPROPERTY()
	FSolverBreakingFilterSettings BreakingFilterSettings_DEPRECATED;
	UPROPERTY()
	bool DoGenerateTrailingData_DEPRECATED;
	UPROPERTY()
	FSolverTrailingFilterSettings TrailingFilterSettings_DEPRECATED;
	UPROPERTY()
	float MassScale_DEPRECATED;
	/** End deprecated properties */
	
	UPROPERTY(EditAnywhere, Category = Settings)
	bool bHasFloor;

	UPROPERTY(EditAnywhere, Category=Settings)
	float FloorHeight;

	/*
	* Control to pause/step/substep the solver to the next synchronization point.
	*/
	UPROPERTY(EditAnywhere, Category = "ChaosPhysics|Debug")
	FChaosDebugSubstepControl ChaosDebugSubstepControl;

	/** Makes this solver the current world solver. Dynamically spawned objects will have their physics state created in this solver. */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	CHAOSSOLVERENGINE_API void SetAsCurrentWorldSolver();

	/** Controls whether the solver is able to simulate particles it controls */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	CHAOSSOLVERENGINE_API virtual void SetSolverActive(bool bActive);

	//~ Begin IDataflowPhysicsSolverInterface interface
	virtual FString GetSimulationName() const override {return GetName();};
	virtual FDataflowSimulationAsset& GetSimulationAsset() override {return SimulationAsset;};
	virtual const FDataflowSimulationAsset& GetSimulationAsset() const override {return SimulationAsset;};
	virtual FDataflowSimulationProxy* GetSimulationProxy() override {return &RigidSolverProxy;}
	virtual const FDataflowSimulationProxy* GetSimulationProxy() const  override {return &RigidSolverProxy;}
	CHAOSSOLVERENGINE_API virtual void BuildSimulationProxy() override;
	CHAOSSOLVERENGINE_API virtual void ResetSimulationProxy() override;
	CHAOSSOLVERENGINE_API virtual void WriteToSimulation(const float DeltaTime, const bool bAsyncTask) override;
	CHAOSSOLVERENGINE_API virtual void ReadFromSimulation(const float DeltaTime, const bool bAsyncTask) override;
	//~ End IDataflowPhysicsSolverInterface interface

	/*
	* Display icon in the editor
	*/
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;

	UChaosGameplayEventDispatcher* GetGameplayEventDispatcher() const { return GameplayEventDispatcherComponent; };

	TSharedPtr<FPhysScene_Chaos> GetPhysicsScene() const { return PhysScene; }
	Chaos::FPhysicsSolver* GetSolver() const { return RigidSolverProxy.Solver; }

	CHAOSSOLVERENGINE_API virtual void PreRegisterAllComponents() override;
	CHAOSSOLVERENGINE_API virtual void PostRegisterAllComponents() override;
	CHAOSSOLVERENGINE_API virtual void PreInitializeComponents() override;
	CHAOSSOLVERENGINE_API virtual void PostUnregisterAllComponents() override;
	
	CHAOSSOLVERENGINE_API virtual void BeginPlay() override;
	CHAOSSOLVERENGINE_API virtual void EndPlay(const EEndPlayReason::Type ReasonEnd) override;
	
	/** UObject interface */
#if WITH_EDITOR
	CHAOSSOLVERENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	CHAOSSOLVERENGINE_API virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif
	CHAOSSOLVERENGINE_API void PostLoad() override;
	CHAOSSOLVERENGINE_API void Serialize(FArchive& Ar) override;
	CHAOSSOLVERENGINE_API void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	/** End UObject interface */

private:

	/* Solver dataflow asset used to advance in time */
	UPROPERTY(EditAnywhere, Category = "Physics", meta=(EditConditionHides), AdvancedDisplay)
	FDataflowSimulationAsset SimulationAsset;

	/** If floor is enabled, make a particle to represent it */
	CHAOSSOLVERENGINE_API void MakeFloor();

	TSharedPtr<FPhysScene_Chaos> PhysScene;

	/** Rigid solver proxy used in dataflow simulation */
	FDataflowRigidSolverProxy RigidSolverProxy;

	/** Ensure that the solver actor has a valid solver and scene */
	void ConditionalCreateSolver();

	/** Migrate the solver onto the right owner (World vs Actor)*/
	void MigrateSolver() const;

	/** Component responsible for harvesting and triggering physics-related gameplay events (hits, breaks, etc) */
	UPROPERTY()
	TObjectPtr<UChaosGameplayEventDispatcher> GameplayEventDispatcherComponent;

	/** If floor is enabled - this will point to the solver particle for it */
	Chaos::FSingleParticlePhysicsProxy* Proxy;
};
