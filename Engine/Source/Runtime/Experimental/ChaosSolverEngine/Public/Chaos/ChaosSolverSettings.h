// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "ChaosSolversModule.h"
#include "ChaosSolverSettings.generated.h"

/** 
 * Settings class for the Chaos Solver
 */

UCLASS(config = Engine, defaultconfig, meta = (DisplayName = "Chaos Solver"), MinimalAPI)
class UChaosSolverSettings : public UDeveloperSettings, public IChaosSolverActorClassProvider
{
	GENERATED_BODY()

public:

	CHAOSSOLVERENGINE_API UChaosSolverSettings();

	// IChaosSolverActorClassProvider interface
	CHAOSSOLVERENGINE_API virtual UClass* GetSolverActorClass() const;

	/** The class to use when auto-creating a default chaos solver actor */
	UPROPERTY(config, noclear, EditAnywhere, Category = GameInstance, meta = (MetaClass = "/Script/ChaosSolverEngine.ChaosSolverActor"))
	FSoftClassPath DefaultChaosSolverActorClass;

#if WITH_EDITOR
	CHAOSSOLVERENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	CHAOSSOLVERENGINE_API virtual void PostInitProperties() override;
	CHAOSSOLVERENGINE_API virtual void PostReloadConfig(class FProperty* PropertyThatWasLoaded) override;

private: 

	// Chaos can't read the properties here as it doesn't depend on engine, the
	// following functions push changes to the chaos module as properties change
	void UpdateProperty(FProperty* InProperty);
	void UpdateAllProperties();
	void RegisterSolverActorProvider();
	//////////////////////////////////////////////////////////////////////////
};
