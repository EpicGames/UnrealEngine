// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Chaos/ChaosVDRemoteSessionsManager.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

namespace Chaos::VD
{
class ITraceDataRelayTransport;
}

namespace UE::TraceBasedDebuggers
{
struct FRemoteSessionsManager;
class IDataRelayTransport;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FChaosVDRemoteSessionsManager;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if WITH_CHAOS_VISUAL_DEBUGGER

#include "ChaosLog.h"
#include "Chaos/ParticleHandleFwd.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "Delegates/IDelegateInstance.h"
#include "EngineEditorBridge.h"

class IPhysicsProxyBase;
class UGameInstance;

namespace Chaos::VisualDebugger
{
struct FChaosVDOptionalDataChannel;
}

/** Object that bridges the gap between the ChaosVD runtime module and the Engine & CVD editor.
 * As the ChaosVDRuntime module does not have access to the engine module, this object reacts to events and performs necessary operation the runtime module cannot do directly
 */
class FChaosVDEngineEditorBridge : public UE::TraceBasedDebuggers::FEngineEditorBridge
{
public:
	CHAOSSOLVERENGINE_API FChaosVDEngineEditorBridge();

	~FChaosVDEngineEditorBridge() = default;

	CHAOSSOLVERENGINE_API static FChaosVDEngineEditorBridge& Get();

	/** @return Whether the static instance has been created (i.e., Get() was called and teardown is required) */
	CHAOSSOLVERENGINE_API static bool IsInstantiated();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use GetSessionsManager instead")
	TSharedPtr<FChaosVDRemoteSessionsManager> GetRemoteSessionsManager()
	{
		return nullptr;
	}

	using FChaosVDDataDataChannel = Chaos::VisualDebugger::FChaosVDOptionalDataChannel;

	UE_DEPRECATED(5.8, "Use GetDataTransportInstance instead")
	TSharedPtr<Chaos::VD::ITraceDataRelayTransport> GetRelayTransportInstance()
	{
		return nullptr;
	}

	UE_DEPRECATED(5.8, "Use SetDataRelayTransportInstance instead")
	CHAOSSOLVERENGINE_API void SetExternalTraceRelayInstance(const TSharedPtr<Chaos::VD::ITraceDataRelayTransport>& InExternalTraceRelayInstance);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:
	virtual void OnInitializeInternal() override;
	virtual void OnTearDownInternal() override;
	virtual void OnRecordingStartedInternal() override;
	virtual void OnRecordingStoppedInternal() override;
	virtual void BuildRecordingStatusInternal(UE::TraceBasedDebuggers::FRecordingStatusMessage& OutStatusMessage) const override;

	void HandleTraceConnectionDetailsUpdated() const;
	bool AddOnScreenRecordingMessage(float DummyDeltaTime = 0.1f);
	void RemoveOnScreenRecordingMessage();
	void HandleCVDRecordingStarted();
	void HandleCVDPostRecordingStarted();
	void HandleCVDRecordingStopped();
	void HandleCVDRecordingStartFailed(const FText& InFailureReason) const;
	void HandlePIEStarted(UGameInstance* GameInstance);

	void HandleDataChannelChanged(TWeakPtr<FChaosVDDataDataChannel> ChannelWeakPtr);

	void SerializeCollisionChannelsNames();

	static FChaosVDParticleMetadata GenerateParticleMetadata(const IPhysicsProxyBase* ParticleProxy, const Chaos::FGeometryParticleHandle* ParticleHandle);

	FDelegateHandle RecordingStartedHandle;
	FDelegateHandle PostRecordingStartedHandle;
	FDelegateHandle RecordingStoppedHandle;
	FDelegateHandle RecordingStartFailedHandle;
	uint64 CVDRecordingMessageKey = 0;

#if WITH_EDITOR
	FDelegateHandle PIEStartedHandle;
#endif

	FTSTicker::FDelegateHandle DeferredShowMessageOnScreenHandle;
	static bool bIsInstantiated;
};

#else

class FChaosVDEngineEditorBridge
{
public:
	CHAOSSOLVERENGINE_API static FChaosVDEngineEditorBridge& Get();

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use GetSessionsManager instead")
	TSharedPtr<FChaosVDRemoteSessionsManager> GetRemoteSessionsManager()
	{
		return nullptr;
	}

	UE_DEPRECATED(5.8, "Use GetDataTransportInstance instead")
	TSharedPtr<Chaos::VD::ITraceDataRelayTransport> GetRelayTransportInstance()
	{
		return nullptr;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager> GetSessionsManager()
	{
		return nullptr;
	}

	TSharedPtr<UE::TraceBasedDebuggers::IDataRelayTransport> GetDataTransportInstance()
	{
		return nullptr;
	}
};

#endif
