// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "ChaosVDRuntimeModule.h"
#include "Containers/Ticker.h"
#include "IMessageContext.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#endif //UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "ChaosVDRecordingDetails.h"
#include "RemoteSessionsManager.h"
#include "SessionInfo.h"
#include "ChaosVDRemoteSessionsManager.generated.h"

#define UE_API CHAOSSOLVERENGINE_API

struct UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FSessionPing instead.") FChaosVDSessionPing : UE::TraceBasedDebuggers::FSessionPing
{
};

struct UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FSessionPong instead.") FChaosVDSessionPong : UE::TraceBasedDebuggers::FSessionPong
{
};

/** Message specific to ChaosVD to send a stop recording command. */
USTRUCT()
struct FChaosVDStopRecordingCommandMessage : public UE::TraceBasedDebuggers::FStopRecordingCommandMessage
{
	GENERATED_BODY()
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FRecordingStatusMessage instead.") FChaosVDRecordingStatusMessage : UE::TraceBasedDebuggers::FRecordingStatusMessage
{
	FChaosVDRecordingStatusMessage()
	{
	}

	FChaosVDRecordingStatusMessage(const FChaosVDRecordingStatusMessage& Other) = default;
	FChaosVDRecordingStatusMessage(FChaosVDRecordingStatusMessage&& Other) noexcept = default;
	FChaosVDRecordingStatusMessage& operator=(const FChaosVDRecordingStatusMessage& Other) = default;
	FChaosVDRecordingStatusMessage& operator=(FChaosVDRecordingStatusMessage&& Other) noexcept = default;

	UE_DEPRECATED(5.7, "Please get the trace details directly from the session info object")
	FChaosVDTraceDetails TraceDetails;
};

struct UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FTraceConnectionDetailsMessage instead.") FChaosVDTraceConnectionDetailsMessage : UE::TraceBasedDebuggers::FTraceConnectionDetailsMessage
{
	FChaosVDTraceDetails TraceDetails;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

USTRUCT()
struct FChaosVDRelayTraceDataMessage
{
	GENERATED_BODY()
	
	FChaosVDRelayTraceDataMessage()
	{
	}

	UPROPERTY()
	FGuid InstanceId = FGuid();
	
	UPROPERTY()
	TArray<uint8> DataBuffer;
};

UENUM()
enum class ERelayThrottlingState : uint8
{
	Inactive,
	Active
};

USTRUCT()
struct FChaosVDRelayTraceStatusMessage
{
	GENERATED_BODY()
	
	FChaosVDRelayTraceStatusMessage()
	{
	}

	UPROPERTY()
	FGuid InstanceId = FGuid();
	
	UPROPERTY()
	int64 QueuedDataBytesNum = 0;

	UPROPERTY()
	ERelayThrottlingState ThrottlingState = ERelayThrottlingState::Inactive;
};

USTRUCT()
struct FChaosVDDataChannelState
{
	GENERATED_BODY()

	UPROPERTY()
	FString ChannelName;

	UPROPERTY()
	bool bIsEnabled = false;

	UPROPERTY()
	bool bCanChangeChannelState = false;

	bool bWaitingUpdatedState = false;
};

USTRUCT()
struct FChaosVDChannelStateChangeCommandMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FChaosVDDataChannelState NewState;
};

USTRUCT()
struct FChaosVDChannelStateChangeResponseMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid InstanceID = FGuid();

	UPROPERTY()
	FChaosVDDataChannelState NewState;
};

/**
 * Structure specific to ChaosVD to store the states of the data channels.
 * The struct is stored as the DebuggerSpecificData for the message type UE::TraceBasedDebuggers::FFullSessionInfoResponseMessage.
 */
USTRUCT()
struct FChaosVDFullSessionInfoResponseData
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FChaosVDDataChannelState> DataChannelsStates;
};

/**
 * Structure specific to ChaosVD to store the states of the data channels in the session info struct.
 * The struct is stored as a DebuggerSpecificSessionData inside UE::TraceBasedDebuggers::FSessionInfo
 * and get be retrieved using GetDebuggerData<FChaosVDSessionData>().
 */
USTRUCT()
struct FChaosVDSessionData
{
	GENERATED_BODY()

	TMap<FString, FChaosVDDataChannelState> DataChannelsStatesByName;
};

#if WITH_TRACE_BASED_DEBUGGERS
/**
 * Remote sessions handler to handle ChaosVD specific message types.
 */
struct FChaosVDRemoteSessionsHandler : UE::TraceBasedDebuggers::IRemoteSessionsHandler
{
	UE_API virtual void OnCreatingMessageEndpoint(const TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager>&, FMessageEndpointBuilder&) override;
	UE_API virtual void OnSubscribingMessageTypes(const TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager>&, FMessageEndpoint&) override;
	UE_API virtual void OnBuildingFullSessionInfoResponseMessage(UE::TraceBasedDebuggers::FFullSessionInfoResponseMessage&) override;
	UE_API virtual void OnHandlingFullSessionInfoResponseMessage(const UE::TraceBasedDebuggers::FFullSessionInfoResponseMessage&, const TSharedPtr<UE::TraceBasedDebuggers::FSessionInfo>&) override;
};
#endif //WITH_TRACE_BASED_DEBUGGERS

PRAGMA_DISABLE_DEPRECATION_WARNINGS

struct UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FFullSessionInfoRequestMessage instead.") FChaosVDFullSessionInfoRequestMessage
{
};

struct UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FFullSessionInfoResponseMessage instead.") FChaosVFFullSessionInfoResponseMessage : UE::TraceBasedDebuggers::FFullSessionInfoResponseMessage
{
	TArray<FChaosVDDataChannelState> DataChannelsStates;
};

enum class UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::ERemoteSessionAttributes instead.") EChaosVDRemoteSessionAttributes
{
	None = 0,
	SupportsDataChannelChange = 1 << 0,
	CanExpire = 1 << 1,
	IsMultiSessionWrapper = 1 << 2,
};
ENUM_CLASS_FLAGS(EChaosVDRemoteSessionAttributes)

enum class UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::ERemoteSessionReadyState instead.") EChaosVDRemoteSessionReadyState : uint8
{
	Ready,
	Busy
};

struct UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FSessionInfo instead.") FChaosVDSessionInfo
{
	explicit FChaosVDSessionInfo() : InstanceId(FGuid()),
									SessionTypeAttributes(EChaosVDRemoteSessionAttributes::CanExpire | EChaosVDRemoteSessionAttributes::SupportsDataChannelChange)
	{
	}

	virtual ~FChaosVDSessionInfo() = default;

protected:

	explicit FChaosVDSessionInfo(EChaosVDRemoteSessionAttributes InSessionTypeAttributes) : SessionTypeAttributes(InSessionTypeAttributes)
	{
	}

public:

	FGuid InstanceId;
	FString SessionName;
	FMessageAddress Address;
	FDateTime LastPingTime;
	EBuildTargetType BuildTargetType = EBuildTargetType::Unknown;
	EChaosVDRemoteSessionReadyState ReadyState = EChaosVDRemoteSessionReadyState::Ready;

	FChaosVDRecordingStatusMessage LastKnownRecordingState;

	TMap<FString, FChaosVDDataChannelState> DataChannelsStatesByName;

	CHAOSSOLVERENGINE_API const FChaosVDTraceDetails& GetConnectionDetails();

	CHAOSSOLVERENGINE_API virtual bool IsRecording() const;
	CHAOSSOLVERENGINE_API virtual uint64 GetBufferedBytesNum() const;

	UE_DEPRECATED(5.8, "This method will no longer be used and there is not a replacement planned.")
	CHAOSSOLVERENGINE_API virtual EChaosVDRecordingMode GetRecordingMode() const;
	UE_DEPRECATED(5.8, "This method will no longer be used and there is not a replacement planned.")
	CHAOSSOLVERENGINE_API virtual EChaosVDRecordingMode GetLastRequestedRecordingMode() const final;
	UE_DEPRECATED(5.8, "This method will no longer be used and there is not a replacement planned.")
	CHAOSSOLVERENGINE_API virtual void SetLastRequestedRecordingMode(EChaosVDRecordingMode NewRecordingMode) final;


	UE_DEPRECATED(5.7, "This method will no longer be used and there is not a replacement planned.")
	virtual bool IsConnected() const final;

	EChaosVDRemoteSessionAttributes GetSessionTypeAttributes() const
	{
		return SessionTypeAttributes;
	}

	CHAOSSOLVERENGINE_API void SetReceivedBytesPerSecond(uint64 InNewBytesPerSecond);
	CHAOSSOLVERENGINE_API uint64 GetReceivedBytesPerSecond() const;

protected:
	UE_DEPRECATED(5.8, "This method will no longer be used and there is not a replacement planned.")
	EChaosVDRecordingMode LastRequestedRecordingMode = EChaosVDRecordingMode::Invalid;
	FChaosVDTraceDetails LastKnownConnectionDetails;

	uint64 ReceivedBytesPerSecond = 0;
	
	const EChaosVDRemoteSessionAttributes SessionTypeAttributes;
};

struct UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FMultiSessionInfo instead.")  FChaosVDMultiSessionInfo : public FChaosVDSessionInfo
{
	explicit FChaosVDMultiSessionInfo() : FChaosVDSessionInfo(EChaosVDRemoteSessionAttributes::IsMultiSessionWrapper)
	{
	}

	virtual ~FChaosVDMultiSessionInfo() override = default;

	CHAOSSOLVERENGINE_API virtual bool IsRecording() const override;
	CHAOSSOLVERENGINE_API virtual uint64 GetBufferedBytesNum() const override;

	UE_DEPRECATED(5.8, "This method will no longer be used and there is not a replacement planned.")
	CHAOSSOLVERENGINE_API virtual EChaosVDRecordingMode GetRecordingMode() const override;

	template<typename TCallback>
	void EnumerateInnerSessions(const TCallback& Callback) const
	{
		for (const TPair<FGuid, TWeakPtr<FChaosVDSessionInfo>>& InnerSessionWithID : InnerSessionsByInstanceID)
		{
			if (const TSharedPtr<FChaosVDSessionInfo> SessionPtr = InnerSessionWithID.Value.Pin())
			{
				if (!Callback(SessionPtr.ToSharedRef()))
				{
					return;
				}
			}
		}
	}

	TMap<FGuid, TWeakPtr<FChaosVDSessionInfo>> InnerSessionsByInstanceID;
};

DECLARE_LOG_CATEGORY_EXTERN(LogChaosVDRemoteSession, Log, VeryVerbose);

UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FRecordingStateChangeDelegate()")
DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDRecordingStateChangeDelegate, TWeakPtr<FChaosVDSessionInfo> Session)

UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FSessionLifetimeDelegate()")
DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDSessionDiscoverDelegate, FGuid SessionID)

/** Object that is able to discover, issue and execute commands back and forth between CVD and client/server/editor instances */
class UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FRemoteSessionsManager instead.") FChaosVDRemoteSessionsManager : public UE::TraceBasedDebuggers::FRemoteSessionsManager
{
public:
	FChaosVDRemoteSessionsManager();

	UE_DEPRECATED(5.7, "Use Initialize()")
	CHAOSSOLVERENGINE_API void Initialize(const TSharedPtr<IMessageBus>& InMessageBus);

	FChaosVDSessionDiscoverDelegate& OnSessionDiscovered()
	{
		return SessionDiscoveredDelegate;
	}

	FChaosVDSessionDiscoverDelegate& OnSessionExpired()
	{
		return SessionExpiredDelegate;
	}

	/**
	 * Issues a command to the provided address that will start a CVD recording
	 * @param InDestinationAddress Message bus address that will execute the command
	 * @param RecordingStartCommandParams Desired parameters to be used to start the recording 
	 */
	CHAOSSOLVERENGINE_API void SendStartRecordingCommand(const FMessageAddress& InDestinationAddress, const FChaosVDStartRecordingCommandMessage& RecordingStartCommandParams);

	/**
	 * Issues a command to the provided address to change the state of a data channel
	 * 	@param InDestinationAddress Message bus address that will execute the command
	 * 	@param InNewStateData New data channel state to apply in the receiving instance
	 */
	CHAOSSOLVERENGINE_API void SendDataChannelStateChangeCommand(const FMessageAddress& InDestinationAddress, const FChaosVDChannelStateChangeCommandMessage& InNewStateData);

	/**
	 * Returns the session info object for the provided ID 
	 * @param Id CVD SessionID
	 */
	CHAOSSOLVERENGINE_API TWeakPtr<FChaosVDSessionInfo> GetSessionInfo(FGuid Id);

	/**
	 * Returns the first available session info 
	 */
	CHAOSSOLVERENGINE_API TWeakPtr<FChaosVDSessionInfo> GetFirstAvailableSingleSessionInfo();

	/**
	 * Iterates through all active and valid cvd sessions, and executes the provided callback to it.
	 * if the callbacks returns false, the iteration will stop
	* */
	template<typename CallbackType>
	void EnumerateActiveSessions(const CallbackType& Callback)
	{
		for (const TPair<FGuid, TSharedPtr<FChaosVDSessionInfo>>& ActiveSession : ActiveSessionsByInstanceId)
		{
			if (ActiveSession.Value)
			{
				bool bContinue = Callback(ActiveSession.Value.ToSharedRef());

				if (!bContinue)
				{
					return;
				}
			}
		}
	}

	/**
	 * Broadcast to the network a recording state update
	 * @param InUpdateMessage latest recording state of the issuing instance
	 */
	void PublishRecordingStatusUpdate(const FChaosVDRecordingStatusMessage& InUpdateMessage);

	/**
	 * Broadcast to the network new trace connection details as they become available
	 * @param InUpdateMessage latest connection details
	 */
	void PublishTraceConnectionDetailsUpdate(const FChaosVDTraceConnectionDetailsMessage& InUpdateMessage);
	
	/**
	 * Broadcast to the network a data channel state update
	 * @param InNewStateData latest data channel state of the issuing instance
	 */
	void PublishDataChannelStateChangeUpdate(const FChaosVDChannelStateChangeResponseMessage& InNewStateData);

	/**
	 * Delegate that broadcast when a recording was started in a session (either local or remote)
	 */
	FChaosVDRecordingStateChangeDelegate& OnSessionRecordingStarted()
	{
		return RecordingStartedDelegate;
	}

	/**
	 * Delegate that broadcast when a recording stops in a session (either local or remote)
	 */
	FChaosVDRecordingStateChangeDelegate& OnSessionRecordingStopped()
	{
		return RecordingStoppedDelegate;
	}

private:
	TMap<FGuid, TSharedPtr<FChaosVDSessionInfo>> ActiveSessionsByInstanceId;

	FChaosVDSessionDiscoverDelegate SessionDiscoveredDelegate;
	FChaosVDSessionDiscoverDelegate SessionExpiredDelegate;

	FChaosVDRecordingStateChangeDelegate RecordingStartedDelegate;
	FChaosVDRecordingStateChangeDelegate RecordingStoppedDelegate;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef UE_API