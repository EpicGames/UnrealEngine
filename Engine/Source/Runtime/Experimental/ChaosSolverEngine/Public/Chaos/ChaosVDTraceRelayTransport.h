// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/CoreMiscDefines.h"

namespace Chaos::VD
{

/** Structure containing the details necessary to connect to a relay instance */
struct UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FRelayConnectionInfo instead") FRelayConnectionInfo
{
	FString Address;
	uint16 Port = 0;
	TArray<uint8> CertificateAuthority;
};
}

#if WITH_CHAOS_VISUAL_DEBUGGER
#include "Delegates/Delegate.h"

struct FGuid;

namespace Chaos::VD
{
class FRelayTraceWriter;

/** Possible connection attempt results */
enum UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::EConnectionAttemptResult instead") EConnectionAttemptResult
{
	NotStarted,
	InProgress,
	Success,
	Failed
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
/**
 * Interface for any CVD trace relay transport implementation
 */
class UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::IDataRelayTransport instead") ITraceDataRelayTransport
{
public:
	virtual ~ITraceDataRelayTransport() = default;

	virtual void Initialize() = 0;
	virtual void Shutdown() = 0;

	DECLARE_DELEGATE_OneParam(FProcessReceivedRelayDataDelegate, const TConstArrayView<uint8> DataPacket);
	virtual void RegisterRelayDataReceiverForSessionID(FGuid RemoteSessionID, const FProcessReceivedRelayDataDelegate& InProcessDataDelegate) = 0;
	virtual void UnregisterRelayDataReceiverForSessionID(FGuid RemoteSessionID) = 0;

	virtual void RelayTraceDataFromWriter(FRelayTraceWriter& InRelayTraceWriter) = 0;

	virtual FRelayConnectionInfo GetConnectionInfo() = 0;
	virtual EConnectionAttemptResult ConnectToRelay(FGuid RemoteSessionID, const FRelayConnectionInfo ConnectionInfo) = 0;
	virtual EConnectionAttemptResult GetConnectionAttemptResult(FGuid RemoteSessionID) = 0;
};

}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_CHAOS_VISUAL_DEBUGGER