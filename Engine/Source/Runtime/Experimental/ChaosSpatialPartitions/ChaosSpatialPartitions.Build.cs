// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using System.Collections.Generic;

namespace UnrealBuildTool.Rules
{
	public class ChaosSpatialPartitions : ModuleRules
	{
		public ChaosSpatialPartitions(ReadOnlyTargetRules Target) : base(Target)
		{
			BinariesSubFolder = "NoRedist";

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"ChaosCore",
				}
			);

			// Add the tests directory to our private include paths so we can include common test objects.
			PrivateIncludePaths.Add(TestsDirectory);

			PublicDefinitions.Add("COMPILE_WITHOUT_UNREAL_SUPPORT=0");
			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");

			if (Target.bUseChaosMemoryTracking == true)
			{
				PublicDefinitions.Add("CHAOS_MEMORY_TRACKING=1");
			}
			else
			{
				PublicDefinitions.Add("CHAOS_MEMORY_TRACKING=0");
			}

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
