// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSpatialPartitions/Algorithms/AabbTreeAlgorithm.h"

namespace Chaos::SpatialPartition::AabbTreeAlgorithm
{
	FAABB3 Union(const FAABB3& Aabb0, const FAABB3& Aabb1)
	{
		FAABB3 Result = Aabb0;
		Result.GrowToInclude(Aabb1);
		return Result;
	}

	FReal ComputeDeltaSurfaceArea(const FAABB3& OldAabb, const FAABB3& AabbToInclude)
	{
		const FReal OldArea = OldAabb.GetArea();
		const FAABB3 CombinedAabb = Union(OldAabb, AabbToInclude);
		const FReal NewArea = CombinedAabb.GetArea();
		return NewArea - OldArea;
	}

	int32 AllocateNode(TArray<FAabbTreeNode>& Nodes, int32& FreeListHead)
	{
		FAabbTreeNode Node;
#ifdef UE_ENABLE_CHAOS_AABBTREE_VISUALIZER
		Node.DebugNodeList = &Nodes;
#endif

		if (FreeListHead != INDEX_NONE)
		{
			int32 Index = FreeListHead;
			FreeListHead = Nodes[FreeListHead].Right;
			Nodes[Index] = Node;
			return Index;
		}
		else
		{
			return Nodes.Emplace(Node);
		}
	}

	void DeallocateNode(TArray<FAabbTreeNode>& Nodes, int32& FreeListHead, int32 IndexToFree)
	{
		Nodes[IndexToFree].Right = FreeListHead;
		FreeListHead = IndexToFree;
	}

	bool IsLeaf(const FAabbTreeNode& Node)
	{
		return Node.Left == INDEX_NONE;
	}

	int32& GetIndexRef(FAabbTreeNode& ParentNode, const int32 NodeIndex)
	{
		return (ParentNode.Left == NodeIndex) ? ParentNode.Left : ParentNode.Right;
	}

	int32& GetSiblingRef(FAabbTreeNode& ParentNode, const int32 NodeIndex)
	{
		return (ParentNode.Left == NodeIndex) ? ParentNode.Right : ParentNode.Left;
	}

	void RecomputeAabb(TArray<FAabbTreeNode>& Nodes, const int32 Index)
	{
		check(Index != INDEX_NONE);
		// Only valid to update the bounds of a non-leaf node
		check(!AabbTreeAlgorithm::IsLeaf(Nodes[Index]));

		FAabbTreeNode& Node = Nodes[Index];
		const FAabbTreeNode& Left = Nodes[Node.Left];
		const FAabbTreeNode& Right = Nodes[Node.Right];
		Node.Aabb = AabbTreeAlgorithm::Union(Left.Aabb, Right.Aabb);
	}

	bool RotateNodes(TArray<FAabbTreeNode>& Nodes, const int32 AIndex)
	{
		check(AIndex < Nodes.Num());
		// Given the Tree:
		//    A
		//  B   C
		// D E F G
		FAabbTreeNode& A = Nodes[AIndex];
		if (IsLeaf(A))
		{
			return false;
		}

		const int32 BIndex = A.Left;
		const int32 CIndex = A.Right;
		FAabbTreeNode& B = Nodes[BIndex];
		FAabbTreeNode& C = Nodes[CIndex];
		const bool bBIsLeaf = IsLeaf(B);
		const bool bCIsLeaf = IsLeaf(C);
		if (bBIsLeaf && bCIsLeaf)
		{
			return false;
		}
		else if (bBIsLeaf)
		{
			const int32 FIndex = C.Left;
			const int32 GIndex = C.Right;
			FAabbTreeNode& F = Nodes[FIndex];
			FAabbTreeNode& G = Nodes[GIndex];
			const FReal BaseCost = C.Aabb.GetArea();
			const FAABB3 AabbBG = Union(B.Aabb, G.Aabb);
			const FAABB3 AabbBF = Union(B.Aabb, F.Aabb);
			// Note: The cost of swapping BF only changes the area of node C which would now be BG.
			const FReal BFCost = AabbBG.GetArea();
			const FReal BGCost = AabbBF.GetArea();
			if (BaseCost <= BFCost && BaseCost <= BGCost)
			{
				return false;
			}

			if (BFCost < BGCost)
			{
				// swap b and f
				F.Parent = AIndex;
				A.Left = FIndex;

				B.Parent = CIndex;
				C.Left = BIndex;

				C.Aabb = AabbBG;
			}
			else
			{
				// swap b and g
				G.Parent = AIndex;
				A.Left = GIndex;

				B.Parent = CIndex;
				C.Right = BIndex;

				C.Aabb = AabbBF;
			}
			return true;
		}
		else if (bCIsLeaf)
		{
			const int32 DIndex = B.Left;
			const int32 EIndex = B.Right;
			FAabbTreeNode& D = Nodes[DIndex];
			FAabbTreeNode& E = Nodes[EIndex];
			const FReal BaseCost = B.Aabb.GetArea();
			const FAABB3 AabbCD = Union(C.Aabb, D.Aabb);
			const FAABB3 AabbCE = Union(C.Aabb, E.Aabb);
			const FReal CDCost = AabbCE.GetArea();
			const FReal CECost = AabbCD.GetArea();
			// Make sure to use `<=` so we don't do a rotation unless it's strictly better
			if (BaseCost <= CDCost && BaseCost <= CECost)
			{
				return false;
			}

			if (CDCost < CECost)
			{
				// swap C and D
				D.Parent = AIndex;
				A.Right = DIndex;

				C.Parent = BIndex;
				B.Left = CIndex;

				B.Aabb = AabbCE;
			}
			else
			{
				// swap C and E
				E.Parent = AIndex;
				A.Right = EIndex;

				C.Parent = BIndex;
				B.Right = CIndex;
				B.Aabb = AabbCD;
			}
			return true;
		}
		else
		{
			// BF, BG, CD, CE
			const int32 DIndex = B.Left;
			const int32 EIndex = B.Right;
			const int32 FIndex = C.Left;
			const int32 GIndex = C.Right;
			FAabbTreeNode& D = Nodes[DIndex];
			FAabbTreeNode& E = Nodes[EIndex];
			FAabbTreeNode& F = Nodes[FIndex];
			FAabbTreeNode& G = Nodes[GIndex];

			const FAABB3 AabbBG = Union(B.Aabb, G.Aabb);
			const FAABB3 AabbBF = Union(B.Aabb, F.Aabb);
			const FAABB3 AabbCD = Union(C.Aabb, D.Aabb);
			const FAABB3 AabbCE = Union(C.Aabb, E.Aabb);
			const FReal AreaB = B.Aabb.GetArea();
			const FReal AreaC = C.Aabb.GetArea();
			const FReal BaseCost = AreaB + AreaC;
			const FReal CostBF = AreaB + AabbBG.GetArea();
			const FReal CostBG = AreaB + AabbBF.GetArea();
			const FReal CostCD = AreaC + AabbCE.GetArea();
			const FReal CostCE = AreaC + AabbCD.GetArea();
			if (BaseCost <= CostBF && BaseCost <= CostBG && BaseCost <= CostCD && BaseCost <= CostCE)
			{
				return false;
			}

			if (CostBF <= CostBG && CostBF <= CostCD && CostBF <= CostCE)
			{
				// swap b and f
				F.Parent = AIndex;
				A.Left = FIndex;

				B.Parent = CIndex;
				C.Left = BIndex;

				C.Aabb = AabbBG;
				return true;
			}
			if (CostBG <= CostBF && CostBG <= CostCD && CostBG <= CostCE)
			{
				// swap b and g
				G.Parent = AIndex;
				A.Left = GIndex;

				B.Parent = CIndex;
				C.Right = BIndex;

				C.Aabb = AabbBF;
				return true;
			}
			if (CostCD <= CostBF && CostCD <= CostBG && CostCD <= CostCE)
			{
				// swap C and D
				D.Parent = AIndex;
				A.Right = DIndex;

				C.Parent = BIndex;
				B.Left = CIndex;

				B.Aabb = AabbCE;
				return true;
			}
			if (CostCE <= CostBF && CostCE <= CostBG && CostCE <= CostCD)
			{
				// swap C and E
				E.Parent = AIndex;
				A.Right = EIndex;

				C.Parent = BIndex;
				B.Right = CIndex;
				B.Aabb = AabbCD;
				return true;
			}
		}

		return false;
	}

	void RecomputeAncestorAabbsAndRotate(TArray<FAabbTreeNode>& Nodes, int32 Index, bool bDoRotation)
	{
		// Only valid to update the bounds of a non-leaf node
		check(Index != INDEX_NONE && !AabbTreeAlgorithm::IsLeaf(Nodes[Index]));

		while (Index != INDEX_NONE)
		{
			RecomputeAabb(Nodes, Index);

			// TODO: Investigate the bool here. Legacy sometimes doesn't do the rotation, but it's unclear why.
			if (bDoRotation)
			{
				AabbTreeAlgorithm::RotateNodes(Nodes, Index);
			}
			Index = Nodes[Index].Parent;
		}
	}

	int32 SelectChildNodeBySurfaceArea(const TArray<FAabbTreeNode>& Nodes, const FAabbTreeNode& Node, const FAABB3& Aabb)
	{
		const FReal DeltaLeftArea = ComputeDeltaSurfaceArea(Nodes[Node.Left].Aabb, Aabb);
		const FReal DeltaRightArea = ComputeDeltaSurfaceArea(Nodes[Node.Right].Aabb, Aabb);
		if (DeltaLeftArea < DeltaRightArea)
		{
			return Node.Left;
		}
		return Node.Right;
	}

	int32 FindBestSiblingGreedy(const TArray<FAabbTreeNode>& Nodes, const int32 RootIndex, const FAABB3& NewAabb)
	{
		if (RootIndex == INDEX_NONE)
		{
			return INDEX_NONE;
		}

		int32 Index = RootIndex;
		const FAabbTreeNode* Node = &Nodes[Index];
		while (!IsLeaf(*Node))
		{
			Index = SelectChildNodeBySurfaceArea(Nodes, *Node, NewAabb);
			Node = &Nodes[Index];
		}
		return Index;
	}

	int32 FindBestSiblingAdvancedGreedySAH(const TArray<FAabbTreeNode>& Nodes, const int32 RootIndex, const FAABB3& NewAabb)
	{
		// This algorithm is a better greedy surface area heuristic than above. 
		// It effectively does the same calculation as the global algorithm, 
		// but will only ever traverse down one side of the tree (it keeps track of the SA being added at every step).
		// In general, when trying to insert Aabb N at the sub-tree A(B,C), we have to check both sides to see which is better.
		// Without loss of generality, if we go down the left side of the tree, there's two cases to consider:
		// 1. B is a leaf node.
		// 2. B is an internal node.
		// In case 1, the new tree would be: A(P(B,N), C). This tree's surface area grows by: deltaA + areaP.
		// Note: Technically we also add areaN, however this is a constant that will always be added, so we can ignore it.
		// In a more general case where we are deeper down the tree, deltaA is actually all of the delta's to get to our current node, so the full cost is:
		// deltaA0 + deltaA1 + ... deltaAN + areaP = inheritedCost + areaP.
		// Case 2 is trickier and can be broken up into 2 sub-cases:
		// 2a. We pair N up with B: A(P(B(...),N), C)
		// 2b. We push N down into a child of B: A(B(...,P(...,N)), C)
		// We can compute the lower bound of these two costs to estimate which branch is better to go down.
		// The lower bound cost of 2a is the same as the cost of 1: inheritedCost + areaP.
		// For 2b, the lower bound is when we do not increase any of the aabbs under B and P's SA is as small as possible, that is when P is equal to N.
		// So the lower bound for 2b is: deltaA + deltaB + areaP.
		// Doing a little math, we can re-arrange this to: deltaA + areaBN - areaB + areaN = deltaA + areaP + (areaBN - areaB) = inheritedCost + areaP + (areaBN - areaB).
		// So the lower bound is the min(2a, 2b) = min(inheritedCost + areaP, inheritedCost + areaP + (areaBN - areaB)) = inheritedCost + areaP + min(0, areaBN - areaB).
		if (RootIndex == INDEX_NONE)
		{
			return INDEX_NONE;
		}

		const FReal NewAabbArea = NewAabb.GetArea();
		const FVec3 NewAabbCenter = NewAabb.GetCenter();

		FReal InheritedCost = 0;
		FReal BestCost = std::numeric_limits<FReal>::max();
		int32 BestIndex = RootIndex;
		int32 Index = RootIndex;

		const FAabbTreeNode& RootNode = Nodes[RootIndex];
		const FAABB3 UnionAabb = Union(RootNode.Aabb, NewAabb);
		FReal BaseArea = RootNode.Aabb.GetArea();
		FReal DirectCost = UnionAabb.GetArea();

		while (!IsLeaf(Nodes[Index]))
		{
			const FReal DeltaCost = DirectCost - BaseArea;

			const FAabbTreeNode& Node = Nodes[Index];
			const FAabbTreeNode& LeftNode = Nodes[Node.Left];
			const FAabbTreeNode& RightNode = Nodes[Node.Right];

			// Always check the cost for the current node. This allows us to pick the root / internal nodes
			const FReal TotalCost = InheritedCost + DirectCost;
			if (TotalCost < BestCost)
			{
				BestIndex = Index;
				BestCost = TotalCost;
			}

			InheritedCost += DeltaCost;

			const bool bLeftIsLeaf = IsLeaf(LeftNode);
			const FAABB3 LeftUnionAabb = Union(LeftNode.Aabb, NewAabb);
			const FReal LeftDirectCost = LeftUnionAabb.GetArea();
			const FReal LeftBaseArea = LeftNode.Aabb.GetArea();
			FReal LeftLowerBound = std::numeric_limits<FReal>::max();
			if (bLeftIsLeaf)
			{
				// Case 1: Left is leaf: Cost = inheritedCost + directCost.
				const FReal LeftTotalCost = InheritedCost + LeftDirectCost;
				if (LeftTotalCost < BestCost)
				{
					BestCost = LeftTotalCost;
					BestIndex = Node.Left;
				}
			}
			else
			{
				// Case 2: Left is internal: lower bound is minimum of 2a and 2b.
				LeftLowerBound = InheritedCost + LeftDirectCost + FMath::Min(NewAabbArea - LeftBaseArea, 0);
			}

			const bool bRightIsLeaf = IsLeaf(RightNode);
			const FAABB3 RightUnionAabb = Union(RightNode.Aabb, NewAabb);
			const FReal RightDirectCost = RightUnionAabb.GetArea();
			const FReal RightBaseArea = RightNode.Aabb.GetArea();
			FReal RightLowerBound = std::numeric_limits<FReal>::max();
			if (bRightIsLeaf)
			{
				// Case 1: Right is leaf: Cost = inheritedCost + directCost.
				const FReal RightTotalCost = InheritedCost + RightDirectCost;
				if (RightTotalCost < BestCost)
				{
					BestCost = RightTotalCost;
					BestIndex = Node.Right;
				}
			}
			else
			{
				// Case 2: Right is internal: lower bound is minimum of 2a and 2b.
				RightLowerBound = InheritedCost + RightDirectCost + FMath::Min(NewAabbArea - RightBaseArea, 0);
			}

			// If both nodes are leaves, there's no point in checking sides and we can early out.
			if (bLeftIsLeaf && bRightIsLeaf)
			{
				break;
			}

			// If our current best cost is lower than the lower bound of both sides, we can't possibly do better.
			if (BestCost < LeftLowerBound && BestCost < RightLowerBound)
			{
				break;
			}

			// If there's a tie, switch to a distance heuristic to decide which side to traverse down.
			// This can happen if both leaves contain the new aabb.
			if (LeftLowerBound == RightLowerBound)
			{
				const FVec3 LeftDistanceVec = LeftNode.Aabb.GetCenter() - NewAabbCenter;
				const FVec3 RightDistanceVec = RightNode.Aabb.GetCenter() - NewAabbCenter;
				LeftLowerBound = LeftDistanceVec.SquaredLength();
				RightLowerBound = RightDistanceVec.SquaredLength();
			}

			// Greedily search down the side with the better lower bound
			if (LeftLowerBound < RightLowerBound)
			{
				Index = Node.Left;
				DirectCost = LeftDirectCost;
				BaseArea = LeftBaseArea;
			}
			else
			{
				Index = Node.Right;
				DirectCost = RightDirectCost;
				BaseArea = RightBaseArea;
			}
		}
		return BestIndex;
	}

	void GlobalSearch(const TArray<FAabbTreeNode>& Nodes, const FAABB3& NewAabb, int32 NodeIndex, FReal InheritedCost, FReal& BestCost, int32& BestIndex)
	{
		if (NodeIndex == INDEX_NONE)
		{
			return;
		}

		const FAabbTreeNode& Node = Nodes[NodeIndex];
		const FAABB3 UnionAabb = Union(Node.Aabb, NewAabb);
		const FReal DirectCost = UnionAabb.GetArea();
		const FReal DeltaCost = DirectCost - Node.Aabb.GetArea();
		const FReal TotalCost = DirectCost + InheritedCost;
		if (TotalCost < BestCost)
		{
			BestCost = TotalCost;
			BestIndex = NodeIndex;
		}

		InheritedCost += DeltaCost;
		GlobalSearch(Nodes, NewAabb, Node.Left, InheritedCost, BestCost, BestIndex);
		GlobalSearch(Nodes, NewAabb, Node.Right, InheritedCost, BestCost, BestIndex);
	}

	int32 FindBestSiblingGlobalSearch(const TArray<FAabbTreeNode>& Nodes, const int32 RootIndex, const FAABB3& NewAabb)
	{
		if (RootIndex == INDEX_NONE)
		{
			return INDEX_NONE;
		}

		// Prime the cost with the root being the best initial node.
		// Note: This technically is recomputing the root cost twice, however the global search algorithm is mostly for testing.
		const FAabbTreeNode& Root = Nodes[RootIndex];
		FReal BestCost = Union(Root.Aabb, NewAabb).GetArea();
		FReal InheritedCost = 0;
		int32 BestIndex = RootIndex;

		GlobalSearch(Nodes, NewAabb, RootIndex, InheritedCost, BestCost, BestIndex);
		return BestIndex;
	}

	int32 FindBestSiblingBranchAndBound(const TArray<FAabbTreeNode>& Nodes, const int32 RootIndex, const FAABB3& NewAabb)
	{
		TArray<TPair<int32, FReal>> PriorityQueue;
		// Estimate how much memory we'll need from the height of the tree.
		// A multiple of 2 is used since the algorithm may go down multiple branches.
		const int32 Height = FMath::CeilToInt32(FMath::Log2((float)Nodes.Num()));
		PriorityQueue.Reserve(2 * Height);
		return FindBestSiblingBranchAndBound(Nodes, RootIndex, NewAabb, PriorityQueue);
	}

	// Wrapper class to experiment with different traversal orderings (queue, priority queue, stack, etc...)
	struct FCostPriorityQueue
	{
		using FPair = TPair<int32, FReal>;
		enum ETraversalMode { Queue, PriorityQueue, Stack };
		static constexpr ETraversalMode TraversalMode = ETraversalMode::Queue;

		FCostPriorityQueue(TArray<FPair>& Array)
			: Array(Array)
		{
		}

		void Reset()
		{
			Index = 0;
			Array.Reset();
		}

		void Push(int32 NodeIndex, FReal Cost)
		{
			if (TraversalMode == ETraversalMode::Queue)
			{
				Array.Push(FPair{ NodeIndex, Cost });
			}
			else if (TraversalMode == ETraversalMode::PriorityQueue)
			{
				Array.HeapPush(FPair{ NodeIndex, Cost }, Compare);
			}
			else
			{
				Array.Push(FPair{ NodeIndex, Cost });
			}
		}

		FPair Pop()
		{
			if (TraversalMode == ETraversalMode::Queue)
			{
				FPair Result = Array[Index];
				++Index;
				return Result;
			}
			else if (TraversalMode == ETraversalMode::PriorityQueue)
			{
				FPair Result;
				Array.HeapPop(Result, Compare);
				return Result;
			}
			else
			{
				FPair Result = Array.Pop();
				return Result;
			}
		}

		bool IsEmpty() const
		{
			if (TraversalMode == ETraversalMode::Queue)
			{
				return Index >= Array.Num();
			}
			else if (TraversalMode == ETraversalMode::PriorityQueue)
			{
				return Array.IsEmpty();
			}
			else
			{
				return Array.IsEmpty();
			}
		}

		static bool Compare(const FPair& Lhs, const FPair& Rhs)
		{
			return Lhs.Value < Rhs.Value;
		}

		int32 Index = 0;
		TArray<FPair>& Array;
	};

	int32 FindBestSiblingBranchAndBound(const TArray<FAabbTreeNode>& Nodes, const int32 RootIndex, const FAABB3& NewAabb, TArray<TPair<int32, FReal>>& Queue)
	{
		// The cost for inserting a new node at the current node is how much surface area we're adding to the tree. There's two kinds of SA increases:
		// 1. The inherited cost: This is how much every parent aabb has been enlarged to reach here.
		// 2. The direct cost: This is the new parent node that has to be added.
		// The inherited cost is computed as the delta area from the old to new aabb. 
		// The direct cost is the new node added which will have an aabb that is the union of the two children.
		// Note: The new node's SA is constant so it can be ignored from the calculations.
		// At any point, we need to decide if we should traverse into the children. 
		// We can compute a lower bound for our search and terminate if that's not better than the current best cost. 
		// The smallest possible new parent node we could add is one that has the SA of the new node, so the lower bound is just: SA(NewNode) + InheritedCost.
		const FReal NewAabbArea = NewAabb.GetArea();

		// Worst case is the root grows by the new aabb
		FReal BestCost = Union(Nodes[RootIndex].Aabb, NewAabb).GetArea();
		int32 BestIndex = RootIndex;

		// TODO: Investigate this algorithm a bit more. Branch and bound seems to be breadth first in how it traverses nodes.
		// It's possible that doing a stack approach, or even priming with one greedy search will yield large early outs.
		FCostPriorityQueue PriorityQueue(Queue);
		PriorityQueue.Reset();
		PriorityQueue.Push(RootIndex, 0);
		while (!PriorityQueue.IsEmpty())
		{
			const FCostPriorityQueue::FPair Pair = PriorityQueue.Pop();
			const int32 NodeIndex = Pair.Key;
			const FReal InheritedCost = Pair.Value;

			const FAabbTreeNode& Node = Nodes[NodeIndex];
			const FAABB3 UnionAabb = Union(Node.Aabb, NewAabb);
			const FReal UnionArea = UnionAabb.GetArea();
			const FReal Cost = UnionArea + InheritedCost;

			if (Cost < BestCost)
			{
				BestCost = Cost;
				BestIndex = NodeIndex;
			}

			if (!IsLeaf(Node))
			{
				const FReal OriginalArea = Node.Aabb.GetArea();
				const FReal DeltaArea = UnionArea - OriginalArea;
				const FReal NewInheritedCost = InheritedCost + DeltaArea;
				const FReal LowerBound = NewAabbArea + NewInheritedCost;
				if (LowerBound < BestCost)
				{
					PriorityQueue.Push(Node.Left, NewInheritedCost);
					PriorityQueue.Push(Node.Right, NewInheritedCost);
				}
			}
		}
		return BestIndex;
	}
} // namespace Chaos::SpatialPartition::AabbTreeAlgorithm
