// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Common.h"
#include "ChaosSpatialPartitions/Library/AabbTreeNode.h"
#include "ChaosSpatialPartitions/QueryData.h"
#include "ChaosSpatialPartitions/Visitors.h"

namespace Chaos::SpatialPartition::AabbTreeAlgorithm
{
	FAABB3 Union(const FAABB3& Aabb0, const FAABB3& Aabb1);
	FReal ComputeDeltaSurfaceArea(const FAABB3& OldAabb, const FAABB3& AabbToInclude);

	int32 AllocateNode(TArray<FAabbTreeNode>& Nodes, int32& FreeListHead);
	void DeallocateNode(TArray<FAabbTreeNode>& Nodes, int32& FreeListHead, int32 IndexToFree);
	
	bool IsLeaf(const FAabbTreeNode& Node);
	// Given a node and an expected child, returns the reference to the child on the parent.
	int32& GetIndexRef(FAabbTreeNode& ParentNode, const int32 NodeIndex);
	// Given a node and an expected child, returns the reference to the sibling on the parent.
	int32& GetSiblingRef(FAabbTreeNode& ParentNode, const int32 NodeIndex);

	// Recomputes the aabb of the given node from the two children. Only valid for non-leaf nodes.
	void RecomputeAabb(TArray<FAabbTreeNode>& Nodes, const int32 Index);
	// Attempts to perform a rotation of the children of the given node by minimizing surface area.
	bool RotateNodes(TArray<FAabbTreeNode>& Nodes, const int32 Index);
	// Recomputes the aabbs of all nodes from the given node up through the root. 
	// This may also performs rotations if necessary to keep the tree balanced.
	void RecomputeAncestorAabbsAndRotate(TArray<FAabbTreeNode>& Nodes, int32 Index, bool bDoRotation);

	// Finds the best sibling to insert a node at using a greedy surface area heuristic.
	int32 FindBestSiblingGreedy(const TArray<FAabbTreeNode>& Nodes, const int32 RootIndex, const FAABB3& NewAabb);
	// Finds the best sibling to insert a node at using a more advanced greedy surface area heuristic. 
	// This version keeps track of the overall SA increase and can select internal nodes.
	int32 FindBestSiblingAdvancedGreedySAH(const TArray<FAabbTreeNode>& Nodes, const int32 RootIndex, const FAABB3& NewAabb);
	// Finds the best sibling to insert a node at using a global search. This is slow and primarily used for testing / validation.
	int32 FindBestSiblingGlobalSearch(const TArray<FAabbTreeNode>& Nodes, const int32 RootIndex, const FAABB3& NewAabb);
	// Finds the best sibling to insert a node at by using an efficient global search.
	int32 FindBestSiblingBranchAndBound(const TArray<FAabbTreeNode>& Nodes, const int32 RootIndex, const FAABB3& NewAabb);
	// Finds the best sibling to insert a node at by using an efficient global search. 
	// This takes a priority queue for performance so as to minimize allocations between calls.
	int32 FindBestSiblingBranchAndBound(const TArray<FAabbTreeNode>& Nodes, const int32 RootIndex, const FAABB3& NewAabb, TArray<TPair<int32, FReal>>& PriorityQueue);

	template <typename QueryDataType, typename LeafCallbackType>
	EVisitResult Query(const TArray<FAabbTreeNode>& Nodes, const int32 RootIndex, const QueryDataType& QueryData, LeafCallbackType LeafCallback)
	{
		if (RootIndex == INDEX_NONE)
		{
			return EVisitResult::Continue;
		}

		// TODO: Investigate the traversal order for rays/sweeps. 
		// The previous implementation would push onto the stack so that sooner hits were traversed first, 
		// however this is more difficult when the child aabbs are not embedded in the parent. 
		// In addition, legacy only ordered the direct children and did not do a full priority queue.
		constexpr int32 MaxNodeStackNumOnSystemStack = 255;
		TArray<int32, TSizedInlineAllocator<MaxNodeStackNumOnSystemStack, 32> > NodeStack;

		NodeStack.Push(RootIndex);
		while (!NodeStack.IsEmpty())
		{
			const int32 NodeIndex = NodeStack.Pop(EAllowShrinking::No);
			const FAabbTreeNode& Node = Nodes[NodeIndex];

			if (!QueryData.Test(Node.Aabb))
			{
				continue;
			}

			if (AabbTreeAlgorithm::IsLeaf(Node))
			{
				EVisitResult VisitResult = LeafCallback(NodeIndex, Node);
				if (VisitResult == EVisitResult::Stop)
				{
					return EVisitResult::Stop;
				}
			}
			else
			{
				NodeStack.Push(Node.Left);
				NodeStack.Push(Node.Right);
			}
		}
		return EVisitResult::Continue;
	}
} // namespace Chaos::SpatialPartition::AabbTreeAlgorithm
