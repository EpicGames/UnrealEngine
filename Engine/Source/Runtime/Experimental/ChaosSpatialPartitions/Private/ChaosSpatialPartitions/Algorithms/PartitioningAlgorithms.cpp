// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSpatialPartitions/Algorithms/PartitioningAlgorithms.h"

#include "ChaosSpatialPartitions/Algorithms/AabbTreeAlgorithm.h"

namespace Chaos::SpatialPartition::AabbTreeAlgorithm
{
	int32 PickLargestAxis(const FVec3& Vector)
	{
		if (Vector.X > Vector.Y)
		{
			return Vector.X > Vector.Z ? 0 : 2;
		}
		else
		{
			return Vector.Y > Vector.Z ? 1 : 2;
		}
	}

	void ComputeScaledVariance(const FVec3& Center, int32& InOutCurrentCount, FVec3& OutAverageCenter, FVec3& OutScaledCenterVariance)
	{
		const FVec3 Delta = Center - OutAverageCenter;
		++InOutCurrentCount;
		OutAverageCenter += Delta / InOutCurrentCount;
		const FVec3 Delta2 = Center - OutAverageCenter;
		OutScaledCenterVariance += Delta2 * Delta;
	}

	void ComputeBoundsForBins(const FAABB3& CentroidAabb, const int32 Axis, FReal& OutBinsLowerBound, FReal& OutBinBoundsInvSize)
	{
		OutBinsLowerBound = CentroidAabb.Min()[Axis];
		const FReal BoundsUpperBound = CentroidAabb.Max()[Axis];
		const FReal BoundsSize = BoundsUpperBound - OutBinsLowerBound;
		OutBinBoundsInvSize = (BoundsSize != 0) ? (1.0f / BoundsSize) : 0;
	}

	void ComputeBin(const FAABB3& ElementAabb, const int32 Axis, const FReal BinsLowerBound, const FReal BinBoundsInvSize, TArray<FAabbBin>& Bins, int32& OutBinIndex)
	{
		// Compute a [0, 1] value for where this element falls within the full bounds.
		const FReal ElementValue = ElementAabb.GetCenter()[Axis];
		const FReal Ratio = (ElementValue - BinsLowerBound) * BinBoundsInvSize;

		// Convert the ratio to a bin index. 
		// Note: The max object will have a ratio that ends up being 1, which will overflow to by 1. Clamp to prevent this.
		const int32 BinCount = Bins.Num();
		int32 BinIndex = static_cast<int32>(BinCount * Ratio);
		BinIndex = FMath::Clamp(BinIndex, 0, BinCount - 1);

		Bins[BinIndex].Aabb.GrowToInclude(ElementAabb);
		++Bins[BinIndex].Count;
		OutBinIndex = BinIndex;
	}

	void ComputeBinSplitPlanes(const TArray<FAabbBin>& Bins, TArray<FBinSplitPlane>& BinSplitPlanes)
	{
		const int32 SplitCount = Bins.Num() - 1;
		BinSplitPlanes.SetNum(SplitCount);

		// Walk left-to-right, build the left side split plane info
		BinSplitPlanes[0].LeftAabb = Bins[0].Aabb;
		BinSplitPlanes[0].LeftCount = Bins[0].Count;
		for (int32 I = 1; I < SplitCount; ++I)
		{
			BinSplitPlanes[I].LeftAabb = AabbTreeAlgorithm::Union(BinSplitPlanes[I - 1].LeftAabb, Bins[I].Aabb);
			BinSplitPlanes[I].LeftCount = BinSplitPlanes[I - 1].LeftCount + Bins[I].Count;
		}

		// Walk right-to-left, build the right side split plane info
		BinSplitPlanes[SplitCount - 1].RightAabb = Bins[SplitCount].Aabb;
		BinSplitPlanes[SplitCount - 1].RightCount = Bins[SplitCount].Count;
		for (int32 I = SplitCount - 2; I >= 0; --I)
		{
			BinSplitPlanes[I].RightAabb = AabbTreeAlgorithm::Union(BinSplitPlanes[I + 1].RightAabb, Bins[I + 1].Aabb);
			BinSplitPlanes[I].RightCount = BinSplitPlanes[I + 1].RightCount + Bins[I + 1].Count;
		}
	}

	int32 PickBestBinSplitPlane(const TArray<FBinSplitPlane>& BinSplitPlanes)
	{
		int32 BestSplit = 0;
		FReal BestCost = std::numeric_limits<FReal>::max();
		for (int32 I = 0; I < BinSplitPlanes.Num(); ++I)
		{
			const FBinSplitPlane& SplitPlane = BinSplitPlanes[I];
			const FReal LeftArea = SplitPlane.LeftAabb.GetArea();
			const FReal RightArea = SplitPlane.RightAabb.GetArea();
			const int LeftCount = SplitPlane.LeftCount;
			const int RightCount = SplitPlane.RightCount;
			const FReal Cost = LeftArea * LeftCount + RightArea * RightCount;
			if (Cost < BestCost)
			{
				BestCost = Cost;
				BestSplit = I;
			}
		}
		return BestSplit;
	}
} // namespace Chaos::SpatialPartition::AabbTreeAlgorithm
