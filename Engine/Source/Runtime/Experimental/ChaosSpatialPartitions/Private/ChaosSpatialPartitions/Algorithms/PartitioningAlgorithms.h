// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Common.h"
#include "ChaosSpatialPartitions/Algorithms/PartitioningStructures.h"

namespace Chaos::SpatialPartition::AabbTreeAlgorithm
{
	int32 PickLargestAxis(const FVec3& Vector);

	// Computes the aabb of the element centers (not the element aabbs). 
	// This does not reset the given aabb so this method can be used incrementally.
	template <typename ElementType>
	void ComputeCentroidAabb(const TArrayView<const ElementType>& Elements, FAABB3& OutAabb)
	{
		for (const ElementType& Element : Elements)
		{
			OutAabb.GrowToInclude(Element.GetCenter());
		}
	}

	// Compute the scaled variance using Welford's method. Assumes Count, OutAverageCenter, and OutScaledCenterVariance 
	// are all initialized to zero before the first call. This method is meant to be used incrementally.
	void ComputeScaledVariance(const FVec3& Center, int32& InOutCurrentCount, FVec3& OutAverageCenter, FVec3& OutScaledCenterVariance);

	// Compute the scaled variance using Welford's method. Assumes Count, OutAverageCenter, and OutScaledCenterVariance 
	// are all initialized to zero before the first call. This method is meant to be used incrementally.
	template <typename ElementType>
	void ComputeScaledVariance(const TArrayView<const ElementType>& Elements, int32& InOutCurrentCount, FVec3& OutAverageCenter, FVec3& OutScaledCenterVariance)
	{
		for (const ElementType& Element : Elements)
		{
			ComputeScaledVariance(Element.GetCenter(), InOutCurrentCount, OutAverageCenter, OutScaledCenterVariance);
		}
	}

	// Computes the bounds information for all of the bins given the centroid aabb.
	void ComputeBoundsForBins(const FAABB3& CentroidAabb, const int32 Axis, FReal& OutBinsLowerBound, FReal& OutBinBoundsInvSize);
	// Given a full bounds range (min + inverse size) on an axis, partitions an element into a binned area.
	void ComputeBin(const FAABB3& ElementAabb, const int32 Axis, const FReal BinsLowerBound, const FReal BinBoundsInvSize, TArray<FAabbBin>& Bins, int32& OutBinIndex);
	// Given a full bounds range (min + inverse size) on an axis, partitions elements into a binned area.
	template <typename ElementType>
	void ComputeBins(TArrayView<const ElementType>& Elements, const int32 Axis, const FReal BoundsLowerBound, const FReal InvBoundsSize, TArray<FAabbBin>& Bins, TArrayView<int32>& BinIndices)
	{
		check(Elements.Num() == BinIndices.Num());

		for (int32 I = 0; I < Elements.Num(); ++I)
		{
			ComputeBin(Elements[I].GetAabb(), Axis, BoundsLowerBound, InvBoundsSize, Bins, BinIndices[I]);
		}
	}

	// Computes all of the potential "split planes" for the bins. The split plane N lies between bin N and N + 1.
	void ComputeBinSplitPlanes(const TArray<FAabbBin>& Bins, TArray<FBinSplitPlane>& BinSplitPlanes);
	// Returns the index of the split plane with the lowest cost (using a surface area heuristic).
	int32 PickBestBinSplitPlane(const TArray<FBinSplitPlane>& BinSplitPlanes);

	// Partitions the elements into two sets: left < split <= right.
	// The return value is the size of the left split.
	// This will swap up to MaxCount items.
	template <typename ElementType>
	bool PartitionElementsInPlace(TArrayView<const ElementType>& Elements, const int32 SplitAxis, const FReal SplitPosition, const int32 MaxCount, int32& Start, int32& End, int32& OutSplitIndex)
	{
		int32 Count = 0;
		while (Start < End && Count < MaxCount)
		{
			// Move start to the right until it's greater than or equal to the pivot value.
			while (Start < End && Elements[Start].GetCenter()[SplitAxis] < SplitPosition)
			{
				++Start;
			}
			// Move end to the left until it's less than the pivot value.
			while (Start < End && SplitPosition <= Elements[End - 1].GetCenter()[SplitAxis])
			{
				--End;
			}

			// As long as the indices haven't crossed, we know this is a valid swap.
			if (Start < End)
			{
				Swap(Elements[Start], Elements[End - 1]);
				++Start;
				--End;
				++Count;
			}
		}
		OutSplitIndex = Start;
		return Start < End;
	}

	// Partitions the elements into two sets: left < split <= right.
	// The return value is the size of the left split.
	template <typename ElementType>
	int32 PartitionElementsInPlace(TArrayView<const ElementType>& Elements, const int32 SplitAxis, const FReal SplitPosition)
	{
		int32 Start = 0;
		int32 End = Elements.Num();
		int32 SplitIndex = 0;
		PartitionElementsInPlace(Elements, SplitAxis, SplitPosition, Elements.Num(), Start, End, SplitIndex);
		return SplitIndex;
	}

	// Partitions the elements into two sets using the bin indices: left < split <= right.
	// The bin indices are not updated.
	// The return value is the size of the left split.
	// This will swap up to MaxCount items.
	template <typename ElementType>
	bool PartitionElementsInPlace(TArrayView<const ElementType>& Elements, const TArrayView<int32>& BinIndices, const int32 BestSplit, const int32 MaxCount, int32& Start, int32& End, int32& OutSplitIndex)
	{
		check(Elements.Num() == BinIndices.Num());

		int32 Count = 0;
		while (Start < End && Count < MaxCount)
		{
			// Move start to the right until it's greater than or equal to the pivot value.
			while (Start < End && BinIndices[Start] < BestSplit)
			{
				++Start;
			}
			// Move end to the left until it's less than the pivot value.
			while (Start < End && BestSplit <= BinIndices[End - 1])
			{
				--End;
			}

			// As long as the indices haven't crossed, we know this is a valid swap.
			if (Start < End)
			{
				Swap(Elements[Start], Elements[End - 1]);
				++Start;
				--End;
				++Count;
			}
		}
		OutSplitIndex = Start;
		return Start < End;
	}

	// Partitions the elements into two sets using the bin indices: left < split <= right.
	// The bin indices are not updated.
	// The return value is the size of the left split.
	template <typename ElementType>
	int32 PartitionElementsInPlace(TArrayView<const ElementType>& Elements, const TArrayView<int32>& BinIndices, const int32 BestSplit)
	{
		int32 Start = 0;
		int32 End = Elements.Num();
		int32 SplitIndex = 0;
		PartitionElementsInPlace(Elements, BinIndices, BestSplit, Elements.Num(), Start, End, SplitIndex);
		return SplitIndex;
	}

	// Computes an axis aligned split plane (axis + value) from the spatial median of the objects.
	// The axis with the largest spread is picked as the split axis.
	// The center of the centroid aabb on the split axis is picked as the split value.
	template <typename ElementType>
	void ComputeSplitPlaneWithSpatialMedianHeuristic(const TArrayView<const ElementType>& Elements, int32& OutSplitAxis, FReal& OutSplitPosition)
	{
		FAABB3 Aabb = FAABB3::EmptyAABB();
		ComputeCentroidAabb(Elements, Aabb);

		const FVec3 Center = Aabb.GetCenter();
		const FVec3 Extents = Aabb.Extents();
		OutSplitAxis = PickLargestAxis(Extents);

		OutSplitPosition = Center[OutSplitAxis];
	}

	// Computes an axis aligned split plane (axis + value) by using the variance of the centroid.
	// The axis with the largest variance on the centroid is picked as the split axis.
	// The average of the centroids on the split axis is picked as the split value.
	template <typename ElementType>
	void ComputeSplitPlaneWithMedianVarianceHeuristic(const TArrayView<const ElementType>& Elements, int32& OutSplitAxis, FReal& OutSplitPosition)
	{
		FVec3 AverageCenter = FVec3::Zero();
		FVec3 ScaledCenterVariance = FVec3::Zero();

		int32 Count = 0;
		ComputeScaledVariance(Elements, Count, AverageCenter, ScaledCenterVariance);

		// Technically the variance is (ScaledCenterVariance/Num) and the sample variance is (ScaledCenterVariance / (Num - 1)). 
		// Since we're only picking largest, we can ignore the constant.
		OutSplitAxis = PickLargestAxis(ScaledCenterVariance);
		OutSplitPosition = AverageCenter[OutSplitAxis];
	}

	// Returns the bin index that should be used to split the data into two sets such that: left < index <= right
	template <typename ElementType>
	int32 ComputeSplitPlaneWithSurfaceAreaHeuristic(TArrayView<const ElementType>& Elements, const int32 BinCount, TArrayView<int32>& BinIndices)
	{
		FAABB3 CentroidAabb = FAABB3::EmptyAABB();
		ComputeCentroidAabb(Elements, CentroidAabb);

		const int32 SplitAxis = CentroidAabb.LargestAxis();

		TArray<FAabbBin> Bins;
		Bins.SetNum(BinCount);

		FReal BinLowerBound;
		FReal BinBoundsInvSize;
		ComputeBoundsForBins(CentroidAabb, SplitAxis, BinLowerBound, BinBoundsInvSize);
		ComputeBins(Elements, SplitAxis, BinLowerBound, BinBoundsInvSize, Bins, BinIndices);

		TArray<FBinSplitPlane> BinSplitPlanes;
		ComputeBinSplitPlanes(Bins, BinSplitPlanes);
		const int32 SplitPlaneIndex = PickBestBinSplitPlane(BinSplitPlanes);
		// If we decide that the best split plane is 2, then we have:
		// [0, N - 1], [1, N - 2], [2, N - 3], [3, N - 4], ...
		// Split plane 2 is [2, N - 3]. This should return the first bin index such that left < index <= right, so the result is SplitPlaneIndex(2) + 1.
		return SplitPlaneIndex + 1;
	}

	// Partitions the given entries into two sets using the spatial median heuristic. The result is the size of the left partition.
	template <typename ElementType>
	int32 PartitionEntriesWithSpatialMedian(TArrayView<const ElementType>& Elements)
	{
		int32 SplitAxis;
		FReal SplitPosition;
		ComputeSplitPlaneWithSpatialMedianHeuristic(Elements, SplitAxis, SplitPosition);
		return PartitionElementsInPlace(Elements, SplitAxis, SplitPosition);
	}

	// Partitions the given entries into two sets using the median variance heuristic. The result is the size of the left partition.
	template <typename ElementType>
	int32 PartitionEntriesWithMedianVariance(TArrayView<const ElementType>& Elements)
	{
		int32 SplitAxis;
		FReal SplitPosition;
		ComputeSplitPlaneWithMedianVarianceHeuristic(Elements, SplitAxis, SplitPosition);
		return PartitionElementsInPlace(Elements, SplitAxis, SplitPosition);
	}

	// Partitions the given entries into two sets using a binned surface area heuristic. The result is the size of the left partition.
	template <typename ElementType>
	int32 PartitionEntriesWithSurfaceArea(TArrayView<const ElementType>& Elements, const int32 BinCount, TArrayView<int32>& BinIndices)
	{
		check(BinIndices.Num() == Elements.Num());

		const int32 BestBinSplitIndex = ComputeSplitPlaneWithSurfaceAreaHeuristic(Elements, BinCount, BinIndices);
		return PartitionElementsInPlace(Elements, BinIndices, BestBinSplitIndex);
	}
} // namespace Chaos::SpatialPartition::AabbTreeAlgorithm
