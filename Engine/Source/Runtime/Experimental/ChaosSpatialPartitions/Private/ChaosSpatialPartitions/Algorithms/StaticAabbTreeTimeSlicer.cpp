// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSpatialPartitions/Algorithms/StaticAabbTreeTimeSlicer.h"

#include "ChaosSpatialPartitions/Algorithms/PartitioningAlgorithms.h"
#include "ChaosSpatialPartitions/Algorithms/AabbTreeAlgorithm.h"

namespace Chaos::SpatialPartition::AabbTreeAlgorithm
{
	FStaticAabbTreeTimeSlicer::FStaticAabbTreeTimeSlicer(const FConfig& InConfig, TArray<FAabbTreeLeafElement>&& InElements)
		: Config(InConfig)
		, Elements(MoveTemp(InElements))
	{
		const int32 ElementCount = Elements.Num();
		if (ElementCount == 0)
		{
			return;
		}
		if (!ensure(Config.MaxElementsPerLeaf != 0))
		{
			Config.MaxElementsPerLeaf = 1;
		}
		if (!ensure(Config.SurfaceAreaHeuristicBinCount > 0))
		{
			Config.SurfaceAreaHeuristicBinCount = 1;
		}

		const int32 MinimumLeafCount = (ElementCount / Config.MaxElementsPerLeaf) + 1;
		const int32 MinimumNodeCount = MinimumLeafCount * 2 - 1;
		Nodes.Reserve(MinimumNodeCount);
		RootIndex = AllocateNode();
		WorkloadStack.Reserve(MinimumNodeCount);
		Leaves.Reserve(MinimumLeafCount);
		AddNewWorkload(TArrayView<const FAabbTreeLeafElement>(Elements), RootIndex, 0);
	}

	bool FStaticAabbTreeTimeSlicer::Run()
	{
		if (IsFinished())
		{
			return false;
		}

		FWorkload& Workload = WorkloadStack.Last();
		switch (Workload.State)
		{
			case EState::Setup:
			{
				RunSetup(Workload);
				break;
			}
			case EState::Partition:
			{
				RunPartition(Workload);
				break;
			}
			case EState::Finish:
			{
				RunFinish(Workload);
				break;
			}
		}

		return !IsFinished();
	}

	bool FStaticAabbTreeTimeSlicer::IsFinished() const
	{
		return WorkloadStack.IsEmpty();
	}

	int32 FStaticAabbTreeTimeSlicer::GetRootIndex() const
	{
		return RootIndex;
	}

	TArray<FAabbTreeNode>& FStaticAabbTreeTimeSlicer::GetNodes()
	{
		return Nodes;
	}

	TArray<FAabbTreeLeaf>& FStaticAabbTreeTimeSlicer::GetLeaves()
	{
		return Leaves;
	}

	int32 FStaticAabbTreeTimeSlicer::AllocateNode()
	{
		// Since we're building from scratch, we don't need to maintain a free list.
		int32 UnusedFreeListHead = INDEX_NONE;
		return AabbTreeAlgorithm::AllocateNode(Nodes, UnusedFreeListHead);
	}

	int32 FStaticAabbTreeTimeSlicer::BuildLeaf(const TArrayView<const FAabbTreeLeafElement>& InElements, const int32 NodeIndex)
	{
		const int32 LeafIndex = Leaves.Emplace();
		Leaves[LeafIndex].Build(InElements, NodeIndex);
		return LeafIndex;
	}

	void FStaticAabbTreeTimeSlicer::AddNewWorkload(const TArrayView<const FAabbTreeLeafElement>& InElements, const int32 NodeIndex, const int32 Depth)
	{
		FWorkload& Workload = WorkloadStack.Emplace_GetRef();
		Workload.Elements = InElements;
		Workload.NodeIndex = NodeIndex;
		Workload.State = EState::Setup;
		Workload.Depth = Depth;
	}

	void FStaticAabbTreeTimeSlicer::SplitWorkload(const FWorkload& ParentWorkload, int32 SplitIndex)
	{
		const int32 Count = ParentWorkload.Elements.Num();
		int32 RightSize = Count - SplitIndex;
		// If the split doesn't actually reduce the workload size, then just divide it in half.
		// If this is not done, then the exact same split would be produced and this will loop forever (or until max depth).
		// One case this would happen is if all aabb's are identical.
		if (SplitIndex == 0 || RightSize == 0)
		{
			SplitIndex = Count / 2;
			RightSize = Count - SplitIndex;
		}

		TArrayView<const FAabbTreeLeafElement> LeftSlice = ParentWorkload.Elements.Slice(0, SplitIndex);
		TArrayView<const FAabbTreeLeafElement> RightSlice = ParentWorkload.Elements.Slice(SplitIndex, RightSize);

		const int32 NewDepth = ParentWorkload.Depth + 1;
		const FAabbTreeNode& Node = Nodes[ParentWorkload.NodeIndex];
		AddNewWorkload(LeftSlice, Node.Left, NewDepth);
		AddNewWorkload(RightSlice, Node.Right, NewDepth);
	}

	void FStaticAabbTreeTimeSlicer::Initialize(FWorkload& Workload)
	{
		switch (Config.PartitioningMethod)
		{
			case EPartitioningMethod::CentroidSpatialMedian:
			{
				CentroidSpatialMedianWorkload.Reset();
				break;
			}
			case EPartitioningMethod::CentroidVariance:
			{
				CentroidVarianceWorkload.Reset();
				break;
			}
			case EPartitioningMethod::SurfaceArea:
			{
				BinnedSurfaceAreaWorkload.Reset();
				break;
			}
		}
	}

	void FStaticAabbTreeTimeSlicer::RunSetup(FWorkload& Workload)
	{
		Initialize(Workload);

		if (Workload.Elements.Num() <= Config.MaxElementsPerLeaf || Workload.Depth >= Config.MaxTreeDepth)
		{
			const int32 LeafIndex = BuildLeaf(Workload.Elements, Workload.NodeIndex);
			Nodes[Workload.NodeIndex].UserData = LeafIndex;
			Nodes[Workload.NodeIndex].Aabb = Leaves[LeafIndex].Aabb;
			WorkloadStack.Pop(EAllowShrinking::No);
		}
		else
		{
			// We know this node will have children. Pre-allocate the children nodes and hook up the indices.
			// Note: Allocation of new nodes can invalidate references.
			const int32 LeftIndex = AllocateNode();
			const int32 RightIndex = AllocateNode();
			FAabbTreeNode& Node = Nodes[Workload.NodeIndex];
			Node.Left = LeftIndex;
			Node.Right = RightIndex;
			Nodes[Node.Left].Parent = Workload.NodeIndex;
			Nodes[Node.Right].Parent = Workload.NodeIndex;
			// Transition to the partitioning state
			Workload.State = EState::Partition;
		}
	}

	void FStaticAabbTreeTimeSlicer::RunPartition(FWorkload& Workload)
	{
		switch (Config.PartitioningMethod)
		{
			case EPartitioningMethod::CentroidSpatialMedian:
			{
				CentroidSpatialMedianWorkload.Run(*this, Workload);
				break;
			}
			case EPartitioningMethod::CentroidVariance:
			{
				CentroidVarianceWorkload.Run(*this, Workload);
				break;
			}
			case EPartitioningMethod::SurfaceArea:
			{
				BinnedSurfaceAreaWorkload.Run(*this, Workload);
				break;
			}
		}
	}

	void FStaticAabbTreeTimeSlicer::RunFinish(FWorkload& Workload)
	{
		// In this state, the node's children are completely built. We can just build our aabb from the children.
		// Note: This aabb is the actual element's aabb which is different from the aabb used during partitioning.
		AabbTreeAlgorithm::RecomputeAabb(Nodes, Workload.NodeIndex);

		WorkloadStack.Pop(EAllowShrinking::No);
	}

	bool FStaticAabbTreeTimeSlicer::ComputeCentroidAabb(FWorkload& Workload, int32& Start, FAABB3& OutCentroidAabb) const
	{
		const int32 RemainingSize = Workload.Elements.Num() - Start;
		const int32 SliceSize = FMath::Min(Config.BatchSize, RemainingSize);

		TArrayView<const FAabbTreeLeafElement> WorkSlice = Workload.Elements.Slice(Start, SliceSize);
		AabbTreeAlgorithm::ComputeCentroidAabb(WorkSlice, OutCentroidAabb);

		Start += SliceSize;
		return SliceSize == RemainingSize;
	}

	bool FStaticAabbTreeTimeSlicer::ComputeCentroidVariance(FWorkload& Workload, int32& Start, int32& ProcessedCount, FVec3& OutAverageCenter, FVec3& OutScaledCenterVariance) const
	{
		const int32 RemainingSize = Workload.Elements.Num() - Start;
		const int32 SliceSize = FMath::Min(Config.BatchSize, RemainingSize);

		TArrayView<const FAabbTreeLeafElement> WorkSlice = Workload.Elements.Slice(Start, SliceSize);
		AabbTreeAlgorithm::ComputeScaledVariance(WorkSlice, ProcessedCount, OutAverageCenter, OutScaledCenterVariance);

		Start += SliceSize;
		return SliceSize == RemainingSize;
	}

	void FStaticAabbTreeTimeSlicer::FCentroidSpatialMedianWorkload::Reset()
	{
		PartitionState = EPartitionState::Select;
		Start = 0;
		End = 0;
		SplitAxis = 0;
		SplitPosition = 0;
		CentroidAabb = FAABB3::EmptyAABB();
	}

	void FStaticAabbTreeTimeSlicer::FCentroidSpatialMedianWorkload::Run(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload)
	{
		if (PartitionState == EPartitionState::Select)
		{
			RunSelect(TimeSlicer, Workload);
		}
		else if (PartitionState == EPartitionState::Partition)
		{
			RunPartition(TimeSlicer, Workload);
		}
	}

	void FStaticAabbTreeTimeSlicer::FCentroidSpatialMedianWorkload::RunSelect(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload)
	{
		const bool bFinished = TimeSlicer.ComputeCentroidAabb(Workload, Start, CentroidAabb);
		if (bFinished)
		{
			// Pull out the partitioning info from the centroid aabb.
			SplitAxis = AabbTreeAlgorithm::PickLargestAxis(CentroidAabb.Extents());
			SplitPosition = CentroidAabb.GetCenter()[SplitAxis];

			// Transition to the partitioning state.
			PartitionState = EPartitionState::Partition;
			Start = 0;
			End = Workload.Elements.Num();
		}
	}

	void FStaticAabbTreeTimeSlicer::FCentroidSpatialMedianWorkload::RunPartition(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload)
	{
		const int32 MaxCount = TimeSlicer.Config.BatchSize;

		int32 SplitIndex = 0;
		const bool bFinished = !AabbTreeAlgorithm::PartitionElementsInPlace(Workload.Elements, SplitAxis, SplitPosition, MaxCount, Start, End, SplitIndex);
		if (bFinished)
		{
			// Make sure to set the update the current workload's state before adding new workloads as that could invalidate our reference by resizing the array.
			Workload.State = EState::Finish;
			TimeSlicer.SplitWorkload(Workload, SplitIndex);
		}
	}

	void FStaticAabbTreeTimeSlicer::FCentroidVarianceWorkload::Reset()
	{
		PartitionState = EPartitionState::Select;
		Start = 0;
		End = 0;
		SplitAxis = 0;
		SplitPosition = 0;
		CentroidAabb = FAABB3::EmptyAABB();
		ProcessedCount = 0;
		AverageCenter = FVec3::Zero();
		ScaledCenterVariance = FVec3::Zero();
	}

	void FStaticAabbTreeTimeSlicer::FCentroidVarianceWorkload::Run(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload)
	{
		if (PartitionState == EPartitionState::Select)
		{
			RunSelect(TimeSlicer, Workload);
		}
		else if (PartitionState == EPartitionState::Partition)
		{
			RunPartition(TimeSlicer, Workload);
		}
	}

	void FStaticAabbTreeTimeSlicer::FCentroidVarianceWorkload::RunSelect(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload)
	{
		bool bFinished = TimeSlicer.ComputeCentroidVariance(Workload, Start, ProcessedCount, AverageCenter, ScaledCenterVariance);
		if (bFinished)
		{
			// Technically the variance is (ScaledCenterVariance/Num) and the sample variance is (ScaledCenterVariance / (Num - 1)). 
			// Since we're only picking largest, we can ignore the constant.
			SplitAxis = PickLargestAxis(ScaledCenterVariance);
			SplitPosition = AverageCenter[SplitAxis];

			// Transition to the partitioning state.
			PartitionState = EPartitionState::Partition;
			Start = 0;
			End = Workload.Elements.Num();
		}
	}

	void FStaticAabbTreeTimeSlicer::FCentroidVarianceWorkload::RunPartition(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload)
	{
		const int32 MaxCount = TimeSlicer.Config.BatchSize;

		int32 SplitIndex = 0;
		const bool bFinished = !AabbTreeAlgorithm::PartitionElementsInPlace(Workload.Elements, SplitAxis, SplitPosition, MaxCount, Start, End, SplitIndex);
		if (bFinished)
		{
			// Make sure to set the update the current workload's state before adding new workloads as that could invalidate our reference by resizing the array.
			Workload.State = EState::Finish;
			TimeSlicer.SplitWorkload(Workload, SplitIndex);
		}
	}

	void FStaticAabbTreeTimeSlicer::FBinnedSurfaceAreaWorkload::Reset()
	{
		State = EPartitionState::ComputeBounds;

		CentroidAabb = FAABB3::EmptyAABB();
		Bins.Reset();
		BinIndices.Reset();
		BinSplitPlanes.Reset();
		BoundsLowerBound = 0;
		InvBoundsSize = 0;
		Start = 0;
		End = 0;
		SplitAxis = 0;
		BinSplitIndex = 0;
		SplitIndex = 0;
	}

	void FStaticAabbTreeTimeSlicer::FBinnedSurfaceAreaWorkload::Run(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload)
	{
		if (State == EPartitionState::ComputeBounds)
		{
			RunComputeBounds(TimeSlicer, Workload);
		}
		else if (State == EPartitionState::ComputeBins)
		{
			RunComputeBins(TimeSlicer, Workload);
		}
		else if (State == EPartitionState::Partition)
		{
			RunPartition(TimeSlicer, Workload);
		}
	}

	void FStaticAabbTreeTimeSlicer::FBinnedSurfaceAreaWorkload::RunComputeBounds(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload)
	{
		const bool bFinished = TimeSlicer.ComputeCentroidAabb(Workload, Start, CentroidAabb);
		if (bFinished)
		{
			// Pick the largest axis as the one we partition on.
			SplitAxis = CentroidAabb.LargestAxis();

			// Transition to the partitioning state.
			State = EPartitionState::ComputeBins;
			Start = 0;

			BoundsLowerBound = CentroidAabb.Min()[SplitAxis];
			const FReal BinUpperBound = CentroidAabb.Max()[SplitAxis];
			const FReal BoundsSize = BinUpperBound - BoundsLowerBound;
			InvBoundsSize = BoundsSize != 0 ? (1.0f / BoundsSize) : 0;
			Bins.SetNum(TimeSlicer.Config.SurfaceAreaHeuristicBinCount);
			BinIndices.SetNum(Workload.Elements.Num());
		}
	}

	void FStaticAabbTreeTimeSlicer::FBinnedSurfaceAreaWorkload::RunComputeBins(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload)
	{
		const int32 RemainingSize = Workload.Elements.Num() - Start;
		const int32 SliceSize = FMath::Min(TimeSlicer.Config.BatchSize, RemainingSize);

		TArrayView<const FAabbTreeLeafElement> WorkSlice = Workload.Elements.Slice(Start, SliceSize);
		TArrayView<int32> BinIndicesSlice = TArrayView<int32>(BinIndices.GetData() + Start, SliceSize);
		AabbTreeAlgorithm::ComputeBins(WorkSlice, SplitAxis, BoundsLowerBound, InvBoundsSize, Bins, BinIndicesSlice);
		Start += SliceSize;

		const bool bFinished = (SliceSize == RemainingSize);
		if (bFinished)
		{
			// Picking the best split from the individual bins should be quick as it's bound by the bin size, not the element size.
			// We could split this if desired, but it's probably not important enough.
			ComputeBinSplitPlanes(Bins, BinSplitPlanes);
			BinSplitIndex = AabbTreeAlgorithm::PickBestBinSplitPlane(BinSplitPlanes) + 1;

			Start = 0;
			End = Workload.Elements.Num();

			State = EPartitionState::Partition;
		}
	}

	void FStaticAabbTreeTimeSlicer::FBinnedSurfaceAreaWorkload::RunPartition(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload)
	{
		const int32 MaxCount = TimeSlicer.Config.BatchSize;

		TArrayView<int32> BinIndicesView(BinIndices);
		const bool bFinished = !AabbTreeAlgorithm::PartitionElementsInPlace(Workload.Elements, BinIndicesView, BinSplitIndex, MaxCount, Start, End, SplitIndex);

		if (bFinished)
		{
			// Make sure to set the update the current workload's state before adding new workloads as that could invalidate our reference by resizing the array.
			Workload.State = EState::Finish;
			TimeSlicer.SplitWorkload(Workload, SplitIndex);
		}
	}
} // namespace Chaos::SpatialPartition::AabbTreeAlgorithm
