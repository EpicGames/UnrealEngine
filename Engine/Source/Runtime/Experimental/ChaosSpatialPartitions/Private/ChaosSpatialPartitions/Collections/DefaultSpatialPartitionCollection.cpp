// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSpatialPartitions/Collections/DefaultSpatialPartitionCollection.h"

namespace Chaos::SpatialPartition
{
	ENUM_CLASS_FLAGS(FDefaultSpatialPartitionCollection::EInternalCategory);

	FDefaultSpatialPartitionCollection::FDefaultSpatialPartitionCollection(const FConfig& Config)
		: StaticTree(Config.StaticConfig)
		, DynamicTree(Config.DynamicConfig)
	{
	}

	void FDefaultSpatialPartitionCollection::Insert(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& OutHandle)
	{
		// Changes are not valid during a rebuild
		check(!IsRebuilding());

		const FEntryHandle Handle = Entries.Create();
		const uint32 HandleValue = Handle.AsUint();

		const bool bIsGlobal = IsGlobal(Aabb);
		FEntry* Entry = Entries.Get(Handle);
		Entry->Category = Classification.GetCategory();
		Entry->UserData = UserData;
		Entry->Aabb = Aabb;
		Entry->InternalCategory = ToInternalCategory(Entry->Category, true, bIsGlobal);

		InsertInternal(Handle, Aabb, Entry->InternalCategory, Entry->Handle);

		OutHandle.SetValue(HandleValue);
	}

	void FDefaultSpatialPartitionCollection::Update(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& InOutHandle)
	{
		// Changes are not valid during a rebuild
		check(!IsRebuilding());

		const uint32 HandleValue = (uint32)InOutHandle.GetValue();
		FEntryHandle Handle;
		Handle.FromUint(HandleValue);

		FEntry* Entry = Entries.Get(Handle);
		if (!ensure(Entry != nullptr))
		{
			return;
		}

		const bool bIsGlobal = IsGlobal(Aabb);
		const EInternalCategory OldCategory = Entry->InternalCategory;
		Entry->Category = Classification.GetCategory();
		Entry->UserData = UserData;
		Entry->Aabb = Aabb;
		Entry->InternalCategory = ToInternalCategory(Entry->Category, true, bIsGlobal);

		// Note: This relies on dynamic and dynamic dirty not actually being different states (which we enforce and check),
		// otherwise we'd do a remove + insert instead of update.
		if (Entry->InternalCategory != OldCategory)
		{
			RemoveInternal(OldCategory, Entry->Handle);
			InsertInternal(Handle, Aabb, Entry->InternalCategory, Entry->Handle);
		}
		else
		{
			UpdateInternal(Handle, Aabb, Entry->InternalCategory, Entry->Handle);
		}
	}

	void FDefaultSpatialPartitionCollection::Remove(FSpatialHandle& InOutHandle)
	{
		// Changes are not valid during a rebuild
		check(!IsRebuilding());

		const uint32 HandleValue = (uint32)InOutHandle.GetValue();
		InOutHandle.SetValue(INDEX_NONE);
		FEntryHandle EntryHandle;
		EntryHandle.FromUint(HandleValue);

		FEntry* Entry = Entries.Get(EntryHandle);
		if (!ensure(Entry != nullptr))
		{
			return;
		}

		RemoveInternal(Entry->InternalCategory, Entry->Handle);
		Entries.Destroy(EntryHandle);
	}

	void FDefaultSpatialPartitionCollection::Overlap(const FOverlapQueryData& QueryData, FOverlapVisitor& Visitor, ESpatialCategoryMask CategoryMask) const
	{
		FInternalOverlapVisitor CollectionVisitor(this, CategoryMask, Visitor);
		FOverlapQueryRuntimeData QueryRuntimeData(QueryData);
		QueryInternal(CategoryMask, QueryRuntimeData, CollectionVisitor);
	}

	void FDefaultSpatialPartitionCollection::Raycast(const FRaycastQueryData& QueryData, FRaycastVisitor& Visitor, ESpatialCategoryMask CategoryMask) const
	{
		FInternalRaycastVisitor CollectionVisitor(this, CategoryMask, Visitor);
		FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
		QueryInternal(CategoryMask, QueryRuntimeData, CollectionVisitor);
	}

	void FDefaultSpatialPartitionCollection::Sweep(const FSweepQueryData& QueryData, FSweepVisitor& Visitor, ESpatialCategoryMask CategoryMask) const
	{
		FInternalSweepVisitor CollectionVisitor(this, CategoryMask, Visitor);
		FSweepQueryRuntimeData QueryRuntimeData(QueryData);
		QueryInternal(CategoryMask, QueryRuntimeData, CollectionVisitor);
	}

	bool FDefaultSpatialPartitionCollection::NeedsRebuilding() const
	{
		return !DirtyStaticEntries.IsEmpty() || StaticRebuildContext.IsValid();
	}

	FDefaultSpatialPartitionCollection::ERebuildStatus FDefaultSpatialPartitionCollection::Rebuild()
	{
		// If we don't have an active rebuild going, try to start a new one
		if (!StaticRebuildContext.IsValid())
		{
			// There's no work to do
			if (DirtyStaticEntries.IsEmpty())
			{
				return ERebuildStatus::Finished;
			}

			// Insert all dirty static objects into the static tree
			for (const FEntryHandle& Handle : DirtyStaticEntries)
			{
				FEntry* Entry = Entries.Get(Handle);
				check(Entry != nullptr);

				// This list contains every static object that became dirty, however a dirty static object can become dynamic after the fact.
				// It would be expensive to search for and remove those objects from this array, so instead we simply skip those objects when doing a clean rebuild.
				if (Entry->Category == ESpatialCategory::Static)
				{
					check(EnumHasAnyFlags(Entry->InternalCategory, EInternalCategory::Dirty));
					EnumRemoveFlags(Entry->InternalCategory, EInternalCategory::Dirty);
					StaticTree.InsertDeferred(Handle.AsUint(), Entry->Aabb, Entry->Handle);
				}
			}
			DirtyStaticEntries.Reset();

			// Begin rebuilding
			StaticRebuildContext = MakeUnique<FStaticAabbTree::FRebuildContext>();
			StaticTree.BeginRebuild(*StaticRebuildContext.Get());
		}

		ERebuildStatus RebuildStatus = StaticRebuildContext->Run();
		if (RebuildStatus == ERebuildStatus::Finished)
		{
			StaticTree.CommitRebuild(*StaticRebuildContext.Get());
			StaticRebuildContext = nullptr;
			// We keep the dirty static set valid until we finish the rebuild, this way queries keep working.
			StaticDirty = FDynamicAabbTree();
		}
		return RebuildStatus;
	}

	bool FDefaultSpatialPartitionCollection::IsGlobal(const FAABB3& Aabb) const
	{
		const FVec3& Min = Aabb.Min();
		const FVec3& Max = Aabb.Max();

		for (int32 i = 0; i < 3; ++i)
		{
			const FReal& MinComponent = Min[i];
			const FReal& MaxComponent = Max[i];

			// Are we an empty aabb?
			if (MinComponent > MaxComponent)
			{
				return true;
			}

			// Are we NaN/Inf?
			if (!FMath::IsFinite(MinComponent) || !FMath::IsFinite(MaxComponent))
			{
				return true;
			}
		}

		return false;
	}

	FDefaultSpatialPartitionCollection::EInternalCategory FDefaultSpatialPartitionCollection::ToInternalCategory(const ESpatialCategory& Category, const bool bDirty, const bool bGlobal)
	{
		// Kinematics get grouped with dynamics
		EInternalCategory InternalCategory = (Category == ESpatialCategory::Static) ? EInternalCategory::Static : EInternalCategory::Dynamic;
		InternalCategory = SetOrClearFlag(InternalCategory, EInternalCategory::Dirty, bDirty);
		InternalCategory = SetOrClearFlag(InternalCategory, EInternalCategory::Global, bGlobal);
		return InternalCategory;
	}

	FDefaultSpatialPartitionCollection::EInternalCategory FDefaultSpatialPartitionCollection::SetOrClearFlag(const EInternalCategory Flags, const EInternalCategory Flag, bool bState)
	{
		if (bState)
		{
			return Flags | Flag;
		}
		else
		{
			return Flags & ~Flag;
		}
	}

	const FDefaultSpatialPartitionCollection::FEntry* FDefaultSpatialPartitionCollection::GetEntryFromHandleValue(uint32 Index) const
	{
		FEntryHandle Handle;
		Handle.FromUint(Index);
		return Entries.Get(Handle);
	}

	bool FDefaultSpatialPartitionCollection::IsRebuilding() const
	{
		return StaticRebuildContext.IsValid();
	}

	void FDefaultSpatialPartitionCollection::InsertInternal(const FEntryHandle& EntryHandle, const FAABB3& Aabb, const EInternalCategory Category, FSpatialHandle& OutHandle)
	{
		const uint32 UserData = EntryHandle.AsUint();
		switch (Category)
		{
			case EInternalCategory::Static:
				ensureMsgf(false, TEXT("InsertInternal: State 'Static' shouldn't be possible as static insert are always marked dirty."));
				StaticTree.InsertDeferred(UserData, Aabb, OutHandle);
				break;
			case EInternalCategory::StaticDirty:
				StaticDirty.Insert(UserData, Aabb, OutHandle);
				DirtyStaticEntries.Add(EntryHandle);
				break;
			case EInternalCategory::StaticGlobal:
				StaticGlobal.Insert(UserData, Aabb, OutHandle);
				break;
			case EInternalCategory::Dynamic:
				ensureMsgf(false, TEXT("InsertInternal: State 'Dynamic' shouldn't be possible as inserts are always marked dirty."));
			case EInternalCategory::DynamicDirty:
				DynamicTree.Insert(UserData, Aabb, OutHandle);
				break;
			case EInternalCategory::DynamicGlobal:
				DynamicGlobal.Insert(UserData, Aabb, OutHandle);
				break;
			default:
				ensure(false);
				break;
		}
	}

	void FDefaultSpatialPartitionCollection::UpdateInternal(const FEntryHandle& EntryHandle, const FAABB3& Aabb, const EInternalCategory Category, FSpatialHandle& InOutHandle)
	{
		const uint32 UserData = EntryHandle.AsUint();
		switch (Category)
		{
			case EInternalCategory::Static:
				ensureMsgf(false, TEXT("UpdateInternal: State 'Static' shouldn't be possible as static updates are always marked dirty."));
				StaticTree.UpdateDeferred(UserData, Aabb, InOutHandle);
				break;
			case EInternalCategory::StaticDirty:
				StaticDirty.Update(UserData, Aabb, InOutHandle);
				break;
			case EInternalCategory::StaticGlobal:
				StaticGlobal.Update(UserData, Aabb, InOutHandle);
				break;
			case EInternalCategory::Dynamic:
				ensureMsgf(false, TEXT("UpdateInternal: State 'Dynamic' shouldn't be possible as updates are always marked dirty."));
			case EInternalCategory::DynamicDirty:
				DynamicTree.Update(UserData, Aabb, InOutHandle);
				break;
			case EInternalCategory::DynamicGlobal:
				DynamicGlobal.Update(UserData, Aabb, InOutHandle);
				break;
			default:
				ensure(false);
				break;
		}
	}

	void FDefaultSpatialPartitionCollection::RemoveInternal(const EInternalCategory Category, FSpatialHandle& InOutHandle)
	{
		switch (Category)
		{
			case EInternalCategory::Static:
				StaticTree.RemoveMinimal(InOutHandle);
				break;
			case EInternalCategory::StaticDirty:
				StaticDirty.Remove(InOutHandle);
				break;
			case EInternalCategory::StaticGlobal:
				StaticGlobal.Remove(InOutHandle);
				break;
			case EInternalCategory::Dynamic:
				ensureMsgf(false, TEXT("RemoveInternal: State 'Dynamic' shouldn't be possible as dynamic objects are always marked dirty."));
			case EInternalCategory::DynamicDirty:
				DynamicTree.Remove(InOutHandle);
				break;
			case EInternalCategory::DynamicGlobal:
				DynamicGlobal.Remove(InOutHandle);
				break;
			default:
				ensure(false);
				break;
		}
	}
} // namespace Chaos::SpatialPartition
