// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSpatialPartitions/Collections/NSquaredAabbCollection.h"

namespace Chaos::SpatialPartition
{
	void FNSquaredAabbCollection::Insert(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& OutHandle)
	{
		const ESpatialCategory Category = Classification.GetCategory();
		if (!ValidateCategory(Category))
		{
			return;
		}

		const FEntryHandle EntryHandle = Entries.Create();

		FEntry* Entry = Entries.Get(EntryHandle);
		Entry->Aabb = Aabb;
		Entry->UserData = UserData;
		Entry->Classification = Classification;

		SpatialPartitions[(int32)Category].Insert(UserData, Aabb, Entry->Handle);

		OutHandle.SetValue((int32)EntryHandle.AsUint());
	}

	void FNSquaredAabbCollection::Update(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& InOutHandle)
	{
		const ESpatialCategory Category = Classification.GetCategory();
		if (!ValidateCategory(Category))
		{
			return;
		}

		FEntryHandle EntryHandle;
		EntryHandle.FromUint((uint32)InOutHandle.GetValue());

		FEntry* Entry = Entries.Get(EntryHandle);
		if (Entry == nullptr)
		{
			return;
		}

		const int32 OldCategoryIndex = (int32)Entry->Classification.GetCategory();
		const int32 NewCategoryIndex = (int32)Category;

		Entry->Aabb = Aabb;
		Entry->UserData = UserData;
		Entry->Classification = Classification;

		if (OldCategoryIndex == NewCategoryIndex)
		{
			SpatialPartitions[NewCategoryIndex].Update(UserData, Aabb, Entry->Handle);
		}
		else
		{
			SpatialPartitions[OldCategoryIndex].Remove(Entry->Handle);
			SpatialPartitions[NewCategoryIndex].Insert(UserData, Aabb, Entry->Handle);
		}
	}

	void FNSquaredAabbCollection::Remove(FSpatialHandle& InOutHandle)
	{
		FEntryHandle EntryHandle;
		EntryHandle.FromUint((uint32)InOutHandle.GetValue());
		InOutHandle.SetValue(INDEX_NONE);

		FEntry* Entry = Entries.Get(EntryHandle);
		if (Entry == nullptr)
		{
			return;
		}

		const int32 CategoryIndex = (int32)(Entry->Classification.GetCategory());
		SpatialPartitions[CategoryIndex].Remove(Entry->Handle);

		Entries.Destroy(EntryHandle);
	}

	void FNSquaredAabbCollection::Overlap(const FOverlapQueryData& QueryData, FOverlapVisitor& Visitor, ESpatialCategoryMask CategoryMask) const
	{
		FOverlapQueryRuntimeData QueryRuntimeData(QueryData);
		RunQueries(CategoryMask, QueryRuntimeData, Visitor);
	}

	void FNSquaredAabbCollection::Raycast(const FRaycastQueryData& QueryData, FRaycastVisitor& Visitor, ESpatialCategoryMask CategoryMask) const
	{
		FRaycastQueryRuntimeData QueryRuntimeData(QueryData);
		RunQueries(CategoryMask, QueryRuntimeData, Visitor);
	}

	void FNSquaredAabbCollection::Sweep(const FSweepQueryData& QueryData, FSweepVisitor& Visitor, ESpatialCategoryMask CategoryMask) const
	{
		FSweepQueryRuntimeData QueryRuntimeData(QueryData);
		RunQueries(CategoryMask, QueryRuntimeData, Visitor);
	}

	EVisitResult FNSquaredAabbCollection::Query(int32 CategoryIndex, FOverlapQueryRuntimeData& QueryRuntimeData, FOverlapVisitor& Visitor) const
	{
		return SpatialPartitions[CategoryIndex].Overlap(QueryRuntimeData, Visitor);
	}

	EVisitResult FNSquaredAabbCollection::Query(int32 CategoryIndex, FRaycastQueryRuntimeData& QueryRuntimeData, FRaycastVisitor& Visitor) const
	{
		return SpatialPartitions[CategoryIndex].Raycast(QueryRuntimeData, Visitor);
	}

	EVisitResult FNSquaredAabbCollection::Query(int32 CategoryIndex, FSweepQueryRuntimeData& QueryRuntimeData, FSweepVisitor& Visitor) const
	{
		return SpatialPartitions[CategoryIndex].Sweep(QueryRuntimeData, Visitor);
	}

	bool FNSquaredAabbCollection::ValidateCategory(const ESpatialCategory Category) const
	{
		if (Category == ESpatialCategory::Static || Category == ESpatialCategory::Kinematic || Category == ESpatialCategory::Dynamic)
		{
			return true;
		}
		ensureMsgf(false, TEXT("Category %d is invalid."), EnumToUnderlyingType(Category));
		return false;
	}
} // namespace Chaos::SpatialPartition
