// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSpatialPartitions/Library/AabbTreeLeaf.h"

namespace Chaos::SpatialPartition
{
	FVec3 FAabbTreeLeafElement::GetCenter() const
	{
		return Aabb.Center();
	}

	const FAABB3& FAabbTreeLeafElement::GetAabb() const
	{
		return Aabb;
	}

	void FAabbTreeLeaf::Build(const TArrayView<const FAabbTreeLeafElement>& InElements, const int32 InNodeIndex)
	{
		NodeIndex = InNodeIndex;
		Elements = InElements;
		RecomputeAabb();
	}

	void FAabbTreeLeaf::RecomputeAabb()
	{
		Aabb = FAABB3::EmptyAABB();
		for (const FAabbTreeLeafElement& Element : Elements)
		{
			Aabb.GrowToInclude(Element.Aabb);
		}
	}
} // namespace Chaos::SpatialPartition
