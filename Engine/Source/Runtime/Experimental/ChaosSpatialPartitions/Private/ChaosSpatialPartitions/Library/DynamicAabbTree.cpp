// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSpatialPartitions/Library/DynamicAabbTree.h"
#include "ChaosSpatialPartitions/Algorithms/AabbTreeAlgorithm.h"

namespace Chaos::SpatialPartition
{
	FDynamicAabbTree::FDynamicAabbTree(const FConfig& Config)
		: Config(Config)
	{
	}

	void FDynamicAabbTree::Insert(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& OutHandle)
	{
		const FAABB3 ExpandedAabb = Expand(Aabb);
		Insert(UserData, Aabb, ExpandedAabb, OutHandle);
	}

	void FDynamicAabbTree::Insert(const FUserDataType& UserData, const FAABB3& Aabb, const FAABB3& ExpandedAabb, FSpatialHandle& OutHandle)
	{
		// Allocate and setup the leaf
		const int32 LeafIndex = AabbTreeAlgorithm::AllocateNode(Nodes, FreeListHeadIndex);
		FAabbTreeNode& Node = Nodes[LeafIndex];
		Node.Aabb = ExpandedAabb;
		Node.UserData = UserData;

		InsertLeaf(LeafIndex, true);

		OutHandle.SetValue(LeafIndex);
	}

	void FDynamicAabbTree::Update(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& InOutHandle)
	{
		const FAABB3 ExpandedAabb = Expand(Aabb);
		Update(UserData, Aabb, ExpandedAabb, InOutHandle);
	}

	void FDynamicAabbTree::Update(const FUserDataType& UserData, const FAABB3& Aabb, const FAABB3& ExpandedAabb, FSpatialHandle& InOutHandle)
	{
		const int32 NodeIndex = (int32)InOutHandle.GetValue();
		// Always update the user data
		Nodes[NodeIndex].UserData = UserData;

		if (Nodes[NodeIndex].Aabb.Contains(Aabb))
		{
			return;
		}

		// The new aabb is not contained in the old one. We need to re-insert the object to build a good tree.
		// Note: It's slightly faster to use Insert/Remove Leaf vs. Insert/Remove since we don't re-allocate the leaf.
		RemoveLeaf(NodeIndex, true);

		Nodes[NodeIndex].Aabb = ExpandedAabb;

		InsertLeaf(NodeIndex, true);
	}

	void FDynamicAabbTree::Remove(FSpatialHandle& InOutHandle)
	{
		const int32 NodeIndex = (int32)InOutHandle.GetValue();
		InOutHandle.SetValue(INDEX_NONE);
		RemoveLeaf(NodeIndex, Config.bRebalanceOnRemove);
		AabbTreeAlgorithm::DeallocateNode(Nodes, FreeListHeadIndex, NodeIndex);
	}

	bool FDynamicAabbTree::CheckNeedsUpdate(const FAABB3& Aabb, const FSpatialHandle& InHandle)
	{
		const int32 NodeIndex = (int32)InHandle.GetValue();
		const FAabbTreeNode& Node = Nodes[NodeIndex];
		return !Node.Aabb.Contains(Aabb);
	}

	EVisitResult FDynamicAabbTree::Overlap(FOverlapQueryRuntimeData& QueryData, FOverlapVisitor& Visitor) const
	{
		auto LeafCallback = [&QueryData, &Visitor](const int32 LeafIndex, const FAabbTreeNode& LeafNode) -> EVisitResult { return Visitor.Visit(LeafNode.UserData, QueryData); };
		return AabbTreeAlgorithm::Query(Nodes, RootIndex, QueryData, LeafCallback);
	}

	EVisitResult FDynamicAabbTree::Raycast(FRaycastQueryRuntimeData& QueryData, FRaycastVisitor& Visitor) const
	{
		auto LeafCallback = [&QueryData, &Visitor](const int32 LeafIndex, const FAabbTreeNode& LeafNode) -> EVisitResult { return Visitor.Visit(LeafNode.UserData, QueryData); };
		return AabbTreeAlgorithm::Query(Nodes, RootIndex, QueryData, LeafCallback);
	}

	EVisitResult FDynamicAabbTree::Sweep(FSweepQueryRuntimeData& QueryData, FSweepVisitor& Visitor) const
	{
		auto LeafCallback = [&QueryData, &Visitor](const int32 LeafIndex, const FAabbTreeNode& LeafNode) -> EVisitResult { return Visitor.Visit(LeafNode.UserData, QueryData); };
		return AabbTreeAlgorithm::Query(Nodes, RootIndex, QueryData, LeafCallback);
	}

	void FDynamicAabbTree::SelfQuery(FSelfQueryVisitor& Visitor) const
	{
		if (RootIndex == INDEX_NONE)
		{
			return;
		}

		// TODO: Currently recursive (was also in legacy). Investigate a stack approach?
		SelfQuery(Visitor, RootIndex);
	}

	void FDynamicAabbTree::Dump(TArray<FAabbTreeNode>& OutData, int32& OutRootIndex) const
	{
		OutData = Nodes;
		OutRootIndex = RootIndex;
	}

	FAABB3 FDynamicAabbTree::Expand(const FAABB3& Aabb) const
	{
		FAABB3 ExpandedAabb = Aabb;
		ExpandedAabb.Thicken(Config.PaddingFactor);
		return ExpandedAabb;
	}

	void FDynamicAabbTree::InsertLeaf(int32 LeafIndex, bool bDoRotation)
	{
		// No Root, just set the root and return.
		if (RootIndex == INDEX_NONE)
		{
			RootIndex = LeafIndex;
			return;
		}

		const int32 InternalNodeIndex = AabbTreeAlgorithm::AllocateNode(Nodes, FreeListHeadIndex);
		FAabbTreeNode& InternalNode = Nodes[InternalNodeIndex];
		FAabbTreeNode& LeafNode = Nodes[LeafIndex];
		const FAABB3& LeafAabb = LeafNode.Aabb;

		int32 SiblingIndex = INDEX_NONE;
		if (Config.SearchAlgorithm == FConfig::ESearchAlgorithm::BranchAndBound)
		{
			SiblingIndex = AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, LeafAabb, PriorityQueue);
		}
		else
		{
			SiblingIndex = AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, LeafAabb);
		}
		FAabbTreeNode& SiblingNode = Nodes[SiblingIndex];
		const int32 OldParentIndex = SiblingNode.Parent;

		// Fixup the tree structure
		InternalNode.Parent = OldParentIndex;
		InternalNode.Left = SiblingIndex;
		InternalNode.Right = LeafIndex;
		LeafNode.Parent = InternalNodeIndex;
		SiblingNode.Parent = InternalNodeIndex;

		// Fixup the aabbs
		InternalNode.Aabb = AabbTreeAlgorithm::Union(SiblingNode.Aabb, LeafAabb);

		// There's two cases to consider:
		// 1. The old sibling had a parent
		//	- We have to update that parent to point at the new internal node.
		// 2. The old sibling was the root
		//	- We have to update the root index.
		const int32 GrandParentIndex = InternalNode.Parent;
		if (GrandParentIndex != INDEX_NONE)
		{
			FAabbTreeNode& GrandParent = Nodes[GrandParentIndex];
			int32& OldSiblingRef = AabbTreeAlgorithm::GetIndexRef(GrandParent, SiblingIndex);
			OldSiblingRef = InternalNodeIndex;
		}
		else
		{
			RootIndex = InternalNodeIndex;
		}

		// TODO: This can probably start with the grand parent or the union above could be removed
		AabbTreeAlgorithm::RecomputeAncestorAabbsAndRotate(Nodes, InternalNodeIndex, bDoRotation);
	}

	void FDynamicAabbTree::RemoveLeaf(int32 LeafIndex, bool bDoRotation)
	{
		// We're removing the root, just set the root index and return.
		if (LeafIndex == RootIndex)
		{
			RootIndex = INDEX_NONE;
			return;
		}

		const FAabbTreeNode& LeafNode = Nodes[LeafIndex];
		const int32 ParentIndex = LeafNode.Parent;
		FAabbTreeNode& Parent = Nodes[ParentIndex];
		const int32 SiblingIndex = AabbTreeAlgorithm::GetSiblingRef(Parent, LeafIndex);
		FAabbTreeNode& Sibling = Nodes[SiblingIndex];
		const int32 GrandParentIndex = Parent.Parent;

		// We know the tree is at least height 2. We're always going to free two nodes (the leaf and its parent), so there's two cases to consider:
		// 1. The parent is not the root:
		//	- We have to hook up the old sibling with the grand parent node.
		// 2. The parent is the root:
		//	- Simply promote the old sibling to be the root.
		if (GrandParentIndex != INDEX_NONE)
		{
			FAabbTreeNode& GrandParent = Nodes[GrandParentIndex];
			int32& OldParentIndex = AabbTreeAlgorithm::GetIndexRef(GrandParent, ParentIndex);
			OldParentIndex = SiblingIndex;
			Sibling.Parent = GrandParentIndex;

			AabbTreeAlgorithm::RecomputeAncestorAabbsAndRotate(Nodes, GrandParentIndex, bDoRotation);
		}
		else
		{
			RootIndex = SiblingIndex;
			Sibling.Parent = INDEX_NONE;
		}

		AabbTreeAlgorithm::DeallocateNode(Nodes, FreeListHeadIndex, ParentIndex);
	}

	void FDynamicAabbTree::SelfQuery(FSelfQueryVisitor& Visitor, const int32 NodeIndex) const
	{
		// Doing a self query can be efficiently implemented by turning it into two separate queries:
		// 1. Do a tandem traversal of the left tree vs. the right tree.
		// 2. Recursively traverse left and right.
		const FAabbTreeNode& Node = Nodes[NodeIndex];
		if (AabbTreeAlgorithm::IsLeaf(Node))
		{
			return;
		}

		SelfQuery(Visitor, Node.Left);
		SelfQuery(Visitor, Node.Right);
		SelfQuery(Visitor, Node.Left, Node.Right);
	}

	void FDynamicAabbTree::SelfQuery(FSelfQueryVisitor& Visitor, const int32 Node0Index, const int32 Node1Index) const
	{
		const FAabbTreeNode& Node0 = Nodes[Node0Index];
		const FAabbTreeNode& Node1 = Nodes[Node1Index];
		// Node's don't intersect, there's nothing further to do
		if (!Node0.Aabb.Intersects(Node1.Aabb))
		{
			return;
		}

		const bool bIsNode0Leaf = AabbTreeAlgorithm::IsLeaf(Node0);
		const bool bIsNode1Leaf = AabbTreeAlgorithm::IsLeaf(Node1);
		if (bIsNode0Leaf && bIsNode1Leaf)
		{
			Visitor.Visit(Node0.UserData, Node1.UserData);
			return;
		}
		else if (bIsNode0Leaf)
		{
			SelfQuery(Visitor, Node0Index, Node1.Left);
			SelfQuery(Visitor, Node0Index, Node1.Right);
		}
		else if (bIsNode1Leaf)
		{
			SelfQuery(Visitor, Node0.Left, Node1Index);
			SelfQuery(Visitor, Node0.Right, Node1Index);
		}
		else
		{
			// Both nodes are internal. There's 3 choices: Split 0, Split 1, Split both.
			// Do a surface area heuristic to break down the larger one.
			// The idea is that the larger surface area will cause more overlaps, so we want to reduce that quicker.
			const FReal Area0 = Node0.Aabb.GetArea();
			const FReal Area1 = Node1.Aabb.GetArea();
			if (Area0 > Area1)
			{
				SelfQuery(Visitor, Node0.Left, Node1Index);
				SelfQuery(Visitor, Node0.Right, Node1Index);
			}
			else
			{
				SelfQuery(Visitor, Node0Index, Node1.Left);
				SelfQuery(Visitor, Node0Index, Node1.Right);
			}
		}
	}
} // namespace Chaos::SpatialPartition
