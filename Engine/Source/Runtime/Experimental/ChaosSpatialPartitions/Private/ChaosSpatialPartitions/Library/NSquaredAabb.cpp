// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSpatialPartitions/Library/NSquaredAabb.h"

namespace Chaos::SpatialPartition
{
	void FNSquaredAabb::Insert(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& OutHandle)
	{
		const int32 Index = AllocateEntry();
		OutHandle.SetValue(Index);

		FEntry& Entry = Entries[Index];
		Entry.Aabb = Aabb;
		Entry.UserData = UserData;
	}

	void FNSquaredAabb::Update(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& InOutHandle)
	{
		const int32 Index = (int32)InOutHandle.GetValue();
		check(0 <= Index && Index < Entries.Num());

		FEntry& Entry = Entries[Index];
		check(Entry.bIsUsed);
		Entry.Aabb = Aabb;
		Entry.UserData = UserData;
	}

	void FNSquaredAabb::Remove(FSpatialHandle& InOutHandle)
	{
		const int32 Index = (int32)InOutHandle.GetValue();
		check(0 <= Index && Index < Entries.Num());

		FEntry& Entry = Entries[Index];
		DeallocateEntry(Index);
	}

	EVisitResult FNSquaredAabb::Overlap(FOverlapQueryRuntimeData& QueryData, FOverlapVisitor& Visitor) const
	{
		return QueryImpl(QueryData, Visitor);
	}

	EVisitResult FNSquaredAabb::Raycast(FRaycastQueryRuntimeData& QueryData, FRaycastVisitor& Visitor) const
	{
		return QueryImpl(QueryData, Visitor);
	}

	EVisitResult FNSquaredAabb::Sweep(FSweepQueryRuntimeData& QueryData, FSweepVisitor& Visitor) const
	{
		return QueryImpl(QueryData, Visitor);
	}

	void FNSquaredAabb::SelfQuery(FSelfQueryVisitor& Visitor) const
	{
		// This is written with iterators in the hope that this can use `THandleArray` eventually.
		using FIterator = TArray<FEntry>::RangedForConstIteratorType;
		const FIterator End = Entries.end();
		for (FIterator It0 = Entries.begin(); It0 != End; ++It0)
		{
			const FEntry& Entry0 = *It0;
			if (!Entry0.bIsUsed)
			{
				continue;
			}

			FIterator It1 = It0;
			++It1;
			for (; It1 != End; ++It1)
			{
				const FEntry& Entry1 = *It1;
				if (!Entry1.bIsUsed)
				{
					continue;
				}

				if (!Entry0.Aabb.Intersects(Entry1.Aabb))
				{
					continue;
				}

				Visitor.Visit(Entry0.UserData, Entry1.UserData);
			}
		}
	}

	template <typename QueryDataType, typename VisitorType>
	EVisitResult FNSquaredAabb::QueryImpl(QueryDataType& QueryData, VisitorType& Visitor) const
	{
		for (const FEntry& Entry : Entries)
		{
			if (Entry.bIsUsed)
			{
				if (ShouldVisitEntry(Entry, QueryData))
				{
					const EVisitResult VisitResult = Visitor.Visit(Entry.UserData, QueryData);
					if (VisitResult == EVisitResult::Stop)
					{
						return EVisitResult::Stop;
					}
				}
			}
		}
		return EVisitResult::Continue;
	}

	bool FNSquaredAabb::ShouldVisitEntry(const FEntry& Entry, const FOverlapQueryRuntimeData& QueryData) const
	{
		return QueryData.Test(Entry.Aabb);
	}

	bool FNSquaredAabb::ShouldVisitEntry(const FEntry& Entry, const FRaycastQueryRuntimeData& QueryData) const
	{
		return QueryData.Test(Entry.Aabb);
	}

	bool FNSquaredAabb::ShouldVisitEntry(const FEntry& Entry, const FSweepQueryRuntimeData& QueryData) const
	{
		return QueryData.Test(Entry.Aabb);
	}

	int32 FNSquaredAabb::AllocateEntry()
	{
		int32 Index;
		if (!FreeIndices.IsEmpty())
		{
			Index = FreeIndices.Pop(EAllowShrinking::No);
		}
		else
		{
			Index = Entries.Emplace();
		}
		check(!Entries[Index].bIsUsed);
		Entries[Index].bIsUsed = true;
		return Index;
	}

	void FNSquaredAabb::DeallocateEntry(int32 Index)
	{
		check(Entries[Index].bIsUsed);
		Entries[Index].bIsUsed = false;
		FreeIndices.Add(Index);
	}
} // namespace Chaos::SpatialPartition
