// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSpatialPartitions/Library/StaticAabbTree.h"

#include "ChaosSpatialPartitions/Algorithms/AabbTreeAlgorithm.h"
#include "ChaosSpatialPartitions/Algorithms/StaticAabbTreeTimeSlicer.h"

#include "ChaosCheck.h"
#include "HAL/PlatformTime.h"

namespace Chaos::SpatialPartition
{
	FStaticAabbTree::FRebuildContext::FRebuildContext()
	{
	}

	FStaticAabbTree::FRebuildContext::FRebuildContext(const FStaticAabbTree* OwningTree, AabbTreeAlgorithm::FStaticAabbTreeTimeSlicer&& TimeSlicer)
		: OwningTree(OwningTree)
		, TimeSlicer(MoveTemp(TimeSlicer))
		, TargetTimeSeconds(OwningTree->Config.TargetProcessingTimeInSeconds)
	{
	}

	FStaticAabbTree::ERebuildStatus FStaticAabbTree::FRebuildContext::Run()
	{
		const double StartTimeSeconds = FPlatformTime::Seconds();
		// Run batches until we finish or don't have enough time to continue.
		ERebuildStatus Result = TimeSlicer.Run() ? ERebuildStatus::Continue : ERebuildStatus::Finished;
		while (Result != ERebuildStatus::Finished)
		{
			const double CurrentTime = FPlatformTime::Seconds();
			const double ElapsedTime = CurrentTime - StartTimeSeconds;
			if (ElapsedTime > TargetTimeSeconds)
			{
				break;
			}
			Result = TimeSlicer.Run() ? ERebuildStatus::Continue : ERebuildStatus::Finished;
		}
		return Result;
	}

	FStaticAabbTree::FStaticAabbTree()
	{
	}

	FStaticAabbTree::FStaticAabbTree(const FConfig& Config)
		: Config(Config)
	{
	}

	void FStaticAabbTree::InsertDeferred(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& OutHandle)
	{
		const int32 EntryIndex = AllocateEntry();
		UpdateEntry(EntryIndex, UserData, Aabb);
		OutHandle.SetValue(EntryIndex);
	}

	bool FStaticAabbTree::UpdateIfWithinLeaf(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& InOutHandle)
	{
		const int32 EntryIndex = (int32)InOutHandle.GetValue();
		if (!ensure(EntryIndex != INDEX_NONE))
		{
			return false;
		}

		FEntry& Entry = Entries[EntryIndex];

		if (Leaves.IsValidIndex(Entry.LeafIndex))
		{
			const FAabbTreeLeaf& Leaf = Leaves[Entry.LeafIndex];
			if (Leaf.Aabb.Contains(Aabb))
			{
				Entry.Aabb = Aabb;
				Entry.UserData = UserData;
				return true;
			}
		}
		return false;
	}

	void FStaticAabbTree::UpdateDeferred(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& InOutHandle)
	{
		const int32 EntryIndex = (int32)InOutHandle.GetValue();
		if (ensure(EntryIndex != INDEX_NONE))
		{
			UpdateEntry(EntryIndex, UserData, Aabb);
		}
	}

	void FStaticAabbTree::RemoveMinimal(FSpatialHandle& InOutHandle)
	{
		const int32 EntryIndex = (int32)InOutHandle.GetValue();
		InOutHandle.SetValue(INDEX_NONE);
		if (!ensure(EntryIndex != INDEX_NONE))
		{
			return;
		}

		// Do the absolute minimal to remove the object: Remove the object from the leaf and free the entry.
		RemoveEntryFromLeaf(EntryIndex);
		FreeEntry(EntryIndex);
	}

	void FStaticAabbTree::BeginRebuild(FRebuildContext& Context) const
	{
		TArray<FAabbTreeLeafElement> Elements;
		GatherUsedElements(Elements);

		AabbTreeAlgorithm::FStaticAabbTreeTimeSlicer::FConfig TimeSlicerConfig
		{
			.PartitioningMethod = Config.PartitioningMethod,
			.BatchSize = Config.BatchSize,
			.MaxElementsPerLeaf = Config.MaxElementsPerLeaf,
			.MaxTreeDepth = Config.MaxTreeDepth,
			.SurfaceAreaHeuristicBinCount = Config.SurfaceAreaHeuristicBinCount,
		};
		Context = FRebuildContext(this, AabbTreeAlgorithm::FStaticAabbTreeTimeSlicer(TimeSlicerConfig, MoveTemp(Elements)));
	}

	void FStaticAabbTree::CommitRebuild(FRebuildContext& Context)
	{
		if (!ensureMsgf(Context.OwningTree == this, TEXT("StaticAabbTree can only commit a rebuild context from the same tree.")))
		{
			return;
		}
		if (!ensureMsgf(Context.TimeSlicer.IsFinished(), TEXT("StaticAabbTree can only commit a completed rebuild context.")))
		{
			return;
		}

		RootIndex = Context.TimeSlicer.GetRootIndex();
		Nodes = MoveTemp(Context.TimeSlicer.GetNodes());
		Leaves = MoveTemp(Context.TimeSlicer.GetLeaves());

		// Do a final pass to hook the entries back up to the leaves
		RecomputeElementLeafIndices();
	}

	EVisitResult FStaticAabbTree::Overlap(FOverlapQueryRuntimeData& QueryData, FOverlapVisitor& Visitor) const
	{
		auto LeafCallback = [this, &QueryData, &Visitor](const int32 NodeIndex, const FAabbTreeNode& Node) -> EVisitResult
		{
			return CastLeafCallback(Node, QueryData, Visitor);
		};

		return AabbTreeAlgorithm::Query(Nodes, RootIndex, QueryData, LeafCallback);
	}

	EVisitResult FStaticAabbTree::Raycast(FRaycastQueryRuntimeData& QueryData, FRaycastVisitor& Visitor) const
	{
		auto LeafCallback = [this, &QueryData, &Visitor](const int32 NodeIndex, const FAabbTreeNode& Node) -> EVisitResult
		{
			return CastLeafCallback(Node, QueryData, Visitor);
		};
		return AabbTreeAlgorithm::Query(Nodes, RootIndex, QueryData, LeafCallback);
	}

	EVisitResult FStaticAabbTree::Sweep(FSweepQueryRuntimeData& QueryData, FSweepVisitor& Visitor) const
	{
		auto LeafCallback = [this, &QueryData, &Visitor](const int32 NodeIndex, const FAabbTreeNode& Node) -> EVisitResult
		{
			return CastLeafCallback(Node, QueryData, Visitor);
		};
		return AabbTreeAlgorithm::Query(Nodes, RootIndex, QueryData, LeafCallback);
	}

	FStaticAabbTree::FStats FStaticAabbTree::ComputeStats() const
	{
		if (RootIndex == INDEX_NONE)
		{
			return FStats();
		}
		return ComputeStats(RootIndex);
	}

	void FStaticAabbTree::Dump(TArray<FAabbTreeNode>& OutNodes, int32& OutRootNodeIndex, TArray<FAabbTreeLeaf>& OutLeaves) const
	{
		OutNodes = Nodes;
		OutRootNodeIndex = RootIndex;
		OutLeaves = Leaves;
		// Convert the index in the entries to the user data
		for (FAabbTreeLeaf& Leaf : OutLeaves)
		{
			for (FAabbTreeLeafElement& Element : Leaf.Elements)
			{
				const FEntry& Entry = Entries[Element.Index];
				Element.Index = Entry.UserData;
			}
		}
	}

	int32 FStaticAabbTree::AllocateEntry()
	{
		if (EntriesFreeListHead != INDEX_NONE)
		{
			const int32 Index = EntriesFreeListHead;
			EntriesFreeListHead = Entries[Index].UserData;

			Entries[Index] = FEntry();
			UsedEntries[Index] = true;
			return Index;
		}
		else
		{
			const int32 Index = Entries.Emplace();
			UsedEntries.Add(true);
			return Index;
		}
	}

	void FStaticAabbTree::UpdateEntry(const int32 EntryIndex, const FUserDataType& UserData, const FAABB3& Aabb)
	{
		check(Entries.IsValidIndex(EntryIndex));
		FEntry& Entry = Entries[EntryIndex];
		Entry.Aabb = Aabb;
		Entry.UserData = UserData;
	}

	void FStaticAabbTree::FreeEntry(const int32 EntryIndex)
	{
		Entries[EntryIndex].UserData = EntriesFreeListHead;
		Entries[EntryIndex].Aabb = FAABB3::EmptyAABB();
		EntriesFreeListHead = EntryIndex;
		UsedEntries[EntryIndex] = false;
	}

	int32 FStaticAabbTree::AllocateNode()
	{
		return AabbTreeAlgorithm::AllocateNode(Nodes, NodesFreeListHead);
	}

	void FStaticAabbTree::FreeNode(const int32 NodeIndex)
	{
		AabbTreeAlgorithm::DeallocateNode(Nodes, NodesFreeListHead, NodeIndex);
	}

	int32 FStaticAabbTree::AllocateLeaf()
	{
		if (LeavesFreeListHead != INDEX_NONE)
		{
			const int32 Index = LeavesFreeListHead;
			LeavesFreeListHead = Leaves[Index].NodeIndex;

			Leaves[Index] = FAabbTreeLeaf();
			return Index;
		}
		else
		{
			const int32 Index = Leaves.Emplace();
			return Index;
		}
	}

	void FStaticAabbTree::FreeLeaf(const int32 LeafIndex)
	{
		Leaves[LeafIndex].Elements.Reset();
		Leaves[LeafIndex].NodeIndex = LeavesFreeListHead;
		Leaves[LeafIndex].Aabb = FAABB3::EmptyAABB();
		LeavesFreeListHead = LeafIndex;
	}

	FStaticAabbTree::FStats FStaticAabbTree::ComputeStats(const int32 NodeIndex) const
	{
		FStats Stats;

		const FAabbTreeNode& Node = Nodes[NodeIndex];
		if (AabbTreeAlgorithm::IsLeaf(Node))
		{
			const FAabbTreeLeaf& Leaf = Leaves[Node.UserData];
			Stats.MinElementsPerLeaf = Stats.MaxElementsPerLeaf = Leaf.Elements.Num();
			Stats.MinHeight = Stats.MaxHeight = 0;
			// Note: Skipping the leaf's surface area since it's currently just a tight fitting aabb around the elements.
			Stats.TotalSurfaceArea = Stats.NodeSurfaceArea = Node.Aabb.GetArea();
			for (const FAabbTreeLeafElement& Element : Leaf.Elements)
			{
				Stats.TotalSurfaceArea += Element.Aabb.GetArea();
			}
		}
		else
		{
			const FStats LeftStats = ComputeStats(Node.Left);
			const FStats RightStats = ComputeStats(Node.Right);
			Stats.MinHeight = FMath::Min(LeftStats.MinHeight, RightStats.MinHeight) + 1;
			Stats.MaxHeight = FMath::Max(LeftStats.MaxHeight, RightStats.MaxHeight) + 1;
			Stats.MinElementsPerLeaf = FMath::Min(LeftStats.MinElementsPerLeaf, RightStats.MinElementsPerLeaf);
			Stats.MaxElementsPerLeaf = FMath::Max(LeftStats.MaxElementsPerLeaf, RightStats.MaxElementsPerLeaf);
			Stats.NodeSurfaceArea += LeftStats.NodeSurfaceArea + RightStats.NodeSurfaceArea;
			Stats.TotalSurfaceArea += LeftStats.TotalSurfaceArea + RightStats.TotalSurfaceArea;
		}
		return Stats;
	}

	void FStaticAabbTree::GatherUsedElements(TArray<FAabbTreeLeafElement>& Elements) const
	{
		const int32 EntryCount = Entries.Num();
		Elements.Reset();
		Elements.Reserve(EntryCount);
		for (int32 I = 0; I < EntryCount; ++I)
		{
			if (UsedEntries[I])
			{
				Elements.Emplace(FAabbTreeLeafElement{ .Aabb = Entries[I].Aabb, .Index = I });
			}
		}
	}

	void FStaticAabbTree::RecomputeElementLeafIndices()
	{
		// For safety, clear old leaf entries so there's no dangling indices. This should only affect leaves removed during a rebuild.
		for (FEntry& Entry : Entries)
		{
			Entry.LeafIndex = INDEX_NONE;
		}

		// Hook each entry back up to its leaf. If any entries have been removed since this rebuild,
		// make sure to cleanup the leaf so there's no stale references.
		for (int32 LeafIndex = 0; LeafIndex < Leaves.Num(); ++LeafIndex)
		{
			FAabbTreeLeaf& LeafNode = Leaves[LeafIndex];
			bool bContainsRemovedEntries = false;
			for (const FAabbTreeLeafElement& Element : LeafNode.Elements)
			{
				Entries[Element.Index].LeafIndex = LeafIndex;
				bContainsRemovedEntries |= !UsedEntries[Element.Index];
			}
			// There's at least one entry in the leaf that has been removed. Get them all out of the leaf.
			if (bContainsRemovedEntries)
			{
				LeafNode.Elements.RemoveAllSwap([this](const FAabbTreeLeafElement& Element) { return UsedEntries[Element.Index] == false; });
			}
		}
	}

	void FStaticAabbTree::RemoveEntryFromLeaf(const int32 EntryIndex)
	{
		FEntry& Entry = Entries[EntryIndex];
		// Leaf might never be set if the object was inserted and the tree was never rebuilt
		if (Entry.LeafIndex != INDEX_NONE)
		{
			FAabbTreeLeaf& LeafNode = Leaves[Entry.LeafIndex];
			LeafNode.Elements.RemoveAllSwap([EntryIndex](const FAabbTreeLeafElement& Element) {return Element.Index == EntryIndex; });
			Entry.LeafIndex = INDEX_NONE;
		}
	}

	template <typename QueryDataType, typename VisitorType>
	EVisitResult FStaticAabbTree::CastLeafCallback(const FAabbTreeNode& Node, QueryDataType& QueryData, VisitorType& Visitor) const
	{
		const int32 LeafIndex = Node.UserData;
		const FAabbTreeLeaf& Leaf = Leaves[LeafIndex];
		for (const FAabbTreeLeafElement& Element : Leaf.Elements)
		{
			if (QueryData.Test(Element.Aabb))
			{
				const FEntry& Entry = Entries[Element.Index];
				const EVisitResult VisitResult = Visitor.Visit(Entry.UserData, QueryData);
				if (VisitResult == EVisitResult::Stop)
				{
					return EVisitResult::Stop;
				}
			}
		}
		return EVisitResult::Continue;
	}
} // namespace Chaos::SpatialPartition
