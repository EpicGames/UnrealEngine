// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSpatialPartitions/QueryData.h"

namespace Chaos::SpatialPartition
{
	FReal SafeInverse(const FReal Value)
	{
		return (FMath::Abs(Value) < UE_SMALL_NUMBER) ? 0 : (1 / Value);
	}

	bool IsParallel(const FReal Value)
	{
		return FMath::Abs(Value) < UE_SMALL_NUMBER;
	}

	FQueryRuntimeData::FQueryRuntimeData(const FVec3& Dir, const FReal Length)
		: InvDir(SafeInverse(Dir[0]), SafeInverse(Dir[1]), SafeInverse(Dir[2]))
		, bParallel{ IsParallel(Dir[0]), IsParallel(Dir[1]), IsParallel(Dir[2]) }
	{
		SetLength(Length);
	}

	void FQueryRuntimeData::SetLength(const FReal Length) 
	{
		CurrentLength = Length;
		if (CurrentLength != 0.0f)
		{
			InvCurrentLength = 1 / CurrentLength;
		}
		else
		{
			InvCurrentLength = 0;
		}
	}

	FOverlapQueryRuntimeData::FOverlapQueryRuntimeData(const FOverlapQueryData& InputData)
		: InputData(InputData)
	{
	}

	FOverlapQueryRuntimeData::FOverlapQueryRuntimeData(const FAABB3& Aabb)
		: InputData(FOverlapQueryData{ .Aabb = Aabb })
	{
	}

	const FOverlapQueryData& FOverlapQueryRuntimeData::GetInputData() const
	{
		return InputData;
	}

	bool FOverlapQueryRuntimeData::Test(const FAABB3& Aabb) const
	{
		return Aabb.Intersects(InputData.Aabb);
	}

	FRaycastQueryRuntimeData::FRaycastQueryRuntimeData(const FRaycastQueryData& InputData)
		: InputData(InputData)
		, RuntimeData(InputData.Direction, InputData.Length)
	{
	}

	const FRaycastQueryData& FRaycastQueryRuntimeData::GetInputData() const
	{
		return InputData;
	}

	FReal FRaycastQueryRuntimeData::GetCurrentLength() const
	{
		return RuntimeData.CurrentLength;
	}

	void FRaycastQueryRuntimeData::SetCurrentLength(const FReal InLength)
	{
		RuntimeData.SetLength(InLength);
	}

	bool FRaycastQueryRuntimeData::Test(const FAABB3& Aabb) const
	{
		FReal Time;
		return Test(Aabb, Time);
	}

	bool FRaycastQueryRuntimeData::Test(const FAABB3& Aabb, FReal& OutTime) const
	{
		FVec3 Position;
		return Aabb.RaycastFast(InputData.Start, InputData.Direction, RuntimeData.InvDir, RuntimeData.bParallel, RuntimeData.CurrentLength, RuntimeData.InvCurrentLength, OutTime, Position);
	}

	FSweepQueryRuntimeData::FSweepQueryRuntimeData(const FSweepQueryData& InputData)
		: InputData(InputData)
		, RuntimeData(InputData.Direction, InputData.Length)
	{
	}

	const FSweepQueryData& FSweepQueryRuntimeData::GetInputData() const
	{
		return InputData;
	}

	FReal FSweepQueryRuntimeData::GetCurrentLength() const
	{
		return RuntimeData.CurrentLength;
	}

	void FSweepQueryRuntimeData::SetCurrentLength(const FReal InLength)
	{
		RuntimeData.SetLength(InLength);
	}

	bool FSweepQueryRuntimeData::Test(const FAABB3& Aabb) const
	{
		FReal Time;
		return Test(Aabb, Time);
	}

	bool FSweepQueryRuntimeData::Test(const FAABB3& Aabb, FReal& OutTime) const
	{
		FAABB3 ExpandedAabb(Aabb.Min() - InputData.HalfExtents, Aabb.Max() + InputData.HalfExtents);

		FVec3 Position;
		return ExpandedAabb.RaycastFast(InputData.Start, InputData.Direction, RuntimeData.InvDir, RuntimeData.bParallel, RuntimeData.CurrentLength, RuntimeData.InvCurrentLength, OutTime, Position);
	}
} // namespace Chaos::SpatialPartition
