// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSpatialPartitions/SpatialClassification.h"

namespace Chaos::SpatialPartition
{
	FString ToString(ESpatialCategory Category)
	{
		switch (Category)
		{
			case ESpatialCategory::Static:
				return "Static";
			case ESpatialCategory::Kinematic:
				return "Kinematic";
			case ESpatialCategory::Dynamic:
				return "Dynamic";
			default:
				return "Invalid";
		}
	}

	FString ToString(ESpatialCategoryMask Mask)
	{
		using FMaskNamePair = TPair< ESpatialCategoryMask, const char*>;
		static const TArray<FMaskNamePair> MaskNames
		{
			FMaskNamePair(ESpatialCategoryMask::Static, "Static"),
			FMaskNamePair(ESpatialCategoryMask::Kinematic, "Kinematic"),
			FMaskNamePair(ESpatialCategoryMask::Dynamic, "Dynamic"),
		};

		TStringBuilder<32> Builder;
		int32 Count = 0;
		for (int32 I = 0; I < MaskNames.Num(); ++I)
		{
			if (EnumHasAnyFlags(Mask, MaskNames[I].Key))
			{
				if (Count != 0)
				{
					Builder << "|";
				}
				Builder << MaskNames[I].Value;
				++Count;
			}
		}
		return FString(Builder.ToString());
	}

	ESpatialCategoryMask ToCategoryMask(const ESpatialCategory Category)
	{
		check(Category != ESpatialCategory::Invalid);
		uint32 CategoryValue = (uint32)(Category);
		return (ESpatialCategoryMask)(1 << CategoryValue);
	}

	FSpatialClassification::FSpatialClassification(ESpatialCategory InCategory)
		: Category(InCategory)
	{
	}

	ESpatialCategory FSpatialClassification::GetCategory() const
	{
		return Category;
	}

	void FSpatialClassification::SetCategory(ESpatialCategory InCategory)
	{
		Category = InCategory;
	}
} // namespace Chaos::SpatialPartition
