// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSpatialPartitions/SpatialHandle.h"

namespace Chaos::SpatialPartition
{
	int64 FSpatialHandle::GetValue() const
	{
		return Value;
	}

	void FSpatialHandle::SetValue(int64 InValue)
	{
		Value = InValue;
	}
} // namespace Chaos::SpatialPartition
