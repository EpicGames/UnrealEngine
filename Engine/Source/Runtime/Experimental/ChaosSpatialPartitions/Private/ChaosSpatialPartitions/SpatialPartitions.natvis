<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <Type Name="Chaos::SpatialPartition::FAabbTreeNode">
    <Intrinsic Name="IsLeaf" Expression="Index == -1">
      <Parameter Name="Index" Type="unsigned int" />
    </Intrinsic>
    <!--The last valid version of GetNode is chosen. This handles the ifdef for DebugNodeList existing. If it doesn't exist, the index is returned instead of the node.-->
    <Intrinsic Name="GetNode" Expression="Index">
      <Parameter Name="Index" Type="unsigned int" />
    </Intrinsic>
    <Intrinsic Optional="true" Name="GetNode" Expression="(*DebugNodeList)[Index]">
      <Parameter Name="Index" Type="unsigned int" />
    </Intrinsic>

    <DisplayString Condition="UserData!=-1">Leaf: {UserData}</DisplayString>
    <DisplayString>Internal: ({Left}, {Right})</DisplayString>
    <Expand>
      <Item Name="Aabb">Aabb</Item>
      <Item Name="Parent">Parent</Item>
      <Item Name="Left">Left</Item>
      <Item Name="Right">Right</Item>
      <Item Name="UserData" Condition="IsLeaf(Right)">UserData</Item>
      <Item Name="ParentNode" Condition="!IsLeaf(Parent)">GetNode(Parent)</Item>
      <Item Name="LeftNode" Condition="!IsLeaf(Left)">GetNode(Left)</Item>
      <Item Name="RightNode" Condition="!IsLeaf(Right)">GetNode(Right)</Item>
    </Expand>
  </Type>

  <Type Name="Chaos::SpatialPartition::FAabbTreeLeafElement">
    <DisplayString>{Index}</DisplayString>
  </Type>


  <Type Name="Chaos::SpatialPartition::FDynamicAabbTree">
    <Intrinsic Name="GetNode" Expression="Nodes[Index]">
      <Parameter Name="Index" Type="unsigned int" />
    </Intrinsic>
    <DisplayString>Tree</DisplayString>
    <Expand>
      <Item Name="Root">GetNode(RootIndex)</Item>
      <Item Name="Config">Config</Item>
    </Expand>
  </Type>

</AutoVisualizer>
