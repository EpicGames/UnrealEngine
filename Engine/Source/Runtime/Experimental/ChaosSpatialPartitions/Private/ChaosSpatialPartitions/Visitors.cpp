// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosSpatialPartitions/Visitors.h"

namespace Chaos::SpatialPartition
{
	FOverlapVisitor::FOverlapVisitor(FCallbackType Callback, void* CallbackData)
		: Callback(Callback)
		, CallbackData(CallbackData)
	{
		check(Callback != nullptr);
	}

	EVisitResult FOverlapVisitor::Visit(const FUserDataType& UserData, FOverlapQueryRuntimeData& QueryRuntimeData)
	{
		return Callback(CallbackData, UserData, QueryRuntimeData);
	}

	FRaycastVisitor::FRaycastVisitor(FCallbackType Callback, void* CallbackData)
		: Callback(Callback)
		, CallbackData(CallbackData)
	{
		check(Callback != nullptr);
	}

	EVisitResult FRaycastVisitor::Visit(const FUserDataType& UserData, FRaycastQueryRuntimeData& QueryRuntimeData)
	{
		return Callback(CallbackData, UserData, QueryRuntimeData);
	}

	FSweepVisitor::FSweepVisitor(FCallbackType Callback, void* CallbackData)
		: Callback(Callback)
		, CallbackData(CallbackData)
	{
		check(Callback != nullptr);
	}

	EVisitResult FSweepVisitor::Visit(const FUserDataType& UserData, FSweepQueryRuntimeData& QueryRuntimeData)
	{
		return Callback(CallbackData, UserData, QueryRuntimeData);
	}

	FSelfQueryVisitor::FSelfQueryVisitor(FCallbackType Callback, void* CallbackData)
		: Callback(Callback)
		, CallbackData(CallbackData)
	{
		check(Callback != nullptr);
	}

	void FSelfQueryVisitor::Visit(const FUserDataType& UserData0, const FUserDataType& UserData1)
	{
		Callback(CallbackData, UserData0, UserData1);
	}
} // namespace Chaos::SpatialPartition
