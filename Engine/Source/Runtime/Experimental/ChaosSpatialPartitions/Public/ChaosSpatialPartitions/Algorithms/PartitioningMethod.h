// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	// Methods for partitioning objects when building a spatial partition.
	enum EPartitioningMethod
	{
		// Split using the spatial median (aabb center) of the object centers.
		CentroidSpatialMedian,
		// Split using the variance of the object centers.
		CentroidVariance,
		// Split using a surface area heuristic.
		SurfaceArea,
	};
} // namespace Chaos::SpatialPartition
