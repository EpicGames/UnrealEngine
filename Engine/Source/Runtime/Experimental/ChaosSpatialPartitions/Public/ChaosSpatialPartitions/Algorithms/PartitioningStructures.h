// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Common.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition::AabbTreeAlgorithm
{
	struct UE_INTERNAL FAabbBin
	{
		FAABB3 Aabb = FAABB3::EmptyAABB();
		int32 Count = 0;
	};

	struct UE_INTERNAL FBinSplitPlane
	{
		FAABB3 LeftAabb = FAABB3::EmptyAABB();
		FAABB3 RightAabb = FAABB3::EmptyAABB();
		int32 LeftCount = 0;
		int32 RightCount = 0;
	};
} // namespace Chaos::SpatialPartition::AabbTreeAlgorithm
