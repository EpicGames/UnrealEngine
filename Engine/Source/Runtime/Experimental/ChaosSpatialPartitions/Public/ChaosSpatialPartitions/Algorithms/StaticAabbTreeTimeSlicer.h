// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Algorithms/PartitioningMethod.h"
#include "ChaosSpatialPartitions/Algorithms/PartitioningStructures.h"
#include "ChaosSpatialPartitions/Library/AabbTreeLeaf.h"
#include "ChaosSpatialPartitions/Library/AabbTreeNode.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition::AabbTreeAlgorithm
{
	// Builds a static aabb tree incrementally. This allows building over several frames to fit within a time budget.
	struct UE_INTERNAL FStaticAabbTreeTimeSlicer
	{
	public:
		struct FConfig
		{
			EPartitioningMethod PartitioningMethod = EPartitioningMethod::CentroidVariance;

			int32 BatchSize = std::numeric_limits<int32>::max();
			int32 MaxElementsPerLeaf = 8;
			int32 MaxTreeDepth = 200;
			int32 SurfaceAreaHeuristicBinCount = 8;
		};

		FStaticAabbTreeTimeSlicer() = default;
		UE_INTERNAL CHAOSSPATIALPARTITIONS_API FStaticAabbTreeTimeSlicer(const FConfig& InConfig, TArray<FAabbTreeLeafElement>&& InElements);
		FStaticAabbTreeTimeSlicer(FStaticAabbTreeTimeSlicer&&) = default;

		FStaticAabbTreeTimeSlicer& operator=(FStaticAabbTreeTimeSlicer&&) = default;

		// Runs the next iteration of work. Returns true as long as there is remaining work to do.
		UE_INTERNAL CHAOSSPATIALPARTITIONS_API bool Run();
		UE_INTERNAL CHAOSSPATIALPARTITIONS_API bool IsFinished() const;

		UE_INTERNAL CHAOSSPATIALPARTITIONS_API int32 GetRootIndex() const;
		UE_INTERNAL CHAOSSPATIALPARTITIONS_API TArray<FAabbTreeNode>& GetNodes();
		UE_INTERNAL CHAOSSPATIALPARTITIONS_API TArray<FAabbTreeLeaf>& GetLeaves();

	private:
		enum class EState
		{
			Setup,
			Partition,
			Finish,
		};

		struct FWorkload;

		int32 AllocateNode();
		int32 BuildLeaf(const TArrayView<const FAabbTreeLeafElement>& Elements, const int32 NodeIndex);

		void AddNewWorkload(const TArrayView<const FAabbTreeLeafElement>& Elements, const int32 NodeIndex, const int32 Depth);
		void SplitWorkload(const FWorkload& ParentWorkload, const int32 SplitIndex);

		void Initialize(FWorkload& Workload);
		void RunSetup(FWorkload& Workload);
		void RunPartition(FWorkload& Workload);
		void RunFinish(FWorkload& Workload);

		// Incrementally compute the centroid aabb for a workload. Returns true if the computation is finished.
		bool ComputeCentroidAabb(FWorkload& Workload, int32& Start, FAABB3& OutCentroidAabb) const;
		// Incrementally compute the variance of the centroids for a workload. Returns true if the computation is finished.
		bool ComputeCentroidVariance(FWorkload& Workload, int32& Start, int32& ProcessedCount, FVec3& OutAverageCenter, FVec3& OutScaledCenterVariance) const;

		struct FMedianWorkloadCommon
		{
			enum class EPartitionState
			{
				Select,
				Partition,
			};
			EPartitionState PartitionState = EPartitionState::Select;

			FAABB3 CentroidAabb = FAABB3::EmptyAABB();
			int32 Start = 0;
			int32 End = 0;
			int32 SplitAxis = 0;
			FReal SplitPosition = 0;
		};

		struct FCentroidSpatialMedianWorkload : public FMedianWorkloadCommon
		{
			void Reset();
			void Run(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload);

		private:
			void RunSelect(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload);
			void RunPartition(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload);
		};

		struct FCentroidVarianceWorkload : public FMedianWorkloadCommon
		{
			void Reset();
			void Run(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload);

		private:
			void RunSelect(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload);
			void RunPartition(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload);

			FVec3 AverageCenter = FVec3::Zero();
			FVec3 ScaledCenterVariance = FVec3::Zero();
			int32 ProcessedCount = 0;
		};

		struct FBinnedSurfaceAreaWorkload
		{
			void Reset();
			void Run(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload);

		private:
			void RunComputeBounds(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload);
			void RunComputeBins(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload);
			void RunPartition(FStaticAabbTreeTimeSlicer& TimeSlicer, FWorkload& Workload);

			enum class EPartitionState
			{
				ComputeBounds,
				ComputeBins,
				Partition,
			};
			EPartitionState State = EPartitionState::ComputeBounds;

			FAABB3 CentroidAabb = FAABB3::EmptyAABB();
			TArray<FAabbBin> Bins;
			TArray<int32> BinIndices;
			TArray<FBinSplitPlane> BinSplitPlanes;
			FReal BoundsLowerBound = 0;
			FReal InvBoundsSize = 0;
			int32 Start = 0;
			int32 End = 0;
			int32 SplitAxis = 0;
			int32 BinSplitIndex = 0;
			int32 SplitIndex = 0;
		};

		struct FWorkload
		{
			TArrayView<const FAabbTreeLeafElement> Elements;
			int32 NodeIndex = INDEX_NONE;
			int32 Depth = 0;
			EState State = EState::Setup;
		};

		FConfig Config;
		TArray<FAabbTreeLeafElement> Elements;
		TArray<FAabbTreeNode> Nodes;
		TArray<FAabbTreeLeaf> Leaves;

		// This data is shared across all workloads since we only ever process one workload at a time. This allows sharing memory/allocations.
		FCentroidSpatialMedianWorkload CentroidSpatialMedianWorkload;
		FCentroidVarianceWorkload CentroidVarianceWorkload;
		FBinnedSurfaceAreaWorkload BinnedSurfaceAreaWorkload;

		int32 RootIndex = INDEX_NONE;
		TArray<FWorkload> WorkloadStack;
	};
} // namespace Chaos::SpatialPartition::AabbTreeAlgorithm
