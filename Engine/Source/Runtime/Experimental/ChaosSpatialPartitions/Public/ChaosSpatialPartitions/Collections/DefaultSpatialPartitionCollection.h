// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/ISpatialPartition.h"

#include "ChaosSpatialPartitions/Library/DynamicAabbTree.h"
#include "ChaosSpatialPartitions/Library/StaticAabbTree.h"
#include "ChaosSpatialPartitions/Library/NSquaredAabb.h"
#include "Chaos/HandleArray.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	// The default spatial partition collection to manage dynamic and static objects.
	// Static object updates are handled by moving objects between clean/dirty states 
	// where the structures can be fully rebuilt at a larger cost.
	// This is meant to be the default implementation for Chaos.
	class FDefaultSpatialPartitionCollection : public ISpatialPartition
	{
	public:
		struct FConfig
		{
			FStaticAabbTree::FConfig StaticConfig;
			FDynamicAabbTree::FConfig DynamicConfig;
		};

		CHAOSSPATIALPARTITIONS_API FDefaultSpatialPartitionCollection(const FConfig& Config = FConfig());
		FDefaultSpatialPartitionCollection(FDefaultSpatialPartitionCollection&&) = default;
		virtual ~FDefaultSpatialPartitionCollection() override = default;

		FDefaultSpatialPartitionCollection& operator=(FDefaultSpatialPartitionCollection&&) = default;

		CHAOSSPATIALPARTITIONS_API virtual void Insert(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& OutHandle) override;
		CHAOSSPATIALPARTITIONS_API virtual void Update(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& InOutHandle) override;
		CHAOSSPATIALPARTITIONS_API virtual void Remove(FSpatialHandle& InOutHandle) override;

		CHAOSSPATIALPARTITIONS_API virtual void Overlap(const FOverlapQueryData& QueryData, FOverlapVisitor& Visitor, ESpatialCategoryMask CategoryMask = ESpatialCategoryMask::All) const override;
		CHAOSSPATIALPARTITIONS_API virtual void Raycast(const FRaycastQueryData& QueryData, FRaycastVisitor& Visitor, ESpatialCategoryMask CategoryMask = ESpatialCategoryMask::All) const override;
		CHAOSSPATIALPARTITIONS_API virtual void Sweep(const FSweepQueryData& QueryData, FSweepVisitor& Visitor, ESpatialCategoryMask CategoryMask = ESpatialCategoryMask::All) const override;

		CHAOSSPATIALPARTITIONS_API virtual bool NeedsRebuilding() const override;
		CHAOSSPATIALPARTITIONS_API virtual ERebuildStatus Rebuild() override;
		// TODO: Timeslice copying

	private:
		enum class EInternalCategory : uint8
		{
			None = 0,
			Static = 1 << 1,
			Dynamic = 1 << 2,
			Dirty = 1 << 3,
			Global = 1 << 4,
			StaticDirty = Static | Dirty,
			StaticGlobal = Static | Global,
			DynamicDirty = Dynamic | Dirty,
			DynamicGlobal = Dynamic | Global,
		};
		FRIEND_ENUM_CLASS_FLAGS(EInternalCategory);
		struct FEntry
		{
			FAABB3 Aabb = FAABB3::EmptyAABB();
			FSpatialHandle Handle;
			FUserDataType UserData = INDEX_NONE;
			ESpatialCategory Category = ESpatialCategory::Invalid;
			EInternalCategory InternalCategory = EInternalCategory::None;
		};
		using FEntryArray = Chaos::THandleArray<FEntry>;
		using FEntryHandle = typename FEntryArray::FHandle;

		template <typename InternalVisitorType, typename QueryRuntimeDataType>
		struct TInternalVisitor : public TVisitorAdapter<TInternalVisitor<InternalVisitorType, QueryRuntimeDataType>, InternalVisitorType, QueryRuntimeDataType>
		{
			TInternalVisitor(const FDefaultSpatialPartitionCollection* Collection, const ESpatialCategoryMask QueryCategoryMask, InternalVisitorType& ExternalVisitor)
				: Collection(Collection)
				, ExternalVisitor(ExternalVisitor)
				, QueryCategoryMask(QueryCategoryMask)
			{
			}

			EVisitResult Visit(const FUserDataType& UserData, QueryRuntimeDataType& QueryRuntimeData)
			{
				const FEntry& Entry = *Collection->GetEntryFromHandleValue(UserData);
				const ESpatialCategoryMask EntryCategoryMask = ToCategoryMask(Entry.Category);
				// Only visit an object if it passes the mask filter and it's aabb overlaps the query data.
				if (EnumHasAnyFlags(EntryCategoryMask, QueryCategoryMask) && QueryRuntimeData.Test(Entry.Aabb))
				{
					return ExternalVisitor.Visit(Entry.UserData, QueryRuntimeData);
				}
				return EVisitResult::Continue;
			}

			const FDefaultSpatialPartitionCollection* Collection = nullptr;
			InternalVisitorType& ExternalVisitor;
			ESpatialCategoryMask QueryCategoryMask;
		};

		using FInternalOverlapVisitor = TInternalVisitor<FOverlapVisitor, FOverlapQueryRuntimeData>;
		using FInternalRaycastVisitor = TInternalVisitor<FRaycastVisitor, FRaycastQueryRuntimeData>;
		using FInternalSweepVisitor = TInternalVisitor<FSweepVisitor, FSweepQueryRuntimeData>;

		template <typename SpatialPartitionType>
		static EVisitResult Query(const SpatialPartitionType& SpatialPartition, FOverlapQueryRuntimeData& QueryRuntimeData, FOverlapVisitor& Visitor)
		{
			return SpatialPartition.Overlap(QueryRuntimeData, Visitor);
		}

		template <typename SpatialPartitionType>
		static EVisitResult Query(const SpatialPartitionType& SpatialPartition, FRaycastQueryRuntimeData& QueryRuntimeData, FRaycastVisitor& Visitor)
		{
			return SpatialPartition.Raycast(QueryRuntimeData, Visitor);
		}

		template <typename SpatialPartitionType>
		static EVisitResult Query(const SpatialPartitionType& SpatialPartition, FSweepQueryRuntimeData& QueryRuntimeData, FSweepVisitor& Visitor)
		{
			return SpatialPartition.Sweep(QueryRuntimeData, Visitor);
		}

		template <typename QueryRuntimeDataType, typename VisitorType>
		void QueryInternal(const ESpatialCategoryMask QueryCategoryMask, QueryRuntimeDataType& QueryRuntimeData, VisitorType& Visitor) const
		{
			if (EnumHasAnyFlags(QueryCategoryMask, ESpatialCategoryMask::Static))
			{
				if (EVisitResult::Stop == Query(StaticTree, QueryRuntimeData, Visitor))
				{
					return;
				}
				if (EVisitResult::Stop == Query(StaticDirty, QueryRuntimeData, Visitor))
				{
					return;
				}
				if (EVisitResult::Stop == Query(StaticGlobal, QueryRuntimeData, Visitor))
				{
					return;
				}
			}
			if (EnumHasAnyFlags(QueryCategoryMask, ESpatialCategoryMask::Dynamic | ESpatialCategoryMask::Kinematic))
			{
				if (EVisitResult::Stop == Query(DynamicTree, QueryRuntimeData, Visitor))
				{
					return;
				}
				Query(DynamicGlobal, QueryRuntimeData, Visitor);
			}
		}

		bool IsGlobal(const FAABB3& Aabb) const;
		static EInternalCategory ToInternalCategory(const ESpatialCategory& Category, const bool bDirty, const bool bGlobal);
		static EInternalCategory SetOrClearFlag(const EInternalCategory Flags, const EInternalCategory Flag, bool bState);
		const FEntry* GetEntryFromHandleValue(uint32 Index) const;
		bool IsRebuilding() const;

		void InsertInternal(const FEntryHandle& EntryHandle, const FAABB3& Aabb, const EInternalCategory Category, FSpatialHandle& OutHandle);
		void UpdateInternal(const FEntryHandle& EntryHandle, const FAABB3& Aabb, const EInternalCategory Category, FSpatialHandle& InOutHandle);
		void RemoveInternal(const EInternalCategory Category, FSpatialHandle& InOutHandle);

		FEntryArray Entries;
		FStaticAabbTree StaticTree;
		FDynamicAabbTree StaticDirty;
		FNSquaredAabb StaticGlobal;
		FDynamicAabbTree DynamicTree;
		FNSquaredAabb DynamicGlobal;
		TUniquePtr<FStaticAabbTree::FRebuildContext> StaticRebuildContext;
		TArray<FEntryHandle> DirtyStaticEntries;
	};
} // namespace Chaos::SpatialPartition
