// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/ISpatialPartition.h"
#include "ChaosSpatialPartitions/Library/NSquaredAabb.h"

#include "Chaos/HandleArray.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	// A simple test spatial partition collection that shows basic usage. 
	// Mostly a proof of concept and used to validate basic tests.
	// Can also be used for validation when comparing against other spatial partitions due to its simplicity.
	class FNSquaredAabbCollection : public ISpatialPartition
	{
	public:
		virtual ~FNSquaredAabbCollection() override = default;

		FNSquaredAabbCollection& operator=(const FNSquaredAabbCollection&) = default;
		FNSquaredAabbCollection& operator=(FNSquaredAabbCollection&&) = default;

		CHAOSSPATIALPARTITIONS_API virtual void Insert(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& OutHandle) override;
		CHAOSSPATIALPARTITIONS_API virtual void Update(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& InOutHandle) override;
		CHAOSSPATIALPARTITIONS_API virtual void Remove(FSpatialHandle& InOutHandle) override;

		CHAOSSPATIALPARTITIONS_API virtual void Overlap(const FOverlapQueryData& QueryData, FOverlapVisitor& Visitor, ESpatialCategoryMask CategoryMask = ESpatialCategoryMask::All) const override;
		CHAOSSPATIALPARTITIONS_API virtual void Raycast(const FRaycastQueryData& QueryData, FRaycastVisitor& Visitor, ESpatialCategoryMask CategoryMask = ESpatialCategoryMask::All) const override;
		CHAOSSPATIALPARTITIONS_API virtual void Sweep(const FSweepQueryData& QueryData, FSweepVisitor& Visitor, ESpatialCategoryMask CategoryMask = ESpatialCategoryMask::All) const override;

	private:
		struct FEntry
		{
			FUserDataType UserData = INDEX_NONE;
			FAABB3 Aabb = FAABB3::EmptyAABB();
			FSpatialClassification Classification;
			FSpatialHandle Handle;
		};

		using FEntryArray = Chaos::THandleArray<FEntry>;
		using FEntryHandle = typename FEntryArray::FHandle;

		EVisitResult Query(int32 CategoryIndex, FOverlapQueryRuntimeData& QueryRuntimeData, FOverlapVisitor& Visitor) const;
		EVisitResult Query(int32 CategoryIndex, FRaycastQueryRuntimeData& QueryRuntimeData, FRaycastVisitor& Visitor) const;
		EVisitResult Query(int32 CategoryIndex, FSweepQueryRuntimeData& QueryRuntimeData, FSweepVisitor& Visitor) const;

		template <typename ... ArgTypes>
		void RunQueries(ESpatialCategoryMask CategoryMask, ArgTypes... Args) const
		{
			for (int32 I = 0; I < SpatialPartitionCount; ++I)
			{
				const ESpatialCategoryMask TestCategory = (ESpatialCategoryMask)(1 << I);
				if (EnumHasAnyFlags(CategoryMask, TestCategory))
				{
					const EVisitResult VisitResult = Query(I, Args...);
					if (VisitResult == EVisitResult::Stop)
					{
						return;
					}
				}
			}
		}

		bool ValidateCategory(const ESpatialCategory Category) const;

		static constexpr int32 SpatialPartitionCount = 3;
		FEntryArray Entries;
		FNSquaredAabb SpatialPartitions[SpatialPartitionCount];
	};
} // namespace Chaos::SpatialPartition
