// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Chaos/Real.h"
#include "Chaos/Vector.h"
#include "Chaos/AABB.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	// Bring some types into our namespace.
	using FReal = Chaos::FReal;
	using FVec3 = Chaos::FVec3;
	using FAABB3 = Chaos::FAABB3;

	// The user data type is what is inserted into any spatial partition.
	// This is a typedef to allow us to easily swap/upgrade if needed.
	using FUserDataType = int32;
} // namespace Chaos::SpatialPartition
