// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/SpatialClassification.h"
#include "ChaosSpatialPartitions/SpatialHandle.h"
#include "ChaosSpatialPartitions/Visitors.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	class ISpatialPartition
	{
	public:
		virtual ~ISpatialPartition() = default;

		virtual void Insert(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& OutHandle) = 0;
		virtual void Update(const FUserDataType& UserData, const FAABB3& Aabb, const FSpatialClassification& Classification, FSpatialHandle& InOutHandle) = 0;
		virtual void Remove(FSpatialHandle& InOutHandle) = 0;

		virtual void Overlap(const FOverlapQueryData& QueryData, FOverlapVisitor& Visitor, ESpatialCategoryMask CategoryMask = ESpatialCategoryMask::All) const = 0;
		virtual void Raycast(const FRaycastQueryData& QueryData, FRaycastVisitor& Visitor, ESpatialCategoryMask CategCategoryMaskory = ESpatialCategoryMask::All) const = 0;
		virtual void Sweep(const FSweepQueryData& QueryData, FSweepVisitor& Visitor, ESpatialCategoryMask CategoryMask = ESpatialCategoryMask::All) const = 0;

		enum class ERebuildStatus
		{
			Continue,
			Finished,
		};
		virtual bool NeedsRebuilding() const { return false; }
		virtual ERebuildStatus Rebuild() { return ERebuildStatus::Finished; }
		// TODO: Timeslice copying
	private:
	};
} // namespace Chaos::SpatialPartition
