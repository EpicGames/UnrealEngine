// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Common.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	struct FAabbTreeLeafElement
	{
		CHAOSSPATIALPARTITIONS_API FVec3 GetCenter() const;
		CHAOSSPATIALPARTITIONS_API const FAABB3& GetAabb() const;

		FAABB3 Aabb = FAABB3::EmptyAABB();
		int32 Index = INDEX_NONE;
	};

	struct FAabbTreeLeaf
	{
		CHAOSSPATIALPARTITIONS_API void Build(const TArrayView<const FAabbTreeLeafElement>& InElements, const int32 InNodeIndex);
		CHAOSSPATIALPARTITIONS_API void RecomputeAabb();

		FAABB3 Aabb = FAABB3::EmptyAABB();
		TArray<FAabbTreeLeafElement> Elements;
		int32 NodeIndex = INDEX_NONE;
	};
} // namespace Chaos::SpatialPartition
