// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Common.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	// Enable this to get some visualizers for aabb tree nodes.
	//#define UE_ENABLE_CHAOS_AABBTREE_VISUALIZER

	struct FAabbTreeNode
	{
		FAABB3 Aabb = FAABB3::EmptyAABB();
		int32 Parent = INDEX_NONE;
		int32 Left = INDEX_NONE;
		int32 Right = INDEX_NONE;
		// Note: The user data could be packed into the Right since it's only valid on leaves. Currently the node is 64 bytes with this so it's unneeded.
		int32 UserData = INDEX_NONE;

#ifdef UE_ENABLE_CHAOS_AABBTREE_VISUALIZER
		UE_INTERNAL TArray<FAabbTreeNode>* DebugNodeList = nullptr;
#endif
	};
} // namespace Chaos::SpatialPartition
