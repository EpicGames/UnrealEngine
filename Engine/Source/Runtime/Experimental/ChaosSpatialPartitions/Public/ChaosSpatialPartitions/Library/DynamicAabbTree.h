// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Library/AabbTreeNode.h"
#include "ChaosSpatialPartitions/SpatialHandle.h"
#include "ChaosSpatialPartitions/Visitors.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	// A binary tree of aabbs that allows efficient runtime changes.
	// The tree will attempt to balance itself for optimal query performance (typically a surface area heuristic).
	// Insertion, Update, and Removal are typical log(n). Queries are also typically log(n).
	class FDynamicAabbTree
	{
	public:
		struct FConfig
		{
			enum class ESearchAlgorithm
			{
				Greedy,
				BranchAndBound,
			};
			FReal PaddingFactor = 0;
			bool bRebalanceOnRemove = false;
			ESearchAlgorithm SearchAlgorithm = ESearchAlgorithm::BranchAndBound;
		};

		FDynamicAabbTree() = default;
		CHAOSSPATIALPARTITIONS_API FDynamicAabbTree(const FConfig& Config);
		FDynamicAabbTree(const FDynamicAabbTree&) = default;
		FDynamicAabbTree(FDynamicAabbTree&&) = default;

		FDynamicAabbTree& operator=(const FDynamicAabbTree&) = default;
		FDynamicAabbTree& operator=(FDynamicAabbTree&&) = default;

		// Inserts an object with the given aabb. The aabb will be expanded by the PaddingFactor of the config.
		CHAOSSPATIALPARTITIONS_API void Insert(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& OutHandle);
		// Inserts an object with the given aabb and expanded aabb. The expanded aabb is used to determine the true node size.
		CHAOSSPATIALPARTITIONS_API void Insert(const FUserDataType& UserData, const FAABB3& Aabb, const FAABB3& ExpandedAabb, FSpatialHandle& OutHandle);
		// Updates an object in the tree, possibly performing a rebalance.
		CHAOSSPATIALPARTITIONS_API void Update(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& InOutHandle);
		// Updates an object in the tree. If the given aabb is contained within the old aabb, no work will be done.
		// Otherwise rebalancing will happen using the expanded aabb.
		CHAOSSPATIALPARTITIONS_API void Update(const FUserDataType& UserData, const FAABB3& Aabb, const FAABB3& ExpandedAabb, FSpatialHandle& InOutHandle);
		CHAOSSPATIALPARTITIONS_API void Remove(FSpatialHandle& InOutHandle);

		// Checks if an update is needed. An update is needed if the handle's node does not fully contains the given aabb. 
		CHAOSSPATIALPARTITIONS_API bool CheckNeedsUpdate(const FAABB3& Aabb, const FSpatialHandle& InHandle);

		CHAOSSPATIALPARTITIONS_API EVisitResult Overlap(FOverlapQueryRuntimeData& QueryData, FOverlapVisitor& Visitor) const;
		CHAOSSPATIALPARTITIONS_API EVisitResult Raycast(FRaycastQueryRuntimeData& QueryData, FRaycastVisitor& Visitor) const;
		CHAOSSPATIALPARTITIONS_API EVisitResult Sweep(FSweepQueryRuntimeData& QueryData, FSweepVisitor& Visitor) const;
		CHAOSSPATIALPARTITIONS_API void SelfQuery(FSelfQueryVisitor& Visitor) const;

		UE_INTERNAL CHAOSSPATIALPARTITIONS_API void Dump(TArray<FAabbTreeNode>& OutData, int32& OutRootIndex) const;

	private:
		FAABB3 Expand(const FAABB3& Aabb) const;
		void InsertLeaf(int32 LeafIndex, bool bDoRotation);
		void RemoveLeaf(int32 LeafIndex, bool bDoRotation);

		void SelfQuery(FSelfQueryVisitor& Visitor, const int32 NodeIndex) const;
		void SelfQuery(FSelfQueryVisitor& Visitor, const int32 NodeIndex0, const int32 NodeIndex1) const;

		FConfig Config;
		int32 RootIndex = INDEX_NONE;
		int32 FreeListHeadIndex = INDEX_NONE;
		TArray<FAabbTreeNode> Nodes;
		TArray<TPair<int32, FReal>> PriorityQueue;
	};
} // namespace Chaos::SpatialPartition
