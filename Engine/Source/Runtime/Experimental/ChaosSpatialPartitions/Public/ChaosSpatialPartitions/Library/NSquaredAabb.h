// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/SpatialHandle.h"
#include "ChaosSpatialPartitions/Visitors.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	// A simple spatial partition that just stores a list of aabbs. 
	// Self query is O(n^2) and all other queries are O(n).
	// This is exceptionally useful for comparing correctness due to its simplicity.
	// This also is useful for cases where the known data set is small or cases like the "global object list".
	class FNSquaredAabb
	{
	public:
		CHAOSSPATIALPARTITIONS_API void Insert(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& OutHandle);
		CHAOSSPATIALPARTITIONS_API void Update(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& InOutHandle);
		CHAOSSPATIALPARTITIONS_API void Remove(FSpatialHandle& InOutHandle);

		CHAOSSPATIALPARTITIONS_API EVisitResult Overlap(FOverlapQueryRuntimeData& QueryData, FOverlapVisitor& Visitor) const;
		CHAOSSPATIALPARTITIONS_API EVisitResult Raycast(FRaycastQueryRuntimeData& QueryData, FRaycastVisitor& Visitor) const;
		CHAOSSPATIALPARTITIONS_API EVisitResult Sweep(FSweepQueryRuntimeData& QueryData, FSweepVisitor& Visitor) const;
		CHAOSSPATIALPARTITIONS_API void SelfQuery(FSelfQueryVisitor& Visitor) const;

	private:
		struct FEntry
		{
			FAABB3 Aabb = FAABB3::EmptyAABB();
			FUserDataType UserData = INDEX_NONE;
			bool bIsUsed = false;
		};

		int32 AllocateEntry();
		void DeallocateEntry(int32 Index);

		template <typename QueryDataType, typename VisitorType>
		EVisitResult QueryImpl(QueryDataType& QueryData, VisitorType& Visitor) const;
		bool ShouldVisitEntry(const FEntry& Entry, const FOverlapQueryRuntimeData& QueryData) const;
		bool ShouldVisitEntry(const FEntry& Entry, const FRaycastQueryRuntimeData& QueryData) const;
		bool ShouldVisitEntry(const FEntry& Entry, const FSweepQueryRuntimeData& QueryData) const;

		TArray<int32> FreeIndices;
		TArray<FEntry> Entries;
	};
} // namespace Chaos::SpatialPartition
