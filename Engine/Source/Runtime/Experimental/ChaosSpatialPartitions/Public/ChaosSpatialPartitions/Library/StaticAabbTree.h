// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Algorithms/PartitioningMethod.h"
#include "ChaosSpatialPartitions/Algorithms/StaticAabbTreeTimeSlicer.h"
#include "ChaosSpatialPartitions/ISpatialPartition.h"
#include "ChaosSpatialPartitions/Library/AabbTreeLeaf.h"
#include "ChaosSpatialPartitions/Library/AabbTreeNode.h"
#include "ChaosSpatialPartitions/SpatialHandle.h"
#include "ChaosSpatialPartitions/Visitors.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	// An aabb tree meant for infrequently moving / updated objects. 
	// Building this tree is more computationally expensive than its dynamic counterpart, but should produce a better quality tree.
	// When Insert/Updating/Removing objects, there are two primary APIs:
	// 1. Deferred: does the minimal work to make sure the tree will be correct the next build and won't return removed objects.
	// 2. Incremental (TODO): does incremental changes to the tree (similar to the dynamic tree) to ensure everything is up-to-date for all queries.
	// In general, incremental changes are faster to perform but will result in a slightly worse quality tree. Thus, it is recommended to perform periodic full rebuilds.
	class FStaticAabbTree
	{
	public:
		using ERebuildStatus = ISpatialPartition::ERebuildStatus;

		struct FConfig
		{
			EPartitioningMethod PartitioningMethod = EPartitioningMethod::SurfaceArea;
			int32 BatchSize = std::numeric_limits<int32>::max();
			int32 MaxElementsPerLeaf = 8;
			int32 MaxTreeDepth = 200;
			int32 SurfaceAreaHeuristicBinCount = 8;
			double TargetProcessingTimeInSeconds = 0.001;
		};

		// Stores information about a pending rebuild of the tree.
		struct FRebuildContext
		{
		public:
			CHAOSSPATIALPARTITIONS_API FRebuildContext();
			CHAOSSPATIALPARTITIONS_API FRebuildContext(const FStaticAabbTree* OwningTree, AabbTreeAlgorithm::FStaticAabbTreeTimeSlicer&& TimeSlicer);

			CHAOSSPATIALPARTITIONS_API ERebuildStatus Run();

		private:
			friend class FStaticAabbTree;
			const FStaticAabbTree* OwningTree = nullptr;
			AabbTreeAlgorithm::FStaticAabbTreeTimeSlicer TimeSlicer;
			double TargetTimeSeconds = 0;
		};

		CHAOSSPATIALPARTITIONS_API FStaticAabbTree();
		CHAOSSPATIALPARTITIONS_API FStaticAabbTree(const FConfig& Config);
		FStaticAabbTree(const FStaticAabbTree&) = default;
		FStaticAabbTree(FStaticAabbTree&&) = default;

		FStaticAabbTree& operator=(const FStaticAabbTree&) = default;
		FStaticAabbTree& operator=(FStaticAabbTree&&) = default;

		// Inserts an object such that it will be included the next time the tree is built. This object will not be returned in any queries.
		CHAOSSPATIALPARTITIONS_API void InsertDeferred(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& OutHandle);

		// Updates only if the object stays within the old leaf node.
		CHAOSSPATIALPARTITIONS_API bool UpdateIfWithinLeaf(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& InOutHandle);
		// Updates the object such that its new aabb will be reflected in the next build. 
		// Any changes to the user data will be immediate, however changes to the aabb will be deferred.
		CHAOSSPATIALPARTITIONS_API void UpdateDeferred(const FUserDataType& UserData, const FAABB3& Aabb, FSpatialHandle& InOutHandle);

		// Does the minimal work to remove an entry so it won't be returned from any queries.
		// This will not shrink any aabbs or remove empty nodes. 
		// The quality of the tree will deteriorate over time if it is not periodically rebuilt.
		CHAOSSPATIALPARTITIONS_API void RemoveMinimal(FSpatialHandle& InOutHandle);

		CHAOSSPATIALPARTITIONS_API void BeginRebuild(FRebuildContext& Context) const;
		// Commits the result of a tree rebuild. The context must be from this same tree. 
		// The only change guaranteed to be observed correctly during a rebuild is a remove. 
		// Insert and Update will typically be treated like a deferred operation, but some combined operations (e.g. remove + insert) will be caught in an in-between state.
		// A subsequent rebuild is necessary to pickup inserts/updates.
		CHAOSSPATIALPARTITIONS_API void CommitRebuild(FRebuildContext& Context);

		CHAOSSPATIALPARTITIONS_API EVisitResult Overlap(FOverlapQueryRuntimeData& QueryData, FOverlapVisitor& Visitor) const;
		CHAOSSPATIALPARTITIONS_API EVisitResult Raycast(FRaycastQueryRuntimeData& QueryData, FRaycastVisitor& Visitor) const;
		CHAOSSPATIALPARTITIONS_API EVisitResult Sweep(FSweepQueryRuntimeData& QueryData, FSweepVisitor& Visitor) const;

		struct FStats
		{
			int32 MinHeight = 0;
			int32 MaxHeight = 0;
			int32 MinElementsPerLeaf = 0;
			int32 MaxElementsPerLeaf = 0;
			// Surface area of all the nodes
			FReal NodeSurfaceArea = 0;
			// Surface area of everything (nodes + leaf elements)
			FReal TotalSurfaceArea = 0;
		};
		UE_INTERNAL FStats ComputeStats() const;

		// Returns the structure of the tree, mostly for tests. The leaf elements will have the user data in the Index field.
		UE_INTERNAL CHAOSSPATIALPARTITIONS_API void Dump(TArray<FAabbTreeNode>& OutNodes, int32& OutRootNodeIndex, TArray<FAabbTreeLeaf>& OutLeaves) const;

	private:
		struct FEntry
		{
			FAABB3 Aabb = FAABB3::EmptyAABB();
			FUserDataType UserData = INDEX_NONE;
			int32 LeafIndex = INDEX_NONE;
		};

		int32 AllocateEntry();
		void UpdateEntry(const int32 EntryIndex, const FUserDataType& UserData, const FAABB3& Aabb);
		void FreeEntry(const int32 EntryIndex);
		int32 AllocateNode();
		void FreeNode(const int32 NodeIndex);
		int32 AllocateLeaf();
		void FreeLeaf(const int32 LeafIndex);

		FStats ComputeStats(const int32 NodeIndex) const;

		void GatherUsedElements(TArray<FAabbTreeLeafElement>& Elements) const;
		void RecomputeElementLeafIndices();

		void RemoveEntryFromLeaf(const int32 EntryIndex);

		template <typename QueryDataType, typename VisitorType>
		EVisitResult CastLeafCallback(const FAabbTreeNode& Node, QueryDataType& QueryData, VisitorType& Visitor) const;

		FConfig Config;
		int32 EntriesFreeListHead = INDEX_NONE;
		TArray<FEntry> Entries;
		TBitArray<> UsedEntries;
		int32 RootIndex = INDEX_NONE;
		int32 NodesFreeListHead = INDEX_NONE;
		TArray<FAabbTreeNode> Nodes;
		int32 LeavesFreeListHead = INDEX_NONE;
		TArray<FAabbTreeLeaf> Leaves;
	};
} // namespace Chaos::SpatialPartition
