// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Common.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	struct FOverlapQueryData
	{
		FAABB3 Aabb = FAABB3::EmptyAABB();
	};

	struct FRaycastQueryData
	{
		FVec3 Start = FVec3::Zero();
		FVec3 Direction = FVec3::Zero();
		FReal Length = 0;
	};

	struct FSweepQueryData
	{
		FVec3 Start = FVec3::Zero();
		FVec3 Direction = FVec3::Zero();
		FVec3 HalfExtents = FVec3::Zero();
		FReal Length = 0;
	};

	struct FQueryRuntimeData
	{
		CHAOSSPATIALPARTITIONS_API FQueryRuntimeData() = default;
		CHAOSSPATIALPARTITIONS_API FQueryRuntimeData(const FVec3& Dir, const FReal Length);

		CHAOSSPATIALPARTITIONS_API void SetLength(const FReal Length);

		FVec3 InvDir = FVec3::Zero();
		
		bool bParallel[3] = {false};

		FReal CurrentLength = 0;
		FReal InvCurrentLength = 0;
	};

	struct FOverlapQueryRuntimeData
	{
		CHAOSSPATIALPARTITIONS_API FOverlapQueryRuntimeData(const FOverlapQueryData& InputData);
		CHAOSSPATIALPARTITIONS_API FOverlapQueryRuntimeData(const FAABB3& Aabb);

		FOverlapQueryRuntimeData(const FOverlapQueryRuntimeData&) = default;
		FOverlapQueryRuntimeData(FOverlapQueryRuntimeData&&) = default;
		FOverlapQueryRuntimeData& operator=(const FOverlapQueryRuntimeData&) = default;
		FOverlapQueryRuntimeData& operator=(FOverlapQueryRuntimeData&&) = default;

		CHAOSSPATIALPARTITIONS_API const FOverlapQueryData& GetInputData() const;

		CHAOSSPATIALPARTITIONS_API bool Test(const FAABB3& Aabb) const;

	private:
		FOverlapQueryData InputData;
	};

	struct FRaycastQueryRuntimeData
	{
		CHAOSSPATIALPARTITIONS_API FRaycastQueryRuntimeData(const FRaycastQueryData& InputData);

		FRaycastQueryRuntimeData(const FRaycastQueryRuntimeData&) = default;
		FRaycastQueryRuntimeData(FRaycastQueryRuntimeData&&) = default;
		FRaycastQueryRuntimeData& operator=(const FRaycastQueryRuntimeData&) = default;
		FRaycastQueryRuntimeData& operator=(FRaycastQueryRuntimeData&&) = default;

		CHAOSSPATIALPARTITIONS_API const FRaycastQueryData& GetInputData() const;
		CHAOSSPATIALPARTITIONS_API FReal GetCurrentLength() const;
		CHAOSSPATIALPARTITIONS_API void SetCurrentLength(const FReal InLength);

		CHAOSSPATIALPARTITIONS_API bool Test(const FAABB3& Aabb) const;
		CHAOSSPATIALPARTITIONS_API bool Test(const FAABB3& Aabb, FReal& OutTime) const;

	private:
		FRaycastQueryData InputData;
		FQueryRuntimeData RuntimeData;
	};

	struct FSweepQueryRuntimeData
	{
		CHAOSSPATIALPARTITIONS_API FSweepQueryRuntimeData(const FSweepQueryData& InputData);
		
		FSweepQueryRuntimeData(const FSweepQueryRuntimeData&) = default;
		FSweepQueryRuntimeData(FSweepQueryRuntimeData&&) = default;
		FSweepQueryRuntimeData& operator=(const FSweepQueryRuntimeData&) = default;
		FSweepQueryRuntimeData& operator=(FSweepQueryRuntimeData&&) = default;

		CHAOSSPATIALPARTITIONS_API const FSweepQueryData& GetInputData() const;
		CHAOSSPATIALPARTITIONS_API FReal GetCurrentLength() const;
		CHAOSSPATIALPARTITIONS_API void SetCurrentLength(const FReal InLength);

		CHAOSSPATIALPARTITIONS_API bool Test(const FAABB3& Aabb) const;
		CHAOSSPATIALPARTITIONS_API bool Test(const FAABB3& Aabb, FReal& OutTime) const;

	private:
		FSweepQueryData InputData;
		FQueryRuntimeData RuntimeData;
	};
} // namespace Chaos::SpatialPartition
