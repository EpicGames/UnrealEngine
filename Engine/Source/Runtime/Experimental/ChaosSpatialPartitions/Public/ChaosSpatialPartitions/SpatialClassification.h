// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Common.h"

#include "Misc/EnumClassFlags.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	// Used to separate objects into different buckets based upon common usage/movement patterns.
	// Currently just used to separate out dynamic, static, etc...
	enum class ESpatialCategory : uint8
	{
		Invalid = static_cast<uint8>(-1),
		Static = 0,
		Kinematic = 1,
		Dynamic = 2,
		Count,
	};
	enum class ESpatialCategoryMask : uint8
	{
		None = 0,
		Static = 1 << 0,
		Kinematic = 1 << 1,
		Dynamic = 1 << 2,
		All = (1 << 3) - 1,
	};
	ENUM_CLASS_FLAGS(ESpatialCategoryMask);

	CHAOSSPATIALPARTITIONS_API FString ToString(ESpatialCategory Category);
	CHAOSSPATIALPARTITIONS_API FString ToString(ESpatialCategoryMask Mask);
	CHAOSSPATIALPARTITIONS_API ESpatialCategoryMask ToCategoryMask(const ESpatialCategory Category);

	// Contains information used to specify how a spatial partition should treat an object when it's inserted/updated/removed. 
	// Currently, this only contains the category, but this may be extended.
	struct FSpatialClassification
	{
		FSpatialClassification() = default;
		CHAOSSPATIALPARTITIONS_API FSpatialClassification(ESpatialCategory InCategory);
		FSpatialClassification(const FSpatialClassification&) = default;
		FSpatialClassification(FSpatialClassification&&) = default;

		FSpatialClassification& operator=(const FSpatialClassification&) = default;
		FSpatialClassification& operator=(FSpatialClassification&&) = default;

		CHAOSSPATIALPARTITIONS_API ESpatialCategory GetCategory() const;
		CHAOSSPATIALPARTITIONS_API void SetCategory(ESpatialCategory InCategory);

	private:
		ESpatialCategory Category = ESpatialCategory::Static;
		// TODO: Investigate more data. 
		// For instance, what if we decide we want a separate structure for dynamic vs. dynamics queries (e.g. SAP for dynamic). 
		// We may need to say "this object can be queried but this one can't".
	};
} // namespace Chaos::SpatialPartition
