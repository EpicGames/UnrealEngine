// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Common.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	// This handle is used by a spatial partition to uniquely identify an insert and provide O(1) lookup times. 
	// All subsequent update/remove operations require the same handle to be passed back in.
	// This data is expected to be stored by the user.
	struct FSpatialHandle
	{
		// Note: These methods should only be used by a spatial partition.
		UE_INTERNAL CHAOSSPATIALPARTITIONS_API int64 GetValue() const;
		UE_INTERNAL CHAOSSPATIALPARTITIONS_API void SetValue(int64 InValue);

	private:
		int64 Value = INDEX_NONE;
	};
} // namespace Chaos::SpatialPartition
