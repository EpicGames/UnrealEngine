// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosSpatialPartitions/Common.h"
#include "ChaosSpatialPartitions/QueryData.h"

UE_EXPERIMENTAL(5.8, "The new spatial partition module is experimental")
namespace Chaos::SpatialPartition
{
	enum class EVisitResult { Continue, Stop };

	class FOverlapVisitor
	{
	public:
		typedef EVisitResult(*FCallbackType)(void* CallbackData, const FUserDataType& UserData, FOverlapQueryRuntimeData& QueryData);

		CHAOSSPATIALPARTITIONS_API FOverlapVisitor(FCallbackType Callback, void* CallbackData);

		CHAOSSPATIALPARTITIONS_API EVisitResult Visit(const FUserDataType& UserData, FOverlapQueryRuntimeData& QueryRuntimeData);

	private:
		FCallbackType Callback = nullptr;
		void* CallbackData = nullptr;
	};

	class FRaycastVisitor
	{
	public:
		typedef EVisitResult(*FCallbackType)(void* CallbackData, const FUserDataType& UserData, FRaycastQueryRuntimeData& QueryData);

		CHAOSSPATIALPARTITIONS_API FRaycastVisitor(FCallbackType Callback, void* CallbackData);

		CHAOSSPATIALPARTITIONS_API EVisitResult Visit(const FUserDataType& UserData, FRaycastQueryRuntimeData& QueryRuntimeData);

	private:
		FCallbackType Callback = nullptr;
		void* CallbackData = nullptr;
	};

	class FSweepVisitor
	{
	public:
		typedef EVisitResult(*FCallbackType)(void* CallbackData, const FUserDataType& UserData, FSweepQueryRuntimeData& QueryData);

		CHAOSSPATIALPARTITIONS_API FSweepVisitor(FCallbackType Callback, void* CallbackData);

		CHAOSSPATIALPARTITIONS_API EVisitResult Visit(const FUserDataType& UserData, FSweepQueryRuntimeData& QueryRuntimeData);

	private:
		FCallbackType Callback = nullptr;
		void* CallbackData = nullptr;
	};

	class FSelfQueryVisitor
	{
	public:
		typedef void(*FCallbackType)(void* CallbackData, const FUserDataType& UserData0, const FUserDataType& UserData1);

		CHAOSSPATIALPARTITIONS_API FSelfQueryVisitor(FCallbackType Callback, void* CallbackData);

		CHAOSSPATIALPARTITIONS_API void Visit(const FUserDataType& UserData0, const FUserDataType& UserData1);

	private:
		FCallbackType Callback = nullptr;
		void* CallbackData = nullptr;
	};

	// Adapters to make it easier to take a class. These expect the class to have a Visit method with the same signature as the underlying visitor type.
	// TODO: Investigate some kind of a concept.
	template <typename SelfType, typename InternalVisitorType, typename QueryRuntimeDataType>
	struct TVisitorAdapter
	{
		TVisitorAdapter()
			: InternalVisitor(&Callback, this)
		{
		}

		// This implicit operator allows the user to pass in the adapter directly.
		operator InternalVisitorType& ()
		{
			return InternalVisitor;
		}

		static EVisitResult Callback(void* CallbackData, const FUserDataType& UserData, QueryRuntimeDataType& QueryData)
		{
			SelfType* Self = static_cast<SelfType*>(CallbackData);
			return Self->Visit(UserData, QueryData);
		}

	private:
		InternalVisitorType InternalVisitor;
	};

	template <typename SelfType>
	using TOverlapVisitorAdapter = TVisitorAdapter<SelfType, FOverlapVisitor, FOverlapQueryRuntimeData>;

	template <typename SelfType>
	using TRaycastVisitorAdapter = TVisitorAdapter<SelfType, FRaycastVisitor, FRaycastQueryRuntimeData>;

	template <typename SelfType>
	using TSweepVisitorAdapter = TVisitorAdapter<SelfType, FSweepVisitor, FSweepQueryRuntimeData>;

	template <typename SelfType>
	struct TSelfQueryVisitorAdapter
	{
		TSelfQueryVisitorAdapter()
			: InternalVisitor(&Callback, this)
		{
		}

		operator FSelfQueryVisitor& ()
		{
			return InternalVisitor;
		}

		static void Callback(void* CallbackData, const FUserDataType& UserData0, const FUserDataType& UserData1)
		{
			SelfType* Self = static_cast<SelfType*>(CallbackData);
			Self->Visit(UserData0, UserData1);
		}

	private:
		FSelfQueryVisitor InternalVisitor;
	};
} // namespace Chaos::SpatialPartition
