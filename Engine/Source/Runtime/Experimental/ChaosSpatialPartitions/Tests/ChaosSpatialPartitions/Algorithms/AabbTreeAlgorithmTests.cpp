// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "CoreMinimal.h"

#include "ChaosSpatialPartitions/Algorithms/AabbTreeAlgorithm.h"

#include "ChaosSpatialPartitions/TestHarness/AabbTreeTestHelpers.h"
#include "ChaosSpatialPartitions/TestHarness/Common.h"

namespace Chaos::SpatialPartition::LowLevelTest
{
	TEST_CASE("AabbTreeRotate", "[Chaos][AabbTreeAlgorithm][spatial-partition]")
	{
		// Test names are for the tree:
		//    A
		//  B   C
		// D E F G
		// Where the leaves (D,E,F,G) are constructed from the nodes (0, 1, 2, 3).

		// Prime both trees with the leaves. All leaves may not be used in the final tree construction.
		TArray<FAabbTreeNode> ActualNodes;
		BuildNodeList(ActualNodes, 4);
		TArray<FAabbTreeNode> ExpectedNodes = ActualNodes;

		SECTION("1 Leaf")
		{
			// Tree: 0
			const int32 ActualRoot = 0;
			const int32 ExpectedRoot = 0;

			const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
			CHECK(bRotated == false);
			Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
		}
		SECTION("2 Leaves")
		{
			//  A
			// 0 1
			const int32 ActualRoot = BuildAndAppendNode(ActualNodes, 0, 1);
			const int32 ExpectedRoot = BuildAndAppendNode(ExpectedNodes, 0, 1);

			const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
			CHECK(bRotated == false);
			Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
		}
		SECTION("3 Leaves")
		{
			SECTION("B Leaf")
			{
				SECTION("No Rotation")
				{
					//    A	         A
					//  0   C  ->  0   C 
					//     2 3        2 3
					const int32 ActualC = BuildAndAppendNode(ActualNodes, 2, 3);
					const int32 ActualRoot = BuildAndAppendNode(ActualNodes, 0, ActualC);
					ExpectedNodes = ActualNodes;
					const int32 ExpectedRoot = ActualRoot;

					const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
					CHECK(bRotated == false);
					Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
				}
				SECTION("BF Rotation")
				{
					//    A	         A
					//  2   C  ->  0   C 
					//     0 3        2 3
					const int32 ActualC = BuildAndAppendNode(ActualNodes, 0, 3);
					const int32 ActualRoot = BuildAndAppendNode(ActualNodes, 2, ActualC);
					const int32 ExpectedC = BuildAndAppendNode(ExpectedNodes, 2, 3);
					const int32 ExpectedRoot = BuildAndAppendNode(ExpectedNodes, 0, ExpectedC);

					const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
					CHECK(bRotated == true);
					Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
				}
				SECTION("BF Score Tie - No Rotation")
				{
					//    A	         A
					//  1   C  ->  1   C 
					//     3 2        3 2
					const int32 ActualC = BuildAndAppendNode(ActualNodes, 3, 2);
					const int32 ActualRoot = BuildAndAppendNode(ActualNodes, 1, ActualC);
					ExpectedNodes = ActualNodes;
					const int32 ExpectedRoot = ActualRoot;

					const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
					CHECK(bRotated == false);
					Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
				}
				SECTION("BG Rotation")
				{
					//    A	         A
					//  2   C  ->  0   C
					//     3 0        3 2
					const int32 ActualC = BuildAndAppendNode(ActualNodes, 3, 0);
					const int32 ActualRoot = BuildAndAppendNode(ActualNodes, 2, ActualC);
					const int32 ExpectedC = BuildAndAppendNode(ExpectedNodes, 3, 2);
					const int32 ExpectedRoot = BuildAndAppendNode(ExpectedNodes, 0, ExpectedC);

					const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
					CHECK(bRotated == true);
					Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
				}
				SECTION("BG Score Tie - No Rotation")
				{
					//    A	         A
					//  1   C  ->  1   C 
					//     2 3        2 3
					const int32 ActualC = BuildAndAppendNode(ActualNodes, 2, 3);
					const int32 ActualRoot = BuildAndAppendNode(ActualNodes, 1, ActualC);
					ExpectedNodes = ActualNodes;
					const int32 ExpectedRoot = ActualRoot;

					const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
					CHECK(bRotated == false);
					Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
				}
			}
			SECTION("C Leaf")
			{
				SECTION("No Rotation")
				{
					//    A	         A
					//  B   3  ->  B   3
					// 0 1        0 1
					const int32 ActualB = BuildAndAppendNode(ActualNodes, 0, 1);
					const int32 ActualRoot = BuildAndAppendNode(ActualNodes, ActualB, 3);
					ExpectedNodes = ActualNodes;
					const int32 ExpectedRoot = ActualRoot;

					const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
					CHECK(bRotated == false);
					Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
				}
				SECTION("CD Rotation")
				{
					//    A	         A
					//  B   0  ->  B   3
					// 3 1        0 1
					const int32 ActualB = BuildAndAppendNode(ActualNodes, 3, 1);
					const int32 ActualRoot = BuildAndAppendNode(ActualNodes, ActualB, 0);
					const int32 ExpectedB = BuildAndAppendNode(ExpectedNodes, 0, 1);
					const int32 ExpectedRoot = BuildAndAppendNode(ExpectedNodes, ExpectedB, 3);

					const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
					CHECK(bRotated == true);
					Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
				}
				SECTION("CD Score Tie - No Rotation")
				{
					//    A	         A
					//  B   1  ->  B   1
					// 3 2        3 2
					const int32 ActualB = BuildAndAppendNode(ActualNodes, 3, 2);
					const int32 ActualRoot = BuildAndAppendNode(ActualNodes, ActualB, 1);
					ExpectedNodes = ActualNodes;
					const int32 ExpectedRoot = ActualRoot;

					const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
					CHECK(bRotated == false);
					Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
				}
				SECTION("CE Rotation")
				{
					//    A	         A
					//  B   0  ->  B   3
					// 1 3        1 0
					const int32 ActualB = BuildAndAppendNode(ActualNodes, 1, 3);
					const int32 ActualRoot = BuildAndAppendNode(ActualNodes, ActualB, 0);
					const int32 ExpectedB = BuildAndAppendNode(ExpectedNodes, 1, 0);
					const int32 ExpectedRoot = BuildAndAppendNode(ExpectedNodes, ExpectedB, 3);

					const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
					CHECK(bRotated == true);
					Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
				}
				SECTION("CE Score Tie - No Rotation")
				{
					//    A	         A
					//  B   1  ->  B   1
					// 2 3        2 3
					const int32 ActualB = BuildAndAppendNode(ActualNodes, 2, 3);
					const int32 ActualRoot = BuildAndAppendNode(ActualNodes, ActualB, 1);
					ExpectedNodes = ActualNodes;
					const int32 ExpectedRoot = ActualRoot;

					const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
					CHECK(bRotated == false);
					Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
				}
			}
		}
		SECTION("4 Leaves")
		{
			SECTION("No Rotation")
			{
				//    A
				//  B   C
				// 0 1 2 3
				const int32 ActualB = BuildAndAppendNode(ActualNodes, 0, 1);
				const int32 ActualC = BuildAndAppendNode(ActualNodes, 2, 3);
				const int32 ActualRoot = BuildAndAppendNode(ActualNodes, ActualB, ActualC);
				ExpectedNodes = ActualNodes;
				const int32 ExpectedRoot = ActualRoot;

				const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
				CHECK(bRotated == false);
				Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
			}

			// Move node 2 down far away such that it's better to group 3 with (0, 1) than with 2.
			// This looks like:
			// 0 1  3
			// 
			// 
			// 2
			ExpectedNodes[2].Aabb = ActualNodes[2].Aabb = FAABB3(FVec3(0, 3, 0), FVec3(1, 4, 1));
			SECTION("BF Rotation")
			{
				//    A	         A
				//  B   C  ->  2   C
				// 0 1 2 3        B 3
				//               0 1
				const int32 ActualB = BuildAndAppendNode(ActualNodes, 0, 1);
				const int32 ActualC = BuildAndAppendNode(ActualNodes, 2, 3);
				const int32 ActualRoot = BuildAndAppendNode(ActualNodes, ActualB, ActualC);
				const int32 ExpectedB = BuildAndAppendNode(ExpectedNodes, 0, 1);
				const int32 ExpectedC = BuildAndAppendNode(ExpectedNodes, ExpectedB, 3);
				const int32 ExpectedRoot = BuildAndAppendNode(ExpectedNodes, 2, ExpectedC);

				const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
				CHECK(bRotated == true);
				Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
			}
			SECTION("BG Rotation")
			{
				//    A	         A
				//  B   C  ->  2   C
				// 0 1 3 2        3 B
				//                 0 1
				const int32 ActualB = BuildAndAppendNode(ActualNodes, 0, 1);
				const int32 ActualC = BuildAndAppendNode(ActualNodes, 3, 2);
				const int32 ActualRoot = BuildAndAppendNode(ActualNodes, ActualB, ActualC);
				const int32 ExpectedB = BuildAndAppendNode(ExpectedNodes, 0, 1);
				const int32 ExpectedC = BuildAndAppendNode(ExpectedNodes, 3, ExpectedB);
				const int32 ExpectedRoot = BuildAndAppendNode(ExpectedNodes, 2, ExpectedC);

				const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
				CHECK(bRotated == true);
				Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
			}
			SECTION("CD Rotation")
			{
				//    A	         A
				//  B   C  ->  B   2
				// 2 3 0 1    C 3
				//           0 1
				const int32 ActualB = BuildAndAppendNode(ActualNodes, 2, 3);
				const int32 ActualC = BuildAndAppendNode(ActualNodes, 0, 1);
				const int32 ActualRoot = BuildAndAppendNode(ActualNodes, ActualB, ActualC);
				const int32 ExpectedC = BuildAndAppendNode(ExpectedNodes, 0, 1);
				const int32 ExpectedB = BuildAndAppendNode(ExpectedNodes, ExpectedC, 3);
				const int32 ExpectedRoot = BuildAndAppendNode(ExpectedNodes, ExpectedB, 2);

				const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
				CHECK(bRotated == true);
				Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
			}
			SECTION("CE Rotation")
			{
				//    A	         A
				//  B   C  ->  B   2
				// 3 2 0 1    3 C
				//             0 1
				const int32 ActualB = BuildAndAppendNode(ActualNodes, 3, 2);
				const int32 ActualC = BuildAndAppendNode(ActualNodes, 0, 1);
				const int32 ActualRoot = BuildAndAppendNode(ActualNodes, ActualB, ActualC);
				const int32 ExpectedC = BuildAndAppendNode(ExpectedNodes, 0, 1);
				const int32 ExpectedB = BuildAndAppendNode(ExpectedNodes, 3, ExpectedC);
				const int32 ExpectedRoot = BuildAndAppendNode(ExpectedNodes, ExpectedB, 2);

				const bool bRotated = AabbTreeAlgorithm::RotateNodes(ActualNodes, ActualRoot);
				CHECK(bRotated == true);
				Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
			}
		}
	}

	TEST_CASE("RecomputeAncestorAabbsAndRotate", "[Chaos][AabbTreeAlgorithm][spatial-partition]")
	{
		// Build a grid laid out like:
		// 01
		// 
		// 23
		// 
		// 45
		// 
		// 67
		// The natural tree that would be built from this is:
		//        A
		//    B       C
		//  D   E   F   G
		// 0 1 2 3 4 5 6 7
		// Test below will mutate the layout / aabbs of this tree in such a way that they produce this tree when updated.
		constexpr uint32 CountX = 2;
		constexpr uint32 CountY = 4;
		const FVec3 Spacing(0, 1, 0);
		TArray<FAabbTreeNode> ExpectedNodes;
		BuildNodeList(ExpectedNodes, CountX, CountY, 1, Spacing);
		const int32 Parent01 = BuildAndAppendNode(ExpectedNodes, 0, 1);
		const int32 Parent23 = BuildAndAppendNode(ExpectedNodes, 2, 3);
		const int32 Parent45 = BuildAndAppendNode(ExpectedNodes, 4, 5);
		const int32 Parent67 = BuildAndAppendNode(ExpectedNodes, 6, 7);
		const int32 Parent01_23 = BuildAndAppendNode(ExpectedNodes, Parent01, Parent23);
		const int32 Parent45_67 = BuildAndAppendNode(ExpectedNodes, Parent45, Parent67);
		const int32 ExpectedRoot = BuildAndAppendNode(ExpectedNodes, Parent01_23, Parent45_67);
		TArray<FAabbTreeNode> ActualNodes = ExpectedNodes;

		SECTION("No Change")
		{
			const int32 ActualRoot = ExpectedRoot;
			// Clear out the parent aabbs to ensure that they're all recomputed
			ActualNodes[Parent01].Aabb = ActualNodes[Parent01_23].Aabb = ActualNodes[ActualRoot].Aabb = FAABB3::EmptyAABB();
			AabbTreeAlgorithm::RecomputeAncestorAabbsAndRotate(ActualNodes, Parent01, true);
			Compare(ExpectedNodes, ActualRoot, ActualNodes, ActualRoot);
		}
		SECTION("Aabbs Shrink - No Rotation")
		{
			// Shrink nodes 0 and 1. This causes all nodes above to shrink.
			const int32 ActualRoot = ExpectedRoot;
			// The tree layout shouldn't change, however all of the nodes from 01 and up should shrink
			ExpectedNodes[0].Aabb = ActualNodes[0].Aabb.ShrinkSymmetrically(FVec3(0.1f));
			ExpectedNodes[1].Aabb = ActualNodes[1].Aabb.ShrinkSymmetrically(FVec3(0.1f));
			// Update our expected parent aabbs
			ExpectedNodes[Parent01].Aabb = AabbTreeAlgorithm::Union(ExpectedNodes[0].Aabb, ExpectedNodes[1].Aabb);
			ExpectedNodes[Parent01_23].Aabb = AabbTreeAlgorithm::Union(ExpectedNodes[Parent01].Aabb, ExpectedNodes[Parent23].Aabb);
			ExpectedNodes[ExpectedRoot].Aabb = AabbTreeAlgorithm::Union(ExpectedNodes[Parent01_23].Aabb, ExpectedNodes[Parent45_67].Aabb);
			AabbTreeAlgorithm::RecomputeAncestorAabbsAndRotate(ActualNodes, Parent01, true);

			Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
		}
		SECTION("Aabbs Grow - No Rotation")
		{
			// Grow nodes 0 and 1. This causes all nodes above to grow
			const int32 ActualRoot = ExpectedRoot;
			// The tree layout shouldn't change, however all of the nodes from 01 and up should grow
			ExpectedNodes[0].Aabb = ActualNodes[0].Aabb.ThickenSymmetrically(FVec3(0.1f));
			ExpectedNodes[1].Aabb = ActualNodes[1].Aabb.ThickenSymmetrically(FVec3(0.1f));
			// Update our expected parent aabbs
			ExpectedNodes[Parent01].Aabb = AabbTreeAlgorithm::Union(ExpectedNodes[0].Aabb, ExpectedNodes[1].Aabb);
			ExpectedNodes[Parent01_23].Aabb = AabbTreeAlgorithm::Union(ExpectedNodes[Parent01].Aabb, ExpectedNodes[Parent23].Aabb);
			ExpectedNodes[ExpectedRoot].Aabb = AabbTreeAlgorithm::Union(ExpectedNodes[Parent01_23].Aabb, ExpectedNodes[Parent45_67].Aabb);
			AabbTreeAlgorithm::RecomputeAncestorAabbsAndRotate(ActualNodes, Parent01, true);

			Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
		}
		SECTION("Single Rotation At Height 3")
		{
			// Build the input tree of:
			//          A
			//    B          C
			// 3    E      F   G
			//    2   D   4 5 6 7
			//       0 1
			// When recomputing from D, we expect a rotation at B that swaps 3 and D.
			const int32 ActualE = BuildAndAppendNode(ActualNodes, 2, Parent01);
			const int32 ActualB = BuildAndAppendNode(ActualNodes, 3, ActualE);
			const int32 ActualRoot = BuildAndAppendNode(ActualNodes, ActualB, Parent45_67);

			AabbTreeAlgorithm::RecomputeAncestorAabbsAndRotate(ActualNodes, Parent01, true);
			Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
		}
		SECTION("Single Rotation At Height 4")
		{
			// Build the input tree of:
			//         A
			//   G            C
			// 6   7      F       B
			//          4  5    D   E
			//                 0 1 2 3
			// When recomputing from D, we expect a rotation at A that swaps G and B.
			const int32 ActualC = BuildAndAppendNode(ActualNodes, Parent45, Parent01_23);
			const int32 ActualRoot = BuildAndAppendNode(ActualNodes, Parent67, ActualC);

			AabbTreeAlgorithm::RecomputeAncestorAabbsAndRotate(ActualNodes, Parent01, true);
			Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
		}
		SECTION("Two Rotations")
		{
			// Build the input tree of:
			//        A
			//  G          C
			// 6 7     F        B
			//        4 5    D     1
			//              0  E 
			//                2 3
			// When recomputing from E, we expect two rotations:
			// 1. a rotation at B will swap E and 1
			// 2. A rotation at A will swap G and B.
			const int32 ActualD = BuildAndAppendNode(ActualNodes, 0, Parent23);
			const int32 ActualB = BuildAndAppendNode(ActualNodes, ActualD, 1);
			const int32 ActualC = BuildAndAppendNode(ActualNodes, Parent45, ActualB);
			const int32 ActualRoot = BuildAndAppendNode(ActualNodes, Parent67, ActualC);

			AabbTreeAlgorithm::RecomputeAncestorAabbsAndRotate(ActualNodes, Parent23, true);
			Compare(ExpectedNodes, ExpectedRoot, ActualNodes, ActualRoot);
		}
	}

	TEST_CASE("FindBestSibling - Simple", "[Chaos][AabbTreeAlgorithm][spatial-partition]")
	{
		const FVec3 AabbSize = FVec3(1, 1, 1);
		// Build the grid:
		// 0|1
		// -+-
		// 2|3
		// With the tree:
		//    A
		//  B   C
		// 0 1 2 3
		const FAabbTreeNode Node0{ .Aabb = BuildAabbCenterExtents(FVec3(-1, +1, 0), AabbSize), .UserData = 0, };
		const FAabbTreeNode Node1{ .Aabb = BuildAabbCenterExtents(FVec3(+1, +1, 0), AabbSize), .UserData = 1, };
		const FAabbTreeNode Node2{ .Aabb = BuildAabbCenterExtents(FVec3(-1, -1, 0), AabbSize), .UserData = 2, };
		const FAabbTreeNode Node3{ .Aabb = BuildAabbCenterExtents(FVec3(+1, -1, 0), AabbSize), .UserData = 3, };
		TArray<FAabbTreeNode> Nodes{ Node0, Node1, Node2, Node3 };

		const int32 BIndex = BuildAndAppendNode(Nodes, 0, 1);
		const int32 CIndex = BuildAndAppendNode(Nodes, 2, 3);
		const int32 RootIndex = BuildAndAppendNode(Nodes, BIndex, CIndex);

		// Do trivial checks at each node's aabb. These should match across all of the different algorithms.
		CHECK(0 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, Node0.Aabb));
		CHECK(1 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, Node1.Aabb));
		CHECK(2 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, Node2.Aabb));
		CHECK(3 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, Node3.Aabb));

		CHECK(0 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, Node0.Aabb));
		CHECK(1 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, Node1.Aabb));
		CHECK(2 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, Node2.Aabb));
		CHECK(3 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, Node3.Aabb));

		CHECK(0 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, Node0.Aabb));
		CHECK(1 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, Node1.Aabb));
		CHECK(2 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, Node2.Aabb));
		CHECK(3 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, Node3.Aabb));

		CHECK(0 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, Node0.Aabb));
		CHECK(1 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, Node1.Aabb));
		CHECK(2 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, Node2.Aabb));
		CHECK(3 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, Node3.Aabb));
	}

	TEST_CASE("FindBestSibling - Pick Non Root", "[Chaos][AabbTreeAlgorithm][spatial-partition]")
	{
		const FVec3 AabbSize = FVec3(1, 1, 1);
		// It's actually possible for a globally optimum solution to pick a non root. In the tree:
		//|0 
		//| 1
		//|   2
		//+-----
		// A global solution will produce the tree A(B(0,1),2) whereas a local optimum will produce A(0, B(1, 2))
		const FAabbTreeNode Node0{ .Aabb = BuildAabbMinExtents(FVec3(0, 2, 0), AabbSize), .UserData = 0, };
		const FAabbTreeNode Node1{ .Aabb = BuildAabbMinExtents(FVec3(1, 1, 0), AabbSize), .UserData = 1, };
		const FAabbTreeNode Node2{ .Aabb = BuildAabbMinExtents(FVec3(3, 0, 0), AabbSize), .UserData = 2, };
		TArray<FAabbTreeNode> Nodes{ Node0, Node1, Node2 };
		const int32 Index01 = BuildAndAppendNode(Nodes, 0, 1);

		CHECK(1 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, Index01, Node2.Aabb));
		CHECK(Index01 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, Index01, Node2.Aabb));
		CHECK(Index01 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, Index01, Node2.Aabb));
		CHECK(Index01 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, Index01, Node2.Aabb));
	}

	TEST_CASE("FindBestSibling2", "[Chaos][AabbTreeAlgorithm][spatial-partition]")
	{
		// Build the scene:
		// 0  |
		//	  |
		// 1  |
		// ---+---
		//    |
		//    |
		//    |2 3
		const FVec3 AabbSize = FVec3(1, 1, 1);
		const FAabbTreeNode Node0{ .Aabb = BuildAabbMinExtents(FVec3(-3, 2, 0), AabbSize), .UserData = 0, };
		const FAabbTreeNode Node1{ .Aabb = BuildAabbMinExtents(FVec3(-3, 0, 0), AabbSize), .UserData = 1, };
		const FAabbTreeNode Node2{ .Aabb = BuildAabbMinExtents(FVec3(0, -3, 0), AabbSize), .UserData = 2, };
		const FAabbTreeNode Node3{ .Aabb = BuildAabbMinExtents(FVec3(2, -3, 0), AabbSize), .UserData = 3, };

		TArray<FAabbTreeNode> Nodes{ Node0, Node1, Node2, Node3 };
		SECTION("Balanced Tree")
		{
			// Build the Tree A(B(0, 1), C(2, 3))
			const int32 BIndex = BuildAndAppendNode(Nodes, 0, 1);
			const int32 CIndex = BuildAndAppendNode(Nodes, 2, 3);
			const int32 RootIndex = BuildAndAppendNode(Nodes, BIndex, CIndex);

			FAABB3 TestAabb = BuildAabbMinExtents(FVec3(4, 4, 0), AabbSize);
			CHECK(RootIndex == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb));
			CHECK(RootIndex == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb));

			TestAabb = BuildAabbMinExtents(FVec3(0, 2, 0), AabbSize);
			CHECK(BIndex == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb));
			CHECK(BIndex == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb));

			TestAabb = BuildAabbMinExtents(FVec3(2, 0, 0), AabbSize);
			CHECK(CIndex == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb));
			CHECK(CIndex == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb));

			TestAabb = BuildAabbMinExtents(FVec3(-3, 3, 0), AabbSize);
			CHECK(0 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb));
			CHECK(0 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb));

			TestAabb = BuildAabbMinExtents(FVec3(-3, -1, 0), AabbSize);
			CHECK(1 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb));
			CHECK(1 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb));

			TestAabb = BuildAabbMinExtents(FVec3(-1, -3, 0), AabbSize);
			CHECK(2 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb));
			CHECK(2 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb));

			TestAabb = BuildAabbMinExtents(FVec3(3, -3, 0), AabbSize);
			CHECK(3 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb));
			CHECK(3 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb));
		}
		SECTION("Left Leaning Tree")
		{
			// Build the Tree A(B(0, 1), 3)
			const int32 BIndex = BuildAndAppendNode(Nodes, 0, 1);
			const int32 RootIndex = BuildAndAppendNode(Nodes, BIndex, 3);

			FAABB3 TestAabb = BuildAabbMinExtents(FVec3(1, 0, 0), AabbSize);
			CHECK(3 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb));
			CHECK(3 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb));

			TestAabb = BuildAabbMinExtents(FVec3(0, 1, 0), AabbSize);
			CHECK(BIndex == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb));
			CHECK(BIndex == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb));
		}
		SECTION("Right Leaning Tree")
		{
			// Build the Tree A(0, C(2, 3))
			const int32 CIndex = BuildAndAppendNode(Nodes, 2, 3);
			const int32 RootIndex = BuildAndAppendNode(Nodes, 0, CIndex);

			FAABB3 TestAabb = BuildAabbMinExtents(FVec3(1, 0, 0), AabbSize);
			CHECK(CIndex == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb));
			CHECK(CIndex == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb));

			TestAabb = BuildAabbMinExtents(FVec3(0, 1, 0), AabbSize);
			CHECK(0 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb));
			CHECK(0 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb));
		}
		SECTION("Tied SA Hueristic")
		{
			// Build a new test scene:
			// 4 | 6
			// 	 |	
			// 	Y|Z	
			// --+--
			//  7|5	
			// With the tree: A(B(4,5),C(6,7)
			// The important thing about this tree is that the regions Y and Z are fully contained in by B and C. 
			// This will lead to a tied heuristic. We bias using the center distance, so Y should go to B(4,5) and Z should go to C(6,7).
			// Note: greedy will use this heuristic, however global will actually pick the other node.
			const FAabbTreeNode Node4{ .Aabb = BuildAabbMinExtents(FVec3(-2, 2, 0), AabbSize), .UserData = 4, };
			const FAabbTreeNode Node5{ .Aabb = BuildAabbMinExtents(FVec3(0, -1, 0), AabbSize), .UserData = 5, };
			const FAabbTreeNode Node6{ .Aabb = BuildAabbMinExtents(FVec3(1, 2, 0), AabbSize), .UserData = 6, };
			const FAabbTreeNode Node7{ .Aabb = BuildAabbMinExtents(FVec3(-1, -1, 0), AabbSize), .UserData = 7, };
			const int32 Index4 = Nodes.Emplace(Node4);
			const int32 Index5 = Nodes.Emplace(Node5);
			const int32 Index6 = Nodes.Emplace(Node6);
			const int32 Index7 = Nodes.Emplace(Node7);
			const int32 BIndex = BuildAndAppendNode(Nodes, Index4, Index5);
			const int32 CIndex = BuildAndAppendNode(Nodes, Index6, Index7);
			const int32 RootIndex = BuildAndAppendNode(Nodes, BIndex, CIndex);

			FAABB3 TestAabb = BuildAabbMinExtents(FVec3(0, 0, 0), AabbSize);
			CHECK(Index5 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb));
			CHECK(Index7 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb));

			TestAabb = BuildAabbMinExtents(FVec3(-1, 0, 0), AabbSize);
			CHECK(Index7 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb));
			CHECK(Index5 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb));
		}
	}

	TEST_CASE("FindBestSibling - Complex", "[Chaos][AabbTreeAlgorithm][spatial-partition]")
	{
		const FVec3 AabbSize = FVec3(1, 1, 1);

		// Build the grid of:
		//  1| 3
		// 0 |2
		// --+--
		//  5| 7
		// 4 |6
		// With the tree of:
		//        A
		//    B       C
		//  D   E   F   G
		// 0 1 2 3 4 5 6 7
		const FAabbTreeNode Node0{ .Aabb = BuildAabbMinExtents(FVec3(-2, 0, 0), AabbSize), .UserData = 0, };
		const FAabbTreeNode Node1{ .Aabb = BuildAabbMinExtents(FVec3(-1, 1, 0), AabbSize), .UserData = 1, };
		const FAabbTreeNode Node2{ .Aabb = BuildAabbMinExtents(FVec3(0, 0, 0), AabbSize), .UserData = 2, };
		const FAabbTreeNode Node3{ .Aabb = BuildAabbMinExtents(FVec3(1, 1, 0), AabbSize), .UserData = 3, };
		const FAabbTreeNode Node4{ .Aabb = BuildAabbMinExtents(FVec3(-2, -2, 0), AabbSize), .UserData = 4, };
		const FAabbTreeNode Node5{ .Aabb = BuildAabbMinExtents(FVec3(-1, -1, 0), AabbSize), .UserData = 5, };
		const FAabbTreeNode Node6{ .Aabb = BuildAabbMinExtents(FVec3(0, -2, 0), AabbSize), .UserData = 6, };
		const FAabbTreeNode Node7{ .Aabb = BuildAabbMinExtents(FVec3(1, -1, 0), AabbSize), .UserData = 7, };
		TArray<FAabbTreeNode> Nodes{ Node0, Node1, Node2, Node3, Node4, Node5, Node6, Node7 };
		const int32 DIndex = BuildAndAppendNode(Nodes, 0, 1);
		const int32 EIndex = BuildAndAppendNode(Nodes, 2, 3);
		const int32 FIndex = BuildAndAppendNode(Nodes, 4, 5);
		const int32 GIndex = BuildAndAppendNode(Nodes, 6, 7);
		const int32 BIndex = BuildAndAppendNode(Nodes, DIndex, EIndex);
		const int32 CIndex = BuildAndAppendNode(Nodes, FIndex, GIndex);
		const int32 RootIndex = BuildAndAppendNode(Nodes, BIndex, CIndex);

		// Test trivial values (the nodes themselves). These work for all searches
		CHECK(0 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, Node0.Aabb));
		CHECK(1 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, Node1.Aabb));
		CHECK(2 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, Node2.Aabb));
		CHECK(3 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, Node3.Aabb));
		CHECK(4 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, Node4.Aabb));
		CHECK(5 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, Node5.Aabb));
		CHECK(6 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, Node6.Aabb));
		CHECK(7 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, Node7.Aabb));

		CHECK(0 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, Node0.Aabb));
		CHECK(1 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, Node1.Aabb));
		CHECK(2 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, Node2.Aabb));
		CHECK(3 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, Node3.Aabb));
		CHECK(4 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, Node4.Aabb));
		CHECK(5 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, Node5.Aabb));
		CHECK(6 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, Node6.Aabb));
		CHECK(7 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, Node7.Aabb));

		CHECK(0 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, Node0.Aabb));
		CHECK(1 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, Node1.Aabb));
		CHECK(2 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, Node2.Aabb));
		CHECK(3 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, Node3.Aabb));
		CHECK(4 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, Node4.Aabb));
		CHECK(5 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, Node5.Aabb));
		CHECK(6 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, Node6.Aabb));
		CHECK(7 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, Node7.Aabb));

		CHECK(0 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, Node0.Aabb));
		CHECK(1 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, Node1.Aabb));
		CHECK(2 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, Node2.Aabb));
		CHECK(3 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, Node3.Aabb));
		CHECK(4 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, Node4.Aabb));
		CHECK(5 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, Node5.Aabb));
		CHECK(6 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, Node6.Aabb));
		CHECK(7 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, Node7.Aabb));

		// Now test the grid boundaries along an axis. These tests produce different results with greedy vs. global searches.
		// The test aabbs are slightly shifted so that there are no ties.
		// The greedy algorithm will chose which top-level aabb is closest while the global will chose the leaf aabb closest.
		CHECK(4 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-2, -0.51f, 0), AabbSize)));
		CHECK(1 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-1, -0.49f, 0), AabbSize)));
		CHECK(6 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, BuildAabbMinExtents(FVec3(0, -0.51f, 0), AabbSize)));
		CHECK(3 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, BuildAabbMinExtents(FVec3(1, -0.49f, 0), AabbSize)));
		CHECK(4 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.51f, -2, 0), AabbSize)));
		CHECK(7 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.49f, -1, 0), AabbSize)));
		CHECK(0 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.51f, 0, 0), AabbSize)));
		CHECK(3 == AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.49f, 1, 0), AabbSize)));

		CHECK(4 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-2, -0.51f, 0), AabbSize)));
		CHECK(1 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-1, -0.49f, 0), AabbSize)));
		CHECK(6 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, BuildAabbMinExtents(FVec3(0, -0.51f, 0), AabbSize)));
		CHECK(3 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, BuildAabbMinExtents(FVec3(1, -0.49f, 0), AabbSize)));
		CHECK(4 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.51f, -2, 0), AabbSize)));
		CHECK(7 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.49f, -1, 0), AabbSize)));
		CHECK(0 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.51f, 0, 0), AabbSize)));
		CHECK(3 == AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.49f, 1, 0), AabbSize)));

		CHECK(0 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-2, -0.51f, 0), AabbSize)));
		CHECK(5 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-1, -0.49f, 0), AabbSize)));
		CHECK(2 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, BuildAabbMinExtents(FVec3(0, -0.51f, 0), AabbSize)));
		CHECK(7 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, BuildAabbMinExtents(FVec3(1, -0.49f, 0), AabbSize)));
		CHECK(6 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.51f, -2, 0), AabbSize)));
		CHECK(5 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.49f, -1, 0), AabbSize)));
		CHECK(2 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.51f, 0, 0), AabbSize)));
		CHECK(1 == AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.49f, 1, 0), AabbSize)));

		CHECK(0 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-2, -0.51f, 0), AabbSize)));
		CHECK(5 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-1, -0.49f, 0), AabbSize)));
		CHECK(2 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, BuildAabbMinExtents(FVec3(0, -0.51f, 0), AabbSize)));
		CHECK(7 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, BuildAabbMinExtents(FVec3(1, -0.49f, 0), AabbSize)));
		CHECK(6 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.51f, -2, 0), AabbSize)));
		CHECK(5 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.49f, -1, 0), AabbSize)));
		CHECK(2 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.51f, 0, 0), AabbSize)));
		CHECK(1 == AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, BuildAabbMinExtents(FVec3(-0.49f, 1, 0), AabbSize)));
	}

	TEST_CASE("FindBestSibling - performance", "[Chaos][AabbTreeAlgorithm][spatial-partition][!benchmark]")
	{
		constexpr int32 Count = 1 << 15;
		TArray<FAabbTreeNode> Nodes;
		BuildNodeList(Nodes, Count);
		// Build a trivial tree that groups neighboring pairs together recursively
		const int32 RootIndex = BuildSimpleTree(Nodes);

		SECTION("Greedy")
		{
			BENCHMARK("Aabb[0]")
			{
				const FAABB3& TestAabb = Nodes[0].Aabb;
				return AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, TestAabb);
			};
			BENCHMARK("Aabb[N - 1]")
			{
				const FAABB3& TestAabb = Nodes[Count - 1].Aabb;
				return AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, TestAabb);
			};
			BENCHMARK("Aabb[N/2]")
			{
				const FAABB3& TestAabb = Nodes[Count / 2].Aabb;
				return AabbTreeAlgorithm::FindBestSiblingGreedy(Nodes, RootIndex, TestAabb);
			};
		}
		SECTION("AdvancedGreedySAH")
		{
			BENCHMARK("Aabb[0]")
			{
				const FAABB3& TestAabb = Nodes[0].Aabb;
				return AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb);
			};
			BENCHMARK("Aabb[N - 1]")
			{
				const FAABB3& TestAabb = Nodes[Count - 1].Aabb;
				return AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb);
			};
			BENCHMARK("Aabb[N/2]")
			{
				const FAABB3& TestAabb = Nodes[Count / 2].Aabb;
				return AabbTreeAlgorithm::FindBestSiblingAdvancedGreedySAH(Nodes, RootIndex, TestAabb);
			};
		}
		SECTION("Global")
		{
			BENCHMARK("Aabb[0]")
			{
				const FAABB3& TestAabb = Nodes[0].Aabb;
				return AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb);
			};
			BENCHMARK("Aabb[N - 1]")
			{
				const FAABB3& TestAabb = Nodes[Count - 1].Aabb;
				return AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb);
			};
			BENCHMARK("Aabb[N/2]")
			{
				const FAABB3& TestAabb = Nodes[Count / 2].Aabb;
				return AabbTreeAlgorithm::FindBestSiblingGlobalSearch(Nodes, RootIndex, TestAabb);
			};
		}
		SECTION("BranchAndBound")
		{
			BENCHMARK("Aabb[0]")
			{
				const FAABB3& TestAabb = Nodes[0].Aabb;
				return AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, TestAabb);
			};
			BENCHMARK("Aabb[N - 1]")
			{
				const FAABB3& TestAabb = Nodes[Count - 1].Aabb;
				return AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, TestAabb);
			};
			BENCHMARK("Aabb[N/2]")
			{
				const FAABB3& TestAabb = Nodes[Count / 2].Aabb;
				return AabbTreeAlgorithm::FindBestSiblingBranchAndBound(Nodes, RootIndex, TestAabb);
			};
		}
	}
} // namespace Chaos::SpatialPartition::LowLevelTest

#endif // WITH_LOW_LEVEL_TESTS
