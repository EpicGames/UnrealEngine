// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "CoreMinimal.h"

#include "ChaosSpatialPartitions/Algorithms/PartitioningAlgorithms.h"

#include "ChaosSpatialPartitions/TestHarness/AabbTreeTestHelpers.h"
#include "ChaosSpatialPartitions/TestHarness/Common.h"

namespace Chaos::SpatialPartition::LowLevelTest
{
	void CheckArraySplit(const TArray<FTestObject>& Objects, const int32 SplitAxis, const int32 SplitIndex, const FReal SplitValue)
	{
		for (int32 I = 0; I < SplitIndex; ++I)
		{
			CHECK(Objects[I].GetCenter()[SplitAxis] < SplitValue);
		}
		for (int32 I = SplitIndex; I < Objects.Num(); ++I)
		{
			CHECK(Objects[I].GetCenter()[SplitAxis] >= SplitValue);
		}
	}

	// Validates the objects are in an expected relative ordering around a pivot index. That is, for each object it makes sure that it's final index < pivot index.
	// This is used to validate the relative ordering of objects after partitioning, but when there's no clear value that can be used for comparisons.
	// Note: This assumes that the object's UserData maps to the original index in the array.
	void CheckObjectOrdering(const TArray<FTestObject>& Objects, const TArray<bool>& ExpectedLessThanValues, const int32 PivotIndex)
	{
		check(Objects.Num() == ExpectedLessThanValues.Num());

		// First, build an array that maps the input indices (user data) to the output (current) array indices.
		TArray<int32> InputToOutputIndices;
		InputToOutputIndices.SetNum(Objects.Num());
		for (int32 I = 0; I < Objects.Num(); ++I)
		{
			REQUIRE(InputToOutputIndices.IsValidIndex(Objects[I].UserData));
			InputToOutputIndices[Objects[I].UserData] = I;
		}

		// Now we can validate the expected ordering by checking that each object ends above or below the pivot point
		for (int32 I = 0; I < ExpectedLessThanValues.Num(); ++I)
		{
			CHECK((InputToOutputIndices[I] < PivotIndex) == ExpectedLessThanValues[I]);
		}
	}

	TEST_CASE("ComputeBin", "[Chaos][PartitioningAlgorithms][spatial-partition]")
	{
		// Setup with 5 bins, each with a size of 2, with a total bounds from 1 to 11.
		// This should be the bins: [1, 3), [3, 5), [5, 7), [7, 9), [9, 11]
		constexpr int32 BinCount = 5;
		const FReal BinsLowerBound = 1;
		const FReal BinBoundsInvSize = 1 / 10.0f;
		const int32 Axis = 0;
		const FVec3 AabbSize(1);
		TArray< AabbTreeAlgorithm::FAabbBin> Bins;
		Bins.SetNum(BinCount);

		int32 ResultBinIndex;
		SECTION("Test Bins Lower Bound: Result Bin 0")
		{
			FAABB3 Aabb = BuildAabbCenterExtents(FVec3(BinsLowerBound), AabbSize);
			AabbTreeAlgorithm::ComputeBin(Aabb, Axis, BinsLowerBound, BinBoundsInvSize, Bins, ResultBinIndex);
			CHECK(ResultBinIndex == 0);
			CHECK(Bins[0].Count == 1);
			CHECK_THAT(Bins[0].Aabb, Catch::Equal(Aabb));
		}
		SECTION("Test Bins Upper Bound: Result Bin 0")
		{
			FAABB3 Aabb = BuildAabbCenterExtents(FVec3(11), AabbSize);
			AabbTreeAlgorithm::ComputeBin(Aabb, Axis, BinsLowerBound, BinBoundsInvSize, Bins, ResultBinIndex);

			const int32 ExpectedBinIndex = BinCount - 1;
			CHECK(ResultBinIndex == ExpectedBinIndex);
			CHECK(Bins[ExpectedBinIndex].Count == 1);
			CHECK_THAT(Bins[ExpectedBinIndex].Aabb, Catch::Equal(Aabb));
		}
		SECTION("Test Lower End of Bin")
		{
			FAABB3 Aabb = BuildAabbCenterExtents(FVec3(5.1f), AabbSize);
			AabbTreeAlgorithm::ComputeBin(Aabb, Axis, BinsLowerBound, BinBoundsInvSize, Bins, ResultBinIndex);

			const int32 ExpectedBinIndex = 2;
			CHECK(ResultBinIndex == ExpectedBinIndex);
			CHECK(Bins[ExpectedBinIndex].Count == 1);
			CHECK_THAT(Bins[ExpectedBinIndex].Aabb, Catch::Equal(Aabb));
		}
		SECTION("Test Lower End of Bin")
		{
			FAABB3 Aabb = BuildAabbCenterExtents(FVec3(5.1f), AabbSize);
			AabbTreeAlgorithm::ComputeBin(Aabb, Axis, BinsLowerBound, BinBoundsInvSize, Bins, ResultBinIndex);

			const int32 ExpectedBinIndex = 2;
			CHECK(ResultBinIndex == ExpectedBinIndex);
			CHECK(Bins[ExpectedBinIndex].Count == 1);
			CHECK_THAT(Bins[ExpectedBinIndex].Aabb, Catch::Equal(Aabb));
		}
		SECTION("Test Upper End of Bin")
		{
			FAABB3 Aabb = BuildAabbCenterExtents(FVec3(8.9f), AabbSize);
			AabbTreeAlgorithm::ComputeBin(Aabb, Axis, BinsLowerBound, BinBoundsInvSize, Bins, ResultBinIndex);

			const int32 ExpectedBinIndex = 3;
			CHECK(ResultBinIndex == ExpectedBinIndex);
			CHECK(Bins[ExpectedBinIndex].Count == 1);
			CHECK_THAT(Bins[ExpectedBinIndex].Aabb, Catch::Equal(Aabb));
		}
	}

	TEST_CASE("PartitionElementsInPlace - SplitValue", "[Chaos][PartitioningAlgorithms][spatial-partition]")
	{
		constexpr int32 SplitAxis = 0;
		const FVec3 Extents(1);

		TArray<FTestObject> Objects;
		SECTION("0 Elements")
		{
			TArrayView<const FTestObject> View(Objects);
			CHECK(0 == AabbTreeAlgorithm::PartitionElementsInPlace(View, SplitAxis, 1));
		}
		SECTION("1 Element")
		{
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3::Zero(), Extents) });
			TArrayView<const FTestObject> View(Objects);

			SECTION("SplitValue: -1")
			{
				CHECK(0 == AabbTreeAlgorithm::PartitionElementsInPlace(View, SplitAxis, -1));
				CheckArraySplit(Objects, SplitAxis, 0, -1);
			}
			SECTION("SplitValue: 1")
			{
				CHECK(1 == AabbTreeAlgorithm::PartitionElementsInPlace(View, SplitAxis, 1));
				CheckArraySplit(Objects, SplitAxis, 1, 1);
			}
		}
		SECTION("Multiple Elements")
		{
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(1), Extents) });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3::Zero(), Extents) });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(-1), Extents) });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(-1), Extents) });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3::Zero(), Extents) });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(1), Extents) });
			TArrayView<const FTestObject> View(Objects);

			SECTION("SplitValue: -2")
			{
				CHECK(0 == AabbTreeAlgorithm::PartitionElementsInPlace(View, SplitAxis, -2));
				CheckArraySplit(Objects, SplitAxis, 0, -2);
			}
			SECTION("SplitValue: -0.5")
			{
				CHECK(2 == AabbTreeAlgorithm::PartitionElementsInPlace(View, SplitAxis, -0.5f));
				CheckArraySplit(Objects, SplitAxis, 2, -0.5f);
			}
			SECTION("SplitValue: 0.5")
			{
				CHECK(4 == AabbTreeAlgorithm::PartitionElementsInPlace(View, SplitAxis, 0.5f));
				CheckArraySplit(Objects, SplitAxis, 4, 0.5f);
			}
			SECTION("SplitValue: 2")
			{
				CHECK(6 == AabbTreeAlgorithm::PartitionElementsInPlace(View, SplitAxis, 2));
				CheckArraySplit(Objects, SplitAxis, 6, 2);
			}
		}
		SECTION("All elements the same")
		{
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3::Zero(), Extents) });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3::Zero(), Extents) });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3::Zero(), Extents) });
			TArrayView<const FTestObject> View(Objects);

			SECTION("Split Value: -0.1")
			{
				CHECK(0 == AabbTreeAlgorithm::PartitionElementsInPlace(View, SplitAxis, -0.1f));
				CheckArraySplit(Objects, SplitAxis, 0, 0);
			}
			SECTION("Split Value: 0.1")
			{
				CHECK(3 == AabbTreeAlgorithm::PartitionElementsInPlace(View, SplitAxis, 0.1f));
				CheckArraySplit(Objects, SplitAxis, 3, 0.1f);
			}
		}
	}

	TEST_CASE("PartitionElementsInPlace - Bins", "[Chaos][PartitioningAlgorithms][spatial-partition]")
	{
		constexpr int32 SplitAxis = 0;
		const FVec3 Extents(1);

		TArray<FTestObject> Objects;
		TArray<int32> Bins;
		// To make life easy for testing, compute bins from the object centers. Just pretend the bins lower bound is 0 and use the give size to compute the bin index.
		auto ComputeBins = [](const TArray<FTestObject>& Objects, const FReal BinSize, TArray<int32>& Bins)
		{
			for (const FTestObject& Obj : Objects)
			{
				Bins.Add((int32)(Obj.GetCenter()[0] / BinSize));
			}
		};

		SECTION("0 Elements")
		{
			TArrayView<const FTestObject> ObjectsView(Objects);
			TArrayView<int32> BinsView(Bins);
			CHECK(0 == AabbTreeAlgorithm::PartitionElementsInPlace(ObjectsView, BinsView, 0));
		}
		SECTION("1 Element")
		{
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3::Zero(), Extents) });
			ComputeBins(Objects, 2, Bins);
			TArrayView<const FTestObject> ObjectsView(Objects);
			TArrayView<int32> BinsView(Bins);

			SECTION("Split Bin Index: -1")
			{
				CHECK(0 == AabbTreeAlgorithm::PartitionElementsInPlace(ObjectsView, BinsView, -1));
				CheckArraySplit(Objects, SplitAxis, 0, 0);
			}
			SECTION("Split Bin Index: 1")
			{
				CHECK(1 == AabbTreeAlgorithm::PartitionElementsInPlace(ObjectsView, BinsView, 1));
				CheckArraySplit(Objects, SplitAxis, 1, 1);
			}
		}
		SECTION("Multiple Elements")
		{
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(2.5), Extents) }); // Bin: 1
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3::Zero(), Extents) }); // Bin: 0
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(-3.5), Extents) }); // Bin: -1
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(-2.5), Extents) }); // Bin: -1
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3::Zero(), Extents) }); // Bin: 0
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(3.5), Extents) }); // Bin: 1
			ComputeBins(Objects, 2, Bins);
			TArrayView<const FTestObject> ObjectsView(Objects);
			TArrayView<int32> BinsView(Bins);

			// Check all bin indices, including ties. In order to verify the array, we have to check the actual objects that are swapped. 
			// To make life easy, this re-uses CheckArraySplit and picks a value safely in-between that should guarantee the object count we want.
			SECTION("Split Bin Index: -2")
			{
				CHECK(0 == AabbTreeAlgorithm::PartitionElementsInPlace(ObjectsView, BinsView, -2));
				CheckArraySplit(Objects, SplitAxis, 0, -4);
			}
			SECTION("Split Bin Index: -1")
			{
				CHECK(0 == AabbTreeAlgorithm::PartitionElementsInPlace(ObjectsView, BinsView, -1));
				CheckArraySplit(Objects, SplitAxis, 0, -4);
			}
			SECTION("Split Bin Index: 0")
			{
				CHECK(2 == AabbTreeAlgorithm::PartitionElementsInPlace(ObjectsView, BinsView, 0));
				CheckArraySplit(Objects, SplitAxis, 2, -2);
			}
			SECTION("Split Bin Index: 1")
			{
				CHECK(4 == AabbTreeAlgorithm::PartitionElementsInPlace(ObjectsView, BinsView, 1));
				CheckArraySplit(Objects, SplitAxis, 4, 2);
			}
			SECTION("Split Bin Index: 2")
			{
				CHECK(6 == AabbTreeAlgorithm::PartitionElementsInPlace(ObjectsView, BinsView, 2));
				CheckArraySplit(Objects, SplitAxis, 6, 4);
			}
		}
		SECTION("All elements the same")
		{
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(0), Extents) }); // Bin: 0
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(1), Extents) }); // Bin: 0
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(1.9), Extents) }); // Bin: 0
			ComputeBins(Objects, 2, Bins);
			TArrayView<const FTestObject> ObjectsView(Objects);
			TArrayView<int32> BinsView(Bins);

			SECTION("Split Bin Index: 0")
			{
				CHECK(0 == AabbTreeAlgorithm::PartitionElementsInPlace(ObjectsView, BinsView, 0));
				CheckArraySplit(Objects, SplitAxis, 0, -1);
			}
			SECTION("Split Bin Index: 1")
			{
				CHECK(3 == AabbTreeAlgorithm::PartitionElementsInPlace(ObjectsView, BinsView, 1));
				CheckArraySplit(Objects, SplitAxis, 3, 2);
			}
		}
	}

	TEST_CASE("ComputeSplitPlaneWithSpatialMedianHeuristic", "[Chaos][PartitioningAlgorithms][spatial-partition]")
	{
		TArray<FTestObject> Objects;

		int32 SplitAxis;
		FReal SplitPosition;
		for (int32 Axis = 0; Axis < 3; ++Axis)
		{
			DYNAMIC_SECTION("Uniform Size: Axis " << Axis)
			{
				// Make the objects on the test axis more spread out than the other axes.
				FVec3 Spacing(1);
				Spacing[Axis] = 2;
				BuildObjectList(Objects, 3, 3, 3, Spacing);

				TArrayView<const FTestObject> View(Objects);
				AabbTreeAlgorithm::ComputeSplitPlaneWithSpatialMedianHeuristic(View, SplitAxis, SplitPosition);
				CHECK(SplitAxis == Axis);
				CHECK_THAT(SplitPosition, Catch::Matchers::WithinRel(3.5f));
			}
		}
		// Test that size doesn't affect things
		for (int32 Axis = 0; Axis < 3; ++Axis)
		{
			DYNAMIC_SECTION("Large Size: Axis " << Axis)
			{
				FVec3 Spacing(1);
				Spacing[Axis] = 2;
				BuildObjectList(Objects, 3, 3, 3, Spacing);
				
				// Make every other axis larger. This shouldn't affect anything as only the center is considered.
				FVec3 AabbSize(5);
				AabbSize[Axis] = 1;
				for (FTestObject& Object : Objects)
				{
					Object.Aabb = BuildAabbCenterExtents(Object.Aabb.GetCenter(), AabbSize);
				}

				TArrayView<const FTestObject> View(Objects);
				AabbTreeAlgorithm::ComputeSplitPlaneWithSpatialMedianHeuristic(View, SplitAxis, SplitPosition);
				CHECK(SplitAxis == Axis);
				CHECK_THAT(SplitPosition, Catch::Matchers::WithinRel(3.5f));
			}
		}
	}

	TEST_CASE("ComputeSplitPlaneWithMedianVarianceHeuristic", "[Chaos][PartitioningAlgorithms][spatial-partition]")
	{
		// Build a set of evenly spaced boxes on each axis. For these tests, we'll add an extra box within the group's aabb.
		// This should not affect the spatial bounds, but does affect the variance which should bias that axis to be picked.
		const FVec3 AabbSize(1);
		TArray<FTestObject> Objects;
		Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(-5, 0, 0), AabbSize) });
		Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(5, 0, 0), AabbSize) });
		Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(0, -5, 0), AabbSize) });
		Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(0, 5, 0), AabbSize) });
		Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(0, 0, -5), AabbSize) });
		Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(0, 0, 5), AabbSize) });

		int32 SplitAxis;
		FReal SplitPosition;
		for (int32 Axis = 0; Axis < 3; ++Axis)
		{
			DYNAMIC_SECTION("Axis " << Axis)
			{
				FVec3 Center(0);
				Center[Axis] = 4;
				Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(Center, AabbSize) });
				// (5 + -5 + 4) / 7
				const FReal ExpectedSplitPosition = 4.0 / 7.0;

				TArrayView<const FTestObject> View(Objects);
				AabbTreeAlgorithm::ComputeSplitPlaneWithMedianVarianceHeuristic(View, SplitAxis, SplitPosition);

				CHECK(SplitAxis == Axis);
				CHECK_THAT(SplitPosition, Catch::Matchers::WithinRel(ExpectedSplitPosition, 0.0001));
			}
		}
	}

	TEST_CASE("ComputeSplitPlaneWithSurfaceAreaHeuristic", "[Chaos][PartitioningAlgorithms][spatial-partition]")
	{
		constexpr int32 BinCount = 8;
		TArray<FTestObject> Objects;
		TArray<int32> BinIndices;

		SECTION("Balanced Split")
		{
			// Setup objects that are furthest spread on the x-axis. The objects are somewhat evenly spaced, so a SA heuristic should produce an even split.
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(-5, 0, 0), FVec3(1)), .UserData = 0 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(5, 0, 0), FVec3(1)), .UserData = 1 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(3, 1, 0), FVec3(1)), .UserData = 2 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(-3, -1, 0), FVec3(1)), .UserData = 3 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(-1, 0, 1), FVec3(1)), .UserData = 4 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(1, 0, -1), FVec3(1)), .UserData = 5 });
			BinIndices.SetNum(Objects.Num());
			const TArray<bool> ExpectedOrders = { true, false, false, true, true, false };

			TArrayView<const FTestObject> ObjectsView(Objects);
			TArrayView<int32> BinIndicesView(BinIndices);
			const int32 SplitIndex = AabbTreeAlgorithm::PartitionEntriesWithSurfaceArea(ObjectsView, BinCount, BinIndicesView);

			CHECK(SplitIndex == 3);
			CheckObjectOrdering(Objects, ExpectedOrders, SplitIndex);
		}
		SECTION("Imbalanced Split")
		{
			// Setup objects 0 and 4 to be really close together, such that we end up with [0, 4], [1, 2, 3, 5] as the splits.
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(-5, 0, 0), FVec3(1)), .UserData = 0 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(5, -1, 0), FVec3(1)), .UserData = 1 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(2, 1, -1), FVec3(1)), .UserData = 2 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(4, 0, 0), FVec3(1)), .UserData = 3 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(-5, 1, 0), FVec3(1)), .UserData = 4 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(3, 0, 1), FVec3(1)), .UserData = 5 });
			BinIndices.SetNum(Objects.Num());
			const TArray<bool> ExpectedOrders = { true, false, false, false, true, false };

			TArrayView<const FTestObject> ObjectsView(Objects);
			TArrayView<int32> BinIndicesView(BinIndices);
			const int32 SplitIndex = AabbTreeAlgorithm::PartitionEntriesWithSurfaceArea(ObjectsView, BinCount, BinIndicesView);

			CHECK(SplitIndex == 2);
			CheckObjectOrdering(Objects, ExpectedOrders, SplitIndex);
		}
		SECTION("Bin Counts")
		{
			// Setup boxes:
			//     03
			//    4
			// 152 
			// With a bin count of 3, this must choose between [1524,03] and [15,2403] and will choose the first.
			// With a bin count of 6 this can now build an even split which will generate [152, 403]
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(4, 2, 0), FVec3(1)), .UserData = 0 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(0, 0, 0), FVec3(1)), .UserData = 1 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(2, 0, 0), FVec3(1)), .UserData = 2 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(5, 2, 0), FVec3(1)), .UserData = 3 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(3, 1, 0), FVec3(1)), .UserData = 4 });
			Objects.Add(FTestObject{ .Aabb = BuildAabbCenterExtents(FVec3(1, 0, 0), FVec3(1)), .UserData = 5 });
			BinIndices.SetNum(Objects.Num());
			TArrayView<const FTestObject> ObjectsView(Objects);
			TArrayView<int32> BinIndicesView(BinIndices);

			SECTION("Bin Count: 3")
			{
				const int32 SplitIndex = AabbTreeAlgorithm::PartitionEntriesWithSurfaceArea(ObjectsView, 3, BinIndicesView);

				const TArray<bool> ExpectedOrders = { false, true, true, false, true, true };
				CHECK(SplitIndex == 4);
				CheckObjectOrdering(Objects, ExpectedOrders, 4);
			}
			SECTION("Bin Count: 6")
			{
				const int32 SplitIndex = AabbTreeAlgorithm::PartitionEntriesWithSurfaceArea(ObjectsView, 6, BinIndicesView);

				const TArray<bool> ExpectedOrders = { false, true, true, false, false, true };
				CHECK(SplitIndex == 3);
				CheckObjectOrdering(Objects, ExpectedOrders, 3);
			}
		}
	}
} // namespace Chaos::SpatialPartition::LowLevelTest

#endif // WITH_LOW_LEVEL_TESTS
