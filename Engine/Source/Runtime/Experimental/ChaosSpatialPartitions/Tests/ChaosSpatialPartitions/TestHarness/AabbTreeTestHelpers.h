// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_LOW_LEVEL_TESTS

#include "CoreMinimal.h"

#include "ChaosSpatialPartitions/TestHarness/Common.h"
#include "ChaosSpatialPartitions/TestHarness/TestObject.h"

#include "ChaosSpatialPartitions/Algorithms/AabbTreeAlgorithm.h"

namespace Chaos::SpatialPartition::LowLevelTest
{
	inline bool IsValidNodeIndex(int32 NodeIndex)
	{
		return NodeIndex != INDEX_NONE;
	}

	inline void Compare(const TArray<FAabbTreeNode>& ExpectedNodes, const int32 ExpectedIndex, const TArray<FAabbTreeNode>& ActualNodes, const int32 ActualIndex)
	{
		const FAabbTreeNode& ExpectedNode = ExpectedNodes[ExpectedIndex];
		const FAabbTreeNode& ActualNode = ActualNodes[ActualIndex];

		CHECK(ExpectedNode.UserData == ActualNode.UserData);
		CHECK_THAT(ExpectedNode.Aabb, Catch::Equal(ActualNode.Aabb));

		CHECK(IsValidNodeIndex(ExpectedNode.Left) == IsValidNodeIndex(ActualNode.Left));
		if (IsValidNodeIndex(ExpectedNode.Left) && IsValidNodeIndex(ActualNode.Left))
		{
			Compare(ExpectedNodes, ExpectedNode.Left, ActualNodes, ActualNode.Left);
		}

		CHECK(IsValidNodeIndex(ExpectedNode.Right) == IsValidNodeIndex(ActualNode.Right));
		if (IsValidNodeIndex(ExpectedNode.Right) && IsValidNodeIndex(ActualNode.Right))
		{
			Compare(ExpectedNodes, ExpectedNode.Right, ActualNodes, ActualNode.Right);
		}
	}

	inline void BuildNodeList(TArray<FAabbTreeNode>& Nodes, const int32 CountX = 1, const int32 CountY = 1, const int32 CountZ = 1, const FVec3& Spacing = FVec3(1), const FVec3& AabbSize = FVec3(1))
	{
		const FVec3 ScaleFactors = AabbSize + Spacing;
		Nodes.SetNum(CountX * CountY * CountZ);
		for (int32 Z = 0; Z < CountZ; ++Z)
		{
			for (int32 Y = 0; Y < CountY; ++Y)
			{
				for (int32 X = 0; X < CountX; ++X)
				{
					const int32 I = GetIndex(X, Y, Z, CountX, CountY, CountZ);

					const FVec3 Min = FVec3(X, Y, Z) * ScaleFactors;
					Nodes[I].Aabb = FAABB3(Min, Min + AabbSize);
					Nodes[I].UserData = I;
#ifdef UE_ENABLE_CHAOS_AABBTREE_VISUALIZER
					Nodes[I].DebugNodeList = &Nodes;
#endif
				}
			}
		}
	}

	// Adds a new node the the node list that is the parent of the given two nodes. 
	// This automatically computes the aabb of the parent node from the two children and links them child/parent indices up.
	inline int32 BuildAndAppendNode(TArray<FAabbTreeNode>& Nodes, const int32 LeftIndex, const int32 RightIndex)
	{
		const int32 ParentIndex = Nodes.Emplace();

		FAabbTreeNode& LeftNode = Nodes[LeftIndex];
		FAabbTreeNode& RightNode = Nodes[RightIndex];
		FAabbTreeNode& ParentNode = Nodes[ParentIndex];
		ParentNode.Left = LeftIndex;
		ParentNode.Right = RightIndex;
		ParentNode.Aabb = AabbTreeAlgorithm::Union(RightNode.Aabb, LeftNode.Aabb);
		LeftNode.Parent = ParentIndex;
		RightNode.Parent = ParentIndex;
#ifdef UE_ENABLE_CHAOS_AABBTREE_VISUALIZER
		ParentNode.DebugNodeList = &Nodes;
#endif
		return ParentIndex;
	}

	inline int32 BuildSimpleTree(TArray<FAabbTreeNode>& Nodes)
	{
		if (Nodes.IsEmpty())
		{
			return INDEX_NONE;
		}

		// Build the initial list of nodes to start grouping (the leaves)
		TArray<int32> NodeIndices;
		for (int32 I = 0; I < Nodes.Num(); ++I)
		{
			NodeIndices.Add(I);
		}

		// Do a simple grouping of pairs until there's one node left
		while (NodeIndices.Num() != 1)
		{
			TArray<int32> NewNodeIndices;
			for (int32 I = 1; I < NodeIndices.Num(); I += 2)
			{
				int32 NewNodeIndex = BuildAndAppendNode(Nodes, NodeIndices[I - 1], NodeIndices[I]);
				NewNodeIndices.Add(NewNodeIndex);
			}
			// If there's a remaining node, just carry it over to the next level
			if (NodeIndices.Num() % 2 == 1)
			{
				NewNodeIndices.Add(NodeIndices.Last());
			}
			NodeIndices = NewNodeIndices;
		}
		return Nodes.Num() - 1;
	}
} // namespace Chaos::SpatialPartition::LowLevelTest

#endif // WITH_LOW_LEVEL_TESTS
