// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_LOW_LEVEL_TESTS

// The base test harness file doesn't include the benchmark files from Catch2
// so those macros show up as undefined, so we add those files here.

#if defined(THIRD_PARTY_INCLUDES_START) && defined(THIRD_PARTY_INCLUDES_END)
THIRD_PARTY_INCLUDES_START
#endif // defined(THIRD_PARTY_INCLUDES_START) && defined(THIRD_PARTY_INCLUDES_END)

#ifdef _MSC_VER
#pragma pack(push, 8)
#pragma warning(push)
#pragma warning(disable: 4005) // 'identifier': macro redefinition
#pragma warning(disable: 4582) // 'type': constructor is not implicitly called
#pragma warning(disable: 4583) // 'type': destructor is not implicitly called
#endif
#include <catch2/benchmark/catch_benchmark_all.hpp>
#include <catch2/generators/catch_generators.hpp>
#ifdef _MSC_VER
#pragma warning(pop)
#pragma pack(pop)
#endif

#if defined(THIRD_PARTY_INCLUDES_START) && defined(THIRD_PARTY_INCLUDES_END)
THIRD_PARTY_INCLUDES_END
#endif // defined(THIRD_PARTY_INCLUDES_START) && defined(THIRD_PARTY_INCLUDES_END)

// Support for e.g.:
// CHECK_THAT(Z, Catch::Matchers::WithinAbs(100.0f, 0.1f));
#include <catch2/matchers/catch_matchers_floating_point.hpp>

#endif // WITH_LOW_LEVEL_TESTS
