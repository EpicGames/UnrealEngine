// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_LOW_LEVEL_TESTS

// From LowLevelTest Module
#include "TestHarness.h"

#include "ChaosSpatialPartitions/TestHarness/Catch2.h"
#include "ChaosSpatialPartitions/TestHarness/StringMakers.h"
#include "ChaosSpatialPartitions/TestHarness/Matchers.h"

#endif // WITH_LOW_LEVEL_TESTS
