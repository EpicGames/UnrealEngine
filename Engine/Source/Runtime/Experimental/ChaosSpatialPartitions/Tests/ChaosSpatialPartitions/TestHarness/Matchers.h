// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_LOW_LEVEL_TESTS

#include "Chaos/Core.h"
#include "ChaosSpatialPartitions/TestHarness/Catch2.h"

namespace Catch
{
	template <typename T, int D>
	class FVectorEqualityMatcher : public Catch::Matchers::MatcherBase<Chaos::TVector<T, D>>
	{
	public:
		using FVector = Chaos::TVector<T, D>;
		FVector Target;
		T Epsilon;

		FVectorEqualityMatcher(const FVector& InTarget, const T InEpsilon = UE_KINDA_SMALL_NUMBER)
			: Target(InTarget)
			, Epsilon(InEpsilon)
		{
		}

		// Performs the test for this matcher
		bool match(const FVector& ToCompare) const override
		{
			return FVector::PointsAreNear(ToCompare, Target, Epsilon);
		}

		// Produces a string describing what this matcher does. It should
		// include any provided data (the begin/ end in this case) and
		// be written as if it were stating a fact (in the output it will be
		// preceded by the value under test).
		virtual std::string describe() const override
		{
			std::ostringstream Stream;
			Stream << "is equal to (" << Catch::StringMaker<FVector>::convert(Target) << ") with epsilon " << Epsilon;
			return Stream.str();
		}
	};

	template <typename T, int D>
	FVectorEqualityMatcher<T, D> ApproxEq(const Chaos::TVector<T, D>& Target, const T Epsilon = UE_KINDA_SMALL_NUMBER)
	{
		return FVectorEqualityMatcher<T, D>(Target, Epsilon);
	}
	
	template <typename T, int D>
	class TAABBEqualityMatcher : public Catch::Matchers::MatcherBase<Chaos::TAABB<T, D>>
	{
	public:
		using FAABB = Chaos::TAABB<T, D>;
		FAABB Target;
		T Epsilon;

		TAABBEqualityMatcher(const FAABB& InTarget, const T InEpsilon = static_cast<T>(UE_KINDA_SMALL_NUMBER))
			: Target(InTarget)
			, Epsilon(InEpsilon)
		{
		}

		// Performs the test for this matcher
		bool match(const FAABB& ToCompare) const override
		{
			if (Epsilon == 0)
			{
				return ToCompare.Min() == Target.Min() && ToCompare.Max() == Target.Max();
			}
			return FVector::PointsAreNear(ToCompare.Min(), Target.Min(), Epsilon) && FVector::PointsAreNear(ToCompare.Max(), Target.Max(), Epsilon);
		}

		// Produces a string describing what this matcher does. It should
		// include any provided data (the begin/ end in this case) and
		// be written as if it were stating a fact (in the output it will be
		// preceded by the value under test).
		virtual std::string describe() const override
		{
			std::ostringstream Stream;
			Stream << "is equal to (" << Catch::StringMaker<FAABB>::convert(Target) << ") with epsilon " << Epsilon;
			return Stream.str();
		}
	};
	
	template <typename T, int D>
	TAABBEqualityMatcher<T, D> Equal(const Chaos::TAABB<T, D>& Target)
	{
		return TAABBEqualityMatcher<T, D>(Target, 0);
	}
	
	template <typename T, int D>
	TAABBEqualityMatcher<T, D> ApproxEq(const Chaos::TAABB<T, D>& Target, const T Epsilon = static_cast<T>(UE_KINDA_SMALL_NUMBER))
	{
		return TAABBEqualityMatcher<T, D>(Target, Epsilon);
	}
} // namespace Catch 

#endif // WITH_LOW_LEVEL_TESTS
