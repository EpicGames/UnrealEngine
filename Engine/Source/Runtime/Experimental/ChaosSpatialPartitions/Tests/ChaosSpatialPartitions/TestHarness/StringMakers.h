// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_LOW_LEVEL_TESTS

#include "Chaos/Core.h"
#include "Chaos/AABB.h"
#include "Chaos/ChaosCoreLogUtil.h"
#include "ChaosSpatialPartitions/TestHarness/Catch2.h"

#include <string>

namespace Catch 
{
	template<>
	struct StringMaker<Chaos::FRotation3>
	{
		static std::string convert(Chaos::FRotation3 const& Value) { return std::string(TCHAR_TO_UTF8(*Chaos::ToString(Value))); }
	};

	template<>
	struct StringMaker<Chaos::FRotation3f>
	{
		static std::string convert(Chaos::FRotation3f const& Value) { return std::string(TCHAR_TO_UTF8(*Chaos::ToString(Value))); }
	};

	template<>
	struct StringMaker<Chaos::FVec3>
	{
		static std::string convert(Chaos::FVec3 const& Value) { return std::string(TCHAR_TO_UTF8(*Chaos::ToString(Value))); }
	};

	template<>
	struct StringMaker<Chaos::FVec3f>
	{
		static std::string convert(Chaos::FVec3f const& Value) { return std::string(TCHAR_TO_UTF8(*Chaos::ToString(Value))); }
	};

	template<>
	struct StringMaker<FVector3d>
	{
		static std::string convert(FVector3d const& Value) { return StringMaker<Chaos::FVec3>::convert(Chaos::FVec3(Value)); }
	};

	template<>
	struct StringMaker<FVector3f>
	{
		static std::string convert(FVector3f const& Value) { return StringMaker<Chaos::FVec3f>::convert(Chaos::FVec3f(Value)); }
	};

	template<typename T, int d>
	struct StringMaker<Chaos::TAABB<T, d>>
	{
		static std::string convert(Chaos::TAABB<T, d> const& Value) { return StringMaker<FString>::convert(Value.ToString()); }
	};
}

#endif // WITH_LOW_LEVEL_TESTS
