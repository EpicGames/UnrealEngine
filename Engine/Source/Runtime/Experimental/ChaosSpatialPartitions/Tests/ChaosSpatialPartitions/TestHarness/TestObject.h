// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_LOW_LEVEL_TESTS

#include "CoreMinimal.h"

#include "ChaosSpatialPartitions/SpatialClassification.h"
#include "ChaosSpatialPartitions/SpatialHandle.h"

namespace Chaos::SpatialPartition::LowLevelTest
{
	// Helper class to represent an object. This just makes it a bit easier to manage groups of objects for tests.
	struct FTestObject
	{
		FVec3 GetCenter() const { return Aabb.Center(); }
		const FAABB3& GetAabb() const { return Aabb; }

		FAABB3 Aabb = FAABB3::EmptyAABB();
		FUserDataType UserData = INDEX_NONE;
		FSpatialClassification Classification;
		FSpatialHandle Handle;
	};
	
	inline FAABB3 BuildAabbMinExtents(const FVec3& Min, const FVec3& Extents)
	{
		return FAABB3(Min, Min + Extents);
	}

	inline FAABB3 BuildAabbCenterExtents(const FVec3& Center, const FVec3& Extents)
	{
		const FVec3 Min = Center - Extents * 0.5f;
		const FVec3 Max = Center + Extents * 0.5f;
		return FAABB3(Min, Max);
	}

	// Build the index for a 1D array using 3D coordinates.
	inline int32 GetIndex(const int32 X, const int32 Y, const int32 Z, const int32 CountX, const int32 CountY, const int32 /*CountZ*/)
	{
		const int32 Index = X + Y * CountX + Z * (CountX * CountY);
		return Index;
	}

	inline void BuildObjectList(TArray<FTestObject>& Objects, const int32 CountX = 1, const int32 CountY = 1, const int32 CountZ = 1, const FVec3& Spacing = FVec3(1), const FVec3& AabbSize = FVec3(1))
	{
		const FVec3 ScaleFactors = AabbSize + Spacing;
		Objects.SetNum(CountX * CountY * CountZ);
		for (int32 Z = 0; Z < CountZ; ++Z)
		{
			for (int32 Y = 0; Y < CountY; ++Y)
			{
				for (int32 X = 0; X < CountX; ++X)
				{
					const int32 I = GetIndex(X, Y, Z, CountX, CountY, CountZ);

					const FVec3 Min = FVec3(X, Y, Z) * ScaleFactors;
					Objects[I].Aabb = FAABB3(Min, Min + AabbSize);
					Objects[I].UserData = I;
				}
			}
		}
	}
} // namespace Chaos::SpatialPartition::LowLevelTest

#endif // WITH_LOW_LEVEL_TESTS
