// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using System.Collections.Generic;

namespace UnrealBuildTool.Rules
{
	public class ChaosVDData : ModuleRules
	{
		public ChaosVDData(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Chaos",
					"ChaosVDRuntime"
				}
			);
		}
	}
}