// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDRecording.h"
#include "Chaos/ImplicitObject.h"
#include "ChaosVisualDebugger/ChaosVDSerializedNameTable.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDRecording)

FName FChaosVDCustomUserDataHandle::GetTypeName() const
{
	const UStruct* Struct = DataStruct->GetStruct();
	if (ensure(Struct != nullptr))
	{
		return Struct->GetFName();
	}

	return NAME_Name;
}

void FChaosVDCustomFrameData::AddData(const FChaosVDCustomUserDataHandle& InData)
{
	CustomDataHandlesByType.Add(InData.GetTypeName(), InData);
}

FChaosVDRecording::FChaosVDRecording()
{
}

int32 FChaosVDRecording::GetAvailableGameFramesNumber() const
{
	FReadScopeLock ReadLock(RecordingDataLock);
	
	return GetAvailableGameFramesNumber_AssumesLocked();
}

int32 FChaosVDRecording::GetAvailableGameFramesNumber_AssumesLocked() const
{
	return GameFrames.Num(); 
}

int32 FChaosVDRecording::GetAvailableSolverFramesNumber(const int32 SolverID) const
{
	FReadScopeLock ReadLock(RecordingDataLock);

	return GetAvailableSolverFramesNumber_AssumesLocked(SolverID);
}

int32 FChaosVDRecording::GetAvailableSolverFramesNumber_AssumesLocked(int32 SolverID) const
{
	if (const TArray<FChaosVDSolverFrameData>* SolverData = RecordedFramesDataPerSolver.Find(SolverID))
	{
		return SolverData->Num();
	}

	return INDEX_NONE;
}

FName FChaosVDRecording::GetSolverFName(int32 SolverID)
{
	FReadScopeLock ReadLock(RecordingDataLock);

	return GetSolverFName_AssumedLocked(SolverID);
}

FName FChaosVDRecording::GetSolverFName_AssumedLocked(int32 SolverID)
{
	static FName DefaultName("Invalid");

	// Currently we don't create an entry per solver, so we need to get the name from the frame data
	// TODO: Record Solver specific data per instance and not per frame
	if (TArray<FChaosVDSolverFrameData>* SolverFramesData = RecordedFramesDataPerSolver.Find(SolverID))
	{
		if(!SolverFramesData->IsEmpty())
		{
			return (*SolverFramesData)[0].DebugFName;
		}
	}

	return DefaultName;
}

bool FChaosVDRecording::IsServerSolver_AssumesLocked(int32 SolverID)
{
	return GetSolverFName_AssumedLocked(SolverID).ToString().Contains(TEXT("Server"));
}

bool FChaosVDRecording::IsServerSolver(int32 SolverID)
{
	FReadScopeLock ReadLock(RecordingDataLock);
	return IsServerSolver_AssumesLocked(SolverID);
}

FChaosVDSolverFrameData* FChaosVDRecording::GetSolverFrameData_AssumesLocked(const int32 SolverID, const int32 FrameNumber, bool bKeyFrameOnly)
{
	if (TArray<FChaosVDSolverFrameData>* SolverFrames = RecordedFramesDataPerSolver.Find(SolverID))
	{
		//TODO: Find a safer way of do this. If someone stores this ptr bad things will happen
		if (FChaosVDSolverFrameData* FoundFrame = SolverFrames->IsValidIndex(FrameNumber) ? &(*SolverFrames)[FrameNumber] : nullptr)
		{
			if (!FoundFrame->bIsKeyFrame && bKeyFrameOnly)
			{
				if (TMap<int32, FChaosVDSolverFrameData>* GenerateSolverFrameByNumber = GeneratedKeyFrameDataPerSolver.Find(SolverID))
				{
					if (FChaosVDSolverFrameData* GeneratedKeyFrame = GenerateSolverFrameByNumber->Find(FrameNumber))
					{
						return GeneratedKeyFrame;
					}
				}
				ensureMsgf(false, TEXT("Failed to find generated KeyFrame [%d] for Solver [%d]"), FrameNumber, SolverID);
			}
			else
			{
				return FoundFrame;
			}			
		}
	}

	return nullptr;
}

FChaosVDSolverFrameData* FChaosVDRecording::GetSolverFrameDataAtCycle_AssumesLocked(int32 SolverID, uint64 Cycle)
{
	if (TArray<FChaosVDSolverFrameData>* SolverFramesPtr = RecordedFramesDataPerSolver.Find(SolverID))
	{
		TArray<FChaosVDSolverFrameData>& SolverFrames = *SolverFramesPtr;
		int32 FrameIndex = Algo::BinarySearchBy(SolverFrames, Cycle, &FChaosVDSolverFrameData::FrameCycle);
		return SolverFrames.IsValidIndex(FrameIndex) ? &SolverFrames[FrameIndex] : nullptr;
	}

	return nullptr;
}

int32 FChaosVDRecording::GetLowestSolverFrameNumberAtCycle(int32 SolverID, uint64 Cycle)
{
	FReadScopeLock ReadLock(RecordingDataLock);

	return GetLowestSolverFrameNumberAtCycle_AssumesLocked(SolverID, Cycle);
}

int32 FChaosVDRecording::GetLowestSolverFrameNumberAtCycle_AssumesLocked(int32 SolverID, uint64 Cycle)
{
	if (TArray<FChaosVDSolverFrameData>* SolverFramesPtr = RecordedFramesDataPerSolver.Find(SolverID))
	{
		TArray<FChaosVDSolverFrameData>& SolverFrames = *SolverFramesPtr;
		return Algo::LowerBoundBy(SolverFrames, Cycle, &FChaosVDSolverFrameData::FrameCycle);
	}

	return INDEX_NONE;
}


int32 FChaosVDRecording::GetLowestSolverFrameNumberAtNetworkFrameNumber_AssumesLocked(int32 SolverID, int32 NetworkFrameNumber)
{
	if (TArray<FChaosVDSolverFrameData>* SolverFramesPtr = RecordedFramesDataPerSolver.Find(SolverID))
	{
		TArray<FChaosVDSolverFrameData>& SolverFrames = *SolverFramesPtr;
		
		return Algo::LowerBoundBy(SolverFrames, NetworkFrameNumber, &FChaosVDSolverFrameData::InternalFrameNumber);
	}

	return INDEX_NONE;
}

int32 FChaosVDRecording::FindFirstSolverKeyFrameNumberFromFrame_AssumesLocked(int32 SolverID, int32 StartFrameNumber)
{
	if (TArray<int32>* KeyFrameNumbersPtr = RecordedKeyFramesNumberPerSolver.Find(SolverID))
	{
		TArray<int32>& KeyFrameNumbers = *KeyFrameNumbersPtr;

		const int32 IndexFound = Algo::LowerBound(KeyFrameNumbers, StartFrameNumber);
		
		// If StartFrameNumber is larger than the last keyframe recorded
		// IndexFound will be outside the array's bounds. In that case we want to use the last key frame available
		if (IndexFound >= KeyFrameNumbers.Num())
		{
			return KeyFrameNumbers.Last();
		}

		if (KeyFrameNumbers.IsValidIndex(IndexFound))
		{
			// Frame numbers are not repeated, so the lower bound search should give us the
			// index containing the provided "StartFrameNumber" if it was already a key frame
			const int32 FoundKeyFrame = KeyFrameNumbers[IndexFound];
			if (FoundKeyFrame == StartFrameNumber)
			{
				return FoundKeyFrame;
			}

			// If StartFrameNumber was not a keyframe, we will get the lowest index number containing a key frame number larger than "StartFrameNumber"
			// in which case we want the previous one;
			const int32 PrevKeyFrameIndex = IndexFound - 1;			
			if (KeyFrameNumbers.IsValidIndex(PrevKeyFrameIndex))
			{
				return KeyFrameNumbers[PrevKeyFrameIndex];
			}
		}		
	}

	return INDEX_NONE;
}

int32 FChaosVDRecording::GetLowestSolverFrameNumberGameFrame(int32 SolverID, int32 GameFrame)
{
	FReadScopeLock ReadLock(RecordingDataLock);
	return GetLowestSolverFrameNumberGameFrame_AssumesLocked(SolverID, GameFrame);
}

int32 FChaosVDRecording::GetLowestSolverFrameNumberGameFrame_AssumesLocked(int32 SolverID, int32 GameFrame)
{
	if (!GameFrames.IsValidIndex(GameFrame))
	{
		return INDEX_NONE;
	}

	if (TArray<FChaosVDSolverFrameData>* SolverFramesPtr = RecordedFramesDataPerSolver.Find(SolverID))
	{
		TArray<FChaosVDSolverFrameData>& SolverFrames = *SolverFramesPtr;
		
		return Algo::LowerBoundBy(SolverFrames, GameFrames[GameFrame].FirstCycle, &FChaosVDSolverFrameData::FrameCycle);
	}

	return INDEX_NONE;
}

int32 FChaosVDRecording::GetLowestGameFrameAtSolverFrameNumber(int32 SolverID, int32 SolverFrame)
{
	FReadScopeLock ReadLock(RecordingDataLock);
	return GetLowestGameFrameAtSolverFrameNumber_AssumesLocked(SolverID, SolverFrame);
}

int32 FChaosVDRecording::GetLowestGameFrameAtSolverFrameNumber_AssumesLocked(int32 SolverID, int32 SolverFrame)
{
	if (TArray<FChaosVDSolverFrameData>* SolverFramesPtr = RecordedFramesDataPerSolver.Find(SolverID))
	{
		TArray<FChaosVDSolverFrameData>& SolverFrames = *SolverFramesPtr;

		if (SolverFrames.IsValidIndex(SolverFrame))
		{
			return Algo::LowerBoundBy(GameFrames, SolverFrames[SolverFrame].FrameCycle, &FChaosVDGameFrameData::FirstCycle);
		}	
	}

	return INDEX_NONE;
}


void FChaosVDRecording::AddKeyFrameNumberForSolver(const int32 SolverID, int32 FrameNumber)
{
	FWriteScopeLock WriteLock(RecordingDataLock);
	AddKeyFrameNumberForSolver_AssumesLocked(SolverID, FrameNumber);
}

void FChaosVDRecording::AddKeyFrameNumberForSolver_AssumesLocked(int32 SolverID, int32 FrameNumber)
{
	if (TArray<int32>* KeyFrameNumber = RecordedKeyFramesNumberPerSolver.Find(SolverID))
	{
		KeyFrameNumber->Add(FrameNumber);
	}
	else
	{
		RecordedKeyFramesNumberPerSolver.Add(SolverID, { FrameNumber });
	}
}

void FChaosVDRecording::GenerateAndStoreKeyframeForSolver_AssumesLocked(const int32 SolverID, int32 CurrentFrameNumber, const int32 LastKeyFrameNumber)
{
	FChaosVDSolverFrameData GeneratedKeyFrame;
	CollapseSolverFramesRange_AssumesLocked(SolverID, LastKeyFrameNumber, CurrentFrameNumber, GeneratedKeyFrame);

	// We don't replace an existing delta frame with a generated keyframe because processing keyframes during playback is expensive
	// So we keep the generated keyframes on its own map, so we can access them when needed
	// (usually when we are skipping frames and we need to collapse frame data from the closest keyframe)
	if (TMap<int32, FChaosVDSolverFrameData>* GenerateSolverFrameByNumber = GeneratedKeyFrameDataPerSolver.Find(SolverID))
	{
		GenerateSolverFrameByNumber->Add(CurrentFrameNumber, GeneratedKeyFrame);
	}
	else
	{
		TMap<int32, FChaosVDSolverFrameData> GeneratedSolverFramesByNumber;
		GeneratedSolverFramesByNumber.Add(CurrentFrameNumber, GeneratedKeyFrame);
		GeneratedKeyFrameDataPerSolver.Add(SolverID, GeneratedSolverFramesByNumber);
	}
}

void FChaosVDRecording::AddFrameForSolver(const int32 SolverID, FChaosVDSolverFrameData&& InFrameData)
{
	int32 CurrentFrameNumber;
	bool bIsKeyFrame = InFrameData.bIsKeyFrame;

	{
		FWriteScopeLock WriteLock(RecordingDataLock);

		auto FindLastKeyFrameNumberForSolver = [this](int32 SolverID)-> int32
		{
			if (TArray<int32>* KeyFrameNumber = RecordedKeyFramesNumberPerSolver.Find(SolverID))
			{
				return KeyFrameNumber->Num() ? KeyFrameNumber->Last() : INDEX_NONE;
			}
			return INDEX_NONE;
		};

		if (TArray<FChaosVDSolverFrameData>* SolverFrames = RecordedFramesDataPerSolver.Find(SolverID))
		{
			CurrentFrameNumber = SolverFrames->Num();

			SolverFrames->Add(MoveTemp(InFrameData));

			if (!bIsKeyFrame)
			{
				// If not a keyframe, see if we should generate a to keyframe for the frame number we just added.
				// This greatly reduces the cost during playback when we are skipping more than one frame or going backwards because with more keyframe
				// we have less data to process on the process "Play from last key frame", needed in such situations.
				const int32 LastKeyFrameNumber = FindLastKeyFrameNumberForSolver(SolverID);
				if (LastKeyFrameNumber != INDEX_NONE)
				{
					constexpr int32 MaxDeltaBetweenKeyframes = 5;
					const int32 FrameDiffSinceLastKeyframe = FMath::Abs(CurrentFrameNumber - LastKeyFrameNumber);

					if (FrameDiffSinceLastKeyframe > MaxDeltaBetweenKeyframes)
					{
						GenerateAndStoreKeyframeForSolver_AssumesLocked(SolverID, CurrentFrameNumber, LastKeyFrameNumber);
						AddKeyFrameNumberForSolver_AssumesLocked(SolverID, CurrentFrameNumber);
					}
				}
			}
		}
		else
		{
			CurrentFrameNumber = 0;
			RecordedFramesDataPerSolver.Add(SolverID, {}).Emplace(MoveTemp(InFrameData));

			CommitSolverID_AssumesLocked(SolverID);
		}
	}

	if (bIsKeyFrame)
	{
		AddKeyFrameNumberForSolver(SolverID, CurrentFrameNumber);
	}

	LastUpdatedTimeAsCycle = FPlatformTime::Cycles64();
}

void FChaosVDRecording::AddGameFrameData(const FChaosVDGameFrameData& InFrameData)
{
	FWriteScopeLock WriteLock(RecordingDataLock);
	GameFrames.Add(InFrameData);
}

FChaosVDGameFrameData* FChaosVDRecording::GetGameFrameDataAtCycle_AssumesLocked(uint64 Cycle)
{
	int32 FrameIndex = Algo::BinarySearchBy(GameFrames, Cycle, &FChaosVDGameFrameData::FirstCycle);

	if (FrameIndex != INDEX_NONE)
	{
		return &GameFrames[FrameIndex];
	}

	return nullptr;
}

FChaosVDGameFrameData* FChaosVDRecording::GetGameFrameData_AssumesLocked(int32 FrameNumber)
{
	return GameFrames.IsValidIndex(FrameNumber) ? &GameFrames[FrameNumber] : nullptr;
}

FChaosVDGameFrameData* FChaosVDRecording::GetLastGameFrameData_AssumesLocked()
{
	return GameFrames.Num() > 0 ? &GameFrames.Last() : nullptr;
}

int32 FChaosVDRecording::GetLowestGameFrameNumberAtCycle(uint64 Cycle)
{
	FReadScopeLock ReadLock(RecordingDataLock);
	return GetLowestGameFrameNumberAtCycle_AssumesLocked(Cycle);
	
}

int32 FChaosVDRecording::GetLowestGameFrameNumberAtCycle_AssumesLocked(uint64 Cycle)
{
	return Algo::LowerBoundBy(GameFrames, Cycle, &FChaosVDGameFrameData::FirstCycle);
}

int32 FChaosVDRecording::GetLowestGameFrameNumberAtTime(double Time)
{
	FReadScopeLock ReadLock(RecordingDataLock);
	return Algo::LowerBoundBy(GameFrames, Time, &FChaosVDGameFrameData::StartTime);
}

void FChaosVDRecording::CollapseSolverFramesRange_AssumesLocked(int32 SolverID, int32 StartFrame, int32 EndFrame, FChaosVDSolverFrameData& OutCollapsedFrameData)
{
	// Make sure we start with a clear map
	ParticlesOnCurrentGeneratedKeyframe.Reset();
	
	for (int32 CurrentFrameNumber = StartFrame; CurrentFrameNumber <= EndFrame; CurrentFrameNumber++)
	{
		auto ProcessDestroyedParticles = [](const TSet<int32>& InDestroyedParticleIDs, TMap<int32, TSharedPtr<FChaosVDParticleDataWrapper>>& InOutParticlesMap)
		{
			for (int32 DestroyedPArticleID : InDestroyedParticleIDs)
			{
				InOutParticlesMap.Remove(DestroyedPArticleID);
			}
		};
	
		const bool bRequestingKeyFrameOnly = CurrentFrameNumber == StartFrame;
		if (const FChaosVDSolverFrameData* SolverFrameData = GetSolverFrameData_AssumesLocked(SolverID, CurrentFrameNumber, bRequestingKeyFrameOnly))
		{
			if (SolverFrameData->SolverSteps.Num() > 0)
			{
				for (const FChaosVDFrameStageData& SolverStageData : SolverFrameData->SolverSteps)
				{
					for (const TSharedPtr<FChaosVDParticleDataWrapper>& ParticleData : SolverStageData.RecordedParticlesData)
					{
						if (!ParticleData)
						{
							continue;
						}

						if (TSharedPtr<FChaosVDParticleDataWrapper>* FoundParticleData = ParticlesOnCurrentGeneratedKeyframe.Find(ParticleData->ParticleIndex))
						{
							(*FoundParticleData) = ParticleData;
						}
						else
						{
							ParticlesOnCurrentGeneratedKeyframe.Add(ParticleData->ParticleIndex, ParticleData);
						}

						// Remove any particle data that was remoted in this stage
						ProcessDestroyedParticles(SolverStageData.ParticlesDestroyedIDs, ParticlesOnCurrentGeneratedKeyframe);	
					}
				}
			}

			// Remove any particle data that was remove outside a recorded stage
			ProcessDestroyedParticles(SolverFrameData->ParticlesDestroyedIDs, ParticlesOnCurrentGeneratedKeyframe);

			// If this is the end frame, Copy all the "metadata" for the generated frame, and generate the Solver step
			if (CurrentFrameNumber == EndFrame)
			{
				OutCollapsedFrameData.EndTime = SolverFrameData->EndTime;
				OutCollapsedFrameData.StartTime = SolverFrameData->StartTime;
				OutCollapsedFrameData.FrameCycle = SolverFrameData->FrameCycle;
				OutCollapsedFrameData.bIsKeyFrame = true;
				OutCollapsedFrameData.SolverID = SolverFrameData->SolverID;
				OutCollapsedFrameData.SimulationTransform = SolverFrameData->SimulationTransform;
				OutCollapsedFrameData.DebugFName = SolverFrameData->DebugFName;

				FChaosVDFrameStageData CollapsedStageData;
				CollapsedStageData.StepName = TEXT("Auto Generated Stage");

				// Although this is a generated on load stage, is based on explicit stage data, therefore it needs to be treated as such
				EnumAddFlags(CollapsedStageData.StageFlags, EChaosVDSolverStageFlags::ExplicitStage);

				ParticlesOnCurrentGeneratedKeyframe.GenerateValueArray(CollapsedStageData.RecordedParticlesData);

				ParticlesOnCurrentGeneratedKeyframe.Reset();
				
				OutCollapsedFrameData.SolverSteps.Add(MoveTemp(CollapsedStageData));
			}
		}
	}
}

void FChaosVDRecording::AddImplicitObject(const uint32 ID, const Chaos::FImplicitObjectPtr& InImplicitObject)
{
	FWriteScopeLock WriteLock(RecordingDataLock);
	if (!ImplicitObjects.Contains(ID))
	{
		AddImplicitObject_Internal(ID, Chaos::FConstImplicitObjectPtr(InImplicitObject));
	}
}

bool FChaosVDRecording::IsEmpty() const
{
	FReadScopeLock ReadLock(RecordingDataLock);
	return GetAvailableSolversNumber_AssumesLocked() == 0 && GetAvailableGameFrames_AssumesLocked().Num() == 0 && ImplicitObjects.Num() == 0;
}

void FChaosVDRecording::SetCollisionChannelsInfoContainer(const TSharedPtr<FChaosVDCollisionChannelsInfoContainer>& InCollisionChannelsInfo)
{
	CollisionChannelsInfoContainer = InCollisionChannelsInfo;
}

bool FChaosVDRecording::HasSolverID(int32 SolverID)
{
	FReadScopeLock ReadLock(RecordingDataLock);
	return HasSolverID_AssumesLocked(SolverID);
}

bool FChaosVDRecording::HasSolverID_AssumesLocked(int32 SolverID)
{
	return SolverIDs.Contains(SolverID) || ReservedSolverIDs.Contains(SolverID);
}

void FChaosVDRecording::ReserveSolverID(int32 SolverID)
{
	FWriteScopeLock ReadLock(RecordingDataLock);
	ReserveSolverID_AssumesLocked(SolverID);
}

void FChaosVDRecording::ReserveSolverID_AssumesLocked(int32 SolverID)
{
	ReservedSolverIDs.Emplace(SolverID);
}

void FChaosVDRecording::CommitSolverID(int32 SolverID)
{
	FWriteScopeLock ReadLock(RecordingDataLock);
	CommitSolverID_AssumesLocked(SolverID);
}

void FChaosVDRecording::CommitSolverID_AssumesLocked(int32 SolverID)
{
	SolverIDs.Emplace(SolverID);
	ReservedSolverIDs.Remove(SolverID);
}

void FChaosVDRecording::AddImplicitObject(const uint32 ID, const Chaos::FImplicitObject* InImplicitObject)
{
	FWriteScopeLock WriteLock(RecordingDataLock);
	if (!ImplicitObjects.Contains(ID))
	{
		// Only take ownership after we know we will add it to the map
		const Chaos::FConstImplicitObjectPtr ImplicitObjectPtr(InImplicitObject);
		AddImplicitObject_Internal(ID, ImplicitObjectPtr);
	}
}

void FChaosVDRecording::AddImplicitObject_Internal(uint32 ID, const Chaos::FConstImplicitObjectPtr& InImplicitObject)
{
	ImplicitObjects.Add(ID, InImplicitObject);
	GeometryDataLoaded.Broadcast(InImplicitObject, ID);
}
