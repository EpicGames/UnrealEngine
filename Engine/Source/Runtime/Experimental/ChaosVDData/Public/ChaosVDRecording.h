// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Containers/UnrealString.h"
#include "DataWrappers/ChaosVDCollisionDataWrappers.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StructOnScope.h"
#include "Containers/UnrealString.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ImplicitObject.h"
#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "DataWrappers/ChaosVDCharacterGroundConstraintDataWrappers.h"
#include "DataWrappers/ChaosVDJointDataWrappers.h"
#include "DataWrappers/ChaosVDQueryDataWrappers.h"
#include <atomic>

#include "DataWrappers/ChaosVDAccelerationStructureDataWrappers.h"
#include "DataWrappers/ChaosVDDebugShapeDataWrapper.h"

#include "ChaosVDRecording.generated.h"

#define UE_API CHAOSVDDATA_API

namespace Chaos::VisualDebugger
{
	class FChaosVDSerializableNameTable;
}

DECLARE_MULTICAST_DELEGATE_TwoParams(FChaosVDGeometryDataLoaded, const Chaos::FConstImplicitObjectPtr&, const uint32 GeometryID)

/** Handle to user-defined data types in a CVD recorded frame.
 * These handles share ownership of the data
 */
struct FChaosVDCustomUserDataHandle
{
	/** Creates a handle for the provided data instance. This handle will hold a reference to the data
	 * @param DataPtr Shared ref pointer to the data we want to rpovide access to
	 */
	template <typename DataStructType>
	static FChaosVDCustomUserDataHandle MakeHandle(const TSharedRef<DataStructType>& DataPtr);

	/** Returns the FName of the UStruct type that this handle represents */
	UE_API FName GetTypeName() const;

	/** Returns a raw ptr to the data this handle provides access to.*/
	template <typename DataStructType>
	DataStructType* GetData() const;

	/** Returns a shared ptr to the data this handle provides access to */
	template <typename DataStructType>
	TSharedPtr<DataStructType> GetDataAsShared() const;
private:

	/** Checks if this handle is of another type, using the UStruct data we parsed on creation */
	template <typename DataStructType>
	bool IsA_Internal(const TSharedPtr<FStructOnScope>& InStructOnScope) const;
	
	TSharedPtr<FStructOnScope> DataStruct;
	TSharedPtr<void> DataSharedPtr;
};

template <typename DataStructType>
TSharedPtr<DataStructType> FChaosVDCustomUserDataHandle::GetDataAsShared() const
{
	if (IsA_Internal<DataStructType>(DataStruct))
	{
		return StaticCastSharedPtr<DataStructType>(DataSharedPtr);
	}
	return nullptr;
}

template <typename DataStructType>
FChaosVDCustomUserDataHandle FChaosVDCustomUserDataHandle::MakeHandle(const TSharedRef<DataStructType>& DataPtr)
{
	FChaosVDCustomUserDataHandle NewCustomDataHandle;
	
	NewCustomDataHandle.DataStruct = MakeShared<FStructOnScope>(DataStructType::StaticStruct(), reinterpret_cast<uint8*>(&DataPtr.Get()));
	NewCustomDataHandle.DataSharedPtr = DataPtr;

	return NewCustomDataHandle;
}

template <typename DataStructType>
DataStructType* FChaosVDCustomUserDataHandle::GetData() const
{
	if (IsA_Internal<DataStructType>(DataStruct))
	{
		return reinterpret_cast<DataStructType*>(DataStruct->GetStructMemory());
	}

	return nullptr;
}

template <typename DataStructType>
bool FChaosVDCustomUserDataHandle::IsA_Internal(const TSharedPtr<FStructOnScope>& InStructOnScope) const
{
	if (InStructOnScope)
	{
		const UStruct* HandleStruct = InStructOnScope->GetStruct();
		return HandleStruct && (DataStructType::StaticStruct() == HandleStruct || HandleStruct->IsChildOf(DataStructType::StaticStruct()));
	}

	return false;
}

struct FChaosVDCustomFrameData
{
	UE_API void AddData(const FChaosVDCustomUserDataHandle& InData);

	/** Returns the shared ptr to a custom data instance, if such data type was added to this frame.
	 * If the data type was not loaded, the ptr will be null
	 */
	template<typename DataType>
	TSharedPtr<DataType> GetData() const;

	/** Returns the shared ptr to a custom data instance, if such data type was added to this frame.
	 * If the data type was not loaded, a new instance will be created and automatically added to the frame's data.
	 */
	template<typename DataType>
	TSharedPtr<DataType> GetOrAddDefaultData();

private:
	TMap<FName, FChaosVDCustomUserDataHandle> CustomDataHandlesByType;
};

/** Set of flags used to define characteristics of a loaded solver stage */
enum class EChaosVDSolverStageFlags : uint8
{
	None = 0,
	/** Set if the solver stage is open and can take new data */
	Open = 1 << 0,
	/** Set if the solver stage was explicitly recorded - If not set, this stage was created on the fly during load */
	ExplicitStage = 1 << 1,
};

ENUM_CLASS_FLAGS(EChaosVDSolverStageFlags)

struct FChaosVDFrameStageData
{
	FString StepName;
	TArray<TSharedPtr<FChaosVDParticleDataWrapper>> RecordedParticlesData;
	TArray<TSharedPtr<FChaosVDParticlePairMidPhase>> RecordedMidPhases;
	TArray<TSharedPtr<FChaosVDJointConstraint>> RecordedJointConstraints;
	TArray<FChaosVDConstraint> RecordedConstraints;
	TMap<int32, TArray<FChaosVDConstraint>> RecordedConstraintsByParticleID;
	TMap<int32, TArray<TSharedPtr<FChaosVDParticlePairMidPhase>>> RecordedMidPhasesByParticleID;
	TSet<int32> ParticlesDestroyedIDs;

	// Used for de-duplication during trace analysis
	TMap<int32, int32> CurrentRecordedParticlesIndexes;

	EChaosVDSolverStageFlags StageFlags = EChaosVDSolverStageFlags::None;

	FChaosVDCustomFrameData& GetCustomDataHandler()
	{
		return CustomData;
	}
	
	const FChaosVDCustomFrameData& GetCustomDataHandler() const
	{
		return CustomData;
	}

private:
	FChaosVDCustomFrameData CustomData;
};

UE_DEPRECATED(5.6, "FChaosVDStepData is deprecated and will be removed. Please use FChaosVDFrameStageData instead.")
typedef FChaosVDFrameStageData FChaosVDStepData;

struct FChaosVDTrackedLocation
{
	FString DebugName;
	FVector Location;
};

struct FChaosVDTrackedTransform
{
	FString DebugName;
	FTransform Transform;
};

enum class EChaosVDNetworkSyncDataRequirements
{
	None = 0,
	InternalFrameNumber  = 1 << 0,
	NetworkTickOffset = 1 << 1,

	All = InternalFrameNumber | NetworkTickOffset
};

ENUM_CLASS_FLAGS(EChaosVDNetworkSyncDataRequirements)

UE_DEPRECATED(5.6, "This FChaosVDStepsContainer is deprecated and it will be removed in the future. Use FChaosVDStagesContainer instead.")
typedef TArray<FChaosVDFrameStageData, TInlineAllocator<16>> FChaosVDStepsContainer;

typedef TArray<FChaosVDFrameStageData, TInlineAllocator<16>> FChaosVDFrameStagesContainer;

UENUM()
enum class EChaosVDSolverFrameAttributes : uint16
{
	None = 0,
	HasGTDataToReRoute = 1 << 0,
};
ENUM_CLASS_FLAGS(EChaosVDSolverFrameAttributes)

struct FChaosVDSolverFrameData
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FChaosVDSolverFrameData() = default;
	FChaosVDSolverFrameData(const FChaosVDSolverFrameData& Other) = default;
	FChaosVDSolverFrameData(FChaosVDSolverFrameData&& Other) noexcept = default;
	FChaosVDSolverFrameData& operator=(const FChaosVDSolverFrameData& Other) = default;
	FChaosVDSolverFrameData& operator=(FChaosVDSolverFrameData&& Other) noexcept = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	FName DebugFName;
	UE_DEPRECATED(5.5, "Please use the DebugFName instead")
	FString DebugName;
	int32 SolverID = INDEX_NONE;
	int32 InternalFrameNumber = INDEX_NONE;
	int32 NetworkTickOffset = INDEX_NONE;
	uint64 FrameCycle = 0;
	Chaos::FRigidTransform3 SimulationTransform;
	bool bIsKeyFrame = false;
	bool bIsResimulated = false;
	FChaosVDFrameStagesContainer SolverSteps;
	TSet<int32> ParticlesDestroyedIDs;
	double StartTime = -1.0;
	double EndTime = -1.0;

	TArray<TSharedPtr<FChaosVDCharacterGroundConstraint>> RecordedCharacterGroundConstraints;
	
	/** Calculates and returns the frame time for this recorded frame.
	 * @return Calculated frame time. -1 if it was not recorded
	 */
	double GetFrameTime() const
	{
		if (StartTime < 0 || EndTime < 0)
		{
			return -1.0;
		}

		return EndTime - StartTime;
	}

	/** Returns true if we have the necessary data to sync this frame with other frame based on network ticks offsets */
	bool HasNetworkSyncData(EChaosVDNetworkSyncDataRequirements Requirements = EChaosVDNetworkSyncDataRequirements::All) const
	{
		bool bHasRequiredSyncData = true;
		if (EnumHasAnyFlags(Requirements, EChaosVDNetworkSyncDataRequirements::InternalFrameNumber))
		{
			bHasRequiredSyncData &= InternalFrameNumber != INDEX_NONE;
		}
		
		if (EnumHasAnyFlags(Requirements, EChaosVDNetworkSyncDataRequirements::NetworkTickOffset))
		{
			bHasRequiredSyncData &= NetworkTickOffset != INDEX_NONE;
		}

		return bHasRequiredSyncData;
	}

	/** Returns the current network tick offset. If we didn't have recorded a network tick, we will still return 0 to keep compatibility with other files
	 */
	int32 GetClampedNetworkTickOffset() const
	{
		return NetworkTickOffset >= 0 ? NetworkTickOffset : 0;
	}

	FChaosVDCustomFrameData& GetCustomData()
	{
		return CustomData;
	}
	
	const FChaosVDCustomFrameData& GetCustomData() const
	{
		return CustomData;
	}
	
	EChaosVDSolverFrameAttributes GetAttributes() const
	{
		return FrameAttributes;	
	}
	
	void AddAttributes(EChaosVDSolverFrameAttributes NewAttributes)
	{
		EnumAddFlags(FrameAttributes, NewAttributes);	
	}

	void RemoveAttributes(EChaosVDSolverFrameAttributes AttributesToRemove)
	{
		EnumRemoveFlags(FrameAttributes, AttributesToRemove);	
	}

private:
	FChaosVDCustomFrameData CustomData;
	
	EChaosVDSolverFrameAttributes FrameAttributes = EChaosVDSolverFrameAttributes::None;
};

struct FChaosVDGameFrameData
{

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FChaosVDGameFrameData() = default;
	FChaosVDGameFrameData(const FChaosVDGameFrameData& Other) = default;
	FChaosVDGameFrameData(FChaosVDGameFrameData&& Other) noexcept = default;
	FChaosVDGameFrameData& operator=(const FChaosVDGameFrameData& Other) = default;
	FChaosVDGameFrameData& operator=(FChaosVDGameFrameData&& Other) noexcept = default;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	uint64 FirstCycle;
	uint64 LastCycle;
	double StartTime = -1.0;
	double EndTime = -1.0;

	/** Calculates and returns the frame time for this recorded frame.
	 * @return Calculated frame time. -1 if it was not recorded
	 */
	double GetFrameTime() const
	{
		if (StartTime < 0 || EndTime < 0)
		{
			return -1.0;
		}

		return EndTime - StartTime;
	}

	bool IsDirty() const
	{
		return bIsDirty;
	}

	void MarkDirty()
	{
		bIsDirty = true;
	}

	UE_DEPRECATED(5.6, "RecordedNonSolverLocationsByID is deprecated and will be removed in a future release.")
	TMap<FName, FChaosVDTrackedLocation> RecordedNonSolverLocationsByID;
	UE_DEPRECATED(5.6, "RecordedNonSolverTransformsByID is deprecated and will be removed in a future release.")
	TMap<FName, FChaosVDTrackedTransform> RecordedNonSolverTransformsByID;

	UE_DEPRECATED(5.6, "RecordedDebugDrawBoxesBySolverID is deprecated and will be removed in a future release, use GetCustomDataContainer instead.")
	TMap<int32, TMap<int32, TSharedPtr<FChaosVDQueryDataWrapper>>> RecordedSceneQueriesBySolverID;

	UE_DEPRECATED(5.5, "RecordedSceneQueries is deprecated and will be removed in a future release, use RecordedSceneQueriesByQueryID instead.")
	TMap<int32, TSharedPtr<FChaosVDQueryDataWrapper>> RecordedSceneQueries;

	UE_DEPRECATED(5.6, "RecordedDebugDrawBoxesBySolverID is deprecated and will be removed in a future release, use GetCustomDataContainer instead.")
	TMap<int32, TSharedPtr<FChaosVDQueryDataWrapper>> RecordedSceneQueriesByQueryID;

	UE_DEPRECATED(5.6, "RecordedAABBTreesBySolverID is deprecated and will be removed in a future release, use GetCustomDataContainer instead.")
	TMap<int32, TArray<TSharedPtr<FChaosVDAABBTreeDataWrapper>>> RecordedAABBTreesBySolverID;

	UE_DEPRECATED(5.6, "RecordedDebugDrawBoxesBySolverID is deprecated and will be removed in a future release, use GetCustomDataContainer instead.")
	TMap<int32,TArray<TSharedPtr<FChaosVDDebugDrawBoxDataWrapper>>> RecordedDebugDrawBoxesBySolverID;
	UE_DEPRECATED(5.6, "RecordedDebugDrawLinesBySolverID is deprecated and will be removed in a future release, use GetCustomDataContainer instead.")
	TMap<int32,TArray<TSharedPtr<FChaosVDDebugDrawLineDataWrapper>>> RecordedDebugDrawLinesBySolverID;
	UE_DEPRECATED(5.6, "RecordedDebugDrawSpheresBySolverID is deprecated and will be removed in a future release, use GetCustomDataContainer instead.")
	TMap<int32,TArray<TSharedPtr<FChaosVDDebugDrawSphereDataWrapper>>> RecordedDebugDrawSpheresBySolverID;
	UE_DEPRECATED(5.6, "RecordedDebugDrawImplicitObjectsBySolverID is deprecated and will be removed in a future release, use GetCustomDataContainer instead.")
	TMap<int32,TArray<TSharedPtr<FChaosVDDebugDrawImplicitObjectDataWrapper>>> RecordedDebugDrawImplicitObjectsBySolverID;

	FChaosVDCustomFrameData& GetCustomDataHandler()
	{
		return CustomData;
	}
	
	const FChaosVDCustomFrameData& GetCustomDataHandler() const
	{
		return CustomData;
	}

private:
	FChaosVDCustomFrameData CustomData;
	bool bIsDirty = false;
};

USTRUCT()
struct FChaosVDGameFrameDataWrapper
{
	GENERATED_BODY()

	TSharedPtr<FChaosVDGameFrameData> FrameData;
};

USTRUCT()
struct FChaosVDGameFrameDataWrapperContext
{
	GENERATED_BODY()

	TArray<int32, TInlineAllocator<25>> SupportedSolverIDs;
};

enum class EChaosVDRecordingAttributes : uint8
{
	None = 0,
	/** Set if this recording is being populated from a live session */
	Live = 1 << 0,
	/** Set if this recording contains data from multiple recordings */
	Merged = 1 << 1,
};
ENUM_CLASS_FLAGS(EChaosVDRecordingAttributes)

/**
 * Struct that represents a recorded Physics simulation.
 * It is currently populated while analyzing a Trace session
 */
struct FChaosVDRecording
{
	UE_API FChaosVDRecording();

	/* Constant used to define inline allocators -
	* Unless there are some scenarios with a lot of RBAN solvers in the recording, we usually don't go over 3 tracks most of the time so 16 should be plenty by default */
	static constexpr int32 CommonTrackCount = 16;

	/** Returns the current available recorded solvers number */
	int32 GetAvailableSolversNumber_AssumesLocked() const
	{
		return RecordedFramesDataPerSolver.Num();
	}
	
	/** Returns the current available Game Frames */
	UE_API int32 GetAvailableGameFramesNumber() const;
	UE_API int32 GetAvailableGameFramesNumber_AssumesLocked() const;

	/** Returns a reference to the array holding all the available game frames */
	const TArray<FChaosVDGameFrameData>& GetAvailableGameFrames_AssumesLocked() const
	{
		return GameFrames;
	}

	/** Returns a reference to the map containing the available solver data */
	const TMap<int32, TArray<FChaosVDSolverFrameData>>& GetAvailableSolvers_AssumesLocked() const
	{
		return RecordedFramesDataPerSolver;
	}

	/**
	 * Returns the number of available frame data for the specified solver ID
	 * @param SolverID ID of the solver 
	 */
	UE_API int32 GetAvailableSolverFramesNumber(int32 SolverID) const;
	UE_API int32 GetAvailableSolverFramesNumber_AssumesLocked(int32 SolverID) const;
	
	/**
	 * Returns the name of the specified solver id
	 * @param SolverID ID of the solver 
	 */
	UE_API FName GetSolverFName(int32 SolverID);

	UE_DEPRECATED(5.6, "Please use the GetSolverFName instead")
	FString GetSolverName(int32 SolverID)
	{
		return TEXT("");
	}

	/**
	 * Returns the name of the specified solver id. Must be called from within a ReadLock
	 * @param SolverID ID of the solver
	 */
	UE_API FName GetSolverFName_AssumedLocked(int32 SolverID);

	UE_API bool IsServerSolver_AssumesLocked(int32 SolverID);
	UE_API bool IsServerSolver(int32 SolverID);

	UE_DEPRECATED(5.6, "Please use the GetSolverFName_AssumedLocked instead")
	FString GetSolverName_AssumedLocked(int32 SolverID)
	{
		return TEXT("");
	}

	/**
	 * Return a ptr to the existing solver frame data from the specified ID and Frame number
	 * @param SolverID ID of the solver
	 * @param FrameNumber Frame number
	 * @param bKeyFrameOnly True if we should return a keyframe (real or generated) for the provided frame number if available or nothing
	 * @return Ptr to the existing solver frame data from the specified ID and Frame number - It is a ptr to the array element, Do not store
	 */
	UE_API FChaosVDSolverFrameData* GetSolverFrameData_AssumesLocked(int32 SolverID, int32 FrameNumber, bool bKeyFrameOnly = false);
	
	/**
	 * Return a ptr to the existing solver frame data from the specified ID and Frame number
	 * @param SolverID ID if the solver
	 * @param Cycle Platform cycle at which the solver frame was recorded
	 * @return Ptr to the existing solver frame data from the specified ID and Frame number - It is a ptr to the array element, Do not store
	 */
	UE_API FChaosVDSolverFrameData* GetSolverFrameDataAtCycle_AssumesLocked(int32 SolverID, uint64 Cycle);

	/**
	 * Searches and returns the lowest frame number of a solver at the specified cycle
	 * @param SolverID ID if the solver
	 * @param Cycle Platform cycle to use as lower bound
	 * @return Found frame number. INDEX_NONE if no frame is found for the specified cycle
	 */
	UE_API int32 GetLowestSolverFrameNumberAtCycle(int32 SolverID, uint64 Cycle);
	UE_API int32 GetLowestSolverFrameNumberAtCycle_AssumesLocked(int32 SolverID, uint64 Cycle);

	UE_API int32 GetLowestSolverFrameNumberAtNetworkFrameNumber_AssumesLocked(int32 SolverID, int32 NetworkFrameNumber);

	UE_API int32 FindFirstSolverKeyFrameNumberFromFrame_AssumesLocked(int32 SolverID, int32 StartFrameNumber);
	
	/**
	 * Searches and returns the lowest frame number of a solver at the specified cycle
	 * @param SolverID ID if the solver
	 * @param GameFrame Platform cycle to use as lower bound
	 * @return Found frame number. INDEX_NONE if no frame is found for the specified cycle
	 */
	UE_API int32 GetLowestSolverFrameNumberGameFrame(int32 SolverID, int32 GameFrame);
	UE_API int32 GetLowestSolverFrameNumberGameFrame_AssumesLocked(int32 SolverID, int32 GameFrame);
	
	/**
	 * Searches and returns the lowest game frame number at the specified solver frame
	 * @param SolverID ID if the solver to evaluate
	 * @param SolverFrame Frame number of the solver to evaluate
	 * @return Found Game frame number. INDEX_NONE if no frame is found for the specified cycle
	 */
	UE_API int32 GetLowestGameFrameAtSolverFrameNumber(int32 SolverID, int32 SolverFrame);
	UE_API int32 GetLowestGameFrameAtSolverFrameNumber_AssumesLocked(int32 SolverID, int32 SolverFrame);

	/**
	 * Adds a Solver Frame Data entry for a specific Solver ID. Creates a solver entry if it does not exist 
	 * @param SolverID ID of the solver to add
	 * @param InFrameData Reference to the frame data we want to add
	 */
	UE_API void AddFrameForSolver(const int32 SolverID, FChaosVDSolverFrameData&& InFrameData);

	/**
	 * Adds a Game Frame Data entry. Creates a solver entry if it does not exist 
	 * @param InFrameData Reference to the frame data we want to add
	 */
	UE_API void AddGameFrameData(const FChaosVDGameFrameData& InFrameData);

	/** Called each time new geometry data becomes available in the recording - Mainly when a new frame is added from the Trace analysis */
	FChaosVDGeometryDataLoaded& OnGeometryDataLoaded()
	{
		return GeometryDataLoaded;
	}

	/**
	 * Searches for a recorded Game frame at the specified cycle 
	 * @param Cycle Platform Cycle to be used in the search
	 * @return A ptr to the recorded game frame data - This is a ptr to the array element. Do not store
	 */
	UE_API FChaosVDGameFrameData* GetGameFrameDataAtCycle_AssumesLocked(uint64 Cycle);

	/**
	 * Searches for a recorded Game frame at the specified cycle 
	 * @param FrameNumber Frame Number
	 * @return A ptr to the recorded game frame data - This is a ptr to the array element. Do not store
	 */
	UE_API FChaosVDGameFrameData* GetGameFrameData_AssumesLocked(int32 FrameNumber);

	/** Returns a ptr to the last recorded game frame - This is a ptr to the array element. Do not store */
	UE_API FChaosVDGameFrameData* GetLastGameFrameData_AssumesLocked();

	/**
	 * Searches and returns the lowest game frame number at the specified cycle
	 * @param Cycle Platform Cycle to be used in the search as lower bound
	 * @return Found Game frame number. INDEX_NONE if no frame is found for the specified cycle
	 */
	UE_API int32 GetLowestGameFrameNumberAtCycle(uint64 Cycle);
	UE_API int32 GetLowestGameFrameNumberAtCycle_AssumesLocked(uint64 Cycle);

	/**
	 * Searches and returns the lowest game frame number at the specified cycle
	 * @param Time Platform Time to be used in the search as lower bound
	 * @return Found Game frame number. INDEX_NONE if no frame is found for the specified cycle
	 */
	UE_API int32 GetLowestGameFrameNumberAtTime(double Time);

	/**
     * Gathers all available solvers IDs at the given Game frame number
     * @param FrameNumber Game Frame number to evaluate
     * @param OutSolversID Solver's ID array to be filled with any IDs found
     */
	template<typename TAllocator>
	void GetAvailableSolverIDsAtGameFrameNumber(int32 FrameNumber,TArray<int32, TAllocator>& OutSolversID);
	template<typename TAllocator>
	void GetAvailableSolverIDsAtGameFrameNumber_AssumesLocked(int32 FrameNumber, TArray<int32, TAllocator>& OutSolversID);
	template<typename TAllocator>
	void GetAvailableSolverIDsAtGameFrame(const FChaosVDGameFrameData& GameFrameData, TArray<int32, TAllocator>& OutSolversID);
	template<typename TAllocator>
	void GetAvailableSolverIDsAtGameFrame_AssumesLocked(const FChaosVDGameFrameData& GameFrameData, TArray<int32, TAllocator>& OutSolversID);

	/** Collapses the most important frame data from a range of solver frames into a single solver frame data */
	UE_API void CollapseSolverFramesRange_AssumesLocked(int32 SolverID, int32 StartFrame, int32 EndFrame, FChaosVDSolverFrameData& OutCollapsedFrameData);

	/** Returns a reference to the GeometryID-ImplicitObject map of this recording */
	const TMap<uint32, Chaos::FConstImplicitObjectPtr>& GetGeometryMap() const
	{
		return ImplicitObjects;
	}

	UE_DEPRECATED(5.4, "Please use GetGeometryMap instead")
	const TMap<uint32, TSharedPtr<const Chaos::FImplicitObject>>& GetGeometryDataMap() const
	{
		check(false);
		static TMap<uint32, TSharedPtr<const Chaos::FImplicitObject>> DummyMap;
		return DummyMap;
	}

	/** Adds a shared Implicit Object to the recording */
	UE_API void AddImplicitObject(const uint32 ID, const Chaos::FImplicitObjectPtr& InImplicitObject);
	
	UE_DEPRECATED(5.4, "Please use AddImplicitObject with FImplicitObjectPtr instead")
	UE_API void AddImplicitObject(const uint32 ID, const TSharedPtr<Chaos::FImplicitObject>& InImplicitObject);

	/** Session name of the trace session used to re-build this recording */
	FString SessionName;

	FRWLock& GetRecordingDataLock()
	{
		return RecordingDataLock;
	}

	/** Returns true if this recording is being populated from a live session */
	bool IsLive() const
	{
		return EnumHasAnyFlags(RecordingAttributes, EChaosVDRecordingAttributes::Live);
	}

	/** Sets if this recording is being populated from a live session */
	void SetIsLive(bool bNewIsLive)
	{
		if (bNewIsLive)
		{
			EnumAddFlags(RecordingAttributes, EChaosVDRecordingAttributes::Live);
		}
		else
		{
			EnumRemoveFlags(RecordingAttributes, EChaosVDRecordingAttributes::Live);
		}	
	}

	void AddAttributes(EChaosVDRecordingAttributes Attributes)
	{
		FWriteScopeLock WriteLock(RecordingDataLock);
		EnumAddFlags(RecordingAttributes, Attributes);
	}

	void RemoveAttributes(EChaosVDRecordingAttributes Attributes)
	{
		FWriteScopeLock WriteLock(RecordingDataLock);
		EnumAddFlags(RecordingAttributes, Attributes);
	}

	EChaosVDRecordingAttributes GetAttributes() const
	{
		FReadScopeLock ReadLock(RecordingDataLock);
		return GetAttributes_AssumesLocked();
	}

	EChaosVDRecordingAttributes GetAttributes_AssumesLocked() const
	{
		return RecordingAttributes;
	}

	/** Returns true if this recording does not have any usable data */
	UE_API bool IsEmpty() const;

	/** Returns the last Platform Cycle on which this recording was updated (A new frame was added) */
	uint64 GetLastUpdatedTimeAsCycle()
	{
		return LastUpdatedTimeAsCycle;
	}

	TSharedPtr<FChaosVDCollisionChannelsInfoContainer> GetCollisionChannelsInfoContainer()
	{
		return CollisionChannelsInfoContainer;
	}
	
	UE_API void SetCollisionChannelsInfoContainer(const TSharedPtr<FChaosVDCollisionChannelsInfoContainer>& InCollisionChannelsInfo);

	UE_API bool HasSolverID(int32 SolverID);
	UE_API bool HasSolverID_AssumesLocked(int32 SolverID);

	UE_API void ReserveSolverID(int32 SolverID);
	UE_API void ReserveSolverID_AssumesLocked(int32 SolverID);

	UE_API void CommitSolverID(int32 SolverID);
	UE_API void CommitSolverID_AssumesLocked(int32 SolverID);

	int32 GetAvailableTrackIDForRemapping()
	{
		return AvailableTrackIDForRemapping++;
	}

	FChaosVDCustomFrameData& GetCustomDataHandler()
	{
		return CustomData;
	}
	
	const FChaosVDCustomFrameData& GetCustomDataHandler() const
	{
		return CustomData;
	}

private:
	FChaosVDCustomFrameData CustomData;

protected:

	/** Adds an Implicit Object to the recording and takes ownership of it */
	UE_API void AddImplicitObject(const uint32 ID, const Chaos::FImplicitObject* InImplicitObject);
	
	UE_API void AddImplicitObject_Internal(const uint32 ID, const Chaos::FConstImplicitObjectPtr& InImplicitObject);

	/** Stores a frame number of a solver that is a Key Frame -
	 * These are used when scrubbing to make sure the visualization is in sync with what was recorded
	 */
	UE_API void AddKeyFrameNumberForSolver(int32 SolverID, int32 FrameNumber);
	UE_API void AddKeyFrameNumberForSolver_AssumesLocked(int32 SolverID, int32 FrameNumber);
	UE_API void GenerateAndStoreKeyframeForSolver_AssumesLocked(int32 SolverID, int32 CurrentFrameNumber, int32 LastKeyFrameNumber);

	TMap<int32, TArray<FChaosVDSolverFrameData>> RecordedFramesDataPerSolver;
	TMap<int32, TMap<int32, FChaosVDSolverFrameData>> GeneratedKeyFrameDataPerSolver;
	TMap<int32, TArray<int32>> RecordedKeyFramesNumberPerSolver;
	TArray<FChaosVDGameFrameData> GameFrames;

	FChaosVDGeometryDataLoaded GeometryDataLoaded;

	/** Id to Ptr map of all shared geometry data required to visualize */
	TMap<uint32, Chaos::FConstImplicitObjectPtr> ImplicitObjects;

	mutable FRWLock RecordingDataLock;

	EChaosVDRecordingAttributes RecordingAttributes = EChaosVDRecordingAttributes::None; 

	/** Last Platform Cycle on which this recording was updated */
	std::atomic<uint64> LastUpdatedTimeAsCycle;

	/** Map that temporary holds generated particle data during the key frame generation process, keeping its memory allocation between generated frames*/
	TMap<int32, TSharedPtr<FChaosVDParticleDataWrapper>> ParticlesOnCurrentGeneratedKeyframe;

	TSharedPtr<FChaosVDCollisionChannelsInfoContainer> CollisionChannelsInfoContainer;

	TSet<int32> ReservedSolverIDs;
	TSet<int32> SolverIDs;

	std::atomic<int32> AvailableTrackIDForRemapping = 1;

	friend class FChaosVDTraceProvider;
	friend class FChaosVDTraceImplicitObjectProcessor;
};

template <typename DataType>
TSharedPtr<DataType> FChaosVDCustomFrameData::GetData() const
{
	if (const FChaosVDCustomUserDataHandle* FoundDataHandle = CustomDataHandlesByType.Find(DataType::StaticStruct()->GetFName()))
	{
		return FoundDataHandle->GetDataAsShared<DataType>();
	}

	return nullptr;
}

template <typename DataType>
TSharedPtr<DataType> FChaosVDCustomFrameData::GetOrAddDefaultData()
{
	TSharedPtr<DataType> DataSharedPtr = GetData<DataType>();
	if (!DataSharedPtr)
	{
		TSharedPtr<DataType> CustomData = nullptr;

		CustomData = MakeShared<DataType>();

		AddData(FChaosVDCustomUserDataHandle::MakeHandle<DataType>(CustomData.ToSharedRef()));

		return CustomData;
	}

	return DataSharedPtr;
}

template <typename TAllocator>
void FChaosVDRecording::GetAvailableSolverIDsAtGameFrameNumber(int32 FrameNumber, TArray<int32, TAllocator>& OutSolversID)
{
	FReadScopeLock ReadLock(RecordingDataLock);
	return GetAvailableSolverIDsAtGameFrameNumber_AssumesLocked(FrameNumber, OutSolversID);
}

template <typename TAllocator>
void FChaosVDRecording::GetAvailableSolverIDsAtGameFrameNumber_AssumesLocked(int32 FrameNumber, TArray<int32, TAllocator>& OutSolversID)
{
	if (!GameFrames.IsValidIndex(FrameNumber))
	{
		return;
	}
	
	GetAvailableSolverIDsAtGameFrame_AssumesLocked(GameFrames[FrameNumber], OutSolversID);
}

template <typename TAllocator>
void FChaosVDRecording::GetAvailableSolverIDsAtGameFrame(const FChaosVDGameFrameData& GameFrameData, TArray<int32, TAllocator>& OutSolversID)
{
	FReadScopeLock ReadLock(RecordingDataLock);
	GetAvailableSolverIDsAtGameFrame_AssumesLocked(GameFrameData, OutSolversID);
}

template <typename TAllocator>
void FChaosVDRecording::GetAvailableSolverIDsAtGameFrame_AssumesLocked(const FChaosVDGameFrameData& GameFrameData, TArray<int32, TAllocator>& OutSolversID)
{
	OutSolversID.Reserve(RecordedFramesDataPerSolver.Num());

	for (const TPair<int32, TArray<FChaosVDSolverFrameData>>& SolverFramesWithIDPair : RecordedFramesDataPerSolver)
	{
		if (SolverFramesWithIDPair.Value.IsEmpty())
		{
			continue;
		}

		if (SolverFramesWithIDPair.Value.Num() == 1 && SolverFramesWithIDPair.Value[0].FrameCycle < GameFrameData.FirstCycle)
		{
			OutSolversID.Add(SolverFramesWithIDPair.Key);
		}
		else
		{
			if (GameFrameData.FirstCycle > SolverFramesWithIDPair.Value[0].FrameCycle && GameFrameData.FirstCycle < SolverFramesWithIDPair.Value.Last().FrameCycle)
			{
				OutSolversID.Add(SolverFramesWithIDPair.Key);
			}
		}
	}
}

#undef UE_API
