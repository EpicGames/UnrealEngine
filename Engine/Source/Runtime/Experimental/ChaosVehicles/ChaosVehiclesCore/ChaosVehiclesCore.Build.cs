// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
    public class ChaosVehiclesCore : ModuleRules
    {
        public ChaosVehiclesCore(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
                new string[] {
                "Core",
                "CoreUObject",
				"Chaos"
				}
            ); 

			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
		}
    }
}
