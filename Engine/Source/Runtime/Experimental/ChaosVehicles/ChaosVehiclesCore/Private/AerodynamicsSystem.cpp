// Copyright Epic Games, Inc. All Rights Reserved.

#include "AerodynamicsSystem.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{

	FSimpleAerodynamicsSim::FSimpleAerodynamicsSim(const FSimpleAerodynamicsConfig* SetupIn) : TVehicleSystem< FSimpleAerodynamicsConfig>(SetupIn)
		, DownforceCoefficient(SetupIn->DownforceCoefficient)
		, DragCoefficient(SetupIn->DragCoefficient)
		, DensityOfMedium(RealWorldConsts::AirDensity())
	{
		// pre-calculate static values
		EffectiveDragConstant = 0.5f * Setup().AreaMetresSquared * DragCoefficient;
		EffectiveLiftConstant = 0.5f * Setup().AreaMetresSquared * DownforceCoefficient;
	}

	FVector FSimpleAerodynamicsSim::GetCombinedForces(float VelocityIn)
	{
		// aerodynamic forces applied in opposite direction to velocity
		float Sign = (VelocityIn < 0.0f) ? 1.0f : -1.0f;
		float CommonSum = DensityOfMedium * VelocityIn * VelocityIn * Sign;
		FVector CombinedForces(EffectiveDragConstant * CommonSum, 0.f, EffectiveLiftConstant * CommonSum);
		return CombinedForces;
	}

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
