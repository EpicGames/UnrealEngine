// Copyright Epic Games, Inc. All Rights Reserved.

#include "AerofoilSystem.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{

	FAerofoil::FAerofoil(const FAerofoilConfig* SetupIn) : TVehicleSystem<FAerofoilConfig>(SetupIn)
		, CurrentAirDensity(RealWorldConsts::AirDensity())
		, AngleOfAttack(0.f)
		, ControlSurfaceAngle(0.f)
		, AirflowNormal(FVector::ZeroVector)
		, AerofoilId(0)
	{

	}

	float FAerofoil::CalcAngleOfAttackDegrees(const FVector& UpAxis, const FVector& InAirflowVector)
	{
		float fMag = FVector::DotProduct(UpAxis, InAirflowVector);
		return RadToDeg(FMath::Asin(fMag));
	}

	float FAerofoil::CalcLiftCoefficient(float InAngleOfAttack, float InControlSurfaceAngle)
	{
		float PeakValue = 2.0f; // typically the Coefficient can reach this peak value
		float TotalAngle = InAngleOfAttack + InControlSurfaceAngle;

		if (FMath::Abs(TotalAngle) > (Setup().StallAngle * 2.0f))
		{
			return 0.0f;
		}

		return FMath::Sin(TotalAngle * (PI * 0.5f) / Setup().StallAngle) * PeakValue;
	}

	float FAerofoil::CalcDragCoefficient(float InAngleOfAttack, float InControlSurfaceAngle)
	{
		if (InAngleOfAttack > 90.f)
		{
			InAngleOfAttack = 180.f - InAngleOfAttack;
		}

		if (InAngleOfAttack < -90.f)
		{
			InAngleOfAttack = -180.f - InAngleOfAttack;
		}

		float Value = (InAngleOfAttack + InControlSurfaceAngle) / (Setup().StallAngle + FMath::Abs(InControlSurfaceAngle));
		return (0.05f + Value * Value);
	}

	float FAerofoil::CalcCentreOfLift()
	{
		// moves backwards past stall angle
		if (AngleOfAttack > Setup().StallAngle)
		{
			return (AngleOfAttack - Setup().StallAngle) * 10.0f + 20.0f;
		}

		// moves forwards below stall angle
		return (Setup().StallAngle - AngleOfAttack) * 20.0f / Setup().StallAngle + 20.0f;
	}

	float FAerofoil::CalcDynamicPressure(float VelocitySqr, float InAltitude)
	{
		float AltitudeMultiplierEffect = 1.0f;

		//if (InAltitude > Setup().MaxCeiling)
		//{
		//	AltitudeMultiplierEffect = 0.4f;
		//}
		//else if (InAltitude > Setup().MinCeiling)
		//{
		//	AltitudeMultiplierEffect = (Setup().MaxCeiling - InAltitude) / (Setup().MaxCeiling - Setup().MinCeiling);
		//}

		//FMath::Clamp(AltitudeMultiplierEffect, Setup().MinCeiling, Setup().MaxCeiling);

		return AltitudeMultiplierEffect * 0.5f * CurrentAirDensity * VelocitySqr;
	}

	FVector FAerofoil::GetCenterOfLiftOffset()
	{
		float X = 0.0f;

		if (Setup().Type == EAerofoilType::Wing)
		{
			X = (CalcCentreOfLift() - 50.0f) / 100.0f;
		}

		return Setup().Offset + FVector(X, 0.0f, 0.0f);
	}

	FVector FAerofoil::GetForce(FTransform& BodyTransform, const FVector& v, float Altitude, float DeltaTime)
	{
		FVector Force(0.0f, 0.0f, 0.0f);

		float AirflowMagnitudeSqr = v.SizeSquared();

		// can only generate lift if there is airflow over aerofoil, early out
		if (FMath::Abs(AirflowMagnitudeSqr) < SMALL_NUMBER)
		{
			return Force;
		}

		// airflow direction in opposite direction to vehicle direction of travel
		AirflowNormal = -v;
		AirflowNormal.Normalize();

		// determine angle of attack for control surface
		AngleOfAttack = CalcAngleOfAttackDegrees(Setup().UpAxis, AirflowNormal);

		// Aerofoil Camber and Control Surface just lumped together
		float TotalControlAngle = ControlSurfaceAngle + Setup().Camber;

		// dynamic pressure dependent on speed, altitude (air pressure)
		float Common = Setup().Area * CalcDynamicPressure(AirflowMagnitudeSqr, Altitude);

		// Lift and Drag coefficients are based on the angle of attack and Control Angle
		float LiftCoef = CalcLiftCoefficient(AngleOfAttack, TotalControlAngle) * Setup().LiftMultiplier;
		float DragCoef = CalcDragCoefficient(AngleOfAttack, TotalControlAngle) * Setup().DragMultiplier;

		// Combine to create a single force vector
		Force = Setup().UpAxis * (Common * LiftCoef) + AirflowNormal * (Common * DragCoef);

		return Force;
	}

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
