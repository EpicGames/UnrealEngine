// Copyright Epic Games, Inc. All Rights Reserved.

#include "ArcadeSystem.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{

	FTorqueControlSim::FTorqueControlSim(const FTorqueControlConfig* SetupIn) 
		: TVehicleSystem<FTorqueControlConfig>(SetupIn)
	{

	}

	FTargetRotationControlSim::FTargetRotationControlSim(const FTargetRotationControlConfig* SetupIn) 
		: TVehicleSystem<FTargetRotationControlConfig>(SetupIn)
	{

	}

	FStabilizeControlSim::FStabilizeControlSim(const FStabilizeControlConfig* SetupIn) 
		: TVehicleSystem<FStabilizeControlConfig>(SetupIn)
	{

	}

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
