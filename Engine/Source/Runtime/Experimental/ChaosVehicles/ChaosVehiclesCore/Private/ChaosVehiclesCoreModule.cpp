// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVehiclesCoreModule.h"
#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FChaosVehiclesCoreModule, ChaosVehiclesCore);
