// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineSystem.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{


	FSimpleEngineSim::FSimpleEngineSim(const FSimpleEngineConfig* StaticDataIn) : TVehicleSystem<FSimpleEngineConfig>(StaticDataIn)
		, MaxTorque(Setup().MaxTorque)
		, ThrottlePosition(0.f)
		, TargetSpeed(0.f)
		, CurrentRPM(0.f)
		, DriveTorque(0.f)
		, EngineIdleSpeed(RPMToOmega(Setup().EngineIdleRPM))
		, MaxEngineSpeed(RPMToOmega(Setup().MaxRPM))
		, EngineStarted(true)
		, FreeRunning(false)
		, Omega(0.f)
		, RevRate(0.)
	{

	}

	float FSimpleEngineSim::GetTorqueFromRPM(float RPM, bool LimitToIdle /*= true*/)
	{
		if (!EngineStarted || (FMath::Abs(RPM - Setup().MaxRPM) < 1.0f) || Setup().MaxRPM == 0)
		{
			return 0.f;
		}

		if (LimitToIdle)
		{
			RPM = FMath::Clamp(RPM, (float)Setup().EngineIdleRPM, (float)Setup().MaxRPM);
		}

		return Setup().TorqueCurve.GetValue(RPM, Setup().MaxRPM, MaxTorque);
	}

	void FSimpleEngineSim::Simulate(float DeltaTime)
	{
		if (!EngineStarted)
		{
			return;
		}

		if (FreeRunning)
		{
			float PrevOmega = Omega;
			Omega += GetEngineTorque() * DeltaTime / Setup().EngineRevUpMOI;

			float Decel = Setup().EngineRevDownRate * Sqr((Omega - 0.5f*EngineIdleSpeed) / MaxEngineSpeed);
			Omega -= Decel * DeltaTime;

			RevRate = (Omega - PrevOmega) / DeltaTime;
		}
		else
		{
			float PrevOmega = Omega;
			Omega += (TargetSpeed - Omega) * 4.0f * DeltaTime;// / Setup().EngineRevUpMOI;
			RevRate = (Omega - PrevOmega) / DeltaTime;
		}

		// we don't let the engine stall
		if (Omega < EngineIdleSpeed)
		{
			Omega = EngineIdleSpeed;
		}	
		
		// EngineSpeed == Omega
		CurrentRPM = OmegaToRPM(Omega);
	}


} // namespace Chaos


#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
