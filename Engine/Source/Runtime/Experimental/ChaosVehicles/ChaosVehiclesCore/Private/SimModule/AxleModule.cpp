// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/AxleModule.h"
#include "SimModule/SimModuleTree.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{

	FAxleSimModule::FAxleSimModule(const FAxleSettings& Settings)
		: TSimModuleSettings<FAxleSettings>(Settings)
	{

	}

	void FAxleSimModule::Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem)
	{
		float BrakeTorque = 0.0f;
		TransmitTorque(VehicleModuleSystem, DriveTorque, BrakeTorque);
		IntegrateAngularVelocity(DeltaTime, Setup().AxleInertia);
	}


} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
