// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/ChassisModule.h"
#include "SimModule/SimModuleTree.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{

	FChassisSimModule::FChassisSimModule(const FChassisSettings& Settings)
		: TSimModuleSettings<FChassisSettings>(Settings)
	{

	}

	void FChassisSimModule::Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem)
	{
		FVector VelocityMs = Chaos::CmToM(LocalLinearVelocity);
		float DragForceMagnitude = -VelocityMs.SizeSquared() * Setup().DensityOfMedium * 0.5f * Setup().AreaMetresSquared * Setup().DragCoefficient * Chaos::MToCmScaling();
		FVector ForceVector = LocalLinearVelocity.GetSafeNormal() * DragForceMagnitude;
		ForceVector.X *= Setup().XAxisMultiplier;
		ForceVector.Y *= Setup().YAxisMultiplier;
		AddLocalForce(ForceVector);

		float Value = Setup().AngularDamping * Chaos::MToCmScaling();
		FVector Torque = LocalAngularVelocity * -Value;
		AddLocalTorque(Torque);
	}


} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
