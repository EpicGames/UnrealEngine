// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/ClutchModule.h"
#include "SimModule/SimModuleTree.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION_SHIP
#endif

namespace Chaos
{
	FClutchSimModule::FClutchSimModule(const FClutchSettings& Settings) : TSimModuleSettings<FClutchSettings>(Settings)
		, ClutchValue(0.f)
	{
	}

	void FClutchSimModule::Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem)
	{
		//FTorqueSimModule* Parent = static_cast<FTorqueSimModule*>(GetParent());
		//FTorqueSimModule* Child = static_cast<FTorqueSimModule*>(GetFirstChild());

		//float EngineSpeed = Parent->GetAngularVelocity();
		//float TransmissionSpeed = Child->GetAngularVelocity();

		//// difference in speed between the two plates
		//float AngularVelocityDifference = EngineSpeed - TransmissionSpeed;

		// Inputs.Clutch 0 is engaged/locked, 1 is depressed/open
		ClutchValue = (1.0f - Inputs.GetControls().GetMagnitude(ClutchControlName)) * Setup().ClutchStrength;

		FTorqueSimModule* Parent = static_cast<FTorqueSimModule*>(GetParent());
		FTorqueSimModule* Child = static_cast<FTorqueSimModule*>(GetFirstChild());

		if (Parent && Child)
		{
			float EngineSpeed = Parent->GetAngularVelocity();
			float TransmissionSpeed = Child->GetAngularVelocity();

			// difference in speed between the two plates
			float AngularVelocityDifference = EngineSpeed - TransmissionSpeed;

			Parent->AddAngularVelocity(-AngularVelocityDifference * 0.1f);
			Child->AddAngularVelocity(AngularVelocityDifference * 0.1f);

			float BrakeTorque = 0.0f;
			TransmitTorque(VehicleModuleSystem, DriveTorque, BrakeTorque, 1.0f, ClutchValue);
		}
	}

	bool FClutchSimModule::GetDebugString(FString& StringOut) const
	{
		FTorqueSimModule::GetDebugString(StringOut);
		StringOut += FString::Format(TEXT("Value {0}")
			, { ClutchValue });
		return true;

	}


} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION_SHIP
#endif
