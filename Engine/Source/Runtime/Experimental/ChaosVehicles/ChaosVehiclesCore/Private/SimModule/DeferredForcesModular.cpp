// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/DeferredForcesModular.h"
#include "SimModule/SimulationModuleBase.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "Chaos/Utilities.h"
#include "Chaos/ParticleHandle.h"

#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"

#if CHAOS_DEBUG_DRAW
#include "Chaos/DebugDrawQueue.h"
#endif

FCoreModularVehicleDebugParams GCoreModularVehicleDebugParams;

#if CHAOS_DEBUG_DRAW
FAutoConsoleVariableRef CVarChaosModularVehiclesShowMass(TEXT("p.ModularVehicle.ShowMass"), GCoreModularVehicleDebugParams.ShowMass, TEXT("Show mass and inertia above objects."));
FAutoConsoleVariableRef CVarChaosModularVehiclesShowForces(TEXT("p.ModularVehicle.ShowForces"), GCoreModularVehicleDebugParams.ShowForces, TEXT("Show forces on modular vehicles."));
FAutoConsoleVariableRef CVarChaosModularVehiclesDrawForceScaling(TEXT("p.ModularVehicle.DrawForceScaling"), GCoreModularVehicleDebugParams.DrawForceScaling, TEXT("Set scaling of modular vehicle debug draw forces."));
FAutoConsoleVariableRef CVarChaosModularVehiclesDisableForces(TEXT("p.ModularVehicle.DisableForces"), GCoreModularVehicleDebugParams.DisableForces, TEXT("Disable all forces."));
#endif

Chaos::FPBDRigidParticleHandle* FDeferredForcesModular::GetParticleFromUniqueIndex(int32 ParticleUniqueIdx, const TArray<Chaos::FPBDRigidParticleHandle*>& Particles) const
{
	for (Chaos::FPBDRigidParticleHandle* Particle : Particles)
	{
		if (Particle && Particle->UniqueIdx().IsValid())
		{
			if (ParticleUniqueIdx == Particle->UniqueIdx().Idx)
			{
				return Particle;
			}
		}
	}

	return nullptr;
}

// #TODO: passing all these parameters in is horrible, tidy this up now we know what needs to be done
Chaos::FPBDRigidParticleHandle* FDeferredForcesModular::GetParticle(FGeometryCollectionPhysicsProxy* Proxy
		, int TransformIndex
		, int32 ParticleIdx
		, const FVector& PositionalOffset
		, const TManagedArray<FTransform>& Transforms
		, const TManagedArray<FTransform>& CollectionMassToLocal
		, const TManagedArray<int32>& Parent
		, FTransform& TransformOut)
{

	Chaos::FPBDRigidParticleHandle* SimParticle = Proxy->GetParticle_Internal(TransformIndex);
	Chaos::FPBDRigidClusteredParticleHandle* SimClusterParticle = Proxy->GetSolverClusterHandle_Internal(TransformIndex);

	if (SimParticle && !SimParticle->Disabled())
	{
		TransformOut = FTransform::Identity;
		TransformOut.AddToTranslation(PositionalOffset);
		return SimParticle;		// apply to broken off child
	}
	else if (SimClusterParticle && !SimClusterParticle->Disabled())
	{
		TransformOut.SetLocation(CollectionMassToLocal[Parent[TransformIndex]].InverseTransformPosition(Transforms[TransformIndex].GetLocation() + PositionalOffset));
		TransformOut.SetRotation(Transforms[TransformIndex].GetRotation());

		return SimClusterParticle; // apply to intact cluster
	}

	return nullptr;
}

// #TODO: passing all these parameters in is horrible, tidy this up now we know what needs to be done
Chaos::FPBDRigidParticleHandle* FDeferredForcesModular::GetParticle(FGeometryCollectionPhysicsProxy* Proxy
	, int TransformIndex
	, int32 ParticleIdx
	, const FVector& PositionalOffset
	, const FTransform& Transform
	, const TManagedArray<FTransform>& CollectionMassToLocal
	, const TManagedArray<int32>& Parent
	, FTransform& TransformOut)
{

	Chaos::FPBDRigidParticleHandle* SimParticle = Proxy->GetParticle_Internal(TransformIndex);
	Chaos::FPBDRigidClusteredParticleHandle* SimClusterParticle = Proxy->GetSolverClusterHandle_Internal(TransformIndex);

	if (SimParticle && !SimParticle->Disabled())
	{
		TransformOut = FTransform::Identity;
		TransformOut.AddToTranslation(PositionalOffset);
		return SimParticle;		// apply to broken off child
	}
	else if (SimClusterParticle && !SimClusterParticle->Disabled())
	{
		TransformOut.SetLocation(CollectionMassToLocal[Parent[TransformIndex]].InverseTransformPosition(Transform.GetLocation() + PositionalOffset));
		TransformOut.SetRotation(Transform.GetRotation());

		return SimClusterParticle; // apply to intact cluster
	}

	return nullptr;
}

Chaos::FPBDRigidParticleHandle* FDeferredForcesModular::GetParticle(const FTransform& OffsetTransform
	, FGeometryCollectionPhysicsProxy* Proxy
	, int TransformIndex
	, const FVector& PositionalOffset
	, FTransform& TransformOut)
{
	using namespace Chaos;

	const int SingleChassisIndex = 0;
	TransformOut = FTransform::Identity;

	Chaos::FPBDRigidClusteredParticleHandle* ClusterParticle = Proxy->GetSolverClusterHandle_Internal(SingleChassisIndex);
	Chaos::FPBDRigidParticleHandle* Child = Proxy->GetParticle_Internal(TransformIndex);
	FRigidTransform3 Frame = FRigidTransform3::Identity;

	if (ClusterParticle != nullptr && Child != nullptr)
	{
		const FRigidTransform3 ClusterWorldTM(ClusterParticle->GetX(), ClusterParticle->GetR());
		
		Proxy->GetParticle_Internal(TransformIndex);

		if (FPBDRigidClusteredParticleHandle* ClusterChild = Child->CastToClustered(); ClusterChild && ClusterChild->IsChildToParentLocked())
		{
			Frame = ClusterChild->ChildToParent();
		}
		else
		{
			//FTransform ChildT = Child->GetTransformXRCom();
			//const FRigidTransform3 ChildWorldTM(ChildT.GetLocation(), ChildT.GetRotation());
			const FRigidTransform3 ChildWorldTM(Child->GetX(), Child->GetR());
			Frame = ChildWorldTM.GetRelativeTransform(ClusterWorldTM);
			Frame.SetLocation(Frame.GetLocation() + OffsetTransform.TransformVector(PositionalOffset));

			//UE_LOGF(LogChaos, Warning, "ApplyForce Data TIdx %d %ls", TransformIndex, *Frame.GetTranslation().ToString());
		}
	}

	// local transform
	TransformOut.SetTranslation(Frame.GetTranslation());
	return ClusterParticle;
}

Chaos::FPBDRigidParticleHandle* FDeferredForcesModular::GetParticle(const FTransform& OffsetTransform
	, TArray<Chaos::FPBDRigidParticleHandle*>& Particles
	, TArray<Chaos::FPBDRigidClusteredParticleHandle*>& ClusterParticles
	, int32 ParticleIdx
	, const FVector& PositionalOffset
	, FTransform& TransformOut)
{
	using namespace Chaos;

	ensure(ClusterParticles.Num() == 1);
	const int SingleChassisIndex = 0;
	TransformOut = FTransform::Identity;

	const FRigidTransform3 ClusterWorldTM(ClusterParticles[SingleChassisIndex]->GetX(), ClusterParticles[SingleChassisIndex]->GetR());
	FRigidTransform3 Frame = FRigidTransform3::Identity;

	if (Chaos::FPBDRigidParticleHandle* Child = GetParticleFromUniqueIndex(ParticleIdx, Particles))
	{
		if (FPBDRigidClusteredParticleHandle* ClusterChild = Child->CastToClustered(); ClusterChild && ClusterChild->IsChildToParentLocked())
		{
			Frame = ClusterChild->ChildToParent();
		}
		else
		{
			//FTransform ChildT = Child->GetTransformXRCom();
			//const FRigidTransform3 ChildWorldTM(ChildT.GetLocation(), ChildT.GetRotation());
			const FRigidTransform3 ChildWorldTM(Child->GetX(), Child->GetR());
			Frame = ChildWorldTM.GetRelativeTransform(ClusterWorldTM);
			Frame.SetLocation(Frame.GetLocation() + OffsetTransform.TransformVector(PositionalOffset));
		}
	}

	// local transform
	TransformOut.SetTranslation(Frame.GetTranslation());
	return ClusterParticles[SingleChassisIndex];
}

Chaos::FPBDRigidParticleHandle* FDeferredForcesModular::GetClusterParticle(TArray<Chaos::FPBDRigidClusteredParticleHandle*>& ClusterParticles)
{
	using namespace Chaos;
	ensure(ClusterParticles.Num() == 1);
	const int SingleChassisIndex = 0;
	return ClusterParticles[SingleChassisIndex];
}

template<typename TransformType>
void FDeferredForcesModular::ApplyTemplate(FGeometryCollectionPhysicsProxy* Proxy
		, const TManagedArray<TransformType>& Transforms
		, const TManagedArray<FTransform>& CollectionMassToLocal
		, const TManagedArray<int32>& Parent)
{
	if (!GCoreModularVehicleDebugParams.DisableForces)
	{
		FTransform RelativeTransform;
		for (const FApplyForceData& Data : ApplyForceDatas)
		{
			Chaos::FPBDRigidParticleHandle* RigidHandle = GetParticle(Proxy, Data.TransformIndex, Data.ParticleIdx, FVector::ZeroVector, FTransform(Transforms[Data.TransformIndex]), CollectionMassToLocal, Parent, RelativeTransform);
			if (RigidHandle)
			{
				AddForce(RigidHandle, Data, RelativeTransform);
			}
		}

		for (const FApplyForceAtPositionData& Data : ApplyForceAtPositionDatas)
		{
			Chaos::FPBDRigidParticleHandle* RigidHandle = GetParticle(Proxy, Data.TransformIndex, Data.ParticleIdx, Data.Position, FTransform(Transforms[Data.TransformIndex]), CollectionMassToLocal, Parent, RelativeTransform);
			if (RigidHandle)
			{
				AddForceAtPosition(RigidHandle, Data, RelativeTransform);
			}
		}

		for (const FAddTorqueInRadiansData& Data : ApplyTorqueDatas)
		{
			Chaos::FPBDRigidParticleHandle* RigidHandle = GetParticle(Proxy, Data.TransformIndex, Data.ParticleIdx, FVector::ZeroVector, FTransform(Transforms[Data.TransformIndex]), CollectionMassToLocal, Parent, RelativeTransform);
			if (RigidHandle)
			{
				AddTorque(RigidHandle, Data, RelativeTransform);
			}
		}
	}

	ApplyForceDatas.Empty();
	ApplyForceAtPositionDatas.Empty();
	ApplyTorqueDatas.Empty();
}

void FDeferredForcesModular::Apply(FGeometryCollectionPhysicsProxy* Proxy
	, const TManagedArray<FTransform>& Transforms
	, const TManagedArray<FTransform>& CollectionMassToLocal
	, const TManagedArray<int32>& Parent)
{
	ApplyTemplate(Proxy, Transforms, CollectionMassToLocal, Parent);
}

void FDeferredForcesModular::Apply(FGeometryCollectionPhysicsProxy* Proxy
	, const TManagedArray<FTransform3f>& Transforms
	, const TManagedArray<FTransform>& CollectionMassToLocal
	, const TManagedArray<int32>& Parent)
{
	ApplyTemplate(Proxy, Transforms, CollectionMassToLocal, Parent);
}


void FDeferredForcesModular::Apply(FGeometryCollectionPhysicsProxy* Proxy)
{
	if (!GCoreModularVehicleDebugParams.DisableForces)
	{
		FTransform RelativeTransform;
		for (const FApplyForceData& Data : ApplyForceDatas)
		{
			Chaos::FPBDRigidParticleHandle* RigidHandle = GetParticle(Data.OffsetTransform, Proxy, Data.TransformIndex, FVector::ZeroVector, RelativeTransform);
			if (RigidHandle)
			{
				AddForce(RigidHandle, Data, RelativeTransform);
			}
		}

		for (const FApplyForceAtPositionData& Data : ApplyForceAtPositionDatas)
		{
			Chaos::FPBDRigidParticleHandle* RigidHandle = GetParticle(Data.OffsetTransform, Proxy, Data.TransformIndex, Data.Position, RelativeTransform);
			if (RigidHandle)
			{
				AddForceAtPosition(RigidHandle, Data, RelativeTransform);
			}
		}

		for (const FAddTorqueInRadiansData& Data : ApplyTorqueDatas)
		{
			Chaos::FPBDRigidParticleHandle* RigidHandle = GetParticle(Data.OffsetTransform, Proxy, Data.TransformIndex, FVector::ZeroVector, RelativeTransform);
			if (RigidHandle)
			{
				AddTorque(RigidHandle, Data, RelativeTransform);
			}
		}
	}

	ApplyForceDatas.Empty();
	ApplyForceAtPositionDatas.Empty();
	ApplyTorqueDatas.Empty();
}

void FDeferredForcesModular::Apply(TArray<Chaos::FPBDRigidParticleHandle*>& Particles
	, TArray<Chaos::FPBDRigidClusteredParticleHandle*>& ClusterParticles)
{
	if (!GCoreModularVehicleDebugParams.DisableForces)
	{
		FTransform RelativeTransform;
		for (const FApplyForceData& Data : ApplyForceDatas)
		{
			Chaos::FPBDRigidParticleHandle* RigidHandle = GetParticle(Data.OffsetTransform, Particles, ClusterParticles, Data.ParticleIdx, FVector::ZeroVector, RelativeTransform);
			if (RigidHandle)
			{
				AddForce(RigidHandle, Data, RelativeTransform);
			}
		}

		for (const FApplyForceAtPositionData& Data : ApplyForceAtCOMDatas)
		{
			Chaos::FPBDRigidParticleHandle* RigidHandle = GetClusterParticle(ClusterParticles);
			if (RigidHandle)
			{
				AddForceAtCOM(RigidHandle, Data);
			}
		}


		for (const FApplyForceAtPositionData& Data : ApplyForceAtPositionDatas)
		{
			Chaos::FPBDRigidParticleHandle* RigidHandle = GetParticle(Data.OffsetTransform, Particles, ClusterParticles, Data.ParticleIdx, Data.Position, RelativeTransform);
			if (RigidHandle)
			{
				AddForceAtPosition(RigidHandle, Data, RelativeTransform);
			}
		}

		for (const FAddTorqueInRadiansData& Data : ApplyTorqueDatas)
		{
			Chaos::FPBDRigidParticleHandle* RigidHandle = GetParticle(Data.OffsetTransform, Particles, ClusterParticles, Data.ParticleIdx, FVector::ZeroVector, RelativeTransform);
			if (RigidHandle)
			{
				AddTorque(RigidHandle, Data, RelativeTransform);
			}
		}
	}

	ApplyForceDatas.Empty();
	ApplyForceAtCOMDatas.Empty();
	ApplyForceAtPositionDatas.Empty();
	ApplyTorqueDatas.Empty();
}

void FDeferredForcesModular::Apply(Chaos::FPBDRigidParticleHandle* RigidHandle)
{
	if (!GCoreModularVehicleDebugParams.DisableForces)
	{
		FTransform RelativeTransform;

		for (const FApplyForceData& Data : ApplyForceDatas)
		{
			if (RigidHandle)
			{
				AddForce(RigidHandle, Data, RelativeTransform);
			}
		}

		for (const FApplyForceAtPositionData& Data : ApplyForceAtCOMDatas)
		{
			if (RigidHandle)
			{
				AddForceAtCOM(RigidHandle, Data);
			}
		}

		for (const FApplyForceAtPositionData& Data : ApplyForceAtPositionDatas)
		{
			if (RigidHandle)
			{
				AddForceAtPosition(RigidHandle, Data, RelativeTransform);
			}
		}

		for (const FAddTorqueInRadiansData& Data : ApplyTorqueDatas)
		{
			if (RigidHandle)
			{
				AddTorque(RigidHandle, Data, RelativeTransform);
			}
		}
	}

	ApplyForceDatas.Empty();
	ApplyForceAtCOMDatas.Empty();
	ApplyForceAtPositionDatas.Empty();
	ApplyTorqueDatas.Empty();
}

void AddForce_Implementation(Chaos::FPBDRigidParticleHandle* RigidHandle, const FTransform& ParticleOffsetTransform, const FVector& LocalForce, const FTransform& OffsetTransform, bool bLevelSlope, const FColor& DebugColor)
{
	const Chaos::FVec3 WorldCOM = Chaos::FParticleUtilitiesGT::GetCoMWorldPosition(RigidHandle);

	// local to world
	const FTransform WorldTM(RigidHandle->GetR(), RigidHandle->GetX());
	Chaos::FVec3 Position = WorldTM.TransformPosition(OffsetTransform.GetLocation());
	Chaos::FVec3 Force = WorldTM.TransformVector(ParticleOffsetTransform.TransformVector(OffsetTransform.TransformVector(LocalForce)));


	if (bLevelSlope && WorldTM.GetUnitAxis(EAxis::Z).Z > GCoreModularVehicleDebugParams.LevelSlopeThreshold)
	{
		static float ValueToUse = 0.5f;
		Force.X = ValueToUse;
		Force.Y = ValueToUse;
	}

	Chaos::FVec3 Arm = Position - WorldCOM;

	Chaos::FVec3 WorldTorque = Chaos::FVec3::CrossProduct(Arm, Force);
	Chaos::FPBDRigidParticleHandle *RP = RigidHandle->CastToRigidParticle();
	if (RP)
	{
		RP->AddForce(Force);
		RP->AddTorque(WorldTorque);

		// #TODO: remove HACK - doing this because vehicles are falling asleep and not waking up when forces are applied
	//	RP->SetSleepType(Chaos::ESleepType::NeverSleep);

#if CHAOS_DEBUG_DRAW
		if (GCoreModularVehicleDebugParams.ShowMass)
		{
			FString OutputText = FString::Format(TEXT("{0} {1}"), { RP->M(), RP->I().ToString() });
			Chaos::FDebugDrawQueue::GetInstance().DrawDebugString(RigidHandle->GetX() + FVector(0,0,200), OutputText, nullptr, FColor::White, -1, false, -1.0f);
		}
#endif
	}

#if CHAOS_DEBUG_DRAW
	if (GCoreModularVehicleDebugParams.ShowForces)
	{
		// #TODO: perhaps render different force types/sources in different colors, i.e. aerofoil, suspension, friction, etc.
		Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(Position, Position + Force * GCoreModularVehicleDebugParams.DrawForceScaling, DebugColor, false, -1.f, 0, 5.f);
		Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(Position, 5, 8, FColor::White, false, -1.f, 0, 5.f);
	}
#endif

}

void AddTorque_Implementation(Chaos::FPBDRigidParticleHandle* RigidHandle, const FTransform& ParticleOffsetTransform, const FVector& LocalTorque, const FTransform& OffsetTransform, const FColor& DebugColor)
{
	Chaos::FPBDRigidParticleHandle* RP = RigidHandle->CastToRigidParticle();
	if (RP)
	{
		// local to world
		const FTransform WorldTM(RigidHandle->GetR(), RigidHandle->GetX());
		FVector T1 = OffsetTransform.TransformVector(LocalTorque);
		FVector T2 = ParticleOffsetTransform.TransformVector(T1);
		Chaos::FVec3 WorldTorque = WorldTM.TransformVector(T2);
		RP->AddTorque(WorldTorque);
	}

}

void FDeferredForcesModular::AddForce(Chaos::FPBDRigidParticleHandle* RigidHandle, const FApplyForceData& DataIn, const FTransform& OffsetTransform)
{
	if (ensure(RigidHandle))
	{
		AddForce_Implementation(RigidHandle, DataIn.OffsetTransform, DataIn.Force, OffsetTransform, (DataIn.Flags & EForceFlags::LevelSlope) == EForceFlags::LevelSlope, DataIn.DebugColor);
	}

}

void FDeferredForcesModular::AddForceAtCOM(Chaos::FPBDRigidParticleHandle* RigidHandle, const FApplyForceAtPositionData& DataIn)
{
	if (ensure(RigidHandle))
	{
		FTransform OffsetTransform = FTransform::Identity;
		OffsetTransform.SetTranslation(RigidHandle->CenterOfMass() + DataIn.Position);
		AddForce_Implementation(RigidHandle, FTransform::Identity, DataIn.Force, OffsetTransform, (DataIn.Flags & EForceFlags::LevelSlope) == EForceFlags::LevelSlope, DataIn.DebugColor);
	}
}

void FDeferredForcesModular::AddForceAtPosition(Chaos::FPBDRigidParticleHandle* RigidHandle, const FApplyForceAtPositionData& DataIn, const FTransform& OffsetTransform)
{
	if (ensure(RigidHandle))
	{
		FTransform FinalOffsetTransform = OffsetTransform;
		FinalOffsetTransform.AddToTranslation(DataIn.Position);
		AddForce_Implementation(RigidHandle, DataIn.OffsetTransform, DataIn.Force, FinalOffsetTransform, (DataIn.Flags & EForceFlags::LevelSlope) == EForceFlags::LevelSlope, DataIn.DebugColor);
	}
}

void FDeferredForcesModular::AddTorque(Chaos::FPBDRigidParticleHandle* RigidHandle, const FAddTorqueInRadiansData& DataIn, const FTransform& OffsetTransform)
{	
	AddTorque_Implementation(RigidHandle, DataIn.OffsetTransform, DataIn.Torque, OffsetTransform, DataIn.DebugColor);
}
