// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/EngineModule.h"
#include "SimModule/SimModuleTree.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION_SHIP
#endif

namespace Chaos
{
	FEngineSimModule::FEngineSimModule(const FEngineSettings& Settings) : TSimModuleSettings<FEngineSettings>(Settings)
		, EngineIdleSpeed(RPMToOmega(Setup().IdleRPM))
		, MaxEngineSpeed(RPMToOmega(Setup().MaxRPM))
		, EngineStarted(true)
	{
	}

	void FEngineSimModule::Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem)
	{
		if (!EngineStarted)
		{
			return;
		}

		// TODO: Engine braking effect
		DriveTorque = GetEngineTorque(Inputs.GetControls().GetMagnitude(ThrottleControlName), GetRPM());

		if (DriveTorque < SMALL_NUMBER)
		{
			BrakingTorque = Setup().EngineBrakeEffect;
		}
		else
		{
			BrakingTorque = 0.0f;
		}
		TransmitTorque(VehicleModuleSystem, DriveTorque, BrakingTorque);
		IntegrateAngularVelocity(DeltaTime, Setup().EngineInertia);

		// clamp RPM
		AngularVelocity = FMath::Clamp(AngularVelocity, -MaxEngineSpeed, MaxEngineSpeed);

		// we don't let the engine stall
		if (AngularVelocity < EngineIdleSpeed)
		{
			AngularVelocity = EngineIdleSpeed;
		}

	}

	bool FEngineSimModule::GetDebugString(FString& StringOut) const
	{
		FTorqueSimModule::GetDebugString(StringOut);
		StringOut += FString::Format(TEXT("Drive {0}, Brake {1}, Load {2} RPM {3}"), { DriveTorque, BrakingTorque, LoadTorque, GetRPM() });
		return true;
	}


	float FEngineSimModule::GetEngineTorque(float ThrottlePosition, float EngineRPM)
	{
		if (EngineStarted)
		{
			return ThrottlePosition * GetTorqueFromRPM(EngineRPM);
		}

		return 0.f;
	}

	float FEngineSimModule::GetTorqueFromRPM(float RPM, bool LimitToIdle)
	{
		if (!EngineStarted || (FMath::Abs(RPM - Setup().MaxRPM) < 1.0f) || Setup().MaxRPM == 0)
		{
			return 0.0f;
		}

		if (LimitToIdle)
		{
			RPM = FMath::Clamp(RPM, (float)Setup().IdleRPM, (float)Setup().MaxRPM);
		}

		return Setup().TorqueCurve.GetValue(RPM, Setup().MaxRPM, Setup().MaxTorque);
	}

	void FEngineOutputData::FillOutputState(const ISimulationModuleBase* SimModule)
	{
		check(SimModule->IsSimType<class FEngineSimModule>());

		FSimOutputData::FillOutputState(SimModule);

		if (const FEngineSimModule* Sim = static_cast<const FEngineSimModule*>(SimModule))
		{
			RPM = Sim->GetRPM();
			Torque = Sim->GetDriveTorque();
		}
	}

	void FEngineOutputData::Lerp(const FSimOutputData& InCurrent, const FSimOutputData& InNext, float Alpha)
	{
		FSimOutputData::Lerp(InCurrent, InNext, Alpha);

		const FEngineOutputData& Current = static_cast<const FEngineOutputData&>(InCurrent);
		const FEngineOutputData& Next = static_cast<const FEngineOutputData&>(InNext);

		RPM = FMath::Lerp(Current.RPM, Next.RPM, Alpha);
		Torque = FMath::Lerp(Current.Torque, Next.Torque, Alpha);
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FString FEngineOutputData::ToString()
	{
		return FString::Printf(TEXT("%s, RPM=%3.3f, Torque=%3.3f")
			, *DebugString, RPM, Torque);
	}
#endif

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION_SHIP
#endif
