// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/ModuleFactoryRegister.h"
#include "SimModule/SimulationModuleBase.h"

namespace Chaos
{

	FModuleFactoryRegister& FModuleFactoryRegister::Get()
	{
		static FModuleFactoryRegister Instance;
		return Instance;
	}

	void FModuleFactoryRegister::RegisterFactory(const FName TypeName, TWeakPtr<IFactoryModule> InFactory)
	{
		RegisterFactory(GetModuleHash(TypeName), InFactory);
	}

	void FModuleFactoryRegister::RegisterFactory(const uint32 TypeNameHash, TWeakPtr<IFactoryModule> InFactory)
	{
		if (!ContainsFactory(TypeNameHash))
		{
			RegisteredFactoriesByName.Add(TypeNameHash, InFactory);
		}
	}

	void FModuleFactoryRegister::RemoveFactory(TWeakPtr<IFactoryModule> InFactory)
	{
		for (TPair<uint32, TWeakPtr<IFactoryModule>> Pair : RegisteredFactoriesByName)
		{
			if (Pair.Value == InFactory)
			{
				RegisteredFactoriesByName.Remove(Pair.Key);
			
				return;
			}
		}
	}

	void FModuleFactoryRegister::Reset()
	{
		RegisteredFactoriesByName.Reset();
	}

	bool FModuleFactoryRegister::ContainsFactory(const FName TypeName) const
	{
		return ContainsFactory(GetModuleHash(TypeName));
	}

	bool FModuleFactoryRegister::ContainsFactory(const uint32 TypeNameHash) const
	{
		return RegisteredFactoriesByName.Contains(TypeNameHash);
	}

	TSharedPtr<Chaos::FModuleNetData> FModuleFactoryRegister::GenerateNetData(const uint32 TypeNameHash, const int32 SimArrayIndex)
	{
		using namespace Chaos;

		if (RegisteredFactoriesByName.Contains(TypeNameHash))
		{
			TSharedPtr<IFactoryModule> PinnedFactory = RegisteredFactoriesByName[TypeNameHash].Pin();

			if (PinnedFactory.IsValid())
			{
				return PinnedFactory->GenerateNetData(SimArrayIndex);
			}
		}
		else
		{
			UE_LOGF(LogTemp, Error, "No factory registered for hashed type %d", TypeNameHash);
		}

		return nullptr;
	}
}