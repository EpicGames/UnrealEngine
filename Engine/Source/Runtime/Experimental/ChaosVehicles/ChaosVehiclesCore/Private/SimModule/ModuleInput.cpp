// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/ModuleInput.h"

#include "HAL/IConsoleManager.h"
#include "Net/Core/NetBitArray.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModuleInput)

DEFINE_LOG_CATEGORY(LogModularInput);

TArray<struct FModuleInputSetup>* FScopedModuleInputInitializer::InitSetupData = nullptr;

namespace ChaosModularVehicleCVars
{
	bool bEnableInputSendingTypeInfo = true;
	FAutoConsoleVariableRef EnableInputSendingTypeInfo(TEXT("p.ModularVehicle.EnableInputSendingTypeInfo"), bEnableInputSendingTypeInfo, TEXT("Enable/Disable Input sending type info. Default: true"));

	bool bEnableCustomInputQuantization = false;
	FAutoConsoleVariableRef EnableCustomInputQuantization(TEXT("p.ModularVehicle.EnableCustomInputQuantization"), bEnableCustomInputQuantization, TEXT("Enable/Disable Custom Input Quantization info. Default: false"));	
};

namespace UInt8Quantize 
{
	uint8 Compress(const double InValue)
	{
		const double Result = FMath::RoundToDouble(128.0 + FMath::FloatSelect(InValue, FMath::Min(127 * InValue, 127.0), FMath::Max(128 * InValue, -128.0)));
		return static_cast<uint8>(Result);
	}

	double UnCompress(const uint8 InValue)
	{
		const double Result = (InValue < 128) ? (InValue - 128.0) / 128.0 : (InValue - 128.0) / 127.0;
		return Result; 
	}

	void Quantize(double& InOutValue)
	{
		const uint8 CompressedValue = Compress(InOutValue);
		InOutValue = UnCompress(CompressedValue);
	}

	bool WriteCompressed(const double Value, FArchive& Ar)
	{
		uint8 CompressedValue = Compress(Value);
		Ar << CompressedValue;
		return true;
	}

	bool ReadCompressed(double& Value, FArchive& Ar)
	{
		uint8 CompressedValue;
		Ar << CompressedValue;
		Value = UnCompress(CompressedValue);
		return true;
	}

	bool Serialize(double& InOutValue, FArchive& Ar)
	{
		if (Ar.IsSaving())
		{
			return WriteCompressed(InOutValue, Ar);
		}

		return ReadCompressed(InOutValue, Ar);
	}
}

bool FModuleInputValue::IsNonZero(float Tolerance) const
{
	switch (ValueType)
	{
	case EModuleInputValueType::MBoolean:
	case EModuleInputValueType::MInteger:
		return ValueInt != 0;

	case EModuleInputValueType::MAxis2D:
	case EModuleInputValueType::MAxis1D:
	case EModuleInputValueType::MAxis3D:
		return Value.SizeSquared() >= Tolerance * Tolerance;
	}

	checkf(false, TEXT("Unsupported value type for module input value!"));
	return false;
}
float FModuleInputValue::GetMagnitudeSq() const
{
	switch (GetValueType())
	{
	case EModuleInputValueType::MBoolean:
	case EModuleInputValueType::MInteger:
		return ValueInt * ValueInt;

	case EModuleInputValueType::MAxis1D:
		return Value.X * Value.X;
	case EModuleInputValueType::MAxis2D:
		return Value.SizeSquared2D();
	case EModuleInputValueType::MAxis3D:
		return Value.SizeSquared();
	}

	checkf(false, TEXT("Unsupported value type for module input value!"));
	return 0.f;
}

float FModuleInputValue::GetMagnitude() const
{
	switch (GetValueType())
	{
	case EModuleInputValueType::MBoolean:
	case EModuleInputValueType::MInteger:
		return ValueInt;
	case EModuleInputValueType::MAxis1D: 
		return Value.X;
	case EModuleInputValueType::MAxis2D:
		return Value.Size2D();
	case EModuleInputValueType::MAxis3D:
		return Value.Size();
	}

	checkf(false, TEXT("Unsupported value type for module input value!"));
	return 0.f;
}

int32 FModuleInputValue::GetMagnitudeInt() const
{
	float Tolerance = KINDA_SMALL_NUMBER;

	switch (GetValueType())
	{
	case EModuleInputValueType::MBoolean:
	case EModuleInputValueType::MInteger:
		return ValueInt;

	case EModuleInputValueType::MAxis2D:
	case EModuleInputValueType::MAxis1D:
	case EModuleInputValueType::MAxis3D:
		return Value.SizeSquared() >= Tolerance * Tolerance;

	}

	checkf(false, TEXT("Unsupported value type for module input value!"));
	return 0.f;
}

void FModuleInputValue::SetMagnitude(float NewSize)
{
	switch (GetValueType())
	{
	case EModuleInputValueType::MBoolean:
	case EModuleInputValueType::MInteger:
		ValueInt = (int32)(NewSize);
		break;
	case EModuleInputValueType::MAxis1D:
		Value.X = NewSize;
		break;
	case EModuleInputValueType::MAxis2D:
		Value = Value.GetSafeNormal2D(UE_SMALL_NUMBER,FVector::UnitX()) * NewSize;
		break;
	case EModuleInputValueType::MAxis3D:
		Value = Value.GetSafeNormal(UE_SMALL_NUMBER,FVector::UnitX())* NewSize;
		break;
	default:
		checkf(false, TEXT("Unsupported value type for module input value!"));
		break;
	}
	bHasValue = true;
}

void FModuleInputValue::Quantize(double& InOutValue, EModuleInputQuantizationType InInputQuantizationType)
{
	if (ChaosModularVehicleCVars::bEnableCustomInputQuantization == false)
	{
		ModularQuantize::QuantizeValue<1, 16>(InOutValue);
		return;
	}
	
	switch (InInputQuantizationType)
	{
		case EModuleInputQuantizationType::Default_16Bits:
			ModularQuantize::QuantizeValue<1, 16>(InOutValue);
			break;

		case EModuleInputQuantizationType::Custom_8Bits:
			UInt8Quantize::Quantize(InOutValue);
			break;

		default:
			checkf(false, TEXT("Unsupported value type for input quantization type!"));
	}
}

bool FModuleInputValue::SerializeQuantized(double& InOutValue, FArchive& Ar, EModuleInputQuantizationType InInputQuantizationType)
{
	if (ChaosModularVehicleCVars::bEnableCustomInputQuantization == false)
	{
		return ModularQuantize::SerializeFixedFloat<1, 16>(InOutValue, Ar);
	}
	
	switch (InInputQuantizationType)
	{
		case EModuleInputQuantizationType::Default_16Bits:
			return ModularQuantize::SerializeFixedFloat<1, 16>(InOutValue, Ar);

		case EModuleInputQuantizationType::Custom_8Bits:
			return UInt8Quantize::Serialize(InOutValue, Ar);
			
		default:
			checkf(false, TEXT("Unsupported value type for input quantization type!"));
			return false;
	}
}

FModuleInputValue FModuleInputValue::ReturnQuantized(EModuleInputQuantizationType InInputQuantizationType) const
{
	FModuleInputValue OutValue = FModuleInputValue(ValueType);
	OutValue.Set(*this);
	switch (ValueType)
	{
		case EModuleInputValueType::MInteger:
		case EModuleInputValueType::MBoolean:
		break;
		case EModuleInputValueType::MAxis1D:
			Quantize(OutValue.Value.X, InInputQuantizationType);
			break;

		case EModuleInputValueType::MAxis2D:
			Quantize(OutValue.Value.X, InInputQuantizationType);
			Quantize(OutValue.Value.Y, InInputQuantizationType);
			break;

		case EModuleInputValueType::MAxis3D:
			Quantize(OutValue.Value.X, InInputQuantizationType);
			Quantize(OutValue.Value.Y, InInputQuantizationType);
			Quantize(OutValue.Value.Z, InInputQuantizationType);
			break;

		default:
			checkf(false, TEXT("Unsupported value type for module input value!"));
			break;
	}

	//UE_LOGF(LogTemp, Warning, "Original %f vs Quantized %f", Value.X, OutValue.Value.X);

	return OutValue;
}

void FModuleInputValue::Serialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess, EModuleInputQuantizationType InInputQuantizationType)
{
	// #TODO: send only value changes/deltas

	bool bIsNonZero = Ar.IsLoading() ? false : IsQuantizedNonZero(InInputQuantizationType);
	Ar.SerializeBits(&bIsNonZero, 1);
	Ar.SerializeBits(&ValueType, 3);
	Ar.SerializeBits(&bApplyInputDecay, 1);
	Ar.SerializeBits(&bInverseInputDecay, 1);
	Ar.SerializeBits(&bClearAfterConsumed, 1);
	Ar.SerializeBits(&bHasValue, 1);


	if (bIsNonZero)
	{
		switch (GetValueType())
		{
		case EModuleInputValueType::MBoolean:
			{
				bool bState = (ValueInt != 0);
				Ar.SerializeBits(&bState, 1);
				if (Ar.IsLoading())
				{
					ValueInt = bState ? 1 : 0;
				}
			}
			break;

		case EModuleInputValueType::MInteger:
			{
				Ar.SerializeIntPacked((uint32&)ValueInt);
			}
			break;


		case EModuleInputValueType::MAxis3D:
			SerializeQuantized(Value.Z, Ar, InInputQuantizationType);
		case EModuleInputValueType::MAxis2D:
			SerializeQuantized(Value.Y, Ar, InInputQuantizationType);
		case EModuleInputValueType::MAxis1D:
			SerializeQuantized(Value.X, Ar, InInputQuantizationType);
			break;

		default:
			checkf(false, TEXT("Unsupported value type for module input value!"));
			break;
		}
	}
	else
	{
		Reset();
	}
	bOutSuccess = true;
}

void FModuleInputValue::NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess,  EModuleInputQuantizationType InInputQuantizationType)
{
	bool bIsNonZero = Ar.IsLoading() ? false : IsQuantizedNonZero(InInputQuantizationType);
	Ar.SerializeBits(&bIsNonZero, 1);
	Ar.SerializeBits(&bHasValue, 1);
	if (ChaosModularVehicleCVars::bEnableInputSendingTypeInfo)
	{
		Ar.SerializeBits(&ValueType, 3);
		Ar.SerializeBits(&bApplyInputDecay, 1);
		Ar.SerializeBits(&bInverseInputDecay, 1);
		Ar.SerializeBits(&bClearAfterConsumed, 1);
	}

	if (bIsNonZero)
	{
		switch (GetValueType())
		{
		case EModuleInputValueType::MBoolean:
			{
				bool bState = (ValueInt != 0);
				Ar.SerializeBits(&bState, 1);
				if (Ar.IsLoading())
				{
					ValueInt = bState ? 1 : 0;
				}
			}
			break;

		case EModuleInputValueType::MInteger:
			{
				Ar.SerializeIntPacked((uint32&)ValueInt);
			}
			break;

		case EModuleInputValueType::MAxis3D:
			SerializeQuantized(Value.Z, Ar, InInputQuantizationType);
		case EModuleInputValueType::MAxis2D:
			SerializeQuantized(Value.Y, Ar, InInputQuantizationType);
		case EModuleInputValueType::MAxis1D:
			SerializeQuantized(Value.X, Ar, InInputQuantizationType);
			break;

		default:
			checkf(false, TEXT("Unsupported value type for module input value!"));
			break;
		}
	}
	else
	{
		Reset();
	}
	bOutSuccess = true;
}

void FModuleInputValue::DeltaNetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess, const FModuleInputValue& PreviousInputValue, EModuleInputQuantizationType InInputQuantizationType)
{
	bool bHasSamePreviousValue = Value == PreviousInputValue.Value;
	bool bHasSamePreviousType = ValueType == PreviousInputValue.ValueType;
	bool bHasSamePreviousValueInt = ValueInt == PreviousInputValue.ValueInt;
	bool bIsSameAsPrevious = bHasSamePreviousValue && bHasSamePreviousType && bHasSamePreviousValueInt;
	Ar.SerializeBits(&bIsSameAsPrevious, 1);
	if (bIsSameAsPrevious)
	{
		bOutSuccess = true;
		if (Ar.IsSaving())
		{
			return;
		}

		ConvertToType(PreviousInputValue.ValueType);
		Value = PreviousInputValue.Value;
		ValueInt = PreviousInputValue.ValueInt;
		bApplyInputDecay = PreviousInputValue.bApplyInputDecay;
		bInverseInputDecay = PreviousInputValue.bInverseInputDecay;
		bClearAfterConsumed = PreviousInputValue.bClearAfterConsumed;
		bHasValue = PreviousInputValue.bHasValue;
		return;
	}

	NetSerialize(Ar, Map, bOutSuccess, InInputQuantizationType);
}

/** Set only the value without overriding the other properties in FModuleInputValue */
void FModuleInputValue::Set(const FModuleInputValue& In)
{
	ValueInt = In.ValueInt;
	Value = In.Value;
	bHasValue = In.bHasValue;
}

void FModuleInputValue::Lerp(const FModuleInputValue& Min, const FModuleInputValue& Max, float Alpha)
{
	switch (GetValueType())
	{
	case EModuleInputValueType::MBoolean:
	case EModuleInputValueType::MInteger:
		ValueInt = Max.ValueInt; // don't lerp just use last value, is this correct for both cases?
		break;

	default:
		Value = FMath::Lerp(Min.Value, Max.Value, Alpha);
		break;
	}
	bHasValue = true;
}

void FModuleInputValue::Merge(const FModuleInputValue& From)
{
	switch (GetValueType())
	{
	case EModuleInputValueType::MBoolean:
		// Additive merge: true if either frame had the input active
		ValueInt |= From.ValueInt;
		break;

	case EModuleInputValueType::MInteger:
		// *this is the newer/accumulated frame, keep this.ValueInt unchanged
		break;

	case EModuleInputValueType::MAxis1D:
	case EModuleInputValueType::MAxis2D:
	case EModuleInputValueType::MAxis3D:
		// Average adjacent frames for analog inputs
		Value = (Value + From.Value) * 0.5;
		break;
	default:
		checkf(false, TEXT("Unsupported value type for module input value!"));
		break;
	}
	bHasValue = true;
}

void FModuleInputValue::Combine(const FModuleInputValue& With)
{
	switch (GetValueType())
	{
	case EModuleInputValueType::MBoolean:
		// ensure we capture edges of digital inputs by taking largest absolute value
		if (FMath::Abs(With.ValueInt) >= FMath::Abs(ValueInt))
		{
			ValueInt = With.ValueInt;
		}
		break;

	case EModuleInputValueType::MInteger:
		ValueInt += With.ValueInt;
		break;

	case EModuleInputValueType::MAxis1D:
	case EModuleInputValueType::MAxis2D:
	case EModuleInputValueType::MAxis3D:
		Value += With.Value;
		break;
	default:
		checkf(false, TEXT("Unsupported value type for module input value!"));
		break;
	}
	bHasValue = true;
}

void FModuleInputValue::Average(const FModuleInputValue& With)
{
	if (!bHasValue)
	{
		Set(With);
		return;
	}
	switch (GetValueType())
	{
	case EModuleInputValueType::MBoolean:
		// ensure we capture edges of digital inputs by taking largest absolute value
		if (FMath::Abs(With.ValueInt) >= FMath::Abs(ValueInt))
		{
			ValueInt = With.ValueInt;
		}
		break;

	case EModuleInputValueType::MInteger:
		ValueInt += With.ValueInt;
		ValueInt *= 0.5f;
		break;

	case EModuleInputValueType::MAxis1D:
	case EModuleInputValueType::MAxis2D:
	case EModuleInputValueType::MAxis3D:
		Value += With.Value;
		Value *= 0.5f;
		break;
	default:
		checkf(false, TEXT("Unsupported value type for module input value!"));
		break;
	}
	bHasValue = true;
}

void FModuleInputValue::Decay(const float DecayAmount)
{
	if (!ShouldApplyInputDecay())
	{
		return;
	}

	/** Set the multiplier that that applies the decay amount, decay is passed in as for example 0.1 to decay 10 */
	const float DecayMultiplier = bInverseInputDecay ? DecayAmount : (1.0f - DecayAmount);

	switch (GetValueType())
	{
	case EModuleInputValueType::MBoolean:
		// Don't decay boolean
		break;
	case EModuleInputValueType::MInteger:
		// Don't decay integers
		break;
	case EModuleInputValueType::MAxis1D:
	case EModuleInputValueType::MAxis2D:
	case EModuleInputValueType::MAxis3D:
		// Reduce input value by DecayAmount
		Value *= DecayMultiplier;
		break;
	default:
		checkf(false, TEXT("Unsupported value type for module input value!"));
		break;
	}
}

FString FModuleInputValue::ToString() const
{
	switch (GetValueType())
	{
	case EModuleInputValueType::MBoolean:
		return FString(IsNonZero() ? TEXT("true") : TEXT("false"));
	case EModuleInputValueType::MInteger:
		return FString(TEXT("%d"), ValueInt);
	case EModuleInputValueType::MAxis1D:
		return FString::Printf(TEXT("%3.3f"), Value.X);
	case EModuleInputValueType::MAxis2D:
		return FString::Printf(TEXT("X=%3.3f Y=%3.3f"), Value.X, Value.Y);
	case EModuleInputValueType::MAxis3D:
		return FString::Printf(TEXT("X=%3.3f Y=%3.3f Z=%3.3f"), Value.X, Value.Y, Value.Z);
	}

	checkf(false, TEXT("Unsupported value type for module input value!"));
	return FString{};
}



FModuleInputValue UDefaultModularVehicleInputModifier::InterpInputValue(float DeltaTime, const FModuleInputValue& CurrentValue, const FModuleInputValue& NewValue) const
{
	const FModuleInputValue DeltaValue = NewValue - CurrentValue;

	// We are "rising" when DeltaValue has the same sign as CurrentValue (i.e. delta causes an absolute magnitude gain)
	// OR we were at 0 before, and our delta is no longer 0.
	const bool bRising = ((DeltaValue.GetMagnitude() > 0.0f) == (CurrentValue.GetMagnitude() > 0.0f)) ||
		((DeltaValue.GetMagnitude() != 0.f) && (CurrentValue.GetMagnitude() == 0.f));

	const float MaxMagnitude = DeltaTime * (bRising ? RiseRate : FallRate);

	const FModuleInputValue ClampedDeltaValue = FModuleInputValue::Clamp(DeltaValue, -MaxMagnitude, MaxMagnitude);

	return CurrentValue + ClampedDeltaValue;
}

float UDefaultModularVehicleInputModifier::CalcControlFunction(float InputValue)
{
	// #TODO: Reinstate ?
	check(false);
	return 0.0f;
	// user defined curve

	//// else use option from drop down list
	//switch (InputCurveFunction)
	//{
	//case EFunctionType::CustomCurve:
	//{
	//	if (UserCurve.GetRichCurveConst() && !UserCurve.GetRichCurveConst()->IsEmpty())
	//	{
	//		float Output = FMath::Clamp(UserCurve.GetRichCurveConst()->Eval(FMath::Abs(InputValue)), 0.0f, 1.0f);
	//		return (InputValue < 0.f) ? -Output : Output;
	//	}
	//	else
	//	{
	//		return InputValue;
	//	}
	//}
	//break;
	//case EFunctionType::SquaredFunction:
	//{
	//	return (InputValue < 0.f) ? -InputValue * InputValue : InputValue * InputValue;
	//}
	//break;

	//case EFunctionType::LinearFunction:
	//default:
	//{
	//	return InputValue;
	//}
	//break;

	//}

}


void FModuleInputContainer::Initialize(TArray<FModuleInputSetup>& SetupData, FInputNameMap& NameMapOut)
{
	NameMapOut.Reset();
	InputValues.Reset();

	for (FModuleInputSetup& Setup : SetupData)
	{
		int Index = AddInput(Setup.Type);
		InputValues[Index].SetApplyInputDecay(Setup.bApplyInputDecay);
		InputValues[Index].SetInverseInputDecay(Setup.bInverseInputDecay);
		InputValues[Index].SetClearAfterConsumed(Setup.bClearAfterConsumed);
		NameMapOut.Add(Setup.Name, Index);
	}
}

void FModuleInputContainer::ZeroValues()
{
	for (int I = 0; I < InputValues.Num(); I++)
	{
		InputValues[I].Reset();
	}
}

void FModuleInputContainer::Serialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess, EModuleInputQuantizationType InInputQuantizationType)
{
	using namespace UE::Net;

	bOutSuccess = true;

	uint32 Number = InputValues.Num();
	Ar.SerializeIntPacked(Number);

	if (Ar.IsLoading())
	{
		InputValues.SetNum(Number);
	}

	for (uint32 I = 0; I < Number; I++)
	{
		InputValues[I].Serialize(Ar, Map, bOutSuccess, InInputQuantizationType);
	}
	return;
}

int FModuleInputContainer::AddInput(EModuleInputValueType Type)
{
	InputValues.Add(FModuleInputValue(Type));
	return InputValues.Num() - 1;
}

void FModuleInputContainer::RemoveAllInputs()
{
	InputValues.Reset();
}

void FModuleInputContainer::Lerp(const FModuleInputContainer& Min, const FModuleInputContainer& Max, float Alpha)
{
	for (int I = 0; I < FMath::Min3(InputValues.Num(), Min.InputValues.Num(), Max.InputValues.Num()); I++)
	{
		InputValues[I].Lerp(Min.InputValues[I], Max.InputValues[I], Alpha);
	}
}

void FModuleInputContainer::Merge(const FModuleInputContainer& From)
{
	int Num = FMath::Min(InputValues.Num(), From.InputValues.Num());
	for (int I = 0; I < Num; I++)
	{
		InputValues[I].Merge(From.InputValues[I]);
	}
}

void FModuleInputContainer::Combine(const FModuleInputContainer& With)
{
	int Num = FMath::Min(InputValues.Num(), With.InputValues.Num());
	for (int I = 0; I < Num; I++)
	{
		InputValues[I].Combine(With.InputValues[I]);
	}
}

void FModuleInputContainer::Decay(const float DecayAmount)
{
	for (int I = 0; I < InputValues.Num(); I++)
	{
		InputValues[I].Decay(DecayAmount);
	}
}

void FModuleInputContainer::ClearConsumedInputs()
{
	for (int I = 0; I < InputValues.Num(); I++)
	{
		if (InputValues[I].ShouldClearAfterConsumed())
		{
			InputValues[I].Reset();
		}
	}
}

void FInputInterface::SetValue(const FName& InName, const FModuleInputValue& InValue, bool Quantize /*= true*/)
{
	if (ValueContainer.GetNumInputs() > 0)
	{
		if (const int* Index = NameMap.Find(InName))
		{
			ValueContainer.SetValueAtIndex(*Index, InValue, InputQuantizationType, Quantize);
		}
		else
		{
			UE_LOGF(LogModularInput, Warning, "Trying to set the value of an undefined control input %ls", *InName.ToString());
		}
	}
}

void FInputInterface::CombineValue(const FName& InName, const FModuleInputValue& InValue)
{
	if (ValueContainer.GetNumInputs() > 0)
	{
		if (const int* Index = NameMap.Find(InName))
		{
			ValueContainer.CombineValueAtIndex(*Index, InValue, InputQuantizationType);
		}
		else
		{
			UE_LOGF(LogModularInput, Warning, "Trying to set the value of an undefined control input %ls", *InName.ToString());
		}
	}
}

void FInputInterface::AverageValue(const FName& InName, const FModuleInputValue& InValue)
{
	if (ValueContainer.GetNumInputs() > 0)
	{
		if (const int* Index = NameMap.Find(InName))
		{
			ValueContainer.AverageValueAtIndex(*Index, InValue, InputQuantizationType);
		}
		else
		{
			UE_LOGF(LogModularInput, Warning, "Trying to set the value of an undefined control input %ls", *InName.ToString());
		}
	}
}

void FInputInterface::MergeValue(const FName& InName, const FModuleInputValue& InValue)
{
	if (ValueContainer.GetNumInputs() > 0)
	{
		if (const int* Index = NameMap.Find(InName))
		{
			ValueContainer.MergeValueAtIndex(*Index, InValue, InputQuantizationType);
		}
		else
		{
			UE_LOGF(LogModularInput, Warning, "Trying to set the value of an undefined control input %ls", *InName.ToString());
		}
	}
}


FModuleInputValue FInputInterface::GetValue(const FName& InName) const
{
	if (ValueContainer.GetNumInputs() > 0)
	{
		if (const int* Index = NameMap.Find(InName))
		{
			return ValueContainer.GetValueAtIndex(*Index);
		}
		else
		{
			UE_LOGF(LogModularInput, Warning, "Trying to get the value of an undefined control input %ls", *InName.ToString());
		}
	}

	return FModuleInputValue(false);
}

EModuleInputValueType FInputInterface::GetValueType(const FName& InName) const
{
	if (ValueContainer.GetNumInputs() > 0)
	{
		if (const int* Index = NameMap.Find(InName))
		{
			return ValueContainer.GetValueAtIndex(*Index).GetValueType();
		}
		else
		{
			UE_LOGF(LogModularInput, Warning, "Trying to get the value type of an undefined control input %ls", *InName.ToString());
		}
	}

	return EModuleInputValueType::MBoolean;
}


float FInputInterface::GetMagnitude(const FName& InName) const
{
	if (ValueContainer.GetNumInputs() > 0)
	{
		if (const int* Index = NameMap.Find(InName))
		{
			return ValueContainer.GetValueAtIndex(*Index).GetMagnitude();
		}
	}

	return 0.0f;
}

int32 FInputInterface::GetMagnitudeInt(const FName& InName) const
{
	if (ValueContainer.GetNumInputs() > 0)
	{
		if (const int* Index = NameMap.Find(InName))
		{
			return ValueContainer.GetValueAtIndex(*Index).GetMagnitudeInt();
		}
	}

	return 0;
}

bool FInputInterface::InputsNonZero() const
{
	for (int I = 0; I < ValueContainer.GetNumInputs(); I++)
	{
		if (ValueContainer.GetValueAtIndex(I).IsNonZero())
		{
			return true;
		}
	}

	return false;
}
