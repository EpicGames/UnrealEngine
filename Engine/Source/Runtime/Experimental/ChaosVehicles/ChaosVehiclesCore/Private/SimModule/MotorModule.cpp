// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/MotorModule.h"
#include "SimModule/SimModuleTree.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif


namespace Chaos
{
	FMotorSimModule::FMotorSimModule(const FMotorSettings& Settings)
		: TSimModuleSettings<FMotorSettings>(Settings)
	{
	}

	void FMotorSimModule::Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem)
	{
		float MaxRPM = Setup().MaxRPM;

		// protect against divide by zero
		if (MaxRPM > SMALL_NUMBER)
		{
			float NormalizedRPM = GetRPM() / Setup().MaxRPM;

			float NormalizedTorque = 0.0f;
			if (NormalizedRPM > 0.0f)
			{
				NormalizedTorque = 1.0f - FMath::Square(1.0f - 2.0f * NormalizedRPM) * 0.5f;
			}
			else
			{
				NormalizedTorque = -1.0f - FMath::Square(1.0f + 2.0f * NormalizedRPM);
			}

			DriveTorque = NormalizedTorque * Setup().MaxTorque * Inputs.GetControls().GetMagnitude(ThrottleControlName);

			float BrakeTorque = 0.f;
			TransmitTorque(VehicleModuleSystem, DriveTorque, BrakeTorque);
			IntegrateAngularVelocity(DeltaTime, Setup().EngineInertia);
		}

	}


} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
