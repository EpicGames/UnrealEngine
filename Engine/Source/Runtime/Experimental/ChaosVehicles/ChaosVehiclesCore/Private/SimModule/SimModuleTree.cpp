// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/SimModuleTree.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "Chaos/DebugDrawQueue.h"
#include "VehicleUtility.h"
#include "SimModule/SimModulesInclude.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SimModuleTree)

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION_SHIP
#endif

DECLARE_CYCLE_STAT(TEXT("ModularVehicle_SimulateTree"), STAT_ModularVehicle_SimulateTree, STATGROUP_ModularVehicleSimTree);
DECLARE_CYCLE_STAT(TEXT("ModularVehicle_GenerateReplicationStructure"), STAT_ModularVehicle_GenerateReplicationStructure, STATGROUP_ModularVehicleSimTree);
DECLARE_CYCLE_STAT(TEXT("ModularVehicle_SetNetState"), STAT_ModularVehicle_SetNetState, STATGROUP_ModularVehicleSimTree);
DECLARE_CYCLE_STAT(TEXT("ModularVehicle_SetSimState"), STAT_ModularVehicle_SetSimState, STATGROUP_ModularVehicleSimTree);
DECLARE_CYCLE_STAT(TEXT("ModularVehicle_AppendTreeUpdates"), STAT_ModularVehicle_AppendTreeUpdates, STATGROUP_ModularVehicleSimTree);

bool bModularVehicle_NetworkData_Enable = true;
FAutoConsoleVariableRef CVarModularVehicleNetworkDataEnable(TEXT("p.ModularVehicle.NetworkData.Enable"), bModularVehicle_NetworkData_Enable, TEXT("Enable/Disable additional module network data."));

bool bModularVehicle_DisableAllSimulationAfterDestruction_Enable = false;
FAutoConsoleVariableRef CVarModularVehicleDisableAllSimulationAfterDestruction(TEXT("p.ModularVehicle.DisableAllSimulationAfterDestruction.Enable"), bModularVehicle_DisableAllSimulationAfterDestruction_Enable, TEXT("Enable/Disable whole vehicle simulation after destruction has occured."));

namespace Chaos
{

int FSimModuleTree::AddRoot(ISimulationModuleBase* SimModule)
{
	return AddNodeBelow(-1, SimModule);
}

void FSimModuleTree::Reparent(int AtIndex, int ParentIndex)
{
	check(AtIndex < SimulationModuleTree.Num());
	check(ParentIndex < SimulationModuleTree.Num());

	UE_LOGF(LogSimulationModule, Log, "Reparent %ls To %ls"
		, *SimulationModuleTree[AtIndex].SimModule->GetDebugName()
		, *SimulationModuleTree[ParentIndex].SimModule->GetDebugName());

	int OrginalParent = SimulationModuleTree[AtIndex].Parent;
	if (OrginalParent != ParentIndex)
	{
		SimulationModuleTree[AtIndex].Parent = ParentIndex;
		SimulationModuleTree[ParentIndex].Children.Add(AtIndex);

		// if had a parent and wasn't a root
		if (OrginalParent != -1)
		{
			SimulationModuleTree[OrginalParent].Children.Remove(AtIndex);
		}
	}
}

int FSimModuleTree::AddNodeBelow(int AtIndex, ISimulationModuleBase* SimModule)
{
	int NewIndex = GetNextIndex();
	FSimModuleNode& Node = SimulationModuleTree[NewIndex];
	SimModule->SetTreeIndex(NewIndex);
	Node.SimModule = SimModule;
	Node.Parent = AtIndex;
	if (AtIndex >= 0)
	{
		SimulationModuleTree[AtIndex].Children.Add(NewIndex);
	}
	else
	{
		Node.Parent = FSimModuleNode::INVALID_IDX;
	}

	return NewIndex;
}

void FSimModuleTree::AppendTreeUpdates(const FSimTreeUpdates& TreeUpdates)
{
	SCOPE_CYCLE_COUNTER(STAT_ModularVehicle_AppendTreeUpdates);

	int TreeIndex = -1;
	TMap<int, int> SimTreeMapping;

	//if (GetNumNodes() == 0) //Always add a null node, so there is a parent when the root chassis is removed
	//{
	//	// add a single chassis root component
	//	Chaos::FChassisSettings Settings;
	//	Chaos::ISimulationModuleBase* Chassis = new Chaos::FChassisSimModule(Settings);
	//	ParentIndex = AddRoot(Chassis);
	//	Chassis->SetTransformIndex(-1);
	//}

	int LocalIndex = 0;
	for (const FPendingModuleAdds& TreeUpdate : TreeUpdates.GetNewModules())
	{
		int AddIndex = -1;
		if (LocalIndex == 0)
		{
			// the first tree update contains the actual parent index in the real tree..
			AddIndex = TreeUpdate.ParentIndex;
		}
		else
		{
			// all other tree updates have a parent index that is relative to the first
			if (int* AddIndexPtr = SimTreeMapping.Find(TreeUpdate.ParentIndex))
			{
				AddIndex = *AddIndexPtr;
			}
		}

		TreeIndex = AddNodeBelow(AddIndex, TreeUpdate.NewSimModule);
		SimTreeMapping.Add(LocalIndex, TreeIndex);
		LocalIndex++;
	}

	if (!SimulationModuleTree.IsEmpty())
	{
		for (const FPendingModuleDeletions& TreeUpdate : TreeUpdates.GetDeletedModules())
		{
			for (int Index = 0; Index < SimulationModuleTree.Num(); Index++)
			{
				if (Chaos::ISimulationModuleBase* SimModule = GetNode(Index).SimModule)
				{
					if (SimModule->GetGuid() == TreeUpdate.Guid)
					{
						DeleteNode(Index);
						break;
					}
				}
			}
		}				
	}

}


int FSimModuleTree::GetNextIndex()
{
	int NewIndex = FSimModuleNode::INVALID_IDX;
	if (FreeList.IsEmpty())
	{
		NewIndex = SimulationModuleTree.Num();
		SimulationModuleTree.AddZeroed(1);
		SimulationModuleTree[NewIndex].Parent = FSimModuleNode::INVALID_IDX;
		SimulationModuleTree[NewIndex].SimModule = nullptr;
	}
	else
	{
		NewIndex = FreeList.Pop();
	}

	return NewIndex;
}

int FSimModuleTree::InsertNodeAbove(int AtIndex, ISimulationModuleBase* SimModule)
{
	int NewIndex = FSimModuleNode::INVALID_IDX;

	if (ensure(AtIndex < SimulationModuleTree.Num()))
	{
		NewIndex = GetNextIndex();
		FSimModuleNode& Node = SimulationModuleTree[NewIndex];

		int OriginalParentIdx = SimulationModuleTree[AtIndex].Parent;

		// remove current idx from children of parent & add new index in its place
		SimulationModuleTree[OriginalParentIdx].Children.Remove(AtIndex);
		SimulationModuleTree[OriginalParentIdx].Children.Add(NewIndex);

		SimulationModuleTree[AtIndex].Parent = NewIndex;

		Node.SimModule = SimModule;
		Node.Parent = OriginalParentIdx;	// new node takes parent from existing node
		Node.Children.Add(AtIndex);			// existing node becomes child of new node
		SimModule->SetTreeIndex(NewIndex);
	}

	return NewIndex;
}

void FSimModuleTree::DeleteNode(int AtIndex)
{
	// if is there is ever an issue then we have the option of disabling ALL module simulation after first destruction occurs
	if (bModularVehicle_DisableAllSimulationAfterDestruction_Enable)
	{
		SetSimulationEnabled(false);
	}

	// multiple children might become equal parents?
	
	int ParentIndex = SimulationModuleTree[AtIndex].Parent;
	
	if (ParentIndex >= 0)
	{
		// remove from parents children list
		SimulationModuleTree[ParentIndex].Children.Remove(AtIndex);
	}

	// move deleted nodes children to parent and these children need new parent
	for (int ChildIndex : SimulationModuleTree[AtIndex].Children)
	{
		if (ParentIndex >= 0)
		{
			SimulationModuleTree[ParentIndex].Children.Add(ChildIndex);
		}
		SimulationModuleTree[ChildIndex].Parent = ParentIndex;
	}	

	SimulationModuleTree[AtIndex].Parent = FSimModuleNode::INVALID_IDX;
	SimulationModuleTree[AtIndex].Children.Empty();
	delete SimulationModuleTree[AtIndex].SimModule;
	SimulationModuleTree[AtIndex].SimModule = nullptr;

	FreeList.Push(AtIndex);

}


void FSimModuleTree::Simulate(float DeltaTime, FAllInputs& Inputs, IPhysicsProxyBase* PhysicsProxy, Chaos::FPBDRigidParticleHandle* RootParticle)
{
	SCOPE_CYCLE_COUNTER(STAT_ModularVehicle_SimulateTree);

	if (IsSimulationEnabled())
	{
		if (PhysicsProxy && RootParticle)
		{
			UpdateVehicleState(RootParticle);

			UpdateModuleVelocites(PhysicsProxy, RootParticle, Inputs.GetControls().InputsNonZero() || Inputs.bKeepVehicleAwake);
		}

		TArray<int> RootNodes;
		GetRootNodes(RootNodes);

		if (SimTreeProcessingOrder == ESimTreeProcessingOrder::LeafFirstBFS)
		{
			SimulateNodeBFS(DeltaTime, Inputs, RootNodes, PhysicsProxy, RootParticle);
		}
		else if(SimTreeProcessingOrder != ESimTreeProcessingOrder::ManualOverride)
		{
			for (int RootIndex : RootNodes)
			{
				SimulateNode(DeltaTime, Inputs, RootIndex, PhysicsProxy, RootParticle);
			}
		}

	}

}

void FSimModuleTree::OnContactModification(FCollisionContactModifier& Modifier, IPhysicsProxyBase* PhysicsProxy)
{
	TArray<int> RootNodes;
	GetRootNodes(RootNodes);
	for (int RootIndex : RootNodes)
	{
		OnContactModificationInternal(RootIndex, Modifier, PhysicsProxy);
	}
}

void FSimModuleTree::SimulateNode(float DeltaTime, FAllInputs& Inputs, int NodeIndex, IPhysicsProxyBase* PhysicsProxy, Chaos::FPBDRigidParticleHandle* ParticleHandle)
{
	if (ISimulationModuleBase* Module = AccessSimModule(NodeIndex))
	{
		if (SimTreeProcessingOrder == ESimTreeProcessingOrder::RootFirst || SimTreeProcessingOrder == ESimTreeProcessingOrder::ManualOverride)
		{
			if (Module->IsEnabled())
			{
				Module->Simulate(PhysicsProxy, ParticleHandle, DeltaTime, Inputs, *this);

				if (IsAnimationEnabled() && Module->IsAnimationEnabled())
				{
					Module->Animate();
					UpdateClusterUnionTransformsIfRequired(PhysicsProxy, Module);
				}
			}
		}

		if (SimTreeProcessingOrder != ESimTreeProcessingOrder::ManualOverride)
		{
			for (int ChildIdx : GetChildren(NodeIndex))
			{
				SimulateNode(DeltaTime, Inputs, ChildIdx, PhysicsProxy, ParticleHandle);
			}
		}

		if (SimTreeProcessingOrder == ESimTreeProcessingOrder::LeafFirst)
		{
			if (Module->IsEnabled())
			{
				Module->Simulate(PhysicsProxy, ParticleHandle, DeltaTime, Inputs, *this);

				if (IsAnimationEnabled() && Module->IsAnimationEnabled())
				{
					Module->Animate();
					UpdateClusterUnionTransformsIfRequired(PhysicsProxy, Module);
				}
			}
		}

	}
}


void FSimModuleTree::UpdateClusterUnionTransformsIfRequired(IPhysicsProxyBase* PhysicsProxy, ISimulationModuleBase* Module)
{
	if (PhysicsProxy && Module && PhysicsProxy->GetType() == EPhysicsProxyType::ClusterUnionProxy)
	{
		if (FClusterUnionPhysicsProxy* CUProxy = static_cast<FClusterUnionPhysicsProxy*>(PhysicsProxy))
		{
			if (FPBDRigidClusteredParticleHandle* Particle = Module->GetClusterParticle(CUProxy))
			{
				const FSimModuleAnimationData& AnimData = Module->GetAnimationData();
				uint16 AnimFlags = AnimData.AnimFlags;
				if (AnimFlags == Chaos::EAnimationFlags::AnimateRotation)
				{
					const FQuat& RestRotation = Module->GetInitialParticleTransform().GetRotation();
					Particle->ChildToParent().SetRotation(RestRotation * AnimData.CombinedRotation);
				}

				if (AnimFlags == Chaos::EAnimationFlags::AnimatePosition)
				{
					FVector Movement = Module->GetComponentTransform().TransformVector(AnimData.AnimationLocOffset);
					FVector RestPos = Module->GetInitialParticleTransform().GetTranslation();
					Particle->ChildToParent().SetTranslation(RestPos + Movement);
				}
			}
		}
	}
}

void FSimModuleTree::SimulateNodeBFS(float DeltaTime, FAllInputs& Inputs, const TArray<int>& RootNodes, IPhysicsProxyBase* PhysicsProxy, Chaos::FPBDRigidParticleHandle* ParticleHandle)
{
	TQueue<int> Queue;
	TArray<int> Stack;

	for (int Idx : RootNodes)
	{
		if (AccessSimModule(Idx))
		{
			Queue.Enqueue(Idx);
		}
	}

	while (!Queue.IsEmpty())
	{
		int OutNode = -1;
		Queue.Dequeue(OutNode);
		if (OutNode >= 0)
		{
			Stack.Push(OutNode);
			for (int Idx : GetChildren(OutNode))
			{
				if (AccessSimModule(Idx))
				{
					Queue.Enqueue(Idx);
				}
			}
		}
	}

	while (!Stack.IsEmpty())
	{
		int Node = Stack.Pop();
		if (ISimulationModuleBase* Module = AccessSimModule(Node))
		{
			if (Module->IsEnabled())
			{
				Module->Simulate(PhysicsProxy, ParticleHandle, DeltaTime, Inputs, *this);

				if (IsAnimationEnabled() && Module->IsAnimationEnabled())
				{
					Module->Animate();
					UpdateClusterUnionTransformsIfRequired(PhysicsProxy, Module);
				}
			}
		}
	}
}

void FSimModuleTree::OnContactModificationInternal(int NodeIndex, FCollisionContactModifier& Modifier, IPhysicsProxyBase* PhysicsProxy)
{
	if (ISimulationModuleBase* Module = AccessSimModule(NodeIndex))
	{
		Module->OnContactModification(Modifier, PhysicsProxy);
	}
	for (int ChildIdx : GetChildren(NodeIndex))
	{
		OnContactModificationInternal(ChildIdx, Modifier, PhysicsProxy);
	}
}


void FSimModuleTree::DeleteNodesBelow(int AtIndex)
{
	if (IsValidNode(AtIndex))
	{
		for (int ChildIdx : GetChildren(AtIndex))
		{
			DeleteNodesBelow(ChildIdx);
		}

		delete SimulationModuleTree[AtIndex].SimModule;
		SimulationModuleTree[AtIndex].SimModule = nullptr;
		SimulationModuleTree[AtIndex].Children.Empty();
		SimulationModuleTree[AtIndex].Parent = FSimModuleNode ::INVALID_IDX;

		FreeList.Push(AtIndex);

	}
}


void FSimModuleTree::GetRootNodes(TArray<int>& RootNodesOut)
{
	RootNodesOut.Empty();

	// never assume the root bone is always index 0
	for (int i = 0; i < SimulationModuleTree.Num(); i++)
	{
		if (SimulationModuleTree[i].SimModule != nullptr && SimulationModuleTree[i].Parent == FSimModuleNode::INVALID_IDX)
		{
			RootNodesOut.Add(i);
		}
	}
}

void FSimModuleTree::UpdateModuleVelocites(IPhysicsProxyBase* PhysicsProxy, Chaos::FPBDRigidParticleHandle* RootParticle, bool bWake)
{
	check(RootParticle);
	Chaos::EnsureIsInPhysicsThreadContext();

	if (PhysicsProxy && bWake && !SimulationModuleTree.IsEmpty())
	{
		if (PhysicsProxy->GetType() == EPhysicsProxyType::ClusterUnionProxy)
		{
			if (FClusterUnionPhysicsProxy* CUProxy = static_cast<FClusterUnionPhysicsProxy*>(PhysicsProxy))
			{
				if (bWake && !SimulationModuleTree.IsEmpty())
				{
					const FPhysicsObjectHandle& PhysicsObject = CUProxy->GetPhysicsObjectHandle();

					Chaos::FWritePhysicsObjectInterface_Internal WriteInterface = Chaos::FPhysicsObjectInternalInterface::GetWrite();
					WriteInterface.WakeUp({ &PhysicsObject, 1 });
				}
			}

		}

		if (PhysicsProxy->GetType() == EPhysicsProxyType::SingleParticleProxy)
		{
			if (FSingleParticlePhysicsProxy* ParticleProxy = static_cast<FSingleParticlePhysicsProxy*>(PhysicsProxy))
			{
				const FPhysicsObjectHandle& PhysicsObject = ParticleProxy->GetPhysicsObject();

				Chaos::FWritePhysicsObjectInterface_Internal WriteInterface = Chaos::FPhysicsObjectInternalInterface::GetWrite();
				WriteInterface.WakeUp({ &PhysicsObject, 1 });
			}

		}
	}

	if (RootParticle)
	{
		// capture the velocities at the start of each sim iteration
		for (int i = 0; i < SimulationModuleTree.Num(); i++)
		{
			if (ISimulationModuleBase* Module = SimulationModuleTree[i].SimModule)
			{
				const FTransform BodyTransform(RootParticle->GetR(), RootParticle->GetX());

				if (Module->IsBehaviourType(eSimModuleTypeFlags::Velocity))
				{
					Chaos::FPBDRigidParticleHandle* Particle = RootParticle;
					if (Particle)
					{
						const FTransform& OffsetTransform = Module->GetComponentTransform();
						FVector LocalPos = Module->GetParentRelativeTransform().GetLocation();
						FVector WorldLocation = BodyTransform.TransformPosition(LocalPos);
						const Chaos::FVec3 Arm = WorldLocation - Particle->GetX();

						// Take surface velocity into account
						Chaos::FPBDRigidParticleHandle* GroundParticle = nullptr;
						FVector GroundVel = FVector::ZeroVector;
						if (FWheelBaseInterface* Wheel = Module->Cast<FWheelBaseInterface>())
						{
							GroundParticle = Wheel->GetGroundBody();
							if (GroundParticle && GroundParticle != RootParticle)
							{
								const Chaos::FVec3 GroundArm = WorldLocation - GroundParticle->GetX();
								GroundVel = GroundParticle->GetV() - Chaos::FVec3::CrossProduct(GroundArm, GroundParticle->GetW());
							}
						}

						FVector WorldVelocity = Particle->GetV() - Chaos::FVec3::CrossProduct(Arm, Particle->GetW()) - GroundVel;
						FVector LocalVelocity = OffsetTransform.InverseTransformVector(BodyTransform.InverseTransformVector(WorldVelocity));

						FVector LocalAngular = BodyTransform.InverseTransformVector(Particle->GetW());
						LocalAngular = Module->GetClusteredTransform().InverseTransformVector(LocalAngular);

						Module->SetLocalLinearVelocity(LocalVelocity);
						Module->SetLocalAngularVelocity(LocalAngular);

						//Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(WorldLocation, WorldLocation + WorldVelocity, FColor::White, false, -1.f, 0, 5.f);

					}

				}
			}
		}
	}
}

void FSimModuleTree::UpdateVehicleState(Chaos::FPBDRigidParticleHandle* RootParticle)
{
	check(RootParticle);
	Chaos::EnsureIsInPhysicsThreadContext();

	if (RootParticle)
	{
		const FTransform BodyTransform(RootParticle->GetR(), RootParticle->GetX());

		VehicleState.Position =  RootParticle->GetX();
		VehicleState.Rotation = RootParticle->GetR();
		VehicleState.ForwardDir = BodyTransform.GetUnitAxis(EAxis::X);
		VehicleState.UpDir = BodyTransform.GetUnitAxis(EAxis::Z);
		VehicleState.RightDir = BodyTransform.GetUnitAxis(EAxis::Y);
		VehicleState.ForwardSpeedKmh = Chaos::CmSToKmH(FVector::DotProduct(RootParticle->GetV(), VehicleState.ForwardDir));
		VehicleState.AngularVelocityRad = RootParticle->GetW();
		VehicleState.LinearVelocity = RootParticle->GetV();
	}

}

void FSimModuleTree::GenerateReplicationStructure(Chaos::FModuleNetDataArray& NetData)
{
	SCOPE_CYCLE_COUNTER(STAT_ModularVehicle_GenerateReplicationStructure);

	if (!bModularVehicle_NetworkData_Enable)
	{
		return;
	}

	const TArray<FSimModuleNode>& Tree = SimulationModuleTree;
	NetData.Reset(Tree.Num());
	for (int Index = 0; Index < Tree.Num(); Index++)
	{
		if (ISimulationModuleBase* SimModule = Tree[Index].SimModule)
		{
			TSharedPtr<FModuleNetData>&& Data = SimModule->GenerateNetData(Index);
			// not all modules will have net replication data - nullptr is a valid response
			if (Data)
			{
				NetData.Emplace(Data);
			}
		}
	}
}

void FSimModuleTree::SetNetState(Chaos::FModuleNetDataArray& ModuleDatas)
{
	SCOPE_CYCLE_COUNTER(STAT_ModularVehicle_SetNetState);

	if (!bModularVehicle_NetworkData_Enable)
	{
		return;
	}

	//Always Regenerate Replication Structure to use unique data per ReplicationStructure
	{
		GenerateReplicationStructure(ModuleDatas);
	}

	for (TSharedPtr<FModuleNetData>& DataElement : ModuleDatas)
	{
		if (!SimulationModuleTree.IsEmpty() && DataElement && DataElement->SimArrayIndex < SimulationModuleTree.Num())
		{
			if (ISimulationModuleBase* SimModule = SimulationModuleTree[DataElement->SimArrayIndex].SimModule)
			{
				DataElement->FillNetState(SimModule);
			}
		}
	}
}

void FSimModuleTree::SetSimState(const Chaos::FModuleNetDataArray& ModuleDatas)
{
	SCOPE_CYCLE_COUNTER(STAT_ModularVehicle_SetSimState);

	if (!bModularVehicle_NetworkData_Enable)
	{
		return;
	}

	for (const TSharedPtr<FModuleNetData>& DataElement : ModuleDatas)
	{
		if (!SimulationModuleTree.IsEmpty() && DataElement && DataElement->SimArrayIndex < SimulationModuleTree.Num())
		{
			if (ISimulationModuleBase* SimModule = SimulationModuleTree[DataElement->SimArrayIndex].SimModule)
			{
				DataElement->FillSimState(SimModule);
			}
		}
	}
}


void FSimModuleTree::InterpolateState(const float LerpFactor, Chaos::FModuleNetDataArray& LerpDatas, const Chaos::FModuleNetDataArray& MinDatas, const Chaos::FModuleNetDataArray& MaxDatas)
{

}

} // namespace Chaos


#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION_SHIP
#endif

