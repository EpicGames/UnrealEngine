// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/SimulationModuleBase.h"
#include "SimModule/SimModuleTree.h"
#include "SimModule/DeferredForcesModular.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"

DEFINE_LOG_CATEGORY(LogSimulationModule);

namespace Chaos
{

void ISimulationModuleBase::AddLocalForceAtPosition(const FVector& Force, const FVector& Position, bool bAllowSubstepping, bool bIsLocalForce, bool bLevelSlope, const FColor& DebugColorIn)
{
	AppliedForce = Force;
	if (SimModuleTree)
	{
		SimModuleTree->AccessDeferredForces().Add(FDeferredForcesModular::FApplyForceAtPositionData(ComponentTransform, TransformIndex, ParticleIdx.Idx, Force, GetParentRelativeTransform().GetTranslation()+Position, bAllowSubstepping, bIsLocalForce, bLevelSlope, DebugColorIn));
	}
}

void ISimulationModuleBase::AddForceAtCOMPosition(const FVector& Force, const FVector& OffsetFromCOM /*= = FVector::ZeroVector*/, bool bAllowSubstepping /*= true*/, bool bLevelSlope /*= false*/, const FColor& DebugColorIn /*= FColor::Blue*/)
{
	AppliedForce = Force;
	if (SimModuleTree)
	{
		SimModuleTree->AccessDeferredForces().AddCOM(FDeferredForcesModular::FApplyForceAtPositionData(ComponentTransform, TransformIndex, ParticleIdx.Idx, Force, OffsetFromCOM, bAllowSubstepping, false, bLevelSlope, DebugColorIn));
	}
}

void ISimulationModuleBase::AddLocalForce(const FVector& Force, bool bAllowSubstepping, bool bIsLocalForce, bool bLevelSlope, const FColor& DebugColorIn)
{
	AppliedForce = Force;
	if (SimModuleTree)
	{
		SimModuleTree->AccessDeferredForces().Add(FDeferredForcesModular::FApplyForceAtPositionData(ComponentTransform, TransformIndex, ParticleIdx.Idx, Force, GetParentRelativeTransform().GetTranslation(), bAllowSubstepping, bIsLocalForce, bLevelSlope, DebugColorIn));
	}
}

void ISimulationModuleBase::AddLocalTorque(const FVector& Torque, bool bAllowSubstepping, bool bAccelChangeIn, const FColor& DebugColorIn)
{
	if (SimModuleTree)
	{
		SimModuleTree->AccessDeferredForces().Add(FDeferredForcesModular::FAddTorqueInRadiansData(ComponentTransform, TransformIndex, ParticleIdx.Idx, Torque, bAllowSubstepping, bAccelChangeIn, DebugColorIn));
	}
}

ISimulationModuleBase* ISimulationModuleBase::GetParent()
{
	return (SimModuleTree != nullptr) ? SimModuleTree->AccessSimModule(SimModuleTree->GetParent(SimTreeIndex)) : nullptr;
}

ISimulationModuleBase* ISimulationModuleBase::GetFirstChild()
{
	if (SimModuleTree)
	{
		const TSet<int32>& Children = SimModuleTree->GetChildren(SimTreeIndex);
		if (const auto It = Children.CreateConstIterator(); It)
		{ 
			return SimModuleTree->AccessSimModule(*It);
		}
	}
	return nullptr;
}

FVehicleBlackboard* ISimulationModuleBase::GetSimBlackboard()
{
	return SimModuleTree ? SimModuleTree->GetSimBlackboard() : nullptr;
}

FPBDRigidClusteredParticleHandle* ISimulationModuleBase::GetClusterParticle(FClusterUnionPhysicsProxy* Proxy)
{ 
	// TODO: This caching method fails if the component that owns the particle is destroyed
	//if (ParticleIdx.IsValid() && CachedParticle && (CachedParticle->UniqueIdx() == ParticleIdx))
	//{
	//	return CachedParticle;
	//}

	CachedParticle = nullptr;

	FPBDRigidsEvolutionGBF& Evolution = *static_cast<FPBDRigidsSolver*>(Proxy->GetSolver<FPBDRigidsSolver>())->GetEvolution();
	FClusterUnionManager& ClusterUnionManager = Evolution.GetRigidClustering().GetClusterUnionManager();
	const FClusterUnionIndex& CUI = Proxy->GetClusterUnionIndex();

	if (FClusterUnion* ClusterUnion = ClusterUnionManager.FindClusterUnion(CUI))
	{
		FPBDRigidClusteredParticleHandle* ClusterHandle = ClusterUnion->InternalCluster;
		TArray<FPBDRigidParticleHandle*> Particles = ClusterUnion->ChildParticles;

		if (FPBDRigidParticleHandle* Particle = GetParticleFromUniqueIndex(ParticleIdx.Idx, Particles))
		{
			CachedParticle = Particle->CastToClustered();
		}
	}

	return CachedParticle;
}

FPBDRigidParticleHandle* ISimulationModuleBase::GetParticleFromUniqueIndex(int32 ParticleUniqueIdx, TArray<FPBDRigidParticleHandle*>& Particles)
{
	for (FPBDRigidParticleHandle* Particle : Particles)
	{
		if (Particle && Particle->UniqueIdx().IsValid())
		{
			if (ParticleUniqueIdx == Particle->UniqueIdx().Idx)
			{
				return Particle;
			}
		}
	}

	return nullptr;
}

void ISimulationModuleBase::SetAnimationData(const FName& BoneNameIn, const FVector& AnimationOffsetIn, int AnimationSetupIndexIn)
{
	AnimationData.BoneName = BoneNameIn;
	AnimationData.AnimationSetupIndex = AnimationSetupIndexIn;
	AnimationOffset = AnimationOffsetIn;
}

bool ISimulationModuleBase::GetDebugString(FString& StringOut) const
{
	StringOut += FString::Format(TEXT("{0}: TreeIndex {1}, Enabled {2}, InCluster {3}, TFormIdx {4}, ")
		, { GetDebugName(), GetTreeIndex(), IsEnabled(), IsClustered(), GetTransformIndex() });

	return true; 
}

const FTransform& ISimulationModuleBase::GetParentRelativeTransform() const
{
	if (bClustered)
	{
		return GetClusteredTransform();
	}
	else
	{
		return GetIntactTransform();
	}
}


void FSimOutputData::FillOutputState(const ISimulationModuleBase* SimModule)
{
	if (SimModule)
	{
		AnimationData = SimModule->GetAnimationData();
		AnimationData.AnimationLocOffset += SimModule->GetAnimationOffset();
		ModuleGuid = SimModule->GetGuid();
	}

#if UE_BUILD_DEBUG
	if (SimModule)
	{	
		DebugString.Empty();
		SimModule->GetDebugString(DebugString);
	}
#endif
}

void FSimOutputData::Lerp(const FSimOutputData& InCurrent, const FSimOutputData& InNext, float Alpha)
{
	bEnabled = InNext.bEnabled;
	ModuleGuid = InNext.ModuleGuid;

	AnimationData.AnimationSetupIndex = InNext.AnimationData.AnimationSetupIndex;
	AnimationData.AnimFlags = InNext.AnimationData.AnimFlags;
	AnimationData.CombinedRotation = InNext.AnimationData.CombinedRotation;
	if (AnimationData.AnimFlags & EAnimationFlags::AnimatePosition)
	{
		AnimationData.AnimationLocOffset = FMath::Lerp(InCurrent.AnimationData.AnimationLocOffset, InNext.AnimationData.AnimationLocOffset, Alpha);
	}

	if (AnimationData.AnimFlags & EAnimationFlags::AnimateRotation)
	{
		AnimationData.AnimationRotOffset = FMath::Lerp(InCurrent.AnimationData.AnimationRotOffset, InNext.AnimationData.AnimationRotOffset, Alpha);
	}
}


} //namespace Chaos