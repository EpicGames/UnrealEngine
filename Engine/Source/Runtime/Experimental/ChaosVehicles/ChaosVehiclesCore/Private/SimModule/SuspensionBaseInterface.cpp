// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/SuspensionBaseInterface.h"

namespace Chaos
{

bool FSuspensionBaseInterface::IsBehaviourType(eSimModuleTypeFlags InType) const
{ 
	return (InType & Raycast); 
}

void FSuspensionBaseInterface::SetTargetPoint(const FSuspensionTargetPoint& InTargetPoint)
{
	TargetPoint = InTargetPoint;
}

} // namespace Chaos
