// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/ThrusterModule.h"
#include "SimModule/SimModuleTree.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{

	FThrusterSimModule::FThrusterSimModule(const FThrusterSettings& Settings)
		: TSimModuleSettings<FThrusterSettings>(Settings)
	{

	}

	void FThrusterSimModule::Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem)
	{
		SteerAngleDegrees = 0.0f;
		if (Setup().SteeringEnabled)
		{
			SteerAngleDegrees = Setup().SteeringEnabled ? Inputs.GetControls().GetMagnitude(TEXT("Steering")) * Setup().MaxSteeringAngle : 0.0f;
		}

		// applies continuous force
		float BoostEffect = Inputs.GetControls().GetMagnitude(BoostControlName) * Setup().BoostMultiplier;
		FVector Force = Setup().ForceAxis * Setup().MaxThrustForce * Inputs.GetControls().GetMagnitude(ThrottleControlName) * (1.0f + BoostEffect);
		FQuat Steer = FQuat(Setup().SteeringAxis, FMath::DegreesToRadians(SteerAngleDegrees) * Setup().SteeringForceEffect);
		AddLocalForceAtPosition(Steer.RotateVector(Force), Setup().ForceOffset, true, false, false, FColor::Magenta);
	}

	void FThrusterSimModule::Animate()
	{
		AnimationData.AnimFlags = EAnimationFlags::AnimateRotation;
		AnimationData.AnimationRotOffset.Yaw = SteerAngleDegrees;
	}

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
