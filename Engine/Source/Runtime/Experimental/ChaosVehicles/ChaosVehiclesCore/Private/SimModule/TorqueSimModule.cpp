// Copyright Epic Games, Inc. All Rights Reserved.


#include "SimModule/TorqueSimModule.h"
#include "SimModule/SimModuleTree.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{

void FTorqueSimModule::TransmitTorque(const FSimModuleTree& ModuleTree, float PushedTorque, float BrakeTorque, float GearingRatio /*= 1.0f*/, float ClutchSlip /* = 1.0f*/)
{
	if (FMath::Abs(GearingRatio) > SMALL_NUMBER && !ModuleTree.IsEmpty())
	{
		int ParentIndex = ModuleTree.GetParent(SimTreeIndex);

		const TSet<int>& Children = ModuleTree.GetChildren(SimTreeIndex);

		LoadTorque = 0; // clear torques prior to summation from child nodes
		BrakingTorque = BrakeTorque;
		float AverageChildVelocity = 0.0f;
		int NumVelocities = 0;
		for (int ChildIndex : Children)
		{
			// currently doesn't make sense to transmit from one wheel to next in hierarchy - only because hierarchy is 1 deep at present, this may change in future
			if (ModuleTree.GetSimModule(SimTreeIndex)->GetSimType() == ModuleTree.GetSimModule(ChildIndex)->GetSimType())
			{
				continue;
			}

			ISimulationModuleBase* Module = ModuleTree.AccessSimModule(ChildIndex);
			if (FTorqueSimModule* Interface = Module->Cast<FTorqueSimModule>())
			{
				// push the torque value down to the children
				// TODO: We are performing equal torque splitting here is that correct/desired?
				Interface->SetDriveTorque(PushedTorque * GearingRatio * ClutchSlip / Children.Num());

				// summing up braking value back from all children
				BrakingTorque += Interface->GetBrakingTorque() * ClutchSlip;

				// diff velocity 
				AverageChildVelocity += (Interface->AngularVelocity * GearingRatio);

				NumVelocities++;
			}
		}

		if (NumVelocities > 0)
		{
			float DiffVelocity = (AverageChildVelocity / NumVelocities) - AngularVelocity;
			AngularVelocity += DiffVelocity * ClutchSlip;
		}

		BrakingTorque /= FMath::Abs(GearingRatio);

	}
}

/**
* Integrate angular velocity using specified DeltaTime & Inertia value, Note the inertia should be the combined inertia of all the connected pieces otherwise things will rotate at different rates
*/

void FTorqueSimModule::IntegrateAngularVelocity(float DeltaTime, float Inertia, float MaxRotationVel)
{
	ensure(BrakingTorque >= 0.0f);

	// drive torque taken into account
	AngularVelocity += ((DriveTorque + LoadTorque) * DeltaTime) / Inertia;

	// braking resists velocity no matter what way we are spinning
	// also has check that we are not overshooting and starting to accelerating in the opposite direction
	if (AngularVelocity > 0.0f)
	{
		AngularVelocity -= (BrakingTorque * DeltaTime) / Inertia;
		if (AngularVelocity < 0.0f)
		{
			AngularVelocity = 0.0f;
		}
	}
	else
	{
		AngularVelocity += (BrakingTorque * DeltaTime) / Inertia;
		if (AngularVelocity > 0.0f)
		{
			AngularVelocity = 0.0f;
		}

	}
	
	AngularVelocity = FMath::Clamp(AngularVelocity, -MaxRotationVel, MaxRotationVel);

	// angluar position integration
	AngularPosition += AngularVelocity * DeltaTime;

	int ExcessRotations = (int)(AngularPosition / TWO_PI);
	AngularPosition -= ExcessRotations * TWO_PI;

	UE_LOGF(LogSimulationModule, Log, "%ls: DriveTorque %4.2f, BrakingTorque %4.2f, LoadTorque %4.2f, Speed %4.2f rad/sec, RPM %4.2f, AngularPosition %4.2f, Inertia %4.2f"
		, *GetDebugName(), DriveTorque, BrakingTorque, LoadTorque, AngularVelocity, GetRPM(), AngularPosition, Inertia);
}

void FTorqueSimModuleData::FillSimState(ISimulationModuleBase* SimModule)
{
	if (FTorqueSimModule* Sim = static_cast<FTorqueSimModule*>(SimModule))
	{
		Sim->AngularVelocity = AngularVelocity;
		Sim->AngularPosition = AngularPosition;
	}
}

void FTorqueSimModuleData::FillNetState(const ISimulationModuleBase* SimModule)
{
	if (const FTorqueSimModule* Sim = static_cast<const FTorqueSimModule*>(SimModule))
	{
		AngularVelocity = Sim->AngularVelocity;
		AngularPosition = Sim->AngularPosition;
	}
}

void FTorqueSimModuleData::Lerp(const float LerpFactor, const FModuleNetData& Min, const FModuleNetData& Max)
{
	const FTorqueSimModuleData& MinData = static_cast<const FTorqueSimModuleData&>(Min);
	const FTorqueSimModuleData& MaxData = static_cast<const FTorqueSimModuleData&>(Max);

	AngularVelocity = FMath::Lerp(MinData.AngularVelocity, MaxData.AngularVelocity, LerpFactor);
	AngularPosition = FMath::Lerp(MinData.AngularPosition, MaxData.AngularPosition, LerpFactor);
	DriveTorque = FMath::Lerp(MinData.DriveTorque, MaxData.DriveTorque, LerpFactor);
	LoadTorque = FMath::Lerp(MinData.LoadTorque, MaxData.LoadTorque, LerpFactor);
	BrakingTorque = FMath::Lerp(MinData.BrakingTorque, MaxData.BrakingTorque, LerpFactor);
}


#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
FString FTorqueSimModuleData::ToString() const
{
	return FString::Printf(TEXT("Module:%s AngularVelocity:%f AngularPosition:%f"),
		*DebugString, AngularVelocity, AngularPosition);
}
#endif

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
