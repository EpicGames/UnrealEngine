// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/TransmissionModule.h"
#include "SimModule/SimModuleTree.h"
#include "SimModule/SimulationModuleBase.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION_SHIP
#endif

namespace Chaos
{
	FTransmissionSimModule::FTransmissionSimModule(const FTransmissionSettings& Settings)
		: TSimModuleSettings<FTransmissionSettings>(Settings)
		, CurrentGear(1)
		, TargetGear(1)
		, CurrentGearChangeTime(0.f)
		, PreviousGear(1)
		, AllowedToChangeGear(true)
		, GearHysteresisTimer(0.f)
	{
	}

	void FTransmissionSimModule::Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem)
	{

		if (Setup().AutoReverse)
		{
			if (Inputs.GetControls().GetMagnitude(ReverseControlName))
			{
				// if reversing change to reverse gear if currently in a forwards gear
				if (TargetGear > 0)
				{
					TargetGear = -1;
				}
			}
			else
			{
				// if not revering change to forwards gear if currently in a reverse gear
				if (TargetGear < 0)
				{
					TargetGear = 1;
				}
			}
		}

		if (Setup().TransmissionType == FTransmissionSettings::ETransType::AutomaticType)
		{
			if (AllowedToChangeGear == false)
			{
				GearHysteresisTimer -= DeltaTime;
				if (GearHysteresisTimer <= 0.0f)
				{
					AllowedToChangeGear = true;
				}
			}
			// not currently changing gear, also don't want to change up because the wheels are spinning up due to having no load
			if (!IsCurrentlyChangingGear() && AllowedToChangeGear)
			{
				// in Automatic & currently in neutral and not currently changing gear then automatically change up to 1st
				if (CurrentGear == 0)
				{
					ChangeUp();
				}

				float EngineRPM = GetRPM();

				if (EngineRPM >= Setup().ChangeUpRPM)
				{
					if (CurrentGear > 0)
					{
						ChangeUp();
						AllowedToChangeGear = false;
						GearHysteresisTimer = Setup().GearHysteresisTime;
					}
					else
					{
						ChangeDown();
					}
				}
				else if (EngineRPM <= Setup().ChangeDownRPM && FMath::Abs(CurrentGear) > 1) // don't change down to neutral
				{
					if (CurrentGear > 0)
					{
						ChangeDown();
					}
					else
					{
						ChangeUp();
					}
				}
			}
		}
		else
		{
			if (Inputs.GetControls().GetMagnitude(ChangeUpControlName))
			{
				ChangeUp();
			}
			else if (Inputs.GetControls().GetMagnitude(ChangeDownControlName))
			{
				ChangeDown();
			}
		}

		if (CurrentGear != TargetGear)
		{
			CurrentGearChangeTime -= DeltaTime;
			if (CurrentGearChangeTime <= 0.f)
			{
				CurrentGearChangeTime = 0.f;
				CurrentGear = TargetGear;
			}
		}

		float GearRatio = GetGearRatio(CurrentGear);

		// if there IS a selected gear then connect the parent and child building blocks transmitting their torque
		if (FMath::Abs(GearRatio) > SMALL_NUMBER)
		{
			float BrakeTorque = 0.0f;
			TransmitTorque(VehicleModuleSystem, DriveTorque, BrakeTorque, GearRatio);
		}

	}

	float FTransmissionSimModule::GetGearRatio(int32 InGear) const
	{
		CorrectGearInputRange(InGear);

		if (InGear > 0) // a forwards gear
		{
			return Setup().ForwardRatios[InGear - 1] * Setup().FinalDriveRatio;
		}
		else if (InGear < 0) // a reverse gear
		{
			return -Setup().ReverseRatios[FMath::Abs(InGear) - 1] * Setup().FinalDriveRatio;
		}
		else
		{
			return 0.f; // neutral has no ratio
		}
	}

	void FTransmissionSimModule::SetGear(int32 InGear, bool Immediate /*= false*/)
	{
		CorrectGearInputRange(InGear);

		TargetGear = InGear;

		if (TargetGear != CurrentGear)
		{
			if (Immediate || Setup().GearChangeTime == 0.f)
			{
				CurrentGear = TargetGear;
			}
			else
			{
				CurrentGear = 0;	// go through neutral for GearChangeTime time period
				CurrentGearChangeTime = Setup().GearChangeTime;
			}
		}
	}

	bool FTransmissionSimModule::GetDebugString(FString& StringOut) const
	{
		FTorqueSimModule::GetDebugString(StringOut);
		StringOut += FString::Format(TEXT("CurrentGear {0}, Ratio {1}")
			, { CurrentGear, GetGearRatio(CurrentGear) });
		return true;
	}

	void FTransmissionSimModuleData::FillSimState(ISimulationModuleBase* SimModule)
	{
		if (FTransmissionSimModule* Sim = SimModule->Cast<FTransmissionSimModule>())
		{
			Sim->CurrentGear = CurrentGear;
			Sim->TargetGear = TargetGear;
			Sim->CurrentGearChangeTime = CurrentGearChangeTime;
		}
	}

	void FTransmissionSimModuleData::FillNetState(const ISimulationModuleBase* SimModule)
	{
		if (const FTransmissionSimModule* Sim = SimModule->Cast<const FTransmissionSimModule>())
		{
			CurrentGear = Sim->CurrentGear;
			TargetGear = Sim->TargetGear;
			CurrentGearChangeTime = Sim->CurrentGearChangeTime;
		}
	}

	void FTransmissionSimModuleData::Lerp(const float LerpFactor, const FModuleNetData& Min, const FModuleNetData& Max)
	{
		const FTransmissionSimModuleData& MinData = static_cast<const FTransmissionSimModuleData&>(Min);
		const FTransmissionSimModuleData& MaxData = static_cast<const FTransmissionSimModuleData&>(Max);

		CurrentGear = LerpFactor < 0.5 ? MinData.CurrentGear : MaxData.CurrentGear;
		TargetGear = LerpFactor < 0.5 ? MinData.TargetGear : MaxData.TargetGear;
		CurrentGearChangeTime = FMath::Lerp(MinData.CurrentGearChangeTime, MaxData.CurrentGearChangeTime, LerpFactor);
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FString FTransmissionSimModuleData::ToString() const
	{
		return FString::Printf(TEXT("Module:%s CurrentGear:%d TargetGear:%d CurrentGearChangeTime:%f"),
			*DebugString, CurrentGear, TargetGear, CurrentGearChangeTime);
	}
#endif

	void FTransmissionOutputData::FillOutputState(const ISimulationModuleBase* SimModule)
	{
		FSimOutputData::FillOutputState(SimModule);

		if (const FTransmissionSimModule* Sim = SimModule->Cast<const FTransmissionSimModule>())
		{
			CurrentGear = Sim->CurrentGear;
		
			if (Sim->CurrentGear != Sim->PreviousGear)
			{
				GearChangeEvents.Add(FGearChangeEvent(CurrentGear));
				Sim->PreviousGear = Sim->CurrentGear;
			}
		}
	}

	void FTransmissionOutputData::Lerp(const FSimOutputData& InCurrent, const FSimOutputData& InNext, float Alpha)
	{
		FSimOutputData::Lerp(InCurrent, InNext, Alpha);

		const FTransmissionOutputData& Current = static_cast<const FTransmissionOutputData&>(InCurrent);
		const FTransmissionOutputData& Next = static_cast<const FTransmissionOutputData&>(InNext);

		CurrentGear = Current.CurrentGear;
		GearChangeEvents = Next.GearChangeEvents;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FString FTransmissionOutputData::ToString()
	{
		return FString::Printf(TEXT("%s CurrentGear=%d")
			, *DebugString, CurrentGear);
	}
#endif

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION_SHIP
#endif
