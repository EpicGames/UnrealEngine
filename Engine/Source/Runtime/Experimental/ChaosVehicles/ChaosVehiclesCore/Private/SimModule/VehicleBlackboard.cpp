// Copyright Epic Games, Inc. All Rights Reserved.


#include "SimModule/VehicleBlackboard.h"


void FVehicleBlackboard::Invalidate(FName ObjName)
{
	ObjectsByName.Remove(ObjName);
}

void FVehicleBlackboard::Invalidate(EInvalidationReason Reason)
{
	switch (Reason)
	{
		default:
		case EInvalidationReason::FullReset:
			ObjectsByName.Empty();
			break;

		// TODO: Support other reasons
	}
}

