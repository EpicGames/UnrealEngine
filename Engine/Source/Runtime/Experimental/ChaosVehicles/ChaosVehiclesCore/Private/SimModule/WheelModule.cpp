// Copyright Epic Games, Inc. All Rights Reserved.

#include "SimModule/WheelModule.h"
#include "SimModule/SimModuleTree.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/Particle/ParticleUtilities.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION_SHIP
#endif

bool bModularVehicle_ApplyWheelForceToGroundBody = true;
FAutoConsoleVariableRef CVarModularVehicleApplyWheelForceToGroundBody(TEXT("p.ModularVehicle.ApplyWheelForceToGroundBody.Enabled"), bModularVehicle_ApplyWheelForceToGroundBody, TEXT("Enable/Disable aplication of wheel forces to the ground body."));

float bModularVehicle_WheelForceToGroundBodyScaling = 1.0f;
FAutoConsoleVariableRef CVarModularVehicleWheelForceToGroundBodyScaling(TEXT("p.ModularVehicle.WheelForceToGroundBodyScaling"), bModularVehicle_WheelForceToGroundBodyScaling, TEXT("Sacle froces applied to ground body from wheel."));

namespace Chaos
{
	FWheelSimModule::FWheelSimModule(const FWheelSettings& Settings)
		: TSimModuleSettings<FWheelSettings>(Settings)
		, BrakeTorque(0.0f)
		, ForceFromFriction(FVector::ZeroVector)
		, MassPerWheel(500.0f)
		, SteerAngleDegrees(0.0f)
		, bTouchingGround(false)
		, bPrevTouchingGround(false)
		, SlipAngle(0.0f)
	{
	}

	void FWheelSimModule::Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem)
	{
		float Re = Setup().Radius;
		float K = 0.4f;
		float TorqueScaling = 1.0f;
		float TractionControlAndABSScaling = 0.98f;	// how close to perfection is the system working

		float HandbrakeTorque = Setup().HandbrakeEnabled ? Inputs.GetControls().GetMagnitude(HandbrakeControlName) * Setup().HandbrakeTorque : 0.0f;
		SteerAngleDegrees = Setup().SteeringEnabled ? Inputs.GetControls().GetMagnitude(SteeringControlName) * Setup().MaxSteeringAngle : 0.0f;
		BrakeTorque = Inputs.GetControls().GetMagnitude(BrakeControlName) * Setup().MaxBrakeTorque + HandbrakeTorque;
		LoadTorque = 0.0f;
		ForceFromFriction = FVector::ZeroVector;
		float TorqueFromGroundInteraction = 0.0f;
		float AvailableGrip = 0.0f;

		// TODO: think about doing this properly, stops vehicles rolling around on their own too much
		// i.e. an auto handbrake feature
		if (Setup().AutoHandbrakeEnabled && LocalLinearVelocity.X < Setup().AutoHandbrakeVelocityThreshold && (Inputs.GetControls().GetMagnitude(BrakeControlName) < SMALL_NUMBER && Inputs.GetControls().GetMagnitude(ThrottleControlName) < SMALL_NUMBER))
		{
			BrakeTorque = Setup().HandbrakeTorque;
		}

		bTouchingGround = ForceIntoSurface > SMALL_NUMBER;

		if (bTouchingGround)
		{
			FRotator SteeringRotator(0.f, SteerAngleDegrees, 0.f);
			FVector Vel = SteeringRotator.UnrotateVector(LocalLinearVelocity);
			FVector LocalWheelVelocity = (Setup().Axis == EWheelAxis::X) ? FVector(Vel.X, Vel.Y, Vel.Z) : FVector(Vel.Y, Vel.X, Vel.Z); // Potential Axis Swap
			LocalWheelVelocity = Setup().ReverseDirection ? -LocalWheelVelocity : LocalWheelVelocity;

			float GroundAngularVelocity = LocalWheelVelocity.X / Re;
			float Delta = GroundAngularVelocity - AngularVelocity;
			TorqueFromGroundInteraction = Delta * Setup().WheelInertia / DeltaTime; // torque from wheels moving over terrain

			// X is longitudinal direction, Y is lateral
			SlipAngle = FVehicleUtility::CalculateSlipAngle(LocalWheelVelocity.Y, LocalWheelVelocity.X);

			float AppliedLinearDriveForce = DriveTorque / Re;
			float AppliedLinearBrakeForce = FMath::Abs(BrakeTorque) / Re;

			// Longitudinal multiplier now affecting both brake and steering equally
			AvailableGrip = ForceIntoSurface * SurfaceFriction * Setup().FrictionMultiplier;

			float FinalLongitudinalForce = 0.f;
			float FinalLateralForce = 0.f;

			// currently just letting the brake override the throttle
			bool Braking = (BrakeTorque > FMath::Abs(DriveTorque));
			bool WheelLocked = false;

			// are we actually touching the ground
			if (ForceIntoSurface > SMALL_NUMBER)
			{
				// ABS limiting brake force to match force from the grip available
				if (Setup().ABSEnabled && Braking && FMath::Abs(AppliedLinearBrakeForce) > AvailableGrip)
				{
					if ((Braking && Setup().ABSEnabled) || (!Braking && Setup().TractionControlEnabled))
					{
						float Sign = (AppliedLinearBrakeForce > 0.0f) ? 1.0f : -1.0f;
						AppliedLinearBrakeForce = AvailableGrip * TractionControlAndABSScaling * Sign;
					}
				}

				// Traction control limiting drive force to match force from grip available
				if (Setup().TractionControlEnabled && !Braking && FMath::Abs(AppliedLinearDriveForce) > AvailableGrip)
				{
					float Sign = (AppliedLinearDriveForce > 0.0f) ? 1.0f : -1.0f;
					AppliedLinearDriveForce = AvailableGrip * TractionControlAndABSScaling * Sign;
				}

				if (Braking)
				{
					// whether the velocity is +ve or -ve when we brake we are slowing the vehicle down
					// so force is opposing current direction of travel.
					float ForceRequiredToBringToStop = MassPerWheel * K * (LocalWheelVelocity.X) / DeltaTime;
					FinalLongitudinalForce = AppliedLinearBrakeForce;

					// check we are not applying more force than required so we end up overshooting 
					// and accelerating in the opposite direction
					FinalLongitudinalForce = FMath::Clamp(FinalLongitudinalForce, -FMath::Abs(ForceRequiredToBringToStop), FMath::Abs(ForceRequiredToBringToStop));

					// ensure the brake opposes current direction of travel
					if (LocalWheelVelocity.X > 0.0f)
					{
						FinalLongitudinalForce = -FinalLongitudinalForce;
					}

				}
				else
				{
					FinalLongitudinalForce = AppliedLinearDriveForce;
				}

				float ForceRequiredToBringToStop = (MassPerWheel * K * LocalWheelVelocity.Y) / DeltaTime;

				// use slip angle to generate a sideways force
				if (Setup().LateralSlipGraph.IsEmpty())
				{
					float AngleLimit = FMath::DegreesToRadians(Setup().SlipAngleLimit);
					float ClippedAngle = FMath::Clamp(SlipAngle, -AngleLimit, AngleLimit);
					FinalLateralForce = FMath::Abs(SlipAngle) * Setup().CorneringStiffness;
				}
				else
				{
					FinalLateralForce = Setup().LateralSlipGraph.EvaluateY(FMath::RadiansToDegrees(SlipAngle)) * Setup().LateralSlipGraphMultiplier;
				}

				if (FinalLateralForce > FMath::Abs(ForceRequiredToBringToStop))
				{
					FinalLateralForce = FMath::Abs(ForceRequiredToBringToStop);
				}
				if (LocalWheelVelocity.Y > 0.0f)
				{
					FinalLateralForce = -FinalLateralForce;
				}

				float LengthSquared = FinalLongitudinalForce * FinalLongitudinalForce + FinalLateralForce * FinalLateralForce;
				bool bClipping = false;
				if (LengthSquared > 0.05f)
				{
					float Length = FMath::Sqrt(LengthSquared);

					float Clip = (AvailableGrip) / Length;
					if (Clip < 1.0f)
					{
						if (Braking /*&& !bEngineBraking*/)
						{
							WheelLocked = true;
						}

						bClipping = true;
						FinalLongitudinalForce *= Clip;
						FinalLateralForce *= Clip;

						// make the resulting forces less than ideal since there is slippage
						FinalLongitudinalForce *= Setup().SlipModifier;
						FinalLateralForce *= Setup().SlipModifier;

					}
				}

			}

			// Potential Axis Swap
			if (Setup().Axis == EWheelAxis::X)
			{
				ForceFromFriction.X = FinalLongitudinalForce;
				ForceFromFriction.Y = FinalLateralForce;
			}
			else
			{
				check(Setup().Axis == EWheelAxis::Y);
				ForceFromFriction.Y = FinalLongitudinalForce;
				ForceFromFriction.X = FinalLateralForce;
			}

			// This code applies torque to the body the wheel is touching.
			FVector AppliedWorldForce = -ForceFromFriction;

			if (Setup().ReverseDirection)
			{
				ForceFromFriction = -ForceFromFriction;
			}

			AddLocalForceAtPosition(SteeringRotator.RotateVector(ForceFromFriction), Setup().ForceOffset);

			if (bModularVehicle_ApplyWheelForceToGroundBody && GroundBody && GroundBody->IsDynamic())
			{
				const Chaos::FVec3 WorldCOM = Chaos::FParticleUtilitiesXR::GetCoMWorldPosition(GroundBody);

				const FTransform VehicleTransform = Inputs.VehicleWorldTransform;
				FTransform WheelRelativeTransform = GetParentRelativeTransform();

				AppliedWorldForce = SteeringRotator.RotateVector(AppliedWorldForce);
				Chaos::FVec3 WorldForce = WheelRelativeTransform.TransformVector(AppliedWorldForce);
				WorldForce = VehicleTransform.TransformVector(WorldForce);
				WorldForce *= bModularVehicle_WheelForceToGroundBodyScaling;
				Chaos::FVec3 Arm = ImpactPoint - WorldCOM;
				Chaos::FVec3 WorldTorque = Chaos::FVec3::CrossProduct(Arm, WorldForce);

				GroundBody->AddForce(WorldForce);
				GroundBody->AddTorque(WorldTorque);

				//ImpactPoint.Z += 10.0f;
				//Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(ImpactPoint, ImpactPoint + WorldForce * 0.01f, FColor::Cyan, false, -1.f, 0, 2.f);
				//Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(ImpactPoint, ImpactPoint - Arm, FColor::Red, false, -1.f, 0, 2.f);
				//Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(WorldCOM, 20, 16, FColor::Red, false, -1.f, 0, 2.f);
			}
		}

		TransmitTorque(VehicleModuleSystem, DriveTorque, BrakeTorque);

		DriveTorque -= AvailableGrip;
		if (DriveTorque < 0.0f)
		{
			DriveTorque = 0.0f;
		}

		DriveTorque *= TorqueScaling;

		BrakingTorque -= AvailableGrip;
		if (BrakingTorque < 0.0f)
		{
			BrakingTorque = 0.0f;
		}

		BrakingTorque *= TorqueScaling;

		LoadTorque = TorqueFromGroundInteraction;

		IntegrateAngularVelocity(DeltaTime, Setup().WheelInertia, Setup().MaxRotationVel);
	}

	void FWheelSimModule::Animate()
	{
		float Direction = Setup().ReverseDirection ? -1.0f : 1.0f;
		FQuat Rot = (Setup().Axis == Chaos::EWheelAxis::Y) ? FQuat(FVector(1, 0, 0), -GetAngularPosition() * Direction) : FQuat(FVector(0, 1, 0), GetAngularPosition() * Direction);
		FQuat Steer = FQuat(FVector(0, 0, 1), FMath::DegreesToRadians(GetSteerAngleDegrees()));

		float AngularPositionDegrees = -AngularPosition * 180.0f / PI;
		float SteeringAngleDegrees = SteerAngleDegrees;

		AnimationData.AnimFlags = EAnimationFlags::AnimateRotation;
		AnimationData.AnimationRotOffset.Pitch = AngularPositionDegrees;
		AnimationData.AnimationRotOffset.Yaw = SteeringAngleDegrees;
		AnimationData.CombinedRotation = (Steer * Rot);

	}

	bool FWheelSimModule::GetDebugString(FString& StringOut) const
	{
		FTorqueSimModule::GetDebugString(StringOut);
		StringOut += FString::Format(TEXT("Drive {0}, Brake {1}, Load {2} RPM {3}  AngVel {4} LongitudinalForce {5}")
			, { DriveTorque, BrakingTorque, LoadTorque, GetRPM(), AngularVelocity, ForceFromFriction.X });
		return true;
	}


	inline void FWheelOutputData::FillOutputState(const ISimulationModuleBase* SimModule)
	{
		check(SimModule->IsSimType<class FWheelSimModule>());

		FSimOutputData::FillOutputState(SimModule);

		if (const FWheelSimModule* Sim = static_cast<const FWheelSimModule*>(SimModule))
		{
			bTouchingGround = Sim->bTouchingGround;
			ForceIntoSurface = Sim->ForceIntoSurface;
			SlipAngle = Sim->SlipAngle;
			RPM = Sim->GetRPM();
			AngularPositionDegrees = -Sim->AngularPosition * 180.0f / PI;
			SteeringAngleDegrees = Sim->SteerAngleDegrees;

			if (Sim->bPrevTouchingGround != Sim->bTouchingGround)
			{
				WheelTouchEvents.Add(FWheelTouchChangeEvent(bTouchingGround));
				Sim->bPrevTouchingGround = Sim->bTouchingGround;
			}
		}
	}

	void FWheelOutputData::Lerp(const FSimOutputData& InCurrent, const FSimOutputData& InNext, float Alpha)
	{
		FSimOutputData::Lerp(InCurrent, InNext, Alpha);

		const FWheelOutputData& Current = static_cast<const FWheelOutputData&>(InCurrent);
		const FWheelOutputData& Next = static_cast<const FWheelOutputData&>(InNext);

		bTouchingGround = Current.bTouchingGround;
		ForceIntoSurface = FMath::Lerp(Current.ForceIntoSurface, Next.ForceIntoSurface, Alpha);
		SlipAngle = FMath::Lerp(Current.SlipAngle, Next.SlipAngle, Alpha);
		RPM = FMath::Lerp(Current.RPM, Next.RPM, Alpha);

		WheelTouchEvents = Next.WheelTouchEvents;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FString FWheelOutputData::ToString()
	{
		return  FString::Printf(TEXT("%s, bTouchingGround=%d, ForceIntoSurface=%3.3f, SlipAngle=%3.3f, RPM=%3.3f")
			, *DebugString, bTouchingGround, ForceIntoSurface, SlipAngle, RPM);
	}
#endif

} // namespace Chaos


#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION_SHIP
#endif
