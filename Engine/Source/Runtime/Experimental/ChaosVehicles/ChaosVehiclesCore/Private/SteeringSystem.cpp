// Copyright Epic Games, Inc. All Rights Reserved.

#include "SteeringSystem.h"

namespace Chaos
{

	void FAckermannSim::CalculateAkermannAngle(float Input, float& OutSteerLeft, float& OutSteerRight)
	{
		FSteeringUtility::CalculateAkermannAngle(true, Input * SteerInputScaling, C2, R1, R2, OutSteerLeft, LeftRodPt, LeftPivot);
		FSteeringUtility::CalculateAkermannAngle(false, Input * SteerInputScaling, C2, R1, R2, OutSteerRight, RightRodPt, RightPivot);

		OutSteerLeft -= RestAngle;
		OutSteerRight -= RestAngle;
	}

	void FAckermannSim::GetRightHingeLocations(FVector2D& OutC1, FVector2D& OutP, FVector2D& OutC2)
	{
		OutC1 = RightRodPt;
		OutP = RightPivot;
		OutC2 = C2;
	}

	void FAckermannSim::GetLeftHingeLocations(FVector2D& OutC1, FVector2D& OutP, FVector2D& OutC2)
	{
		OutC1 = LeftRodPt;
		OutP = LeftPivot;
		OutC2 = C2;
		OutC1.X = -OutC1.X;
		OutP.X = -OutP.X;
		OutC2.X = -OutC2.X;
	}

	FAckermannSim::FAckermannSim(const FSimpleSteeringConfig* SetupIn) : TVehicleSystem<FSimpleSteeringConfig>(SetupIn)
	{
		float TrackWidth = Setup().TrackWidth;
		float WheelBase = Setup().WheelBase;

		float Beta = FSteeringUtility::CalculateBetaDegrees(TrackWidth, WheelBase);

		FSteeringUtility::CalcJointPositions(TrackWidth, Beta, TrackWidth/8.f, C1, R1, C2, R2);

		FSteeringUtility::CalculateAkermannAngle(false, 0.0f, C2, R1, R2, RestAngle, LeftRodPt, RightPivot);

		// #todo: Calculate this from Max Wheel Angle
		SteerInputScaling = 10.0f;

		float SteerLHS; float SteerRHS;
		CalculateAkermannAngle(1.0f, SteerLHS, SteerRHS);

		MaxAckermanAngle = FMath::Max(FMath::Abs(SteerLHS), FMath::Abs(SteerRHS));
	}

} // namespace Chaos
