// Copyright Epic Games, Inc. All Rights Reserved.

#include "SuspensionSystem.h"

namespace Chaos
{

	float FSimpleSuspensionSim::GetSpringLength()
	{
		if (Setup().SuspensionSmoothing)
		{
			// Trying smoothing the suspension movement out - looks Sooo much better when wheel traveling over pile of bricks
			// The digital up and down of the wheels is slowed/smoothed out
			float NewValue = SpringDisplacement - Setup().MaxLength;

			if (AveragingNum < Setup().SuspensionSmoothing)
			{
				AveragingNum++;
			}

			AveragingLength[AveragingCount++] = NewValue;

			if (AveragingCount >= Setup().SuspensionSmoothing)
			{
				AveragingCount = 0;
			}

			float Total = 0.0f;
			for (int i = 0; i < AveragingNum; i++)
			{
				Total += AveragingLength[i];
			}
			float Average = Total / AveragingNum;

			return Average;
		}
		else
		{
			return  (SpringDisplacement - Setup().MaxLength);
		}
	}

	void FSimpleSuspensionSim::Simulate(float DeltaTime)
	{
		float Damping = (SpringDisplacement < LastDisplacement) ? Setup().CompressionDamping : Setup().ReboundDamping;
		float SpringSpeed = (LastDisplacement - SpringDisplacement) / DeltaTime;

		const float StiffnessForce = SpringDisplacement * Setup().SpringRate;
		const float DampingForce = SpringSpeed * Damping;
		SuspensionForce = StiffnessForce - DampingForce;
		LastDisplacement = SpringDisplacement;
	}

} // namespace Chaos

