// Copyright Epic Games, Inc. All Rights Reserved.

#include "SuspensionUtility.h"

DEFINE_LOG_CATEGORY(LogVehicleUtility);

bool FSuspensionUtility::ComputeSprungMasses(const TArray<FVector>& MassSpringPositions, const float TotalMass, TArray<float>& OutSprungMasses)
{
	/*

	For a body which is supported by a collection of parallel springs,
	this method will compute a distribution of masses among the springs
	which minimizes the variance between them.

	This method assumes that spring positions are given relative to the
	center of mass of the body, and that gravity occurs in the local -Z
	direction.

	Different methods are used for 1, 2, and >= 3 numbers of springs.

	*/

	// Make sure we have enough space in the spring mass results array
	const uint32 Count = MassSpringPositions.Num();
	OutSprungMasses.Init(0, Count);

	// Check essential values
	if (!ensureMsgf(Count > 0, TEXT("Must have at least one spring to compute sprung masses.")))
	{
		return false;
	}

	if (!ensureMsgf(TotalMass > SMALL_NUMBER, TEXT("Total mass must be greater than zero to compute sprung masses.")))
	{
		return false;
	}

	// The cases of one or two springs are special snowflakes
	if (Count == 1)
	{
		OutSprungMasses[0] = TotalMass;
		return true;
	}
	else if (Count == 2)
	{
		/*

		For two springs, we project the CM (which is 0,0 since we're
		in the mass frame of the object) onto the line between the
		springs, and compute the ratio of the distances.

		For example in the graph below, if the springs are at points
		A and B, then the distances d0 and d1 will be computed.

			 d0      d1
		A---------p------B
				  |
				  c

		Then the masses m0 and m1 at A and B will be distributed according
		to the magnitudes of d0 and d1.

		m0 = m * d0 / (d0 + d1);
		m1 = m - m0;

		*/

		const float AX = MassSpringPositions[0].X;
		const float AY = MassSpringPositions[0].Y;
		const float DiffX = MassSpringPositions[1].X - AX;
		const float DiffY = MassSpringPositions[1].Y - AY;

		// If the springs are close together, just divide the mass in 2.
		const float DistSquared = (DiffX * DiffX) + (DiffY * DiffY);
		const float Dist = FMath::Sqrt(DistSquared);
		if (Dist <= SMALL_NUMBER)
		{
			OutSprungMasses[0] = OutSprungMasses[1] = TotalMass * .5f;
			return true;
		}

		// The springs are far enough apart, compute the distribution
		const float DistInv = 1.f / Dist;
		const float DirX = DiffX * DistInv;
		const float DirY = DiffY * DistInv;
		const float DirDotA = (DirX * AX) + (DirY * AY);
		OutSprungMasses[0] = -TotalMass * DirDotA * DistInv;
		OutSprungMasses[1] = TotalMass - OutSprungMasses[0];
		if (ensureMsgf(OutSprungMasses[0] >= 0.f, TEXT("Spring configuration is invalid! Please make sure the center of mass is located between the springs.")) &&
			ensureMsgf(OutSprungMasses[1] >= 0.f, TEXT("Spring configuration is invalid! Please make sure the center of mass is located between the springs.")))
		{
			return true;
		}
		return false;
	}

	/*

	In the case that we have N >= 3 springs, we solve a constrained minimization
	problem using Lagrange multipliers (https://en.wikipedia.org/wiki/Lagrange_multiplier).

	Our constraints express the following: the mass-weighted sum of the spring
	positions must equal the center of mass, and the sum of the sprung masses must equal
	the total mass of the body. Thus, we have three constraint equations:

	g0 = sum(x_i * m_i) = x_c * m
	g1 = sum(y_i * m_i) = y_c * m
	g2 = sum(m_i) = m

	When there are many springs, there are many ways in which we could distribute the
	masses among them. In order to get an even distribution, we want to minimize the
	function whose value is the total mass variance.

	f = sum((m_i - m_u)^2)

	where m_u is the average mass per spring, m / N.

	The equation which constrains the minimization of f will be the gradient of the sum
	of f and the constraints multiplied by scalars, set to zero.

	grad(f) + lambda0 grad(g0) + lambda1 grad(g1) + lambda2 grad(g2) = 0

	That is,

	2 m_i + lambda0 * x_i + lambda1 * y_i + lambda2 = 2m_u

	where 0 < i <= N. In combination with the constraint equations, we now have a system of
	N+3 equations and N+3 unknowns.

	[  0   0   0   x0  x1  x2  ... ] [ lambda0 ] = [ m x_c ]
	[  0   0   0   y0  y1  y2  ... ] [ lambda1 ]   [ m z_c ]
	[  0   0   0   1   1   1   ... ] [ lambda2 ]   [ m     ]
	[  x0  y0  1   2   0   0   ... ] [ m0      ]   [ 2 m_u ]
	[  x1  y1  1   0   2   0       ] [ m1      ]   [ 2 m_u ]
	[  x2  y2  1   0   0   2       ] [ m2      ]   [ 2 m_u ]
	[  .   .   .             .     ] [ .       ]   [ .     ]
	[  .   .   .               .   ] [ .       ]   [ .     ]
	[  .   .   .                 . ] [ .       ]   [ .     ]


	The linear (N+3)x(N+3) system which results takes a form which can be simplified
	using the Schur complement (https://en.wikipedia.org/wiki/Schur_complement) of the
	system matrix.

	[  0   Lt  ] [ lambda_vec ] = [ u ]
	[  L   2 I ] [ m_vec      ]   [ v ]

	where Lt = Transpose(L), lambda_vec = { lambda0, lambda1, lambda2 },
	m_vec = { m0, m1, m2, ... }, u = m { x_c, z_c, 1 }, and v = 2 m_u { 1, 1, 1, ...}.

	This system can be solved for lambda_vec and subsequently m_vec, which is our solution vector.

	lambda_vec = (Lt L)^-1 (Lt v - 2 u)
	m_vec = (v - L lambda_vec) / 2

	The matrix (Lt L) is a 3x3 matrix, and its inverse is the only one which must be found
	in order to finally solve for the masses.

	*/

	// Cache values we'll need later, and clear out the results array
	const float CountN = (float)Count;
	const float CountInverse = 1.f / CountN;
	const float AverageMass = TotalMass * CountInverse;
	float SumX = 0.f;
	float SumY = 0.f;
	float XDotX = 0.f;
	float YDotY = 0.f;
	float XDotY = 0.f;
	float B0 = 0.f;
	float B1 = 0.f;
	float B2 = 0.f;
	for (uint32 Index = 0; Index < Count; ++Index)
	{
		const float X = MassSpringPositions[Index].X;
		const float Y = MassSpringPositions[Index].Y;
		SumX += X;
		SumY += Y;
		XDotX += X * X;
		YDotY += Y * Y;
		XDotY += X * Y;
	}

	// Compute determinant of system matrix, in prep for inversion
	const float DetLL
		= (XDotX * YDotY * Count)
		+ (2.f * XDotY * SumX * SumY)
		- (YDotY * SumX * SumX)
		- (XDotX * SumY * SumY)
		- (XDotY * Count);

	// Make sure the matrix is invertible!
	if (!(DetLL > SMALL_NUMBER || DetLL < -SMALL_NUMBER))
	{
		UE_LOGF(LogVehicleUtility, Warning, "Spring configuration is invalid! Please make sure no two springs are at the same location.");
		return false;
	}

	// Compute the elements of the inverse matrix
	const float DetLLInv = 1.f / DetLL;
	const float InvLL00 = ((Count * YDotY) - (SumY * SumY)) * DetLLInv;
	const float InvLL01 = ((SumX * SumY) - (Count * XDotY)) * DetLLInv;
	const float InvLL02 = ((SumY * XDotY) - (SumX * YDotY)) * DetLLInv;
	const float InvLL10 = InvLL01; // Symmetry!
	const float InvLL11 = ((Count * XDotX) - (SumX * SumX)) * DetLLInv;
	const float InvLL12 = ((SumX * XDotY) - (SumY * XDotX)) * DetLLInv; // = InvLL21. Symmetry!
	const float InvLL20 = InvLL02; // Symmetry!
	const float InvLL21 = InvLL12; // Symmetry!
	const float InvLL22 = ((XDotX * YDotY) - (XDotY * XDotY)) * DetLLInv;

	// Compute the Lagrange multipliers
	const float LambdaB0 = 2.f * AverageMass * SumX;
	const float LambdaB1 = 2.f * AverageMass * SumY;
	const float LambdaB2 = (2.f * AverageMass * CountN) - (2.f * TotalMass);
	const float Lambda0 = (InvLL00 * LambdaB0) + (InvLL01 * LambdaB1) + (InvLL02 * LambdaB2);
	const float Lambda1 = (InvLL10 * LambdaB0) + (InvLL11 * LambdaB1) + (InvLL12 * LambdaB2);
	const float Lambda2 = (InvLL20 * LambdaB0) + (InvLL21 * LambdaB1) + (InvLL22 * LambdaB2);

	// Compute the masses
	for (uint32 Index = 0; Index < Count; ++Index)
	{
		const float X = MassSpringPositions[Index].X;
		const float Y = MassSpringPositions[Index].Y;
		const float LLambda = (X * Lambda0) + (Y * Lambda1) + Lambda2;
		OutSprungMasses[Index] = AverageMass - (0.5f * LLambda);
		if (OutSprungMasses[Index] < 0.f)
		{
			// #TODO: it is technically possible for say a trailer to have all of its wheels at the rear and none at the front 
			// hence violating this condition, need to think about handling this.
			UE_LOGF(LogVehicleUtility, Warning, "Spring configuration is invalid! Please make sure the center of mass is located inside the area covered by the springs.");
			return false;
		}
	}

	return true;
}

bool FSuspensionUtility::ComputeSprungMasses(const TArray<FVector>& LocalSpringPositions, const FVector& LocalCenterOfMass, const float TotalMass, TArray<float>& OutSprungMasses)
{
	// Compute support origin's in center of mass space
	const int32 SpringCount = LocalSpringPositions.Num();
	TArray<FVector> MassSpringPositions;
	MassSpringPositions.SetNum(SpringCount);
	for (int32 Index = 0; Index < SpringCount; ++Index)
	{
		MassSpringPositions[Index] = LocalSpringPositions[Index] - LocalCenterOfMass;
	}

	// Do the calculation
	return ComputeSprungMasses(MassSpringPositions, TotalMass, OutSprungMasses);
}
