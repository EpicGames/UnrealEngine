// Copyright Epic Games, Inc. All Rights Reserved.

#include "ThrustSystem.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{

	FSimpleThrustSim::FSimpleThrustSim(const FSimpleThrustConfig* SetupIn) : TVehicleSystem<FSimpleThrustConfig>(SetupIn)
		, ThrottlePosition(0.f)
		, ThrustForce(FVector::ZeroVector)
		, ThrustDirection(FVector::ZeroVector)
		, ThrusterStarted(false)
		, WorldVelocity(FVector::ZeroVector)
		, Pitch(0.f)
		, Roll(0.f)
		, Yaw(0.f)
	{

	}

	const FVector FSimpleThrustSim::GetThrustLocation() const
	{
		FVector Location = Setup().Offset;

		//if (Setup().Type == EThrustType::HelicopterRotor)
		//{
		//	Location += FVector(Pitch, -Roll, 0.f) * Setup().MaxControlAngle;
		//}

		return Location;
	}

	void FSimpleThrustSim::Simulate(float DeltaTime)
	{
		ThrustDirection = Setup().Axis;

		//if (Setup().Type != EThrustType::HelicopterRotor)
		//{
		//	FRotator SteeringRotator(Pitch, Yaw, Roll);
		//	ThrustDirection = SteeringRotator.RotateVector(ThrustDirection);
		//}
		ThrustForce = ThrustDirection * (ThrottlePosition * Setup().MaxThrustForce);
	}

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
