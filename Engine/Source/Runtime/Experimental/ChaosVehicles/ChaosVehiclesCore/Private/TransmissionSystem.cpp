// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransmissionSystem.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION_SHIP
#endif

namespace Chaos
{
	FSimpleTransmissionSim::FSimpleTransmissionSim(const FSimpleTransmissionConfig* SetupIn) : TVehicleSystem<FSimpleTransmissionConfig>(SetupIn)
		, CurrentGear(0)
		, TargetGear(0)
		, CurrentGearChangeTime(0.f)
		, EngineRPM(0)
		, AllowedToChangeGear(true)
	{

	}

	void FSimpleTransmissionSim::SetGear(int32 InGear, bool Immediate /*= false*/)
	{
		CorrectGearInputRange(InGear);

		TargetGear = InGear;

		if (TargetGear != CurrentGear)
		{
			if (Immediate || Setup().GearChangeTime == 0.f)
			{
				CurrentGear = TargetGear;
			}
			else
			{
				CurrentGear = 0;	// go through neutral for GearChangeTime time period
				CurrentGearChangeTime = Setup().GearChangeTime;
			}
		}
	}

	float FSimpleTransmissionSim::GetGearRatio(int32 InGear)
	{
		CorrectGearInputRange(InGear);

		if (InGear > 0) // a forwards gear
		{
			return Setup().ForwardRatios[InGear - 1] * Setup().FinalDriveRatio;
		}
		else if (InGear < 0) // a reverse gear
		{
			return -Setup().ReverseRatios[FMath::Abs(InGear) - 1] * Setup().FinalDriveRatio;
		}
		else
		{
			return 0.f; // neutral has no ratio
		}
	}

	void FSimpleTransmissionSim::Simulate(float DeltaTime)
	{
		if (Setup().TransmissionType == ETransmissionType::Automatic)
		{
			// not currently changing gear, also don't want to change up because the wheels are spinning up due to having no load
			if (!IsCurrentlyChangingGear() && AllowedToChangeGear)
			{
				if (EngineRPM >= Setup().ChangeUpRPM)
				{
					if (CurrentGear > 0)
					{
						ChangeUp();
					}
					else
					{
						ChangeDown();
					}
				}
				else if (EngineRPM <= Setup().ChangeDownRPM && FMath::Abs(CurrentGear) > 1) // don't change down to neutral
				{
					if (CurrentGear > 0)
					{
						ChangeDown();
					}
					else
					{
						ChangeUp();
					}
				}
			}
		}

		if (CurrentGear != TargetGear)
		{
			CurrentGearChangeTime -= DeltaTime;
			if (CurrentGearChangeTime <= 0.f)
			{
				CurrentGearChangeTime = 0.f;
				CurrentGear = TargetGear;
			}
		}
	}

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION_SHIP
#endif
