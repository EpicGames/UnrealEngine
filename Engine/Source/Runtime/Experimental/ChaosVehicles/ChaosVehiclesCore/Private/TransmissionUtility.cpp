// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransmissionUtility.h"

namespace Chaos
{

	bool FTransmissionUtility::IsWheelPowered(const EDifferentialType DifferentialType, const FSimpleWheelConfig::EAxleType AxleType, const bool EngineEnabled)
	{
		bool IsWheelPowered = false;

		if ((DifferentialType == EDifferentialType::UndefinedDrive) || (AxleType == FSimpleWheelConfig::UndefinedAxle))
		{
			IsWheelPowered = EngineEnabled;
		}
		else if (DifferentialType == EDifferentialType::AllWheelDrive)
		{
			IsWheelPowered = true;
		}
		else if (DifferentialType == EDifferentialType::FrontWheelDrive)
		{
			IsWheelPowered = (AxleType == FSimpleWheelConfig::EAxleType::Front);
		}
		else if (DifferentialType == EDifferentialType::RearWheelDrive)
		{
			IsWheelPowered = (AxleType == FSimpleWheelConfig::EAxleType::Rear);
		}
		else
		{
			IsWheelPowered = EngineEnabled;
		}

		return IsWheelPowered;
	}

	int FTransmissionUtility::GetNumWheelsOnAxle(FSimpleWheelConfig::EAxleType AxleType, const TArray<FSimpleWheelSim>& Wheels)
	{
		int Count = 0;

		for (int WheelIdx = 0; WheelIdx < Wheels.Num(); WheelIdx++)
		{
			const FSimpleWheelSim& PWheel = Wheels[WheelIdx];
			if (PWheel.Setup().AxleType == AxleType)
			{
				Count++;
			}
		}

		return Count;
	}

	int FTransmissionUtility::GetNumDrivenWheels(const TArray<FSimpleWheelSim>& Wheels)
	{
		int Count = 0;

		for (int WheelIdx = 0; WheelIdx < Wheels.Num(); WheelIdx++)
		{
			const FSimpleWheelSim& PWheel = Wheels[WheelIdx];
			if (PWheel.EngineEnabled)
			{
				Count++;
			}
		}

		return Count;
	}

	float FTransmissionUtility::GetTorqueRatioForWheel(const FSimpleDifferentialSim& PDifferential, const int WheelIndex, const TArray<FSimpleWheelSim>& Wheels)
	{
		float TorqueRatio = 0;

		const FSimpleWheelSim& PWheel = Wheels[WheelIndex];
		const EDifferentialType DifferentialType = PDifferential.Setup().DifferentialType;

		// no differential override specifed - just divide the torque by the number of powered wheels
		if ((DifferentialType == EDifferentialType::UndefinedDrive) || (PWheel.Setup().AxleType == FSimpleWheelConfig::EAxleType::UndefinedAxle))
		{
			if (PWheel.EngineEnabled)
			{
				TorqueRatio = 1.f / FTransmissionUtility::GetNumDrivenWheels(Wheels);
			}
			else
			{
				TorqueRatio = 0.f;
			}
		}
		else if (DifferentialType == EDifferentialType::AllWheelDrive)
		{
			if (Wheels.Num() > 0)
			{
				float SplitTorque = 1.0f;

				if (PWheel.Setup().AxleType == FSimpleWheelConfig::EAxleType::Front)
				{
					SplitTorque = (1.0f - PDifferential.FrontRearSplit);
				}
				else
				{
					SplitTorque = PDifferential.FrontRearSplit;
				}

				int NumWheelsOnAxle = FTransmissionUtility::GetNumWheelsOnAxle(PWheel.Setup().AxleType, Wheels);

				TorqueRatio = SplitTorque / (float)NumWheelsOnAxle;
			}
			else
			{
				TorqueRatio = 0;
			}
		}
		else if (DifferentialType == EDifferentialType::FrontWheelDrive
			&& PWheel.Setup().AxleType == FSimpleWheelConfig::EAxleType::Front)
		{
			TorqueRatio = 1.0f / FTransmissionUtility::GetNumWheelsOnAxle(FSimpleWheelConfig::EAxleType::Front, Wheels);
		}
		else if (DifferentialType == EDifferentialType::RearWheelDrive
			&& PWheel.Setup().AxleType == FSimpleWheelConfig::EAxleType::Rear)
		{
			TorqueRatio = 1.0f / FTransmissionUtility::GetNumWheelsOnAxle(FSimpleWheelConfig::EAxleType::Rear, Wheels);
		}

		return TorqueRatio;
	}




} // namespace Chaos