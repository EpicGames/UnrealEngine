// Copyright Epic Games, Inc. All Rights Reserved.

#include "VehicleUtility.h"
#include "HAL/PlatformTime.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{
	FTimeAndDistanceMeasure::FTimeAndDistanceMeasure(const FString& DescriptionIn, float InitialVelocityIn, float TargetVelocityIn, float TargetDistanceIn)
	{
		Description = DescriptionIn;
		InitialVelocityMPH = InitialVelocityIn;
		FinalTargetVelocityMPH = TargetVelocityIn;
		FinalTargetDistanceMiles = TargetDistanceIn;
		Reset();
	}

	void FTimeAndDistanceMeasure::Reset()
	{
		PreStartConditionsMet = false;
		StartConditionsMet = false;
		MeasurementComplete = false;
		VelocityResultMPH = 0.f;
		DistanceResultMiles = 0.f;
		TimeResultSeconds = 0.f;
	}

	void FTimeAndDistanceMeasure::Update(float DeltaTime, const FVector& CurrentLocation, float CurrentVelocity)
	{
		if (MeasurementComplete)
		{
			return;
		}

		float CurrentVelocityMPH = CmSToMPH(CurrentVelocity);
		float Tolerance = 0.1f;

		if (!PreStartConditionsMet)
		{
			if (FinalTargetDistanceMiles || FinalTargetVelocityMPH > InitialVelocityMPH)
			{
				if (CurrentVelocityMPH < (InitialVelocityMPH + Tolerance))
				{
					PreStartConditionsMet = true;
				}
			}
			else
			{
				if (CurrentVelocityMPH >= InitialVelocityMPH)
				{
					PreStartConditionsMet = true;
				}
			}

			return;
		}

		if (!StartConditionsMet)
		{
			if (FinalTargetDistanceMiles || FinalTargetVelocityMPH > InitialVelocityMPH)
			{
				if (CurrentVelocityMPH >= (InitialVelocityMPH + Tolerance))
				{
					StartConditionsMet = true;
					InitialTime = FPlatformTime::Seconds();
					InitialLocation = CurrentLocation;
				}
			}
			else
			{
				if (CurrentVelocityMPH < InitialVelocityMPH)
				{
					StartConditionsMet = true;
					InitialTime = FPlatformTime::Seconds();
					InitialLocation = CurrentLocation;
				}
			}

			return;
		}

		if (FinalTargetDistanceMiles)
		{
			// distance measure
			float DistanceTravelledMiles = CmToMiles((CurrentLocation - InitialLocation).Size());
			if (DistanceTravelledMiles > FinalTargetDistanceMiles)
			{
				MeasurementComplete = true;
			}
		}
		else if (FinalTargetVelocityMPH > InitialVelocityMPH)
		{
			// acceleration measure
			if (CurrentVelocityMPH >= FinalTargetVelocityMPH)
			{
				MeasurementComplete = true;
			}
		}
		else
		{
			// deceleration measure
			if (CurrentVelocityMPH <= FinalTargetVelocityMPH)
			{
				MeasurementComplete = true;
			}

		}

		if (MeasurementComplete)
		{
			VelocityResultMPH = CurrentVelocityMPH;
			DistanceResultMiles = CmToMiles((CurrentLocation - InitialLocation).Size());
			TimeResultSeconds = (float)(FPlatformTime::Seconds() - InitialTime);
		}
	}

	FString FTimeAndDistanceMeasure::ToString() const
	{
		return FString::Printf(TEXT("%s   Time: %1.2f Sec,   Dist: %1.3f Miles,   Speed: %3.1f MPH")
			, *Description, TimeResultSeconds, DistanceResultMiles, VelocityResultMPH);
	}

	FPerformanceMeasure::FPerformanceMeasure() : IsEnabledThisFrame(false)
	{
		FTimeAndDistanceMeasure ZeroToThirtyMPH("0 to 30 MPH", 0.f, 30.f, 0.f);
		PerformanceMeasure.Add(ZeroToThirtyMPH);

		FTimeAndDistanceMeasure ZeroToSixtyMPH("0 to 60 MPH", 0.f, 60.f, 0.f);
		PerformanceMeasure.Add(ZeroToSixtyMPH);

		FTimeAndDistanceMeasure QuarterMile("Quarter Mile Drag", 0.f, 0.f, 0.25f);
		PerformanceMeasure.Add(QuarterMile);

		FTimeAndDistanceMeasure ThirtyToZeroMPH("30 to 0 MPH", 30.f, 0.f, 0.f);
		PerformanceMeasure.Add(ThirtyToZeroMPH);

		FTimeAndDistanceMeasure SixtyToZeroMPH("60 to 0 MPH", 60.f, 0.f, 0.f);
		PerformanceMeasure.Add(SixtyToZeroMPH);
	}

	float FVehicleUtility::TurnRadiusFromThreePoints(const FVector& PtA, const FVector& PtB, const FVector& PtC)
	{
		float Radius = 0.f;

		FVector VecA = (PtB - PtA);
		FVector VecB = (PtC - PtB);
		FVector VecC = (PtA - PtC);

		float CosAlpha = VecB.CosineAngle2D(VecC);
		float Alpha = FMath::Acos(CosAlpha);

		float A = VecA.Size();
		float B = VecB.Size();
		float C = VecC.Size();

		float K = 0.5f * B * C * FMath::Sin(Alpha);

		if (K > SMALL_NUMBER)
		{
			Radius = (A * B * C) / (4.0f * K);
		}

		return Radius;
	}

	void FGraph::Add(const FVec2& Value)
	{
		Graph.Add(Value);

		if (Value.X < BoundsX.X)
		{
			BoundsX.X = Value.X;
		}
		if (Value.X > BoundsX.Y)
		{
			BoundsX.Y = Value.X;
		}
		if (Value.X < BoundsY.X)
		{
			BoundsY.X = Value.X;
		}
		if (Value.X > BoundsY.Y)
		{
			BoundsY.Y = Value.X;
		}
	}

	float FGraph::EvaluateY(float InX) const
	{
		float Result = 0.0f;

		if (!Graph.IsEmpty())
		{
			float RangeX = BoundsX.Y - BoundsX.X;
			float Step = (RangeX) / (Graph.Num() - 1);
			float Start = (InX - BoundsX.X) / Step;

			if (InX <= BoundsX.X || Start < 0.f)
			{
				return Graph[0].Y;
			}
			else if (InX >= BoundsX.Y)
			{
				return Graph[Graph.Num() - 1].Y;
			}
			
			int StartIndex = (int)Start;

			if (StartIndex >= Graph.Num() - 1)
			{
				return Graph[Graph.Num() - 1].Y;
			}

			float NormalisedRamp = (InX - Graph[StartIndex].X) / Step;

			Result = Graph[StartIndex].Y * (1.f - NormalisedRamp) + Graph[StartIndex + 1].Y * NormalisedRamp;
		}

		return Result;
	}

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
