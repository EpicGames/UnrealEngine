// Copyright Epic Games, Inc. All Rights Reserved.

#include "WheelSystem.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{
	FSimpleWheelSim::FSimpleWheelSim(const FSimpleWheelConfig* SetupIn) : TVehicleSystem<FSimpleWheelConfig>(SetupIn)
		, BrakeEnabled(SetupIn->BrakeEnabled)
		, HandbrakeEnabled(SetupIn->HandbrakeEnabled)
		, SteeringEnabled(SetupIn->SteeringEnabled)
		, EngineEnabled(SetupIn->EngineEnabled)
		, TractionControlEnabled(SetupIn->TractionControlEnabled)
		, ABSEnabled(SetupIn->ABSEnabled)
		, FrictionMultiplier(SetupIn->FrictionMultiplier)
		, LateralSlipGraphMultiplier(SetupIn->LateralSlipGraphMultiplier)
		, CorneringStiffness(SetupIn->CorneringStiffness)
		, MaxSteeringAngle(SetupIn->MaxSteeringAngle)
		, MaxBrakeTorque(SetupIn->MaxBrakeTorque)
		, HandbrakeTorque(SetupIn->HandbrakeTorque)
		, ExternalTorqueCombineMethod(SetupIn->ExternalTorqueCombineMethod)
		, Re(SetupIn->WheelRadius)
		, Omega(0.f)
		, Sx(0.f)
		, Inertia(0.5f * SetupIn->WheelMass * SetupIn->WheelRadius * SetupIn->WheelRadius)
		, DriveTorque(0.f)
		, BrakeTorque(0.f)
		, ForceIntoSurface(0.f)
		, GroundVelocityVector(FVector::ZeroVector)
		, AngularPosition(0.f)
		, SteeringAngle(0.f)
		, SurfaceFriction(1.f)
		, ExternalDriveTorque(0.0f)
		, ExternalBrakeTorque(0.0f)
		, ForceFromFriction(FVector::ZeroVector)
		, MassPerWheel(250.f)
		, SlipVelocity(0.f)
		, SlipAngle(0.f)
		, bInContact(false)
		, WheelIndex(0)
		, bEngineBraking(false)
		, Spin(0.f)
		, AvailableGrip(0.f)
		, InputForces(FVector::ZeroVector)
		, bClipping(false)
		, bABSActivated(false)
	{

	}

	void FSimpleWheelSim::Simulate(float DeltaTime)
	{
		float K = 0.4f;
		float TractionControlAndABSScaling = 0.98f;	// how close to perfection is the system working

		// X is longitudinal direction, Y is lateral
		SlipAngle = FVehicleUtility::CalculateSlipAngle(GroundVelocityVector.Y, GroundVelocityVector.X);

		if (ExternalTorqueCombineMethod == FSimpleWheelConfig::EExternalTorqueCombineMethod::Override)
		{ 
			// Set and forget or per frame torque override
			DriveTorque = ExternalDriveTorque;
			BrakeTorque = ExternalBrakeTorque;
		}
		else if (ExternalTorqueCombineMethod == FSimpleWheelConfig::EExternalTorqueCombineMethod::Additive)
		{
			// per frame torque added to engine torque
			DriveTorque += ExternalDriveTorque;
			BrakeTorque += ExternalBrakeTorque;
			ExternalDriveTorque = 0.0f;
			ExternalBrakeTorque = 0.0f;
		}

		// The physics system is mostly unit-less i.e. can work in meters or cm, however there are 
		// a couple of places where the results are wrong if Cm is used. This is one of them, the simulated radius
		// for torque must be real size to obtain the correct output values.
		AppliedLinearDriveForce = DriveTorque / Re;
		AppliedLinearBrakeForce = BrakeTorque / Re;

		// Longitudinal multiplier now affecting both brake and steering equally
		AvailableGrip = ForceIntoSurface * SurfaceFriction * FrictionMultiplier;

		float FinalLongitudinalForce = 0.f;
		float ExcessTorque = 0.0f;
		float FinalLateralForce = 0.f;
		float ABSGripThresholdSpeed = 5.0f;

		// currently just letting the brake override the throttle
		bool Braking = BrakeTorque > FMath::Abs(DriveTorque);
		bool WheelLocked = false;
		float SlipOmega = 0.0f;
		bABSActivated = 0.0f;

		// are we actually touching the ground
		if (ForceIntoSurface > SMALL_NUMBER)
		{
			// ABS limiting brake force to match force from the grip available
			if (ABSEnabled && Braking && FMath::Abs(AppliedLinearBrakeForce) > AvailableGrip)
			{
				if ((Braking && ABSEnabled) || (!Braking && TractionControlEnabled))
				{
					float Sign = (AppliedLinearBrakeForce > 0.0f) ? 1.0f : -1.0f;
					AppliedLinearBrakeForce = AvailableGrip * TractionControlAndABSScaling * Sign;
					if (FMath::Abs(GroundVelocityVector.X) > ABSGripThresholdSpeed)
					{
						bABSActivated = true;
					}
				}
			}

			if (FMath::Abs(AppliedLinearDriveForce) > AvailableGrip)
			{
				float SignTorque = AppliedLinearDriveForce < 0.0f ? -1.0f : 1.0f;
				ExcessTorque = (FMath::Abs(AppliedLinearDriveForce) - AvailableGrip) * Re * SignTorque;

				// Traction control limiting drive force to match force from grip available
				if (TractionControlEnabled && !Braking)
				{
					float Sign = (AppliedLinearDriveForce > 0.0f) ? 1.0f : -1.0f;
					AppliedLinearDriveForce = AvailableGrip * TractionControlAndABSScaling * Sign;
				}
			}

			if (Braking)
			{
				// whether the velocity is +ve or -ve when we brake we are slowing the vehicle down
				// so force is opposing current direction of travel.
				float ForceRequiredToBringToStop = MassPerWheel * K * (GroundVelocityVector.X) / DeltaTime;
				FinalLongitudinalForce = AppliedLinearBrakeForce;

				// check we are not applying more force than required so we end up overshooting 
				// and accelerating in the opposite direction
				FinalLongitudinalForce = FMath::Clamp(FinalLongitudinalForce, -FMath::Abs(ForceRequiredToBringToStop), FMath::Abs(ForceRequiredToBringToStop));

				// ensure the brake opposes current direction of travel
				if (GroundVelocityVector.X > 0.0f)
				{
					FinalLongitudinalForce = -FinalLongitudinalForce;
				}

			}
			else
			{
				FinalLongitudinalForce = AppliedLinearDriveForce;
			}

			float ForceRequiredToBringToStop = -(MassPerWheel * K * GroundVelocityVector.Y) / DeltaTime;

			// use slip angle to generate a sideways force
			if (Setup().LateralSlipGraph.IsEmpty())
			{
				float AngleLimit = FMath::DegreesToRadians(8.0f);
				float ClippedAngle = FMath::Clamp(SlipAngle, -AngleLimit, AngleLimit);
				FinalLateralForce = FMath::Abs(SlipAngle) * CorneringStiffness;
			}
			else
			{ 
				FinalLateralForce = Setup().LateralSlipGraph.EvaluateY(FMath::RadiansToDegrees(SlipAngle)) * LateralSlipGraphMultiplier;
			}

			if (FinalLateralForce > FMath::Abs(ForceRequiredToBringToStop))
			{
				FinalLateralForce = FMath::Abs(ForceRequiredToBringToStop);
			}
			if (GroundVelocityVector.Y > 0.0f)
			{
				FinalLateralForce = -FinalLateralForce;
			}

			// Friction circle
			InputForces.X = FinalLongitudinalForce;
			InputForces.Y = FinalLateralForce;

			float LengthSquared = FinalLongitudinalForce * FinalLongitudinalForce + FinalLateralForce * FinalLateralForce;
			bClipping = false;
			if (LengthSquared > 0.05f)
			{
				float Length = FMath::Sqrt(LengthSquared);

				float Clip = (AvailableGrip) / Length;
				if (Clip < 1.0f)
				{
					if (Braking && !bEngineBraking)
					{
						WheelLocked = true;
					}

					bClipping = true;
					FinalLongitudinalForce *= Clip;
					FinalLateralForce *= Clip;

					FinalLongitudinalForce *= Setup().SideSlipModifier;
					FinalLateralForce *= Setup().SideSlipModifier;
				}
			}
		}
		else
		{
			// only apply more spin torque if we haven't reached our max spin rotation
			if (FMath::Abs(Omega) < Setup().MaxSpinRotation)
			{
				ExcessTorque = DriveTorque;
			}
		}

		SlipOmega = ExcessTorque / Inertia * DeltaTime;
		SlipOmega = FMath::Clamp(SlipOmega, -Setup().MaxSpinRotation, Setup().MaxSpinRotation);

		if (WheelLocked)
		{
			Omega *= 0.9f; // velocity reduced quickly
		}
		else
		{ 
			if (bInContact)
			{
				float GroundOmega = GroundVelocityVector.X / FMath::Max(Re, KINDA_SMALL_NUMBER);
				Omega += ((GroundOmega - Omega + SlipOmega));
			}
			else
			{
				Omega += SlipOmega;
			}
		}

		// Wheel angular position integrated
		AngularPosition += Omega * DeltaTime;

		// Handle wrap around of wheel position
		float IntegerPart = 0.f;
		AngularPosition = FMath::Modf(AngularPosition / TWO_PI, &IntegerPart) * TWO_PI;

		if (!bInContact)
		{
			ForceFromFriction = FVector::ZeroVector;
		}
		else
		{
			ForceFromFriction.X = FinalLongitudinalForce;
			ForceFromFriction.Y = FinalLateralForce;
		}

		return;
	}



	FAxleSim::FAxleSim() : TVehicleSystem<FAxleConfig>(&Setup)
	{
	}



} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
