// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "VehicleSystemTemplate.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{
	/*
	*	Simple Aerodynamics - calculates drag and down-force/lift-force for a given speed
	*
	*	#todo: Add options for drafting/Slipstreaming effect
	*	#todo: Proper defaults
	*/

	struct CHAOSVEHICLESCORE_API FSimpleAerodynamicsConfig
	{
		FSimpleAerodynamicsConfig()
			: AreaMetresSquared(2.0f)
			, DragCoefficient(0.1f)
			, DownforceCoefficient(0.1f)
		{

		}

	//	FVector Offet;				// local offset relative to CM
		float AreaMetresSquared;	// [meters squared]
		float DragCoefficient;		// always positive
		float DownforceCoefficient;	// positive for downforce, negative lift
								
	};


	class CHAOSVEHICLESCORE_API FSimpleAerodynamicsSim : public TVehicleSystem<FSimpleAerodynamicsConfig>
	{
	public:
		FSimpleAerodynamicsSim(const FSimpleAerodynamicsConfig* SetupIn);

		/** set the density of the medium through which you are traveling Air/Water, etc */
		void SetDensityOfMedium(float DensityIn)
		{
			DensityOfMedium = DensityIn;
		}

		void SetDragCoefficient(float InCoeffient)
		{
			DragCoefficient = InCoeffient;
			EffectiveDragConstant = 0.5f * Setup().AreaMetresSquared * DragCoefficient;
		}

		void SetDownforceCoefficient(float InCoeffient)
		{
			DownforceCoefficient = InCoeffient;
			EffectiveLiftConstant = 0.5f * Setup().AreaMetresSquared * DownforceCoefficient;
		}

		/** get the drag force generated at the given velocity */
		float GetDragForceFromVelocity(float VelocityIn)
		{
			return -EffectiveDragConstant * DensityOfMedium * VelocityIn * VelocityIn;
		}

		/** get the lift/down-force generated at the given velocity */
		float GetLiftForceFromVelocity(float VelocityIn)
		{
			return -EffectiveLiftConstant * DensityOfMedium * VelocityIn * VelocityIn;
		}

		/** Get the drag and down forces combined in a 3D vector, drag on X-axis, down-force on Z-axis*/
		FVector GetCombinedForces(float VelocityIn);

	protected:
		float DownforceCoefficient;
		float DragCoefficient;
		float DensityOfMedium;
		float EffectiveDragConstant;
		float EffectiveLiftConstant;

	};

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
