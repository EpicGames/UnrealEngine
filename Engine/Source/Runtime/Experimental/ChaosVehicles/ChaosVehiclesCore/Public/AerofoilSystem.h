// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "VehicleSystemTemplate.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{
	enum class EAerofoilType : uint8
	{
		Fixed = 0,
		Wing,
		Rudder,
		Elevator
	};

	struct CHAOSVEHICLESCORE_API FAerofoilConfig
	{
		FAerofoilConfig()
			: Offset(FVector(0.f, 0.f, 0.0f))
			, UpAxis(FVector(0.f, 0.f, 1.f))
			, Area(5.0f)
			, Camber(3.0f)
			, MaxControlAngle(1.f)
			, StallAngle(16.0f)
			, MaxCeiling(1E30f)
			, MinCeiling(-1E30f)
			, Type(EAerofoilType::Fixed)
			, LiftMultiplier(1.0f)
			, DragMultiplier(1.0f)
		{

		}

		FVector Offset;
		FVector UpAxis;
		float Area;
		float Camber;
		float MaxControlAngle;
		float StallAngle;

		float MaxCeiling;
		float MinCeiling;

		EAerofoilType Type;
		float LiftMultiplier;
		float DragMultiplier;
	};

	class CHAOSVEHICLESCORE_API FAerofoil : public TVehicleSystem<FAerofoilConfig>
	{
	public:
		FAerofoil()
		{}

		FAerofoil(const FAerofoilConfig* SetupIn);

		/** Set a debug Id so we can identify an individual aerofoil */
		void SetAerofoilId(int Id)
		{
			AerofoilId = Id;
		}

		void SetControlSurface(float CtrlSurfaceInput)
		{
			ControlSurfaceAngle = CtrlSurfaceInput * Setup().MaxControlAngle;
		}

		void SetDensityOfMedium(float InDensity)
		{
			CurrentAirDensity = InDensity;
		}

		FVector GetAxis()
		{
			return Setup().UpAxis;
		}

		FVector GetOffset()
		{
			return Setup().Offset;
		}

		FVector GetCenterOfLiftOffset();

		// returns the combined force of lift and drag at an aerofoil in world coordinates
		// for direct application to the aircrafts rigid body.
		FVector GetForce(FTransform& BodyTransform, const FVector& v, float Altitude, float DeltaTime);

		/**
		 * Dynamic air pressure = 0.5 * AirDensity * Vsqr
		 * This function reduces the dynamic pressure in a linear fashion with altitude between 
		 * MinCeiling & MaxCeiling in order to limit the aircrafts altitude with a 'natural feel'
		 * without having a hard limit
		 */
		float CalcDynamicPressure(float VelocitySqr, float InAltitude);

		/**  Center of lift moves fore/aft based on current AngleOfAttack */
		float CalcCentreOfLift();

		/** Returns drag coefficient for the current angle of attack of the aerofoil surface */
		float CalcDragCoefficient(float InAngleOfAttack, float InControlSurfaceAngle);

		/**
		 * Returns lift coefficient for the current angle of attack of the aerofoil surface
		 * Cheating by making control surface part of entire aerofoil movement
		 */
		float CalcLiftCoefficient(float InAngleOfAttack, float InControlSurfaceAngle);

		/** Angle of attack is the angle between the aerofoil and the airflow vector */
		float CalcAngleOfAttackDegrees(const FVector& UpAxis, const FVector& InAirflowVector);

	private:
		float CurrentAirDensity;
		float AngleOfAttack;
		float ControlSurfaceAngle;
		FVector AirflowNormal;
		int AerofoilId;
	};

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif