// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VehicleSystemTemplate.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{

	struct CHAOSVEHICLESCORE_API FTorqueControlConfig
	{
		/** Torque Control Enabled */
		bool Enabled;

		/** Yaw Torque Scaling */
		float YawTorqueScaling;

		float YawFromSteering;

		float YawFromRollTorqueScaling;

		/** Pitch Torque Scaling */
		float PitchTorqueScaling;

		/** Roll Torque Scaling */
		float RollTorqueScaling;

		float RollFromSteering;

		/** Rotation damping */
		float RotationDamping;

		void InitDefaults()
		{
			Enabled = false;
			YawTorqueScaling = 0.0f;
			YawFromSteering = 0.0f;
			YawFromRollTorqueScaling = 0.0f;
			PitchTorqueScaling = 0.0f;
			RollTorqueScaling = 0.0f;
			RollFromSteering = 0.0f;
			RotationDamping = 0.02f;
		}
	};

	struct CHAOSVEHICLESCORE_API FTargetRotationControlConfig
	{
		/** Rotation Control Enabled */
		bool Enabled;

		bool bRollVsSpeedEnabled;

		float RollControlScaling;

		float RollMaxAngle;

		float PitchControlScaling;

		float PitchMaxAngle;

		/** Rotation stiffness */
		float RotationStiffness;

		/** Rotation damping */
		float RotationDamping;

		/** Rotation mac accel */
		float MaxAccel;

		float AutoCentreRollStrength;

		float AutoCentrePitchStrength;

		float AutoCentreYawStrength;


		void InitDefaults()
		{
			Enabled = false;

			bRollVsSpeedEnabled = false;

			RollControlScaling = 0.f;
			RollMaxAngle = 0.f;
			PitchControlScaling = 0.f;
			PitchMaxAngle = 0.f;

			RotationStiffness = 0.f;
			RotationDamping = 0.2f;
			MaxAccel = 0.f;

			AutoCentreRollStrength = 0.f;
			AutoCentrePitchStrength = 0.f;
			AutoCentreYawStrength = 0.f;
		}
	};

	struct CHAOSVEHICLESCORE_API FStabilizeControlConfig
	{

		/** Torque Control Enabled */
		bool Enabled;

		/** Yaw Torque Scaling */
		float AltitudeHoldZ;

		float PositionHoldXY;

		void InitDefaults()
		{
			Enabled = false;
			AltitudeHoldZ = 4.0f;
			PositionHoldXY = 8.0f;
		}
	};

	struct CHAOSVEHICLESCORE_API FArcadeControlConfig
	{
		FTorqueControlConfig TorqueControl;

		/** Arcade style direct control of vehicle rotation via torque force */
		FTargetRotationControlConfig TargetRotationControl;

		/** Arcade style control of vehicle */
		FStabilizeControlConfig StabilizeControl;
	};


	class CHAOSVEHICLESCORE_API FTorqueControlSim : public TVehicleSystem<FTorqueControlConfig>
	{
	public:
		FTorqueControlSim()
		{}

		FTorqueControlSim(const FTorqueControlConfig* SetupIn);

	};


	class CHAOSVEHICLESCORE_API FTargetRotationControlSim : public TVehicleSystem<FTargetRotationControlConfig>
	{
	public:
		FTargetRotationControlSim()
		{}

		FTargetRotationControlSim(const FTargetRotationControlConfig* SetupIn);

	};


	class CHAOSVEHICLESCORE_API FStabilizeControlSim : public TVehicleSystem<FStabilizeControlConfig>
	{
	public:
		FStabilizeControlSim()
		{}

		FStabilizeControlSim(const FStabilizeControlConfig* SetupIn);
	};


} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif