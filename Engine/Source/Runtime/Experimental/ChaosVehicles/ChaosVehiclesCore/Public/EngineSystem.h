// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/UnrealMathSSE.h"
#include "VehicleSystemTemplate.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif


namespace Chaos
{

	/*
	 *	Simple Normally Aspirated Engine
	 *  Output defined by a single torque curve over the rev range
	 *	No turbo/ turbo lag
	 *
	 * #todo: proper default values
	 * #todo: replace hardcoded graph with curve data
	 */

	struct CHAOSVEHICLESCORE_API FSimpleEngineConfig
	{
		FSimpleEngineConfig()
			: MaxTorque(0.f)
			, MaxRPM(6000)
			, EngineIdleRPM(1200)
			, EngineBrakeEffect(0.2f)
			, EngineRevUpMOI(1.0f)
			, EngineRevDownRate(1.0f)
		{
			// #todo: do we want to provide a default torque curve, maybe even just a straight line

		}

		// #todo: want something like UCurveFloat / FRuntimeFloatCurve / FTorqueCurveEditor
		FNormalisedGraph TorqueCurve;
		float MaxTorque;			// [N.m] The peak torque Y value in the normalized torque graph
		uint16 MaxRPM;				// [RPM] The absolute maximum RPM the engine can theoretically reach (last X value in the normalized torque graph)
		uint16 EngineIdleRPM; 		// [RPM] The RPM at which the throttle sits when the car is not moving			
		float EngineBrakeEffect;	// [0..1] How much the engine slows the vehicle when the throttle is released

		float EngineRevUpMOI;		// Affects speed at which engine RPM increases
		float EngineRevDownRate;		// Affects speed at which engine RPM decreases

	};

	class CHAOSVEHICLESCORE_API FSimpleEngineSim : public TVehicleSystem<FSimpleEngineConfig>
	{
	public:

		FSimpleEngineSim(const FSimpleEngineConfig* StaticDataIn);

		/** Start the engine */
		void StartEngine()
		{
			EngineStarted = true;

			// #todo: event for audio? perhaps a delay before you can drive off
		}

		/** Stop the engine */
		void StopEngine()
		{
			EngineStarted = false;

			// #todo: event for audio? perhaps a delay before you can drive off
		}

		/** Pass in the throttle position to the engine */
		void SetThrottle(float InThrottle)
		{
			FVehicleUtility::ClampNormalRange(InThrottle);
			ThrottlePosition = InThrottle;
		}

		/** When the wheels are in contact with the ground and clutch engaged then the load 
		 * on the engine from the wheels determines the engine speed. With no clutch simulation
		 * just setting the engine RPM directly to match the wheel speed.
		 */
		void SetEngineRPM(bool FreeRunningIn, float InEngineRPM)
		{
			FreeRunning = FreeRunningIn;
			if (!FreeRunning)
			{
				TargetSpeed = RPMToOmega(FMath::Clamp(FMath::Abs(InEngineRPM), (float)Setup().EngineIdleRPM, (float)Setup().MaxRPM));
			}
		}

		void SetMaxTorque(float InTorque)
		{
			MaxTorque = InTorque;
		}

		float GetEngineTorque()
		{
			if (EngineStarted)
			{
				return ThrottlePosition * GetTorqueFromRPM();
			}

			return 0.f;
		}

		float GetEngineRevRate()
		{
			return RevRate;
		}

		float GetTorqueFromRPM(bool LimitToIdle = true)
		{
			return GetTorqueFromRPM(CurrentRPM);
		}

		/* get torque value from torque curve based on input RPM */
		float GetTorqueFromRPM(float RPM, bool LimitToIdle = true);


		/** get the Engine speed in Revolutions Per Minute */
		float GetEngineRPM() const
		{
			if (EngineStarted)
			{
				return CurrentRPM;
			}

			return 0.f;
		}

		/** get the Engine angular velocity */
		float GetEngineOmega() const
		{
			return Omega;
		}

		/** Set the Engine angular velocity */
		void SetEngineOmega( const float EngineOmega)
		{
			Omega = EngineOmega;
		}

		/** 
		 * Simulate - NOP at the moment
		 */
		void Simulate(float DeltaTime);

	protected:
		float MaxTorque;		// [N.m] The peak torque Y value in the normalized torque graph
		float ThrottlePosition; // [0..1 Normalized position]
		float TargetSpeed;		// target RPM
		float CurrentRPM;		// current RPM
		float DriveTorque;		// current torque [N.m]

		float EngineIdleSpeed;
		float MaxEngineSpeed;
		bool EngineStarted;		// is the engine turned off or has it been started
		bool FreeRunning;		// is engine in neutral with no load from the wheels/transmission

		float Omega;			// engine speed
		float RevRate;			// for debug rather than any practical purpose

	};

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
