// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VehicleUtility.h"
#include "SimModule/SimulationModuleBase.h"

#define UE_API CHAOSVEHICLESCORE_API

namespace Chaos
{
	struct FAllInputs;
	class FSimModuleTree;

	enum class EAerofoil : uint8
	{
		Fixed = 0,
		Wing,
		Rudder,
		Elevator
	};

	struct FAerofoilOutputData
		: public FSimOutputData
		, public Chaos::TSimulationModuleTypeable<class FAerofoilSimModule, FAerofoilOutputData>
	{
		virtual FSimOutputData* MakeNewData() override { return FAerofoilOutputData::MakeNew(); }
		static FSimOutputData* MakeNew() { return new FAerofoilOutputData(); }

		UE_API virtual void FillOutputState(const ISimulationModuleBase* SimModule) override;
		UE_API virtual void Lerp(const FSimOutputData& InCurrent, const FSimOutputData& InNext, float Alpha) override;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		UE_API virtual FString ToString() override;
#endif
	};

	struct FAerofoilSettings
	{
		FAerofoilSettings()
			: Offset(FVector::ZeroVector)
			, ForceAxis(FVector(0.f, 0.f, 1.f))
			, ControlRotationAxis(FVector(0.f, 1.f, 0.f))
			, Area(5.0f)
			, Camber(3.0f)
			, MaxControlAngle(1.f)
			, StallAngle(16.0f)
			, Type(EAerofoil::Fixed)
			, LiftMultiplier(1.0f)
			, DragMultiplier(1.0f)
			, AnimationMagnitudeMultiplier(1.0f)
		{
		}
		
		FVector Offset;
		FVector ForceAxis;
		FVector ControlRotationAxis;
		float Area;
		float Camber;
		float MaxControlAngle;
		float StallAngle;

		EAerofoil Type;
		float LiftMultiplier;
		float DragMultiplier;
		float AnimationMagnitudeMultiplier;

	};

	class FAerofoilSimModule : public ISimulationModuleBase, public TSimModuleSettings<FAerofoilSettings>, public TSimulationModuleTypeable<FAerofoilSimModule>
	{
		friend FAerofoilOutputData;

	public:
		DEFINE_CHAOSSIMTYPENAME(FAerofoilSimModule);
		UE_API FAerofoilSimModule(const FAerofoilSettings& Settings);

		virtual ~FAerofoilSimModule() = default;

		virtual TSharedPtr<FModuleNetData> GenerateNetData(const int32 NodeArrayIndex) const override { return nullptr; }

		virtual FSimOutputData* GenerateOutputData() const override
		{
			return FAerofoilOutputData::MakeNew();
		}

		virtual const FString GetDebugName() const { return TEXT("Aerofoil"); }

		virtual bool IsBehaviourType(eSimModuleTypeFlags InType) const override { return (InType & Velocity); }

		UE_API virtual void Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem) override;

		UE_API virtual void Animate() override;

		void SetDensityOfMedium(float InDensity)
		{
			CurrentAirDensity = InDensity;
		}

		void SetControlSurface(float CtrlSurfaceInput)
		{
			ControlSurfaceAngle = CtrlSurfaceInput * Setup().MaxControlAngle;
		}

		UE_API FVector GetCenterOfLiftOffset();

		// returns the combined force of lift and drag at an aerofoil in local coordinates
		// for direct application to the aircrafts rigid body.
		UE_API FVector GetForce(const FVector& v, float Altitude, float DeltaTime);

		/**
		 * Dynamic air pressure = 0.5 * AirDensity * Vsqr
		 */
		UE_API float CalcDynamicPressure(float VelocitySqr, float InAltitude);

		/**  Center of lift moves fore/aft based on current AngleOfAttack */
		UE_API float CalcCentreOfLift();

		/** Returns drag coefficient for the current angle of attack of the aerofoil surface */
		UE_API float CalcDragCoefficient(float InAngleOfAttack, float InControlSurfaceAngle);

		/**
		 * Returns lift coefficient for the current angle of attack of the aerofoil surface
		 * Cheating by making control surface part of entire aerofoil movement
		 */
		UE_API float CalcLiftCoefficient(float InAngleOfAttack, float InControlSurfaceAngle);

		/** Angle of attack is the angle between the aerofoil and the airflow vector */
		UE_API float CalcAngleOfAttackDegrees(const FVector& UpAxis, const FVector& InAirflowVector);

		float CurrentAirDensity;
		float AngleOfAttack;
		float ControlSurfaceAngle;
		FVector AirflowNormal;
		int AerofoilId;

	};

} // namespace Chaos

#undef UE_API
