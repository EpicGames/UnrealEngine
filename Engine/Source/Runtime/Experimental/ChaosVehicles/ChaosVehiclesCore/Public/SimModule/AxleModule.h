// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimModule/TorqueSimModule.h"

#define UE_API CHAOSVEHICLESCORE_API


namespace Chaos
{
	struct FAllInputs;
	class FSimModuleTree;

	/// <summary>
	/// Axle settings
	/// </summary>
	struct FAxleSettings
	{
		FAxleSettings()
			: AxleInertia(1.f) // TODO: defaults
		{

		}

		float AxleInertia;
	};

	/// <summary>
	/// A vehicle component that transmits torque from one source to another, i.e. from an engine or differential to wheels
	///
	/// </summary>
	class FAxleSimModule : public FTorqueSimModule, public TSimModuleSettings<FAxleSettings>, public TSimulationModuleTypeable<FAxleSimModule>
	{
	public:
		DEFINE_CHAOSSIMTYPENAME(FAxleSimModule);
		UE_API FAxleSimModule(const FAxleSettings& Settings);

		virtual const FString GetDebugName() const { return TEXT("Axle"); }

		UE_API virtual void Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem) override;

	private:
	};


} // namespace Chaos

#undef UE_API
