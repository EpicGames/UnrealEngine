// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimModule/SimulationModuleBase.h"
#include "VehicleUtility.h"

#define UE_API CHAOSVEHICLESCORE_API


namespace Chaos
{
	struct FAllInputs;
	class FSimModuleTree;

	/// <summary>
	/// Chassis settings
	/// </summary>
	struct FChassisSettings
	{
		FChassisSettings()
			: AreaMetresSquared(2.0f)
			, DragCoefficient(0.5f)
			, DensityOfMedium(RealWorldConsts::AirDensity())
			, XAxisMultiplier(1.0f)
			, YAxisMultiplier(1.0f)
			, AngularDamping(100000.0f)
		{
		}
		float AreaMetresSquared;	// [meters squared]
		float DragCoefficient;		// always positive
		float DensityOfMedium;
		float XAxisMultiplier;
		float YAxisMultiplier;
		float AngularDamping;
	};

	/// <summary>
	/// A vehicle component that transmits torque from one source to another, i.e. from an engine or differential to wheels
	///
	/// </summary>
	class FChassisSimModule : public ISimulationModuleBase, public TSimModuleSettings<FChassisSettings>, public TSimulationModuleTypeable<FChassisSimModule>
	{
	public:
		DEFINE_CHAOSSIMTYPENAME(FChassisSimModule);
		UE_API FChassisSimModule(const FChassisSettings& Settings);

		virtual TSharedPtr<FModuleNetData> GenerateNetData(const int32 NodeArrayIndex) const override { return nullptr; }

		virtual const FString GetDebugName() const { return TEXT("Chassis"); }

		virtual bool IsBehaviourType(eSimModuleTypeFlags InType) const override { return (InType & Velocity); }

		UE_API virtual void Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem) override;

	private:
	};


} // namespace Chaos

#undef UE_API
