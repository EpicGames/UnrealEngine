// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimModule/TorqueSimModule.h"

#define UE_API CHAOSVEHICLESCORE_API


namespace Chaos
{
	struct FAllInputs;
	class FSimModuleTree;

	struct FClutchSettings
	{
		FClutchSettings()
			: ClutchStrength(1.f)
		{

		}

		float ClutchStrength;
	};


	struct FClutchSimModuleData : public FTorqueSimModuleData, public TSimulationModuleTypeable<class FClutchSimModule, FClutchSimModuleData>
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FClutchSimModuleData(int NodeArrayIndex, const FString& InDebugString) : FTorqueSimModuleData(NodeArrayIndex, InDebugString) {}
#else
		FClutchSimModuleData(int NodeArrayIndex) : FTorqueSimModuleData(NodeArrayIndex) {}
#endif

		virtual void FillSimState(ISimulationModuleBase* SimModule) override
		{
			check(SimModule->IsSimType<class FClutchSimModule>());
			FTorqueSimModuleData::FillSimState(SimModule);
		}

		virtual void FillNetState(const ISimulationModuleBase* SimModule) override
		{
			check(SimModule->IsSimType<class FClutchSimModule>());
			FTorqueSimModuleData::FillNetState(SimModule);
		}

	};


	/// <summary>
	/// 
	/// a vehicle component that transmits torque from one source to another through a clutch system, i.e. connect an engine to a transmission
	///
	/// Input Controls - Clutch pedal, normalized value 0 to 1 expected
	/// Other Inputs - 
	/// Outputs - 
	/// 
	/// </summary>
	class FClutchSimModule : public FTorqueSimModule, public TSimModuleSettings<FClutchSettings>, public TSimulationModuleTypeable<FClutchSimModule>
	{
	public:
		DEFINE_CHAOSSIMTYPENAME(FClutchSimModule);
		UE_API FClutchSimModule(const FClutchSettings& Settings);

		virtual TSharedPtr<FModuleNetData> GenerateNetData(const int32 SimArrayIndex) const override
		{
			return MakeShared<FClutchSimModuleData>(
				SimArrayIndex
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				, GetDebugName()
#endif			
			);
		}

		virtual const FString GetDebugName() const { return TEXT("Clutch"); }

		UE_API virtual bool GetDebugString(FString& StringOut) const override;

		UE_API virtual void Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem);

	private:

		float ClutchValue;
	};
	
	class FClutchSimFactory
		: public FSimFactoryModule<FClutchSimModuleData>
		, public TSimulationModuleTypeable<FClutchSimModule,FClutchSimFactory>
		, public TSimFactoryAutoRegister<FClutchSimFactory>
	
	{
	public:
		FClutchSimFactory() : FSimFactoryModule(TEXT("ClutchFactory")) {}
	};

} // namespace Chaos

#undef UE_API
