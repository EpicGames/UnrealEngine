// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Chaos/ParticleHandleFwd.h"
#include "GeometryCollection/ManagedArray.h"

#define UE_API CHAOSVEHICLESCORE_API

enum class EForceFlags : uint32
{
	None = 0, // No flags.

	AllowSubstepping = 1 << 0,
	AccelChange = 1 << 1,
	VelChange = 1 << 2,
	IsLocalForce = 1 << 3,
	LevelSlope = 1 << 4

};
ENUM_CLASS_FLAGS(EForceFlags);	


class FGeometryCollectionPhysicsProxy;

class FDeferredForcesModular
{
public:

	struct FApplyForceData
	{
		FApplyForceData(const FTransform& OffsetTransformIn, int TransformIndexIn, int ParticleIndexIn, const FVector& ForceIn, bool bAllowSubsteppingIn, bool bAccelChangeIn, bool bLevelSlope, const FColor& ColorIn)
			: OffsetTransform(OffsetTransformIn)
			, TransformIndex(TransformIndexIn)
			, ParticleIdx(ParticleIndexIn)
			, Force(ForceIn)
			, Flags(EForceFlags::None)
			, DebugColor(ColorIn)
		{
			Flags |= bAllowSubsteppingIn ? EForceFlags::AllowSubstepping : EForceFlags::None;
			Flags |= bAccelChangeIn ? EForceFlags::AccelChange : EForceFlags::None;
			Flags |= bLevelSlope ? EForceFlags::LevelSlope : EForceFlags::None;
		}

		FTransform OffsetTransform;
		int TransformIndex;
		int32 ParticleIdx;
		FVector Force;
		EForceFlags Flags;
		FColor DebugColor;
	};

	struct FApplyForceAtPositionData
	{
		FApplyForceAtPositionData(const FTransform& OffsetTransformIn, int TransformIndexIn, int ParticleIndexIn, const FVector& ForceIn, const FVector& PositionIn, bool bAllowSubsteppingIn, bool bIsLocalForceIn, bool bLevelSlope, const FColor& ColorIn)
			: OffsetTransform(OffsetTransformIn)
			, TransformIndex(TransformIndexIn)
			, ParticleIdx(ParticleIndexIn)
			, Force(ForceIn)
			, Position(PositionIn)
			, Flags(EForceFlags::None)
			, DebugColor(ColorIn)
		{
			Flags |= bAllowSubsteppingIn ? EForceFlags::AllowSubstepping : EForceFlags::None;
			Flags |= bIsLocalForceIn ? EForceFlags::IsLocalForce : EForceFlags::None;
			Flags |= bLevelSlope ? EForceFlags::LevelSlope : EForceFlags::None;
		}

		FTransform OffsetTransform;
		int TransformIndex;
		int32 ParticleIdx;
		FVector Force;
		FVector Position;
		EForceFlags Flags;
		FColor DebugColor;
	};

	struct FAddTorqueInRadiansData
	{
		FAddTorqueInRadiansData(const FTransform& OffsetTransformIn, int TransformIndexIn, int ParticleIndexIn, const FVector& TorqueIn, bool bAllowSubsteppingIn, bool bAccelChangeIn, const FColor& ColorIn)
			: OffsetTransform(OffsetTransformIn)
			, TransformIndex(TransformIndexIn)
			, ParticleIdx(ParticleIndexIn)
			, Torque(TorqueIn)
			, Flags(EForceFlags::None)
			, DebugColor(ColorIn)
		{
			Flags |= bAllowSubsteppingIn ? EForceFlags::AllowSubstepping : EForceFlags::None;
			Flags |= bAccelChangeIn ? EForceFlags::AccelChange : EForceFlags::None;
		}

		FTransform OffsetTransform;
		int TransformIndex;
		int32 ParticleIdx;
		FVector Torque;
		EForceFlags Flags;
		FColor DebugColor;
	};

	struct FAddImpulseData
	{
		FAddImpulseData(const FTransform& OffsetTransformIn, int TransformIndexIn, int ParticleIndexIn, const FVector& ImpulseIn, const bool bVelChangeIn)
			: OffsetTransform(OffsetTransformIn)
			, TransformIndex(TransformIndexIn)
			, ParticleIdx(ParticleIndexIn)
			, Impulse(ImpulseIn)
			, Flags(EForceFlags::None)
		{
			Flags |= bVelChangeIn ? EForceFlags::VelChange : EForceFlags::None;
		}

		FTransform OffsetTransform;
		int TransformIndex;
		int32 ParticleIdx;
		FVector Impulse;
		EForceFlags Flags;
	};

	struct FAddImpulseAtPositionData
	{
		FAddImpulseAtPositionData(const FTransform& OffsetTransformIn, int TransformIndexIn, int ParticleIndexIn, const FVector& ImpulseIn, const FVector& PositionIn)
			: OffsetTransform(OffsetTransformIn)
			, TransformIndex(TransformIndexIn)
			, ParticleIdx(ParticleIndexIn)
			, Impulse(ImpulseIn)
			, Position(PositionIn)
		{

		}

		FTransform OffsetTransform;
		int TransformIndex;
		int32 ParticleIdx;
		FVector Impulse;
		FVector Position;
	};

	void Add(const FApplyForceData& ApplyForceDataIn)
	{
		ApplyForceDatas.Add(ApplyForceDataIn);
	}

	void Add(const FApplyForceAtPositionData& ApplyForceAtPositionDataIn)
	{
		ApplyForceAtPositionDatas.Add(ApplyForceAtPositionDataIn);
	}

	void AddCOM(const FApplyForceAtPositionData& ApplyForceAtPositionDataIn)
	{
		ApplyForceAtCOMDatas.Add(ApplyForceAtPositionDataIn);
	}

	void Add(const FAddTorqueInRadiansData& ApplyTorqueDataIn)
	{
		ApplyTorqueDatas.Add(ApplyTorqueDataIn);
	}

	void Add(const FAddImpulseData& ApplyImpulseDataIn)
	{
		ApplyImpulseDatas.Add(ApplyImpulseDataIn);
	}

	void Add(const FAddImpulseAtPositionData& ApplyImpulseAtPositionDataIn)
	{
		ApplyImpulseAtPositionDatas.Add(ApplyImpulseAtPositionDataIn);
	}	

	UE_API Chaos::FPBDRigidParticleHandle* GetParticleFromUniqueIndex(int32 ParticleUniqueIdx, const TArray<Chaos::FPBDRigidParticleHandle*>& Particles) const;

	UE_API Chaos::FPBDRigidParticleHandle* GetParticle(FGeometryCollectionPhysicsProxy* Proxy
			, int TransformIndex
			, int32 ParticleIdx
			, const FVector& PositionalOffset
			, const TManagedArray<FTransform>& Transforms
			, const TManagedArray<FTransform>& CollectionMassToLocal
			, const TManagedArray<int32>& Parent
			, FTransform& TransformOut);

	UE_API Chaos::FPBDRigidParticleHandle* GetParticle(FGeometryCollectionPhysicsProxy* Proxy
		, int TransformIndex
		, int32 ParticleIdx
		, const FVector& PositionalOffset
		, const FTransform& Transform
		, const TManagedArray<FTransform>& CollectionMassToLocal
		, const TManagedArray<int32>& Parent
		, FTransform& TransformOut);

	UE_API Chaos::FPBDRigidParticleHandle* GetParticle(const FTransform& OffsetTransform
		, FGeometryCollectionPhysicsProxy* Proxy
		, int32 ParticleIdx
		, const FVector& PositionalOffset
		, FTransform& TransformOut);

	UE_API Chaos::FPBDRigidParticleHandle* GetParticle(const FTransform& OffsetTransform
		, TArray<Chaos::FPBDRigidParticleHandle*>& Particles
		, TArray<Chaos::FPBDRigidClusteredParticleHandle*>& ClusterParticles
		, int32 ParticleIdx
		, const FVector& PositionalOffset
		, FTransform& TransformOut);

	UE_API Chaos::FPBDRigidParticleHandle* GetClusterParticle(TArray<Chaos::FPBDRigidClusteredParticleHandle*>& ClusterParticles);

	UE_API void Apply(FGeometryCollectionPhysicsProxy* Proxy
		, const TManagedArray<FTransform>& Transforms
		, const TManagedArray<FTransform>& CollectionMassToLocal
		, const TManagedArray<int32>& Parent);

	UE_API void Apply(FGeometryCollectionPhysicsProxy* Proxy
		, const TManagedArray<FTransform3f>& Transforms
		, const TManagedArray<FTransform>& CollectionMassToLocal
		, const TManagedArray<int32>& Parent);
	
	UE_API void Apply(FGeometryCollectionPhysicsProxy* Proxy);

	UE_API void Apply(TArray<Chaos::FPBDRigidParticleHandle*>& Particles
		, TArray<Chaos::FPBDRigidClusteredParticleHandle*>& ClusterParticles);

	UE_API void Apply(Chaos::FPBDRigidParticleHandle* Particle);

private:

	template<typename TransformType>
	void ApplyTemplate(FGeometryCollectionPhysicsProxy* Proxy
		, const TManagedArray<TransformType>& Transforms
		, const TManagedArray<FTransform>& CollectionMassToLocal
		, const TManagedArray<int32>& Parent);

	UE_API void AddForceAtPosition(Chaos::FPBDRigidParticleHandle* RigidHandle, const FApplyForceAtPositionData& DataIn, const FTransform& OffsetTransform);
	UE_API void AddTorque(Chaos::FPBDRigidParticleHandle* RigidHandle, const FAddTorqueInRadiansData& DataIn, const FTransform& OffsetTransform);
	UE_API void AddForce(Chaos::FPBDRigidParticleHandle* RigidHandle, const FApplyForceData& DataIn, const FTransform& OffsetTransform);
	UE_API void AddForceAtCOM(Chaos::FPBDRigidParticleHandle* RigidHandle, const FApplyForceAtPositionData& DataIn);

	TArray<FApplyForceData> ApplyForceDatas;
	TArray<FApplyForceAtPositionData> ApplyForceAtCOMDatas;
	TArray<FApplyForceAtPositionData> ApplyForceAtPositionDatas;
	TArray<FAddTorqueInRadiansData> ApplyTorqueDatas;
	TArray<FAddImpulseData> ApplyImpulseDatas;
	TArray<FAddImpulseAtPositionData> ApplyImpulseAtPositionDatas;

	FTransform ParticleOffsetTransform; // Odd rotation coming through from CU physics bodies
};

#undef UE_API
