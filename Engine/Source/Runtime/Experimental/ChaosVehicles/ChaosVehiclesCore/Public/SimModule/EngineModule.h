// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimModule/TorqueSimModule.h"
#include "SimModule/SimulationModuleBase.h"

#define UE_API CHAOSVEHICLESCORE_API

namespace Chaos
{
	struct FAllInputs;
	class FSimModuleTree;

	struct FEngineSimModuleData
		: public FTorqueSimModuleData
		, public Chaos::TSimulationModuleTypeable<class FEngineSimModule,FEngineSimModuleData>
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FEngineSimModuleData(int NodeArrayIndex, const FString& InDebugString) : FTorqueSimModuleData(NodeArrayIndex, InDebugString) {}
#else
		FEngineSimModuleData(int NodeArrayIndex) : FTorqueSimModuleData(NodeArrayIndex) {}
#endif

		virtual void FillSimState(ISimulationModuleBase* SimModule) override
		{
			check(SimModule->IsSimType<class FEngineSimModule>());
			FTorqueSimModuleData::FillSimState(SimModule);
		}

		virtual void FillNetState(const ISimulationModuleBase* SimModule) override
		{
			check(SimModule->IsSimType<class FEngineSimModule>());
			FTorqueSimModuleData::FillNetState(SimModule);
		}

	};

	struct FEngineOutputData
		: public FSimOutputData
		, public Chaos::TSimulationModuleTypeable<class FEngineSimModule,FEngineOutputData>
	{
		virtual FSimOutputData* MakeNewData() override { return FEngineOutputData::MakeNew(); }
		static FSimOutputData* MakeNew() { return new FEngineOutputData(); }

		UE_API virtual void FillOutputState(const ISimulationModuleBase* SimModule) override;
		UE_API virtual void Lerp(const FSimOutputData& InCurrent, const FSimOutputData& InNext, float Alpha) override;
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		UE_API virtual FString ToString() override;
#endif

		float RPM;
		float Torque;
	};

	struct FEngineSettings
	{
		FEngineSettings()
			: MaxTorque(300.f)
			, MaxRPM(6000)
			, IdleRPM(1200)
			, EngineBrakeEffect(50.0f)
			, EngineInertia(100.0f)
		{
			TorqueCurve.AddNormalized(0.5f);
			TorqueCurve.AddNormalized(0.5f);
			TorqueCurve.AddNormalized(0.5f);
			TorqueCurve.AddNormalized(0.5f);
			TorqueCurve.AddNormalized(0.6f);
			TorqueCurve.AddNormalized(0.7f);
			TorqueCurve.AddNormalized(0.8f);
			TorqueCurve.AddNormalized(0.9f);
			TorqueCurve.AddNormalized(1.0f);
			TorqueCurve.AddNormalized(0.9f);
			TorqueCurve.AddNormalized(0.7f);
			TorqueCurve.AddNormalized(0.5f);
		}

		FNormalisedGraph TorqueCurve;
		float MaxTorque;			// [N.m] The peak torque Y value in the normalized torque graph
		uint16 MaxRPM;				// [RPM] The absolute maximum RPM the engine can theoretically reach (last X value in the normalized torque graph)
		uint16 IdleRPM; 			// [RPM] The RPM at which the throttle sits when the car is not moving			
		float EngineBrakeEffect;	// [N.m] How much the engine slows the vehicle when the throttle is released

		float EngineInertia;
	};

	class FEngineSimModule : public FTorqueSimModule, public TSimModuleSettings<FEngineSettings>, public TSimulationModuleTypeable<FEngineSimModule>
	{
	public:
		DEFINE_CHAOSSIMTYPENAME(FEngineSimModule);
		UE_API FEngineSimModule(const FEngineSettings& Settings);

		virtual ~FEngineSimModule() = default;

		virtual TSharedPtr<FModuleNetData> GenerateNetData(const int32 SimArrayIndex) const override
		{
			return MakeShared<FEngineSimModuleData>(
				SimArrayIndex
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				, GetDebugName()
#endif			
			);
		}
		virtual FSimOutputData* GenerateOutputData() const override
		{
			return FEngineOutputData::MakeNew();
		}

		virtual const FString GetDebugName() const { return TEXT("Engine"); }

		UE_API virtual bool GetDebugString(FString& StringOut) const override;

		UE_API virtual void Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem) override;

		inline float GetEngineIdleSpeed() const { return EngineIdleSpeed; }
		inline float GetEngineTorque(float ThrottlePosition, float EngineRPM);
		inline float GetTorqueFromRPM(float RPM, bool LimitToIdle = true);

	protected:

		float EngineIdleSpeed;
		float MaxEngineSpeed;
		bool EngineStarted;		// is the engine turned off or has it been started

	};

	class FEngineSimFactory
		: public FSimFactoryModule<FEngineSimModuleData>
		, public TSimulationModuleTypeable<FEngineSimModule,FEngineSimFactory>
		, public TSimFactoryAutoRegister<FEngineSimFactory>
	
	{
	public:
		FEngineSimFactory() : FSimFactoryModule(TEXT("EngineSimFactory")) {}
	};

} // namespace Chaos

#undef UE_API
