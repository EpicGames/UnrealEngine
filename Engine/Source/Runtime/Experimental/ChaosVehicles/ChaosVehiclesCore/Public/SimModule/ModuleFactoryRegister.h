// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"
#include "Hash/CityHash.h"

#define UE_API CHAOSVEHICLESCORE_API

namespace Chaos
{

	struct FModuleNetData;
	class IFactoryModule;

	class FModuleFactoryRegister
	{
	public:
		UE_API static FModuleFactoryRegister& Get();
		
		UE_API void RegisterFactory(const FName TypeName, TWeakPtr<IFactoryModule> InFactory);
		UE_API void RegisterFactory(const uint32 TypeNameHash, TWeakPtr<IFactoryModule> InFactory);
		UE_API void RemoveFactory(TWeakPtr<IFactoryModule> InFactory);
		UE_API void Reset();
		UE_API bool ContainsFactory(const FName TypeName) const;
		UE_API bool ContainsFactory(const uint32 TypeNameHash) const;
		UE_API TSharedPtr<Chaos::FModuleNetData> GenerateNetData(const uint32 TypeNameHash, const int32 SimArrayIndex);

		static uint32 GetModuleHash(const FName TypeName)
		{
			uint32 Hash = CityHash32((const char*)TypeName.ToString().GetCharArray().GetData(), sizeof(FString::ElementType) * TypeName.GetStringLength());
			return Hash;
		}
	protected:

		FModuleFactoryRegister() = default;
		TMap<int32, TWeakPtr<IFactoryModule>> RegisteredFactoriesByName;
	};
	
	template<typename _To, typename ..._Rest>
	class TSimulationModuleTypeable;
	//Static helper function to create and register a factory of the correct type. The returned Factory MUST be stored somewhere by the caller.
	template<typename T, typename... Args>
	static bool RegisterFactoryHelper(Args... args)
	{
		FName SimTypeName = T::StaticSimType();
		if(SimTypeName.IsValid() == false)
		{
			return false;
		}
		uint32 SimTypeNameHash = FModuleFactoryRegister::GetModuleHash(SimTypeName);
		if(FModuleFactoryRegister::Get().ContainsFactory(SimTypeNameHash))
		{
			return true;
		}
		static TSharedPtr<T> SharedFactory = MakeShared<T>(args...);
		if (SharedFactory.IsValid())
		{
			FModuleFactoryRegister::Get().RegisterFactory(SimTypeNameHash, SharedFactory);
			return true;
		}
		return false;
	}
} // namespace Chaos

#undef UE_API
