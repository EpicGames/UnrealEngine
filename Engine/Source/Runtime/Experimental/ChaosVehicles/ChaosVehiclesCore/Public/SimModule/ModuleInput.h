// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
//#include "Engine/NetSerialization.h"
#include "Templates/SubclassOf.h"
#include "ModuleInput.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogModularInput, Log, All);

#define UE_API CHAOSVEHICLESCORE_API

UENUM(BlueprintType)
enum class EModuleInputQuantizationType : uint8
{
	Default_16Bits,
	Custom_8Bits, 
};

UENUM(BlueprintType)
enum class EModuleInputValueType : uint8
{
	MBoolean	UMETA(DisplayName = "Digital (bool)"),
	MAxis1D		UMETA(DisplayName = "Axis1D (float)"),
	MAxis2D		UMETA(DisplayName = "Axis2D (Vector2D)"),
	MAxis3D		UMETA(DisplayName = "Axis3D (Vector)"),
	MInteger	UMETA(DisplayName = "Digital (int32)"),

	// NOTE, If adding an entry here, add it to the end, otherwise existing assets get deserialized improperly.
	// ALSO update the number of bits to serialize in FModuleInputValue::Serialize
};

/** Input Options */
UENUM(BlueprintType)
enum class EFunctionType : uint8
{
	LinearFunction = 0,
	SquaredFunction,
	CustomCurve
};

UENUM(BlueprintType)
enum class EModuleInputBufferActionType : uint8
{
	// Replaces current value in buffer with new value
	Override = 0,

	// Adds new value to current value in buffer
	Combine,
	
	// A running average of the inputs.
	Average
};

namespace ModularQuantize
{
	template<int32 MaxValue, uint32 NumBits>
	struct TCompressedFloatDetails
	{
		// NumBits = 8:
		static constexpr int32 MaxBitValue = (1 << (NumBits - 1)) - 1;  //   0111 1111 - Max abs value we will serialize
		static constexpr int32 Bias = (1 << (NumBits - 1));             //   1000 0000 - Bias to pivot around (in order to support signed values)
		static constexpr int32 SerIntMax = (1 << (NumBits - 0));        // 1 0000 0000 - What we pass into SerializeInt
		static constexpr int32 MaxDelta = (1 << (NumBits - 0)) - 1;     //   1111 1111 - Max delta is
	};

	template<int32 MaxValue, uint32 NumBits, typename T UE_REQUIRES(std::is_floating_point_v<T>&& NumBits < 32)>
	bool ToCompressedFloat(const T InValue, uint32& OutCompressedFloat)
	{
		using Details = ModularQuantize::TCompressedFloatDetails<MaxValue, NumBits>;

		bool clamp = false;
		int64 ScaledValue;
		if (MaxValue > Details::MaxBitValue)
		{
			// We have to scale this down
			const T Scale = T(Details::MaxBitValue) / MaxValue;
			ScaledValue = FMath::TruncToInt(Scale * InValue);
		}
		else
		{
			// We will scale up to get extra precision. But keep is a whole number preserve whole values
			constexpr int32 Scale = Details::MaxBitValue / MaxValue;
			ScaledValue = FMath::RoundToInt(Scale * InValue);
		}

		uint32 Delta = static_cast<uint32>(ScaledValue + Details::Bias);

		if (Delta > Details::MaxDelta)
		{
			clamp = true;
			Delta = static_cast<int32>(Delta) > 0 ? Details::MaxDelta : 0;
		}

		OutCompressedFloat = Delta;

		return !clamp;
	}

	template<int32 MaxValue, uint32 NumBits, typename T UE_REQUIRES(std::is_floating_point_v<T>&& NumBits < 32)>
	bool FromCompressedFloat(const uint32 InCompressed, T& OutValue )
	{
		using Details = ModularQuantize::TCompressedFloatDetails<MaxValue, NumBits>;

		uint32 Delta = InCompressed;
		T UnscaledValue = static_cast<T>(static_cast<int32>(Delta) - Details::Bias);

		if constexpr (MaxValue > Details::MaxBitValue)
		{
			// We have to scale down, scale needs to be a float:
			constexpr T InvScale = MaxValue / (T)Details::MaxBitValue;
			OutValue = UnscaledValue * InvScale;
		}
		else
		{
			constexpr int32 Scale = Details::MaxBitValue / MaxValue;
			constexpr T InvScale = T(1) / (T)Scale;

			OutValue = UnscaledValue * InvScale;
		}

		return true;
	}

	template<int32 MaxValue, uint32 NumBits, typename T UE_REQUIRES(std::is_floating_point_v<T>&& NumBits < 32)>
	bool WriteCompressedFloat(const T Value, FArchive& Ar)
	{
		using Details = ModularQuantize::TCompressedFloatDetails<MaxValue, NumBits>;

		uint32 CompressedValue;
		bool clamp = ModularQuantize::ToCompressedFloat<MaxValue, NumBits>(Value, CompressedValue);

		Ar.SerializeInt(CompressedValue, Details::SerIntMax);

		return !clamp;
	}

	template<int32 MaxValue, uint32 NumBits, typename T UE_REQUIRES(std::is_floating_point_v<T>&& NumBits < 32)>
	bool ReadCompressedFloat(T& Value, FArchive& Ar)
	{
		using Details = ModularQuantize::TCompressedFloatDetails<MaxValue, NumBits>;

		uint32 CompressedValue;
		Ar.SerializeInt(CompressedValue, Details::SerIntMax);

		ModularQuantize::FromCompressedFloat<MaxValue, NumBits>(CompressedValue, Value);

		return true;
	}

	// Required because we seialize quantized vector in seperate parts depending on input type
	template<int32 MaxValue, uint32 NumBits>
	bool SerializeFixedFloat(double& InOutValue, FArchive& Ar)
	{
		if (Ar.IsSaving())
		{
			bool success = true;
			success &= ModularQuantize::WriteCompressedFloat<MaxValue, NumBits>(InOutValue, Ar);
			return success;
		}

		ModularQuantize::ReadCompressedFloat<MaxValue, NumBits>(InOutValue, Ar);
		return true;
	}

	template<int32 MaxValue, uint32 NumBits, typename T UE_REQUIRES(std::is_floating_point_v<T>&& NumBits < 32)>
	void QuantizeValue(T& Value)
	{
		uint32 CompressedValue = 0;
		ModularQuantize::ToCompressedFloat<MaxValue, NumBits>(Value, CompressedValue);
		ModularQuantize::FromCompressedFloat<MaxValue, NumBits>(CompressedValue, Value);
	}
	
	template<int32 MaxValue, uint32 NumBits, typename T UE_REQUIRES(std::is_floating_point_v<T>&& NumBits < 32)>
	bool QuantizedIsNearlyEqual(const T& Left,const T& Right)
	{
		uint32 CompressedValue = 0;
		T QuantLeft, QuantRight;
		ModularQuantize::ToCompressedFloat<MaxValue, NumBits>(Left, CompressedValue);
		ModularQuantize::FromCompressedFloat<MaxValue, NumBits>(CompressedValue, QuantLeft);
		ModularQuantize::ToCompressedFloat<MaxValue, NumBits>(Right, CompressedValue);
		ModularQuantize::FromCompressedFloat<MaxValue, NumBits>(CompressedValue, QuantRight);
		return FMath::IsNearlyEqual(QuantLeft, QuantRight);
	}
}

USTRUCT(BlueprintType)
struct FModuleInputValue
{
	GENERATED_BODY()

public:
	using MAxis1D = double; // #TODO: Axis1D (float) double or float? FVector2D and FVector are double
	using MAxis2D = FVector2D;
	using MAxis3D = FVector;
	using MInteger = int32;

	// Support all relevant default constructors (FModuleInputValue isn't movable)
	FModuleInputValue() = default;
	FModuleInputValue(const FModuleInputValue&) = default;
	FModuleInputValue& operator= (const FModuleInputValue&) = default;
	
	// Specialized constructors for supported types
	// Converting a value to a different type (e.g. Val = FVector(1, 1, 1); Val = true;) zeroes out any unused components to ensure getters continue to function correctly.
	FModuleInputValue(bool bInValue) : bHasValue(true), Value(FVector::ZeroVector), ValueInt(bInValue), ValueType(EModuleInputValueType::MBoolean)  {}
	FModuleInputValue(MInteger InValue) : bHasValue(true), Value(FVector::ZeroVector), ValueInt(InValue), ValueType(EModuleInputValueType::MInteger) {}
	FModuleInputValue(MAxis1D InValue) : bHasValue(true), Value(InValue, 0.f, 0.f), ValueInt(0), ValueType(EModuleInputValueType::MAxis1D) {}
	FModuleInputValue(MAxis2D InValue) : bHasValue(true), Value(InValue.X, InValue.Y, 0.f), ValueInt(0), ValueType(EModuleInputValueType::MAxis2D) {}
	FModuleInputValue(MAxis3D InValue) : bHasValue(true), Value(InValue), ValueInt(0), ValueType(EModuleInputValueType::MAxis3D) {}

	FModuleInputValue ReturnQuantized(EModuleInputQuantizationType InInputQuantizationType) const;

	static void Quantize(double& InOutValue, EModuleInputQuantizationType InInputQuantizationType);
	
	static bool SerializeQuantized(double& InOutValue, FArchive& Ar, EModuleInputQuantizationType InInputQuantizationType);
	
	// Build a specific type with no value
	FModuleInputValue(EModuleInputValueType InValueType)
	{
		ValueType = InValueType;
		ValueInt = 0.0f; // not used in this case
		Value = FVector::ZeroVector;
	}

	// Resets Value without affecting ValueType
	void Reset()
	{
		Value = FVector::ZeroVector;
		ValueInt = 0;
		bHasValue = false;
	}

	FModuleInputValue& operator+=(const FModuleInputValue& Rhs)
	{
		ensure(ValueType == Rhs.ValueType);

		Value += Rhs.Value;
		// Promote value type to largest number of bits.
		ValueType = FMath::Max(ValueType, Rhs.ValueType);
		return *this;
	}

	friend FModuleInputValue operator+(const FModuleInputValue& Lhs, const FModuleInputValue& Rhs)
	{
		ensure(Lhs.ValueType == Rhs.ValueType);

		FModuleInputValue Result(Lhs);
		Result += Rhs;
		return Result;
	}

	FModuleInputValue& operator-=(const FModuleInputValue& Rhs)
	{
		ensure(ValueType == Rhs.ValueType);

		Value -= Rhs.Value;
		// Promote value type to largest number of bits.
		ValueType = FMath::Max(ValueType, Rhs.ValueType);
		return *this;
	}

	friend FModuleInputValue operator-(const FModuleInputValue& Lhs, const FModuleInputValue& Rhs)
	{
		ensure(Lhs.ValueType == Rhs.ValueType);

		FModuleInputValue Result(Lhs);
		Result -= Rhs;
		return Result;
	}

	// Scalar operators
	FModuleInputValue& operator*=(float Scalar)
	{
		ensure(ValueType != EModuleInputValueType::MBoolean);
		ensure(ValueType != EModuleInputValueType::MInteger);

		Value *= Scalar;
		return *this;
	}

	friend FModuleInputValue operator*(const FModuleInputValue& Lhs, const float Rhs)
	{
		ensure(Lhs.GetValueType() != EModuleInputValueType::MBoolean);
		ensure(Lhs.GetValueType() != EModuleInputValueType::MInteger);

		FModuleInputValue Result(Lhs);
		Result *= Rhs;
		return Result;
	}

	static FModuleInputValue Clamp(const FModuleInputValue& InValue, const float InMin, const float InMax)
	{
		FModuleInputValue OutValue = InValue;
		float Mag = InValue.GetMagnitude();

		if (Mag < InMin)
		{
			OutValue.SetMagnitude(InMin);
		}
		else if (Mag > InMax)
		{
			OutValue.SetMagnitude(InMax);
		}

		return OutValue;
	}


	template<typename T>
	inline T Get() const { static_assert(sizeof(T) == 0, "Unsupported conversion for type"); }

	// Read only index based value accessor, doesn't care about type. Expect 0 when accessing unused components.
	float operator[](int32 Index) const { return Value[Index]; }

	bool IsQuantizedNonZero(EModuleInputQuantizationType InInputQuantizationType, float Tolerance = KINDA_SMALL_NUMBER) const { return ReturnQuantized(InInputQuantizationType).IsNonZero(Tolerance); }
	UE_API bool IsNonZero(float Tolerance = KINDA_SMALL_NUMBER) const;

	// In-place type conversion
	FModuleInputValue& ConvertToType(EModuleInputValueType Type)
	{
		if (ValueType != Type)
		{
			Reset();
			ValueType = Type;
		}
		return *this;
	}
	FModuleInputValue& ConvertToType(const FModuleInputValue& Other) { return ConvertToType(Other.GetValueType()); }

	EModuleInputValueType GetValueType() const { return ValueType; }

	UE_API float GetMagnitudeSq() const;
	UE_API float GetMagnitude() const;
	UE_API int32 GetMagnitudeInt() const;

	// Serialize values
	UE_API void Serialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess, EModuleInputQuantizationType InInputQuantizationType);
	UE_API void NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess, EModuleInputQuantizationType InInputQuantizationType);
	UE_API void DeltaNetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess,const FModuleInputValue& PreviousInputValue, EModuleInputQuantizationType InInputQuantizationType);

	UE_API void Set(const FModuleInputValue& In);

	UE_API void Lerp(const FModuleInputValue& Min, const FModuleInputValue& Max, float Alpha);

	UE_API void Merge(const FModuleInputValue& From);
	UE_API void Combine(const FModuleInputValue& With);
	UE_API void Average(const FModuleInputValue& With);

	UE_API void Decay(const float DecayAmount);

	// Type sensitive debug stringify
	UE_API FString ToString() const;

	/** During physics resimulation, apply decay for this input while it's being extrapolated */
	void SetApplyInputDecay(const bool bInApplyInputDecay) { bApplyInputDecay = bInApplyInputDecay; }
	bool ShouldApplyInputDecay() const { return  bApplyInputDecay; }

	/** During physics resimulation, if input is being decayed, inverse the decay value, this produces better results sometimes for some inputs like steering */
	void SetInverseInputDecay(const bool bInInverseInputDecay) { bInverseInputDecay = bInInverseInputDecay; }
	bool ShouldInverseInputDecay() const { return  bInverseInputDecay; }

	/** After simulation first consumes the input, clear the value so that it  doesn't persist across multiple async physics ticks */
	void SetClearAfterConsumed(const bool bInClearAfterConsumed) { bClearAfterConsumed = bInClearAfterConsumed; }
	bool ShouldClearAfterConsumed() const { return  bClearAfterConsumed; }

protected:
	void SetMagnitude(float NewSize);

	UPROPERTY(Transient)
	bool bHasValue = false;
	
	UPROPERTY()
	FVector Value = FVector::ZeroVector;

	UPROPERTY()
	int32 ValueInt = 0;

	UPROPERTY()
	EModuleInputValueType ValueType = EModuleInputValueType::MBoolean;

	/** During physics resimulation, apply decay for this input while it's being extrapolated */
	UPROPERTY()
	bool bApplyInputDecay = false;

	/** During physics resimulation, if input is being decayed, inverse the decay value, this produces better results sometimes for some inputs like steering */
	UPROPERTY()
	bool bInverseInputDecay = false;

	/** After simulation first consumes the input, clear the value so that it doesn't persist across multiple async physics ticks */
	UPROPERTY()
	bool bClearAfterConsumed = false;
};

// Supported getter specializations
template<>
inline bool FModuleInputValue::Get() const
{
	// True if any component is non-zero
	return IsNonZero();
}

template<>
inline FModuleInputValue::MAxis1D FModuleInputValue::Get() const
{
	return Value.X;
}

template<>
inline FModuleInputValue::MAxis2D FModuleInputValue::Get() const
{
	return MAxis2D(Value.X, Value.Y);
}

template<>
inline FModuleInputValue::MAxis3D FModuleInputValue::Get() const
{
	return Value;
}

template<>
inline FModuleInputValue::MInteger FModuleInputValue::Get() const
{
	return ValueInt;
}


class FModuleInputConversion
{
public:

	static bool ToBool(FModuleInputValue& InValue)
	{
		ensure(InValue.GetValueType() == EModuleInputValueType::MBoolean);
		return InValue.Get<bool>();
	}

	static float ToAxis1D(FModuleInputValue& InValue)
	{
		ensure(InValue.GetValueType() == EModuleInputValueType::MAxis1D);
		return static_cast<float>(InValue.Get<FModuleInputValue::MAxis1D>());
	}

	static FVector2D ToAxis2D(FModuleInputValue& InValue)
	{
		ensure(InValue.GetValueType() == EModuleInputValueType::MAxis2D);
		return InValue.Get<FModuleInputValue::MAxis2D>();
	}

	static FVector ToAxis3D(FModuleInputValue& InValue)
	{
		ensure(InValue.GetValueType() == EModuleInputValueType::MAxis3D);
		return InValue.Get<FModuleInputValue::MAxis3D>();
	}

	static int32 ToInteger(FModuleInputValue& InValue)
	{
		ensure(InValue.GetValueType() == EModuleInputValueType::MInteger);
		return InValue.Get<FModuleInputValue::MInteger>();
	}

	static FString ToString(FModuleInputValue& ActionValue)
	{
		return ActionValue.ToString();
	}
};


UCLASS(BlueprintType, Blueprintable, MinimalAPI)
class UDefaultModularVehicleInputModifier : public UObject
{
	GENERATED_BODY()

public:
	UDefaultModularVehicleInputModifier(const class FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
		, RiseRate(5.0f), FallRate(5.0f), InputCurveFunction(EFunctionType::LinearFunction) { }

	virtual ~UDefaultModularVehicleInputModifier()
	{
	}

	/**
		* Rate at which the input value rises
		*/
	UPROPERTY(EditAnywhere, Category = VehicleInputRate)
	float RiseRate;

	/**
	 * Rate at which the input value falls
	 */
	UPROPERTY(EditAnywhere, Category = VehicleInputRate)
	float FallRate;

	/**
	 * Controller input curve, various predefined options, linear, squared, or user can specify a custom curve function
	 */
	UPROPERTY(EditAnywhere, Category = VehicleInputRate)
	EFunctionType InputCurveFunction;

	/**
	 * Controller input curve - should be a normalized float curve, i.e. time from 0 to 1 and values between 0 and 1
	 * This curve is only sued if the InputCurveFunction above is set to CustomCurve
	 */
	//UPROPERTY(EditAnywhere, Category = VehicleInputRate) #TODO: Reinstate?
	//FRuntimeFloatCurve UserCurve;

	/** Change an output value using max rise and fall rates */
	UE_API virtual FModuleInputValue InterpInputValue(float DeltaTime, const FModuleInputValue& CurrentValue, const FModuleInputValue& NewValue) const;
	UE_API virtual float CalcControlFunction(float InputValue);
};

USTRUCT(BlueprintType)
struct FModuleInputSetup
{
	GENERATED_BODY()

	FModuleInputSetup()
	{
		Type = EModuleInputValueType::MBoolean;
		bApplyInputDecay = false;
		bInverseInputDecay = false;
		bClearAfterConsumed = false;
	}

	FModuleInputSetup(const FName& InName, const EModuleInputValueType& InType)
		: Name(InName)
		, Type(InType)
		, bApplyInputDecay(false)
		, bInverseInputDecay(false)
		, bClearAfterConsumed(false)
	{
	}

	bool operator==(const FModuleInputSetup& Rhs) const
	{
		return (Name == Rhs.Name);
	}

	UPROPERTY(EditAnywhere, Category = VehicleInput)
	FName Name;

	UPROPERTY(EditAnywhere, Category = VehicleInput)
	EModuleInputValueType Type;

	/** During physics resimulation, apply decay for this input while it's being extrapolated */
	UPROPERTY(EditAnywhere, Category = VehicleInput)
	bool bApplyInputDecay;

	UPROPERTY(EditAnywhere, Category = VehicleInput)
	bool bInverseInputDecay;

	/** After simulation first consumes the input, clear the value doesn't persist across multiple async physics ticks */
	UPROPERTY(EditAnywhere, Category = VehicleInput)
	bool bClearAfterConsumed;
};

class FScopedModuleInputInitializer
{
public:
	FScopedModuleInputInitializer(TArray<struct FModuleInputSetup>& InSetupData)
	{
		InitSetupData = &InSetupData;
	}

	~FScopedModuleInputInitializer()
	{
		InitSetupData = nullptr;
	}

	static bool HasSetup() { return InitSetupData != nullptr; }
	static TArray<struct FModuleInputSetup>* GetSetup() { return InitSetupData; }
	
private:

	static UE_API TArray<struct FModuleInputSetup>* InitSetupData;
};


USTRUCT()
struct FModuleInputContainer
{
	GENERATED_BODY()

public:
	using FInputNameMap = TMap<FName, int>;
	using FInputValues = TArray<FModuleInputValue>;

	FModuleInputContainer()
	{
		if (FScopedModuleInputInitializer::HasSetup())
		{
			FInputNameMap NameMapOut;
			Initialize(*FScopedModuleInputInitializer::GetSetup(), NameMapOut);
		}
	}

	int GetNumInputs() const { return InputValues.Num(); }
	
	FModuleInputValue GetValueAtIndex(int Index) const { return InputValues[Index]; }
	
	void SetValueAtIndex(int Index, const FModuleInputValue& InValue, EModuleInputQuantizationType InInputQuantizationType, bool Quantize = true)
	{
		if (Quantize)
		{
			InputValues[Index].Set(InValue.ReturnQuantized(InInputQuantizationType));
		}
		else
		{
			InputValues[Index].Set(InValue);
		}
	}

	void AverageValueAtIndex(int Index, const FModuleInputValue& InValue, EModuleInputQuantizationType InInputQuantizationType)
	{
		InputValues[Index].Average(InValue.ReturnQuantized(InInputQuantizationType));
	}
	void MergeValueAtIndex(int Index, const FModuleInputValue& InValue, EModuleInputQuantizationType InInputQuantizationType)
	{
		InputValues[Index].Merge(InValue.ReturnQuantized(InInputQuantizationType));
	}

	void CombineValueAtIndex(int Index, const FModuleInputValue& InValue, EModuleInputQuantizationType InInputQuantizationType)
	{
		InputValues[Index].Combine(InValue.ReturnQuantized(InInputQuantizationType));
	}

	FModuleInputContainer& operator=(const FModuleInputContainer& Other)
	{
		if (&Other != this)
		{
			InputValues.Reset();
			// perform a deep copy
			if (Other.InputValues.Num())
			{
				InputValues = Other.InputValues;
			}
		}
		return *this;
	}

	UE_API void Initialize(TArray<FModuleInputSetup>& SetupData, FInputNameMap& NameMapOut);

	UE_API void ZeroValues();

	UE_API void Serialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess, EModuleInputQuantizationType InInputQuantizationType);

	UE_API int AddInput(EModuleInputValueType Type);

	UE_API void RemoveAllInputs();

	UE_API void Lerp(const FModuleInputContainer& Min, const FModuleInputContainer& Max, float Alpha);

	UE_API void Merge(const FModuleInputContainer& From);

	UE_API void Combine(const FModuleInputContainer& With);

	/**  Decay input during resimulation by DecayAmount which increases over resimulation frames from 0.0 -> 1.0 when the input is being reused */
	UE_API void Decay(const float DecayAmount);

	/** Clear input once it has been consumed if input is marked as sutch. Prevents same input being fed into simulation when the PT tick is running multiple times per GT tick */
	UE_API void ClearConsumedInputs();

	TArray<FModuleInputValue>& AccessInputValues() { return InputValues; }

private:
	UPROPERTY()
	TArray<FModuleInputValue> InputValues;

};


class FInputInterface
{
public:

	using FInputNameMap = TMap<FName, int>;

	FInputInterface(const FInputNameMap& InNameMap, FModuleInputContainer& InValueContainer, EModuleInputQuantizationType InInputQuantizationType)
		: NameMap(InNameMap)
		, ValueContainer(InValueContainer)
		, InputQuantizationType(InInputQuantizationType)
	{
	}

	UE_API void SetValue(const FName& InName, const FModuleInputValue& InValue, bool Quantize = true);
	UE_API void CombineValue(const FName& InName, const FModuleInputValue& InValue);
	UE_API void AverageValue(const FName& InName, const FModuleInputValue& InValue);
	UE_API void MergeValue(const FName& InName, const FModuleInputValue& InValue);
	UE_API FModuleInputValue GetValue(const FName& InName) const;
	UE_API EModuleInputValueType GetValueType(const FName& InName) const;
	UE_API float GetMagnitude(const FName& InName) const;
	UE_API int32 GetMagnitudeInt(const FName& InName) const;
	UE_API bool InputsNonZero() const;

	// Quick Access to data type
	bool GetBool(const FName& InName) const
	{
		FModuleInputValue Value = GetValue(InName);
		return FModuleInputConversion::ToBool(Value);
	}
	int32 GetInteger(const FName& InName) const
	{
		FModuleInputValue Value = GetValue(InName);
		return FModuleInputConversion::ToInteger(Value);
	}
	double GetFloat(const FName& InName) const
	{
		FModuleInputValue Value = GetValue(InName);
		return FModuleInputConversion::ToAxis1D(Value);
	}
	FVector2D GetVector2D(const FName& InName) const
	{
		FModuleInputValue Value = GetValue(InName);
		return FModuleInputConversion::ToAxis2D(Value);
	}
	FVector GetVector(const FName& InName) const
	{
		FModuleInputValue Value = GetValue(InName);
		return FModuleInputConversion::ToAxis3D(Value);
	}

	void SetBool(const FName& InName, bool InBool)
	{
		ensure(GetValueType(InName) == EModuleInputValueType::MBoolean);
		FModuleInputValue Value(InBool);
		SetValue(InName, Value);
	}
	void SetInteger(const FName& InName, int32 InInteger)
	{
		ensure(GetValueType(InName) == EModuleInputValueType::MInteger);
		FModuleInputValue Value(InInteger);
		SetValue(InName, Value);
	}
	void SetFloat(const FName& InName, double InFloat, bool Quantize = false)
	{
		ensure(GetValueType(InName) == EModuleInputValueType::MAxis1D);
		FModuleInputValue Value(InFloat);
		SetValue(InName, Value, Quantize);
	}
	void SetVector2D(const FName& InName, const FVector2D& InVector2D, bool Quantize = false)
	{
		ensure(GetValueType(InName) == EModuleInputValueType::MAxis2D);
		SetValue(InName, FModuleInputValue(InVector2D), Quantize);
	}
	void SetVector(const FName& InName, const FVector& InVector, bool Quantize = false)
	{
		ensure(GetValueType(InName) == EModuleInputValueType::MAxis3D);
		SetValue(InName, FModuleInputValue(InVector), Quantize);
	}


	const FInputNameMap& NameMap;			// per vehicle
	FModuleInputContainer& ValueContainer;	// per vehicle instance
	EModuleInputQuantizationType InputQuantizationType;	
};

UCLASS(Abstract, BlueprintType, Blueprintable, EditInlineNew, MinimalAPI)
class UVehicleInputProducerBase : public UObject
{
	GENERATED_BODY()
	
public:
	using FInputNameMap = TMap<FName, int>;

	/* initialize the input buffer container(s) */
	virtual void InitializeContainer(TArray<FModuleInputSetup>& SetupData, FInputNameMap& NameMapOut, EModuleInputQuantizationType InInputQuantizationType) 
	{
		InputQuantizationType = InInputQuantizationType;
	}

	/** capture input at game thread frequency */
	virtual void BufferInput(const FInputNameMap& InNameMap, const FName InName, const FModuleInputValue& InValue, EModuleInputBufferActionType BufferAction = EModuleInputBufferActionType::Override) {}

	/** produce input for PT simulation at PT frequency */
	virtual void ProduceInput(int32 PhysicsStep, int32 NumSteps, const FInputNameMap& InNameMap, FModuleInputContainer& InOutContainer) {}

	/** Special case override for providing test input straight onto the physics thread */
	virtual TArray<FModuleInputContainer>* GetTestInputBuffer() { return nullptr; }

	/** Special case override for providing test input straight onto the physics thread */
	virtual bool IsLoopingTestInputBuffer() { return false; }

	/** Special case override for providing test input straight onto the physics thread */
	virtual int32 GetTestInputStartFrame() { return -1; }

	EModuleInputQuantizationType InputQuantizationType;
};

#undef UE_API 

