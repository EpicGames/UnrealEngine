// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimModule/TorqueSimModule.h"

#define UE_API CHAOSVEHICLESCORE_API

namespace Chaos
{
	struct FAllInputs;
	class FSimModuleTree;

	/// <summary>
	/// Motor settings
	/// </summary>
	struct FMotorSettings
	{
		FMotorSettings()
			: MaxRPM(1.0f)
			, MaxTorque(100.0f)
			, EngineInertia(1.0f)
		{

		}
		float MaxRPM;
		float MaxTorque;
		float EngineInertia;
	};

	/// <summary>
	/// A vehicle component that provides torque output based on a torque control input
	/// The output torque is based on a square function style curve, zero at 0.0 and MaxRPM, and 1.0 at mid RPM
	/// </summary>
	class FMotorSimModule : public FTorqueSimModule, public TSimModuleSettings<FMotorSettings>, public TSimulationModuleTypeable<FMotorSimModule>
	{
	public:
		DEFINE_CHAOSSIMTYPENAME(FMotorSimModule);
		UE_API FMotorSimModule(const FMotorSettings& Settings);

		virtual const FString GetDebugName() const { return TEXT("Motor"); }

		UE_API virtual void Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem) override;

	private:

	};

} // namespace Chaos

#undef UE_API
