// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimModule/DeferredForcesModular.h"
#include "SimModule/SimulationModuleBase.h"
#include "SimModule/ModuleInput.h"
#include "SimModule/VehicleBlackboard.h"

#include "SimModuleTree.generated.h"

#define UE_API CHAOSVEHICLESCORE_API

DECLARE_STATS_GROUP(TEXT("ModularVehicle.SimTree"), STATGROUP_ModularVehicleSimTree, STATGROUP_Advanced);

UENUM(BlueprintType)
enum ESimTreeProcessingOrder : int8
{
	ManualOverride = 0,	// User calls simulate on the child modules
	LeafFirst = 1,		// modules simulation from the leaf first
	RootFirst = 2,		// modules simulate from the root first
	LeafFirstBFS = 3
};

class FGeometryCollectionPhysicsProxy;
namespace Chaos
{
	class ISimulationModuleBase;
	class FClusterUnionPhysicsProxy;
	struct FAllInputs;

	struct FPendingModuleAdds
	{
		FPendingModuleAdds(int ParentIndexIn, ISimulationModuleBase* NewSimModuleIn)
			: ParentIndex(ParentIndexIn), NewSimModule(NewSimModuleIn) {}

		int ParentIndex;
		ISimulationModuleBase* NewSimModule;
	};

	struct FPendingModuleDeletions
	{
		FPendingModuleDeletions(int GuidIn) : Guid(GuidIn) {}
		int Guid;
	};

	// Each update tree has it's own local tree hierarchy, this will be translated into the actual tree hierarchy.
	class FSimTreeUpdates
	{
	public:
		int AddRoot(ISimulationModuleBase* NewSimModuleIn)
		{
			NewModules.Add(FPendingModuleAdds(-1, NewSimModuleIn));
			return NewModules.Num()-1;
		}

		int AddNodeBelow(int ParentIndex, ISimulationModuleBase* NewSimModuleIn)
		{
			NewModules.Add(FPendingModuleAdds(ParentIndex, NewSimModuleIn));
			return NewModules.Num() - 1;
		}

		void RemoveNode(int Guid)
		{
			DeletedModules.Add(FPendingModuleDeletions(Guid));
		}

		void ClearUpdates()
		{
			NewModules.Empty();
			DeletedModules.Empty();
		}

		const TArray<FPendingModuleAdds>& GetNewModules() const { return NewModules; }
		const TArray<FPendingModuleDeletions>& GetDeletedModules() const { return DeletedModules; }

		TArray<FPendingModuleAdds>& AccessNewModules() { return NewModules; }
		TArray<FPendingModuleDeletions>& AccessDeletedModules() { return DeletedModules; }

	private:
		TArray<FPendingModuleAdds> NewModules;
		TArray<FPendingModuleDeletions> DeletedModules;
	};

	struct FVehicleState
	{
		float ForwardSpeedKmh;
		FVector Position;
		FQuat Rotation;
		FVector ForwardDir;
		FVector RightDir;
		FVector UpDir;
		FVector AngularVelocityRad;
		FVector LinearVelocity;
	};

	class FSimModuleTree
	{
		friend class FModularVehicleBuilder;

	public:
		struct FSimModuleNode
		{
			FSimModuleNode()
				: SimModule(nullptr)
				, Parent(INVALID_IDX)
			{
			}

			bool IsValid() const { return (SimModule != nullptr); }

			ISimulationModuleBase* SimModule;
			int Parent;
			TSet<int> Children;

			const static int INVALID_IDX = -1;

		};

		FSimModuleTree()
		{
			bAnimationEnabled = true;
			bSimulationEnabled = true;
			SimTreeProcessingOrder = ESimTreeProcessingOrder::LeafFirst;
			SimBlackboard = MakeUnique<FVehicleBlackboard>();
		}

		~FSimModuleTree()
		{
			SimBlackboard.Reset();
			DeleteNodesBelow(0);
		}

		void Reset()
		{
			DeleteNodesBelow(0);
		}

		bool IsEmpty() const { return SimulationModuleTree.IsEmpty(); }
		int GetParent(int Index) const { check(!SimulationModuleTree.IsEmpty()); return SimulationModuleTree[Index].Parent; }
		const TSet<int>& GetChildren(int Index) const { check(!SimulationModuleTree.IsEmpty()); return SimulationModuleTree[Index].Children; }
		const ISimulationModuleBase* GetSimModule(int Index) const { return IsValidNode(Index) ? SimulationModuleTree[Index].SimModule : nullptr; }
		ISimulationModuleBase* AccessSimModule(int Index) const { return IsValidNode(Index) ? SimulationModuleTree[Index].SimModule : nullptr; }
		bool IsValidNode(int Index) const { return (SimulationModuleTree.IsEmpty() || Index >= SimulationModuleTree.Num()) ? false : true; }
		int NumActiveNodes() const { return SimulationModuleTree.Num() - FreeList.Num(); }
		UE_API void GetRootNodes(TArray<int>& RootNodesOut);
		int GetNumNodes() const { return SimulationModuleTree.Num(); }
		ESimTreeProcessingOrder GetSimTreeProcessingOrder() const { return SimTreeProcessingOrder; }

		UE_API int AddRoot(ISimulationModuleBase* SimModule);

		UE_API void Reparent(int Index, int ParentIndex);
		FSimModuleNode& GetNode(int Index) { return SimulationModuleTree[Index]; }

		UE_API int AddNodeBelow(int AtIndex, ISimulationModuleBase* SimModule);

		UE_API int InsertNodeAbove(int AtIndex, ISimulationModuleBase* SimModule);

		UE_API void DeleteNode(int AtIndex);

		UE_API void AppendTreeUpdates(const FSimTreeUpdates& TreeUpdates);

		UE_API void Simulate(float DeltaTime, FAllInputs& Inputs, IPhysicsProxyBase* PhysicsProxy, Chaos::FPBDRigidParticleHandle* RootParticle);

		UE_API void OnContactModification(FCollisionContactModifier& Modifier, IPhysicsProxyBase* PhysicsProxy);

		void SetSimTreeProcessingOrder(ESimTreeProcessingOrder OrderIn) { SimTreeProcessingOrder = OrderIn; }

		FDeferredForcesModular& AccessDeferredForces() { return DeferredForces; }
		const FDeferredForcesModular& GetDeferredForces() const { return DeferredForces; }
		const TArray<FSimModuleNode>& GetSimulationModuleTree() { return SimulationModuleTree; }

		void SetAnimationEnabled(bool bInEnabled) { bAnimationEnabled = bInEnabled; }
		bool IsAnimationEnabled() { return bAnimationEnabled; }
		void SetSimulationEnabled(bool bInEnabled) { bSimulationEnabled = bInEnabled; }
		bool IsSimulationEnabled() { return bSimulationEnabled; }


		const FVehicleState& GetVehicleState() const
		{
			return VehicleState;
		}

		template <typename T>
		FSimModuleNode* LocateNodeByType()
		{
			for (FSimModuleNode& Node : SimulationModuleTree)
			{
				if (Node.SimModule && Node.SimModule->IsSimType<T>())
				{
					return &Node;
				}
			}

			return nullptr;
		}

		int GetLargestComponentIndex()
		{
			int LargestIndex = -1;
			for (int I = 0; I < GetNumNodes(); I++)
			{
				if (Chaos::ISimulationModuleBase* SimModule = GetNode(I).SimModule)
				{
					if (SimModule->GetTransformIndex() > LargestIndex)
					{
						LargestIndex = SimModule->GetTransformIndex();
					}
				}
			}
			return LargestIndex;
		}

		UE_API void GenerateReplicationStructure(Chaos::FModuleNetDataArray& NetData);
		UE_API void SetNetState(Chaos::FModuleNetDataArray& ModuleDatas);
		UE_API void SetSimState(const Chaos::FModuleNetDataArray& ModuleDatas);
		UE_API void InterpolateState(const float LerpFactor, Chaos::FModuleNetDataArray& LerpDatas, const Chaos::FModuleNetDataArray& MinDatas, const Chaos::FModuleNetDataArray& MaxDatas);

		FVehicleBlackboard* GetSimBlackboard()
		{
			return SimBlackboard.Get();
		}

		UE_API void SimulateNode(float DeltaTime, FAllInputs& Inputs, int NodeIdx, IPhysicsProxyBase* PhysicsProxy, Chaos::FPBDRigidParticleHandle* ParticleHandle);
	protected:

		UE_API void OnContactModificationInternal(int NodeIndex, FCollisionContactModifier& Modifier, IPhysicsProxyBase* PhysicsProxy);

		UE_API void SimulateNodeBFS(float DeltaTime, FAllInputs& Inputs, const TArray<int>& RootNodes, IPhysicsProxyBase* PhysicsProxy, Chaos::FPBDRigidParticleHandle* ParticleHandle);

		UE_API void DeleteNodesBelow(int NodeIdx);

		UE_API int GetNextIndex();

		UE_API void UpdateClusterUnionTransformsIfRequired(IPhysicsProxyBase* PhysicsProxy, ISimulationModuleBase* Module);


		UE_API void UpdateModuleVelocites(IPhysicsProxyBase* PhysicsProxy, Chaos::FPBDRigidParticleHandle* RootParticle, bool bWake);
		UE_API void UpdateVehicleState(Chaos::FPBDRigidParticleHandle* RootParticle);

		TArray<FSimModuleNode> SimulationModuleTree;
		TArray<int> FreeList;

		FDeferredForcesModular DeferredForces;

		Chaos::FAllInputs AllInputs;

		bool bAnimationEnabled;
		bool bSimulationEnabled;

		FVehicleState VehicleState;
		ESimTreeProcessingOrder SimTreeProcessingOrder;

		TUniquePtr<FVehicleBlackboard> SimBlackboard;
	};




} // namespace Chaos

#undef UE_API
