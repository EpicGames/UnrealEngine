// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/GeometryParticlesfwd.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Chaos/PhysicsObject.h"
#include "Logging/LogMacros.h"
#include "SimModule/ModuleFactoryRegister.h"
#include "SimModule/ModuleInput.h"
#include "SimModule/VehicleBlackboard.h"

#define UE_API CHAOSVEHICLESCORE_API

DECLARE_LOG_CATEGORY_EXTERN(LogSimulationModule, Warning, All);

#define TSIMMODULETYPEABLE(clazz) Chaos::TSimulationModuleTypeableExt<clazz,&FName(#clazz)>

struct FCoreModularVehicleDebugParams
{
	bool ShowMass = false;
	bool ShowForces = false;
	float DrawForceScaling = 0.0004f;
	float LevelSlopeThreshold = 0.86f;
	bool DisableForces = false;
};

namespace Chaos
{
	class FSimModuleTree;
	struct FModuleNetData;
	struct FSimOutputData;
	class FClusterUnionPhysicsProxy;
	class FCollisionContactModifier;

	const FName HandbrakeControlName("Handbrake");
	const FName ThrottleControlName("Throttle");
	const FName SteeringControlName("Steering");
	const FName BrakeControlName("Brake");
	const FName ClutchControlName("Clutch");
	const FName BoostControlName("Boost");
	const FName ReverseControlName("Reverse");
	const FName ChangeUpControlName("ChangeUp");
	const FName ChangeDownControlName("ChangeDown");
	const FName PitchControlName("Pitch");
	const FName RollControlName("Roll");
	const FName YawControlName("Yaw");

	namespace EAnimationFlags
	{
		static uint16 AnimateNone = 0x00000000;
		static uint16 AnimatePosition = 0x00000001;
		static uint16 AnimateRotation = 0x00000002;
	}

	struct FModuleHitResults
	{
		int SimIndex;
		FVector ImpactPoint;
		float Distance;
		bool bBlockingHit;
	};

	struct FAllInputs
	{
		FInputInterface& GetControls() const { check(ControlInputs); return *ControlInputs; }
		FInputInterface& GetState() const { check(StateInputs); return *StateInputs; }

		FTransform VehicleWorldTransform;
		TMap<int32, FModuleHitResults> HitResults;
		FInputInterface* ControlInputs = nullptr;
		FInputInterface* StateInputs = nullptr;
		bool bKeepVehicleAwake = false;
		float CurrentTimeDilation = 1.0f;
	};

	struct FCreatedModule
	{
		FCreatedModule() : SimType("Empty"), Guid(INDEX_NONE), TreeIndex(INDEX_NONE)
		{
		}

		FCreatedModule(const FName& InSimType, int InGuid, int InTreeIndex)
			: SimType(InSimType), Guid(InGuid), TreeIndex(InTreeIndex)
		{
		}

		FCreatedModule(const FCreatedModule& Other)
			: SimType(Other.SimType), Guid(Other.Guid), TreeIndex(Other.TreeIndex)
		{
		}
		FCreatedModule& operator=(const FCreatedModule& Other)
		{
			SimType = Other.SimType;
			Guid = Other.Guid;
			TreeIndex = Other.TreeIndex;
			return *this;
		}

		FName SimType;
		int Guid = INDEX_NONE;
		int32 TreeIndex = INDEX_NONE;
	};

	struct FCreatedModules
	{
		FCreatedModules()
		{
		}

		TArray<Chaos::FCreatedModule> ModuleEvents;
	};

	struct FSimModuleAnimationData
	{
		FSimModuleAnimationData() 
			: BoneName(NAME_None)
			, AnimFlags(EAnimationFlags::AnimateNone)
			, AnimationLocOffset(FVector::ZeroVector)
			, AnimationRotOffset(FRotator::ZeroRotator)
			, CombinedRotation(FQuat::Identity)
			, AnimationSetupIndex(-1)
			{}

		FName BoneName;
		uint16 AnimFlags = EAnimationFlags::AnimateNone;
		FVector AnimationLocOffset;
		FRotator AnimationRotOffset;
		FQuat CombinedRotation;
		int AnimationSetupIndex;
	};


	/**
	 * Code common between all simulation building blocks settings
	 */
	template <typename T>
	class TSimModuleSettings
	{
	public:

		explicit TSimModuleSettings(const T& SetupIn) : SetupData(SetupIn)
		{
			SetupData = SetupIn; // deliberate copy for now
		}

		FORCEINLINE T& AccessSetup()
		{
			return (T&)(SetupData);
		}

		FORCEINLINE const T& Setup() const
		{
			return (SetupData);
		}

	private:
		T SetupData;
	};


	enum eSimModuleState
	{
		Disabled,
		Enabled
	};

	enum eSimModuleTypeFlags
	{
		NonFunctional = (1 << 0),	// bitmask 1,2,4,8
		Raycast = (1 << 1),	// requires raycast data
		TorqueBased = (1 << 2),	// performs torque calculations
		Velocity = (1 << 3),	// requires velocity data
	};

	enum EWheelAxis
	{
		X,	// X forward
		Y	// Y forward
	};

	//These two classes work in concert to provide typing information for any base classes that use ISimulationModuleBase for some base type security
	// The FSimulationModuleTypeContainer must only be in the base class of a hierarchy.
	// The TSimulationModuleTypeable<T> class uses multi-inheritance & CRTP to populate the FSimulationmoduleTypeContainer correctly.
	// - All of the classes that want to be typeable/castable CRTP themselves into the chain.
	// - All of the classes that store the shared typing information must be in the CRTP chain.
	// Together they provide the ability to check and cast to any type within the inheritance chain.
	
	template<typename _To>
	class TSimulationModuleTypeableBase
	{
	public:
		TSimulationModuleTypeableBase()
		{
			static_cast<_To*>(this)->AddType(StaticSimTypeBase());
		}
		static FName StaticSimTypeBase()
		{
			return _To::_CHAOSTYPENAMERETRIVAL_();
		}
	};

	template<typename _To, typename ..._Rest>
	class TSimulationModuleTypeable;
	
	template<typename _To>
	class TSimulationModuleTypeable<_To>
	{
	public:
		TSimulationModuleTypeable()
		{
			static_cast<_To*>(this)->AddType(StaticSimType());
		}
		static FName StaticSimType()
		{
			return _To::_CHAOSTYPENAMERETRIVAL_();
		}
	};
	
	template<typename _To, typename _From>
	class TSimulationModuleTypeable<_To,_From>
	{
	public:
		TSimulationModuleTypeable()
		{
			static_cast<_From*>(this)->AddType(StaticSimType());
		}
		static FName StaticSimType()
		{
			return _To::_CHAOSTYPENAMERETRIVAL_();
		}
		static FName RecurseSimType()
		{
			return StaticSimType();
		}
	};

	
	template<class T, class = void>
	struct TSimModuleTypeIsRecursive
	: std::false_type
	{};

	template<class T>
	struct TSimModuleTypeIsRecursive<T, std::enable_if_t<std::is_invocable_r<FName, decltype(T::RecurseSimType)>::value>>
	: std::integral_constant<bool, true>
	{};

	template<class T>
	constexpr bool TSimModuleTypeIsRecursive_v = TSimModuleTypeIsRecursive<T>::value;
	
	class FSimulationModuleTypeContainer
	{
	public:
		TSet<FName> MyTypes;
		FName MostRecentAdd = NAME_None;
		void AddType(FName InType)
		{
			MyTypes.Emplace(InType);
			MostRecentAdd = InType;
		}
		bool IsSimType(FName InType) const
		{
			return MostRecentAdd == InType || MyTypes.Contains(InType);
		}
		FName GetSimType() const
		{
			return MostRecentAdd;
		}
		template <typename U>
		static FName GetStaticSimType()
		{
			if constexpr (TSimModuleTypeIsRecursive_v<U>)
			{
				return U::RecurseSimType(); 
			}
			else
			{
				return TSimulationModuleTypeable<U>::StaticSimType(); 
			}
		}
		template<typename U>
		bool IsSimType() const
		{
			if constexpr (TSimModuleTypeIsRecursive_v<U>)
			{
				return IsSimType(U::RecurseSimType()); 
			}
			else
			{
				return IsSimType(TSimulationModuleTypeable<U>::StaticSimType()); 
			}
		}
		template<typename U>
		U* Cast()
		{
			if(IsSimType<std::remove_const_t<U>>())
			{
				return static_cast<U*>(this);
			}
			return nullptr;
		}
		template<typename U>
		const U* Cast() const
		{
			if(IsSimType<std::remove_const_t<U>>())
			{
				return static_cast<const U*>(this);
			}
			return nullptr;
		}
	};
	

#define DEFINE_CHAOSSIMTYPENAME(cls) \
	 static FName _CHAOSTYPENAMERETRIVAL_() { static FName Name(#cls); return Name; }
	
	//FName TSimulationModuleTypeName<class cls>::SimModuleTypeName = FName(#cls); 

	
	/**
	 * Interface base class for all simulation module building blocks
	 */
	class ISimulationModuleBase : public FSimulationModuleTypeContainer, public TSimulationModuleTypeableBase<ISimulationModuleBase>
	{
	friend FSimOutputData;
		
		
	public:
		DEFINE_CHAOSSIMTYPENAME(ISimulationModuleBase);
		const static int INVALID_IDX = -1;

		ISimulationModuleBase()
			: SimModuleTree(nullptr)
			, SimTreeIndex(INVALID_IDX)
			, StateFlags(Enabled)
			, TransformIndex(INVALID_IDX)
			, ParticleIdx(INVALID_IDX)
			, LocalLinearVelocity(FVector::ZeroVector)
			, LocalAngularVelocity(FVector::ZeroVector)
			, bClustered(true)
			, bAnimationEnabled(true)
			, AnimationOffset(FVector::ZeroVector)
			, AppliedForce(FVector::ZeroVector)
			, Guid(INDEX_NONE)
			, CachedParticle(nullptr)
		{}
		virtual ~ISimulationModuleBase() = default;

		int GetGuid() const { return Guid; }
		void SetGuid(int GuidIn) { Guid = GuidIn; }
		/**
		* Get the friendly name for this module, primarily for logging & debugging module tree
		 */
		virtual const FString GetDebugName() const = 0;

		/**
		* Is Module of a specific behavioral data type
		 */
		virtual bool IsBehaviourType(eSimModuleTypeFlags InType) const = 0;

		/**
		 * Is Module active and simulating
		 */
		virtual bool IsEnabled() const { return (StateFlags == eSimModuleState::Enabled); }

		/*
		* Set Module state, if simulating or not
		*/
		void SetStateFlags(eSimModuleState StateFlagsIn) { StateFlags = StateFlagsIn; }

		/*
		* Any post construction initialisation - called from Game Thread
		*/
		virtual void OnConstruction_External(const Chaos::FPhysicsObjectHandle& PhysicsObject) {}

		/*
		* Any cleaning up required - called from game thread
		*/
		virtual void OnTermination_External() {}

		/**
		 * The main Simulation function that is called from the physics async callback thread
		 */
		virtual void Simulate(IPhysicsProxyBase* Proxy, Chaos::FPBDRigidParticleHandle* ParticleHandle, float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem)
		{
			Simulate(DeltaTime, Inputs, VehicleModuleSystem);
		}

		virtual void OnContactModification(Chaos::FCollisionContactModifier& Modifier, IPhysicsProxyBase* Proxy) {}

		/**
		* The main Simulation function that is called from the physics async callback thread
		*/
		virtual void Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem) {}

		/**
		 * Animate/modify the childToParent transforms, to say rotate a wheel, or rudder, etc
		 */
		virtual void Animate() {}

		void SetAnimationEnabled(bool bInEnabled) { bAnimationEnabled = bInEnabled; }
		bool IsAnimationEnabled() { return bAnimationEnabled; }

		UE_API void SetAnimationData(const FName& BoneNameIn, const FVector& AnimationOffsetIn, int AnimationSetupIndexIn);
		const FSimModuleAnimationData& GetAnimationData() const { return AnimationData; }
		const FVector& GetAnimationOffset() const { return AnimationOffset; }
		const FName& GetBoneName() const { return AnimationData.BoneName; }
		int GetAnimationSetupIndex() { return AnimationData.AnimationSetupIndex; }

		/**
		 * Option to draw debug for this module requires CVar p.Chaos.DebugDraw.Enabled 1
		 */
		virtual void DrawDebugInfo() {}

		/**
		 * Option to return debug text for drawing on the HUD in the Game Thread
		 */
		UE_API virtual bool GetDebugString(FString& StringOut) const;

		/**
		 * The transform index references the transform collection, mapping the simulation module to the geometry collection data
		 */
		void SetTransformIndex(int TransformIndexIn) { TransformIndex = TransformIndexIn; }
		const int GetTransformIndex() const { return TransformIndex; }

		/**
		 * The Particle unique index, should be valid on game and physics threads
		 */
		void SetParticleIndex(FUniqueIdx ParticleIndexIn) { ParticleIdx = ParticleIndexIn; }
		const FUniqueIdx GetParticleIndex() const { return ParticleIdx; }

		/**
		 * The modules own index in the simulation tree array
		 */
		void SetTreeIndex(int TreeIndexIn) { SimTreeIndex = TreeIndexIn; }
		int GetTreeIndex() const { return SimTreeIndex; }

		/**
		 * Very useful to store the simulation tree pointer in which we are stored, then we can access other modules that we reference through an index
		 */
		void SetSimModuleTree(FSimModuleTree* SimModuleTreeIn) { SimModuleTree = SimModuleTreeIn; }

		/**
		 * Force application function, handles deferred force application and applying the force at the collect location based on whether the GC cluster is intact or fractured
		 * Note: forces are applied in local coordinates of the module
		 */
		UE_API void AddLocalForceAtPosition(const FVector& Force, const FVector& Position, bool bAllowSubstepping = true, bool bIsLocalForce = false, bool bLevelSlope = false, const FColor& DebugColorIn = FColor::Blue);


		UE_API void AddForceAtCOMPosition(const FVector& Force, const FVector& OffsetFromCOM = FVector::ZeroVector, bool bAllowSubstepping = true, bool bLevelSlope = false, const FColor& DebugColorIn = FColor::Blue);

		/**
		 * Force application function, handles deferred force application and applying the force at the collect location based on whether the GC cluster is intact or fractured
		 * Note: forces are applied in local coordinates of the module
		 */
		UE_API void AddLocalForce(const FVector& Force, bool bAllowSubstepping = true, bool bIsLocalForce = false, bool bLevelSlope = false, const FColor& DebugColorIn = FColor::Blue);

		/**
		 * Torque application function
		 * Note: forces are applied in local coordinates of the module
		 */
		UE_API void AddLocalTorque(const FVector& Torque, bool bAllowSubstepping = true, bool bAccelChangeIn = true, const FColor& DebugColorIn = FColor::Magenta);

		//---

		/**
		 * Let the module know if it is still clustered or not
		 */
		void SetClustered(bool IsClusteredIn) { bClustered = IsClusteredIn; }
		bool IsClustered() const { return bClustered; }

		/**
		 * Set the COM relative transform of module when it is clustered, so relative to parent COM
		 */
		void SetClusteredTransform(const FTransform& TransformIn) { ClusteredCOMRelativeTransform = TransformIn; }
		const FTransform& GetClusteredTransform() const { return ClusteredCOMRelativeTransform; }

		void SetInitialParticleTransform(const FTransform& TransformIn) { InitialParticleTransform = TransformIn; }
		const FTransform& GetInitialParticleTransform() const { return InitialParticleTransform; }

		void SetComponentTransform(const FTransform& TransformIn) { ComponentTransform = TransformIn; }
		const FTransform& GetComponentTransform() const { return ComponentTransform; }

		/**
		 * Set the COM relative transform of module when it is broken off, so relative to itself
		 */
		void SetIntactTransform(const FTransform& TransformIn) { IntactCOMRelativeTransform = TransformIn; IsInitialized = true; }
		const FTransform& GetIntactTransform() const { return IntactCOMRelativeTransform; }
		bool IsInitialized = false;
		/**
		 * The modules transform relative to the simulating body will depend on whether the GC is intact (get the transform relative to intact cluster)
		 * or fractured (transform relative to fractured part)
		 */
		UE_API const FTransform& GetParentRelativeTransform() const;

		/**
		 * Update the module with its current velocity
		 */
		void SetLocalLinearVelocity(const FVector& VelocityIn) { LocalLinearVelocity = VelocityIn; }
		const FVector& GetLocalLinearVelocity() const { return LocalLinearVelocity; }
		void SetLocalAngularVelocity(const FVector& VelocityIn) { LocalAngularVelocity = VelocityIn; }
		const FVector& GetLocalAngularVelocity() const { return LocalAngularVelocity; }

		UE_API ISimulationModuleBase* GetParent();
		UE_API ISimulationModuleBase* GetFirstChild();

		UE_API FVehicleBlackboard* GetSimBlackboard();

		// for headless chaos testing
		const FVector& GetAppliedForce() { return AppliedForce; }

		// this is the replication datas
		virtual TSharedPtr<FModuleNetData> GenerateNetData(const int32 NodeArrayIndex) const = 0;

		virtual FSimOutputData* GenerateOutputData() const { return nullptr; }

		//void SetClusterParticle(FPBDRigidClusteredParticleHandle* ParticleIn) { ClusterParticle = ParticleIn; }
		UE_API Chaos::FPBDRigidClusteredParticleHandle* GetClusterParticle(Chaos::FClusterUnionPhysicsProxy* Proxy);

		UE_API Chaos::FPBDRigidParticleHandle* GetParticleFromUniqueIndex(int32 ParticleUniqueIdx, TArray<Chaos::FPBDRigidParticleHandle*>& Particles);

	protected:

		FSimModuleTree* SimModuleTree;	// A pointer back to the simulation tree where we are stored
		int SimTreeIndex;	// Index of this SimModule in the FSimModuleTree
		eSimModuleState StateFlags;	// TODO: make this more like flags
		int TransformIndex; // Index of this Sim Module's node in Geometry Collection Transform array
		FUniqueIdx ParticleIdx; // Physics particle unique index

		FTransform InitialParticleTransform;
		FTransform RelativeOffsetTransform;
		FTransform ComponentTransform;

		FTransform ClusteredCOMRelativeTransform;
		FTransform IntactCOMRelativeTransform;
		FVector LocalLinearVelocity;
		FVector LocalAngularVelocity;
		bool bClustered;
		bool bAnimationEnabled;
		FVector AnimationOffset; 
		FSimModuleAnimationData AnimationData;

		// for headless chaos testing
		FVector AppliedForce;
		int Guid; // needed a way of associating internal module with game thread.

		FPBDRigidClusteredParticleHandle* CachedParticle;
	};

	/**
	* Interface base class for all module network serialization
	*/
	struct FModuleNetData : public FSimulationModuleTypeContainer, public TSimulationModuleTypeableBase<FModuleNetData>
	{
		DEFINE_CHAOSSIMTYPENAME(FModuleNetData);
		FModuleNetData(int InSimArrayIndex, const FString& InDebugString = FString())
			: SimArrayIndex(InSimArrayIndex)
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			, DebugString(InDebugString)
#endif
		{}

		virtual ~FModuleNetData() = default;

		virtual void Serialize(FArchive& Ar) = 0;
		virtual void DeltaSerialize(FArchive& Ar,const FModuleNetData* Previous) {Serialize(Ar);}
		virtual void DeltaSerializeWithStash(FArchive& Ar,const FModuleNetData* Previous, TMap<FName,uint32>& InOutSerializationStash) {DeltaSerialize(Ar,Previous);}
		virtual void FillNetState(const ISimulationModuleBase* SimModule) = 0;
		virtual void FillSimState(ISimulationModuleBase* SimModule) = 0;
		virtual void Lerp(const float LerpFactor, const FModuleNetData& Max, const FModuleNetData& MaxValue) = 0;
		
		/** Return true if the current state of this module is the default state,
		* If the state is default it will not be sent over the network, instead ApplyDefaultState() will be called on the receiving end to apply the default state for this module.
		* Important: Be sure to also implement ApplyDefaultState() function to ensure that the data gets set to their desired values when not serialized.
		* Example: bools that are false or true by default and floats that are 0.0f doesn't need to be networked */
		virtual bool IsDefaultState() { return false; }
		
		/** Set the states data to default values, called during deserialization if IsDefaultState() was true during serialization */
		virtual void ApplyDefaultState() { }

		int SimArrayIndex = -1;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		virtual FString ToString() const = 0;
		FString DebugString;
#endif
	};

	using FModuleNetDataArray = TArray<TSharedPtr<FModuleNetData>>;

	struct FSimOutputData : public FSimulationModuleTypeContainer, public TSimulationModuleTypeableBase<FSimOutputData>
	{
		DEFINE_CHAOSSIMTYPENAME(FSimOutputData);
		FSimOutputData() = default;
		virtual ~FSimOutputData() = default;
		
		virtual bool IsEnabled() { return bEnabled; }
		virtual FSimOutputData* MakeNewData() = 0;
		UE_API virtual void FillOutputState(const ISimulationModuleBase* SimModule);
		UE_API virtual void Lerp(const FSimOutputData& InCurrent, const FSimOutputData& InNext, float Alpha);
		UE_API virtual void GetFinalAnimDataGameThread(const FTransform& NewTransform, Chaos::FSimModuleAnimationData& AnimDataOut) { AnimDataOut = AnimationData; }

		bool bEnabled = true;
		FSimModuleAnimationData AnimationData;
		int ModuleGuid = -1;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		virtual FString ToString() { return FString(); }
		FString DebugString;
#endif
	};

	class IFactoryModule : public FSimulationModuleTypeContainer, public TSimulationModuleTypeableBase<IFactoryModule>
	{
	public:
		DEFINE_CHAOSSIMTYPENAME(IFactoryModule);
		virtual ~IFactoryModule() = default;
		virtual TSharedPtr<Chaos::FModuleNetData> GenerateNetData(const int32 SimArrayIndex) const = 0;

	};

	template<typename _To>
	class TSimFactoryAutoRegister
	{
	private:
		inline static bool bSimFactoryRegistered = RegisterFactoryHelper<_To>();
	};

	template <typename T>
	class FSimFactoryModule : public IFactoryModule
	{
	public:
		FSimFactoryModule(const FString& DebugNameIn)
		{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			DebugString = DebugNameIn;
#endif
		}

		TSharedPtr<Chaos::FModuleNetData> GenerateNetData(const int32 SimArrayIndex) const override
		{
			return MakeShared<T>(
				SimArrayIndex
	#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				, FSimulationModuleTypeContainer::GetStaticSimType<T>().ToString()
	#endif			
			);
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FString DebugString;
#endif

	};
} // namespace Chaos

#undef UE_API
