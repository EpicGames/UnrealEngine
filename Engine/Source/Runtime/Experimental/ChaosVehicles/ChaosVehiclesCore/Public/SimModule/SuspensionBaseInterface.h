// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ChaosEngineInterface.h"
#include "SimModule/SimulationModuleBase.h"

#define UE_API CHAOSVEHICLESCORE_API

namespace Chaos
{

/** Suspension world ray/shape trace start and end positions */
struct FSpringTrace
{
	FVector Start;
	FVector End;

	FVector TraceDir()
	{
		FVector Dir(End - Start);
		return Dir.FVector::GetSafeNormal();
	}

	float Length()
	{
		FVector Dir(End - Start);
		return Dir.Size();
	}
};

/** Suspension target point data. */
struct FSuspensionTargetPoint
{
	FSuspensionTargetPoint() {}

	FSuspensionTargetPoint(const FVector& InTargetPosition, const FVector& InImpactNormal, const float InHitDistance, const bool bInWheelInContact, const TEnumAsByte<EPhysicalSurface> InSurfaceType, IPhysicsProxyBase* InHitProxy)
		: TargetPosition(InTargetPosition)
		, ImpactNormal(InImpactNormal)
		, HitDistance(InHitDistance)
		, bWheelInContact(bInWheelInContact)
		, SurfaceType(InSurfaceType)
		, HitProxy(InHitProxy)
	{}


	FVector TargetPosition = FVector::ZeroVector;
	FVector ImpactNormal = FVector::ZeroVector;
	float HitDistance = 0.0f;
	bool bWheelInContact = false;
	TEnumAsByte<EPhysicalSurface> SurfaceType = EPhysicalSurface::SurfaceType_Default;
	IPhysicsProxyBase* HitProxy = nullptr;
};

class FSuspensionBaseInterface 
	: public ISimulationModuleBase
	, public TSimulationModuleTypeable<FSuspensionBaseInterface>
{
public:

	DEFINE_CHAOSSIMTYPENAME(FSuspensionBaseInterface);
	FSuspensionBaseInterface() {}

	virtual ~FSuspensionBaseInterface() = default;

	UE_API virtual bool IsBehaviourType(eSimModuleTypeFlags InType) const override;

	virtual float GetMaxSpringLength() const = 0;
	
	virtual float GetSpringLength() const = 0;
	virtual void SetSpringLength(float InLength, float WheelRadius) = 0;
	UE_API virtual void GetWorldTraceEndpoints(float DeltaSeconds, const FTransform& BodyTransform, const FVector& Velocity, float WheelRadius, FSpringTrace& OutTrace) const = 0;
	UE_API void SetTargetPoint(const FSuspensionTargetPoint& InTargetPoint);
	const FSuspensionTargetPoint& GetTargetPoint() const { return TargetPoint; }
	bool IsWheelInContact() const { return TargetPoint.bWheelInContact; }
	
	void SetWheelSimTreeIndex(int WheelTreeIndexIn) { WheelSimTreeIndex = WheelTreeIndexIn; }
	int GetWheelSimTreeIndex() const { return WheelSimTreeIndex; }
	
	TEnumAsByte<EPhysicalSurface> GetSurfaceType() const { return TargetPoint.SurfaceType; }
	
	void SetImpactNormal(const FVector& NewValue) { TargetPoint.ImpactNormal = NewValue; }
	const FVector& GetImpactNormal() const { return TargetPoint.ImpactNormal; }

	void SetHitDistance(const float NewValue) { TargetPoint.HitDistance = NewValue; }
	float GetHitDistance() const { return TargetPoint.HitDistance; }

	void SetTargetPosition(const FVector& NewValue) { TargetPoint.TargetPosition = NewValue; }
	const FVector& GetTargetPosition() const { return TargetPoint.TargetPosition; }

	void SetHitProxy(IPhysicsProxyBase* NewValue) { TargetPoint.HitProxy = NewValue; }
	const IPhysicsProxyBase* GetHitProxy() const { return TargetPoint.HitProxy; }
	IPhysicsProxyBase* GetHitProxy() { return TargetPoint.HitProxy; }

protected:

	int WheelSimTreeIndex = INVALID_IDX;

private:

	FSuspensionTargetPoint TargetPoint;
};


} // namespace Chaos

#undef UE_API
