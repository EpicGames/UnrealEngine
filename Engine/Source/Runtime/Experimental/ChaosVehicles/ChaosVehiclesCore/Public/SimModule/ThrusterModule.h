// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimModule/SimulationModuleBase.h"

#define UE_API CHAOSVEHICLESCORE_API


namespace Chaos
{
	struct FAllInputs;
	class FSimModuleTree;

	/// <summary>
	/// Thruster settings
	/// </summary>
	struct FThrusterSettings
	{
		FThrusterSettings()
			: MaxThrustForce(0)
			, ForceAxis(FVector(1.0f, 0.0f, 0.0f))
			, SteeringAxis(FVector(0.0f, 0.0f, 1.0f))
			, ForceOffset(FVector::ZeroVector)
			, MaxSteeringAngle(0)
			, SteeringForceEffect(1.0f)
			, BoostMultiplier(2.0f)
			, MaxSpeed(125.0f)
			, SteeringEnabled(false)
		{

		}

		float MaxThrustForce;
		FVector ForceAxis;
		FVector SteeringAxis;
		FVector ForceOffset;
		float MaxSteeringAngle;
		float SteeringForceEffect;
		float BoostMultiplier;
		float MaxSpeed;
		bool SteeringEnabled;
	};

	/// <summary>
	/// A vehicle component that transmits torque from one source to another, i.e. from an engine or differential to wheels
	///
	/// </summary>
	class FThrusterSimModule : public ISimulationModuleBase, public TSimModuleSettings<FThrusterSettings>, public TSimulationModuleTypeable<FThrusterSimModule>
	{
	public:
		DEFINE_CHAOSSIMTYPENAME(FThrusterSimModule);
		UE_API FThrusterSimModule(const FThrusterSettings& Settings);

		virtual TSharedPtr<FModuleNetData> GenerateNetData(const int32 NodeArrayIndex) const override { return nullptr; }

		virtual const FString GetDebugName() const { return TEXT("Thruster"); }

		virtual bool IsBehaviourType(eSimModuleTypeFlags InType) const override { return (InType & NonFunctional); }

		UE_API virtual void Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem) override;

		UE_API virtual void Animate() override;

		float GetSteerAngleDegrees() const { return SteerAngleDegrees; }

	private:
		float SteerAngleDegrees;
	};


} // namespace Chaos

#undef UE_API
