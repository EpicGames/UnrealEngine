// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimModule/SimulationModuleBase.h"
#include "VehicleUtility.h"

#define UE_API CHAOSVEHICLESCORE_API

namespace Chaos
{
	class FTorqueSimModule;

	struct FTorqueSimModuleData
		: public FModuleNetData
		, public Chaos::TSimulationModuleTypeable<FTorqueSimModule,FTorqueSimModuleData>
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FTorqueSimModuleData(int NodeArrayIndex, const FString& InDebugString) : FModuleNetData(NodeArrayIndex, InDebugString) {}
#else
		FTorqueSimModuleData(int NodeArrayIndex) : FModuleNetData(NodeArrayIndex) {}
#endif

		UE_API virtual void FillSimState(ISimulationModuleBase* SimModule) override;

		UE_API virtual void FillNetState(const ISimulationModuleBase* SimModule) override;

		virtual void Serialize(FArchive& Ar) override
		{
			Ar << AngularVelocity;
			Ar << AngularPosition;
			Ar << DriveTorque;
			Ar << LoadTorque;
			Ar << BrakingTorque;
		}

		UE_API virtual void Lerp(const float LerpFactor, const FModuleNetData& Min, const FModuleNetData& Max) override;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		UE_API virtual FString ToString() const override;
#endif

		float AngularVelocity = 0.0f;
		float AngularPosition = 0.0f;
		float DriveTorque = 0.0f;
		float LoadTorque = 0.0f;
		float BrakingTorque = 0.0f;
	};


	class FTorqueSimModule : public ISimulationModuleBase, public TSimulationModuleTypeable<FTorqueSimModule>
	{
		friend FTorqueSimModuleData;

	public:
		DEFINE_CHAOSSIMTYPENAME(FTorqueSimModule);
		FTorqueSimModule()
			: DriveTorque(0.f)
			, LoadTorque(0.f)
			, BrakingTorque(0.f)
			, AngularVelocity(0.f)
			, AngularPosition(0.0f)
		{
		}

		/**
		 * Is Module of a specific type - used for casting
		 */
		virtual bool IsBehaviourType(eSimModuleTypeFlags InType) const { return (InType & TorqueBased); }

		void SetDriveTorque(float TorqueIn) { DriveTorque = TorqueIn; }
		float GetDriveTorque() const { return DriveTorque; }

		void SetLoadTorque(float TorqueIn) { LoadTorque = TorqueIn; }
		float GetLoadTorque() const { return LoadTorque; }

		void SetBrakingTorque(float TorqueIn) { ensure(TorqueIn >= 0.0f); BrakingTorque = TorqueIn; }
		float GetBrakingTorque() const { return BrakingTorque; }

		void SetAngularVelocity(float AngularVelocityIn) { AngularVelocity = AngularVelocityIn; }
		float GetAngularVelocity() const { return AngularVelocity; }

		void AddAngularVelocity(float AngularVelocityIn) { AngularVelocity += AngularVelocityIn; }

		void SetAngularPosition(float AngularPositionIn) { AngularPosition = AngularPositionIn; }
		float GetAngularPosition() const { return AngularPosition; }

		void SetRPM(float InRPM) { AngularVelocity = RPMToOmega(InRPM); }
		float GetRPM() const { return OmegaToRPM(AngularVelocity); }

		/**
		 * Transmit torque between this module and its Parent and Children. DriveTorque passed down to children, LoadTorque passed from child to parent
		 */
		UE_API void TransmitTorque(const FSimModuleTree& BlockSystem, float PushedTorque, float BrakeTorque = 0.f, float GearingRatio = 1.0f, float ClutchSlip = 1.0f);

		/**
		 * Integrate angular velocity using specified DeltaTime & Inertia value, Note the inertia should be the combined inertia of all the connected pieces otherwise things will rotate at different rates
		 */
		UE_API void IntegrateAngularVelocity(float DeltaTime, float Inertia, float MaxRotationVel = MAX_FLT);

		/**
		 * Cast an ISimulationModuleBase to a FTorqueSimModule if compatible class
		 */
		static FTorqueSimModule* CastToTorqueInterface(ISimulationModuleBase* SimModule)
		{
			if (SimModule && SimModule->IsBehaviourType(eSimModuleTypeFlags::TorqueBased))
			{
				return static_cast<FTorqueSimModule*>(SimModule);
			}

			return nullptr;
		}

	protected:

		float DriveTorque;
		float LoadTorque;
		float BrakingTorque;
		float AngularVelocity;
		float AngularPosition;
	};

	class FWheelBaseInterface : public FTorqueSimModule, public TSimulationModuleTypeable<FWheelBaseInterface>
	{
	public:
		DEFINE_CHAOSSIMTYPENAME(FWheelBaseInterface);
		
		FWheelBaseInterface()
			: SuspensionSimTreeIndex(ISimulationModuleBase::INVALID_IDX)
			, SurfaceFriction(1.0f)
			, ForceIntoSurface(0.0f)
		{}

		void SetSuspensionSimTreeIndex(int IndexIn) { SuspensionSimTreeIndex = IndexIn; }
		int GetSuspensionSimTreeIndex() const { return SuspensionSimTreeIndex; }
		void SetSurfaceFriction(float FrictionIn) { SurfaceFriction = FrictionIn; }
		void SetForceIntoSurface(float ForceIntoSurfaceIn) { ForceIntoSurface = ForceIntoSurfaceIn; }
		float GetForceIntoSurface() const { return ForceIntoSurface; }
		float GetSurfaceFriction() const { return SurfaceFriction; }
		void SetGroundInteraction(Chaos::FPBDRigidParticleHandle* GroundParticleIn, const FVector& ImpactPointIn, const FVector& ImpactNormalIn)
		{ 
			GroundBody = GroundParticleIn; 
			ImpactPoint = ImpactPointIn;
			ImpactNormal = ImpactNormalIn;
		}

		Chaos::FPBDRigidParticleHandle* GetGroundBody() const { return GroundBody; }
		void ClearGroundBody() { GroundBody = nullptr; }
		FVector GetImpactPoint() const { return ImpactPoint; }
		FVector GetImpactNormal() const { return ImpactNormal; }
		virtual float GetWheelRadius() const = 0;

	protected:
		int SuspensionSimTreeIndex;
		float SurfaceFriction;
		float ForceIntoSurface;
		Chaos::FPBDRigidParticleHandle* GroundBody = nullptr;
		FVector ImpactPoint = FVector::ZeroVector;
		FVector ImpactNormal = FVector(0,0,1);

	};


} // namespace Chaos

#undef UE_API
