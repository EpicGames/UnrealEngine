// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimModule/TorqueSimModule.h"

#define UE_API CHAOSVEHICLESCORE_API


namespace Chaos
{
	class FTransmissionSimModule;
	struct FAllInputs;
	class FSimModuleTree;

	struct FTransmissionSimModuleData
		: public FModuleNetData
		, public Chaos::TSimulationModuleTypeable<FTransmissionSimModule,FTransmissionSimModuleData>
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FTransmissionSimModuleData(int NodeArrayIndex, const FString& InDebugString) : FModuleNetData(NodeArrayIndex, InDebugString) {}
#else
		FTransmissionSimModuleData(int NodeArrayIndex) : FModuleNetData(NodeArrayIndex) {}
#endif

		UE_API virtual void FillSimState(ISimulationModuleBase* SimModule) override;

		UE_API virtual void FillNetState(const ISimulationModuleBase* SimModule) override;

		virtual void Serialize(FArchive& Ar) override
		{
			Ar << CurrentGear;
			Ar << TargetGear;
			Ar << CurrentGearChangeTime;
		}

		UE_API virtual void Lerp(const float LerpFactor, const FModuleNetData& Min, const FModuleNetData& Max) override;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		UE_API virtual FString ToString() const override;
#endif

		int32 CurrentGear = 0;
		int32 TargetGear = 0;
		float CurrentGearChangeTime = 0.0f;
	};

	struct FGearChangeEvent
	{
		FGearChangeEvent(int32 InGear) : ChangedToGear(InGear) {}

		int32 ChangedToGear = 0;
	};

	struct FTransmissionOutputData
		: public FSimOutputData
		, public Chaos::TSimulationModuleTypeable<FTransmissionSimModule,FTransmissionOutputData>
	{
		virtual FSimOutputData* MakeNewData() override { return FTransmissionOutputData::MakeNew(); }
		static FSimOutputData* MakeNew() { return new FTransmissionOutputData(); }
		
		UE_API virtual void FillOutputState(const ISimulationModuleBase* SimModule) override;
		UE_API virtual void Lerp(const FSimOutputData& InCurrent, const FSimOutputData& InNext, float Alpha) override;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		UE_API virtual FString ToString() override;
#endif

		int32 CurrentGear;
		TArray<FGearChangeEvent> GearChangeEvents;
	};

	struct FTransmissionSettings
	{
		enum ETransType : uint8
		{
			ManualType,
			AutomaticType
		};

		FTransmissionSettings()
			: FinalDriveRatio(3.f)
			, ChangeUpRPM(5000)
			, ChangeDownRPM(2500)
			, GearChangeTime(0.5f)
			, GearHysteresisTime(2.0f)
			, TransmissionEfficiency(1.f)
			, TransmissionType(ETransType::AutomaticType)
			, AutoReverse(true)
		{
			ForwardRatios.Add(2.85f);
			ForwardRatios.Add(2.02f);
			ForwardRatios.Add(1.35f);
			ForwardRatios.Add(1.0f);

			ReverseRatios.Add(2.86f);
		}

		TArray<float> ForwardRatios;	// Gear ratios for forward gears
		TArray<float> ReverseRatios;	// Gear ratios for reverse Gear(s)
		float FinalDriveRatio;			// Final drive ratio [~4.0]

		uint32 ChangeUpRPM;				// [RPM]
		uint32 ChangeDownRPM;			// [RPM]
		float GearChangeTime; 			// [sec]
		float GearHysteresisTime;		// [sec]

		float TransmissionEfficiency;	// Loss from friction in the system mean we might run at around 0.94 Efficiency

		ETransType TransmissionType;	// Specify Automatic or Manual transmission

		bool AutoReverse;				// Arcade handling - holding Brake switches into reverse after vehicle has stopped
	};

	class FTransmissionSimModule : public FTorqueSimModule, public TSimModuleSettings<FTransmissionSettings>, public TSimulationModuleTypeable<FTransmissionSimModule>
	{
		friend FTransmissionSimModuleData;
		friend FTransmissionOutputData;

	public:
		DEFINE_CHAOSSIMTYPENAME(FTransmissionSimModule);
		UE_API FTransmissionSimModule(const FTransmissionSettings& Settings);

		virtual TSharedPtr<FModuleNetData> GenerateNetData(const int32 SimArrayIndex) const override
		{
			return MakeShared<FTransmissionSimModuleData>(
				SimArrayIndex
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				, GetDebugName()
#endif			
			);
		}

		virtual FSimOutputData* GenerateOutputData() const override
		{
			return FTransmissionOutputData::MakeNew();
		}

		virtual const FString GetDebugName() const { return TEXT("Transmission"); }

		UE_API virtual bool GetDebugString(FString& StringOut) const override;

		virtual bool IsBehaviourType(eSimModuleTypeFlags InType) const override { return FTorqueSimModule::IsBehaviourType(InType) || (InType & Velocity); }

		UE_API virtual void Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem) override;

	protected:

		/** set the target gear number to change to, can change gear immediately if specified
		 *  i.e. rather than waiting for the gear change time to elapse
		 */
		UE_API void SetGear(int32 InGear, bool Immediate = false);

		/** Get the final combined gear ratio for the specified gear (reverse gears < 0, neutral 0, forward gears > 0) */
		UE_API float GetGearRatio(int32 InGear) const;

		/** set the target gear to one higher than current target, will clamp gear index within rage */
		void ChangeUp()
		{
			SetGear(TargetGear + 1);
		}

		/** set the target gear to one lower than current target, will clamp gear index within rage */
		void ChangeDown()
		{
			SetGear(TargetGear - 1);
		}

		/** Are we currently in the middle of a gear change */
		bool IsCurrentlyChangingGear() const
		{
			return CurrentGear != TargetGear;
		}

		void CorrectGearInputRange(int32& GearIndexInOut) const
		{
			GearIndexInOut = FMath::Clamp(GearIndexInOut, -Setup().ReverseRatios.Num(), Setup().ForwardRatios.Num());
		}

		int32 GetCurrentGear() { return CurrentGear; }
		int32 GetTargetGear() { return TargetGear; }

	private:
		int32 CurrentGear; // <0 reverse gear(s), 0 neutral, >0 forward gears
		int32 TargetGear;  // <0 reverse gear(s), 0 neutral, >0 forward gears
		float CurrentGearChangeTime; // Time to change gear, no power transmitted to the wheels during change
		mutable int32 PreviousGear;

		bool AllowedToChangeGear; // conditions are ok for an automatic gear change
		float GearHysteresisTimer;
	};
	
	class FTransmissionSimFactory
		: public FSimFactoryModule<FTransmissionSimModuleData>
		, public TSimulationModuleTypeable<FTransmissionSimModule,FTransmissionSimFactory>
		, public TSimFactoryAutoRegister<FTransmissionSimFactory>
	
	{
	public:
		FTransmissionSimFactory() : FSimFactoryModule(TEXT("TransmissionSimFactory")) {}
	};


} // namespace Chaos

#undef UE_API
