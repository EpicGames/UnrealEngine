// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"
#include "Templates/SharedPointer.h"
#include "Templates/Casts.h"

#define UE_API CHAOSVEHICLESCORE_API



/** VehicleBlackboard: this is a simple generic map that can store any type, used as a way for decoupled modules to 
 *  share calculations or transient state data (on the physics thread only). 
 *  Values submitted are copy-in, copy-out. 
 *  Unlike a traditional blackboard pattern, there is no support for subscribing to changes. 
 */
class FVehicleBlackboard
{
private:
	class BlackboardObject
	{
		// untyped base
		struct ObjectContainerBase
		{
		};

		// typed container
		template<typename T>
		struct ObjectContainer : ObjectContainerBase
		{
			ObjectContainer(const T& t) : Object(t) {}

			const T& Get() { return Object; }
			T& GetMutable() { return Object; }

		private:
			T Object;
		};


	public:
		template<typename T>
		BlackboardObject(const T& obj) : ContainerPtr(MakeShared<ObjectContainer<T>>(obj)) {}

		template<typename T>
		const T& Get() const
		{
			ObjectContainer<T>* TypedContainer = static_cast<ObjectContainer<T>*>(ContainerPtr.Get());
			return TypedContainer->Get();
		}

		template<typename T>
		T& GetMutable() const
		{
			ObjectContainer<T>* TypedContainer = static_cast<ObjectContainer<T>*>(ContainerPtr.Get());
			return TypedContainer->GetMutable();
		}

	private:
		TSharedPtr<ObjectContainerBase> ContainerPtr;
	};	// end BlackboardObject
 
public:
	enum class EInvalidationReason : uint8
	{
		FullReset,	// All blackboard objects should be invalidated
		Rollback,	// Invalidate any rollback-sensitive objects
	};

	/** Attempt to retrieve an object from the blackboard. If found, OutFoundValue will be set. Returns true/false to indicate whether it was found. */
 	template<typename T>
	bool TryGet(FName ObjName, T& OutFoundValue) const
	{
		if (const TSharedPtr<BlackboardObject>* ExistingObject = ObjectsByName.Find(ObjName))
		{
			OutFoundValue = ExistingObject->Get()->Get<T>();
			return true;
		}

		return false;
	}

	/** Returns true/false to indicate if an object is stored with that name */
	bool Contains(FName ObjName)
	{
		return ObjectsByName.Contains(ObjName);
	}

	/** Store object by a named key, overwriting any existing object */
	template<typename T>
	void Set(FName ObjName, T Obj)
	{
		ObjectsByName.Emplace(ObjName, MakeShared<BlackboardObject>(Obj));
	}

	/** Invalidate an object by name */
	UE_API void Invalidate(FName ObjName);

	/** Invalidate all objects that can be affected by a particular circumstance (such as a rollback) */
	UE_API void Invalidate(EInvalidationReason Reason);
	
	/** Invalidate all objects */
	void InvalidateAll() { Invalidate(EInvalidationReason::FullReset); }


private:
	TMap<FName, TSharedPtr<BlackboardObject>> ObjectsByName;
};

#undef UE_API
