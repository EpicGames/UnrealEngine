// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimModule/TorqueSimModule.h"

#define UE_API CHAOSVEHICLESCORE_API

namespace Chaos
{
	struct FAllInputs;
	class FSimModuleTree;
	class FClusterUnionPhysicsProxy;
	struct FModuleNetData;

	struct FWheelSimModuleData
		: public FTorqueSimModuleData
		, public Chaos::TSimulationModuleTypeable<class FWheelSimModule,FWheelSimModuleData>
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		FWheelSimModuleData(int NodeArrayIndex, const FString& InDebugString) : FTorqueSimModuleData(NodeArrayIndex, InDebugString) {}
#else
		FWheelSimModuleData(int NodeArrayIndex) : FTorqueSimModuleData(NodeArrayIndex) {}
#endif

		virtual void FillSimState(ISimulationModuleBase* SimModule) override
		{
			check(SimModule->IsSimType<class FWheelSimModule>());
			FTorqueSimModuleData::FillSimState(SimModule);
		}

		virtual void FillNetState(const ISimulationModuleBase* SimModule) override
		{
			check(SimModule->IsSimType<class FWheelSimModule>());
			FTorqueSimModuleData::FillNetState(SimModule);
		}

	};

	struct FWheelTouchChangeEvent
	{
		FWheelTouchChangeEvent(bool IsInContact) : bIsInContact(IsInContact) {}

		bool bIsInContact = 0;
	};

	struct FWheelOutputData
		: public FSimOutputData
		, public Chaos::TSimulationModuleTypeable<class FWheelSimModule,FWheelOutputData>
	{
		virtual FSimOutputData* MakeNewData() override { return FWheelOutputData::MakeNew(); }
		static FSimOutputData* MakeNew() { return new FWheelOutputData(); }
		
		UE_API virtual void FillOutputState(const ISimulationModuleBase* SimModule) override;
		UE_API virtual void Lerp(const FSimOutputData& InCurrent, const FSimOutputData& InNext, float Alpha) override;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		UE_API virtual FString ToString() override;
#endif

		bool bTouchingGround;
		float ForceIntoSurface;
		float SlipAngle;
		float RPM;
		float AngularPositionDegrees;
		float SteeringAngleDegrees;

		TArray<FWheelTouchChangeEvent> WheelTouchEvents;
	};

	struct FWheelSettings
	{
		FWheelSettings()
			: Radius(30.0f)
			, Width(20.0f)
			, WheelInertia(100.0f)

			, FrictionMultiplier(3.0f)
			, LateralSlipGraphMultiplier(1.0f)
			, CorneringStiffness(1000.0f)
			, SlipAngleLimit(8.0f)
			, SlipModifier(0.9f)

			, ABSEnabled(true)
			, TractionControlEnabled(true)
			, SteeringEnabled(false)
			, HandbrakeEnabled(false)
			, AutoHandbrakeEnabled(false)
			, AutoHandbrakeVelocityThreshold(10.0f)
			, MaxSteeringAngle(45)
			, MaxBrakeTorque(4000)
			, HandbrakeTorque(3000)
			, MaxRotationVel(100.0f)
			, Axis(EWheelAxis::X)
			, ReverseDirection(false)
			, ForceOffset(FVector::ZeroVector)
		{
		}

		float Radius;
		float Width;
		float WheelInertia;

		float FrictionMultiplier;
		float LateralSlipGraphMultiplier;
		float CorneringStiffness;
		FGraph LateralSlipGraph;
		float SlipAngleLimit;
		float SlipModifier;

		bool ABSEnabled;			// Advanced braking system operational
		bool TractionControlEnabled;// Straight Line Traction Control
		bool SteeringEnabled;
		bool HandbrakeEnabled;
		bool AutoHandbrakeEnabled;
		float AutoHandbrakeVelocityThreshold; 

		float MaxSteeringAngle;
		float MaxBrakeTorque;
		float HandbrakeTorque;

		float MaxRotationVel;
		EWheelAxis Axis;
		bool ReverseDirection;
		FVector ForceOffset;

	};

	class FWheelSimModule : public FWheelBaseInterface, public TSimModuleSettings<FWheelSettings>, public TSimulationModuleTypeable<FWheelSimModule>
	{
		friend FWheelOutputData;
	public:
		DEFINE_CHAOSSIMTYPENAME(FWheelSimModule);
		UE_API FWheelSimModule(const FWheelSettings& Settings);

		virtual TSharedPtr<FModuleNetData> GenerateNetData(const int32 SimArrayIndex) const override
		{
			return MakeShared<FWheelSimModuleData>(
				SimArrayIndex
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
				, GetDebugName()
#endif			
			);
		}

		virtual FSimOutputData* GenerateOutputData() const override
		{
			return FWheelOutputData::MakeNew();
		}

		virtual const FString GetDebugName() const { return TEXT("Wheel"); }

		UE_API virtual bool GetDebugString(FString& StringOut) const override;

		UE_API virtual void Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem) override;

		UE_API virtual void Animate() override;

		virtual bool IsBehaviourType(eSimModuleTypeFlags InType) const override { return (InType & TorqueBased) || (InType & Velocity); }

		virtual float GetWheelRadius() const override { return Setup().Radius; }

		float GetSteerAngleDegrees() const { return SteerAngleDegrees; }

		FVector GetForceFromFriction() const { return ForceFromFriction; }
		

		/** set wheel rotational speed to match the specified linear forwards speed */
		void SetLinearSpeed(float LinearMetersPerSecondIn)
		{
			SetAngularVelocity(LinearMetersPerSecondIn / Setup().Radius);
		}

		/** get linear forwards speed from angluar velocity and wheel radius */
		float GetLinearSpeed()
		{
			return AngularVelocity * Setup().Radius;
		}

		/** Get the radius of the wheel [cm] */
		float GetEffectiveRadius() const
		{
			return Setup().Radius;
		}

	private:

		float BrakeTorque;				// [N.m]

		FVector ForceFromFriction;
		float MassPerWheel;
		float SteerAngleDegrees;

		// for output
		bool bTouchingGround;
		mutable bool bPrevTouchingGround;
		float SlipAngle;
	};

	
	class FWheelSimFactory
			: public FSimFactoryModule<FWheelSimModuleData>
			, public TSimulationModuleTypeable<FWheelSimModule,FWheelSimFactory>
			, public TSimFactoryAutoRegister<FWheelSimFactory>
	
	{
	public:
		FWheelSimFactory() : FSimFactoryModule(TEXT("WheelSimFactory")) {}
	};

} // namespace Chaos

#undef UE_API
