// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VehicleUtility.h"
#include "EngineSystem.h"
#include "TransmissionSystem.h"
#include "SuspensionSystem.h"
#include "WheelSystem.h"
#include "SteeringSystem.h"
#include "AerodynamicsSystem.h"
#include "AerofoilSystem.h"
#include "ThrustSystem.h"
#include "ArcadeSystem.h"
//#include "FuelSystem.h"
//#include "BallastSystem.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{

class IVehicleInterface
{
public:
//	virtual void Simulate(float DeltaTime) = 0;
};

/**
 * This class is currently just a container for the simulation components used by a wheeled vehicle
 * Keeping all the physics systems together and accessible through the one vehicle class
 */
class FSimpleWheeledVehicle : public IVehicleInterface
{
public:
	FSimpleWheeledVehicle()
	: bSuspensionEnabled(true)
	, bMechanicalSimEnabled(true)
	, bWheelFrictionEnabled(true)
	, NumDrivenWheels(0)
	, bLegacyWheelFrictionPosition(false)
	{

	}

	virtual ~FSimpleWheeledVehicle()
	{

	}

	/*void Simulate(float DeltaTime) override
	{
	}*/

	bool IsValid()
	{
		return (Transmission.Num() == 1) && (Engine.Num() == 1) && (Aerodynamics.Num() == 1);
	}

	FSimpleEngineSim& GetEngine()
	{
		check(Engine.Num() == 1);
		return Engine[0];
	}

	bool HasEngine() const
	{
		return (Engine.Num() > 0);
	}

	bool HasTransmission() const
	{
		return (Transmission.Num() > 0);
	}

	bool HasTorqueControlSetup()
	{
		return (TorqueControlSim.Num() > 0);
	}

	bool HasTargetRotationControlSetup()
	{
		return (TargetRotationControlSim.Num() > 0);
	}

	bool HasStabilizeControlSetup()
	{
		return (StabilizeControlSim.Num() > 0);
	}


	FSimpleTransmissionSim& GetTransmission()
	{
		check(Transmission.Num() == 1);
		return Transmission[0];
	}

	FSimpleDifferentialSim& GetDifferential()
	{
		check(Differential.Num() == 1);
		return Differential[0];
	}

	FSimpleWheelSim& GetWheel(int WheelIdx)
	{
		check(WheelIdx < Wheels.Num());
		return Wheels[WheelIdx];
	}

	FSimpleSuspensionSim& GetSuspension(int WheelIdx)
	{
		check(WheelIdx < Suspension.Num());
		return Suspension[WheelIdx];
	}

	FSimpleSteeringSim& GetSteering()
	{
		check(Steering.Num() == 1);
		return Steering[0];
	}

	FSimpleAerodynamicsSim& GetAerodynamics()
	{
		check(Aerodynamics.Num() == 1);
		return Aerodynamics[0];
	}
	
	FAerofoil& GetAerofoil(int AerofoilIdx)
	{
		check(AerofoilIdx < Aerofoils.Num());
		return Aerofoils[AerofoilIdx];
	}

	FSimpleThrustSim& GetThruster(int ThrusterIdx)
	{
		check(ThrusterIdx < Thrusters.Num());
		return Thrusters[ThrusterIdx];
	}

	FTorqueControlSim& GetTorqueControl()
	{
		check(TorqueControlSim.Num() == 1);
		return TorqueControlSim[0];
	}

	FTargetRotationControlSim& GetTargetRotationControl()
	{
		check(TargetRotationControlSim.Num() == 1);
		return TargetRotationControlSim[0];
	}

	FStabilizeControlSim& GetStabilizeControl()
	{
		check(StabilizeControlSim.Num() == 1);
		return StabilizeControlSim[0];
	}

	const TArray<FAxleSim>& GetAxles() const
	{
		return Axles;
	}


	TArray<FSimpleEngineSim> Engine;
	TArray<FSimpleTransmissionSim> Transmission;
	TArray<FSimpleDifferentialSim> Differential;
	TArray<FSimpleWheelSim> Wheels;
	TArray<FSimpleSuspensionSim> Suspension;
	TArray<FSimpleSteeringSim> Steering;
	TArray<FSimpleAerodynamicsSim> Aerodynamics;
	TArray<FAerofoil> Aerofoils;
	TArray<FSimpleThrustSim> Thrusters;
	TArray<FAxleSim> Axles;

	TArray<FTorqueControlSim> TorqueControlSim;
	TArray<FTargetRotationControlSim> TargetRotationControlSim;
	TArray<FStabilizeControlSim> StabilizeControlSim;
	//TArray<FSimpleFuelSim> Fuel;
	//TArray<FSimpleBallastSim> Ballast;
	// turbo
	// .. 

	bool bSuspensionEnabled;
	bool bMechanicalSimEnabled;
	bool bWheelFrictionEnabled;
	uint32 NumDrivenWheels;
	bool bLegacyWheelFrictionPosition;
};


} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
