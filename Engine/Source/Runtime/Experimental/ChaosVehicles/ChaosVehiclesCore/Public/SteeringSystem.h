// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Math/Vector2D.h"
#include "SteeringUtility.h"
#include "VehicleSystemTemplate.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{

	enum class ESteerType : uint8
	{
		SingleAngle,
		AngleRatio,
		Ackermann,
	};

	struct CHAOSVEHICLESCORE_API FSimpleSteeringConfig
	{
		FSimpleSteeringConfig()
			: SteeringType(ESteerType::AngleRatio)
			, AngleRatio(0.7f)
			, TrackWidth(1.8f)
			, WheelBase(3.8f)
		{}

		ESteerType SteeringType;
		float AngleRatio;

		float TrackWidth;
		float WheelBase;

		float MaxSteeringAngle;

		FGraph SpeedVsSteeringCurve;
	};

	class CHAOSVEHICLESCORE_API FAckermannSim : public TVehicleSystem<FSimpleSteeringConfig>
	{
	public:

		FAckermannSim(const FSimpleSteeringConfig* SetupIn);

		void GetLeftHingeLocations(FVector2D& OutC1, FVector2D& OutP, FVector2D& OutC2);

		void GetRightHingeLocations(FVector2D& OutC1, FVector2D& OutP, FVector2D& OutC2);

		void CalculateAkermannAngle(float Input, float& OutSteerLeft, float& OutSteerRight);

		float GetMaxAckermanAngle()
		{
			return MaxAckermanAngle;
		}

	private:
		FVector2D C1;
		FVector2D C2;
		float R1;
		float R2;
		float SteerInputScaling;
		float MaxAckermanAngle;

		FVector2D LeftRodPt, RightRodPt;
		FVector2D LeftPivot;
		FVector2D RightPivot;

		float RestAngle;

	};

	class CHAOSVEHICLESCORE_API FSimpleSteeringSim : public TVehicleSystem<FSimpleSteeringConfig>
	{
	public:
		FSimpleSteeringSim(const FSimpleSteeringConfig* SetupIn)
			: TVehicleSystem<FSimpleSteeringConfig>(SetupIn)
			, Ackermann(SetupIn)
		{
		}

		float GetSteeringFromVelocity(float VelocityMPH)
		{
			return Setup().SpeedVsSteeringCurve.EvaluateY(VelocityMPH);
		}

		float GetSteeringAngle(float InNormSteering, float MaxSteeringAngle, float WheelSide)
		{
			float OutSteeringAngle = 0.f;

			switch (Setup().SteeringType)
			{
				case ESteerType::AngleRatio:
				{
					bool OutsideWheel = (InNormSteering * WheelSide) > 0.f;
					OutSteeringAngle = InNormSteering * (OutsideWheel ? MaxSteeringAngle : MaxSteeringAngle * Setup().AngleRatio);

				}
				break;

				case ESteerType::Ackermann:
				{
					float SteerLHS; float SteerRHS;
					
					Ackermann.CalculateAkermannAngle(-InNormSteering, SteerLHS, SteerRHS);

					OutSteeringAngle = (WheelSide < 0.0f) ? -SteerLHS : SteerRHS;
					OutSteeringAngle *= (MaxSteeringAngle / Ackermann.GetMaxAckermanAngle());
				}
				break;

				default:
				case ESteerType::SingleAngle:
				{
					OutSteeringAngle = MaxSteeringAngle * InNormSteering;
				}
				break;

			}

			return OutSteeringAngle;
		}
		FAckermannSim Ackermann;
	};

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif