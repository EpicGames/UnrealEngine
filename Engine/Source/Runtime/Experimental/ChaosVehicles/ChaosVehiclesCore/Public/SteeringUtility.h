// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "VehicleSystemTemplate.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{
	struct FSteeringUtility
	{
		/**
		 *  T - Track width
		 *  W - Wheelbase
		 *  H - Distance form steering rod to center of axle
		 *  R - Rod End Length
		 *  S - Steering Rod Length (Half)
		 *
		*/

		/** Intersection of two axis aligned circles Radius R1, R2 with a separation distance between centers of D */
		static bool IntersectTwoCircles(float R1, float R2, float D, FVector2D& OutIntersection)
		{
			OutIntersection.Set(0, 0);

			// no intersection of circumference if radii are too far apart or one circle is completely inside the other
			if ((D > (R1 + R2)) || (D < (R1 - R2)))
			{
				return false;
			}

			float TwoD = D * 2.0f;
			float R1Sqr = R1 * R1;
			float R2Sqr = R2 * R2;
			float DSqr = D * D;

			OutIntersection.X = (DSqr - R2Sqr + R1Sqr) / TwoD;
			OutIntersection.Y = FMath::Sqrt(4.0f * DSqr * R1Sqr - Sqr(DSqr - R2Sqr + R1Sqr)) / TwoD;

			return true;
		}

		//static void CalculateAkermannAngle(float Input, float& OutSteerLeft, float& OutSteerRight
		//	, FVector2D& OutPtLeft = FVector2D(), FVector2D& OutPtRight = FVector2D())
		//{
		//	float RestAngle;
		//	FVector2D PtRest;
		//	CalculateAkermannAngle(false, 0.0f, PtRest, RestAngle);

		//	CalculateAkermannAngle(true, Input, OutSteerLeft, OutPtLeft);
		//	CalculateAkermannAngle(false, Input, OutSteerRight, OutPtRight);

		//	OutSteerLeft -= RestAngle;
		//	OutSteerRight -= RestAngle;
		//}

		static void CalculateAkermannAngle(bool Flip, float Input, FVector2D& C2, float R1, float R2
			, float& OutSteerAngle, FVector2D& OutC1, FVector2D& OutPt)
		{
			if (Flip)
			{
				Input = -Input;
			}
			
			OutC1.Set(0.0f + Input, 0.0f);

			FVector2D C2RelC1 = C2 - OutC1;
			float Angle = RadToDeg(FMath::Atan2(C2RelC1.Y, C2RelC1.X));

			// D isn't a fixed value because C1 moves with steering rack
			float D = C2RelC1.Size();

			FVector2D Intersection;
			IntersectTwoCircles(R1, R2, D, Intersection);
			Intersection.Y = -Intersection.Y;

			FVector2D Arm= Intersection.GetRotated(Angle);
			OutPt = OutC1 + Arm;

			FVector2D WheelArm = C2 - OutPt;
			OutSteerAngle = RadToDeg(FMath::Atan2(WheelArm.X, WheelArm.Y));
		}

		static float CalculateBetaDegrees(float TrackWidth, float WheelBase)
		{
			return RadToDeg(FMath::Atan2(TrackWidth*0.5f, WheelBase));
		}
		static void AkermannSetup(float T, float Beta, float R, float& OutH, float& OutS)
		{
			OutH = R * FMath::Cos(DegToRad(Beta));
			OutS = T - 2.0f * R * FMath::Sin(DegToRad(Beta));
		}

		static void CalcJointPositions(float T, float Beta, float R, FVector2D& C1, float& R1, FVector2D& C2, float& R2)
		{
			float H = R * FMath::Cos(DegToRad(Beta));
			float S = T - 2.0f * R * FMath::Sin(DegToRad(Beta));

			C1.Set(0.f, 0.f);
			C2.Set(T*0.5f, H);
			R1 = S * 0.5f;
			R2 = R;
		}


/*		static void Akerman(float Input, float& OutSteerA, float& OutSteerB)
		{
			float l1 = 2.0f;
			float l2 = 0.15f;
			float Gamma = DegToRad(18.0f);

			//FMath::Sin(Gamma - A) + FMath::Sin(Gamma + B);


			//float K1 = Sqr(l1 / l2 - 2.0f * FMath::Sin(Gamma));

			//float PartSum = K1 - Sqr(FMath::Cos(Gamma - A) - FMath::Cos(Gamma - B));
			////float Ans = FMath::Sqrt(PartSum);
			//float Gamma = DegToRad(18.0f);


			float t = 2.0f; // width between wheels
			float r = 0.15f; // track length
			float h = r * FMath::Cos(Gamma); // track depth
			float s = t - 2.0f * r * FMath::Sin(Gamma);

			float Beta = FMath::Atan((t - s) / (2.0f * h));

			float dev = Input * 0.5f;
			OutSteerA = RadToDeg(FMath::Atan((t - (s - dev)) / (2.0f * h)));
			OutSteerB = RadToDeg(FMath::Atan((t - (s + dev)) / (2.0f * h)));

			if (Input < 0.f)
			{
				OutSteerA = -OutSteerA;
				OutSteerB = -OutSteerB;
			}
			//for (float dev = 0.0f; dev < 0.3f; dev += 0.02f)
			//{
			//	float BetaA = FMath::Atan((t - (s - dev)) / (2.0f * h));
			//	float BetaB = FMath::Atan((t - (s + dev)) / (2.0f * h));

			//	float BetaADeg = RadToDeg(BetaA);
			//	float BetaBDeg = RadToDeg(BetaB);

			//	UE_LOGF(LogChaos, Warning, "%f %f", BetaADeg - 18.0f, BetaBDeg - 18.0f);
			//}
		}
		*/
	};

}

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif