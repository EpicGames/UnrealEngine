// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Transform.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "VehicleSystemTemplate.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

/**
 * #todo: 
 * -proper suspension setup for resting position - decide on parameters i.e. use SuspensionMaxRaise/SuspensionMaxDrop??
 * -natural frequency stuff
 * -defaults
 */

namespace Chaos
{

	#define NUM_SUS_AVERAGING 10

	struct CHAOSVEHICLESCORE_API FSimpleSuspensionConfig
	{
		FSimpleSuspensionConfig()
			: SuspensionAxis(FVector(0.f, 0.f, -1.f))
			, SuspensionForceOffset(FVector::ZeroVector)
			, SuspensionMaxRaise(0.f)
			, SuspensionMaxDrop(0.f)
			, MaxLength(0.f)
			, SpringRate(1.f)
			, SpringPreload(0.5f)
			, CompressionDamping(0.9f)
			, ReboundDamping(0.9f)
			, RestingForce(0.f)
			, Swaybar(0.5f)
			, DampingRatio(0.3f)
			, WheelLoadRatio(1.f)
			, RaycastSafetyMargin(0.0f)
			, SuspensionSmoothing(6)
		{
			MaxLength = FMath::Abs(SuspensionMaxRaise) + FMath::Abs(SuspensionMaxDrop);
			SuspensionSmoothing = FMath::Clamp(SuspensionSmoothing, 0, NUM_SUS_AVERAGING);
		}

		void SetSuspensionMaxRaise(float InSuspensionMaxRaise)
		{
			SuspensionMaxRaise = InSuspensionMaxRaise;
			MaxLength = FMath::Abs(SuspensionMaxRaise) + FMath::Abs(SuspensionMaxDrop);
		}

		void SetSuspensionMaxDrop(float InSuspensionMaxDrop)
		{
			SuspensionMaxDrop = InSuspensionMaxDrop;
			MaxLength = FMath::Abs(SuspensionMaxRaise) + FMath::Abs(SuspensionMaxDrop);
		}

		FVector SuspensionAxis;		// local axis, direction of suspension force raycast traces
		FVector SuspensionForceOffset;	// relative position from wheel where suspension forces are applied
		float SuspensionMaxRaise;	// distance [cm]
		float SuspensionMaxDrop;	// distance [cm]
		float MaxLength;			// distance [cm]

		float SpringRate;			// spring constant
		float SpringPreload;		// Amount of Spring force (independent spring movement)
		float CompressionDamping;	// limit compression speed
		float ReboundDamping;		// limit rebound speed
		float RestingForce;			// force on spring when vehicle on level with no body roll

		float Swaybar;				// Anti-roll bar

		float DampingRatio;			// value between (0-no damping) and (1-critical damping)
		float WheelLoadRatio;		// Normalized value, 0 no weight transfer, 1 Normal Weight transfer. A lower value cures lift off oversteer.
		float RaycastSafetyMargin;	// raise start of raycast [cm]

		int SuspensionSmoothing;	// [0-off , 10-max] smoothing visual appearance of wheel movement
	};

	/** Suspension world ray/shape trace start and end positions */
	struct CHAOSVEHICLESCORE_API FSuspensionTrace
	{
		FVector Start;
		FVector End;

		FVector TraceDir()
		{
			FVector Dir(End - Start);
			return Dir.FVector::GetSafeNormal();
		}

		float Length()
		{
			FVector Dir(End - Start);
			return Dir.Size();
		}
	};

	class CHAOSVEHICLESCORE_API FSimpleSuspensionSim : public TVehicleSystem<FSimpleSuspensionConfig>
	{
	public:
		FSimpleSuspensionSim(const FSimpleSuspensionConfig* SetupIn)
			: TVehicleSystem<FSimpleSuspensionConfig>(SetupIn)
			, DisplacementInput(0.f)
			, LastDisplacement(0.f)
			, LocalVelocity(FVector::ZeroVector)
			, SuspensionForce(0.f)
			, LocalOffset(FVector::ZeroVector)
			, SpringDisplacement(0.f)
			, SpringIndex(0)
			, LastSpringLength(0.f)
			, AveragingLength()
			, AveragingCount(0)
			, AveragingNum(0)
		{
		}

// Inputs

		/** #todo: Change This ; Set suspension length after determined from raycast */
		void SetSuspensionLength(float InLength, float WheelRadius)
		{
			DisplacementInput = InLength - Setup().RaycastSafetyMargin - WheelRadius;
			DisplacementInput = FMath::Max(0.f, DisplacementInput);
			SpringDisplacement = Setup().MaxLength - DisplacementInput;
		}

		float GetNormalizedLength()
		{
			float NormalizedLength = 1.0f;

			if (Setup().MaxLength > SMALL_NUMBER)
			{
				NormalizedLength = DisplacementInput / Setup().MaxLength;
			}
			NormalizedLength = FMath::Max(FMath::Min(NormalizedLength, 1.0f), 0.0f);
			return NormalizedLength;
		}

		/** set local velocity at suspension position */
		void SetLocalVelocity(const FVector& InVelocity)
		{
			LocalVelocity = InVelocity;
		}

		void SetLocalVelocityFromWorld(const FTransform& InWorldTransform, const FVector& InWorldVelocity)
		{
			LocalVelocity = InWorldTransform.InverseTransformVector(InWorldVelocity);
		}

		void SetLocalRestingPosition(const FVector& InOffset)
		{
			LocalOffset = InOffset;
		}

		void SetSpringIndex(uint32 InIndex)
		{
			SpringIndex = InIndex;
		}

		void UpdateWorldRaycastLocation(const FTransform& BodyTransform, float WheelRadius, FSuspensionTrace& OutTrace)
		{
			FVector LocalDirection = Setup().SuspensionAxis;
			FVector WorldLocation = BodyTransform.TransformPosition(GetLocalRestingPosition());
			FVector WorldDirection = BodyTransform.TransformVector(LocalDirection);

			OutTrace.Start = WorldLocation - WorldDirection * (Setup().SuspensionMaxRaise + Setup().RaycastSafetyMargin);
			OutTrace.End = WorldLocation + WorldDirection * (Setup().SuspensionMaxDrop + WheelRadius);
		}

		float GetTraceLength(float WheelRadius)
		{
			return Setup().SuspensionMaxRaise + Setup().RaycastSafetyMargin + Setup().SuspensionMaxDrop + WheelRadius;
		}

// Outputs

		float GetSpringLength();

		float GetSuspensionForce() const
		{
			return SuspensionForce;
		}

		FVector GetSuspensionForceVector(const FTransform& InTransform)
		{
			FVector LocalDirection(0.f, 0.f, 1.f);
			return InTransform.TransformVector(LocalDirection) * SuspensionForce;
		}

		float GetSuspensionOffset()
		{
			return Setup().SuspensionMaxRaise + GetSpringLength();
		}

		const FVector& GetLocalRestingPosition() const
		{
			return LocalOffset;
		}

// Simulation

		void Simulate(float DeltaTime);

		int32 GetAveragingCount() const {return AveragingCount;}
		int32 GetAveragingNum() const { return AveragingNum; }
		float GetLastSpringLength() const { return LastSpringLength; }
		float GetLastDisplacement() const { return LastDisplacement; }
		float GetAveragingLength(const int32 LengthIndex) const { return AveragingLength[LengthIndex]; }

		void SetAveragingCount(const int32 InAveragingCount) {AveragingCount = InAveragingCount;}
		void SetAveragingNum(const int32 InAveragingNum) { AveragingNum = InAveragingNum; }
		void SetLastSpringLength(const float InLastSpringLength) { LastSpringLength = InLastSpringLength; }
		void SetLastDisplacement(const float InLastDisplacement) { LastDisplacement = InLastDisplacement; }
		void SetAveragingLength(const int32 LengthIndex, const float InAveragingLength) { AveragingLength[LengthIndex] = InAveragingLength; }

	protected:

		float DisplacementInput;	
		float LastDisplacement;	
		FVector LocalVelocity;
		float SuspensionForce;

		FVector LocalOffset;
		float SpringDisplacement;
		uint32 SpringIndex;

		float LastSpringLength; // blend rather than jump to new location
		float AveragingLength[NUM_SUS_AVERAGING];
		int AveragingCount;
		int AveragingNum;
	};

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
