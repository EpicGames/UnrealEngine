// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMath.h"
#include "Math/UnrealMathSSE.h"
#include "Templates/Function.h"

DECLARE_LOG_CATEGORY_EXTERN(LogVehicleUtility, Log, All);


struct FSuspensionUtility
{
	//
	// Setup functions
	//

	/** Compute the distribution of the mass of a body among springs.
		This method assumes that spring positions are given relative
		to the center of mass of the body, and that gravity occurs
		in the local -Z direction.

		Returns true if it was able to find a valid mass configuration.
		If only one or two springs are included, then a valid
		configuration may not result in a stable suspension system -
		a bicycle or pogostick, for example, which is not perfectly centered
		may have a valid sprung mass configuration without being stable. */
	CHAOSVEHICLESCORE_API static bool ComputeSprungMasses(const TArray<FVector>& MassSpringPositions, const float TotalMass, TArray<float>& OutSprungMasses);

	/** Same as above, but allows the caller to specify spring locations
		in a local space which is not necessarily originated at the center
		of mass. */
	CHAOSVEHICLESCORE_API static bool ComputeSprungMasses(const TArray<FVector>& LocalSpringPositions, const FVector& LocalCenterOfMass, const float TotalMass, TArray<float>& OutSprungMasses);

	/** Natural frequency of spring in radians/sec - divide by 2Pi to get result in Hz */
	static float ComputeNaturalFrequency(float SpringRate, float SprungMass)
	{
		return FMath::Sqrt(SpringRate / SprungMass);
	}

	/** Compute spring damping value that will achieve critical damping */
	static float ComputeCriticalDamping(float SpringRate, float SprungMass)
	{
		// critical damping is at point when damping ratio is 1.0
		float NaturalFrequency = FMath::Sqrt(SpringRate / SprungMass);
		return 2.f * SprungMass * NaturalFrequency;
	}

	/** Compute spring damping value that will achieve the desired damping ratio. A damping ratio between 0.2-0.5 is typical of vehicle suspension */
	static float ComputeDamping(float SpringRate, float SprungMass, float DampingRatio)
	{
		DampingRatio = FMath::Clamp(DampingRatio, 0.f, 1.f);
		float NaturalFrequency = FMath::Sqrt(SpringRate / SprungMass);
		return DampingRatio * 2.f * SprungMass * NaturalFrequency;
	}
};
