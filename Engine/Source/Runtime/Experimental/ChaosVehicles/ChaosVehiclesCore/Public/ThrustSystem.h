// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "VehicleSystemTemplate.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{
	enum class EThrustType : uint8
	{
		Fixed = 0,
		Wing,
		Rudder,
		Elevator,
		HelicopterRotor
	};

	struct CHAOSVEHICLESCORE_API FSimpleThrustConfig
	{
		EThrustType Type;
		FVector Offset;
		FVector Axis;
		TArray<float> Curve;
		float MaxSpeed;
		float MaxThrustForce;
		float MaxControlAngle;
		// control axis
	};

	class CHAOSVEHICLESCORE_API FSimpleThrustSim : public TVehicleSystem<FSimpleThrustConfig>
	{
	public:

		FSimpleThrustSim(const FSimpleThrustConfig* SetupIn);

		void SetThrottle(float InThrottle)
		{
			ThrottlePosition = FMath::Clamp(InThrottle, -1.f, 1.f);
		}

		void SetPitch(float InPitch)
		{
			Pitch = -FMath::Clamp(InPitch, -1.f, 1.f) * Setup().MaxControlAngle;
		}

		void SetRoll(float InRoll)
		{
			Roll = FMath::Clamp(InRoll, -1.f, 1.f) * Setup().MaxControlAngle;
		}

		void SetYaw(float InYaw)
		{
			Yaw = FMath::Clamp(InYaw, -1.f, 1.f) * Setup().MaxControlAngle;
		}


		void SetWorldVelocity(const FVector& InVelocity)
		{
			WorldVelocity = InVelocity;
		}

		// Get functions
		const FVector& GetThrustForce() const
		{
			return ThrustForce;
		}

		const FVector& GetThrustDirection() const
		{
			return ThrustDirection;
		}

		const FVector GetThrustLocation() const;

		// simulate
		void Simulate(float DeltaTime);

	protected:
			
		float ThrottlePosition; // [0..1 Normalized position]

		FVector ThrustForce;
		FVector ThrustDirection;

		bool ThrusterStarted;	// is the 'engine' turned off or has it been started

		FVector WorldVelocity;

		float Pitch;
		float Roll;
		float Yaw;

	};

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif