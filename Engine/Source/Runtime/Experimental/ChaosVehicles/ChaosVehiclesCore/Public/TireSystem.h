// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VehicleSystemTemplate.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif


/**
 * #todo: Not really making use of this yet. 
 * Tire nuances more advanced than current simulation which just requires one friction value
 */

namespace Chaos
{
	struct FSimpleTireConfig
	{
		// Tires are normally specified as 192/55R15
		FSimpleTireConfig()
			: Radius(594.f / 2.f / 1000.0f)
			, Width(195.f / 1000.0f)
			, Depth(107.f / 1000.0f)
			, StaticFrictionCoefficient(0.5f)
			, KineticFrictionCoefficient(StaticFrictionCoefficient * 0.7f)
			, InitialTirePressure(0.f)
			, InitialTireWear(0.f)
		{

		}

		float Radius;
		float Width;
		float Depth;

		float StaticFrictionCoefficient;
		float KineticFrictionCoefficient;
		float InitialTirePressure;
		float InitialTireWear;

	};

	class FSimpleTireSim : public TVehicleSystem<FSimpleTireConfig>
	{
	public:
		FSimpleTireSim(const FSimpleTireConfig* SetupIn) 
			: TVehicleSystem<FSimpleTireConfig>(SetupIn)
			, TirePressure(0.f)
			, TireTemperature(0.f)
			, TireWear(0.f)
		{
		}

	private:

		float TirePressure;
		float TireTemperature;
		float TireWear;			// [0 no wear, 1 fully worn]

	};

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
