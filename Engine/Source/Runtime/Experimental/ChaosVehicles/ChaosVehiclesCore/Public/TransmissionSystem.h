// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/UnrealMathSSE.h"
#include "VehicleSystemTemplate.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION_SHIP
#endif

/**
 * Typical gear ratios: Reverse −4.181, 1st 3.818, 2nd 2.294, 3rd 1.500, 4th 1.133, 5th 0.911 
 * Source: Georg Rill. Road Vehicle Dynamics: Fundamentals and Modeling (Ground Vehicle Engineering Series) (p. 121). CRC Press.
 * 
 *	#todo: Add clutch option
 *	#todo: Proper defaults
 */

namespace Chaos
{
	enum ETransmissionType : uint8
	{
		Manual,
		Automatic
	};

	enum EDifferentialType : uint8
	{
		UndefinedDrive,
		AllWheelDrive,
		FrontWheelDrive,
		RearWheelDrive,
	};

	struct CHAOSVEHICLESCORE_API FSimpleDifferentialConfig
	{
		FSimpleDifferentialConfig()
			: DifferentialType(EDifferentialType::RearWheelDrive)
			, FrontRearSplit(0.5f)
		{
		}

		EDifferentialType DifferentialType;
		float FrontRearSplit;
	};

	class CHAOSVEHICLESCORE_API FSimpleDifferentialSim : public TVehicleSystem<FSimpleDifferentialConfig>
	{
	public:
		FSimpleDifferentialSim(const FSimpleDifferentialConfig* SetupIn) 
			: TVehicleSystem<FSimpleDifferentialConfig>(SetupIn)
			, FrontRearSplit(Setup().FrontRearSplit) {}

		float FrontRearSplit;
	};

	struct CHAOSVEHICLESCORE_API FSimpleTransmissionConfig
	{
		FSimpleTransmissionConfig()
			: FinalDriveRatio(1.f)
			, ChangeUpRPM(5000)
			, ChangeDownRPM(2500)
			, GearChangeTime(0.2f)
			, TransmissionEfficiency(1.f)
			, TransmissionType(ETransmissionType::Automatic)
			, AutoReverse(true)
		{
		}

		TArray<float> ForwardRatios;	// Gear ratios for forward gears
		TArray<float> ReverseRatios;	// Gear ratios for reverse Gear(s)
		float FinalDriveRatio;			// Final drive ratio [~4.0]

		uint32 ChangeUpRPM;				// [#todo: RPM or % max RPM?]
		uint32 ChangeDownRPM;			// [#todo: RPM or % max RPM?]
		float GearChangeTime; 			// [sec]

		float TransmissionEfficiency;	// Loss from friction in the system mean we might run at around 0.94 Efficiency

		ETransmissionType TransmissionType;	// Specify Automatic or Manual transmission

		bool AutoReverse;					// Arcade handling - holding Brake switches into reverse after vehicle has stopped
	};


	class CHAOSVEHICLESCORE_API FSimpleTransmissionSim : public TVehicleSystem<FSimpleTransmissionConfig>
	{
	public:
		FSimpleTransmissionSim(const FSimpleTransmissionConfig* SetupIn);

		//////////////////////////////////////////////////////////////////////////
		// Input functions

		/** set the target gear number to change to, can change gear immediately if specified
		 *  i.e. rather than waiting for the gear change time to elapse
		 */
		void SetGear(int32 InGear, bool Immediate = false);

		/** set the target gear to one higher than current target, will clamp gear index within rage */
		void ChangeUp()
		{
			SetGear(TargetGear + 1);		
		}

		/** set the target gear to one lower than current target, will clamp gear index within rage */
		void ChangeDown()
		{
			SetGear(TargetGear - 1);
		}

		/** Tell the transmission system what the current engine RPM is, so we can decide when to change up/down with automatic transmission type */
		void SetEngineRPM(float InRPM)
		{
			EngineRPM = InRPM;
		}

		void SetAllowedToChangeGear(bool OkToChangeIn)
		{
			AllowedToChangeGear = OkToChangeIn;
		}

		//////////////////////////////////////////////////////////////////////////
		// Output functions

		/** Get the current gear index, (reverse gears < 0, neutral 0, forward gears > 0) */
		int32 GetCurrentGear() const
		{
			return CurrentGear;
		}

		/** Get the target gear index, (reverse gears < 0, neutral 0, forward gears > 0) */
		int32 GetTargetGear() const
		{
			return TargetGear;
		}

		/** Get the current gear change time */
		float GetCurrentGearChangeTime() const
		{
			return CurrentGearChangeTime;
		}

		/** Set the current gear index, (reverse gears < 0, neutral 0, forward gears > 0) */
		void SetCurrentGear(const int32 InCurrentGear) 
		{
			CurrentGear = InCurrentGear;
		}

		/** Set the target gear index, (reverse gears < 0, neutral 0, forward gears > 0) */
		void SetTargetGear(const int32 InTargetGear) 
		{
			TargetGear = InTargetGear;
		}

		/** Set the current gear change time */
		void SetCurrentGearChangeTime(const float InCurrentGearChangeTime) 
		{
			CurrentGearChangeTime = InCurrentGearChangeTime;
		}

		/** Are we currently in the middle of a gear change */
		bool IsCurrentlyChangingGear() const
		{
			return CurrentGear != TargetGear;
		}

		bool IsOutOfGear() const
		{
			return (CurrentGear == 0) || IsCurrentlyChangingGear();
		}

		/** Get the final combined gear ratio for the specified gear (reverse gears < 0, neutral 0, forward gears > 0) */
		float GetGearRatio(int32 InGear);

		/** Get the transmission RPM, from the specified engine RPM and gear selection */
		float GetTransmissionRPM(float InEngineRPM, int InGear)
		{
			if (InGear == 0) // neutral, don't want to divide by zero
			{
				return 0.0f;
			}

			return InEngineRPM / GetGearRatio(InGear);
		}

		/** Get the transmission RPM for the current state of the engine RPM and gear selection */
		float GetTransmissionRPM()
		{
			return GetTransmissionRPM(EngineRPM, CurrentGear);
		}

		/** Given the engine torque return the transmission torque taking into account the gear ratios and transmission losses */
		float GetTransmissionTorque(float InEngineTorque)
		{
			return InEngineTorque * GetGearRatio(GetCurrentGear()) * Setup().TransmissionEfficiency;
		}

		///** Given the transmission torque return the engine torque after taking into account the gear ratios */
		//float GetEngineTorque(float InTransmissionTorque)
		//{
		//	return InTransmissionTorque / GetGearRatio(GetCurrentGear()); // #todo: what about transmission frictional losses
		//}

		/** Get the expected engine RPM from the wheel RPM taking into account the current gear ratio (assuming no clutch slip) */
		float GetEngineRPMFromWheelRPM(float InWheelRPM)
		{
			return InWheelRPM * GetGearRatio(GetCurrentGear());
		}

		/*
		 * Simulate - update internal state 
		 * - changes gear when using automatic transmission
		 * - implements gear change time, where gear goes through neutral
		 */
		void Simulate(float DeltaTime);
	
		void CorrectGearInputRange(int32& GearIndexInOut)
		{
			GearIndexInOut = FMath::Clamp(GearIndexInOut, -Setup().ReverseRatios.Num(), Setup().ForwardRatios.Num());
		}

	private:

		int32 CurrentGear; // <0 reverse gear(s), 0 neutral, >0 forward gears
		int32 TargetGear;  // <0 reverse gear(s), 0 neutral, >0 forward gears
		float CurrentGearChangeTime; // Time to change gear, no power transmitted to the wheels during change

		float EngineRPM;	// Engine Revs Per Minute

		bool AllowedToChangeGear; /** conditions are ok for an automatic gear change */

	};

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION_SHIP
#endif

