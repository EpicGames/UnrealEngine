// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "SimpleVehicle.h"
#include "TransmissionSystem.h"
#include "WheelSystem.h"

#define UE_API CHAOSVEHICLESCORE_API

namespace Chaos
{

	class FTransmissionUtility
	{
	public:

		static bool IsWheelPowered(const EDifferentialType DifferentialType, const FSimpleWheelSim& PWheel)
		{
			return IsWheelPowered(DifferentialType, PWheel.Setup().AxleType, PWheel.EngineEnabled);
		}

		static UE_API bool IsWheelPowered(const EDifferentialType DifferentialType, const FSimpleWheelConfig::EAxleType AxleType, const bool EngineEnabled = false);

		static UE_API int GetNumWheelsOnAxle(FSimpleWheelConfig::EAxleType AxleType, const TArray<FSimpleWheelSim>& Wheels);

		static UE_API int GetNumDrivenWheels(const TArray<FSimpleWheelSim>& Wheels);

		static UE_API float GetTorqueRatioForWheel(const FSimpleDifferentialSim& PDifferential, const int WheelIndex, const TArray<FSimpleWheelSim>& Wheels);
	};

}


#undef UE_API
