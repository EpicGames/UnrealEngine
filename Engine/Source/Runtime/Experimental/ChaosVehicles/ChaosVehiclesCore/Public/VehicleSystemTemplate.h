// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// includes common to all vehicle systems
#include "CoreMinimal.h"

/**
 * Code common between all vehicle systems
 */
template <typename T>
class TVehicleSystem
{
public:

	TVehicleSystem() : SetupPtr(nullptr)
	{
	}

	TVehicleSystem(const T* SetupIn) : SetupPtr(SetupIn)
	{
		check(SetupPtr != nullptr);
	}

	T& AccessSetup()
	{
		check(SetupPtr != nullptr);
		return (T&)(*SetupPtr);
	}

	const T& Setup() const
	{
		check(SetupPtr != nullptr);
		return (*SetupPtr);
	}

	/*
	 * Setup data pointer is public if you want to use it directly
	 */
	const T* SetupPtr;
};