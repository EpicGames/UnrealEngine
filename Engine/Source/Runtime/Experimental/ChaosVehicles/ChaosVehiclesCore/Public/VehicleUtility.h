// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Core.h"
#include "Chaos/Real.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/PlatformCrt.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"

#define UE_API CHAOSVEHICLESCORE_API

// Disable Optimizations in non debug build configurations
#define VEHICLE_DEBUGGING_ENABLED 0

namespace Chaos
{

	struct RealWorldConsts
	{
		FORCEINLINE static float WaterDensity()
		{
			return 997.0f; // kg / m3;
		}

		FORCEINLINE static float AirDensity()
		{
			return 1.225f; // kg / m3;
		}

		FORCEINLINE static float HalfAirDensity()
		{
			return 0.6125f; // kg / m3;
		}

		FORCEINLINE static float DryRoadFriction()
		{
			return 0.7f; // friction coefficient
		}

		FORCEINLINE static float WetRoadFriction()
		{
			return 0.4f; // friction coefficient
		}

	};

	class FNormalisedGraph
	{
	public:
		void Empty()
		{
			Graph.Empty();
		}

		void AddNormalized(float Value)
		{
			Graph.Add(Value);
		}

		float GetValue(float InX, float MaxX = 1.0f, float MaxY = 1.0f) const
		{
			float Step = MaxX / (Graph.Num() - 1);
			int StartIndex = InX / Step;
			float NormalisedRamp = ((float)InX - (float)StartIndex * Step) / Step;

			float NormYValue = 0.0f;
			if (StartIndex >= Graph.Num() - 1)
			{
				NormYValue = Graph[Graph.Num() - 1];
			}
			else
			{
				NormYValue = Graph[StartIndex] * (1.f - NormalisedRamp) + Graph[StartIndex + 1] * NormalisedRamp;
			}

			return NormYValue * MaxY;
		}
	private:
		TArray<float> Graph;
	};

	class FGraph
	{
	public:
		FGraph()
		{
			Empty();
		}

		void Empty()
		{
			Graph.Empty();
			BoundsX.X = TNumericLimits<FReal>::Max();
			BoundsX.Y = -TNumericLimits<FReal>::Max();
			BoundsY.X = TNumericLimits<FReal>::Max();
			BoundsY.Y = -TNumericLimits<FReal>::Max();
		}

		UE_API void Add(const FVec2& Value);

		UE_API float EvaluateY(float InX) const;

		bool IsEmpty() const { return Graph.IsEmpty(); }
	private:
		TArray<FVec2> Graph;
		FVector2D BoundsX;
		FVector2D BoundsY;
	};

	class FVehicleUtility
	{
	public:
		/** clamp value between 0 and 1 */
		FORCEINLINE static void ClampNormalRange(float& InOutValue)
		{
			InOutValue = FMath::Clamp(InOutValue, 0.f, 1.f);
		}

		/** Calculate Yaw angle in Radians from a Normalized Forward facing vector */
		static float YawFromForwardVectorRadians(const FVector& NormalizedForwardsVector)
		{
			return FMath::Atan2(NormalizedForwardsVector.Y, NormalizedForwardsVector.X);
		}

		/** Calculate Pitch angle in Radians from a Normalized Forward facing vector */
		static float PitchFromForwardVectorRadians(const FVector& NormalizedForwardsVector)
		{
			return FMath::Atan2(NormalizedForwardsVector.Z, FMath::Sqrt(NormalizedForwardsVector.X * NormalizedForwardsVector.X + NormalizedForwardsVector.Y * NormalizedForwardsVector.Y));
		}

		/** Calculate Roll angle in Radians from a Normalized Right facing vector */
		static float RollFromRightVectorRadians(const FVector& NormalizedRightVector)
		{
			return FMath::Atan2(NormalizedRightVector.Z, FMath::Sqrt(NormalizedRightVector.X * NormalizedRightVector.X + NormalizedRightVector.Y * NormalizedRightVector.Y));
		}

		/** Calculate turn radius from three points. Note: this function is quite inaccurate for large radii. Return 0 if there is no answer, i.e. points lie on a line */
		static UE_API float TurnRadiusFromThreePoints(const FVector& PtA, const FVector& PtB, const FVector& PtC);

		static float CalculateSlipAngle(float Y, float X)
		{
			float Value = 0.0f;

			float LateralSpeedThreshold = 0.05f;
			if (FMath::Abs(Y) > LateralSpeedThreshold)
			{
				Value = FMath::Abs(FMath::Atan2(Y, X));
				if (Value > HALF_PI)
				{
					Value = PI - Value;
				}
			}

			return Value;
		}
	};

	FORCEINLINE float MToCmScaling()
	{
		return 100.f;
	}

	FORCEINLINE float CmToMScaling()
	{
		return 0.01f;
	}

	/** revolutions per minute to radians per second */
	FORCEINLINE float RPMToOmega(float RPM)
	{
		return RPM * PI / 30.f;
	}

	/** radians per second to revolutions per minute */
	FORCEINLINE float OmegaToRPM(float Omega)
	{
		return Omega * 30.f / PI;
	}

	/** km/h to cm/s */
	FORCEINLINE float KmHToCmS(float KmH)
	{
		return KmH * 100000.f / 3600.f;
	}

	/** cm/s to km/h */
	FORCEINLINE float CmSToKmH(float CmS)
	{
		return CmS * 3600.f / 100000.f;
	}

	/** cm/s to miles per hour */
	FORCEINLINE float CmSToMPH(float CmS)
	{
		return CmS * 2236.94185f / 100000.f;
	}

	/** miles per hour to cm/s */
	FORCEINLINE float MPHToCmS(float MPH)
	{
		return MPH * 100000.f / 2236.94185f;
	}

	/** miles per hour to meters per second */
	FORCEINLINE float MPHToMS(float MPH)
	{
		return MPH * 1609.34f / 3600.f;
	}

	/** meters per second to miles per hour */
	FORCEINLINE float MSToMPH(float MS)
	{
		return MS * 3600.f / 1609.34f;
	}

	/** cm to meters */
	FORCEINLINE float CmToM(float Cm)
	{
		return Cm * 0.01f;
	}

	/** cm to meters */
	FORCEINLINE FVector CmToM(const FVector& Cm)
	{
		return Cm * 0.01f;
	}

	/** meters to cm */
	FORCEINLINE float MToCm(float M)
	{
		return M * 100.0f;
	}

	/** cm to meters */
	FORCEINLINE FVector MToCm(const FVector& M)
	{
		return M * 100.0f;
	}

	/** cm to meters - warning loss of precision */
	FORCEINLINE float CmToMiles(float Cm)
	{
		return Cm * 0.0000062137119224f;
	}


	/** Km to miles */
	FORCEINLINE float KmToMile(float Km)
	{
		return Km * 0.62137f;
	}

	/** miles to Km */
	FORCEINLINE float MileToKm(float Miles)
	{
		return Miles * 1.60934f;
	}

	/** meters squared to cm squared */
	FORCEINLINE float M2ToCm2(float M2)
	{
		return M2 * 100.f * 100.f;
	}

	/** cm squared to meters squared */
	FORCEINLINE float Cm2ToM2(float Cm2)
	{
		return Cm2 / (100.f * 100.f);
	}

	FORCEINLINE float DegToRad(float InDeg)
	{
		return InDeg * PI / 180.f;
	}

	FORCEINLINE float RadToDeg(float InRad)
	{
		return InRad * 180.f / PI;
	}

	FORCEINLINE float Sqr(float Val)
	{
		return Val * Val;
	}

	FORCEINLINE float TorqueMToCm(float TorqueIn)
	{
		return TorqueIn * 10000.0f;
	}

	FORCEINLINE float TorqueCmToM(float TorqueIn)
	{
		return TorqueIn / 10000.0f;
	}

	class FTimeAndDistanceMeasure
	{
	public:
		UE_API FTimeAndDistanceMeasure(const FString& DescriptionIn, float InitialVelocityIn, float TargetVelocityIn, float TargetDistanceIn);

		UE_API void Reset();

		bool IsComplete() const { return MeasurementComplete; }

		UE_API void Update(float DeltaTime, const FVector& CurrentLocation, float CurrentVelocity);

		UE_API FString ToString() const;

	private:
		FString Description;
		bool PreStartConditionsMet;
		bool StartConditionsMet;
		bool MeasurementComplete;

		FVector InitialLocation;
		double InitialTime;

		float InitialVelocityMPH;
		float FinalTargetVelocityMPH;
		float FinalTargetDistanceMiles;

		float VelocityResultMPH;
		float DistanceResultMiles;
		float TimeResultSeconds;
	};

	class FPerformanceMeasure
	{
	public:
		enum class EMeasure : uint8
		{
			ZeroToThirtyMPH = 0,
			ZeroToSixtyMPH,
			QuarterMile,
			ThirtyToZeroMPH,
			SixtyToZeroMPH,
		};

		UE_API FPerformanceMeasure();

		void AddMeasure(FTimeAndDistanceMeasure& MeasureIn)
		{
			PerformanceMeasure.Add(MeasureIn);
		}

		const FTimeAndDistanceMeasure& GetMeasure(int MeasurementIdx)
		{
			return PerformanceMeasure[MeasurementIdx];
		}

		void ResetAll()
		{
			for (int I = 0; I < PerformanceMeasure.Num(); I++)
			{
				PerformanceMeasure[I].Reset();
			}
		}

		void Update(float DeltaTime, const FVector& CurrentLocation, float CurrentVelocity)
		{
			for (int I = 0; I < PerformanceMeasure.Num(); I++)
			{
				PerformanceMeasure[I].Update(DeltaTime, CurrentLocation, CurrentVelocity);
			}
		}

		int GetNumMeasures() const
		{
			return PerformanceMeasure.Num();
		}

		void Enable()
		{
			IsEnabledThisFrame = true;
		}

		bool IsEnabled() const
		{
			return IsEnabledThisFrame;
		}

	private:

		bool IsEnabledThisFrame;
		TArray<FTimeAndDistanceMeasure> PerformanceMeasure;
	};

} // namespace Chaos

#undef UE_API
