// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "TireSystem.h"
#include "VehicleSystemTemplate.h"
#include "VehicleUtility.h"

#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION
#endif

namespace Chaos
{

/**
 * Tire load changes, when cornering outer tires are loaded and inner ones unloaded
 * Similarly load changes when accelerating and breaking.
 * Fx : brake/drive force
 * Fy : Cornering Force
 * Fz : Tire load (vehicle weight)
 *
 * Mx : overturning moment
 * My : moment about brake/drive torque
 * Mz : self-aligning moment
 *
 * Fx : forward speed at wheel center
 *
 *
 * Omega : Rotational Speed [radians/sec]
 * Alpha : Slip Angle [radians]
 * k or Sx: Longitudinal Slip - slip is -ve when braking and +ve when accelerating
 * Re : Effective Wheel Radius
 */

/**
* Wheel setup data that doesn't change during simulation
*/
struct CHAOSVEHICLESCORE_API FSimpleWheelConfig
{
	// #todo: use this
	enum EWheelDamageStatus
	{
		NONE,
		BUCKLED,
		FLAT,
		MISSING
	};

	// #todo: use this
	enum EWheelSimulationStatus
	{
		ROLLING,	// wheel speed matches the vehicle ground speed
		SPINNING,	// wheel speed faster than vehicle ground speed
		LOCKED		// wheel is locked and sliding over surface
	};

	// #todo: use this
	enum EFrictionCombineMethod
	{
		Multiply, // default - most correct
		Average
	};

	enum EAxleType : uint8
	{
		UndefinedAxle = 0,
		Front,
		Rear
	};
	
	enum EExternalTorqueCombineMethod : uint8
	{
		None = 0,
		Override,
		Additive
	};

	FSimpleWheelConfig() 
		: Offset(FVector(2.f, 1.f, 0.f))
		, WheelMass(20.f) // [Kg]
		, WheelRadius(30.f) // [cm]
		, WheelWidth(20.f)
		, MaxSteeringAngle(70)
		, MaxBrakeTorque(2000.f)
		, HandbrakeTorque(1000.f)
		, ABSEnabled(false)
		, BrakeEnabled(true)
		, HandbrakeEnabled(true)
		, SteeringEnabled(true)
		, EngineEnabled(false)
		, TractionControlEnabled(false)
		, TorqueRatio(0.f)
		, AxleType(EAxleType::UndefinedAxle)
		, FrictionCombineMethod(EFrictionCombineMethod::Multiply)
		, ExternalTorqueCombineMethod(EExternalTorqueCombineMethod::None)
		, FrictionMultiplier(2.0f)
		, LateralSlipGraphMultiplier(1.0f)
		, CorneringStiffness(1000.0f)
		, SideSlipModifier(1.0f)
		, SlipThreshold(20.0f)
		, SkidThreshold(20.0f)
		, MaxSpinRotation(30.0f)
	{

	}

	// Basic
	//FSimpleTireConfig Tire;

	// wheel tire
	FVector Offset;
	float WheelMass;			// Mass of wheel [Kg]
	float WheelRadius;			// [cm]
	float WheelWidth;			// [cm]

	int	 MaxSteeringAngle;		// Yaw angle of steering [Degrees]

	// brakes
	float MaxBrakeTorque;		// Braking Torque [Nm]
	float HandbrakeTorque;		// Handbrake Torque [Nm]
	bool ABSEnabled;			// Advanced braking system operational

	// setup
	bool BrakeEnabled;			// Regular brakes are enabled for this wheel
	bool HandbrakeEnabled;		// Handbrake is operational on this wheel
	bool SteeringEnabled;		// Steering is operational on this wheel
	bool EngineEnabled;			// Wheel is driven by an engine
	bool TractionControlEnabled;// Straight Line Traction Control
	float TorqueRatio;			// Portion of torque going to this wheel (0->1)

	EAxleType AxleType;

	EFrictionCombineMethod FrictionCombineMethod; //#todo: use this variable
	EExternalTorqueCombineMethod ExternalTorqueCombineMethod;

	float FrictionMultiplier;
	float LateralSlipGraphMultiplier;
	float CorneringStiffness;
	float SideSlipModifier;

	float SlipThreshold;
	float SkidThreshold;

	FGraph LateralSlipGraph;
	float MaxSpinRotation;

	// #todo: simulated Damage
	//EWheelDamageStatus DamageStatus;
	//float BuckleAngle;
};


/**
* Wheel instance data changes during the simulation
*/
class CHAOSVEHICLESCORE_API FSimpleWheelSim : public TVehicleSystem<FSimpleWheelConfig>
{
public:

	FSimpleWheelSim(const FSimpleWheelConfig* SetupIn);

// Inputs

	/** Set the wheel radius - can change dynamically during simulation if desired */
	void SetWheelRadius(float NewRadius)
	{
		Re = NewRadius;
	}

	/** Set the angular position in radians */
	void SetAngularPosition(float PositionIn)
	{
		AngularPosition = PositionIn;
	}

	/** Set the angular position in radians/sec */
	void SetAngularVelocity(float AngularVelocityIn)
	{
		Omega = AngularVelocityIn;
	}

	/** set wheel rotational speed to match the specified linear forwards speed */
	void SetMatchingSpeed(float LinearMetersPerSecondIn)
	{
		Omega = LinearMetersPerSecondIn / Re;
	}

	/** Set the braking torque - decelerating rotational force */
	void SetBrakeTorque(float BrakeTorqueIn, bool bEngineBrakingIn = false)
	{
		BrakeTorque = BrakeTorqueIn;
		bEngineBraking = bEngineBrakingIn;
	}

	/** Set the drive torque - accelerating rotational force */
	void SetDriveTorque(float EngineTorqueIn)
	{
		DriveTorque = EngineTorqueIn;
	}

	void SetTorqueCombineMethod(FSimpleWheelConfig::EExternalTorqueCombineMethod InCombineMethod)
	{
		ExternalTorqueCombineMethod = InCombineMethod;
	}

	void SetBrakeTorqueOverride(float BrakeTorqueIn, bool bEngineBrakingIn = false)
	{
		ExternalBrakeTorque = BrakeTorqueIn;
		bEngineBraking = bEngineBrakingIn;
	}

	void SetDriveTorqueOverride(float EngineTorqueIn)
	{
		ExternalDriveTorque = EngineTorqueIn;
	}

	/** Set the vehicle's speed at the wheels location in local wheel coords */
	void SetVehicleGroundSpeed(const FVector& VIn)
	{
		GroundVelocityVector = VIn;
	}

	/** Set the force pressing the wheel into the terrain - from suspension */
	void SetWheelLoadForce(float WheelLoadForceIn)
	{
		ForceIntoSurface = WheelLoadForceIn;
		
		if (ForceIntoSurface > SMALL_NUMBER)
		{
			bInContact = true;
		}
		else
		{
			bInContact = false;
		}
	}

	/** Set the friction coefficient of the surface under the wheel */
	void SetSurfaceFriction(float InFriction)
	{
		SurfaceFriction = InFriction;
	}

	void SetOnGround(bool OnGround)
	{
		bInContact = OnGround;
	}

	void SetSteeringAngle(float InAngle)
	{
		SteeringAngle = InAngle;
	}

	void SetMaxOmega(float InMaxOmega)
	{
		MaxOmega = InMaxOmega;
	}

	void SetWheelIndex(uint32 InIndex)
	{
		WheelIndex = InIndex;
	}

// Outputs

	/**
	 * Amount of friction we can expect after taking into account the amount the wheel slips
	 */
	static float GetNormalisedFrictionFromSlipAngle(float SlipIn)
	{
		FVehicleUtility::ClampNormalRange(SlipIn);

		// typical slip angle graph; normalized scales
		// Friction between 0 and 1 for values of slip between 0 and 1
		float FunctionResult = 1.125f * (1.0f - exp(-20.0f * SlipIn)) - 0.25f * SlipIn;
		return FMath::Max(0.0f, FMath::Min(1.0f, FunctionResult));
	}

	/** return the calculated available friction force */
	FVector GetForceFromFriction() const
	{
		return ForceFromFriction;
	}

	/** Get the radius of the wheel [cm] */
	float GetEffectiveRadius() const
	{
		return Re;
	}

	/** Get the angular position of the wheel [radians] */
	float GetAngularPosition() const
	{
		return AngularPosition;
	}

	/** Get the angular velocity of the wheel [radians/sec] */
	float GetAngularVelocity() const
	{
		return Omega;
	}

	/** Get the wheel RPM [revolutions per minute] */
	float GetWheelRPM()
	{
		return OmegaToRPM(Omega);
	}

	/** Is the wheel in contact with the terrain or another object */
	bool InContact() const
	{
		return bInContact;
	}

	bool IsSlipping() const
	{ 
		return (FMath::Abs(GetSlipMagnitude()) > Setup().SlipThreshold);
	}

	bool IsSkidding() const
	{
		return (FMath::Abs(GetSkidMagnitude()) > Setup().SkidThreshold);
	}
	bool IsABSActivated() const
	{
		return bABSActivated;
	}

	float GetSteeringAngle() const
	{
		return SteeringAngle;
	}

	/** Get the current longitudinal slip value [0 no slip - using static friction, 1 full slip - using dynamic friction] */
	float GetNormalizedLongitudinalSlip() const
	{
		return Sx;
	}

	float GetNormalizedLateralSlip() const
	{
		return FMath::Clamp(RadToDeg(SlipAngle) / 30.0f, 0.f, 1.f);
	}

	/** Get the magnitude of the force pressing the wheel into the terrain */
	float GetWheelLoadForce() const
	{
		return ForceIntoSurface;
	}

	/** Get the friction coefficient of the surface in contact with the wheel */
	float GetSurfaceFriction() const
	{
		return SurfaceFriction;
	}

	/** Get the slip angle for this wheel - angle between wheel forward axis and velocity vector [degrees] */
	float GetSlipAngle() const
	{
		return SlipAngle;
	}

	/** Get the drive torque being applied to the wheel [N.m] */
	float GetDriveTorque() const
	{
		return DriveTorque;
	}

	/** Get the braking torque being applied to the wheel [N.m] */
	float GetBrakeTorque() const
	{
		return BrakeTorque;
	}

	/** Get the road speed at the wheel */
	float GetRoadSpeed() const
	{
		return GroundVelocityVector.X;
	}

	/** Get the linear ground speed of the wheel based on its current rotational speed */
	float GetWheelGroundSpeed() const
	{
		return Omega * Re;
	}

	/** Get the difference between the wheel speed and the effective ground speed of the vehicle at the wheel
	  * positive if wheel is faster than effective ground speed, negative if wheel is slower */
	float GetSlipMagnitude() const
	{
		return GetWheelGroundSpeed() - GetRoadSpeed();
	}

	/** Get the effective ground speed along the lateral wheel axis
      * positive if wheel is faster than effective ground speed, negative if wheel is slower */
	float GetSkidMagnitude() const
	{
		return GroundVelocityVector.Y;
	}


	/** 
	 * Simulate - figure out wheel lateral and longitudinal forces based on available friction at the wheel
	 *	Wheel load force from body weight and the surface friction together determine the grip available at the wheel
	 *	DriveTorque accelerates the wheel
	 *	BrakeTorque decelerates the wheel
	 */
	void Simulate(float DeltaTime);


	void SetMassPerWheel(float VehicleMassPerWheel)
	{
		MassPerWheel = VehicleMassPerWheel;
	}

public:
	bool BrakeEnabled;			// Regular brakes are enabled for this wheel
	bool HandbrakeEnabled;		// Handbrake is operational on this wheel
	bool SteeringEnabled;		// Steering is operational on this wheel
	bool EngineEnabled;			// Wheel is driven by an engine
	bool TractionControlEnabled;// Straight Line Traction Control
	bool ABSEnabled;			// Advanced braking system operational
	float FrictionMultiplier;
	float LateralSlipGraphMultiplier;
	float CorneringStiffness;
	float MaxSteeringAngle;
	float MaxBrakeTorque;
	float HandbrakeTorque;
	FSimpleWheelConfig::EExternalTorqueCombineMethod ExternalTorqueCombineMethod;

	float Re;		// [cm] Effective Wheel Radius could change dynamically if get a flat?, tire shreds
	float Omega;	// [radians/sec] Wheel Rotation Angular Velocity
	float Sx;
	float Inertia;
	// In
	float DriveTorque;				// [N.m]
	float BrakeTorque;				// [N.m]
	float ForceIntoSurface;			// [N]
	FVector GroundVelocityVector;	// [Unreal Units cm.s-1]
	float AngularPosition;			// [radians]
	float SteeringAngle;			// [degrees ATM]
	float SurfaceFriction;
	float MaxOmega;

	float ExternalDriveTorque;		// [N.m]
	float ExternalBrakeTorque;		// [N.m]

	FVector ForceFromFriction;
	float MassPerWheel;

	// Wheel transform

	float SlipVelocity;			// Relative velocity between tire patch and ground ?? vector ??
	float SlipAngle;			// Angle between wheel forwards and velocity vector
	bool bInContact;			// Is tire in contact with the ground or free in the air
	uint32 WheelIndex;			// purely for debugging purposes
	bool bEngineBraking;		// Is the braking force coming from the engine

	public:
	// debug for now
	float AppliedLinearDriveForce;
	float AppliedLinearBrakeForce;
	float LongitudinalAdhesiveLimit;
	float LateralAdhesiveLimit;
	float SideSlipModifier;
	float Spin;
	float AvailableGrip;
	FVector InputForces;
	bool bClipping;
	bool bABSActivated;

};


struct CHAOSVEHICLESCORE_API FAxleConfig
{
	TArray<uint16> WheelIndex;	// reference to wheels on this axle
	float RollbarScaling;
};

class CHAOSVEHICLESCORE_API FAxleSim : public TVehicleSystem<FAxleConfig>
{
public:

	FAxleSim();

	void Simulate(float DeltaTime) {}

	FAxleConfig Setup;

};

} // namespace Chaos

#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION
#endif
