// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVehicles.h"

UChaosVehicles::UChaosVehicles(const FObjectInitializer& ObjectInitializer)
	: UObject(ObjectInitializer)
{
	check(ObjectInitializer.GetClass() == GetClass());
}


