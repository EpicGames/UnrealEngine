// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Physics/Experimental/PhysScene_Chaos.h"

#include "ChaosVehicles.generated.h"

class UChaosVehicles;

/**
* UChaosVehicles (UObject)
*
*/
UCLASS(customconstructor)
class CHAOSVEHICLESENGINE_API UChaosVehicles : public UObject
{
	GENERATED_UCLASS_BODY()

public:
	UChaosVehicles(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	bool IsVisible() { return true; }
};



