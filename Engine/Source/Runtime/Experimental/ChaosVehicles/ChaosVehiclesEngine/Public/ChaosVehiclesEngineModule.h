// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class CHAOSVEHICLESENGINE_API FChaosVehiclesEngineModule : public IModuleInterface
{
public:
	virtual void StartupModule() override {};
	virtual void ShutdownModule() override {};
};
