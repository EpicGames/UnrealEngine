// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using System.Collections.Generic;

namespace UnrealBuildTool.Rules
{
	public class ChaosVDRuntime : ModuleRules
	{
		public ChaosVDRuntime(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"TraceLog",
					"TraceBasedDebuggers",
				}
			);

			if (IsChaosVisualDebuggerSupported(Target))
			{
				PublicDefinitions.Add("WITH_CHAOS_VISUAL_DEBUGGER=1");
			}
			else
			{
				PublicDefinitions.Add("WITH_CHAOS_VISUAL_DEBUGGER=0");
			}

		}
	}
}