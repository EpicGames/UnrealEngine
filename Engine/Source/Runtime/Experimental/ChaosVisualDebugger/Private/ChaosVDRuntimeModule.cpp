// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDRuntimeModule.h"

#include "Modules/ModuleManager.h"

#if WITH_CHAOS_VISUAL_DEBUGGER

#include "HAL/IConsoleManager.h"

#if WITH_EDITOR
#include "Misc/ScopedSlowTask.h"
#endif

IMPLEMENT_MODULE(FChaosVDRuntimeModule, ChaosVDRuntime);

DEFINE_LOG_CATEGORY(LogChaosVDRuntime);

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FAutoConsoleCommand ChaosVDStartRecordingCommand(
	TEXT("p.Chaos.StartVDRecording"),
	TEXT("Turn on the recording of debugging data"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		UE_AUTORTFM_ONCOMMIT(Args)
		{
			FChaosVDRuntimeModule::Get().StartRecording(Args);
		};
	})
);

FAutoConsoleCommand StopVDStartRecordingCommand(
	TEXT("p.Chaos.StopVDRecording"),
	TEXT("Turn off the recording of debugging data"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		UE_AUTORTFM_ONCOMMIT(Args)
		{
			FChaosVDRuntimeModule::Get().StopRecording();
		};
	})
);

static FAutoConsoleVariable CVarChaosVDGTimeBetweenFullCaptures(
	TEXT("p.Chaos.VD.TimeBetweenFullCaptures"),
	10,
	TEXT("Time interval in seconds after which a full capture (not only delta changes) should be recorded"));

FChaosVDCaptureRequestDelegate FChaosVDRuntimeModule::PerformFullCaptureDelegate = FChaosVDCaptureRequestDelegate();
UE::TraceBasedDebuggers::FRecordingStateChangedDelegate FChaosVDRuntimeModule::RecordingStartedDelegate = UE::TraceBasedDebuggers::FRecordingStateChangedDelegate();
UE::TraceBasedDebuggers::FRecordingStateChangedDelegate FChaosVDRuntimeModule::PostRecordingStartedDelegate = UE::TraceBasedDebuggers::FRecordingStateChangedDelegate();
UE::TraceBasedDebuggers::FRecordingStateChangedDelegate FChaosVDRuntimeModule::RecordingStopDelegate = UE::TraceBasedDebuggers::FRecordingStateChangedDelegate();
UE::TraceBasedDebuggers::FRecordingStartFailedDelegate FChaosVDRuntimeModule::RecordingStartFailedDelegate = UE::TraceBasedDebuggers::FRecordingStartFailedDelegate();
FTransactionallySafeRWLock FChaosVDRuntimeModule::DelegatesRWLock = FTransactionallySafeRWLock();


FChaosVDRuntimeModule::FChaosVDRuntimeModule()
	: FRuntimeModule(LogChaosVDRuntime, Chaos::VD::DebuggerGuid)
{
	DebuggerName = TEXT("ChaosVD");
	AutoRecordCommand = TEXT("StartCVDRecording");
	HostAddressCommand = TEXT("CVDHost");
}

FChaosVDRuntimeModule& FChaosVDRuntimeModule::Get()
{
	return FModuleManager::Get().LoadModuleChecked<FChaosVDRuntimeModule>(TEXT("ChaosVDRuntime"));
}

void FChaosVDRuntimeModule::StartupModule()
{
	if (!TryRecordOnStartup())
	{
#if UE_TRACE_ENABLED || UE_TRACE_MINIMAL_ENABLED
		UE::Trace::ToggleChannel(TEXT("ChaosVDChannel"), false);
#endif
	}
}

void FChaosVDRuntimeModule::ShutdownModule()
{
	if (IsRecording())
	{
		StopRecording();
	}
}

int32 FChaosVDRuntimeModule::GenerateUniqueID()
{
	int32 NewID = 0;
	UE_AUTORTFM_OPEN
	{
		NewID = LastGeneratedID++;
	};

	return NewID;
}

FString FChaosVDRuntimeModule::GetLastRecordingFileNamePath() const
{
	return FString();
}

void FChaosVDRuntimeModule::OnEnableRequiredTraceChannelsInternal()
{
#if UE_TRACE_ENABLED || UE_TRACE_MINIMAL_ENABLED
	UE::Trace::ToggleChannel(TEXT("ChaosVDChannel"), true);
#endif

#if UE_TRACE_ENABLED
	UE::Trace::ToggleChannel(TEXT("Log"), true);
#endif
}

void FChaosVDRuntimeModule::OnRecordingStartedInternal()
{
	constexpr int32 MinAllowedTimeInSecondsBetweenCaptures = 1;
	const int32 ConfiguredTimeBetweenCaptures = CVarChaosVDGTimeBetweenFullCaptures->GetInt();

	ensureAlwaysMsgf(ConfiguredTimeBetweenCaptures > MinAllowedTimeInSecondsBetweenCaptures,
		TEXT("The minimum allowed time interval between full captures is [%d] seconds, but [%d] seconds were configured. Clamping to [%d] seconds"),
		MinAllowedTimeInSecondsBetweenCaptures, ConfiguredTimeBetweenCaptures, MinAllowedTimeInSecondsBetweenCaptures);

	FullCaptureRequesterHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FChaosVDRuntimeModule::RequestFullCapture),
		FMath::Clamp(ConfiguredTimeBetweenCaptures, MinAllowedTimeInSecondsBetweenCaptures, TNumericLimits<int32>::Max()));

	{
		UE::TReadScopeLock ReadLock(DelegatesRWLock);
		RecordingStartedDelegate.Broadcast();
	}

	{
		UE::TReadScopeLock ReadLock(DelegatesRWLock);
		PostRecordingStartedDelegate.Broadcast();
	}
}

void FChaosVDRuntimeModule::OnRecordingStartFailedInternal(const FText& FailureReason)
{
	{
		UE::TReadScopeLock ReadLock(DelegatesRWLock);
		RecordingStartFailedDelegate.Broadcast(FailureReason);
	}
}

void FChaosVDRuntimeModule::OnRecordingStoppedInternal()
{
	{
		UE::TReadScopeLock ReadLock(DelegatesRWLock);
		RecordingStopDelegate.Broadcast();
	}

	if (FullCaptureRequesterHandle.IsValid())
	{
		FTSTicker::RemoveTicker(FullCaptureRequesterHandle);

		FullCaptureRequesterHandle.Reset();
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

FChaosVDTraceDetails FChaosVDRuntimeModule::GetCurrentTraceSessionDetails() const
{
	return {};
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS


bool FChaosVDRuntimeModule::RequestFullCapture(float DeltaTime)
{
	// Full capture intervals are clamped to be no lower than 1 sec
	UE::TReadScopeLock ReadLock(DelegatesRWLock);
	PerformFullCaptureDelegate.Broadcast(EChaosVDFullCaptureFlags::Particles);
	return true;
}

#undef LOCTEXT_NAMESPACE
#else

IMPLEMENT_MODULE(FDefaultModuleImpl, ChaosVDRuntime);

#endif
