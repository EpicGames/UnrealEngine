// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataWrappers/ChaosVDAccelerationStructureDataWrappers.h"

#include "DataWrappers/ChaosVDDataSerializationMacros.h"

#include "UObject/FortniteSeasonBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDAccelerationStructureDataWrappers)

FStringView FChaosVDAABBTreeDataWrapper::WrapperTypeName = TEXT("FChaosVDAABBTreeDataWrapper");

bool FChaosVDBVCellElementDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;

	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << Bounds;
	Ar << ParticleIndex;
	Ar << StartIdx;
	Ar << EndIdx;

	return !Ar.IsError();
}

bool FChaosVDBoundingVolumeDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;

	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << SolverId;
	Ar << Type;
	Ar << MElementsCounts;
	Ar << MElements;
	Ar << MaxPayloadBounds;

	return !Ar.IsError();
}

bool FChaosVDAABBTreeNodeDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;

	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	CVD_SERIALIZE_STATIC_ARRAY(Ar, ChildrenBounds);
	CVD_SERIALIZE_STATIC_ARRAY(Ar, ChildrenNodes);
	
	Ar << ParentNode;

	EChaosVDAABBTreeNodeFlags PackedFlags = EChaosVDAABBTreeNodeFlags::None;

	// Note: When the UI is done to show the enums in the same way we show these bitfiedls as read only booleans in CVD, we can remove all
	// this macro boilerplate and just serialize the the flags directly
	if (Ar.IsLoading())
	{
		Ar << PackedFlags;
		CVD_UNPACK_BITFIELD_DATA(bLeaf, PackedFlags, EChaosVDAABBTreeNodeFlags::IsLeaf);
		CVD_UNPACK_BITFIELD_DATA(bDirtyNode, PackedFlags, EChaosVDAABBTreeNodeFlags::IsDirty);
	}
	else
	{
		CVD_PACK_BITFIELD_DATA(bLeaf, PackedFlags,  EChaosVDAABBTreeNodeFlags::IsLeaf);
		CVD_PACK_BITFIELD_DATA(bDirtyNode, PackedFlags, EChaosVDAABBTreeNodeFlags::IsDirty);
		Ar << PackedFlags;
	}

	return !Ar.IsError();
}

bool FChaosVDAABBTreePayloadBoundsElement::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;

	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << ParticleIndex;
	Ar << Bounds;

	Ar.UsingCustomVersion(FFortniteSeasonBranchObjectVersion::GUID);
	if (Ar.CustomVer(FFortniteSeasonBranchObjectVersion::GUID) >= FFortniteSeasonBranchObjectVersion::CVDSerializationFixMissingSerializationProperties)
	{
		Ar << ActualBounds;
	}

	return !Ar.IsError();
}

bool FChaosVDAABBTreeLeafDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;

	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << Elements;
	Ar << Bounds;

	return !Ar.IsError();
}

bool FChaosVDAABBTreeDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;

	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << SolverId;
	Ar << RootNodeIndex;
	Ar << bDynamicTree;
	Ar << Nodes;
	Ar << BoundingVolumeLeafs;
	Ar << TreeArrayLeafs;
	Ar << MaxPayloadBounds;
	Ar << MaxTreeDepth;
	Ar << MaxChildrenInLeaf;
	Ar << LeavesNum;
	Ar << NodesNum;
	Ar << Type;

	return !Ar.IsError();
}
