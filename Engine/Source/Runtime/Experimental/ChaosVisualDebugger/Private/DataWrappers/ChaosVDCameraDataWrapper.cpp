// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataWrappers/ChaosVDCameraDataWrapper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDCameraDataWrapper)

FStringView FChaosVDCameraDataWrapper::WrapperTypeName = TEXT("FChaosVDCameraDataWrapper");

bool FChaosVDCameraDataWrapper::Serialize(FArchive& Ar)
{
	Ar << Camera.CameraName;
	Ar << Camera.ActorName;
	Ar << Camera.ActorAssetPath;
	Ar << Camera.MapAssetPath;
	Ar << Position;
	Ar << Rotation;
	Ar << FOV;
	
	return !Ar.IsError();
}

