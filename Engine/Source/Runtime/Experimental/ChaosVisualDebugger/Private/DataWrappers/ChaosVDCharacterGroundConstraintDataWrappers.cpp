// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataWrappers/ChaosVDCharacterGroundConstraintDataWrappers.h"

#include "DataWrappers/ChaosVDDataSerializationMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDCharacterGroundConstraintDataWrappers)

FStringView FChaosVDCharacterGroundConstraint::WrapperTypeName = TEXT("FChaosVDCharacterGroundConstraint");

bool FChaosVDCharacterGroundConstraintStateDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << Island;
	Ar << Level;
	Ar << Color;
	Ar << IslandSize;

	Ar << bDisabled;

	Ar << SolverAppliedForce;
	Ar << SolverAppliedTorque;
	
	return !Ar.IsError();
}

bool FChaosVDCharacterGroundConstraintSettingsDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}
	
	Ar << VerticalAxis;
	Ar << TargetHeight;
	Ar << RadialForceLimit;
	Ar << FrictionForceLimit;
	Ar << TwistTorqueLimit;
	Ar << SwingTorqueLimit;
	Ar << CosMaxWalkableSlopeAngle;
	Ar << DampingFactor;
	Ar << AssumedOnGroundHeight;

	return !Ar.IsError();
}

bool FChaosVDCharacterGroundConstraintDataDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << GroundNormal;
	Ar << TargetDeltaPosition;
	Ar << TargetDeltaFacing;
	Ar << GroundDistance;
	Ar << CosMaxWalkableSlopeAngle;

	return !Ar.IsError();
}


bool FChaosVDCharacterGroundConstraint::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << SolverID;
	Ar << ConstraintIndex;

	Ar << CharacterParticleIndex;
	Ar << GroundParticleIndex;

	Ar << State;
	Ar << Settings;
	Ar << Data;

	return !Ar.IsError();
}

int32 FChaosVDCharacterGroundConstraint::GetParticleIDAtSlot(EChaosVDParticlePairIndex IndexSlot) const
{
	switch (IndexSlot)
	{
		case EChaosVDParticlePairIndex::Index_0:
			return CharacterParticleIndex;
		case EChaosVDParticlePairIndex::Index_1:
			return GroundParticleIndex;
		default:
			return INDEX_NONE;
	}
}
