// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataWrappers/ChaosVDCollisionDataWrappers.h"
#include "DataWrappers/ChaosVDDataSerializationMacros.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDCollisionDataWrappers)

FStringView FChaosVDConstraint::WrapperTypeName = TEXT("FChaosVDConstraint");
FStringView FChaosVDParticlePairMidPhase::WrapperTypeName = TEXT("FChaosVDParticlePairMidPhase");
FStringView FChaosVDCollisionChannelsInfoContainer::WrapperTypeName = TEXT("FChaosVDCollisionChannelsInfoContainer");

bool FChaosVDContactPoint::Serialize(FArchive& Ar)
{
	CVD_SERIALIZE_STATIC_ARRAY(Ar, ShapeContactPoints);
	Ar << ShapeContactNormal;
	Ar << Phi;
	Ar << FaceIndex;
	Ar << ContactType;

	return !Ar.IsError();
}

bool FChaosVDManifoldPoint::Serialize(FArchive& Ar)
{
	EChaosVDManifoldPointFlags PackedFlags = EChaosVDManifoldPointFlags::None;

	
	// Note: When the UI is done to show the enums in the same way we show these bitfiedls as read only booleans in CVD, we can remove all
	// this macro boilerplate and just serialize the the flags directly
	if (Ar.IsLoading())
	{
		Ar << PackedFlags;
		CVD_UNPACK_BITFIELD_DATA(bDisabled, PackedFlags, EChaosVDManifoldPointFlags::Disabled);
		CVD_UNPACK_BITFIELD_DATA(bWasRestored, PackedFlags, EChaosVDManifoldPointFlags::WasRestored);
		CVD_UNPACK_BITFIELD_DATA(bWasReplaced, PackedFlags, EChaosVDManifoldPointFlags::WasReplaced);
		CVD_UNPACK_BITFIELD_DATA(bHasStaticFrictionAnchor, PackedFlags, EChaosVDManifoldPointFlags::HasStaticFrictionAnchor);
		CVD_UNPACK_BITFIELD_DATA(bIsValid, PackedFlags, EChaosVDManifoldPointFlags::IsValid);
		CVD_UNPACK_BITFIELD_DATA(bInsideStaticFrictionCone, PackedFlags, EChaosVDManifoldPointFlags::InsideStaticFrictionCone);
	}
	else
	{
		CVD_PACK_BITFIELD_DATA(bDisabled, PackedFlags, EChaosVDManifoldPointFlags::Disabled);
		CVD_PACK_BITFIELD_DATA(bWasRestored, PackedFlags, EChaosVDManifoldPointFlags::WasRestored);
		CVD_PACK_BITFIELD_DATA(bWasReplaced, PackedFlags, EChaosVDManifoldPointFlags::WasReplaced);
		CVD_PACK_BITFIELD_DATA(bHasStaticFrictionAnchor, PackedFlags, EChaosVDManifoldPointFlags::HasStaticFrictionAnchor);
		CVD_PACK_BITFIELD_DATA(bIsValid, PackedFlags, EChaosVDManifoldPointFlags::IsValid);
		CVD_PACK_BITFIELD_DATA(bInsideStaticFrictionCone, PackedFlags, EChaosVDManifoldPointFlags::InsideStaticFrictionCone);
		Ar << PackedFlags;
	}

	Ar << NetPushOut;
	Ar << NetImpulse;

	Ar << TargetPhi;
	Ar << InitialPhi;

	CVD_SERIALIZE_STATIC_ARRAY(Ar, ShapeAnchorPoints);
	CVD_SERIALIZE_STATIC_ARRAY(Ar, InitialShapeContactPoints);

	Ar << ContactPoint;
	CVD_SERIALIZE_STATIC_ARRAY(Ar, ShapeContactPoints);

	return true;
}

bool FChaosVDCollisionMaterial::Serialize(FArchive& Ar)
{
	Ar << FaceIndex;
	Ar << MaterialDynamicFriction;
	Ar << MaterialStaticFriction;
	Ar << MaterialRestitution;
	Ar << DynamicFriction;
	Ar << StaticFriction;
	Ar << Restitution;
	Ar << RestitutionThreshold;
	Ar << InvMassScale0;
	Ar << InvMassScale1;
	Ar << InvInertiaScale0;
	Ar << InvInertiaScale1;

	return true;
}

bool FChaosVDConstraint::Serialize(FArchive& Ar)
{	
	EChaosVDConstraintFlags PackedFlags = EChaosVDConstraintFlags::None;

	
	// Note: When the UI is done to show the enums in the same way we show these bitfiedls as read only booleans in CVD, we can remove all
	// this macro boilerplate and just serialize the the flags directly
	if (Ar.IsLoading())
	{
		Ar << PackedFlags;
		CVD_UNPACK_BITFIELD_DATA(bIsCurrent, PackedFlags, EChaosVDConstraintFlags::IsCurrent);
		CVD_UNPACK_BITFIELD_DATA(bDisabled, PackedFlags, EChaosVDConstraintFlags::Disabled);
		CVD_UNPACK_BITFIELD_DATA(bUseManifold, PackedFlags, EChaosVDConstraintFlags::UseManifold);
		CVD_UNPACK_BITFIELD_DATA(bUseIncrementalManifold, PackedFlags, EChaosVDConstraintFlags::UseIncrementalManifold);
		CVD_UNPACK_BITFIELD_DATA(bCanRestoreManifold, PackedFlags, EChaosVDConstraintFlags::CanRestoreManifold);
		CVD_UNPACK_BITFIELD_DATA(bWasManifoldRestored, PackedFlags, EChaosVDConstraintFlags::WasManifoldRestored);
		CVD_UNPACK_BITFIELD_DATA(bIsQuadratic0, PackedFlags, EChaosVDConstraintFlags::IsQuadratic0);
		CVD_UNPACK_BITFIELD_DATA(bIsProbe, PackedFlags, EChaosVDConstraintFlags::IsProbe);
		CVD_UNPACK_BITFIELD_DATA(bCCDEnabled, PackedFlags, EChaosVDConstraintFlags::CCDEnabled);
		CVD_UNPACK_BITFIELD_DATA(bCCDSweepEnabled, PackedFlags, EChaosVDConstraintFlags::CCDSweepEnabled);
		CVD_UNPACK_BITFIELD_DATA(bModifierApplied, PackedFlags, EChaosVDConstraintFlags::ModifierApplied);
		CVD_UNPACK_BITFIELD_DATA(bMaterialSet, PackedFlags, EChaosVDConstraintFlags::MaterialSet);
	}
	else
	{
		CVD_PACK_BITFIELD_DATA(bIsCurrent, PackedFlags, EChaosVDConstraintFlags::IsCurrent);
		CVD_PACK_BITFIELD_DATA(bDisabled, PackedFlags, EChaosVDConstraintFlags::Disabled);
		CVD_PACK_BITFIELD_DATA(bUseManifold, PackedFlags, EChaosVDConstraintFlags::UseManifold);
		CVD_PACK_BITFIELD_DATA(bUseIncrementalManifold, PackedFlags, EChaosVDConstraintFlags::UseIncrementalManifold);
		CVD_PACK_BITFIELD_DATA(bCanRestoreManifold, PackedFlags, EChaosVDConstraintFlags::CanRestoreManifold);
		CVD_PACK_BITFIELD_DATA(bWasManifoldRestored, PackedFlags, EChaosVDConstraintFlags::WasManifoldRestored);
		CVD_PACK_BITFIELD_DATA(bIsQuadratic0, PackedFlags, EChaosVDConstraintFlags::IsQuadratic0);
		CVD_PACK_BITFIELD_DATA(bIsProbe, PackedFlags, EChaosVDConstraintFlags::IsProbe);
		CVD_PACK_BITFIELD_DATA(bCCDEnabled, PackedFlags, EChaosVDConstraintFlags::CCDEnabled);
		CVD_PACK_BITFIELD_DATA(bCCDSweepEnabled, PackedFlags, EChaosVDConstraintFlags::CCDSweepEnabled);
		CVD_PACK_BITFIELD_DATA(bModifierApplied, PackedFlags, EChaosVDConstraintFlags::ModifierApplied);
		CVD_PACK_BITFIELD_DATA(bMaterialSet, PackedFlags, EChaosVDConstraintFlags::MaterialSet);
		Ar << PackedFlags;
	}

	Ar << Material;
	Ar << AccumulatedImpulse;
	Ar << ShapesType;

	CVD_SERIALIZE_STATIC_ARRAY(Ar, ShapeWorldTransforms);
	CVD_SERIALIZE_STATIC_ARRAY(Ar, ImplicitTransforms);

	Ar << CullDistance;
	Ar << CollisionMargins;	
	Ar << CollisionTolerance;	
	Ar << ClosestManifoldPointIndex;	
	Ar << ExpectedNumManifoldPoints;	
	Ar << LastShapeWorldPositionDelta;
	Ar << LastShapeWorldRotationDelta;
	Ar << Stiffness;
	Ar << MinInitialPhi;
	Ar << InitialOverlapDepenetrationVelocity;
	Ar << CCDTimeOfImpact;
	Ar << CCDEnablePenetration;
	Ar << CCDTargetPenetration;
	Ar << ManifoldPoints;
	Ar << SolverID;
	Ar << Particle0Index;
	Ar << Particle1Index;

	return !Ar.IsError();
}

bool FChaosVDParticlePairMidPhase::Serialize(FArchive& Ar)
{
	Ar << SolverID;

	
	EChaosVDMidPhaseFlags PackedFlags = EChaosVDMidPhaseFlags::None;

	
	// Note: When the UI is done to show the enums in the same way we show these bitfiedls as read only booleans in CVD, we can remove all
	// this macro boilerplate and just serialize the the flags directly
	if (Ar.IsLoading())
	{
		Ar << PackedFlags;
		CVD_UNPACK_BITFIELD_DATA(bIsActive, PackedFlags, EChaosVDMidPhaseFlags::IsActive);
		CVD_UNPACK_BITFIELD_DATA(bIsCCD, PackedFlags, EChaosVDMidPhaseFlags::IsCCD);
		CVD_UNPACK_BITFIELD_DATA(bIsCCDActive, PackedFlags, EChaosVDMidPhaseFlags::IsCCDActive);
		CVD_UNPACK_BITFIELD_DATA(bIsCCDActive, PackedFlags, EChaosVDMidPhaseFlags::IsCCDActive);
		CVD_UNPACK_BITFIELD_DATA(bIsSleeping, PackedFlags, EChaosVDMidPhaseFlags::IsSleeping);
		CVD_UNPACK_BITFIELD_DATA(bIsModified, PackedFlags, EChaosVDMidPhaseFlags::IsModified);
	}
	else
	{
		CVD_PACK_BITFIELD_DATA(bIsActive, PackedFlags, EChaosVDMidPhaseFlags::IsActive);
		CVD_PACK_BITFIELD_DATA(bIsCCD, PackedFlags, EChaosVDMidPhaseFlags::IsCCD);
		CVD_PACK_BITFIELD_DATA(bIsCCDActive, PackedFlags, EChaosVDMidPhaseFlags::IsCCDActive);
		CVD_PACK_BITFIELD_DATA(bIsCCDActive, PackedFlags, EChaosVDMidPhaseFlags::IsCCDActive);
		CVD_PACK_BITFIELD_DATA(bIsSleeping, PackedFlags, EChaosVDMidPhaseFlags::IsSleeping);
		CVD_PACK_BITFIELD_DATA(bIsModified, PackedFlags, EChaosVDMidPhaseFlags::IsModified);
		Ar << PackedFlags;
	}
		
	Ar << LastUsedEpoch;

	Ar << Particle0Idx;
	Ar << Particle1Idx;

	Ar << Constraints;

	return !Ar.IsError();
}

bool FChaosVDCollisionFilterData::Serialize(FArchive& Ar)
{
	Ar << Word0;
	Ar << Word1;
	Ar << Word2;
	Ar << Word3;

	return !Ar.IsError();
}

bool FChaosVDCollisionFilterDataV2::Serialize(FArchive& Ar)
{
	Ar << OwnerId;
	Ar << ComponentId;
	Ar << OverlapMask;
	Ar << BlockMask;
	Ar << CollisionChannelMask;
	Ar << MaskFilterAndFlags;

	return !Ar.IsError();
}

uint8 FChaosVDCollisionFilterDataV2::GetMaskFilter() const
{
	const uint64 Result = (MaskFilterAndFlags & MaskFilterMask) >> MaskFilterOffset;
	return static_cast<uint8>(Result);
}

uint32 FChaosVDCollisionFilterDataV2::GetFlags() const
{
	return MaskFilterAndFlags & FlagsMask;
}

void FChaosVDCollisionFilterDataV2::SetMaskFilterAndFlags(uint32 InMaskFilter, uint32 InFlags)
{
	const uint64 MaskFilterPart = ((uint64)InMaskFilter << MaskFilterOffset) & MaskFilterMask;
	const uint64 FlagsPart = InFlags & FlagsMask;
	MaskFilterAndFlags = MaskFilterPart | FlagsPart;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FChaosVDCollisionFilterDataV2::LoadLegacy(const FChaosVDCollisionFilterData& QueryData, const FChaosVDCollisionFilterData& SimData)
{
	bLoadedFromLegacy = true;
	OwnerId = QueryData.Word0;
	BlockMask = QueryData.Word1;
	OverlapMask = QueryData.Word2;
	ComponentId = SimData.Word2;

	// Pull the collision channel index out and clear the old bits
	const uint32 CollisionChannelIndex = (QueryData.Word3 & CollisionChannelIndexMask) >> CollisionChannelIndexOffset;
	MaskFilterAndFlags = QueryData.Word3 & ~CollisionChannelIndexMask;
	CollisionChannelMask = 1LLU << CollisionChannelIndex;

	LegacySimBlockMask = SimData.Word1;
	LegacySimWord3 = SimData.Word3;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void FChaosVDCollisionFilterDataV2::ExtractWord3(const uint32 Word3, uint8& OutMaskFilter, uint8& OutChannelIndex, uint32& OutFlags)
{
	OutFlags = (Word3 & FlagsMask);
	OutChannelIndex = (Word3 & CollisionChannelIndexMask) >> CollisionChannelIndexOffset;
	OutMaskFilter = (Word3 & MaskFilterMask) >> MaskFilterOffset;
}

bool FChaosVDShapeCollisionData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Ar << CollisionTraceType;

	EChaosVDCollisionShapeDataFlags PackedFlags = EChaosVDCollisionShapeDataFlags::None;

	
	// Note: When the UI is done to show the enums in the same way we show these bitfiedls as read only booleans in CVD, we can remove all
	// this macro boilerplate and just serialize the the flags directly
	if (Ar.IsLoading())
	{
		Ar << PackedFlags;
		CVD_UNPACK_BITFIELD_DATA(bSimCollision, PackedFlags, EChaosVDCollisionShapeDataFlags::SimCollision);
		CVD_UNPACK_BITFIELD_DATA(bQueryCollision, PackedFlags, EChaosVDCollisionShapeDataFlags::QueryCollision);
		CVD_UNPACK_BITFIELD_DATA(bIsProbe, PackedFlags, EChaosVDCollisionShapeDataFlags::IsProbe);
	}
	else
	{
		CVD_PACK_BITFIELD_DATA(bSimCollision, PackedFlags, EChaosVDCollisionShapeDataFlags::SimCollision);
		CVD_PACK_BITFIELD_DATA(bQueryCollision, PackedFlags, EChaosVDCollisionShapeDataFlags::QueryCollision);
		CVD_PACK_BITFIELD_DATA(bIsProbe, PackedFlags, EChaosVDCollisionShapeDataFlags::IsProbe);
		Ar << PackedFlags;
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::SimAndQueryDataSupportInChaosVisualDebugger)
	{
		if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::CollisionFilter64Bit)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if !WITH_EDITORONLY_DATA
			FChaosVDCollisionFilterData SimData_DEPRECATED, QueryData_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

			Ar << SimData_DEPRECATED;
			Ar << QueryData_DEPRECATED;
			if (Ar.IsLoading())
			{
				FilterData.LoadLegacy(QueryData_DEPRECATED, SimData_DEPRECATED);
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
		{
			Ar << FilterData;
		}
	}

	return !Ar.IsError();
}

bool FChaosVDShapeCollisionData::operator==(const FChaosVDShapeCollisionData& Other) const
{
	return CollisionTraceType == Other.CollisionTraceType
			&& bSimCollision == Other.bSimCollision
			&& bQueryCollision == Other.bQueryCollision
			&& bIsProbe == Other.bIsProbe
			&& FilterData == Other.FilterData;
}

bool FChaosVDCollisionChannelInfo::Serialize(FArchive& Ar)
{
	Ar << DisplayName;

	Ar << CollisionChannel;

	Ar << bIsTraceType;

	return !Ar.IsError();
}

bool FChaosVDCollisionChannelsInfoContainer::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::CollisionFilter64Bit)
	{
		constexpr uint32 Size = 32;
		CollisionChannelInfos.SetNum(Size);
		for (uint32 I = 0; I < Size; ++I)
		{
			Ar << CollisionChannelInfos[I];
		}
	}
	else
	{
		Ar << CollisionChannelInfos;
	}

	return !Ar.IsError();
}
