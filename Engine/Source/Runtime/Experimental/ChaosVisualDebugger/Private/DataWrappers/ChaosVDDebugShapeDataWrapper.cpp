// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataWrappers/ChaosVDDebugShapeDataWrapper.h"

#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDDebugShapeDataWrapper)

FStringView FChaosVDDebugDrawSphereDataWrapper::WrapperTypeName = TEXT("FChaosVDDebugDrawSphereDataWrapper");
FStringView FChaosVDDebugDrawLineDataWrapper::WrapperTypeName = TEXT("FChaosVDDebugDrawLineDataWrapper");
FStringView FChaosVDDebugDrawImplicitObjectDataWrapper::WrapperTypeName = TEXT("FChaosVDDebugDrawImplicitObjectDataWrapper");
FStringView FChaosVDDebugDrawBoxDataWrapper::WrapperTypeName = TEXT("FChaosVDDDebugDrawBoxDataWrapper");

void FChaosVDDebugDrawShapeBase::SerializeBase_Internal(FArchive& Ar)
{
	Ar << SolverID;
	Ar << Tag;
	Ar << Color;

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::ThreadContextDataInChaosVisualDebuggerDebugDrawData)
	{
		Ar << ThreadContext;
	}
}

bool FChaosVDDebugDrawBoxDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;

	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	SerializeBase_Internal(Ar);

	Ar << Box;
	
	return !Ar.IsError();
}

bool FChaosVDDebugDrawSphereDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;

	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	SerializeBase_Internal(Ar);

	Ar << Origin;
	Ar << Radius;
	
	return !Ar.IsError();
}

bool FChaosVDDebugDrawLineDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;

	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	SerializeBase_Internal(Ar);

	Ar << StartLocation;
	Ar << EndLocation;
	Ar << bIsArrow;
	
	return !Ar.IsError();
}

bool FChaosVDDebugDrawImplicitObjectDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;

	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	SerializeBase_Internal(Ar);

	Ar << ImplicitObjectHash;
	Ar << ParentTransform;
	
	return !Ar.IsError();
}
