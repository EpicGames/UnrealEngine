// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataWrappers/ChaosVDJointDataWrappers.h"

#include "DataWrappers/ChaosVDDataSerializationMacros.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDJointDataWrappers)

FStringView FChaosVDJointConstraint::WrapperTypeName = TEXT("FChaosVDJointConstraint");

bool FChaosVDJointStateDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << Island;
	Ar << Level;
	Ar << Color;
	Ar << IslandSize;

	EChaosVDJointStateFlags StateFlags = EChaosVDJointStateFlags::None;
	
	if (Ar.IsLoading())
	{
		Ar << StateFlags;
		CVD_UNPACK_BITFIELD_DATA(bDisabled, StateFlags, EChaosVDJointStateFlags::Disabled);
		CVD_UNPACK_BITFIELD_DATA(bBroken, StateFlags, EChaosVDJointStateFlags::Broken);
		CVD_UNPACK_BITFIELD_DATA(bBreaking, StateFlags, EChaosVDJointStateFlags::Breaking);
		CVD_UNPACK_BITFIELD_DATA(bDriveTargetChanged, StateFlags, EChaosVDJointStateFlags::DriveTargetChanged);
		CVD_UNPACK_BITFIELD_DATA(bEnabledDuringResim, StateFlags, EChaosVDJointStateFlags::EnabledDuringResim);
	}
	else
	{
		CVD_PACK_BITFIELD_DATA(bDisabled, StateFlags, EChaosVDJointStateFlags::Disabled);
		CVD_PACK_BITFIELD_DATA(bBroken, StateFlags, EChaosVDJointStateFlags::Broken);
		CVD_PACK_BITFIELD_DATA(bBreaking, StateFlags, EChaosVDJointStateFlags::Breaking);
		CVD_PACK_BITFIELD_DATA(bDriveTargetChanged, StateFlags, EChaosVDJointStateFlags::DriveTargetChanged);
		CVD_PACK_BITFIELD_DATA(bEnabledDuringResim, StateFlags, EChaosVDJointStateFlags::EnabledDuringResim);

		Ar << StateFlags;
	}

	Ar << LinearImpulse;
	Ar << AngularImpulse;
	Ar << ResimType;
	Ar << SyncState;
	
	return !Ar.IsError();
}

bool FChaosVDGTJointStateDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	EChaosVDGTJointStateFlags StateFlags = EChaosVDGTJointStateFlags::None;
	
	if (Ar.IsLoading())
	{
		Ar << StateFlags;
		CVD_UNPACK_BITFIELD_DATA(bIsBreaking, StateFlags, EChaosVDGTJointStateFlags::IsBreaking);
		CVD_UNPACK_BITFIELD_DATA(bIsBroken, StateFlags, EChaosVDGTJointStateFlags::IsBroken);
		CVD_UNPACK_BITFIELD_DATA(bDriveTargetChanged, StateFlags, EChaosVDGTJointStateFlags::DriveTargetChanged);
		CVD_UNPACK_BITFIELD_DATA(bIsViolating, StateFlags, EChaosVDGTJointStateFlags::IsViolating);
	}
	else
	{
		CVD_UNPACK_BITFIELD_DATA(bIsBreaking, StateFlags, EChaosVDGTJointStateFlags::IsBreaking);
		CVD_UNPACK_BITFIELD_DATA(bIsBroken, StateFlags, EChaosVDGTJointStateFlags::IsBroken);
		CVD_UNPACK_BITFIELD_DATA(bDriveTargetChanged, StateFlags, EChaosVDGTJointStateFlags::DriveTargetChanged);
		CVD_UNPACK_BITFIELD_DATA(bIsViolating, StateFlags, EChaosVDGTJointStateFlags::IsViolating);

		Ar << StateFlags;
	}

	Ar <<  Force;
	Ar <<  Torque;
	Ar <<  LinearViolation;
	Ar <<  AngularViolation;

	return !Ar.IsError();
}

bool FChaosVDJointSolverSettingsDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << SwingTwistAngleTolerance;
	Ar << PositionTolerance;
	Ar << AngleTolerance;
	Ar << MinParentMassRatio;
	Ar << MaxInertiaRatio;
	Ar << MinSolverStiffness;
	Ar << MaxSolverStiffness;
	Ar << NumIterationsAtMaxSolverStiffness;
	Ar << NumShockPropagationIterations;
	Ar << LinearStiffnessOverride;
	Ar << TwistStiffnessOverride;
	Ar << SwingStiffnessOverride;
	Ar << LinearProjectionOverride;
	Ar << AngularProjectionOverride;
	Ar << ShockPropagationOverride;
	Ar << LinearDriveStiffnessOverride;
	Ar << LinearDriveDampingOverride;
	Ar << AngularDriveStiffnessOverride;
	Ar << AngularDriveDampingOverride;
	Ar << SoftLinearStiffnessOverride;
	Ar << SoftLinearDampingOverride;
	Ar << SoftTwistStiffnessOverride;
	Ar << SoftTwistDampingOverride;
	Ar << SoftSwingStiffnessOverride;
	Ar << SoftSwingDampingOverride;

	EChaosVDJointSolverSettingsFlags Flags = EChaosVDJointSolverSettingsFlags::None;
	
	if (Ar.IsLoading())
	{
		Ar << Flags;
		CVD_PACK_BITFIELD_DATA(bUseLinearSolver, Flags, EChaosVDJointSolverSettingsFlags::UseLinearSolver);
		CVD_PACK_BITFIELD_DATA(bSortEnabled, Flags, EChaosVDJointSolverSettingsFlags::SortEnabled);
		CVD_PACK_BITFIELD_DATA(bSolvePositionLast, Flags, EChaosVDJointSolverSettingsFlags::SolvePositionLast);
		CVD_PACK_BITFIELD_DATA(bUsePositionBasedDrives, Flags, EChaosVDJointSolverSettingsFlags::UsePositionBasedDrives);
		CVD_PACK_BITFIELD_DATA(bEnableTwistLimits, Flags, EChaosVDJointSolverSettingsFlags::EnableTwistLimits);
		CVD_PACK_BITFIELD_DATA(bEnableSwingLimits, Flags, EChaosVDJointSolverSettingsFlags::EnableSwingLimits);
		CVD_PACK_BITFIELD_DATA(bEnableDrives, Flags, EChaosVDJointSolverSettingsFlags::EnableDrives);
	}
	else
	{
		CVD_UNPACK_BITFIELD_DATA(bUseLinearSolver, Flags, EChaosVDJointSolverSettingsFlags::UseLinearSolver);
		CVD_UNPACK_BITFIELD_DATA(bSortEnabled, Flags, EChaosVDJointSolverSettingsFlags::SortEnabled);
		CVD_UNPACK_BITFIELD_DATA(bSolvePositionLast, Flags, EChaosVDJointSolverSettingsFlags::SolvePositionLast);
		CVD_UNPACK_BITFIELD_DATA(bUsePositionBasedDrives, Flags, EChaosVDJointSolverSettingsFlags::UsePositionBasedDrives);
		CVD_UNPACK_BITFIELD_DATA(bEnableTwistLimits, Flags, EChaosVDJointSolverSettingsFlags::EnableTwistLimits);
		CVD_UNPACK_BITFIELD_DATA(bEnableSwingLimits, Flags, EChaosVDJointSolverSettingsFlags::EnableSwingLimits);
		CVD_UNPACK_BITFIELD_DATA(bEnableDrives, Flags, EChaosVDJointSolverSettingsFlags::EnableDrives);

		Ar << Flags;
	}

	return !Ar.IsError();
}

bool FChaosVDJointSettingsDataWrapper::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}
	
	CVD_SERIALIZE_STATIC_ARRAY(Ar, ConnectorTransforms);
	Ar << Stiffness;
	Ar << LinearProjection;
	Ar << AngularProjection;
	Ar << ShockPropagation;
	Ar << TeleportDistance;
	Ar << TeleportAngle;
	Ar << ParentInvMassScale;
	CVD_SERIALIZE_STATIC_ARRAY(Ar, LinearMotionTypes);
	Ar << LinearLimit;
	CVD_SERIALIZE_STATIC_ARRAY(Ar, AngularMotionTypes);
	Ar << AngularLimits;
	Ar << LinearSoftForceMode;
	Ar << AngularSoftForceMode;
	Ar << SoftLinearStiffness;
	Ar << SoftLinearDamping;
	Ar << SoftTwistStiffness;
	Ar << SoftTwistDamping;
	Ar << SoftSwingStiffness;
	Ar << SoftSwingDamping;
	Ar << LinearRestitution;
	Ar << TwistRestitution;
	Ar << SwingRestitution;
	Ar << LinearContactDistance;
	Ar << TwistContactDistance;
	Ar << SwingContactDistance;
	Ar << LinearDrivePositionTarget;
	Ar << LinearDriveVelocityTarget;
	Ar << LinearDriveForceMode;
	Ar << LinearDriveStiffness;
	Ar << LinearDriveDamping;
	Ar << LinearDriveMaxForce;
	Ar << AngularDrivePositionTarget;
	Ar << AngularDriveVelocityTarget;
	Ar << AngularDriveForceMode;
	Ar << AngularDriveStiffness;
	Ar << AngularDriveDamping;
	Ar << AngularDriveMaxTorque;
	Ar << LinearBreakForce;
	Ar << LinearPlasticityLimit;
	Ar << LinearPlasticityType;
	Ar << LinearPlasticityInitialDistanceSquared;
	Ar << AngularBreakTorque;
	Ar << AngularPlasticityLimit;
	Ar << ContactTransferScale;

	EChaosVDJointSettingsFlags Flags = EChaosVDJointSettingsFlags::None;
	
	if (Ar.IsLoading())
	{
		Ar << Flags;
		CVD_PACK_BITFIELD_DATA(bCollisionEnabled, Flags, EChaosVDJointSettingsFlags::CollisionEnabled);
		CVD_PACK_BITFIELD_DATA(bMassConditioningEnabled, Flags, EChaosVDJointSettingsFlags::MassConditioningEnabled);
		CVD_PACK_BITFIELD_DATA(bSoftLinearLimitsEnabled, Flags, EChaosVDJointSettingsFlags::SoftLinearLimitsEnabled);
		CVD_PACK_BITFIELD_DATA(bSoftTwistLimitsEnabled, Flags, EChaosVDJointSettingsFlags::SoftTwistLimitsEnabled);
		CVD_PACK_BITFIELD_DATA(bSoftSwingLimitsEnabled, Flags, EChaosVDJointSettingsFlags::SoftSwingLimitsEnabled);
		CVD_PACK_BITFIELD_DATA(bAngularSLerpPositionDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularSLerpPositionDriveEnabled);
		CVD_PACK_BITFIELD_DATA(bAngularSLerpVelocityDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularSLerpVelocityDriveEnabled);
		CVD_PACK_BITFIELD_DATA(bAngularTwistPositionDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularTwistPositionDriveEnabled);
		CVD_PACK_BITFIELD_DATA(bAngularSwingPositionDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularSwingPositionDriveEnabled);
		CVD_PACK_BITFIELD_DATA(bAngularTwistVelocityDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularTwistVelocityDriveEnabled);
		CVD_PACK_BITFIELD_DATA(bAngularSwingVelocityDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularSwingVelocityDriveEnabled);
		CVD_PACK_BITFIELD_DATA(bAngularSwingVelocityDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularSwingVelocityDriveEnabled);
		CVD_PACK_BITFIELD_DATA(bLinearPositionDriveEnabled0, Flags, EChaosVDJointSettingsFlags::LinearPositionDriveEnabled0);
		CVD_PACK_BITFIELD_DATA(bLinearPositionDriveEnabled1, Flags, EChaosVDJointSettingsFlags::LinearPositionDriveEnable1);
		CVD_PACK_BITFIELD_DATA(bLinearPositionDriveEnabled2, Flags, EChaosVDJointSettingsFlags::LinearPositionDriveEnable2);
		CVD_PACK_BITFIELD_DATA(bLinearVelocityDriveEnabled0, Flags, EChaosVDJointSettingsFlags::LinearVelocityDriveEnabled0);
		CVD_PACK_BITFIELD_DATA(bLinearVelocityDriveEnabled1, Flags, EChaosVDJointSettingsFlags::LinearVelocityDriveEnabled1);
		CVD_PACK_BITFIELD_DATA(bLinearVelocityDriveEnabled2, Flags, EChaosVDJointSettingsFlags::LinearVelocityDriveEnabled2);
		CVD_PACK_BITFIELD_DATA(bUseLinearSolver, Flags, EChaosVDJointSettingsFlags::UseLinearSolver);
	}
	else
	{
		CVD_UNPACK_BITFIELD_DATA(bCollisionEnabled, Flags, EChaosVDJointSettingsFlags::CollisionEnabled);
		CVD_UNPACK_BITFIELD_DATA(bMassConditioningEnabled, Flags, EChaosVDJointSettingsFlags::MassConditioningEnabled);
		CVD_UNPACK_BITFIELD_DATA(bSoftLinearLimitsEnabled, Flags, EChaosVDJointSettingsFlags::SoftLinearLimitsEnabled);
		CVD_UNPACK_BITFIELD_DATA(bSoftTwistLimitsEnabled, Flags, EChaosVDJointSettingsFlags::SoftTwistLimitsEnabled);
		CVD_UNPACK_BITFIELD_DATA(bSoftSwingLimitsEnabled, Flags, EChaosVDJointSettingsFlags::SoftSwingLimitsEnabled);
		CVD_UNPACK_BITFIELD_DATA(bAngularSLerpPositionDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularSLerpPositionDriveEnabled);
		CVD_UNPACK_BITFIELD_DATA(bAngularSLerpVelocityDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularSLerpVelocityDriveEnabled);
		CVD_UNPACK_BITFIELD_DATA(bAngularTwistPositionDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularTwistPositionDriveEnabled);
		CVD_UNPACK_BITFIELD_DATA(bAngularSwingPositionDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularSwingPositionDriveEnabled);
		CVD_UNPACK_BITFIELD_DATA(bAngularTwistVelocityDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularTwistVelocityDriveEnabled);
		CVD_UNPACK_BITFIELD_DATA(bAngularSwingVelocityDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularSwingVelocityDriveEnabled);
		CVD_UNPACK_BITFIELD_DATA(bAngularSwingVelocityDriveEnabled, Flags, EChaosVDJointSettingsFlags::AngularSwingVelocityDriveEnabled);
		CVD_UNPACK_BITFIELD_DATA(bLinearPositionDriveEnabled0, Flags, EChaosVDJointSettingsFlags::LinearPositionDriveEnabled0);
		CVD_UNPACK_BITFIELD_DATA(bLinearPositionDriveEnabled1, Flags, EChaosVDJointSettingsFlags::LinearPositionDriveEnable1);
		CVD_UNPACK_BITFIELD_DATA(bLinearPositionDriveEnabled2, Flags, EChaosVDJointSettingsFlags::LinearPositionDriveEnable2);
		CVD_UNPACK_BITFIELD_DATA(bLinearVelocityDriveEnabled0, Flags, EChaosVDJointSettingsFlags::LinearVelocityDriveEnabled0);
		CVD_UNPACK_BITFIELD_DATA(bLinearVelocityDriveEnabled1, Flags, EChaosVDJointSettingsFlags::LinearVelocityDriveEnabled1);
		CVD_UNPACK_BITFIELD_DATA(bLinearVelocityDriveEnabled2, Flags, EChaosVDJointSettingsFlags::LinearVelocityDriveEnabled2);
		CVD_UNPACK_BITFIELD_DATA(bUseLinearSolver, Flags, EChaosVDJointSettingsFlags::UseLinearSolver);;

		Ar << Flags;
	}

	return !Ar.IsError();
}


bool FChaosVDJointConstraint::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << SolverID;

	Ar << ConstraintIndex;

	CVD_SERIALIZE_STATIC_ARRAY(Ar, ParticleParIndexes);

	Ar << JointSettings;
	Ar << PhysicsThreadJointState;

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AdditionalGameThreadDataSupportInChaosVisualDebugger)
	{
		Ar << GameThreadJointState;
	}

	return !Ar.IsError();
}
