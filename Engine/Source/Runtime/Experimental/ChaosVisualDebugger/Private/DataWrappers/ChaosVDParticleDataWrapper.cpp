// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataWrappers/ChaosVDParticleDataWrapper.h"

#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/FortniteSeasonBranchObjectVersion.h"
#include "Misc/StringBuilder.h"

// @note: Tracing an scene with 1000 particles moving, Manually serializing the structs is ~20% faster
// than normal UStruct serialization in unversioned mode. As we do this at runtime when tracing in development builds, this is important.
// One of the downside is that this will be more involved to maintain as any versioning needs to be done by hand.

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDParticleDataWrapper)

FStringView FChaosVDParticleDataWrapper::WrapperTypeName = TEXT("FChaosVDParticleDataWrapper");
FStringView FChaosVDParticleMetadata::WrapperTypeName = TEXT("FChaosVDParticleMetadata");

bool FChaosVDParticleMetadata::Serialize(FArchive& Ar)
{
	Ar << OwnerName;
	Ar << ComponentName;
	Ar << BoneName;
	Ar << Index;
	Ar << MapAssetPath;
	Ar << OwnerAssetPath;
	Ar << MetadataID;

	return !Ar.IsError();
}

FString FChaosVDParticleMetadata::ToString() const
{
	TStringBuilder<1024> NameBuilder;

	NameBuilder.Appendf(TEXT("%s | %s "), *OwnerName.ToString(), *ComponentName.ToString());

	if (!BoneName.IsNone())
	{
		NameBuilder.Appendf(TEXT("| Bone Name: %s "), *BoneName.ToString());
	}

	if (Index != INDEX_NONE)
	{
		NameBuilder.Appendf(TEXT("| Index: %d "), Index);
	}

	return NameBuilder.ToString();
}

bool FChaosVDFRigidParticleControlFlags::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteSeasonBranchObjectVersion::GUID);
	if (Ar.CustomVer(FFortniteSeasonBranchObjectVersion::GUID) >= FFortniteSeasonBranchObjectVersion::CVDSerializationFixMissingSerializationProperties)
	{
		Ar << bHasValidData;

		if (!bHasValidData)
		{
			return !Ar.IsError();
		}
	}
	
	Ar << bGravityEnabled;
	Ar << bCCDEnabled;
	Ar << bOneWayInteractionEnabled;
	Ar << bInertiaConditioningEnabled;
	Ar << GravityGroupIndex;
	Ar << bMACDEnabled;

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::SolverIterationsDataSupportInChaosVisualDebugger
		&& Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::PerParticleIterationCountMovedToDynamicMisc)
	{
		// Three uint8 (PositionSolverIterations, VelocitySolverIterations, ProjectionSolverIterations) were moved to FChaosVDParticleDynamicMisc.
		uint8 DummyInt;
		Ar << DummyInt;
		Ar << DummyInt;
		Ar << DummyInt;
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::GyroscopicTorquesSupportInChaosVisualDebugger)
	{
		Ar << bGyroscopicTorqueEnabled;
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::PerParticleFlagToAllowPartialIslandSleepInConnectedIsland)
	{
		Ar << bPartialIslandSleepAllowed;
	}

	return !Ar.IsError();
}

bool FChaosVDParticlePositionRotation::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << MX;
	Ar << MR;

	return !Ar.IsError();
}

bool FChaosVDParticleVelocities::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << MV;
	Ar << MW;

	return !Ar.IsError();
}

bool FChaosVDParticleBounds::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;

	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << MMin;
	Ar << MMax;

	return !Ar.IsError();
}

bool FChaosVDParticleDynamics::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << MAcceleration;
	Ar << MAngularAcceleration;
	Ar << MAngularImpulseVelocity;
	Ar << MLinearImpulseVelocity;

	return !Ar.IsError();
}

bool FChaosVDParticleMassProps::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << MCenterOfMass;
	Ar << MRotationOfMass;
	Ar << MI;
	Ar << MInvI;
	Ar << MM;
	Ar << MInvM;

	return !Ar.IsError();
}

bool FChaosVDParticleDynamicMisc::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << MAngularEtherDrag;
	Ar << MMaxLinearSpeedSq;
	Ar << MMaxAngularSpeedSq;
	Ar << MInitialOverlapDepenetrationVelocity;
	Ar << MCollisionGroup;
	Ar << MObjectState;
	Ar << MSleepType;
	Ar << bDisabled;

	MControlFlags.Serialize(Ar);

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::PerParticleIterationCountMovedToDynamicMisc)
	{
		Ar << PositionSolverIterationCount;
		Ar << VelocitySolverIterationCount;
		Ar << ProjectionSolverIterationCount;
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AddedMissingSerializationForPropertiesInDynamicMisc)
	{
		Ar << MLinearEtherDrag;
		Ar << MSleepThresholdMultiplier;
		Ar << MCollisionConstraintFlag;
	}

	return true;
}

bool FChaosVDParticleCluster::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << ParentParticleID;	
	Ar << NumChildren;
	Ar << ChildToParent;
	Ar << ClusterGroupIndex;
	Ar << bInternalCluster;
	Ar << CollisionImpulse;
	Ar << ExternalStrains;
	Ar << InternalStrains;
	Ar << Strain;
	Ar << ConnectivityEdges;
	Ar << bIsAnchored;
	Ar << bUnbreakable;
	Ar << bIsChildToParentLocked;

	return !Ar.IsError();
}

bool FChaosVDKinematicTarget::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << Rotation;
	Ar << Position;
	Ar << Mode;

	return !Ar.IsError();
}

bool FChaosVDVSmooth::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << MV;
	Ar << MW;

	return !Ar.IsError();
}

bool FChaosVDParticleDataWrapper::HasLegacyDebugName() const
{
#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return bHasDebugName;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#else
	return false;
#endif
}

FString FChaosVDParticleDataWrapper::GetDebugName() const
{
	static FString DefaultDebugName("UnnamedParticle");

	if (ParticleMetadataInstance.IsValid())
	{
		return *ParticleMetadataInstance->ToString();
	}
	else if (HasLegacyDebugName())
	{
#if WITH_EDITOR
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return DebugName;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#else
		return DefaultDebugName;
#endif
	}

	return DefaultDebugName;
}

void FChaosVDParticleDataWrapper::SetMetadataInstance(const TSharedPtr<FChaosVDParticleMetadata>& InMetadata)
{
	if (InMetadata.IsValid())
	{
		check(MetadataId == InMetadata->MetadataID);
	}
	
	ParticleMetadataInstance = InMetadata;
}

const TSharedPtr<FChaosVDParticleMetadata>& FChaosVDParticleDataWrapper::GetMetadataInstance() const
{
	return ParticleMetadataInstance;
}

bool FChaosVDParticleDataWrapper::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);

	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << Type;
	Ar << GeometryHash;

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::DeduplicatedDebugNameSerializationInCVD)
	{
#if WITH_EDITOR
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Ar << bHasDebugName;
		if (bHasDebugName)
		{
			if (Ar.IsLoading())
			{
				Ar << DebugName;
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#else
		bool bDummyHasDebugName;
		Ar << bDummyHasDebugName;
		if (bDummyHasDebugName)
		{
			if (Ar.IsLoading())
			{
				FString DummyString;
				Ar << DummyString;
			}
		}
#endif // WITH_EDITOR
	}
	else
	{
		Ar << MetadataId;
	}
	
	Ar << ParticleIndex;
	Ar << SolverID;

	Ar << ParticlePositionRotation;
	Ar << ParticleVelocities;

	// Bounds data was not exported prior to this version
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::ParticleInflatedBoundsInChaosVisualDebugger)
	{
		Ar << ParticleInflatedBounds;
	}

	Ar << ParticleDynamics;
	Ar << ParticleDynamicsMisc;
	Ar << ParticleMassProps;

	Ar << CollisionDataPerShape;

	Ar << ParticleCluster;
	
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AdditionalGameThreadDataSupportInChaosVisualDebugger)
	{
		Ar << ParticleContext;
		Ar << ParticleKinematicTarget;
		Ar << ParticleVWSmooth;
	}

	return !Ar.IsError();
}
