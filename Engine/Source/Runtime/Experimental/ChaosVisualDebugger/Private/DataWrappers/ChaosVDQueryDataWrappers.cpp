// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataWrappers/ChaosVDQueryDataWrappers.h"

#include "DataWrappers/ChaosVDDataSerializationMacros.h"
#include "UObject/FortniteMainBranchObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDQueryDataWrappers)

FStringView FChaosVDQueryVisitStep::WrapperTypeName = TEXT("FChaosVDQueryVisitStep");
FStringView FChaosVDQueryDataWrapper::WrapperTypeName = TEXT("FChaosVDQueryDataWrapper");

bool FChaosVDCollisionResponseParams::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << FlagsPerChannel;

	return !Ar.IsError();
}

bool FChaosVDCollisionObjectQueryParams::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::CollisionFilter64Bit)
	{
		uint8 OldObjectTypesToQuery;
		Ar << OldObjectTypesToQuery;
		ObjectTypesToQuery64 = OldObjectTypesToQuery;

#if WITH_EDITORONLY_DATA
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ObjectTypesToQuery_DEPRECATED = OldObjectTypesToQuery;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif // WITH_EDITORONLY_DATA
	}
	else
	{
		Ar << ObjectTypesToQuery64;
	}
	Ar << IgnoreMask;

	return !Ar.IsError();
}

bool FChaosVDCollisionQueryParams::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << TraceTag;
	Ar << OwnerTag;
	Ar << IgnoredActorsIDs;
	Ar << IgnoredComponentsNames;
	Ar << IgnoredActorsNames;

	EChaosVDCollisionQueryParamsFlags PackedQueryFlags = EChaosVDCollisionQueryParamsFlags::None;

	// Note: When the UI is done to show the enums in the same way we show these bitfiedls as read only booleans in CVD, we can remove all
	// this macro boilerplate and just serialize the the flags directly
	if (Ar.IsLoading())
	{
		Ar << PackedQueryFlags;
		CVD_UNPACK_BITFIELD_DATA(bTraceComplex, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::TraceComplex);
		CVD_UNPACK_BITFIELD_DATA(bFindInitialOverlaps, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::FindInitialOverlaps);
		CVD_UNPACK_BITFIELD_DATA(bReturnFaceIndex, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::ReturnFaceIndex);
		CVD_UNPACK_BITFIELD_DATA(bReturnPhysicalMaterial, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::ReturnPhysicalMaterial);
		CVD_UNPACK_BITFIELD_DATA(bIgnoreBlocks, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::IgnoreBlocks);
		CVD_UNPACK_BITFIELD_DATA(bIgnoreTouches, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::IgnoreTouches);
		CVD_UNPACK_BITFIELD_DATA(bSkipNarrowPhase, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::SkipNarrowPhase);
		CVD_UNPACK_BITFIELD_DATA(bTraceIntoSubComponents, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::TraceIntoSubComponents);
		CVD_UNPACK_BITFIELD_DATA(bReplaceHitWithSubComponents, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::ReplaceHitWithSubComponents);
	}
	else
	{
		CVD_PACK_BITFIELD_DATA(bTraceComplex, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::TraceComplex);
		CVD_PACK_BITFIELD_DATA(bFindInitialOverlaps, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::FindInitialOverlaps);
		CVD_PACK_BITFIELD_DATA(bReturnFaceIndex, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::ReturnFaceIndex);
		CVD_PACK_BITFIELD_DATA(bReturnPhysicalMaterial, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::ReturnPhysicalMaterial);
		CVD_PACK_BITFIELD_DATA(bIgnoreBlocks, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::IgnoreBlocks);
		CVD_PACK_BITFIELD_DATA(bIgnoreTouches, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::IgnoreTouches);
		CVD_PACK_BITFIELD_DATA(bSkipNarrowPhase, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::SkipNarrowPhase);
		CVD_PACK_BITFIELD_DATA(bTraceIntoSubComponents, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::TraceIntoSubComponents);
		CVD_PACK_BITFIELD_DATA(bReplaceHitWithSubComponents, PackedQueryFlags, EChaosVDCollisionQueryParamsFlags::ReplaceHitWithSubComponents);
		Ar << PackedQueryFlags;
	}

	return !Ar.IsError();
}

bool FChaosVDQueryFastData::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << Dir;
	Ar << InvDir;
	Ar << CurrentLength;
	Ar << InvDir;

	EChaosVDQueryFastDataParallelFlags PackedFlags = EChaosVDQueryFastDataParallelFlags::None;

	
	// Note: When the UI is done to show the enums in the same way we show these bitfiedls as read only booleans in CVD, we can remove all
	// this macro boilerplate and just serialize the the flags directly
	if (Ar.IsLoading())
	{
		Ar << PackedFlags;

		CVD_UNPACK_BITFIELD_DATA(bParallel0, PackedFlags, EChaosVDQueryFastDataParallelFlags::Parallel0);
		CVD_UNPACK_BITFIELD_DATA(bParallel1, PackedFlags, EChaosVDQueryFastDataParallelFlags::Parallel1);
		CVD_UNPACK_BITFIELD_DATA(bParallel2, PackedFlags, EChaosVDQueryFastDataParallelFlags::Parallel2);
	}
	else
	{
		CVD_PACK_BITFIELD_DATA(bParallel0, PackedFlags, EChaosVDQueryFastDataParallelFlags::Parallel0);
		CVD_PACK_BITFIELD_DATA(bParallel1, PackedFlags, EChaosVDQueryFastDataParallelFlags::Parallel1);
		CVD_PACK_BITFIELD_DATA(bParallel2, PackedFlags, EChaosVDQueryFastDataParallelFlags::Parallel2);

		Ar << PackedFlags;
	}

	return !Ar.IsError();
}

bool FChaosVDQueryHitData::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;

	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << Distance;
	Ar << FaceIdx;
	Ar << Flags;
	Ar << WorldPosition;
	Ar << WorldNormal;
	Ar << FaceNormal;
	Ar << bHasValidData;

	return !Ar.IsError();
}

bool FChaosVDQueryVisitStep::Serialize(FArchive& Ar)
{
	Ar << bHasValidData;
	if (!bHasValidData)
	{
		return !Ar.IsError();
	}

	Ar << OwningQueryID;
	Ar << Type;
	Ar << ShapeIndex;
	Ar << ParticleIndex;
	Ar << ParticleTransform;
	Ar << HitType;
	Ar << HitData;

	Ar << QueryFastData;

	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	if (Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) >= FFortniteMainBranchObjectVersion::AdditionalGameThreadDataSupportInChaosVisualDebugger)
	{
		Ar << RejectReason;
	}

	return !Ar.IsError();
}

bool FChaosVDQueryDataWrapper::Serialize(FArchive& Ar)
{
	Ar << ID;
	Ar << ParentQueryID;
	Ar << WorldSolverID;
	Ar << bIsRetryQuery;
	Ar << InputGeometryKey;
	Ar << GeometryOrientation;
	Ar << Type;
	Ar << Mode;
	Ar << StartLocation;
	Ar << EndLocation;
	Ar << CollisionChannel;
	Ar << CollisionQueryParams;
	Ar << CollisionResponseParams;
	Ar << CollisionObjectQueryParams;

	// Hits and steps are intentionally not serialized as they are recorded as separated events, and reconstructed during trace analysis

	return !Ar.IsError();
}
