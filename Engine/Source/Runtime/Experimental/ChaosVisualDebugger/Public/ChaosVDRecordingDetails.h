// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "RemoteSessionsManager.h"

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Containers/UnrealString.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "Misc/Guid.h"
#include "TraceDataRelayTransport.h"
#include "ChaosVDRecordingDetails.generated.h"

namespace Chaos::VD
{
/** Debugger Guid used to identify the debugger type when sending command and update messages */
static constexpr FGuid DebuggerGuid = FGuid(0x34CFFA28, 0x72434D08, 0xB07AD3FE, 0xE2D96272);
}


/** CVD specific message to send a start recording command with optional list of data channels to force to be enabled. */
USTRUCT()
struct FChaosVDStartRecordingCommandMessage : public UE::TraceBasedDebuggers::FStartRecordingCommandMessage
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FString> DataChannelsEnabledOverrideList;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS

enum class UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::ERecordingMode instead.") EChaosVDRecordingMode : uint8
{
	Invalid,
	Live,
	File
};

enum class UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::ETraceTransportMode instead.") EChaosVDTransportMode : uint8
{
	Invalid,
	/** The data will be traced directly to a file in the local file system */
	FileSystem,
	/** Data will be traced to the selected trace store server */
	TraceServer,
	/** Data will be traced directly to the editor, accessible via sockets */
	Direct,
	/** Data will be traced via the trace relay system using a custom transport, like normal UE Networking or a direct socket connection */
	Relay
};

struct UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FTraceConnectionDetails instead.") FChaosVDTraceDetails : public UE::TraceBasedDebuggers::FTraceConnectionDetails
{
	FChaosVDTraceDetails() = default;

	UE_DEPRECATED(5.7, "This property is no longer used and it will removed in the future")
	bool bIsConnected = false;

	UE_DEPRECATED(5.8, "Recording mode is no longer valid and will not be replaced")
	EChaosVDRecordingMode Mode = EChaosVDRecordingMode::Invalid;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS