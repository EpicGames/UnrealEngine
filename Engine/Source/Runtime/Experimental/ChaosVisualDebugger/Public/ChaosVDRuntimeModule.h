// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_CHAOS_VISUAL_DEBUGGER

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "Containers/Ticker.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "Templates/SharedPointer.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "ChaosVDRecordingDetails.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"
#include "Modules/ModuleInterface.h"
#include "TraceBasedDebuggerRuntime.h"

#define UE_API CHAOSVDRUNTIME_API

namespace UE::TraceBasedDebuggers
{
struct FStartRecordingCommandMessage;
struct FRelayTraceDataWriter;
}

struct FChaosVDRecording;
class FText;

namespace Chaos::VD
{
class FRelayTraceWriter;
}

/* Option flags that controls what should be recorded when doing a full capture **/
enum class EChaosVDFullCaptureFlags : int32
{
	Geometry = 1 << 0,
	Particles = 1 << 1,
};
ENUM_CLASS_FLAGS(EChaosVDFullCaptureFlags)

DECLARE_MULTICAST_DELEGATE(FChaosVDRecordingStateChangedDelegate)
DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDCaptureRequestDelegate, EChaosVDFullCaptureFlags)
DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDRecordingStartFailedDelegate, const FText&)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FRelayTraceDataDelegate instead")
DECLARE_DELEGATE_OneParam(FChaosVDRelayTraceDataDelegate, Chaos::VD::FRelayTraceWriter&)
UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FExternalRelayBufferQueueSizeDelegate instead")
DECLARE_DELEGATE_RetVal(uint64, FChaosVDExternalRelayBufferQueueSizeDelegate)
UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FTraceConnectionDetails instead")
DECLARE_DELEGATE_RetVal(FChaosVDTraceDetails, FChaosVDExternalTraceStatusDelegate)
PRAGMA_ENABLE_DEPRECATION_WARNINGS

DECLARE_LOG_CATEGORY_EXTERN(LogChaosVDRuntime, Log, All)

class FChaosVDRuntimeModule : public UE::TraceBasedDebuggers::FRuntimeModule, public IModuleInterface
{
public:

	UE_API FChaosVDRuntimeModule();
	static UE_API FChaosVDRuntimeModule& Get();

	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	using FRuntimeModule::StartRecording;

	/** Starts a CVD recording by starting a Trace session. It will stop any existing trace session
	 * @param InRecordingStartCommand : Used to determine if we want to record to file or a local trace server among other settings
	 */
	void StartRecording(const FChaosVDStartRecordingCommandMessage& InRecordingStartCommand)
	{
		FRuntimeModule::StartRecording(InRecordingStartCommand);
	}

	/** Returns a unique ID used to be used to identify CVD (Chaos Visual Debugger) data */
	UE_API int32 GenerateUniqueID();

	static FDelegateHandle RegisterRecordingStartedCallback(const FChaosVDRecordingStateChangedDelegate::FDelegate& InCallback)
	{
		UE::TWriteScopeLock WriteLock(DelegatesRWLock);
		return RecordingStartedDelegate.Add(InCallback);
	}

	static FDelegateHandle RegisterPostRecordingStartedCallback(const FChaosVDRecordingStateChangedDelegate::FDelegate& InCallback)
	{
		UE::TWriteScopeLock WriteLock(DelegatesRWLock);
		return PostRecordingStartedDelegate.Add(InCallback);
	}

	static FDelegateHandle RegisterRecordingStopCallback(const FChaosVDRecordingStateChangedDelegate::FDelegate& InCallback)
	{
		UE::TWriteScopeLock WriteLock(DelegatesRWLock);
		return RecordingStopDelegate.Add(InCallback);
	}

	static FDelegateHandle RegisterRecordingStartFailedCallback(const FChaosVDRecordingStartFailedDelegate::FDelegate& InCallback)
	{
		UE::TWriteScopeLock WriteLock(DelegatesRWLock);
		return RecordingStartFailedDelegate.Add(InCallback);
	}
	
	static FDelegateHandle RegisterFullCaptureRequestedCallback(const FChaosVDCaptureRequestDelegate::FDelegate& InCallback)
	{
		UE::TWriteScopeLock WriteLock(DelegatesRWLock);
		return PerformFullCaptureDelegate.Add(InCallback);
	}

	static bool RemoveRecordingStartedCallback(const FDelegateHandle& InDelegateToRemove)
	{
		UE::TWriteScopeLock WriteLock(DelegatesRWLock);
		return RecordingStartedDelegate.Remove(InDelegateToRemove);
	}
	
	static bool RemovePostRecordingStartedCallback(const FDelegateHandle& InDelegateToRemove)
	{
		UE::TWriteScopeLock WriteLock(DelegatesRWLock);
		return PostRecordingStartedDelegate.Remove(InDelegateToRemove);
	}

	static bool RemoveRecordingStopCallback(const FDelegateHandle& InDelegateToRemove)
	{
		UE::TWriteScopeLock WriteLock(DelegatesRWLock);
		return RecordingStopDelegate.Remove(InDelegateToRemove);
	}

	static bool RemoveRecordingStartFailedCallback(const FDelegateHandle& InDelegateToRemove)
	{
		UE::TWriteScopeLock WriteLock(DelegatesRWLock);
		return RecordingStartFailedDelegate.Remove(InDelegateToRemove);
	}
	
	static bool RemoveFullCaptureRequestedCallback(const FDelegateHandle& InDelegateToRemove)
	{
		UE::TWriteScopeLock WriteLock(DelegatesRWLock);
		return PerformFullCaptureDelegate.Remove(InDelegateToRemove);
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS

	/** [Deprecated] Returns the full path of the active recording file*/
	UE_DEPRECATED(5.7, "This method is no longer used. Use GetCurrentTraceConnectionDetails")
	UE_API FString GetLastRecordingFileNamePath() const;

	UE_DEPRECATED(5.8, "This method is no longer used. Use GetCurrentTraceConnectionDetails")
	UE_API FChaosVDTraceDetails GetCurrentTraceSessionDetails() const;

	UE_DEPRECATED(5.8, "This method is no longer used and will not be replaced")
	EChaosVDRecordingMode GetCurrentRecordingMode() const
	{
		return EChaosVDRecordingMode::Invalid;
	}

	UE_DEPRECATED(5.8, "Use the overload taking FChaosVDExternalTraceConnectionStatusDelegate instead")
	void RegisterExternalTraceStatusProvider(const FChaosVDExternalTraceStatusDelegate& InExternalTraceStatusCallback)
	{
	}

	UE_DEPRECATED(5.8, "Use GetRelayTraceWriterInstance instead.")
	Chaos::VD::FRelayTraceWriter* GetRelayWriterInstance() const
	{
		return nullptr;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

private:

	virtual void OnRecordingStartedInternal() override;
	virtual void OnRecordingStartFailedInternal(const FText& FailureReason) override;
	virtual void OnRecordingStoppedInternal() override;
	virtual void OnEnableRequiredTraceChannelsInternal() override;

	/** Queues a full Capture of the simulation on the next frame */
	bool RequestFullCapture(float DeltaTime);

	FTSTicker::FDelegateHandle FullCaptureRequesterHandle;

	static UE_API FChaosVDRecordingStateChangedDelegate RecordingStartedDelegate;
	static UE_API FChaosVDRecordingStateChangedDelegate PostRecordingStartedDelegate;
	static UE_API FChaosVDRecordingStateChangedDelegate RecordingStopDelegate;
	static UE_API FChaosVDRecordingStartFailedDelegate RecordingStartFailedDelegate;
	static UE_API FChaosVDCaptureRequestDelegate PerformFullCaptureDelegate;
	static UE_API FTransactionallySafeRWLock DelegatesRWLock;

	std::atomic<int32> LastGeneratedID;
};

#undef UE_API

#endif // WITH_CHAOS_VISUAL_DEBUGGER