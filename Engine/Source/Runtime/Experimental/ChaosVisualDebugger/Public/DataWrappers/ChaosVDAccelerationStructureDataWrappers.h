// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDDataSerializationMacros.h"
#include "ChaosVDParticleDataWrapper.h"

#include "ChaosVDAccelerationStructureDataWrappers.generated.h"

UENUM()
enum class EChaosVDAABBTreeNodeFlags : uint8
{
	None = 0,
	IsLeaf = 1 << 0,
	IsDirty = 1 << 1
};

ENUM_CLASS_FLAGS(EChaosVDAABBTreeNodeFlags)

UENUM()
enum class EChaosVDAccelerationStructureType : uint32
{
	BoundingVolume,
	AABBTree,
	AABBTreeBV,
	Collection,
	Unknown,
};

USTRUCT()
struct FChaosVDAccelerationStructureBase : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="CVD Debug")
	int32 SolverId = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="Data")
	EChaosVDAccelerationStructureType Type = EChaosVDAccelerationStructureType::Unknown;
};

USTRUCT()
struct FChaosVDBVCellElementDataWrapper : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Data")
	FBox Bounds = FBox(EForceInit::ForceInit);

	UPROPERTY(VisibleAnywhere, Category="Data")
	int32 ParticleIndex = INDEX_NONE;

	FIntVector3 StartIdx = FIntVector3::ZeroValue;
	FIntVector3 EndIdx = FIntVector3::ZeroValue;
	
	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

// TODO: Disabling C4996 due to MSVC 14.30 bug
#pragma warning(disable : 4996)
CVD_IMPLEMENT_SERIALIZER(FChaosVDBVCellElementDataWrapper)
#pragma warning(default : 4996)

USTRUCT()
struct FChaosVDBoundingVolumeDataWrapper : public FChaosVDAccelerationStructureBase
{
	GENERATED_BODY()

	FIntVector3 MElementsCounts = FIntVector3::ZeroValue; 

	TArray<TArray<FChaosVDBVCellElementDataWrapper>> MElements;

	UPROPERTY(VisibleAnywhere, Category="Settings")
	double MaxPayloadBounds = 0.0;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

// TODO: Disabling C4996 due to MSVC 14.30 bug
#pragma warning(disable : 4996)
CVD_IMPLEMENT_SERIALIZER(FChaosVDBoundingVolumeDataWrapper)
#pragma warning(default : 4996)

USTRUCT(DisplayName="AABB Tree Node")
struct FChaosVDAABBTreeNodeDataWrapper : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Node")
	FBox ChildrenBounds[2] = { FBox(ForceInitToZero), FBox(ForceInitToZero) };
	UPROPERTY(VisibleAnywhere, Category="Node")
	int32 ChildrenNodes[2] = { INDEX_NONE, INDEX_NONE };
	UPROPERTY(VisibleAnywhere, Category="Node")
	int32 ParentNode = INDEX_NONE;
	UPROPERTY(VisibleAnywhere, Category="Node")
	uint8 bLeaf : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="Node")
	uint8 bDirtyNode : 1 = false;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

// TODO: Disabling C4996 due to MSVC 14.30 bug
#pragma warning(disable : 4996)
CVD_IMPLEMENT_SERIALIZER(FChaosVDAABBTreeNodeDataWrapper)
#pragma warning(default : 4996)

USTRUCT()
struct FChaosVDAABBTreePayloadBoundsElement : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Leaf")
	int32 ParticleIndex = INDEX_NONE;

	/** Bounds used to add this element into the AABBTree */
	UPROPERTY(VisibleAnywhere, Category="Leaf")
	FBox Bounds = FBox(ForceInitToZero);

	/** Real Bounds of the element at the time we recorded the AABB Tree */
	//UPROPERTY(VisibleAnywhere, Category="Leaf")
	FBox ActualBounds = FBox(ForceInitToZero);
	
	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

// TODO: Disabling C4996 due to MSVC 14.30 bug
#pragma warning(disable : 4996)
CVD_IMPLEMENT_SERIALIZER(FChaosVDAABBTreePayloadBoundsElement)
#pragma warning(default : 4996)

USTRUCT(DisplayName="AABB Tree Leaf")
struct FChaosVDAABBTreeLeafDataWrapper : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category="Leaf")
	TArray<FChaosVDAABBTreePayloadBoundsElement> Elements;

	UPROPERTY(VisibleAnywhere, Category="Leaf")
	FBox Bounds = FBox(ForceInitToZero);
	
	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

// TODO: Disabling C4996 due to MSVC 14.30 bug
#pragma warning(disable : 4996)
CVD_IMPLEMENT_SERIALIZER(FChaosVDAABBTreeLeafDataWrapper)
#pragma warning(default : 4996)

USTRUCT()
struct FChaosVDAccelerationStructureContainer
{
	GENERATED_BODY()

	TMap<int32, TArray<TSharedPtr<FChaosVDAABBTreeDataWrapper>>> RecordedAABBTreesBySolverID;
};

USTRUCT(DisplayName="AABB Tree Data")
struct FChaosVDAABBTreeDataWrapper : public FChaosVDAccelerationStructureBase
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	UPROPERTY(VisibleAnywhere, Category="Tree Data")
	int32 RootNodeIndex = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="Tree Data")
	int32 TreeDepth = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="Tree Data")
	int32 NodesNum = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="Tree Data")
	int32 LeavesNum = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="Tree Settings")
	bool bDynamicTree = false;

	UPROPERTY(VisibleAnywhere, Category="Tree Settings")
	int32 MaxChildrenInLeaf = INDEX_NONE;
	
	UPROPERTY(VisibleAnywhere, Category="Tree Settings")
	int32 MaxTreeDepth = INDEX_NONE;
	
	UPROPERTY(VisibleAnywhere, Category="Tree Settings")
	double MaxPayloadBounds = -1.0;

	TArray<FChaosVDAABBTreeNodeDataWrapper> Nodes;

	TArray<FChaosVDAABBTreeLeafDataWrapper> TreeArrayLeafs;

	TArray<FChaosVDBoundingVolumeDataWrapper> BoundingVolumeLeafs;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	/** Returns a valid index for the root node taking ito account if this tree is dynamic or not */
	int32 GetCorrectedRootNodeIndex() const { return bDynamicTree ? RootNodeIndex : 0;};
};

// TODO: Disabling C4996 due to MSVC 14.30 bug
#pragma warning(disable : 4996)
CVD_IMPLEMENT_SERIALIZER(FChaosVDAABBTreeDataWrapper)
#pragma warning(default : 4996)
