// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDDataSerializationMacros.h"

#include "ChaosVDCameraDataWrapper.generated.h"

USTRUCT()
struct FChaosVDCameraIdentifier
{
	GENERATED_BODY()

	UPROPERTY()
	FName CameraName;

	UPROPERTY()
	FName ActorName;

	UPROPERTY()
	FTopLevelAssetPath ActorAssetPath;

	UPROPERTY()
	FTopLevelAssetPath MapAssetPath;

	bool operator==(const FChaosVDCameraIdentifier& CameraIdentifier) const
	{
		return CameraName == CameraIdentifier.CameraName && ActorName == CameraIdentifier.ActorName &&
			MapAssetPath == CameraIdentifier.MapAssetPath && ActorAssetPath == CameraIdentifier.ActorAssetPath; 
	}

	friend uint32 GetTypeHash(const FChaosVDCameraIdentifier& Other)
	{
		uint32 Hash = 0;
		Hash = HashCombine(Hash, GetTypeHash(Other.CameraName));
		Hash = HashCombine(Hash, GetTypeHash(Other.ActorName));
		Hash = HashCombine(Hash, GetTypeHash(Other.ActorAssetPath));
		Hash = HashCombine(Hash, GetTypeHash(Other.MapAssetPath));

		return Hash;
	}
};

USTRUCT()
struct FChaosVDCameraDataWrapper
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	UPROPERTY()
	FChaosVDCameraIdentifier Camera;

	UPROPERTY()
	FVector Position = FVector::ZeroVector;

	UPROPERTY()
	FQuat Rotation = FQuat::Identity;

	UPROPERTY()
	float FOV = 0;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

};

USTRUCT()
struct FChaosVDCameraDataContainer
{
	GENERATED_BODY()

	TArray<TSharedPtr<FChaosVDCameraDataWrapper>> CameraData;
};
