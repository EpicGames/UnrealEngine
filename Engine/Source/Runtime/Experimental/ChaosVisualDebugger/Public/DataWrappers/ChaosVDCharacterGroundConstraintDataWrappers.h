// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDParticleDataWrapper.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "ChaosVDCharacterGroundConstraintDataWrappers.generated.h"

#ifndef CVD_IMPLEMENT_SERIALIZER
		#define CVD_IMPLEMENT_SERIALIZER(Type) \
		inline FArchive& operator<<(FArchive& Ar, Type& Data) \
		{\
			Data.Serialize(Ar); \
			return Ar; \
		} \
		template<>\
		struct TStructOpsTypeTraits<Type> : public TStructOpsTypeTraitsBase2<Type> \
		{\
			enum\
			{\
				WithSerializer = true,\
			};\
		};\

#endif


USTRUCT()
struct FChaosVDCharacterGroundConstraintStateDataWrapper : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()
public:

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	//TODO: Make the island data visible when we add support to record that data

	int32 Island = INDEX_NONE;
	int32 Level = INDEX_NONE;
	int32 Color = INDEX_NONE;
	int32 IslandSize = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category=ConstraintState)
	bool bDisabled = false;

	UPROPERTY(VisibleAnywhere, Category= ConstraintState)
	FVector SolverAppliedForce = FVector::ZeroVector;
	UPROPERTY(VisibleAnywhere, Category= ConstraintState)
	FVector SolverAppliedTorque = FVector::ZeroVector;
};
CVD_IMPLEMENT_SERIALIZER(FChaosVDCharacterGroundConstraintStateDataWrapper)


USTRUCT()
struct FChaosVDCharacterGroundConstraintSettingsDataWrapper : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()
public:
	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	UPROPERTY(VisibleAnywhere, Category = Settings)
	FVector VerticalAxis = FVector(ForceInitToZero);

	UPROPERTY(VisibleAnywhere, Category = Settings, meta = (ForceUnits = "cm"))
	double TargetHeight = 0.0;

	UPROPERTY(VisibleAnywhere, Category = Settings, meta = (ForceUnits = "Newtons"))
	double RadialForceLimit = 0.0;

	UPROPERTY(VisibleAnywhere, Category = Settings, meta = (ForceUnits = "Newtons"))
	double FrictionForceLimit = 0.0;

	UPROPERTY(VisibleAnywhere, Category = Settings, meta = (ForceUnits = "NewtonMeters"))
	double TwistTorqueLimit = 0.0;

	UPROPERTY(VisibleAnywhere, Category = Settings, meta = (ForceUnits = "NewtonMeters"))
	double SwingTorqueLimit = 0.0;

	UPROPERTY(VisibleAnywhere, Category = Settings)
	double CosMaxWalkableSlopeAngle = 0.0;

	UPROPERTY(VisibleAnywhere, Category = Settings)
	double DampingFactor = 0.0;

	UPROPERTY(VisibleAnywhere, Category = Settings, meta = (ForceUnits = "cm"))
	double AssumedOnGroundHeight = 0.0;
};
CVD_IMPLEMENT_SERIALIZER(FChaosVDCharacterGroundConstraintSettingsDataWrapper)

USTRUCT()
struct FChaosVDCharacterGroundConstraintDataDataWrapper : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()
public:
	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	UPROPERTY(VisibleAnywhere, Category = Data)
	FVector GroundNormal = FVector(ForceInitToZero);

	UPROPERTY(VisibleAnywhere, Category = Data, meta = (ForceUnits = "cm"))
	FVector TargetDeltaPosition = FVector(ForceInitToZero);

	UPROPERTY(VisibleAnywhere, Category = Data, meta = (ForceUnits = "radians"))
	double TargetDeltaFacing = 0.0;

	UPROPERTY(VisibleAnywhere, Category = Data, meta = (ForceUnits = "cm"))
	double GroundDistance = 0.0;

	UPROPERTY(VisibleAnywhere, Category = Data)
	double CosMaxWalkableSlopeAngle = 0.0;
};
CVD_IMPLEMENT_SERIALIZER(FChaosVDCharacterGroundConstraintDataDataWrapper)


USTRUCT()
struct FChaosVDCharacterGroundConstraint : public FChaosVDConstraintDataWrapperBase
{
	GENERATED_BODY()
public:

	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	int32 SolverID = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="General")
	int32 ConstraintIndex = INDEX_NONE;

	int32 CharacterParticleIndex = INDEX_NONE;
	int32 GroundParticleIndex = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category=State)
	FChaosVDCharacterGroundConstraintStateDataWrapper State;
	
	UPROPERTY(VisibleAnywhere, Category=Settings)
	FChaosVDCharacterGroundConstraintSettingsDataWrapper Settings;

	UPROPERTY(VisibleAnywhere, Category=Data)
	FChaosVDCharacterGroundConstraintDataDataWrapper Data;

	virtual int32 GetSolverID() const override { return SolverID; }
	CHAOSVDRUNTIME_API virtual int32 GetParticleIDAtSlot(EChaosVDParticlePairIndex IndexSlot) const override;
	virtual int32 GetConstraintIndex () const override {return ConstraintIndex; }
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDCharacterGroundConstraint)



