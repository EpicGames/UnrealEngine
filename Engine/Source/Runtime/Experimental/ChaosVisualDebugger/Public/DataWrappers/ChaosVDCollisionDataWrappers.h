// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDDataSerializationMacros.h"
#include "HAL/Platform.h"

#include "ChaosVDCollisionDataWrappers.generated.h"

namespace Chaos
{
	enum class EParticleType : uint8;
	class FChaosArchive;
}

UENUM()
enum class EChaosVDContactShapesType
{
	Unknown,
	SphereSphere,
	SphereCapsule,
	SphereBox,
	SphereConvex,
	SphereTriMesh,
	SphereHeightField,
	SpherePlane,
	CapsuleCapsule,
	CapsuleBox,
	CapsuleConvex,
	CapsuleTriMesh,
	CapsuleHeightField,
	BoxBox,
	BoxConvex,
	BoxTriMesh,
	BoxHeightField,
	BoxPlane,
	ConvexConvex,
	ConvexTriMesh,
	ConvexHeightField,
	GenericConvexConvex,
	LevelSetLevelSet,

	NumShapesTypes
};

UENUM()
enum class EChaosVDContactPointType : int8
{
	Unknown,
	VertexPlane,
	EdgeEdge,
	PlaneVertex,
	VertexVertex,
};

USTRUCT()
struct FChaosVDContactPoint
{
	GENERATED_BODY()

	// Shape-space contact points on the two bodies
	UPROPERTY(VisibleAnywhere, Category=Contact)
	FVector ShapeContactPoints[2] = { FVector(ForceInit), FVector(ForceInit) };

	// Shape-space contact normal on the second shape with direction that points away from shape 1
	UPROPERTY(VisibleAnywhere, Category=Contact)
	FVector  ShapeContactNormal = FVector(ForceInit);

	// Contact separation (negative for overlap)
	UPROPERTY(VisibleAnywhere, Category=Contact)
	float Phi = 0.f;

	// Face index of the shape we hit. Only valid for Heightfield and Trimesh contact points, otherwise INDEX_NONE
	UPROPERTY(VisibleAnywhere, Category=Contact)
	int32 FaceIndex = 0;

	// Whether this is a vertex-plane contact, edge-edge contact etc.
	UPROPERTY(VisibleAnywhere, Category=Contact)
	EChaosVDContactPointType ContactType = EChaosVDContactPointType::Unknown;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDContactPoint)

UENUM()
enum class EChaosVDManifoldPointFlags : uint8
{
	None = 0,
	Disabled = 1 << 0,
	WasRestored = 1 << 1,
	WasReplaced = 1 << 2,
	HasStaticFrictionAnchor = 1 << 3,
	IsValid = 1 << 4,
	InsideStaticFrictionCone = 1 << 5,
};
ENUM_CLASS_FLAGS(EChaosVDManifoldPointFlags)

USTRUCT()
struct FChaosVDManifoldPoint
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category=ContactData)
	uint8 bDisabled:1 = false;
	UPROPERTY()
	uint8 bWasRestored:1 = false;
	UPROPERTY()
	uint8 bWasReplaced:1 = false;
	UPROPERTY(VisibleAnywhere, Category=ContactData)
	uint8 bHasStaticFrictionAnchor:1 = false;

	UPROPERTY()
	uint8 bIsValid:1 = false;
	UPROPERTY(VisibleAnywhere, Category=ContactData)
	uint8 bInsideStaticFrictionCone:1 = false;

	UPROPERTY(VisibleAnywhere, Category=ContactData)
	FVector NetPushOut = FVector(ForceInit);
	UPROPERTY(VisibleAnywhere, Category=ContactData)
	FVector NetImpulse = FVector(ForceInit);

	UPROPERTY(VisibleAnywhere, Category=ContactData)
	float TargetPhi = 0.f;
	UPROPERTY(VisibleAnywhere, Category=ContactData)
	float InitialPhi = 0.f;
	UPROPERTY()
	FVector ShapeAnchorPoints[2] = { FVector(ForceInit), FVector(ForceInit) };
	UPROPERTY()
	FVector InitialShapeContactPoints[2] = { FVector(ForceInit), FVector(ForceInit) };
	UPROPERTY(VisibleAnywhere, Category=ContactData)
	FChaosVDContactPoint ContactPoint;

	UPROPERTY()
	FVector ShapeContactPoints[2] =  { FVector(ForceInit), FVector(ForceInit) };

	bool bIsSelectedInEditor = false;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDManifoldPoint)

USTRUCT()
struct FChaosVDCollisionMaterial
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = MaterialData)
	int32 FaceIndex = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category = MaterialData)
	float MaterialDynamicFriction = 0.0f;
	
	UPROPERTY(VisibleAnywhere, Category = MaterialData)
	float MaterialStaticFriction = 0.0f;
	
	UPROPERTY(VisibleAnywhere, Category = MaterialData)
	float MaterialRestitution = 0.0f;

	UPROPERTY(VisibleAnywhere, Category = MaterialData)
	float DynamicFriction = 0.0f;
	
	UPROPERTY(VisibleAnywhere, Category = MaterialData)
	float StaticFriction = 0.0f;
	
	UPROPERTY(VisibleAnywhere, Category = MaterialData)
	float Restitution = 0.0f;
	
	UPROPERTY(VisibleAnywhere, Category = MaterialData)
	float RestitutionThreshold = 0.0f;
	
	UPROPERTY(VisibleAnywhere, Category = MaterialData)
	float InvMassScale0 = 0.0f;
	
	UPROPERTY(VisibleAnywhere, Category = MaterialData)
	float InvMassScale1 = 0.0f;
	
	UPROPERTY(VisibleAnywhere, Category = MaterialData)
	float InvInertiaScale0 = 0.0f;
	
	UPROPERTY(VisibleAnywhere, Category = MaterialData)
	float InvInertiaScale1 = 0.0f;

	bool Serialize(FArchive& Ar);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDCollisionMaterial)

UENUM()
enum class EChaosVDConstraintFlags : uint16
{
	None = 0,
	IsCurrent = 1 << 0,
	Disabled = 1 << 1,
	UseManifold = 1 << 2,
	UseIncrementalManifold = 1 << 3,
	CanRestoreManifold = 1 << 4,
	WasManifoldRestored = 1 << 5,
	IsQuadratic0 = 1 << 6,
	IsQuadratic1 = 1 << 7,
	IsProbe = 1 << 8,
	CCDEnabled = 1 << 9,
	CCDSweepEnabled = 1 << 10,
	ModifierApplied = 1 << 11,
	MaterialSet = 1 << 12,
};

USTRUCT()
struct FChaosVDConstraint
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	UPROPERTY()
	uint8 bIsCurrent:1 = false;
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	uint8 bDisabled:1 = false;
	UPROPERTY()
	uint8 bUseManifold:1 = false;
	UPROPERTY()
	uint8 bUseIncrementalManifold:1 = false;
	UPROPERTY()
	uint8 bCanRestoreManifold:1 = false;
	UPROPERTY()
	uint8 bWasManifoldRestored:1 = false;
	UPROPERTY()
	uint8 bIsQuadratic0:1 = false;
	UPROPERTY()
	uint8 bIsQuadratic1:1 = false;
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	uint8 bIsProbe:1 = false;
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	uint8 bCCDEnabled:1 = false;
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	uint8 bCCDSweepEnabled:1 = false;
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	uint8 bModifierApplied:1 = false;
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	uint8 bMaterialSet:1 = false;
	
	UPROPERTY(VisibleAnywhere, Category = ConstraintData)
	FChaosVDCollisionMaterial Material;

	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	FVector AccumulatedImpulse = FVector(ForceInit);
	
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	EChaosVDContactShapesType ShapesType = EChaosVDContactShapesType::Unknown;
	
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	FTransform ShapeWorldTransforms[2] = { FTransform::Identity, FTransform::Identity };

	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	FTransform ImplicitTransforms[2] = { FTransform::Identity, FTransform::Identity };
	
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	float CullDistance = 0.f;
	
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	TArray<float> CollisionMargins;
	
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	float CollisionTolerance = 0.f;
	
	UPROPERTY()
	int32 ClosestManifoldPointIndex = 0;
	
	UPROPERTY()
	int32 ExpectedNumManifoldPoints = 0;
	
	UPROPERTY()
	FVector LastShapeWorldPositionDelta = FVector(ForceInit);
	
	UPROPERTY()
	FQuat LastShapeWorldRotationDelta = FQuat(ForceInit);
	
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	float Stiffness = 0.f;

	UPROPERTY(VisibleAnywhere, Category = ConstraintData)
	float MinInitialPhi = 0.f;

	UPROPERTY(VisibleAnywhere, Category = ConstraintData)
	float InitialOverlapDepenetrationVelocity = -1.f;
	
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	float CCDTimeOfImpact = 0.f;
	
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	float CCDEnablePenetration = 0.f;
	
	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	float CCDTargetPenetration = 0.f;

	UPROPERTY(VisibleAnywhere, Category=ConstraintData)
	TArray<FChaosVDManifoldPoint> ManifoldPoints;

	UPROPERTY()
	int32 Particle0Index = INDEX_NONE;
	UPROPERTY()
	int32 Particle1Index = INDEX_NONE;

	UPROPERTY()
	int32 SolverID = INDEX_NONE;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDConstraint)

UENUM()
enum class EChaosVDMidPhaseFlags : uint8
{
	None = 0,
	IsActive = 1 << 0,
	IsCCD = 1 << 1,
	IsCCDActive = 1 << 2,
	IsSleeping = 1 << 3,
	IsModified = 1 << 4,
};

UENUM()
enum class EChaosVDMidPhaseType : int8
{
	// A general purpose midphase that handle BVHs, Meshes, 
	// Unions of Unions, etc in the geometry hierarchy.
	Generic,

	// A midphase optimized for particle pairs with a small
	// number of shapes. Pre-expands the set of potentially
	// colliding shape pairs.
	ShapePair,

	// A midphase used to collide particles as sphere approximations
	SphereApproximation,

	Unknown
};

USTRUCT()
struct FChaosVDParticlePairMidPhase
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	UPROPERTY()
	int32 SolverID = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category=General)
	EChaosVDMidPhaseType MidPhaseType = EChaosVDMidPhaseType::Unknown;

	UPROPERTY(VisibleAnywhere, Category=Flags)
	uint8 bIsActive:1 = false;
	UPROPERTY(VisibleAnywhere, Category=Flags)
	uint8 bIsCCD:1 = false;
	UPROPERTY(VisibleAnywhere, Category=Flags)
	uint8 bIsCCDActive:1 = false;
	UPROPERTY(VisibleAnywhere, Category=Flags)
	uint8 bIsSleeping:1 = false;
	UPROPERTY(VisibleAnywhere, Category=Flags)
	uint8 bIsModified:1 = false;

	UPROPERTY(VisibleAnywhere, Category=Misc)
	int32 LastUsedEpoch = 0;

	UPROPERTY(VisibleAnywhere, Category=Particle)
	int32 Particle0Idx = 0;
	UPROPERTY(VisibleAnywhere, Category=Particle)
	int32 Particle1Idx = 0;

	UPROPERTY(VisibleAnywhere, Category=Constraints)
	TArray<FChaosVDConstraint> Constraints;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDParticlePairMidPhase)

UENUM()
enum class EChaosVDCollisionTraceFlag
{
	/** Use project physics settings (DefaultShapeComplexity) */
	UseDefault,
	/** Create both simple and complex shapes. Simple shapes are used for regular scene queries and collision tests. Complex shape (per poly) is used for complex scene queries.*/
	UseSimpleAndComplex,
	/** Create only simple shapes. Use simple shapes for all scene queries and collision tests.*/
	UseSimpleAsComplex,
	/** Create only complex shapes (per poly). Use complex shapes for all scene queries and collision tests. Can be used in simulation for static shapes only (i.e can be collided against but not moved through forces or velocity.) */
	UseComplexAsSimple,
	/** */
	MAX,
};

USTRUCT()
struct UE_DEPRECATED(5.8, "This struct is no longer used.") FChaosVDCollisionFilterData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category=CollisionData)
	uint32 Word0 = 0;
	UPROPERTY(VisibleAnywhere, Category=CollisionData)
	uint32 Word1 = 0;
	UPROPERTY(VisibleAnywhere, Category=CollisionData)
	uint32 Word2 = 0;
	UPROPERTY(VisibleAnywhere, Category=CollisionData)
	uint32 Word3 = 0;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	bool operator==(const FChaosVDCollisionFilterData& Other) const = default;
};


PRAGMA_DISABLE_DEPRECATION_WARNINGS
template <>
struct TTypeTraits<FChaosVDCollisionFilterData> : public TTypeTraitsBase <FChaosVDCollisionFilterData>
{
	enum { IsBytewiseComparable = true };
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDCollisionFilterData)
PRAGMA_ENABLE_DEPRECATION_WARNINGS

struct FChaosVDCollisionFilterDataV2
{
	bool operator==(const FChaosVDCollisionFilterDataV2& Other) const = default;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	UE_INTERNAL CHAOSVDRUNTIME_API uint8 GetMaskFilter() const;
	UE_INTERNAL CHAOSVDRUNTIME_API uint32 GetFlags() const;
	UE_INTERNAL CHAOSVDRUNTIME_API void SetMaskFilterAndFlags(uint32 InMaskFilter, uint32 InFlags);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_INTERNAL CHAOSVDRUNTIME_API void LoadLegacy(const FChaosVDCollisionFilterData& QueryData, const FChaosVDCollisionFilterData& SimData);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_INTERNAL CHAOSVDRUNTIME_API static void ExtractWord3(const uint32 Word3, uint8& OutMaskFilter, uint8& OutChannelIndex, uint32& OutFlags);

	UE_INTERNAL uint32 OwnerId = 0;
	UE_INTERNAL uint32 ComponentId = 0;
	UE_INTERNAL uint64 OverlapMask = 0;
	UE_INTERNAL uint64 BlockMask = 0;
	UE_INTERNAL uint64 CollisionChannelMask = 0;
	UE_INTERNAL uint32 MaskFilterAndFlags = 0;

	UE_INTERNAL bool bLoadedFromLegacy = false;
	UE_INTERNAL uint32 LegacySimBlockMask = 0;
	UE_INTERNAL uint32 LegacySimWord3 = 0;

private:
	UE_INTERNAL static constexpr uint32 NumFlagsBits = 21;
	UE_INTERNAL static constexpr uint32 NumCollisionChannelIndexBits = 5;
	UE_INTERNAL static constexpr uint32 NumMaskFilterBits = 6;

	PRAGMA_DISABLE_INTERNAL_WARNINGS
	UE_INTERNAL static constexpr uint32 CollisionChannelIndexOffset = NumFlagsBits;
	UE_INTERNAL static constexpr uint32 MaskFilterOffset = CollisionChannelIndexOffset + NumCollisionChannelIndexBits;

	UE_INTERNAL static constexpr uint32 FlagsMask = ((1 << NumFlagsBits) - 1);
	UE_INTERNAL static constexpr uint32 CollisionChannelIndexMask = ((1 << NumCollisionChannelIndexBits) - 1) << CollisionChannelIndexOffset;
	UE_INTERNAL static constexpr uint32 MaskFilterMask = ((1 << NumMaskFilterBits) - 1) << MaskFilterOffset;
	PRAGMA_ENABLE_INTERNAL_WARNINGS
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDCollisionFilterDataV2)

UENUM()
enum class EChaosVDCollisionShapeDataFlags : uint8
{
	None = 0,
	SimCollision = 1 << 0,
	QueryCollision = 1 << 1,
	IsProbe = 1 << 2,
};

USTRUCT()
struct FChaosVDShapeCollisionData
{
	GENERATED_BODY()
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FChaosVDShapeCollisionData() = default;
	FChaosVDShapeCollisionData(const FChaosVDShapeCollisionData&) = default;
	FChaosVDShapeCollisionData(FChaosVDShapeCollisionData&&) = default;
	FChaosVDShapeCollisionData& operator=(const FChaosVDShapeCollisionData&) = default;
	FChaosVDShapeCollisionData& operator=(FChaosVDShapeCollisionData&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UPROPERTY(VisibleAnywhere, Category=CollisionData)
	EChaosVDCollisionTraceFlag CollisionTraceType = EChaosVDCollisionTraceFlag::UseDefault;

	UPROPERTY()
	uint8 bSimCollision : 1 = false;
	UPROPERTY()
	uint8 bQueryCollision : 1 = false;
	UPROPERTY()
	uint8 bIsProbe : 1 = false;

	#if WITH_EDITORONLY_DATA
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "QueryData has been deprecated. Use FilterData instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "QueryData has been deprecated"))
	FChaosVDCollisionFilterData QueryData_DEPRECATED;

	UE_DEPRECATED(5.8, "SimData has been deprecated. Use FilterData instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "SimData has been deprecated"))
	FChaosVDCollisionFilterData SimData_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	#endif // WITH_EDITORONLY_DATA

	FChaosVDCollisionFilterDataV2 FilterData;

	UPROPERTY(VisibleAnywhere, Category="CVD Data")
	bool bIsComplex = false;

	UPROPERTY(VisibleAnywhere, Category="CVD Data")
	bool bIsValid = false;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	CHAOSVDRUNTIME_API bool operator==(const FChaosVDShapeCollisionData& Other) const;
};

template <>
struct TTypeTraits<FChaosVDShapeCollisionData> : public TTypeTraitsBase <FChaosVDShapeCollisionData>
{
	enum { IsBytewiseComparable = true };
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDShapeCollisionData)

/** Minimum amount of data needed to reconstruct Collision names in CVD
 * based on already serialized flags
 */
USTRUCT()
struct FChaosVDCollisionChannelInfo
{
	GENERATED_BODY()

	UPROPERTY()
	FString DisplayName;

	UPROPERTY()
	int32 CollisionChannel = INDEX_NONE;

	UPROPERTY()
	bool bIsTraceType = false;
	
	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDCollisionChannelInfo)

/** Container for recorded custom collision profile data */
USTRUCT()
struct FChaosVDCollisionChannelsInfoContainer
{
	GENERATED_BODY()
	
	UE_INTERNAL static constexpr int32 MaxSupportedChannels = 64;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FChaosVDCollisionChannelsInfoContainer() = default;
	FChaosVDCollisionChannelsInfoContainer(const FChaosVDCollisionChannelsInfoContainer&) = default;
	FChaosVDCollisionChannelsInfoContainer(FChaosVDCollisionChannelsInfoContainer&&) = default;
	FChaosVDCollisionChannelsInfoContainer& operator=(const FChaosVDCollisionChannelsInfoContainer&) = default;
	FChaosVDCollisionChannelsInfoContainer& operator=(FChaosVDCollisionChannelsInfoContainer&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "CustomChannelsNames has been deprecated")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "CustomChannelsNames has been deprecated"))
	FChaosVDCollisionChannelInfo CustomChannelsNames_DEPRECATED[32] = {};
	#endif // WITH_EDITORONLY_DATA

	UE_INTERNAL TArray<FChaosVDCollisionChannelInfo> CollisionChannelInfos;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDCollisionChannelsInfoContainer)
