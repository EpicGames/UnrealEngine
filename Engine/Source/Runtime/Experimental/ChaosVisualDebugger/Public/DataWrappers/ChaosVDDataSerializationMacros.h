// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef CVD_READ_PACKED_BITFIELD_SERIALIZER
	#define CVD_UNPACK_BITFIELD_DATA(Value, PackedBits, EnumFlag) \
				Value = EnumHasAnyFlags(PackedBits, EnumFlag);
#endif

#ifndef CVD_WRITE_PACKED_BITFIELD_SERIALIZER
	#define CVD_PACK_BITFIELD_DATA(Value, PackedBits, EnumFlag) \
				if (Value) \
				{ \
					EnumAddFlags(PackedBits, EnumFlag);\
				}
#endif

#ifndef CVD_SERIALIZE_STATIC_ARRAY
	#define CVD_SERIALIZE_STATIC_ARRAY(Archive, Array) \
	{ \
		constexpr int32 Size = UE_ARRAY_COUNT(Array) ; \
		for (int32 Index = 0; Index < Size; Index++)\
		{\
			Archive << Array[Index]; \
		}\
	}
#endif

#ifndef CVD_IMPLEMENT_SERIALIZER
		#define CVD_IMPLEMENT_SERIALIZER(Type) \
		inline FArchive& operator<<(FArchive& Ar, Type& Data) \
		{\
			Data.Serialize(Ar); \
			return Ar; \
		} 
#endif