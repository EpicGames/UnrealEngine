// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDDataSerializationMacros.h"
#include "ChaosVDParticleDataWrapper.h"

#include "ChaosVDDebugShapeDataWrapper.generated.h"

USTRUCT()
struct FChaosVDDebugShapeDataContainer
{
	GENERATED_BODY()

	TArray<TSharedPtr<FChaosVDDebugDrawBoxDataWrapper>> RecordedDebugDrawBoxes;
	TArray<TSharedPtr<FChaosVDDebugDrawLineDataWrapper>> RecordedDebugDrawLines;
	TArray<TSharedPtr<FChaosVDDebugDrawSphereDataWrapper>> RecordedDebugDrawSpheres;
	TArray<TSharedPtr<FChaosVDDebugDrawImplicitObjectDataWrapper>> RecordedDebugDrawImplicitObjects;
};

USTRUCT()
struct FChaosVDMultiSolverDebugShapeDataContainer
{
	GENERATED_BODY()

	TMap<int32, TSharedPtr<FChaosVDDebugShapeDataContainer>> DataBySolverID;
};

USTRUCT()
struct FChaosVDDebugDrawShapeBase : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	UPROPERTY()
	int32 SolverID = INDEX_NONE;
	
	UPROPERTY()
	FName Tag = NAME_None;

	UPROPERTY()
	FColor Color = FColor::Blue;

	UPROPERTY()
	EChaosVDParticleContext ThreadContext = EChaosVDParticleContext::GameThread;

protected:
	CHAOSVDRUNTIME_API void SerializeBase_Internal(FArchive& Ar);
};

USTRUCT()
struct FChaosVDDebugDrawBoxDataWrapper : public FChaosVDDebugDrawShapeBase
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	UPROPERTY()
	FBox Box = FBox(ForceInitToZero);

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDDebugDrawBoxDataWrapper)

USTRUCT()
struct FChaosVDDebugDrawSphereDataWrapper : public FChaosVDDebugDrawShapeBase
{
	GENERATED_BODY()
	
	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	UPROPERTY()
	FVector Origin = FVector::ZeroVector;
	
	UPROPERTY()
	float Radius = 0.0f;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDDebugDrawSphereDataWrapper)

USTRUCT()
struct FChaosVDDebugDrawLineDataWrapper : public FChaosVDDebugDrawShapeBase
{
	GENERATED_BODY()
	
	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	UPROPERTY()
	FVector StartLocation = FVector::ZeroVector;

	UPROPERTY()
	FVector EndLocation = FVector::ZeroVector;

	UPROPERTY()
	bool bIsArrow = false;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDDebugDrawLineDataWrapper)

USTRUCT()
struct FChaosVDDebugDrawImplicitObjectDataWrapper : public FChaosVDDebugDrawShapeBase
{
	GENERATED_BODY()
	
	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	uint32 ImplicitObjectHash = 0;

	FTransform ParentTransform = FTransform();

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
};
