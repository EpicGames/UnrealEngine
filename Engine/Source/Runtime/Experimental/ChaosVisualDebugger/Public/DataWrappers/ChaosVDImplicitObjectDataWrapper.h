// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Containers/StringView.h"

/** Data Wrapper for used to Trace Implicit objects
 * @note This should not be templated. But making it so for now so this can live in the ChaosVDRuntime module without referencing Chaos types directly.
 * Once we are close to ship the first version of the tool, this will likely change to the specific types and to the Chaos Module
 */
template<class SerializableImplicitType, class ArchiveType>
struct FChaosVDImplicitObjectDataWrapper
{
	inline static FStringView WrapperTypeName = TEXT("FChaosVDImplicitObjectDataWrapper");

	uint32 Hash;
	SerializableImplicitType ImplicitObject;

	bool Serialize(ArchiveType& Ar);
};

template <class SerializableImplicitType, class ArchiveType>
bool FChaosVDImplicitObjectDataWrapper<SerializableImplicitType, ArchiveType>::Serialize(ArchiveType& Ar)
{
	Ar << Hash;
	Ar << ImplicitObject;

	return !Ar.IsError();
}
