// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDParticleDataWrapper.h"
#include "DataWrappers/ChaosVDDataSerializationMacros.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "ChaosVDJointDataWrappers.generated.h"

UENUM()
enum class EChaosVDJointReSimType
{
	/** Fully re-run simulation and keep results (any forces must be applied again) */
	FullResim = 0,
	/** Use previous forces and snap to previous results regardless of variation - used to push other objects away */
	ResimAsFollower = 1
};

UENUM()
enum class EChaosVDJointSyncType
{
	/** In sync with recorded data */
	InSync,
	/** Recorded data mismatches, must run collision detection again */
	HardDesync,
};

UENUM()
enum class EChaosVDJointStateFlags : uint8
{
	None = 0,
	Disabled = 1 << 0,
	Broken = 1 << 1,
	Breaking = 1 << 2,
	DriveTargetChanged = 1 << 3,
	EnabledDuringResim = 1 << 4,
};
ENUM_CLASS_FLAGS(EChaosVDJointStateFlags)

UENUM()
enum class EChaosVDGTJointStateFlags : uint8
{
	None = 0,
	IsBreaking = 1 << 0,
	IsBroken = 1 << 1,
	DriveTargetChanged = 1 << 2,
	IsViolating = 1 << 3,
};
ENUM_CLASS_FLAGS(EChaosVDGTJointStateFlags)

USTRUCT()
struct FChaosVDJointStateDataWrapper : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()
public:

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	//TODO: Make the island data visible when we add support to record that data

	int32 Island = INDEX_NONE;
	int32 Level = INDEX_NONE;
	int32 Color = INDEX_NONE;
	int32 IslandSize = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category=JointState)
	uint8 bDisabled : 1 = false;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	uint8 bBroken : 1 = false;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	uint8 bBreaking : 1 = false;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	uint8 bDriveTargetChanged : 1 = false;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	uint8 bEnabledDuringResim : 1 = true;

	UPROPERTY(VisibleAnywhere, Category=JointState)
	FVector LinearImpulse = FVector::ZeroVector;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	FVector AngularImpulse = FVector::ZeroVector;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	EChaosVDJointReSimType ResimType = EChaosVDJointReSimType::FullResim;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	EChaosVDJointSyncType SyncState = EChaosVDJointSyncType::InSync;
	
};
CVD_IMPLEMENT_SERIALIZER(FChaosVDJointStateDataWrapper)

USTRUCT()
struct FChaosVDGTJointStateDataWrapper : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()
public:

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	UPROPERTY(VisibleAnywhere, Category=JointState)
	uint8 bIsBreaking : 1 = false;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	uint8 bIsBroken : 1 = false;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	uint8 bDriveTargetChanged : 1 = false;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	uint8 bIsViolating : 1 = false;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	FVector Force = FVector::ZeroVector;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	FVector Torque = FVector::ZeroVector;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	float LinearViolation = 0.f;
	UPROPERTY(VisibleAnywhere, Category=JointState)
	float AngularViolation = 0.f;
};
CVD_IMPLEMENT_SERIALIZER(FChaosVDGTJointStateDataWrapper)

UENUM()
enum class EChaosVDJointSolverSettingsFlags : uint16
{
	None = 0,

	UseLinearSolver = 1 << 0,
	SortEnabled = 1 << 1,
	SolvePositionLast = 1 << 2,
	UsePositionBasedDrives = 1 << 3,
	EnableTwistLimits = 1 << 4,
	EnableSwingLimits = 1 << 5,
	EnableDrives = 1 << 6,
};

ENUM_CLASS_FLAGS(EChaosVDJointSettingsFlags)

USTRUCT()
struct FChaosVDJointSolverSettingsDataWrapper : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()
public:
	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	UPROPERTY(VisibleAnywhere, Category=Tolerances)
	double SwingTwistAngleTolerance = 0.0;
	UPROPERTY(VisibleAnywhere, Category=Tolerances)
	double PositionTolerance = 0.0;
	UPROPERTY(VisibleAnywhere, Category=Tolerances)
	double AngleTolerance = 0.0;

	UPROPERTY(VisibleAnywhere, Category="Stability Control")
	double MinParentMassRatio = 0.0;
	UPROPERTY(VisibleAnywhere, Category="Stability Control")
	double MaxInertiaRatio = 0.0;

	UPROPERTY(VisibleAnywhere, Category="Solver Stiffness")
	double MinSolverStiffness = 0.0;
	UPROPERTY(VisibleAnywhere, Category="Solver Stiffness")
	double MaxSolverStiffness = 0.0;
	UPROPERTY(VisibleAnywhere, Category="Solver Stiffness")
	int32 NumIterationsAtMaxSolverStiffness = INDEX_NONE;
	UPROPERTY(VisibleAnywhere, Category="Solver Stiffness")
	int32 NumShockPropagationIterations = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bUseLinearSolver : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bSortEnabled : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bSolvePositionLast : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bUsePositionBasedDrives : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bEnableTwistLimits : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bEnableSwingLimits : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bEnableDrives : 1 = false;

	UPROPERTY(VisibleAnywhere, Category="General")
	double LinearStiffnessOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double TwistStiffnessOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SwingStiffnessOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double LinearProjectionOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double AngularProjectionOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double ShockPropagationOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double LinearDriveStiffnessOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double LinearDriveDampingOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double AngularDriveStiffnessOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double AngularDriveDampingOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SoftLinearStiffnessOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SoftLinearDampingOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SoftTwistStiffnessOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SoftTwistDampingOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SoftSwingStiffnessOverride = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SoftSwingDampingOverride = 0.0;
};
CVD_IMPLEMENT_SERIALIZER(FChaosVDJointSolverSettingsDataWrapper)

UENUM()
enum class EChaosVDJointMotionType : int32
{
	Free,
	Limited,
	Locked,
};

UENUM()
enum class EChaosVDJointForceMode : int32
{
	Acceleration,
	Force,
};

UENUM()
enum class EChaosVDPlasticityType : int32
{
	Free,
	Shrink,
	Grow,
};

UENUM()
enum class EChaosVDJointSettingsFlags : uint32
{
	None = 0,
	CollisionEnabled = 1 << 0,
	MassConditioningEnabled = 1 << 1,
	AngularSLerpPositionDriveEnabled = 1 << 2,
	AngularSLerpVelocityDriveEnabled = 1 << 3,
	AngularTwistPositionDriveEnabled = 1 << 4,
	AngularTwistVelocityDriveEnabled = 1 << 5,
	AngularSwingPositionDriveEnabled = 1 << 6,
	AngularSwingVelocityDriveEnabled = 1 << 7,
	SoftLinearLimitsEnabled = 1 << 8,
	SoftTwistLimitsEnabled = 1 << 9,
	SoftSwingLimitsEnabled = 1 << 10,
	LinearPositionDriveEnabled0 = 1 << 11,
	LinearPositionDriveEnable1 = 1 << 12,
	LinearPositionDriveEnable2 = 1 << 13,
	LinearVelocityDriveEnabled0 = 1 << 14,
	LinearVelocityDriveEnabled1 = 1 << 15,
	LinearVelocityDriveEnabled2 = 1 << 16,
	UseLinearSolver = 1 << 17,
};

USTRUCT()
struct FChaosVDJointSettingsDataWrapper : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()
public:
	
	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	UPROPERTY(VisibleAnywhere, Category="General")
	FTransform ConnectorTransforms[2];
	UPROPERTY(VisibleAnywhere, Category="General")
	double Stiffness = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double LinearProjection = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double AngularProjection = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double ShockPropagation = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double TeleportDistance = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double TeleportAngle = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double ParentInvMassScale = 0.0;

	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bCollisionEnabled : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bMassConditioningEnabled : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bUseLinearSolver : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bSoftLinearLimitsEnabled : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bSoftTwistLimitsEnabled : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bSoftSwingLimitsEnabled : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bAngularSLerpPositionDriveEnabled : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bAngularSLerpVelocityDriveEnabled : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bAngularTwistPositionDriveEnabled : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bAngularTwistVelocityDriveEnabled : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bAngularSwingPositionDriveEnabled : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint16 bAngularSwingVelocityDriveEnabled : 1 = false;

	UPROPERTY(VisibleAnywhere, Category="General")
	EChaosVDJointMotionType LinearMotionTypes[3] = { EChaosVDJointMotionType::Free, EChaosVDJointMotionType::Free, EChaosVDJointMotionType::Free };
	UPROPERTY(VisibleAnywhere, Category="General")
	double LinearLimit = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	EChaosVDJointMotionType AngularMotionTypes[3] = { EChaosVDJointMotionType::Free, EChaosVDJointMotionType::Free, EChaosVDJointMotionType::Free };
	UPROPERTY(VisibleAnywhere, Category="General")
	FVector AngularLimits = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, Category="General")
	EChaosVDJointForceMode LinearSoftForceMode = EChaosVDJointForceMode::Acceleration;
	UPROPERTY(VisibleAnywhere, Category="General")
	EChaosVDJointForceMode AngularSoftForceMode = EChaosVDJointForceMode::Acceleration;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SoftLinearStiffness = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SoftLinearDamping = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SoftTwistStiffness = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SoftTwistDamping = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SoftSwingStiffness = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SoftSwingDamping = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double LinearRestitution = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double TwistRestitution = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SwingRestitution = 0.0;

	UPROPERTY(VisibleAnywhere, Category="General")
	double LinearContactDistance = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double TwistContactDistance = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double SwingContactDistance = 0.0;

	UPROPERTY(VisibleAnywhere, Category="General")
	FVector LinearDrivePositionTarget = FVector::ZeroVector;
	UPROPERTY(VisibleAnywhere, Category="General")
	FVector LinearDriveVelocityTarget = FVector::ZeroVector;
	
	UPROPERTY(VisibleAnywhere, Category="General")
	uint8 bLinearPositionDriveEnabled0 : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint8 bLinearPositionDriveEnabled1 : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint8 bLinearPositionDriveEnabled2 : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint8 bLinearVelocityDriveEnabled0 : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint8 bLinearVelocityDriveEnabled1 : 1 = false;
	UPROPERTY(VisibleAnywhere, Category="General")
	uint8 bLinearVelocityDriveEnabled2 : 1 = false;
	
	UPROPERTY(VisibleAnywhere, Category="General")
	EChaosVDJointForceMode LinearDriveForceMode = EChaosVDJointForceMode::Acceleration;
	UPROPERTY(VisibleAnywhere, Category="General")
	FVector LinearDriveStiffness = FVector::ZeroVector;
	UPROPERTY(VisibleAnywhere, Category="General")
	FVector LinearDriveDamping = FVector::ZeroVector;
	UPROPERTY(VisibleAnywhere, Category="General")
	FVector LinearDriveMaxForce = FVector::ZeroVector;
	UPROPERTY(VisibleAnywhere, Category="General")
	FQuat AngularDrivePositionTarget = FQuat::Identity;
	UPROPERTY(VisibleAnywhere, Category="General")
	FVector AngularDriveVelocityTarget = FVector::ZeroVector;
	UPROPERTY(VisibleAnywhere, Category="General")
	EChaosVDJointForceMode AngularDriveForceMode = EChaosVDJointForceMode::Acceleration;
	UPROPERTY(VisibleAnywhere, Category="General")
	FVector AngularDriveStiffness = FVector::ZeroVector;
	UPROPERTY(VisibleAnywhere, Category="General")
	FVector AngularDriveDamping = FVector::ZeroVector;
	UPROPERTY(VisibleAnywhere, Category="General")
	FVector AngularDriveMaxTorque = FVector::ZeroVector;
	UPROPERTY(VisibleAnywhere, Category="General")
	double LinearBreakForce = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double LinearPlasticityLimit = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	EChaosVDPlasticityType LinearPlasticityType = EChaosVDPlasticityType::Free;
	UPROPERTY(VisibleAnywhere, Category="General")
	double LinearPlasticityInitialDistanceSquared = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double AngularBreakTorque = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double AngularPlasticityLimit = 0.0;
	UPROPERTY(VisibleAnywhere, Category="General")
	double ContactTransferScale = 0.0;
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDJointSettingsDataWrapper)

USTRUCT()
struct FChaosVDJointConstraint : public FChaosVDConstraintDataWrapperBase
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	int32 SolverID = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="General")
	int32 ConstraintIndex = INDEX_NONE;

	int32 ParticleParIndexes[2] = { INDEX_NONE, INDEX_NONE };

	UPROPERTY(VisibleAnywhere, Category=JointState)
	FChaosVDJointStateDataWrapper PhysicsThreadJointState;

	UPROPERTY()
	FChaosVDGTJointStateDataWrapper GameThreadJointState;
	
	UPROPERTY(VisibleAnywhere, Category=JointSettings)
	FChaosVDJointSettingsDataWrapper JointSettings;
	
	virtual int32 GetSolverID() const override { return SolverID; }
	virtual int32 GetParticleIDAtSlot(EChaosVDParticlePairIndex IndexSlot) const override { return ParticleParIndexes[static_cast<int32>(IndexSlot)]; }
	virtual int32 GetConstraintIndex () const override {return ConstraintIndex; }
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDJointConstraint)



