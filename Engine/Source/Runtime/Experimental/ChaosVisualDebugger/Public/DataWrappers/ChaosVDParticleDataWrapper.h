// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDCollisionDataWrappers.h"
#include "UObject/ObjectMacros.h"
#include "HAL/Platform.h"
#include "StructUtils/InstancedStruct.h"

#include "ChaosVDParticleDataWrapper.generated.h"

UENUM()
enum class EChaosVDParticleType : uint8
{
	Static,
	Kinematic,
	Rigid,
	Clustered,
	StaticMesh,
	SkeletalMesh,
	GeometryCollection,
	Unknown
};

UENUM()
enum class EChaosVDSleepType : uint8
{
	MaterialSleep,
	NeverSleep
};

UENUM()
enum class EChaosVDObjectStateType: int8
{
	Uninitialized = 0,
	Sleeping = 1,
	Kinematic = 2,
	Static = 3,
	Dynamic = 4,

	Count
};

USTRUCT()
struct FChaosVDParticleMetadata
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	UPROPERTY(VisibleAnywhere, Category=ParticleName)
	FName OwnerName;
	UPROPERTY(VisibleAnywhere, Category=ParticleName)
	FName ComponentName;
	UPROPERTY(VisibleAnywhere, Category=ParticleName)
	FName BoneName;
	UPROPERTY(VisibleAnywhere, Category=ParticleName)
	int32 Index = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category=ParticleAssetData)
	FTopLevelAssetPath MapAssetPath;
	UPROPERTY(VisibleAnywhere, Category=ParticleAssetData)
	FTopLevelAssetPath OwnerAssetPath;

	UPROPERTY()
	uint64 MetadataID = 0;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	CHAOSVDRUNTIME_API FString ToString() const;
};

/** Base struct that declares the interface to be used for any ParticleData Viewer */
USTRUCT()
struct FChaosVDWrapperDataBase
{
	GENERATED_BODY()
	virtual ~FChaosVDWrapperDataBase() = default;

	virtual bool HasValidData() const { return bHasValidData; }

	void MarkAsValid() { bHasValidData = true; }

protected:
	UPROPERTY()
	bool bHasValidData = false;
};

enum class EChaosVDParticlePairIndex : uint8
{
	Index_0,
	Index_1
};

/** Base struct that declares the interface to be used for any Constraint data to be visualized */
USTRUCT()
struct FChaosVDConstraintDataWrapperBase : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()
	virtual ~FChaosVDConstraintDataWrapperBase() override = default;

	virtual int32 GetSolverID() const  { return INDEX_NONE; }
	virtual int32 GetParticleIDAtSlot (EChaosVDParticlePairIndex IndexSlot) const { return INDEX_NONE;}
	virtual int32 GetConstraintIndex () const {  return INDEX_NONE; }
};

USTRUCT()
struct FChaosVDFRigidParticleControlFlags : public FChaosVDWrapperDataBase 
{
	GENERATED_BODY()

	FChaosVDFRigidParticleControlFlags()
		: bGravityEnabled(false),
		  bCCDEnabled(false),
		  bOneWayInteractionEnabled(false),
		  bInertiaConditioningEnabled(false), 
		  GravityGroupIndex(0),
		  bMACDEnabled(false), 
		  bPartialIslandSleepAllowed(true),
		  bGyroscopicTorqueEnabled(false)
	{
	}

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		bGravityEnabled = Other.GetGravityEnabled();
		bCCDEnabled = Other.GetCCDEnabled();
		bOneWayInteractionEnabled = Other.GetOneWayInteractionEnabled();
		bInertiaConditioningEnabled = Other.GetInertiaConditioningEnabled();
		GravityGroupIndex = Other.GetGravityGroupIndex();
		bMACDEnabled = Other.GetMACDEnabled();
		bPartialIslandSleepAllowed = Other.GetPartialIslandSleepAllowed();
		bGyroscopicTorqueEnabled = Other.GetGyroscopicTorqueEnabled();

		bHasValidData = true;
	}

	template <typename TOther>
	void CopyTo(TOther& Other) const
	{
		Other.SetGravityEnabled(bGravityEnabled);
		Other.SetCCDEnabled(bCCDEnabled);
		Other.SetOneWayInteractionEnabled(bOneWayInteractionEnabled);
		Other.SetInertiaConditioningEnabled(bInertiaConditioningEnabled);
		Other.SetGravityGroupIndex(GravityGroupIndex);
		Other.SetMACDEnabled(bMACDEnabled);
		Other.SetPartialIslandSleepAllowed(bPartialIslandSleepAllowed);
		Other.SetGyroscopicTorqueEnabled(bGyroscopicTorqueEnabled);
		
	}

	UPROPERTY(VisibleAnywhere, Category= "Particle Control Flags")
	bool bGravityEnabled;
	UPROPERTY(VisibleAnywhere, Category= "Particle Control Flags")
	bool bCCDEnabled;
	UPROPERTY(VisibleAnywhere, Category= "Particle Control Flags")
	bool bOneWayInteractionEnabled;
	UPROPERTY(VisibleAnywhere, Category = "Particle Control Flags")
	bool bInertiaConditioningEnabled;
	UPROPERTY(VisibleAnywhere, Category= "Particle Control Flags")
	int32 GravityGroupIndex;
	UPROPERTY(VisibleAnywhere, Category = "Particle Control Flags")
	bool bMACDEnabled;
	UPROPERTY(VisibleAnywhere, Category = "Particle Control Flags")
	bool bPartialIslandSleepAllowed;
	UPROPERTY(VisibleAnywhere, Category = "Particle Control Flags")
	bool bGyroscopicTorqueEnabled;
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDFRigidParticleControlFlags)

template<>
struct TStructOpsTypeTraits<FChaosVDFRigidParticleControlFlags> : public TStructOpsTypeTraitsBase2<FChaosVDFRigidParticleControlFlags>
{
	enum
	{
		WithSerializer = true,
	};
};

/** Simplified UStruct version of FParticlePositionRotation.
 * Used to be able to show the values in the editor and allow changes via the Property Editor.
 */
USTRUCT()
struct FChaosVDParticlePositionRotation : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	FChaosVDParticlePositionRotation()
	{
	}

	enum class EAccessorType
	{
		PQ,
		XR
	};

	template <typename OtherType, EAccessorType AccessorType>
	void CopyFrom(const OtherType& Other)
	{
		if constexpr (AccessorType == EAccessorType::PQ)
		{
			MX = Other.GetP();
			MR = Other.GetQ();

		}
		else if (AccessorType == EAccessorType::XR)
		{
			MX = Other.GetX();
			MR = Other.GetR();
		}

		bHasValidData = true;
	}

	template <typename OtherType, EAccessorType AccessorType>
	void CopyTo(OtherType& Other) const
	{
		if constexpr (AccessorType == EAccessorType::PQ)
		{
			Other.SetP(MX);
			Other.SetQ(MR);
		}
		else if (AccessorType == EAccessorType::XR)
		{
			Other.SetX(MX);
			Other.SetR(MR);
		}
	}

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	UPROPERTY(VisibleAnywhere, Category= "Particle Velocities")
	FVector MX = FVector(ForceInit);

	UPROPERTY(VisibleAnywhere, Category= "Particle Velocities")
	FQuat MR = FQuat(ForceInit);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDParticlePositionRotation)

/** Simplified UStruct version of FParticleVelocities.
 * Used to be able to show the values in the editor and allow changes via the Property Editor.
 */
USTRUCT()
struct FChaosVDParticleVelocities : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	FChaosVDParticleVelocities()
	{
	}

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		MV = Other.GetV();
		MW = Other.GetW();
		bHasValidData = true;
	}
	
	template <typename TOther>
	void CopyTo(TOther& Other) const
	{
		Other.SetV(MV);
		Other.SetW(MW);
	}
	
	UPROPERTY(VisibleAnywhere, Category= "Particle Velocities")
	FVector MV = FVector(ForceInit);

	UPROPERTY(VisibleAnywhere, Category= "Particle Velocities")
	FVector MW = FVector(ForceInit);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDParticleVelocities)

USTRUCT()
struct FChaosVDParticleBounds : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	FChaosVDParticleBounds()
	{
	}

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		MMin = Other.WorldSpaceInflatedBounds().Min();
		MMax = Other.WorldSpaceInflatedBounds().Max();
		bHasValidData = true;
	}

	template <typename TOther>
	void CopyTo(TOther& Other) const
	{
		// TODO:
	}

	UPROPERTY(VisibleAnywhere, Category = "Particle Bounds Min")
	FVector MMin = FVector(ForceInit);

	UPROPERTY(VisibleAnywhere, Category = "Particle Bounds Max")
	FVector MMax = FVector(ForceInit);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDParticleBounds)

/** Simplified UStruct version of FParticleDynamics.
 * Used to be able to show the values in the editor and allow changes via the Property Editor.
 */
USTRUCT()
struct FChaosVDParticleDynamics : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()
	
	FChaosVDParticleDynamics()
	{
	}

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	template <typename TOther>
	void CopyFrom(TOther& Other)
	{
		MAcceleration = Other.Acceleration();
		MAngularAcceleration = Other.AngularAcceleration();
		MLinearImpulseVelocity = Other.LinearImpulseVelocity();
		MAngularImpulseVelocity = Other.AngularImpulseVelocity();

		bHasValidData = true;
	}

	template <typename TOther>
	void CopyTo(TOther& Other) const
	{
		Other.SetAcceleration(MAcceleration);
        Other.SetAngularAcceleration(MAngularAcceleration);
		Other.SetLinearImpulseVelocity(MLinearImpulseVelocity);
        Other.SetAngularImpulseVelocity(MAngularImpulseVelocity);
	}

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamics")
	FVector MAcceleration = FVector(ForceInit);

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamics")
	FVector MAngularAcceleration = FVector(ForceInit);

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamics")
	FVector MLinearImpulseVelocity = FVector(ForceInit);

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamics")
	FVector MAngularImpulseVelocity = FVector(ForceInit);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDParticleDynamics)

/** Simplified UStruct version of FParticleMassProps.
 * Used to be able to show the values in the editor and allow changes via the Property Editor.
 */
USTRUCT()
struct FChaosVDParticleMassProps : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	FChaosVDParticleMassProps(): MM(0), MInvM(0)
	{
	}

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);
	
	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		MCenterOfMass = Other.CenterOfMass();
		MRotationOfMass = Other.RotationOfMass();
		MI = FVector(Other.I());
		MInvI = FVector(Other.InvI());
		MM = Other.M();
		MInvM = Other.InvM();

		bHasValidData = true;
	}
	
	template <typename TOther>
	void CopyTo(TOther& Other) const
	{
		Other.SetCenterOfMass(MCenterOfMass);
		Other.SetRotationOfMass(MRotationOfMass);
		Other.SetI(MI);
		Other.SetInvI(MInvI);
		Other.SetM(MM);
		Other.SetInvM(MInvM);
	}
	
	UPROPERTY(VisibleAnywhere, Category= "Particle Mass Props")
	FVector MCenterOfMass = FVector(ForceInit);

	UPROPERTY(VisibleAnywhere, Category= "Particle Mass Props")
	FQuat MRotationOfMass = FQuat(ForceInit);

	UPROPERTY(VisibleAnywhere, Category= "Particle Mass Props")
	FVector MI = FVector(ForceInit);

	UPROPERTY(VisibleAnywhere, Category= "Particle Mass Props")
	FVector MInvI = FVector(ForceInit);

	UPROPERTY(VisibleAnywhere, Category= "Particle Mass Props")
	double MM = 0.0;

	UPROPERTY(VisibleAnywhere, Category= "Particle Mass Props")
	double MInvM = 0.0;
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDParticleMassProps)

/** Simplified UStruct version of FParticleDynamicMisc.
 * Used to be able to show the values in the editor and allow changes via the Property Editor.
 */
USTRUCT()
struct FChaosVDParticleDynamicMisc : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	FChaosVDParticleDynamicMisc(): MLinearEtherDrag(0), MAngularEtherDrag(0), MMaxLinearSpeedSq(0), MMaxAngularSpeedSq(0), MInitialOverlapDepenetrationVelocity(0), MSleepThresholdMultiplier(1),
	                               MCollisionGroup(0), MObjectState(), MSleepType(), bDisabled(false), PositionSolverIterationCount(INDEX_NONE), VelocitySolverIterationCount(INDEX_NONE), ProjectionSolverIterationCount(INDEX_NONE)
	{
	}

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		MLinearEtherDrag = Other.LinearEtherDrag();
		MAngularEtherDrag = Other.AngularEtherDrag();
		MMaxLinearSpeedSq = Other.MaxLinearSpeedSq();
		MMaxAngularSpeedSq = Other.MaxAngularSpeedSq();
		MInitialOverlapDepenetrationVelocity = Other.InitialOverlapDepenetrationVelocity();
		MSleepThresholdMultiplier = Other.SleepThresholdMultiplier();
		MObjectState = static_cast<EChaosVDObjectStateType>(Other.ObjectState());
		MCollisionGroup = Other.CollisionGroup();
		MSleepType =  static_cast<EChaosVDSleepType>(Other.SleepType());
		MCollisionConstraintFlag = Other.CollisionConstraintFlags();
	
		MControlFlags.CopyFrom(Other.ControlFlags());
		
		bDisabled = Other.Disabled();

		bHasValidData = true;

		PositionSolverIterationCount = static_cast<int8>(Other.IterationSettings().GetNumPositionIterations());
		VelocitySolverIterationCount = static_cast<int8>(Other.IterationSettings().GetNumVelocityIterations());
		ProjectionSolverIterationCount = static_cast<int8>(Other.IterationSettings().GetNumProjectionIterations());
	}
	
	template <typename OtherType, typename ControlFlagsType, typename SleepStateType>
	void CopyWithoutStateTo(OtherType& Other) const
	{
		Other.SetLinearEtherDrag(MLinearEtherDrag);
		Other.SetAngularEtherDrag(MAngularEtherDrag);
		Other.SetMaxLinearSpeedSq(MMaxLinearSpeedSq);
		Other.SetMaxAngularSpeedSq(MMaxAngularSpeedSq);
		Other.SetInitialOverlapDepenetrationVelocity(MInitialOverlapDepenetrationVelocity);
		Other.SetSleepThresholdMultiplier(MSleepThresholdMultiplier);
		Other.SetCollisionGroup(MCollisionGroup);
		Other.SetSleepType(static_cast<SleepStateType>(MSleepType));
		Other.SetCollisionConstraintFlags(MCollisionConstraintFlag);

		ControlFlagsType ControlFlags;
		MControlFlags.CopyTo(ControlFlags);

		Other.SetControlFlags(ControlFlags);
		
		Other.SetDisabled(bDisabled);

		Other.SetPositionSolverIterations((uint32)PositionSolverIterationCount);
		Other.SetVelocitySolverIterations((uint32)VelocitySolverIterationCount);
		Other.SetProjectionSolverIterations((uint32)ProjectionSolverIterationCount);
	}

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamic Misc")
	double MLinearEtherDrag;

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamic Misc")
	double MAngularEtherDrag;

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamic Misc")
	double MMaxLinearSpeedSq;
	
	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamic Misc")
	double MMaxAngularSpeedSq;

	UPROPERTY(VisibleAnywhere, Category = "Particle Dynamic Misc")
	float MInitialOverlapDepenetrationVelocity;

	UPROPERTY(VisibleAnywhere, Category = "Particle Dynamic Misc")
	float MSleepThresholdMultiplier;

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamic Misc")
	int32 MCollisionGroup;

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamic Misc")
	EChaosVDObjectStateType MObjectState;

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamic Misc")
	EChaosVDSleepType MSleepType;

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamic Misc")
	uint32 MCollisionConstraintFlag = 0;

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamic Misc")
	FChaosVDFRigidParticleControlFlags MControlFlags;

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamic Misc")
	bool bDisabled;

	UPROPERTY(VisibleAnywhere, Category = "Particle Control Flags")
	int8 PositionSolverIterationCount;

	UPROPERTY(VisibleAnywhere, Category = "Particle Control Flags")
	int8 VelocitySolverIterationCount;

	UPROPERTY(VisibleAnywhere, Category = "Particle Control Flags")
	int8 ProjectionSolverIterationCount;
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDParticleDynamicMisc)

/** Represents the data of a connectivity Edge that CVD can use to reconstruct it during playback */
USTRUCT()
struct FChaosVDConnectivityEdge
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category=ConnectivityEdge)
	int32 SiblingParticleID = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category=ConnectivityEdge)
	float Strain = 0.0f;

	bool Serialize(FArchive& Ar)
	{
		Ar << SiblingParticleID;
		Ar << Strain;

		return true;
	}
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDConnectivityEdge)

/** Structure contained data from a Clustered Particle.
 * Used to be able to show the values in the editor and allow changes via the Property Editor.
 */
USTRUCT()
struct FChaosVDParticleCluster : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	FChaosVDParticleCluster()
	{
	}

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		ParentParticleID = Other.ClusterIds().Id ? Other.ClusterIds().Id->UniqueIdx().Idx : INDEX_NONE;

		NumChildren = Other.ClusterIds().NumChildren;

		ChildToParent = Other.ChildToParent();
		ClusterGroupIndex = Other.ClusterGroupIndex();
		bInternalCluster = Other.InternalCluster();
		CollisionImpulse = Other.CollisionImpulses();
		ExternalStrains = Other.GetExternalStrain();
		InternalStrains = Other.GetInternalStrains();
		Strain = Other.Strain();


		ConnectivityEdges.Reserve(Other.ConnectivityEdges().Num());
		for (auto& Edge : Other.ConnectivityEdges())
		{
			int32 SiblingId = Edge.Sibling ? Edge.Sibling->UniqueIdx().Idx : INDEX_NONE;
			ConnectivityEdges.Add( { SiblingId,  Edge.Strain });
		}

		bIsAnchored = Other.IsAnchored();
		bUnbreakable = Other.Unbreakable();
		bIsChildToParentLocked = Other.IsChildToParentLocked();
		
		bHasValidData = true;
	}
	
	template <typename TOther>
	void CopyTo(TOther& Other) const
	{
		//TODO:
	}

	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster")
	int32 ParentParticleID = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster | Cluster Id")
	int32 NumChildren = INDEX_NONE;
	
	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster")
	FTransform ChildToParent;

	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster")
	int32 ClusterGroupIndex = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster")
	bool bInternalCluster = false;

	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster")
	float CollisionImpulse = 0.0f;

	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster")
	float ExternalStrains = 0.0f;
	
	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster")
	float InternalStrains = 0.0f;

	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster")
	float Strain = 0.0f;

	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster")
	TArray<FChaosVDConnectivityEdge> ConnectivityEdges;

	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster")
	bool bIsAnchored = false;

	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster")
	bool bUnbreakable = false;

	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster")
	bool bIsChildToParentLocked = false;
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDParticleCluster)

UENUM()
enum class EChaosVDParticleContext
{
	Invalid,
	GameThread,
	PhysicsThread,
};

UENUM()
enum class EChaosVDKinematicTargetMode
{
	None,			/** Particle does not move and no data is changed */
	Reset,			/** Particle does not move, velocity and angular velocity are zeroed, then mode is set to "None". */
	Position,		/** Particle is moved to Kinematic Target transform, velocity and angular velocity updated to reflect the change, then mode is set to "Reset". */
	Velocity,		/** Particle is moved based on velocity and angular velocity, mode remains as "Velocity" until changed. */
};

USTRUCT()
struct FChaosVDKinematicTarget : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	template <typename OtherType>
	void CopyFrom(const OtherType& Other)
	{
		Mode = static_cast<EChaosVDKinematicTargetMode>(Other.GetMode());

		if (Mode == EChaosVDKinematicTargetMode::Position)
		{
			Position = Other.GetPosition();
			Rotation = FQuat(Other.GetRotation());
		}

		bHasValidData = true;
	}

	template <typename OtherType, class ModeType>
	void CopyTo(OtherType& Other) const
	{
		Other.SetTargetMode(Position, Rotation);
		Other.SetMode(static_cast<ModeType>(Mode));
	}

	UPROPERTY(VisibleAnywhere, Category= "Kinematic Target")
	FQuat Rotation = FQuat(ForceInit);

	UPROPERTY(VisibleAnywhere, Category= "Kinematic Target")
	FVector Position = FVector(ForceInit);;

	UPROPERTY(VisibleAnywhere, Category= "Kinematic Target")
	EChaosVDKinematicTargetMode Mode = EChaosVDKinematicTargetMode::None;
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDKinematicTarget)

USTRUCT()
struct FChaosVDVSmooth : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	template <typename OtherType>
	void CopyFrom(const OtherType& Other)
	{
		MV = Other.VSmooth();
		MW = Other.WSmooth();

		bHasValidData = true;
	}

	template <typename OtherType>
	void CopyTo(OtherType& Other) const
	{
		Other.SetVSmooth(MV);
		Other.SetWSmooth(MW);
	}

	UPROPERTY(VisibleAnywhere, Category= "VSmooth")
	FVector MV = FVector(ForceInit);
	UPROPERTY(VisibleAnywhere, Category= "VSmooth")
	FVector MW = FVector(ForceInit);
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDVSmooth)

/** Simplified UStruct version of FChaosVDParticleDataWrapper.
 * Used to be able to show the values in the editor and allow changes via the Property Editor.
 */
USTRUCT(DisplayName="Particle Data")
struct FChaosVDParticleDataWrapper : public FChaosVDWrapperDataBase
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS

	FChaosVDParticleDataWrapper(const FChaosVDParticleDataWrapper& Other) = default;
	FChaosVDParticleDataWrapper(FChaosVDParticleDataWrapper&& Other) noexcept = default;
	FChaosVDParticleDataWrapper& operator=(const FChaosVDParticleDataWrapper& Other) = default;
	FChaosVDParticleDataWrapper& operator=(FChaosVDParticleDataWrapper&& Other) noexcept = default;
	virtual ~FChaosVDParticleDataWrapper() override = default;
	FChaosVDParticleDataWrapper()
	{
	}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	GENERATED_BODY()

	UPROPERTY()
	int32 DirtyFlagsBits = 0;

	UPROPERTY()
	EChaosVDParticleContext ParticleContext = EChaosVDParticleContext::Invalid;

	UPROPERTY(VisibleAnywhere, Category= "General")
	uint32 GeometryHash = 0;

#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.7, "Please Use the GetDebugName accessor. This property is no longer in use")
	/** Shows the serialized debug name of the particle. This is only valid in CVD recordings from UE 5.6 and earlier.
	 * Particles Debug name now is part of the particle metadata
	* */
	UPROPERTY(VisibleAnywhere, Category= "General", DisplayName="Legacy Debug Name")
	FString DebugName;

	/** Serialized boolean used as a flag to know if a particle has the legacy debug name serialized as part of the particle data.
	 * If false, it means the particle either had no debug name at the time of being traced or it is using the new particle metadata system
	 */
	UE_DEPRECATED(5.7, "Please Use the HasLegacyDebugName accessor. This property is no longer in use")
	bool bHasDebugName = false;
#endif

	UPROPERTY()
	uint64 MetadataId = 0;

	UPROPERTY(VisibleAnywhere, Category= "General")
	int32 ParticleIndex = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category= "General")
	int32 SolverID = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category= "General")
	EChaosVDParticleType Type = EChaosVDParticleType::Unknown;

	UPROPERTY(VisibleAnywhere, Category= "Particle Position Rotation")
	FChaosVDParticlePositionRotation ParticlePositionRotation;

	UPROPERTY(VisibleAnywhere, Category= "Particle Velocities")
	FChaosVDParticleVelocities ParticleVelocities;

	UPROPERTY(VisibleAnywhere, Category = "Particle Inflated Bounds")
	FChaosVDParticleBounds ParticleInflatedBounds;

	UPROPERTY(VisibleAnywhere, Category= "Particle Kinematic Target")
	FChaosVDKinematicTarget ParticleKinematicTarget;

	UPROPERTY(VisibleAnywhere, Category= "Particle V/W Smooth")
	FChaosVDVSmooth ParticleVWSmooth;

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamics")
	FChaosVDParticleDynamics ParticleDynamics;

	UPROPERTY(VisibleAnywhere, Category= "Particle Dynamics Misc")
	FChaosVDParticleDynamicMisc ParticleDynamicsMisc;

	UPROPERTY(VisibleAnywhere, Category= "Particle Mass Props")
	FChaosVDParticleMassProps ParticleMassProps;

	UPROPERTY(VisibleAnywhere, Category= "Particle Cluster Data")
	FChaosVDParticleCluster ParticleCluster;

	UPROPERTY()
	TArray<FChaosVDShapeCollisionData> CollisionDataPerShape;

	CHAOSVDRUNTIME_API bool HasLegacyDebugName() const;
	CHAOSVDRUNTIME_API FString GetDebugName() const;
	CHAOSVDRUNTIME_API void SetMetadataInstance(const TSharedPtr<FChaosVDParticleMetadata>& InMetadataInstance);
	CHAOSVDRUNTIME_API const TSharedPtr<FChaosVDParticleMetadata>& GetMetadataInstance() const;

	virtual bool HasValidData() const override
	{
		return bHasValidData;
	}

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

private:
	/** Debug Name instance */
	TSharedPtr<FChaosVDParticleMetadata> ParticleMetadataInstance;
};

CVD_IMPLEMENT_SERIALIZER(FChaosVDParticleDataWrapper)
