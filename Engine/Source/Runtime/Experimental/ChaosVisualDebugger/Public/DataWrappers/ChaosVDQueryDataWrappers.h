// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDParticleDataWrapper.h"
#include "Math/BoxSphereBounds.h"

#include "ChaosVDQueryDataWrappers.generated.h"

UENUM()
enum class EChaosVDSceneQueryType
{
	Invalid,
	Sweep,
	Overlap,
	RayCast
};

UENUM()
enum class EChaosVDSceneQueryMode
{
	Single,
	Multi,
	Test,
	Invalid
};

USTRUCT()
struct FChaosVDCollisionResponseParams : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		const int32 ArrayNum = UE_ARRAY_COUNT(Other.CollisionResponse.EnumArray);
		for (int32 ChannelIndex = 0; ChannelIndex < ArrayNum; ++ChannelIndex)
		{
			FlagsPerChannel.Add(Other.CollisionResponse.EnumArray[ChannelIndex]);
		}

		bHasValidData = true;
	}

	TArray<uint8, TInlineAllocator<32>> FlagsPerChannel;
};

inline FArchive& operator<<(FArchive& Ar, FChaosVDCollisionResponseParams& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

template<>
struct TStructOpsTypeTraits<FChaosVDCollisionResponseParams> : public TStructOpsTypeTraitsBase2<FChaosVDCollisionResponseParams>
{
	enum
	{
		WithSerializer = true,
	};
};

USTRUCT()
struct FChaosVDCollisionObjectQueryParams : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FChaosVDCollisionObjectQueryParams() = default;
	FChaosVDCollisionObjectQueryParams(const FChaosVDCollisionObjectQueryParams&) = default;
	FChaosVDCollisionObjectQueryParams(FChaosVDCollisionObjectQueryParams&&) = default;
	FChaosVDCollisionObjectQueryParams& operator=(const FChaosVDCollisionObjectQueryParams&) = default;
	FChaosVDCollisionObjectQueryParams& operator=(FChaosVDCollisionObjectQueryParams&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		PRAGMA_DISABLE_INTERNAL_WARNINGS
		ObjectTypesToQuery64 = Other.GetObjectTypesToQuery();
		PRAGMA_ENABLE_INTERNAL_WARNINGS
		IgnoreMask = Other.IgnoreMask;
		bHasValidData = true;
	}
	
	#if WITH_EDITORONLY_DATA
	UE_DEPRECATED(5.8, "ObjectTypesToQuery has been deprecated")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "ObjectTypesToQuery has been deprecated"))
	uint8 ObjectTypesToQuery_DEPRECATED = 0;
	#endif // WITH_EDITORONLY_DATA

	UE_INTERNAL uint64 ObjectTypesToQuery64 = 0;

	UPROPERTY(VisibleAnywhere, Category=QueryData)
	uint8 IgnoreMask = 0;
};

inline FArchive& operator<<(FArchive& Ar, FChaosVDCollisionObjectQueryParams& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

template<>
struct TStructOpsTypeTraits<FChaosVDCollisionObjectQueryParams> : public TStructOpsTypeTraitsBase2<FChaosVDCollisionObjectQueryParams>
{
	enum
	{
		WithSerializer = true,
	};
};

UENUM()
enum class EChaosVDCollisionQueryParamsFlags : uint16
{
	None = 0,
	/** Whether we should trace against complex collision */
	TraceComplex = 1 << 0,
	/** Whether we want to find out initial overlap or not. If true, it will return if this was initial overlap. */
	FindInitialOverlaps = 1 << 1,
	/** Whether we want to return the triangle face index for complex static mesh traces */
	ReturnFaceIndex = 1 << 2,
	/** Whether we want to include the physical material in the results. */
	ReturnPhysicalMaterial = 1 << 3,
	/** Whether to ignore blocking results. */
	IgnoreBlocks = 1 << 4,
	/** Whether to ignore touch/overlap results. */
	IgnoreTouches = 1 << 5,
	/** Whether to skip narrow phase checks (only for overlaps). */
	SkipNarrowPhase = 1 << 6,
	/** Whether to ignore traces to the cluster union and trace against its children instead. */
	TraceIntoSubComponents = 1 << 7,
	/** Extra filtering done on the query. See declaration for filtering logic */
	ReplaceHitWithSubComponents = 1 << 8
};

ENUM_CLASS_FLAGS(EChaosVDCollisionQueryParamsFlags)

USTRUCT()
struct FChaosVDCollisionQueryParams : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	template <typename TOther>
	void CopyFrom(const TOther& Other)
	{
		TraceTag = Other.TraceTag;
		OwnerTag = Other.OwnerTag;
		bTraceComplex = Other.bTraceComplex;
		bFindInitialOverlaps = Other.bFindInitialOverlaps;
		bReturnFaceIndex = Other.bReturnFaceIndex;
		bReturnPhysicalMaterial = Other.bReturnPhysicalMaterial;
		bIgnoreBlocks = Other.bIgnoreBlocks;
		bIgnoreTouches = Other.bIgnoreTouches;
		bSkipNarrowPhase = Other.bSkipNarrowPhase;
		bTraceIntoSubComponents = Other.bTraceIntoSubComponents;
		bReplaceHitWithSubComponents = Other.bReplaceHitWithSubComponents;
		IgnoreMask = Other.IgnoreMask;

		bHasValidData = true;
	}
	
	/** Tag used to provide extra information or filtering for debugging of the trace (e.g. Collision Analyzer) */
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	FName TraceTag;

	/** Tag used to indicate an owner for this trace */
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	FName OwnerTag;

	/** Whether we should trace against complex collision */
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	uint16 bTraceComplex : 1 = false;

	/** Whether we want to find out initial overlap or not. If true, it will return if this was initial overlap. */
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	uint16 bFindInitialOverlaps : 1= false;

	/** Whether we want to return the triangle face index for complex static mesh traces */
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	uint16 bReturnFaceIndex : 1 = false;

	/** Whether we want to include the physical material in the results. */
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	uint16 bReturnPhysicalMaterial : 1 = false;

	/** Whether to ignore blocking results. */
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	uint16 bIgnoreBlocks : 1 = false;

	/** Whether to ignore touch/overlap results. */
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	uint16 bIgnoreTouches : 1 = false;

	/** Whether to skip narrow phase checks (only for overlaps). */
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	uint16 bSkipNarrowPhase : 1 = false;

	/** Whether to ignore traces to the cluster union and trace against its children instead. */
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	uint16 bTraceIntoSubComponents : 1 = false;

	/** If bTraceIntoSubComponents is true, whether to replace the hit of the cluster union with its children instead. */
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	uint16 bReplaceHitWithSubComponents : 1 = false;

	/** Extra filtering done on the query. See declaration for filtering logic */
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	uint8 IgnoreMask = 0;

	TArray<uint32> IgnoredComponentsIDs;

	TArray<uint32> IgnoredActorsIDs;

	UPROPERTY(VisibleAnywhere, Category=QueryData)
	TArray<FName> IgnoredActorsNames;
	
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	TArray<FName> IgnoredComponentsNames;
};

inline FArchive& operator<<(FArchive& Ar, FChaosVDCollisionQueryParams& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

template<>
struct TStructOpsTypeTraits<FChaosVDCollisionQueryParams> : public TStructOpsTypeTraitsBase2<FChaosVDCollisionQueryParams>
{
	enum
	{
		WithSerializer = true,
	};
};

UENUM()
enum class EChaosVDQueryFastDataParallelFlags : uint8
{
	None = 0,
	Parallel0 = 1 << 0,
	Parallel1 = 1 << 1,
	Parallel2 = 1 << 2
};

USTRUCT()
struct FChaosVDQueryFastData : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	UPROPERTY(VisibleAnywhere, Category=QueryFastData)
	FVector Dir = FVector::ZeroVector;
	
	UPROPERTY(VisibleAnywhere, Category=QueryFastData)
	FVector InvDir = FVector::ZeroVector;
	
	UPROPERTY(VisibleAnywhere, Category=QueryFastData)
	double CurrentLength = 0.0;

	UPROPERTY(VisibleAnywhere, Category=QueryFastData)
	double InvCurrentLength = 0.0;

	UPROPERTY(VisibleAnywhere, Category=QueryFastData)
	uint8 bParallel0 : 1 = false;

	UPROPERTY(VisibleAnywhere, Category=QueryFastData)
	uint8 bParallel1 : 1 = false;

	UPROPERTY(VisibleAnywhere, Category=QueryFastData)
	uint8 bParallel2 : 1 = false;
};

inline FArchive& operator<<(FArchive& Ar, FChaosVDQueryFastData& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

template<>
struct TStructOpsTypeTraits<FChaosVDQueryFastData> : public TStructOpsTypeTraitsBase2<FChaosVDQueryFastData>
{
	enum
	{
		WithSerializer = true,
	};
};

/** Possible results from a scene query */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EChaosVDSQHitFlags : uint16
{
	None =			0,
	Position =		1 << 0,
	Normal =		1 << 1,
	Distance =		1 << 2,
	UV =			1 << 3,
	MTD =			1 << 9,
	FaceIndex =		1 << 10
};

ENUM_CLASS_FLAGS(EChaosVDSQHitFlags)

USTRUCT()
struct FChaosVDQueryHitData : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	UPROPERTY(VisibleAnywhere, Category=QueryData)
	float Distance = 0.0f;

	UPROPERTY(VisibleAnywhere, Category=QueryData)
	int32 FaceIdx = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category=QueryData, meta = (Bitmask, BitmaskEnum = "/Script/ChaosVDRuntime.EChaosVDSQHitFlags"))
	uint16 Flags = 0;

	UPROPERTY(VisibleAnywhere, Category=QueryData)
	FVector WorldPosition = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, Category=QueryData)
	FVector WorldNormal = FVector::ZeroVector;

	/** This is not used for Line Traces */
	UPROPERTY(VisibleAnywhere, Category=QueryData)
	FVector FaceNormal = FVector::ZeroVector;
};

inline FArchive& operator<<(FArchive& Ar, FChaosVDQueryHitData& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

template<>
struct TStructOpsTypeTraits<FChaosVDQueryHitData> : public TStructOpsTypeTraitsBase2<FChaosVDQueryHitData>
{
	enum
	{
		WithSerializer = true,
	};
};

UENUM()
enum class EChaosVDCollisionQueryHitType
{
	None = 0,
	Touch = 1,
	Block = 2
};

UENUM()
enum class EChaosVDSQVisitRejectReason
{
	None = 0,
	NoHit = 1,
	PreFilter = 2,
	PostFilter = 3,
	ColocatedHitHasWorseNormal = 4,
	FailedFastBoundTest = 5
};

UENUM()
enum class EChaosVDSceneQueryVisitorType
{
	Invalid,
	BroadPhase,
	NarrowPhase
};

USTRUCT()
struct FChaosVDQueryVisitStep : public FChaosVDWrapperDataBase
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	int32 OwningQueryID = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="SQ Visit Data")
	EChaosVDSceneQueryVisitorType Type = EChaosVDSceneQueryVisitorType::Invalid;

	UPROPERTY(VisibleAnywhere, Category="SQ Visit Data")
	uint32 ShapeIndex = 0;
	
	UPROPERTY(VisibleAnywhere, Category="SQ Visit Data")
	int32 ParticleIndex = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="SQ Visit Data")
	FTransform ParticleTransform;

	UPROPERTY(VisibleAnywhere, Category="SQ Visit Data")
	FChaosVDQueryFastData QueryFastData;
	
	UPROPERTY(VisibleAnywhere, Category="SQ Visit Data")
	EChaosVDCollisionQueryHitType HitType = EChaosVDCollisionQueryHitType::None;

	UPROPERTY(VisibleAnywhere, Category="SQ Visit Data")
	FChaosVDQueryHitData HitData;

	UPROPERTY(VisibleAnywhere, Category="SQ Visit Data")
	EChaosVDSQVisitRejectReason RejectReason = EChaosVDSQVisitRejectReason::None;

	/** Cached solver id property populated and used during playback of an already loaded file .
	 * This should not be serialized
	 */
	int32 SolverID_Editor = INDEX_NONE;
};

inline FArchive& operator<<(FArchive& Ar, FChaosVDQueryVisitStep& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

template<>
struct TStructOpsTypeTraits<FChaosVDQueryVisitStep> : public TStructOpsTypeTraitsBase2<FChaosVDQueryVisitStep>
{
	enum
	{
		WithSerializer = true,
	};
};

USTRUCT()
struct FChaosVDQueryDataWrapper
{
	GENERATED_BODY()

	CHAOSVDRUNTIME_API static FStringView WrapperTypeName;

	CHAOSVDRUNTIME_API bool Serialize(FArchive& Ar);

	UPROPERTY(VisibleAnywhere, Category="CVD QueryData")
	int32 ID = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="CVD QueryData")
	int32 ParentQueryID = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="CVD QueryData")
	int32 WorldSolverID = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category=RecordedQueryData)
	bool bIsRetryQuery = false;

	uint32 InputGeometryKey = 0;

	UPROPERTY(VisibleAnywhere, Category=RecordedQueryData)
	FQuat GeometryOrientation = FQuat::Identity;

	UPROPERTY(VisibleAnywhere, Category=RecordedQueryData)
	EChaosVDSceneQueryType Type = EChaosVDSceneQueryType::Invalid;

	UPROPERTY(VisibleAnywhere, Category=RecordedQueryData)
	EChaosVDSceneQueryMode Mode = EChaosVDSceneQueryMode::Invalid;

	UPROPERTY(VisibleAnywhere, Category=RecordedQueryData)
	FVector StartLocation = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, Category=RecordedQueryData)
	FVector EndLocation = FVector::ZeroVector;

	UPROPERTY(VisibleAnywhere, Category=RecordedQueryData)
	int32 CollisionChannel = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category=RecordedQueryData)
	FChaosVDCollisionQueryParams CollisionQueryParams;

	UPROPERTY(VisibleAnywhere, Category=RecordedQueryData)
	FChaosVDCollisionResponseParams CollisionResponseParams;

	UPROPERTY(VisibleAnywhere, Category=RecordedQueryData)
	FChaosVDCollisionObjectQueryParams CollisionObjectQueryParams;

	TArray<FChaosVDQueryVisitStep> SQVisitData;

	UPROPERTY(VisibleAnywhere, Category=QueryData)
	TArray<FChaosVDQueryVisitStep> Hits;

	TArray<int32> SubQueriesIDs;

	int32 CurrentVisitIndex = 0;
};

inline FArchive& operator<<(FArchive& Ar, FChaosVDQueryDataWrapper& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

template<>
struct TStructOpsTypeTraits<FChaosVDQueryDataWrapper> : public TStructOpsTypeTraitsBase2<FChaosVDQueryDataWrapper>
{
	enum
	{
		WithSerializer = true,
	};
};

USTRUCT()
struct FChaosVDSceneQueriesDataContainer
{
	GENERATED_BODY()

	TMap<int32, TMap<int32, TSharedPtr<FChaosVDQueryDataWrapper>>> RecordedSceneQueriesBySolverID;

	TMap<int32, TSharedPtr<FChaosVDQueryDataWrapper>> RecordedSceneQueriesByQueryID;
};
