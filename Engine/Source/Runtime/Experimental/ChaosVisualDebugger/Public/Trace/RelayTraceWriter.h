// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_CHAOS_VISUAL_DEBUGGER

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include <atomic>
#include "Containers/Array.h"
#include "Containers/SpscQueue.h"
#include "Delegates/Delegate.h"
#include "Serialization/BitWriter.h"
#include "Templates/UniquePtr.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "RelayTraceDataWriter.h"

namespace Chaos::VD
{
class UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FRelayTraceDataWriter directly instead.") FRelayTraceWriter : public UE::TraceBasedDebuggers::FRelayTraceDataWriter
{
public:
	FRelayTraceWriter();
	~FRelayTraceWriter();
};

}
#endif // WITH_CHAOS_VISUAL_DEBUGGER
