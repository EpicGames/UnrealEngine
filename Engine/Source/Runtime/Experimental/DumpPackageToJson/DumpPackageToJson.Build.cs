// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DumpPackageToJson : ModuleRules
{
	public DumpPackageToJson(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(new string[] {
			"StorageServerClient",
		});

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"JsonObjectGraph",
				"StorageServerClient",
			}
		);

		PCHUsage = PCHUsageMode.NoPCHs;
		bEnableExceptions = true;
	}
}
