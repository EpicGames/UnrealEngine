// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetachedStorageServerIoDispatcherBackend.h"

#if !UE_BUILD_SHIPPING

#include "DumpPackageToJsonModule.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ZenPackageHeader.h"

namespace UE::DumpPackageToJson
{

void FDetachedStorageServerIoDispatcherBackend::SetInner(TSharedPtr<IIoDispatcherBackend> InInner)
{
	if (!InInner.IsValid() && Inner.IsValid() && SavedContext.IsValid())
	{
		Inner->Shutdown();
	}
	Inner = InInner;
	if (Inner.IsValid() && SavedContext.IsValid())
	{
		Inner->Initialize(SavedContext.ToSharedRef());
	}
}

void FDetachedStorageServerIoDispatcherBackend::Initialize(TSharedRef<const FIoDispatcherBackendContext> Context)
{
	SavedContext = Context;
	Inner->Initialize(Context);
}

void FDetachedStorageServerIoDispatcherBackend::Shutdown()
{
	Inner->Shutdown();
	SavedContext.Reset();
}

void FDetachedStorageServerIoDispatcherBackend::ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved)
{
	for (FIoRequestImpl& Request : Requests)
	{
		Request.ChunkId = Remapping->Map(Request.ChunkId);
	}

	FIoRequestList OutUnresolvedMapped = {};
	Inner->ResolveIoRequests(MoveTemp(Requests), OutUnresolvedMapped);

	OutUnresolved.AddTail(MoveTemp(OutUnresolvedMapped));
}

bool FDetachedStorageServerIoDispatcherBackend::DoesChunkExist(const FIoChunkId& ChunkId) const
{
	return Inner->DoesChunkExist(Remapping->Map(ChunkId));
}

TIoStatusOr<uint64> FDetachedStorageServerIoDispatcherBackend::GetSizeForChunk(const FIoChunkId& ChunkId) const
{
	return Inner->GetSizeForChunk(Remapping->Map(ChunkId));
}

FIoRequestImpl* FDetachedStorageServerIoDispatcherBackend::GetCompletedIoRequests()
{
	FIoRequestImpl* CompletedMapped = Inner->GetCompletedIoRequests();

	for (FIoRequestList::FIterator It(CompletedMapped); It; ++It)
	{
		FIoRequestImpl& Request = (*It);

		const bool bIsExportBundleData = Request.ChunkId.GetChunkType() == EIoChunkType::ExportBundleData;
		const bool bIsChunkIndex0 = NETWORK_ORDER16(*reinterpret_cast<const uint16*>(&Request.ChunkId.GetData()[8])) == 0;

		Request.ChunkId = Remapping->Remap(Request.ChunkId);

		if (bIsExportBundleData && bIsChunkIndex0 && Request.HasBuffer())
		{
			PatchPackageNameInZenPackageHeader(Request.GetBuffer());
		}
	}

	return CompletedMapped;
}

void FDetachedStorageServerIoDispatcherBackend::PatchPackageNameInZenPackageHeader(FIoBuffer& Buffer)
{
	// There is a check in AsyncLoading2.cpp comparing requested package name with one in package header data: 
	// check(Package->Desc.PackageIdToLoad == FPackageId::FromName(Package->HeaderData.PackageName));
	//
	// To avoid failing the check we either need to modify AsyncLoading2.cpp to support our use case,
	// or patch package header in place. This code does in-place patching of zen package header name map.
	// We can only patch with a new map of same size, hence requirement that temp patch name matches the length of original package name.

	uint8* PackageHeaderDataPtr = Buffer.GetData();
	const FZenPackageSummary* PackageSummary = reinterpret_cast<const FZenPackageSummary*>(PackageHeaderDataPtr);

	// reading package header, see FZenPackageHeader
	TArrayView PackageHeaderDataView(PackageHeaderDataPtr + sizeof(FZenPackageSummary), PackageSummary->HeaderSize - sizeof(FZenPackageSummary));
	FMemoryReaderView PackageHeaderDataReader(PackageHeaderDataView);

	TOptional<FZenPackageVersioningInfo> VersioningInfo;
	if (PackageSummary->bHasVersioningInfo)
	{
		VersioningInfo.Emplace();
		PackageHeaderDataReader << VersioningInfo.GetValue();
	}

	const FZenPackageVersioningInfo* LocalVersioningInfo = VersioningInfo.GetPtrOrNull();
	if (LocalVersioningInfo == nullptr || LocalVersioningInfo->PackageVersion >= EUnrealEngineObjectUE5Version::VERSE_CELLS)
	{
		FZenPackageCellOffsets CellOffsets;
		PackageHeaderDataReader << CellOffsets.CellImportMapOffset;
		PackageHeaderDataReader << CellOffsets.CellExportMapOffset;
	}

	const int64 NameMapOffset = PackageHeaderDataReader.Tell();
	TArray<FDisplayNameEntryId> NameMap = LoadNameBatch(PackageHeaderDataReader);
	const int64 NameMapSize = PackageHeaderDataReader.Tell() - NameMapOffset;

	const FString OriginalName = PackageSummary->Name.ResolveName(NameMap).ToString();
	const FString TempName = Remapping->Remap(OriginalName);
	const FName TempNameFName = *TempName;
	NameMap[PackageSummary->Name.GetIndex()] = FDisplayNameEntryId(TempNameFName);

	TArray<uint8> NewNameMapBytes;
	FMemoryWriter NewNameMapWriter(NewNameMapBytes);

	SaveNameBatch(NameMap, NewNameMapWriter);

	if (NewNameMapBytes.Num() != NameMapSize)
	{
		UE_LOGF(LogDumpPackageToJson, Error, "Unable to patch zen header in place, old name table size %lli new name table size %d", NameMapSize, NewNameMapBytes.Num());
		return;
	}

	// replace name map in-place
	FMemory::Memcpy(PackageHeaderDataView.GetData() + NameMapOffset, NewNameMapBytes.GetData(), NewNameMapBytes.Num());
}

}

#endif