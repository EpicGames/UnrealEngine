// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

#if !UE_BUILD_SHIPPING

#include "CoreTypes.h"
#include "IO/IoDispatcherBackend.h"
#include "PackageNameRemapping.h"

namespace UE::DumpPackageToJson
{

// io dispatcher backend that remaps incoming chunk id's to inner chunk id's and back 
class FDetachedStorageServerIoDispatcherBackend final : public IIoDispatcherBackend
{
public:
	FDetachedStorageServerIoDispatcherBackend(TSharedRef<FPackageNameRemapping> InRemapping)
		: Remapping(InRemapping)
	{
	}
	~FDetachedStorageServerIoDispatcherBackend() = default;

	void SetInner(TSharedPtr<IIoDispatcherBackend> InInner);

	virtual void Initialize(TSharedRef<const FIoDispatcherBackendContext> Context) override;
	virtual void Shutdown() override;
	virtual void ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved) override;
	virtual void CancelIoRequest(FIoRequestImpl* Request) override {}
	virtual void UpdatePriorityForIoRequest(FIoRequestImpl* Request) override {};
	virtual bool DoesChunkExist(const FIoChunkId& ChunkId) const override;
	virtual TIoStatusOr<uint64> GetSizeForChunk(const FIoChunkId& ChunkId) const override;
	virtual FIoRequestImpl* GetCompletedIoRequests() override;

	virtual TIoStatusOr<FIoMappedRegion> OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options) override
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}
	virtual const TCHAR* GetName() const override
	{
		return TEXT("DetachedStorageServerIoDispatcherBackend");
	}

private:
	TSharedPtr<IIoDispatcherBackend> Inner;
	TSharedRef<FPackageNameRemapping> Remapping;
	TSharedPtr<const FIoDispatcherBackendContext> SavedContext;

	void PatchPackageNameInZenPackageHeader(FIoBuffer& Buffer);
};

}

#endif