// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetachedStorageServerPackageStoreBackend.h"

#if !UE_BUILD_SHIPPING

namespace UE::DumpPackageToJson
{

void FDetachedStorageServerPackageStoreBackend::SetInner(TSharedPtr<IPackageStoreBackend> InInner)
{
	Inner = InInner;
}

EPackageStoreEntryStatus FDetachedStorageServerPackageStoreBackend::GetPackageStoreEntry(FPackageId PackageId, FName PackageName, FPackageStoreEntry& OutPackageStoreEntry)
{
	if (!Inner.IsValid())
	{
		return EPackageStoreEntryStatus::Missing;
	}

	// TODO we could also remap imported package id's
	return Inner->GetPackageStoreEntry(Remapping->Map(PackageId), PackageName, OutPackageStoreEntry);
}

}

#endif