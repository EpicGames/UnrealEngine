// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

#if !UE_BUILD_SHIPPING

#include "CoreTypes.h"
#include "PackageNameRemapping.h"
#include "Serialization/PackageStore.h"

namespace UE::DumpPackageToJson
{

// package store backend that remaps incoming chunk id's to inner chunk id's and back
class FDetachedStorageServerPackageStoreBackend	: public IPackageStoreBackend
{
public:
	FDetachedStorageServerPackageStoreBackend(TSharedRef<FPackageNameRemapping> InRemapping)
		: Remapping(InRemapping)
	{
	}
	virtual ~FDetachedStorageServerPackageStoreBackend() = default;

	void SetInner(TSharedPtr<IPackageStoreBackend> InInner);

	virtual void OnMounted(TSharedRef<const FPackageStoreBackendContext> Context) override {}
	virtual void BeginRead() override {}
	virtual void EndRead() override {}

	virtual EPackageStoreEntryStatus GetPackageStoreEntry(FPackageId PackageId, FName PackageName, FPackageStoreEntry& OutPackageStoreEntry) override;
	virtual bool GetPackageRedirectInfo(FPackageId PackageId, FName& OutSourcePackageName, FPackageId& OutRedirectedToPackageId) override
	{
		return false;
	}

private:
	TSharedPtr<IPackageStoreBackend> Inner;
	TSharedRef<FPackageNameRemapping> Remapping;
};

}

#endif