// Copyright Epic Games, Inc. All Rights Reserved.

#include "DumpPackageToJsonModule.h"
#include "Modules/ModuleManager.h"

#if !UE_BUILD_SHIPPING

#include "DetachedStorageServerIoDispatcherBackend.h"
#include "DetachedStorageServerPackageStoreBackend.h"
#include "PackageNameRemapping.h"
#include "StorageServerClientModule.h"
#include "JsonObjectGraph/Stringify.h"
#include "Logging/StructuredLog.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/PackageStore.h"
#include "UObject/LinkerInstancingContext.h"

DEFINE_LOG_CATEGORY(LogDumpPackageToJson);

namespace UE::DumpPackageToJson
{
static FAutoConsoleCommand DumpPackageToJsonConnectToZenCommand(
	TEXT("DumpPackageToJson.ConnectToZen"),
	TEXT("Connects to a zen server to be able to dump cooked packages, pass [zen_project] [zen_platform] ([ip]) ([port]).")
	TEXT("This connection is only used for DumpPackage commands and doesn't influence rest of the engine."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() < 2 || Args.Num() > 4)
		{
			UE_LOGF(LogDumpPackageToJson, Warning, "Invalid amount of arguments passed");
			return;
		}

		auto& Module = FModuleManager::Get().GetModuleChecked<UE::DumpPackageToJson::FDumpPackageToJsonModule>("DumpPackageToJson");

		const FString Project = Args[0];
		const FString Platform = Args[1];
		const FString Host = Args.Num() >= 3 ? Args[2] : TEXT("127.0.0.1");

		uint16 Port = 8558;
		if (Args.Num() >= 4 && !LexTryParseString(Port, *Args[3]))
		{
			UE_LOGF(LogDumpPackageToJson, Warning, "Invalid format of [port]");
			return;
		}

		Module.Connect(Host, Port, Project, Platform);
	}));

static FAutoConsoleCommand DumpPackageToJsonDisconnectCommand(
	TEXT("DumpPackageToJson.Disconnect"),
	TEXT("Disconnects from a zen server"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		auto& Module = FModuleManager::Get().GetModuleChecked<UE::DumpPackageToJson::FDumpPackageToJsonModule>("DumpPackageToJson");
		Module.Disconnect();
	}));


static FAutoConsoleCommand DumpPackageToJsonCommand(
	TEXT("DumpPackageToJson"),
	TEXT("Dump json representation of a package to log, pass [package_name] ([file_path]).")
	TEXT("Will dump remote package if DumpPackageToJson.ConnectToZen was executed before, otherwise will try to use package_name in current context.")
	TEXT("Will write json to a file if [file_path] is provided."),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		if (Args.Num() < 1 || Args.Num() > 2)
		{
			UE_LOGF(LogDumpPackageToJson, Warning, "Invalid amount of arguments passed");
			return;
		}


		auto& Module = FModuleManager::Get().GetModuleChecked<UE::DumpPackageToJson::FDumpPackageToJsonModule>("DumpPackageToJson");

		const FString PackageName = Args[0];
		const FString Json = Module.IsConnected() ? Module.RemotePackageToJson(PackageName) : Module.LocalPackageToJson(PackageName);

		if (Args.Num() >= 2)
		{
			FString FilePath = Args[1];

			FPaths::NormalizeFilename(FilePath);

			if (FFileHelper::SaveStringToFile(Json, *FilePath))
			{
				UE_LOGF(LogDumpPackageToJson, Display, "Dumped package to '%ls'", *FilePath);
			}
			else
			{
				UE_LOGF(LogDumpPackageToJson, Error, "Failed to write to '%ls'", *FilePath);
			}
		}
		else
		{
			UE_LOGF(LogDumpPackageToJson, Display, "%ls", *Json);
		}
	}));

bool FDumpPackageToJsonModule::Connect(FString Host, uint16 Port, FString Project, FString Platform)
{
	if (IsConnected())
	{
		Disconnect();
	}

	IStorageServerClientModule& StorageServerClientModule = FModuleManager::Get().LoadModuleChecked<IStorageServerClientModule>("StorageServerClient");

	IStorageServerClientModule::FDetachedPlatformFileResult StorageServer = {};

	if (!StorageServerClientModule.CreateDetachedPlatformFile(Host, Port, Project, Platform, StorageServer))
	{
		UE_LOGF(LogDumpPackageToJson, Error, "Can't connect to storage server");
		return false;
	}

	StorageServerPlatformFile = MoveTemp(StorageServer.PlatformFile);
	StorageServerPackageStoreBackend = MoveTemp(StorageServer.PackageStoreBackend);
	StorageServerIoDispatcherBackend = MoveTemp(StorageServer.IoDispatcherBackend);

	// TODO do we want remapping platform file as well?

	if (!Remapper.IsValid())
	{
		Remapper = MakeShared<FPackageNameRemapping>();
	}

	if (RemappingIoDispatcherBackend.IsValid())
	{
		RemappingIoDispatcherBackend->SetInner(StorageServerIoDispatcherBackend);
	}
	else
	{
		RemappingIoDispatcherBackend = MakeShared<FDetachedStorageServerIoDispatcherBackend>(Remapper.ToSharedRef());
		RemappingIoDispatcherBackend->SetInner(StorageServerIoDispatcherBackend);
		FIoDispatcher::Get().Mount(RemappingIoDispatcherBackend.ToSharedRef());
	}

	if (RemappingPackageStoreBackend.IsValid())
	{
		RemappingPackageStoreBackend->SetInner(StorageServerPackageStoreBackend);
	}
	else
	{
		RemappingPackageStoreBackend = MakeShared<FDetachedStorageServerPackageStoreBackend>(Remapper.ToSharedRef());
		RemappingPackageStoreBackend->SetInner(StorageServerPackageStoreBackend);
		FPackageStore::Get().Mount(RemappingPackageStoreBackend.ToSharedRef());
	}
	return true;
}

void FDumpPackageToJsonModule::Disconnect()
{
	if (!IsConnected())
	{
		return;
	}

	RemappingIoDispatcherBackend->SetInner(TSharedPtr<IIoDispatcherBackend>());
	RemappingPackageStoreBackend->SetInner(TSharedPtr<IPackageStoreBackend>());

	StorageServerPackageStoreBackend.Reset();
	StorageServerIoDispatcherBackend.Reset();
	StorageServerPlatformFile.Reset();
}

FString FDumpPackageToJsonModule::RemotePackageToJson(FString OriginalPackageName)
{
	const FString TempPackageNameTemplate = FString::Printf(TEXT("/Temp/ToJson%u"), TempPackageIndex++);
	if (OriginalPackageName.Len() < TempPackageNameTemplate.Len())
	{
		// see FDetachedStorageServerIoDispatcherBackend::PatchPackageNameInZenPackageHeader for more info
		UE_LOGF(LogDumpPackageToJson, Error, "Package name '%ls' is too short to patch, shortest supported name is '%ls'", *OriginalPackageName, *TempPackageNameTemplate);
		return FString("");
	}

	const FString TempPackageName = FString::Printf(TEXT("%s%0*d"), *TempPackageNameTemplate, OriginalPackageName.Len() - TempPackageNameTemplate.Len(), 0);
	ensure(TempPackageName.Len() == OriginalPackageName.Len());

	Remapper->AddRemap(TempPackageName, OriginalPackageName);

	const FPackagePath TempPackagePath = FPackagePath::FromPackageNameChecked(TempPackageName);

	const FPackageId TempPackageId = FPackageId::FromName(TempPackagePath.GetPackageFName());

	FPackageStoreEntry PackageStoreEntry = {};
	EPackageStoreEntryStatus PackageStoreEntryStatus;

	{
		FPackageStoreReadScope _(FPackageStore::Get());
		PackageStoreEntryStatus = FPackageStore::Get().GetPackageStoreEntry(TempPackageId, TempPackagePath.GetPackageFName(), PackageStoreEntry);
	}

	if (PackageStoreEntryStatus != EPackageStoreEntryStatus::Ok)
	{
		UE_LOGF(LogDumpPackageToJson, Error, "Failed to resolve temp package id %ls for package name '%ls'", *LexToString(TempPackageId), *TempPackageName);
		Remapper->ClearRemaps();
		return {};
	}

	UE_LOGF(LogDumpPackageToJson, Display, "Package %ls imports %i packages", *OriginalPackageName, PackageStoreEntry.ImportedPackageIds.Num());
	for (FPackageId ImportedPackageId : PackageStoreEntry.ImportedPackageIds)
	{
		UE_LOGF(LogDumpPackageToJson, Display, "Imports package id %ls", *LexToString(ImportedPackageId));
	}

	UPackage* Package = LoadPackage(nullptr, *TempPackageName, LOAD_NoVerify | LOAD_SkipLoadImportedPackages | LOAD_DisableEngineVersionChecks, nullptr, nullptr);
	Remapper->ClearRemaps();

	if (!Package)
	{
		UE_LOGF(LogDumpPackageToJson, Error, "Failed to load temp package id %ls for package name '%ls'", *LexToString(TempPackageId), *TempPackageName);
		return {};
	}

	return UPackageToJson(Package);
}

FString FDumpPackageToJsonModule::LocalPackageToJson(FString PackageName)
{
	UPackage* Package = LoadPackage(nullptr, *PackageName, LOAD_NoVerify | LOAD_SkipLoadImportedPackages);

	return Package ? UPackageToJson(Package) : FString();
}

FString FDumpPackageToJsonModule::UPackageToJson(UPackage* Package)
{
	FUtf8String Json = {};

	try
	{
		Json = UE::JsonObjectGraph::Stringify(
		 {Package}
		);
	}
	catch (...)
	{
		UE_LOGF(LogDumpPackageToJson, Error, "Failed to stingify package '%ls' due to unknown error", *Package->GetName());
	}

	return StringCast<TCHAR>(*Json).Get();
}

}

#endif

IMPLEMENT_MODULE(UE::DumpPackageToJson::FDumpPackageToJsonModule, DumpPackageToJson);
