// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDumpPackageToJsonModule.h"

#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

struct IIoDispatcherBackend;
class IPackageStoreBackend;
class IStorageServerPlatformFile;

#if !UE_BUILD_SHIPPING

DECLARE_LOG_CATEGORY_EXTERN(LogDumpPackageToJson, Log, All);

#endif

namespace UE::DumpPackageToJson
{

#if !UE_BUILD_SHIPPING

class FPackageNameRemapping;
class FDetachedStorageServerIoDispatcherBackend;
class FDetachedStorageServerPackageStoreBackend;

class FDumpPackageToJsonModule : public IDumpPackageToJsonModule
{
public:
	virtual bool Connect(FString Host, uint16 Port, FString Project, FString Platform) override;
	virtual void Disconnect() override;
	virtual bool IsConnected() override
	{
		return StorageServerPlatformFile.IsValid();
	}
	virtual FString RemotePackageToJson(FString PackageName) override;
	virtual FString LocalPackageToJson(FString PackageName) override;

private:
	TUniquePtr<IStorageServerPlatformFile> StorageServerPlatformFile;
	TSharedPtr<IPackageStoreBackend> StorageServerPackageStoreBackend;
	TSharedPtr<IIoDispatcherBackend> StorageServerIoDispatcherBackend;

	TSharedPtr<FPackageNameRemapping> Remapper;
	TSharedPtr<FDetachedStorageServerIoDispatcherBackend> RemappingIoDispatcherBackend;
	TSharedPtr<FDetachedStorageServerPackageStoreBackend> RemappingPackageStoreBackend;

	uint32 TempPackageIndex = 0;

	static FString UPackageToJson(UPackage* Package); 
};

#else

class FDumpPackageToJsonModule : public IDumpPackageToJsonModule
{
public:
	virtual bool Connect(FString /*Host*/, uint16 /*Port*/, FString /*Project*/, FString /*Platform*/) override {return false;}
	virtual void Disconnect() override {}
	virtual bool IsConnected() override {return false;}
	virtual FString RemotePackageToJson(FString /*PackageName*/) override {return TEXT("");}
	virtual FString LocalPackageToJson(FString /*PackageName*/) override {return TEXT("");}
};

#endif

}
