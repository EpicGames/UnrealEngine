// Copyright Epic Games, Inc. All Rights Reserved.

#include "PackageNameRemapping.h"

#if !UE_BUILD_SHIPPING

namespace UE::DumpPackageToJson
{

void FPackageNameRemapping::AddRemap(FString TempName, FString OriginalName)
{
	const FPackageId TempPackageId = PackageIdFromString(TempName);
	const FPackageId OriginalPackageId = PackageIdFromString(OriginalName);

	TempNameToOriginal.Add(TempName, OriginalName);
	OriginalNameToTemp.Add(OriginalName, TempName);

	TempPackageIdToOriginal.Add(TempPackageId, OriginalPackageId);
	OriginalPackageIdToName.Add(OriginalPackageId, TempPackageId);
}

void FPackageNameRemapping::ClearRemaps()
{
	TempNameToOriginal.Empty();
	OriginalNameToTemp.Empty();
	TempPackageIdToOriginal.Empty();
	OriginalPackageIdToName.Empty();
}

FString FPackageNameRemapping::Map(FString TempName) const
{
	const FString OriginalName = TempNameToOriginal.FindRef(TempName, TEXT("%InvalidPackage%"));
	return OriginalName;
}

FPackageId FPackageNameRemapping::Map(FPackageId TempPackageId) const
{
	const FPackageId OriginalPackageId = TempPackageIdToOriginal.FindRef(TempPackageId, FPackageId::FromValue(0));
	return OriginalPackageId;
}

FIoChunkId FPackageNameRemapping::Map(FIoChunkId TempChunkId) const
{
	const FPackageId TempPackageId = PackageIdFromChunkId(TempChunkId);
	const FPackageId OriginalPackageId = Map(TempPackageId);
	const FIoChunkId OriginalChunkId = ReplaceChunkIdsPackageId(TempChunkId, OriginalPackageId);

	// UE_LOGF(LogDumpPackageToJson, Display, "Mapping %ls -> %ls", *LexToString(TempChunkId), *LexToString(OriginalChunkId));
	return OriginalChunkId;
}

FString FPackageNameRemapping::Remap(FString OriginalName) const
{
	const FString TempName = OriginalNameToTemp.FindRef(OriginalName, TEXT("%InvalidPackage%"));
	return TempName;
}

FPackageId FPackageNameRemapping::Remap(FPackageId OriginalPackageId) const
{
	const FPackageId TempPackageId = OriginalPackageIdToName.FindRef(OriginalPackageId, FPackageId::FromValue(0));
	return TempPackageId;
}

FIoChunkId FPackageNameRemapping::Remap(FIoChunkId OriginalChunkId) const
{
	const FPackageId OriginalPackageId = PackageIdFromChunkId(OriginalChunkId);
	const FPackageId TempPackageId = Remap(OriginalPackageId);
	const FIoChunkId TempChunkId = ReplaceChunkIdsPackageId(OriginalChunkId, TempPackageId);

	//UE_LOGF(LogDumpPackageToJson, Display, "Remapping %ls -> %ls", *LexToString(OriginalChunkId), *LexToString(TempChunkId));
	return TempChunkId;
}

FPackageId FPackageNameRemapping::PackageIdFromString(FString String)
{
	// TODO add FPackageId::FromString?
	String.ToLowerInline();
	const uint64 Value = CityHash64(reinterpret_cast<const char*>(String.GetCharArray().GetData()), String.NumBytesWithoutNull());
	return FPackageId::FromValue(Value);
}

FPackageId FPackageNameRemapping::PackageIdFromChunkId(FIoChunkId ChunkId)
{
	uint64 Result = 0;
	FMemory::Memcpy(&Result, ChunkId.GetData(), sizeof(uint64));
	return FPackageId::FromValue(Result);
}

FIoChunkId FPackageNameRemapping::ReplaceChunkIdsPackageId(FIoChunkId ChunkId, FPackageId NewPackageId)
{
	const uint64 Value = NewPackageId.Value();
	// TODO add better way
	FMemory::Memcpy(const_cast<uint8*>(ChunkId.GetData()), &Value, sizeof(uint64));
	return ChunkId;
}

}

#endif