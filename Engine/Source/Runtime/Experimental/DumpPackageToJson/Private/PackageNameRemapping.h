// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

#if !UE_BUILD_SHIPPING

#include "Containers/UnrealString.h"
#include "IO/IoChunkId.h"
#include "IO/PackageId.h"

namespace UE::DumpPackageToJson
{

// Helper class to map between temporary package name/id <-> original package name/id
class FPackageNameRemapping
{
public:
	void AddRemap(FString TempName, FString OriginalName);
	void ClearRemaps();

	// Temp -> Original
	FString Map(FString TempName) const;
	FPackageId Map(FPackageId TempPackageId) const;
	FIoChunkId Map(FIoChunkId TempChunkId) const;

	// Original -> Temp
	FString Remap(FString OriginalName) const;
	FPackageId Remap(FPackageId OriginalPackageId) const;
	FIoChunkId Remap(FIoChunkId OriginalChunkId) const;

private:
	TMap<FString, FString> TempNameToOriginal;
	TMap<FString, FString> OriginalNameToTemp;
	TMap<FPackageId, FPackageId> TempPackageIdToOriginal;
	TMap<FPackageId, FPackageId> OriginalPackageIdToName;

	static FPackageId PackageIdFromString(FString String);
	static FPackageId PackageIdFromChunkId(FIoChunkId ChunkId);
	static FIoChunkId ReplaceChunkIdsPackageId(FIoChunkId ChunkId, FPackageId NewPackageId);
};

}

#endif