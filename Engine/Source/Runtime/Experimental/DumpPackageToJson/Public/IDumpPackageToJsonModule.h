// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Containers/UnrealString.h"

namespace UE::DumpPackageToJson
{

class IDumpPackageToJsonModule : public IModuleInterface
{
public:
	virtual bool Connect(FString Host, uint16 Port, FString Project, FString Platform) = 0;
	virtual void Disconnect() = 0;
	virtual bool IsConnected() = 0;
	virtual FString RemotePackageToJson(FString PackageName) = 0;
	virtual FString LocalPackageToJson(FString PackageName) = 0;
};

}
