// Copyright Epic Games, Inc. All Rights Reserved.


#include "Field/FieldSystemActor.h"

#include "Field/FieldSystemComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FieldSystemActor)

DEFINE_LOG_CATEGORY_STATIC(AFA_Log, NoLogging, All);

AFieldSystemActor::AFieldSystemActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	UE_LOGF(AFA_Log, Log, "AFieldSystemActor::AFieldSystemActor()");

	FieldSystemComponent = CreateDefaultSubobject<UFieldSystemComponent>(TEXT("FieldSystemComponent"));
	RootComponent = FieldSystemComponent;
}

void AFieldSystemActor::OnConstruction(const FTransform& Transform)
{
	if (UFieldSystemComponent* Component = FieldSystemComponent)
	{
		if (UFieldSystem* Asset = FieldSystemComponent->FieldSystem)
		{
			const TArray< FFieldSystemCommand >& ConstructionFields = Component->GetConstructionFields();
			if (Asset->Commands.Num() != ConstructionFields.Num())
			{
				if (!ConstructionFields.Num())
				{
					Asset->Modify();
					Asset->Commands.Reset();
				}
				else
				{
					Asset->Modify();
					Asset->Commands = ConstructionFields;
				}
			}
			else
			{
				bool bEqual = true;
				for (int i = 0; i < Asset->Commands.Num() && bEqual; i++)
				{
					bEqual &= Asset->Commands[i] == ConstructionFields[i];
				}
				if (!bEqual)
				{
					Asset->Modify();
					Asset->Commands = ConstructionFields;
				}
			}
		}
		Component->ConstructionCommands = Component->BufferCommands;
		Component->BufferCommands.ResetFieldCommands();
	}
}




