// Copyright Epic Games, Inc. All Rights Reserved.

#include "Field/FieldSystemAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FieldSystemAsset)


/** Serialize */
void UFieldSystem::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FDestructionObjectVersion::GUID);
	if (Ar.CustomVer(FDestructionObjectVersion::GUID) >= FDestructionObjectVersion::FieldsAdded)
	{
		int32 NumCommands = Commands.Num();
		Ar << NumCommands;

		if (Ar.IsLoading())
		{
			Commands.Init(FFieldSystemCommand(), NumCommands);
		}

		for (int i=0;i<NumCommands;i++)
		{
			Commands[i].Serialize(Ar);
		}
	}
}

