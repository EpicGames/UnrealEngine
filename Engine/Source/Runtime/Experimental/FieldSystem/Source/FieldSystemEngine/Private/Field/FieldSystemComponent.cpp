// Copyright Epic Games, Inc. All Rights Reserved.

#include "Field/FieldSystemComponent.h"

#include "Async/ParallelFor.h"
#include "ChaosSolversModule.h"
#include "Engine/World.h"
#include "Field/FieldSystemCoreAlgo.h"
#include "Field/FieldSystemSceneProxy.h"
#include "Field/FieldSystemNodes.h"
#include "PhysicsField/PhysicsFieldComponent.h"
#include "Modules/ModuleManager.h"
#include "Misc/CoreMiscDefines.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsProxy/PerSolverFieldSystem.h"
#include "PBDRigidsSolver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FieldSystemComponent)

DEFINE_LOG_CATEGORY_STATIC(FSC_Log, NoLogging, All);

UFieldSystemComponent::UFieldSystemComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, FieldSystem(nullptr)
	, bIsWorldField(false)
	, bIsChaosField(true)
	, SupportedSolvers()
	, bEnableInEditorWorlds(false)
	, bEnableInPreviewWorlds(false)
	, ChaosModule(nullptr)
	, bHasPhysicsState(false)
	, SetupConstructionFields()
	, ChaosPersistentFields()
	, WorldGPUPersistentFields()
	, WorldCPUPersistentFields()
{
	UE_LOGF(FSC_Log, Log, "FieldSystemComponent[%p]::UFieldSystemComponent()",this);

	SetGenerateOverlapEvents(false);
}

FPrimitiveSceneProxy* UFieldSystemComponent::CreateSceneProxy()
{
	UE_LOGF(FSC_Log, Log, "FieldSystemComponent[%p]::CreateSceneProxy()", this);

	return new FFieldSystemSceneProxy(this);
}

TSet<FPhysScene_Chaos*> UFieldSystemComponent::GetPhysicsScenes() const
{
	TSet<FPhysScene_Chaos*> Scenes;
	if (SupportedSolvers.Num())
	{
		for (const TSoftObjectPtr<AChaosSolverActor>& Actor : SupportedSolvers)
		{
			if (!Actor.IsValid())
				continue;
			Scenes.Add(Actor->GetPhysicsScene().Get());
		}
	}
	else
	{
		if (ensure(GetOwner()) && ensure(GetOwner()->GetWorld()))
		{
			Scenes.Add(GetOwner()->GetWorld()->GetPhysicsScene());
		}
		else
		{
			check(GWorld);
			Scenes.Add(GWorld->GetPhysicsScene());
		}
	}
	return Scenes;
}

TArray<Chaos::FPhysicsSolverBase*> UFieldSystemComponent::GetPhysicsSolvers() const 
{
	TArray<Chaos::FPhysicsSolverBase*> PhysicsSolvers;

	// Assemble a list of compatible solvers
	TArray<Chaos::FPhysicsSolverBase*> FilterSolvers;
	if (SupportedSolvers.Num() > 0)
	{
		for (const TSoftObjectPtr<AChaosSolverActor>& SolverActorPtr : SupportedSolvers)
		{
			if (AChaosSolverActor* CurrActor = SolverActorPtr.Get())
			{
				FilterSolvers.Add(CurrActor->GetSolver());
			}
		}
	}

	TArray<Chaos::FPhysicsSolverBase*> WorldSolvers;

	//Need to only grab solvers that are owned by our world. In multi-client PIE this avoids solvers for other clients
	UWorld* World = GetWorld();
	FPhysScene* PhysScene = World ? World->GetPhysicsScene() : nullptr;
	if(PhysScene)
	{
		WorldSolvers.Add(PhysScene->GetSolver());
		ChaosModule->GetSolversMutable(World, WorldSolvers);
	}

	
	const int32 NumFilterSolvers = FilterSolvers.Num();

	for (Chaos::FPhysicsSolverBase* Solver : WorldSolvers)
	{
		if (NumFilterSolvers == 0 || FilterSolvers.Contains(Solver))
		{
			PhysicsSolvers.Add(Solver);
		}
	}
	return PhysicsSolvers;
}

void UFieldSystemComponent::OnCreatePhysicsState()
{
	UActorComponent::OnCreatePhysicsState();
	
	if(UWorld* World = GetWorld())
	{
		const bool bValidWorld =
			World->IsGameWorld() ||
			(bEnableInEditorWorlds && World->IsEditorWorld()) ||
			(bEnableInPreviewWorlds && World->IsPreviewWorld());

		if(bValidWorld)
		{
			// Check we can get a suitable dispatcher
			ChaosModule = FChaosSolversModule::GetModule();
			check(ChaosModule);

			bHasPhysicsState = true;
		}
	}
}

void UFieldSystemComponent::OnDestroyPhysicsState()
{
	UActorComponent::OnDestroyPhysicsState();
	ClearFieldCommands();

	ChaosModule = nullptr;
	bHasPhysicsState = false;
}

bool UFieldSystemComponent::ShouldCreatePhysicsState() const
{
	return true;
}

bool UFieldSystemComponent::HasValidPhysicsState() const
{
	return bHasPhysicsState;
}

void UFieldSystemComponent::DispatchFieldCommand(const FFieldSystemCommand& InCommand, const bool IsTransient)
{
	using namespace Chaos;
	if (HasValidPhysicsState() && InCommand.RootNode)
	{
		const FName Name = GetOwner() ? *GetOwner()->GetName() : TEXT("");
		
		// TODO : special case for chaos physics objects because of the supported solvers. should be moved in the general case below
		if (bIsChaosField)
		{
			checkSlow(ChaosModule); // Should already be checked from OnCreatePhysicsState

			TArray<FPhysicsSolverBase*> PhysicsSolvers = GetPhysicsSolvers();

			for (FPhysicsSolverBase* Solver : PhysicsSolvers)
			{
				TArray<FFieldSystemCommand>& LocalFields = ChaosPersistentFields;
				Solver->CastHelper([&InCommand,IsTransient,&LocalFields,Name](auto& Concrete)
				{
					FFieldSystemCommand LocalCommand = InCommand;
					LocalCommand.InitFieldNodes(Concrete.GetSolverTime(), Name);

					if(!IsTransient) LocalFields.Add(LocalCommand);

					Concrete.EnqueueCommandImmediate([ConcreteSolver = &Concrete, NewCommand = LocalCommand, LocalTransient = IsTransient]()
						{
							if (!LocalTransient)
							{
								ConcreteSolver->GetPerSolverField().AddPersistentCommand(NewCommand);
							}
							else
							{
								ConcreteSolver->GetPerSolverField().AddTransientCommand(NewCommand);
							}
						});
				});
			}
		}

		if (bIsWorldField || bIsChaosField)
		{
			UWorld* World = GetWorld();
			if (World && World->PhysicsField)
			{
				FFieldSystemCommand LocalCommand = InCommand;
				LocalCommand.InitFieldNodes(World->GetTimeSeconds(), Name);

				if (!IsTransient)
				{
					if (bIsWorldField)
					{
						WorldGPUPersistentFields.Add(LocalCommand);
						World->PhysicsField->AddPersistentCommand(LocalCommand, true);
					}
					if (bIsChaosField)
					{
						WorldCPUPersistentFields.Add(LocalCommand);
						World->PhysicsField->AddPersistentCommand(LocalCommand, false);
					}
				}
				else
				{
					if (bIsWorldField)
					{
						World->PhysicsField->AddTransientCommand(LocalCommand, true);
					}
					if (bIsChaosField)
					{
						World->PhysicsField->AddTransientCommand(LocalCommand, false);
					}
				}
			}
		}
	}
}

void UFieldSystemComponent::ClearFieldCommands()
{
	using namespace Chaos;
	if (HasValidPhysicsState())
	{
		// TODO : special case for chaos physics objects because of the supported solvers. should be moved in the general case below
		if (bIsChaosField)
		{
			checkSlow(ChaosModule); // Should already be checked from OnCreatePhysicsState

			TArray<FPhysicsSolverBase*> PhysicsSolvers = GetPhysicsSolvers();

			for (FPhysicsSolverBase* Solver : PhysicsSolvers)
			{
				for (auto& FieldCommand : ChaosPersistentFields)
				{
					Solver->CastHelper([&FieldCommand](auto& Concrete)
						{
							Concrete.EnqueueCommandImmediate([ConcreteSolver = &Concrete, NewCommand = FieldCommand]()
								{
									ConcreteSolver->GetPerSolverField().RemovePersistentCommand(NewCommand);
								});
						});
				}
			}
		}
		ChaosPersistentFields.Reset();

		if (bIsWorldField || bIsChaosField)
		{
			UWorld* World = GetWorld();
			if (World && World->PhysicsField)
			{
				if (bIsWorldField)
				{
					for (auto& FieldCommand : WorldGPUPersistentFields)
					{
						World->PhysicsField->RemovePersistentCommand(FieldCommand, true);
					}
				}
				if (bIsChaosField)
				{
					for (auto& FieldCommand : WorldCPUPersistentFields)
					{
						World->PhysicsField->RemovePersistentCommand(FieldCommand, false);
					}
				}
			}
		}
		WorldGPUPersistentFields.Reset();
		WorldCPUPersistentFields.Reset();
	}
}


void UFieldSystemComponent::BuildFieldCommand(bool Enabled, EFieldPhysicsType Target, UFieldSystemMetaData* MetaData, UFieldNodeBase* Field, const bool IsTransient)
{
	if (Enabled && Field && HasValidPhysicsState())
	{
		FFieldSystemCommand Command = FFieldObjectCommands::CreateFieldCommand(Target, Field, MetaData);
		DispatchFieldCommand(Command, IsTransient);
	}
}

void UFieldSystemComponent::AddFieldCommand(bool Enabled, EFieldPhysicsType Target, UFieldSystemMetaData* MetaData, UFieldNodeBase* Field)
{
	if (Enabled && Field)
	{
		if (FieldSystem)
		{
			FFieldSystemCommand Command = FFieldObjectCommands::CreateFieldCommand(Target, Field, MetaData);
			if (Command.RootNode)
			{
				SetupConstructionFields.Add(Command);

				UWorld* World = GetWorld();
				if (World && World->PhysicsField)
				{
					const FName Name = GetOwner() ? *GetOwner()->GetName() : TEXT("");

					FFieldSystemCommand LocalCommand = Command;
					LocalCommand.InitFieldNodes(World->GetTimeSeconds(), Name);

					World->PhysicsField->AddConstructionCommand(LocalCommand);
				}
			}
		}

		BufferCommands.AddFieldCommand(GetFieldPhysicsName(Target), Field, MetaData);
	}
}

void UFieldSystemComponent::ApplyStayDynamicField(bool Enabled, FVector Position, float Radius)
{
	if (Enabled && HasValidPhysicsState())
	{
		FFieldSystemCommand FieldCommand = FFieldObjectCommands::CreateFieldCommand(EFieldPhysicsType::Field_DynamicState,
			new FRadialIntMask(Radius, Position, (int32)Chaos::EObjectStateType::Dynamic,
			(int32)Chaos::EObjectStateType::Kinematic, ESetMaskConditionType::Field_Set_IFF_NOT_Interior));

		DispatchFieldCommand(FieldCommand, true);
	}
}

void UFieldSystemComponent::ApplyLinearForce(bool Enabled, FVector Direction, float Magnitude)
{
	if (Enabled && HasValidPhysicsState())
	{
		FFieldSystemCommand FieldCommand = FFieldObjectCommands::CreateFieldCommand(EFieldPhysicsType::Field_LinearForce, 
			new FUniformVector(Magnitude, Direction));
		DispatchFieldCommand(FieldCommand, true);
	}
}

void UFieldSystemComponent::ApplyRadialForce(bool Enabled, FVector Position, float Magnitude)
{
	if (Enabled && HasValidPhysicsState())
	{
		FFieldSystemCommand FieldCommand = FFieldObjectCommands::CreateFieldCommand(EFieldPhysicsType::Field_LinearForce,
			new FRadialVector(Magnitude, Position));
		DispatchFieldCommand(FieldCommand, true);
	}
}

void UFieldSystemComponent::ApplyRadialVectorFalloffForce(bool Enabled, FVector Position, float Radius, float Magnitude)
{
	if (Enabled && HasValidPhysicsState())
	{
		FRadialFalloff * FalloffField = new FRadialFalloff(Magnitude,0.f, 1.f, 0.f, Radius, Position);
		FRadialVector* VectorField = new FRadialVector(1.f, Position);

		FFieldSystemCommand FieldCommand = FFieldObjectCommands::CreateFieldCommand(EFieldPhysicsType::Field_LinearForce,
			new FSumVector(1.0, FalloffField, VectorField, nullptr, Field_Multiply));
		DispatchFieldCommand(FieldCommand, true);
	}
}

void UFieldSystemComponent::ApplyUniformVectorFalloffForce(bool Enabled, FVector Position, FVector Direction, float Radius, float Magnitude)
{
	if (Enabled && HasValidPhysicsState())
	{
		FRadialFalloff * FalloffField = new FRadialFalloff(Magnitude, 0.f, 1.f, 0.f, Radius, Position);
		FUniformVector* VectorField = new FUniformVector(1.f, Direction);

		FFieldSystemCommand FieldCommand = FFieldObjectCommands::CreateFieldCommand(EFieldPhysicsType::Field_LinearForce,
			new FSumVector(1.0, FalloffField, VectorField, nullptr, Field_Multiply));
		DispatchFieldCommand(FieldCommand, true);
	}
}

void UFieldSystemComponent::ApplyStrainField(bool Enabled, FVector Position, float Radius, float Magnitude, int32 Iterations)
{
	if (Enabled && HasValidPhysicsState())
	{
		FFieldSystemCommand FieldCommand = FFieldObjectCommands::CreateFieldCommand(EFieldPhysicsType::Field_ExternalClusterStrain,
			new FRadialFalloff(Magnitude, 0.f, 1.f, 0.f, Radius, Position));
		DispatchFieldCommand(FieldCommand, true);
	}
}

void UFieldSystemComponent::ApplyPhysicsField(bool Enabled, EFieldPhysicsType Target, UFieldSystemMetaData* MetaData, UFieldNodeBase* Field)
{
	BuildFieldCommand(Enabled, Target, MetaData, Field, true);
}

void UFieldSystemComponent::AddPersistentField(bool Enabled, EFieldPhysicsType Target, UFieldSystemMetaData* MetaData, UFieldNodeBase* Field)
{
	BuildFieldCommand(Enabled, Target, MetaData, Field, false);
}

void UFieldSystemComponent::RemovePersistentFields()
{
	ClearFieldCommands();
}

void UFieldSystemComponent::ResetFieldSystem()
{
	if (FieldSystem)
	{
		SetupConstructionFields.Reset();
	}
	ConstructionCommands.ResetFieldCommands();
	BufferCommands.ResetFieldCommands();
}






