// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, FieldSystemEngine);