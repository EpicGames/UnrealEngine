// Copyright Epic Games, Inc. All Rights Reserved.

#include "Field/FieldSystemObjects.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FieldSystemObjects)


// UFieldSystemMetaDataProcessingResolution
FFieldSystemMetaData*
UFieldSystemMetaDataProcessingResolution::NewMetaData() const
{
	return new FFieldSystemMetaDataProcessingResolution(ResolutionType);
}

UFieldSystemMetaDataProcessingResolution* UFieldSystemMetaDataProcessingResolution::SetMetaDataaProcessingResolutionType(EFieldResolutionType ResolutionTypeIn)
{

	this->ResolutionType = ResolutionTypeIn;
	return this;
}

// UFieldSystemMetaDataFilter
FFieldSystemMetaData*
UFieldSystemMetaDataFilter::NewMetaData() const
{
	return new FFieldSystemMetaDataFilter(FilterType, ObjectType, PositionType);
}

UFieldSystemMetaDataFilter* UFieldSystemMetaDataFilter::SetMetaDataFilterType(EFieldFilterType FilterTypeIn, EFieldObjectType ObjectTypeIn, EFieldPositionType PositionTypeIn)
{
	this->FilterType = FilterTypeIn;
	this->ObjectType = ObjectTypeIn;
	this->PositionType = PositionTypeIn;
	return this;
}

// UFieldSystemMetaDataIteration
FFieldSystemMetaData*
UFieldSystemMetaDataIteration::NewMetaData() const
{
	return new FFieldSystemMetaDataIteration(Iterations);
}

UFieldSystemMetaDataIteration* UFieldSystemMetaDataIteration::SetMetaDataIteration(int IterationsIn)
{
	this->Iterations = IterationsIn;
	return this;
}


// UUniformInteger
FFieldNodeBase*
UUniformInteger::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		return new FUniformInteger(Magnitude);
	}
	return nullptr;
};

UUniformInteger* UUniformInteger::SetUniformInteger(int32 InMagnitude)
{
	this->Magnitude = InMagnitude;
	return this;
}

// URadialIntMask
FFieldNodeBase* 
URadialIntMask::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		return new FRadialIntMask(Radius, Position, InteriorValue, ExteriorValue, SetMaskCondition);
	}
	return nullptr;
}

URadialIntMask*
URadialIntMask::SetRadialIntMask(float InRadius, FVector InPosition, int32 InInteriorValue, int32 InExteriorValue, ESetMaskConditionType InSetMaskCondition)
{
	this->Radius=InRadius;
	this->Position=InPosition;
	this->InteriorValue=InInteriorValue;
	this->ExteriorValue=InExteriorValue;
	this->SetMaskCondition = InSetMaskCondition;
	return this;
}

// UUniformScalar
FFieldNodeBase*
UUniformScalar::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		return new FUniformScalar(Magnitude);
	}
	return nullptr;
};

UUniformScalar* UUniformScalar::SetUniformScalar(float InMagnitude)
{
	this->Magnitude = InMagnitude;
	return this;
}

// UWaveScalar
FFieldNodeBase*
UWaveScalar::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		return new FWaveScalar(Magnitude,Position,Wavelength,Period,Function,Falloff);
	}
	return nullptr;
};

UWaveScalar* UWaveScalar::SetWaveScalar(float InMagnitude, FVector InPosition, float InWavelength, float InPeriod, float Time, EWaveFunctionType InFunction, EFieldFalloffType InFalloff)
{
	this->Magnitude = InMagnitude;
	this->Position = InPosition;
	this->Wavelength = InWavelength;
	this->Period = InPeriod;
	this->Function = InFunction;
	this->Falloff = InFalloff;
	return this;
}

// RadialFalloff
FFieldNodeBase* 
URadialFalloff::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		return new FRadialFalloff(Magnitude, MinRange, MaxRange, Default, Radius, Position, Falloff);
	}
	return nullptr;
}

URadialFalloff* URadialFalloff::SetRadialFalloff(float InMagnitude, float InMinRange,float InMaxRange, float InDefault, float InRadius, FVector InPosition, EFieldFalloffType InFalloff)
{
	this->Magnitude = InMagnitude;
	this->MinRange = InMinRange;
	this->MaxRange = InMaxRange;
	this->Default = InDefault;
	this->Radius=InRadius;
	this->Position=InPosition;
	this->Falloff = InFalloff;
	return this;
}


// PlaneFalloff
FFieldNodeBase*
UPlaneFalloff::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		return new FPlaneFalloff(Magnitude, MinRange, MaxRange, Default, Distance, Position, Normal, Falloff);
	}
	return nullptr;
}

UPlaneFalloff* 
UPlaneFalloff::SetPlaneFalloff(float InMagnitude, float InMinRange, float InMaxRange, float InDefault, float InDistance, FVector InPosition, FVector InNormal, EFieldFalloffType InFalloff)
{
	this->Magnitude = InMagnitude;
	this->MinRange = InMinRange;
	this->MaxRange = InMaxRange;
	this->Default = InDefault;
	this->Position = InPosition;
	this->Distance = InDistance;
	this->Normal = InNormal;
	this->Falloff = InFalloff;
	return this;
}


// BoxFalloff
FFieldNodeBase*
UBoxFalloff::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		return new FBoxFalloff(Magnitude, MinRange, MaxRange, Default, Transform, Falloff);
	}
	return nullptr;
}

UBoxFalloff*
UBoxFalloff::SetBoxFalloff(float InMagnitude, float InMinRange, float InMaxRange, float InDefault, FTransform InTransform, EFieldFalloffType InFalloff)
{
	this->Magnitude = InMagnitude;
	this->MinRange = InMinRange;
	this->MaxRange = InMaxRange;
	this->Default = InDefault;
	this->Transform = InTransform;
	this->Falloff = InFalloff;
	return this;
}

// NoiseField
FFieldNodeBase*
UNoiseField::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		return new FNoiseField(MinRange, MaxRange, Transform);
	}
	return nullptr;
}

UNoiseField*
UNoiseField::SetNoiseField(float InMinRange, float InMaxRange, FTransform InTransform)
{
	this->MinRange = InMinRange;
	this->MaxRange = InMaxRange;
	this->Transform = InTransform;
	return this;
}


// UniformVector
FFieldNodeBase* 
UUniformVector::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		return new FUniformVector(Magnitude, Direction);
	}
	return nullptr;
};

UUniformVector* UUniformVector::SetUniformVector(float InMagnitude, FVector InDirection)
{
	this->Magnitude=InMagnitude;
	this->Direction=InDirection;
	return this;
}


// RadialVector
FFieldNodeBase* URadialVector::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		return new FRadialVector(Magnitude, Position);
	}
	return nullptr;
};

URadialVector* URadialVector::SetRadialVector(float InMagnitude, FVector InPosition)
{
	this->Magnitude=InMagnitude;
	this->Position=InPosition;
	return this;
}


// RotatedRadialVector
FFieldNodeBase* URotatedRadialVector::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		return new FRotatedRadialVector(Magnitude, Position, Angle, Axis);
	}
	return nullptr;
};

URotatedRadialVector* URotatedRadialVector::SetRotatedRadialVector(float InMagnitude, FVector InPosition, float InAngle, FVector InAxis)
{
	this->Magnitude = InMagnitude;
	this->Position = InPosition;
	this->Angle = InAngle;
	this->Axis = InAxis;
	return this;
}


// RadialVector
FFieldNodeBase* URandomVector::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		return new FRandomVector(Magnitude);
	}
	return nullptr;
};

URandomVector* URandomVector::SetRandomVector(float InMagnitude)
{
	this->Magnitude = InMagnitude;
	return this;
}


// UOperatorField
FFieldNodeBase* UOperatorField::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		bool bProcess = true;
		if (RightField) {
			bProcess &= ensureMsgf(RightField->Type() != FFieldNodeBase::EField_Int32,
				TEXT("Operator fields do not support Int32 input connections (Right Input Error)."));
		}
		if (LeftField) {
			bProcess &= ensureMsgf(LeftField->Type() != FFieldNodeBase::EField_Int32,
				TEXT("Operator fields do not support Int32 input connections (Left Input Error)."));
		}

		if (bProcess)
		{

			if (RightField && LeftField)
			{
				if ((RightField->Type() == FFieldNodeBase::EField_Float || RightField->Type() == FFieldNodeBase::EField_Results)
					&& (LeftField->Type() == FFieldNodeBase::EField_Float || LeftField->Type() == FFieldNodeBase::EField_Results))
				{
					return new FSumScalar(Magnitude,
						RightField ? static_cast<FFieldNode<float>*>(RightField->NewEvaluationGraph(Nodes)) : nullptr,
						LeftField ? static_cast<FFieldNode<float>*>(LeftField->NewEvaluationGraph(Nodes)) : nullptr,
						Operation);
				}
				else if (RightField->Type() == FFieldNodeBase::EField_Float && LeftField->Type() == FFieldNodeBase::EField_FVector)
				{
					return new FSumVector(Magnitude,
						RightField ? static_cast<FFieldNode<float>*>(RightField->NewEvaluationGraph(Nodes)) : nullptr,
						nullptr,
						LeftField ? static_cast<FFieldNode<FVector>*>(LeftField->NewEvaluationGraph(Nodes)) : nullptr,
						Operation);
				}
				else if (RightField->Type() == FFieldNodeBase::EField_FVector && LeftField->Type() == FFieldNodeBase::EField_Float)
				{
					return new FSumVector(Magnitude,
						LeftField ? static_cast<FFieldNode<float>*>(LeftField->NewEvaluationGraph(Nodes)) : nullptr,
						RightField ? static_cast<FFieldNode<FVector>*>(RightField->NewEvaluationGraph(Nodes)) : nullptr,
						nullptr,
						Operation);
				}
				else if ((RightField->Type() == FFieldNodeBase::EField_FVector || RightField->Type() == FFieldNodeBase::EField_Results)
					&& (LeftField->Type() == FFieldNodeBase::EField_FVector || RightField->Type() == FFieldNodeBase::EField_Results))
				{
					return new FSumVector(Magnitude,
						nullptr,
						RightField ? static_cast<FFieldNode<FVector>*>(RightField->NewEvaluationGraph(Nodes)) : nullptr,
						LeftField ? static_cast<FFieldNode<FVector>*>(LeftField->NewEvaluationGraph(Nodes)) : nullptr,
						Operation);
				}
			}
			else if (RightField)
			{
				if (RightField->Type() == FFieldNodeBase::EField_Float)
				{
					return new FSumScalar(Magnitude,
						RightField ? static_cast<FFieldNode<float>*>(RightField->NewEvaluationGraph(Nodes)) : nullptr,
						nullptr,
						Operation);
				}
				else if (RightField->Type() == FFieldNodeBase::EField_FVector)
				{
					return new FSumVector(Magnitude,
						nullptr,
						RightField ? static_cast<FFieldNode<FVector>*>(RightField->NewEvaluationGraph(Nodes)) : nullptr,
						nullptr,
						Operation);
				}
			}
			else if (LeftField)
			{
				if (LeftField->Type() == FFieldNodeBase::EField_Float)
				{
					return new FSumScalar(Magnitude,
						nullptr,
						LeftField ? static_cast<FFieldNode<float>*>(LeftField->NewEvaluationGraph(Nodes)) : nullptr,
						Operation);
				}
				else if (LeftField->Type() == FFieldNodeBase::EField_FVector)
				{
					return new FSumVector(Magnitude,
						nullptr,
						nullptr,
						LeftField ? static_cast<FFieldNode<FVector>*>(LeftField->NewEvaluationGraph(Nodes)) : nullptr,
						Operation);
				}
			}
		}
	}
	return nullptr;
}

UOperatorField* UOperatorField::SetOperatorField(float InMagnitude, const UFieldNodeBase* InLeftField, const UFieldNodeBase* InRightField, EFieldOperationType InOperation)
{
	this->Magnitude = InMagnitude;
	this->RightField = InRightField;
	this->LeftField = InLeftField;
	this->Operation = InOperation;
	return this;
}

FFieldNodeBase::EFieldType UOperatorField::Type() const
{
	if (RightField && RightField->Type() == FFieldNodeBase::EField_FVector)
		return FFieldNodeBase::EField_FVector;

	if (LeftField && LeftField->Type() == FFieldNodeBase::EField_FVector)
		return FFieldNodeBase::EField_FVector;

	if (RightField && RightField->Type() == FFieldNodeBase::EField_Float)
		return FFieldNodeBase::EField_Float;

	if (LeftField && LeftField->Type() == FFieldNodeBase::EField_Float)
		return FFieldNodeBase::EField_Float;

	return FFieldNodeBase::EField_None;
}



// UToIntegerField
FFieldNodeBase* UToIntegerField::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		if (FloatField)
		{
			return new FConversionField<float, int32>(static_cast<FFieldNode<float>*>(FloatField->NewEvaluationGraph(Nodes)));
		}
	}
	return nullptr;
}

UToIntegerField* UToIntegerField::SetToIntegerField( const UFieldNodeFloat* InFloatField)
{
	this->FloatField = InFloatField;
	return this;
}

// UToFloatField
FFieldNodeBase* UToFloatField::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		if (IntField)
		{
			return new FConversionField<int32, float>(static_cast<FFieldNode<int32>*>(IntField->NewEvaluationGraph(Nodes)));
		}
	}
	return nullptr;
}

UToFloatField* UToFloatField::SetToFloatField(const UFieldNodeInt* InIntField)
{
	this->IntField = InIntField;
	return this;
}

// CullingField
FFieldNodeBase* UCullingField::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);
		if (Culling && Field)
		{
			if (Culling->Type() == FFieldNodeBase::EField_Float && Field->Type() == FFieldNodeBase::EField_Float)
			{
				return new FCullingField<float>(
					static_cast<FFieldNode<float>*>(Culling->NewEvaluationGraph(Nodes)),
					static_cast<FFieldNode<float>*>(Field->NewEvaluationGraph(Nodes)), Operation);
			}
			else if (Culling->Type() == FFieldNodeBase::EField_Float && Field->Type() == FFieldNodeBase::EField_FVector)
			{
				return new FCullingField<FVector>(
					static_cast<FFieldNode<float>*>(Culling->NewEvaluationGraph(Nodes)),
					static_cast<FFieldNode<FVector>*>(Field->NewEvaluationGraph(Nodes)), Operation);
			}
			else if (Culling->Type() == FFieldNodeBase::EField_Float && Field->Type() == FFieldNodeBase::EField_Int32)
			{
				return new FCullingField<int32>(
					static_cast<FFieldNode<float>*>(Culling->NewEvaluationGraph(Nodes)),
					static_cast<FFieldNode<int32>*>(Field->NewEvaluationGraph(Nodes)), Operation);
			}
		}
	}
	return nullptr;
}

UCullingField* UCullingField::SetCullingField(const UFieldNodeBase* InCulling, const UFieldNodeBase* InField, EFieldCullingOperationType InOperation)
{
	this->Culling = InCulling;
	this->Field = InField;
	this->Operation = InOperation;
	return this;
}

FFieldNodeBase::EFieldType UCullingField::Type() const
{
	return Field ? Field->Type() : Super::Type();
}

// UReturnResultsField
FFieldNodeBase* UReturnResultsTerminal::NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const
{
	if (ensureMsgf(!Nodes.Contains(this), TEXT("Cycle Dependency Error : Graph nodes may not be reused in a single chain.")))
	{
		Nodes.Add(this);

		// find last ResultsNode
		int32 ResultsIndex = INDEX_NONE;
		for (int32 i = Nodes.Num()-1; i >= 0 && ResultsIndex == INDEX_NONE; i--)
			if (Nodes[i]->ResultsExpector())
				ResultsIndex = i;

		if (ensureMsgf(ResultsIndex != INDEX_NONE,
			TEXT("ReturnResults nodes can only be used upstream from a 'results expector', for example as an input "
				"to a Operator Node . See documentation for details.")))
		{
			if (Nodes[ResultsIndex]->Type() == FFieldNodeBase::EFieldType::EField_Int32)
			{
				return new FReturnResultsTerminal<int32>();
			}
			else if (Nodes[ResultsIndex]->Type() == FFieldNodeBase::EFieldType::EField_Float)
			{
				return new FReturnResultsTerminal<float>();
			}
			else if (Nodes[ResultsIndex]->Type() == FFieldNodeBase::EFieldType::EField_FVector)
			{
				return new FReturnResultsTerminal<FVector>();
			}
		}
	}
	return nullptr;
}

UReturnResultsTerminal* UReturnResultsTerminal::SetReturnResultsTerminal()
{
	return this;
}
