// Copyright Epic Games, Inc. All Rights Reserved.

#include "Field/FieldSystemSceneProxy.h"

#include "Async/ParallelFor.h"
#include "Engine/Engine.h"
#include "Field/FieldSystemComponent.h"

DEFINE_LOG_CATEGORY_STATIC(FSSP_Log, NoLogging, All);

FFieldSystemSceneProxy::FFieldSystemSceneProxy(UFieldSystemComponent* Component)
	: FPrimitiveSceneProxy(Component)
{
}

FFieldSystemSceneProxy::~FFieldSystemSceneProxy()
{
}

