// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "EngineGlobals.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"

class UFieldSystemComponent;

/** Immutable rendering data **/
struct FFieldSystemSampleData
{
};


/***
*   FFieldSystemSceneProxy
*    
*/
class FFieldSystemSceneProxy final : public FPrimitiveSceneProxy
{
public:

	FFieldSystemSceneProxy(UFieldSystemComponent* Component);

	/** virtual destructor */
	virtual ~FFieldSystemSceneProxy();

	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	// @todo allocated size : make this reflect internally allocated memory. 
	virtual uint32 GetMemoryFootprint(void) const override { return(sizeof(*this) + GetAllocatedSize()); }

	/** Size of the base class */
	SIZE_T GetAllocatedSize(void) const { return(FPrimitiveSceneProxy::GetAllocatedSize()); }


};

