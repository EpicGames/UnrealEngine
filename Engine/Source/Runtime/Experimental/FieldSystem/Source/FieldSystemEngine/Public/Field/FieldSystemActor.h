// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** This class represents an FieldSystem Actor. */

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "Field/FieldSystem.h"
#include "Field/FieldSystemNodes.h"
#include "UObject/ObjectMacros.h"

#include "FieldSystemActor.generated.h"

class UFieldSystemComponent;

UCLASS(meta=(ChildCanTick), MinimalAPI)
class AFieldSystemActor: public AActor
{
	GENERATED_UCLASS_BODY()

public:

	/* FieldSystemComponent */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Field, meta = (ExposeFunctionCategories = "Components|FieldSystem", AllowPrivateAccess = "true"))
	TObjectPtr<UFieldSystemComponent> FieldSystemComponent;
	UFieldSystemComponent* GetFieldSystemComponent() const { return FieldSystemComponent; }
	FIELDSYSTEMENGINE_API virtual void OnConstruction(const FTransform& Transform) override;
};
