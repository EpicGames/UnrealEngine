// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Field/FieldSystem.h"
#include "Field/FieldSystemCoreAlgo.h"
#include "Field/FieldSystemNodes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "FieldSystemAsset.generated.h"

class FArchive;


UCLASS(MinimalAPI)
class UFieldSystem : public UObject
{
	GENERATED_BODY()

public:

	UFieldSystem() : UObject() {}
	virtual ~UFieldSystem() = default;

	void Reset() { Commands.Empty(); }

	FIELDSYSTEMENGINE_API void Serialize(FArchive& Ar);

	TArray< FFieldSystemCommand > Commands;

};
