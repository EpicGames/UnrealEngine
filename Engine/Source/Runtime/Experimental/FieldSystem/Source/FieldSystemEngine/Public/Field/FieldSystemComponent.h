// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/PrimitiveComponent.h"
#include "Field/FieldSystem.h"
#include "Field/FieldSystemObjects.h"
#include "Field/FieldSystemAsset.h"
#include "Field/FieldSystemComponentTypes.h"
#include "Chaos/ChaosSolverActor.h"

#include "FieldSystemComponent.generated.h"

struct FFieldSystemSampleData;
class FChaosSolversModule;

///**
//*
//*/
//UENUM(BlueprintType)
//enum class EChaosObjectType
//{
//	Chaos_Destruction        UMETA(DisplayName = "Set Always", ToolTip = "The particle output value will be equal to Interior-value if the particle position is inside a sphere / Exterior-value otherwise."),
//	Chaos_Rigid, 
//
//	Chaos_Rban  UMETA(DisplayName = "Merge Interior", ToolTip = "The particle output value will be equal to Interior-value if the particle position is inside the sphere or if the particle input value is already Interior-Value / Exterior-value otherwise."),
//	Chaos_Cloth  UMETA(DisplayName = "Merge Exterior", ToolTip = "The particle output value will be equal to Exterior-value if the particle position is outside the sphere or if the particle input value is already Exterior-Value / Interior-value otherwise."),
//	//~~~
//	//256th entry
//	Chaos_Objects_Max                 UMETA(Hidden)
//};

/**
*	FieldSystemComponent
*/
UCLASS(meta = (BlueprintSpawnableComponent), MinimalAPI)
class UFieldSystemComponent : public UPrimitiveComponent
{
	GENERATED_UCLASS_BODY()
		friend class FFieldSystemEditorCommands;

public:
	//~ Begin USceneComponent Interface.
	virtual bool HasAnySockets() const override { return false; }
	//~ Begin USceneComponent Interface.

	//~ Begin UPrimitiveComponent Interface.
	FIELDSYSTEMENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface.

	/** Set the field system asset @todo(remove the field system, we dont need the asset */
	void SetFieldSystem(UFieldSystem* FieldSystemIn) { FieldSystem = FieldSystemIn; }

	/** Get the field system asset */
	inline const UFieldSystem* GetFieldSystem() const { return FieldSystem; }

	/** Field system asset to be used to store the construction fields */
	UPROPERTY(EditAnywhere, AdvancedDisplay, BlueprintReadOnly, Category = "Field", meta = (ToolTip = "Field system asset to be used to store the construction fields. This asset is not required anymore and will be deprecated soon."))
	TObjectPtr<UFieldSystem> FieldSystem;

	/** If enabled the field will be pushed to the world fields and will be available to materials and niagara */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (ToolTip = "If enabled the field will be pushed to the world fields and will be available to materials and niagara"))
	bool bIsWorldField;

	/** If enabled the field will be used by all the chaos solvers */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (ToolTip = "If enabled the field will be used by all the chaos solvers"))
	bool bIsChaosField;

	/** List of solvers this field will affect. An empty list makes this field affect all solvers. */
	UPROPERTY(EditAnywhere, Category = "Field", meta = (EditCondition = "bIsChaosField == true", ToolTip = "List of chaos solvers that will use the field"))
	TArray<TSoftObjectPtr<AChaosSolverActor>> SupportedSolvers;

	/** List of all the construction command */
	UPROPERTY()
	FFieldObjectCommands ConstructionCommands;

	/** List of all the buffer command */
	UPROPERTY()
	FFieldObjectCommands BufferCommands;

	//
	// Blueprint based field interface
	//

	/**
	*  ApplyUniformForce
	*    This function will dispatch a command to the physics thread to apply
	*    a uniform linear force on each particle within the simulation.
	*
	*    @param Enabled Is this force enabled for evaluation.
	*    @param Direction The direction of the linear force
	*    @param Magnitude The size of the linear force.
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "Field", DisplayName = "Apply Uniform Force", meta = (UnsafeDuringActorConstruction = "true"))
	FIELDSYSTEMENGINE_API void ApplyLinearForce(UPARAM(DisplayName = "Enable Field") bool Enabled,
			UPARAM(DisplayName = "Uniform Direction") FVector Direction,
			UPARAM(DisplayName = "Field Magnitude") float Magnitude);

	/**
	*  SetDynamicState
	*    This function will dispatch a command to the physics thread to apply
	*    a kinematic to dynamic state change for the particles within the field.
	*
	*    @param Enabled Is this force enabled for evaluation.
	*    @param Position The location of the command
	*    @param Radius Radial influence from the position
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "Field", DisplayName = "Set Dynamic State", meta = (UnsafeDuringActorConstruction = "true"))
	FIELDSYSTEMENGINE_API void ApplyStayDynamicField(UPARAM(DisplayName = "Enable Field") bool Enabled,
			UPARAM(DisplayName = "Center Position") FVector Position,
			UPARAM(DisplayName = "Field Radius") float Radius);

	/**
	*  ApplyRadialForce
	*    This function will dispatch a command to the physics thread to apply
	*    a linear force that points away from a position.
	*
	*    @param Enabled Is this force enabled for evaluation.
	*    @param Position The origin point of the force
	*    @param Magnitude The size of the linear force.
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "Field", DisplayName = "Apply Radial Force", meta = (UnsafeDuringActorConstruction = "true"))
	FIELDSYSTEMENGINE_API void ApplyRadialForce(UPARAM(DisplayName = "Enable Field") bool Enabled,
			UPARAM(DisplayName = "Center Position") FVector Position,
			UPARAM(DisplayName = "Field Magnitude") float Magnitude);

	/**
	*  FalloffRadialForce
	*    This function will dispatch a command to the physics thread to apply
	*    a linear force from a position in space. The force vector is weaker as
	*    it moves away from the center.
	*
	*    @param Enabled Is this force enabled for evaluation.
	*    @param Position The origin point of the force
	*    @param Radius Radial influence from the position, positions further away are weaker.
	*    @param Magnitude The size of the linear force.
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "Field", DisplayName = "Falloff Radial Force", meta = (UnsafeDuringActorConstruction = "true"))
	FIELDSYSTEMENGINE_API void ApplyRadialVectorFalloffForce(UPARAM(DisplayName = "Enable Field") bool Enabled,
			UPARAM(DisplayName = "Center Position") FVector Position,
			UPARAM(DisplayName = "Falloff Radius") float Radius,
			UPARAM(DisplayName = "Field Magnitude") float Magnitude);

	/**
	*  FalloffUniformForce
	*    This function will dispatch a command to the physics thread to apply
	*    a linear force in a uniform direction. The force vector is weaker as
	*    it moves away from the center.
	*
	*    @param Enabled Is this force enabled for evaluation.
	*    @param Position The origin point of the force
	*    @param Direction The direction of the linear force
	*    @param Radius Radial influence from the position, positions further away are weaker.
	*    @param Magnitude The size of the linear force.
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "Field", DisplayName = "Falloff Uniform Force", meta = (UnsafeDuringActorConstruction = "true"))
	FIELDSYSTEMENGINE_API void ApplyUniformVectorFalloffForce(UPARAM(DisplayName = "Enable Field") bool Enabled,
			UPARAM(DisplayName = "Center Position") FVector Position,
			UPARAM(DisplayName = "Uniform Direction") FVector Direction,
			UPARAM(DisplayName = "Falloff Radius") float Radius,
			UPARAM(DisplayName = "Field Magnitude") float Magnitude);

	/**
	*  ApplyExternalStran
	*    This function will dispatch a command to the physics thread to apply
	*    a strain field on a clustered set of geometry. This is used to trigger a
	*    breaking event within the solver.
	*
	*    @param Enabled Is this force enabled for evaluation.
	*    @param Position The origin point of the force
	*    @param Radius Radial influence from the position, positions further away are weaker.
	*    @param Magnitude The size of the linear force.
	*    @param Iterations Levels of evaluation into the cluster hierarchy.
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "Field", DisplayName = "Apply External Strain", meta = (UnsafeDuringActorConstruction = "true"))
	FIELDSYSTEMENGINE_API void ApplyStrainField(UPARAM(DisplayName = "Enable Field") bool Enabled,
			UPARAM(DisplayName = "Center Position") FVector Position,
			UPARAM(DisplayName = "Falloff Radius") float Radius,
			UPARAM(DisplayName = "Field Magnitude") float Magnitude,
			UPARAM(DisplayName = "Cluster Levels") int32 Iterations);

	/**
	*  AddTransientField
	*    This function will dispatch a command to the physics thread to apply
	*    a generic evaluation of a user defined transient field network. See documentation,
	*    for examples of how to recreate variations of the above generic
	*    fields using field networks
	*
	*    @param Enabled Is this force enabled for evaluation.
	*    @param Target Type of field supported by the solver.
	*    @param MetaData Meta data used to assist in evaluation
	*    @param Field Base evaluation node for the field network.
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "Field", DisplayName = "Add Transient Field", meta = (UnsafeDuringActorConstruction = "true"))
	FIELDSYSTEMENGINE_API void ApplyPhysicsField(UPARAM(DisplayName = "Enable Field") bool Enabled,
			UPARAM(DisplayName = "Physics Type") EFieldPhysicsType Target,
			UPARAM(DisplayName = "Meta Data") UFieldSystemMetaData* MetaData,
			UPARAM(DisplayName = "Field Node") UFieldNodeBase* Field);

	//
	// Blueprint persistent field interface
	//

	/**
	*  AddPersistentField
	*    This function will dispatch a command to the physics thread to apply
	*    a generic evaluation of a user defined field network. This command will be persistent in time and will live until
	*    the component is destroyed or until the RemovePersistenFields function is called. See documentation,
	*    for examples of how to recreate variations of the above generic
	*    fields using field networks
	*
	*    @param Enabled Is this force enabled for evaluation.
	*    @param Target Type of field supported by the solver.
	*    @param MetaData Meta data used to assist in evaluation
	*    @param Field Base evaluation node for the field network.
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "Field", DisplayName = "Add Persistent Field", meta = (UnsafeDuringActorConstruction = "true"))
	FIELDSYSTEMENGINE_API void AddPersistentField(UPARAM(DisplayName = "Enable Field") bool Enabled,
			UPARAM(DisplayName = "Physics Type") EFieldPhysicsType Target,
			UPARAM(DisplayName = "Meta Data")  UFieldSystemMetaData* MetaData,
			UPARAM(DisplayName = "Field Node") UFieldNodeBase* Field);

	/**
	*  RemovePersistentFields
	*    This function will remove all the field component persistent fields from chaos and from the world
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "Field", DisplayName = "Remove Persistent Fields", meta = (UnsafeDuringActorConstruction = "true"))
	FIELDSYSTEMENGINE_API void RemovePersistentFields();

	//
	// Blueprint construction field interface
	//

	/**
	*  AddConstructionField
	*    This function will dispatch a command to the physics thread to apply
	*    a generic evaluation of a user defined field network. This command will be used in a
	*    construction script to setup some particles properties (anchors...). See documentation,
	*    for examples of how to recreate variations of the above generic
	*    fields using field networks
	*
	*    @param Enabled Is this force enabled for evaluation.
	*    @param Target Type of field supported by the solver.
	*    @param MetaData Meta data used to assist in evaluation
	*    @param Field Base evaluation node for the field network.
	*
	*/

	UFUNCTION(BlueprintCallable, Category = "Field", DisplayName = "Add Construction Field")
	FIELDSYSTEMENGINE_API void AddFieldCommand(UPARAM(DisplayName = "Enable Field") bool Enabled,
			UPARAM(DisplayName = "Physics Type") EFieldPhysicsType Target,
			UPARAM(DisplayName = "Meta Data") UFieldSystemMetaData* MetaData,
			UPARAM(DisplayName = "Field Node") UFieldNodeBase* Field);

	/**
	*  RemoveConstructionFields
	*    This function will remove all the field component construction fields from chaos and from the world
	*
	*/

	UFUNCTION(BlueprintCallable, Category = "Field", DisplayName = "Remove Construction Fields")
	FIELDSYSTEMENGINE_API void ResetFieldSystem();

	/** Get all the construction fields*/
	const TArray< FFieldSystemCommand >& GetConstructionFields() const { return SetupConstructionFields; }

protected:

	/** Get ell ethe supported physics scenes */
	FIELDSYSTEMENGINE_API TSet<FPhysScene_Chaos*> GetPhysicsScenes() const;

	/** Get ell the supported physics solvers */
	FIELDSYSTEMENGINE_API TArray<Chaos::FPhysicsSolverBase*> GetPhysicsSolvers() const;

	/** Build a physics field command and dispatch it */
	FIELDSYSTEMENGINE_API void BuildFieldCommand(bool Enabled, EFieldPhysicsType Target, UFieldSystemMetaData* MetaData, UFieldNodeBase* Field, const bool IsTransient);

	/** Dispatch the field command to chaos/world */
	FIELDSYSTEMENGINE_API void DispatchFieldCommand(const FFieldSystemCommand& InCommand, const bool IsTransient);

	/** Remove the persistent commands from chaos/world  */
	FIELDSYSTEMENGINE_API void ClearFieldCommands();

	//~ Begin UActorComponent Interface.
	FIELDSYSTEMENGINE_API virtual void OnCreatePhysicsState() override;
	FIELDSYSTEMENGINE_API virtual void OnDestroyPhysicsState() override;
	FIELDSYSTEMENGINE_API virtual bool ShouldCreatePhysicsState() const override;
	FIELDSYSTEMENGINE_API virtual bool HasValidPhysicsState() const override;
	//~ End UActorComponent Interface.

	/** Whether to intialise this component in Editor worlds */
	UPROPERTY(EditDefaultsOnly, Category=Field)
	bool bEnableInEditorWorlds;

	/** Whether to intialise this component in Preview worlds (both game and editor) */
	UPROPERTY(EditDefaultsOnly, Category=Field)
	bool bEnableInPreviewWorlds;

	/** Chaos module linked to that component */
	FChaosSolversModule* ChaosModule;

	/** Boolean to check that the physics state has been built*/
	bool bHasPhysicsState;

	/** List of all the field used to setup chaos (anchor...)*/
	TArray< FFieldSystemCommand > SetupConstructionFields;

	/** List of all the chaos peristent fields */
	TArray< FFieldSystemCommand > ChaosPersistentFields;

	/** List of the world GPU peristent fields created by this field component */
	TArray< FFieldSystemCommand > WorldGPUPersistentFields;

	/** List of the world CPU peristent fields created by this field component */
	TArray< FFieldSystemCommand > WorldCPUPersistentFields;
};
