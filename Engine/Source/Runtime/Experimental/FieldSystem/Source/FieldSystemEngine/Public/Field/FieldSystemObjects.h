// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Queue.h"
#include "Field/FieldSystemTypes.h"
#include "Field/FieldSystem.h"
#include "Field/FieldSystemNodes.h"
#include "Math/Vector.h"
#include "Components/PrimitiveComponent.h"
#include "PhysicsField/PhysicsFieldComponent.h"

#include "FieldSystemObjects.generated.h"

/**
* Context :
*   Contexts are used to pass extra data into the field evaluation.
*/
UCLASS(MinimalAPI)
class UFieldSystemMetaData : public UActorComponent
{
	GENERATED_BODY()

public:
	virtual ~UFieldSystemMetaData() = default;
	virtual FFieldSystemMetaData::EMetaType Type() const { return FFieldSystemMetaData::EMetaType::ECommandData_None; }
	virtual FFieldSystemMetaData* NewMetaData() const { return nullptr; }
};

/*
* UFieldSystemMetaDataIteration : Not used anymore, just hiding it right now but will probably be deprecated if not used in the future
*/
UCLASS(MinimalAPI)
class UFieldSystemMetaDataIteration : public UFieldSystemMetaData
{
	GENERATED_BODY()

public:
	virtual ~UFieldSystemMetaDataIteration() = default;
	virtual FFieldSystemMetaData::EMetaType Type() const override { return  FFieldSystemMetaData::EMetaType::ECommandData_Iteration; }

	FIELDSYSTEMENGINE_API virtual FFieldSystemMetaData* NewMetaData() const override;

	/**
	 * Set the number of iteration type
	 * @param    Iterations Number of iterations (WIP)
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (Iterations = "1"))
	FIELDSYSTEMENGINE_API UFieldSystemMetaDataIteration* SetMetaDataIteration(UPARAM(DisplayName = "Iteration Count") int Iterations);

	/** Number of iterations (WIP) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Iteration Count")
	int Iterations;
};

/*
* UFieldSystemMetaDataProcessingResolution
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Control the set of particles on which the field will be applied"), ShowCategories = ("Field"), DisplayName = "MetaDataResolution", MinimalAPI)
class UFieldSystemMetaDataProcessingResolution : public UFieldSystemMetaData
{
	GENERATED_BODY()

public:
	virtual ~UFieldSystemMetaDataProcessingResolution() = default;
	virtual FFieldSystemMetaData::EMetaType Type() const override { return  FFieldSystemMetaData::EMetaType::ECommandData_ProcessingResolution; }
	FIELDSYSTEMENGINE_API virtual FFieldSystemMetaData* NewMetaData() const override;

	/**
	 * Set the processing resolution type
	 * @param    ResolutionType Type of processing resolution used to select the particles on which the field will be applied
	 */
	UFUNCTION(BlueprintPure, Category = "Field", DisplayName = "Set Meta Data Resolution")
	FIELDSYSTEMENGINE_API UFieldSystemMetaDataProcessingResolution* SetMetaDataaProcessingResolutionType(EFieldResolutionType ResolutionType);

	/** Precessing resolution type used to select the particles on which the field will be applied */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Field")
	TEnumAsByte<EFieldResolutionType> ResolutionType;
};

/*
* UFieldSystemMetaDataFilter
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Filter the particles on which the field will be applied"), ShowCategories = ("Field"), DisplayName = "MetaDataFilter", MinimalAPI)
class UFieldSystemMetaDataFilter : public UFieldSystemMetaData
{
	GENERATED_BODY()

public:
	virtual ~UFieldSystemMetaDataFilter() = default;
	virtual FFieldSystemMetaData::EMetaType Type() const override { return  FFieldSystemMetaData::EMetaType::ECommandData_Filter; }
	FIELDSYSTEMENGINE_API virtual FFieldSystemMetaData* NewMetaData() const override;

	/**
	 * Set the particles filter type
	 * @param    FilterType State type used to select the state particles on which the field will be applied
	 * @param    ObjectType Object type used to select the type of objects(rigid, cloth...) on which the field will be applied
	 * @param    PositionType Position type used to compute the samples positions
	 */
	UFUNCTION(BlueprintPure, Category = "Field", DisplayName = "Set Meta Data Filter")
	FIELDSYSTEMENGINE_API UFieldSystemMetaDataFilter* SetMetaDataFilterType(UPARAM(DisplayName = "State Type") EFieldFilterType FilterType, 
													  UPARAM(DisplayName = "Object Type") EFieldObjectType ObjectType,
													  UPARAM(DisplayName = "Position Type") EFieldPositionType PositionType );

	/** Filter state type used to select the particles on which the field will be applied */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Field", DisplayName = "State Type")
	TEnumAsByte<EFieldFilterType> FilterType;

	/** Filter object type used to select the particles on which the field will be applied */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Field", DisplayName = "Object Type")
	TEnumAsByte<EFieldObjectType> ObjectType;

	/** Specify which position type will be used for the field evaluation*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Field", DisplayName = "Position Type")
	TEnumAsByte<EFieldPositionType> PositionType;
};

/**
* Field Evaluation
*/
UCLASS(MinimalAPI)
class UFieldNodeBase : public UActorComponent
{
	GENERATED_BODY()

public:
	virtual ~UFieldNodeBase() = default;
	virtual FFieldNodeBase::EFieldType Type() const { return FFieldNodeBase::EFieldType::EField_None; }
	virtual bool ResultsExpector() const { return false; }
	virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const { return nullptr; }
};


/**
* FieldNodeInt
*/
UCLASS(MinimalAPI)
class UFieldNodeInt : public UFieldNodeBase
{
	GENERATED_BODY()

public:
	virtual ~UFieldNodeInt() = default;
	virtual FFieldNodeBase::EFieldType Type() const override { return  FFieldNodeBase::EFieldType::EField_Int32; }
};

/**
* FieldNodeFloat
*/
UCLASS(MinimalAPI)
class UFieldNodeFloat : public UFieldNodeBase
{
	GENERATED_BODY()

public:
	virtual ~UFieldNodeFloat() = default;
	virtual FFieldNodeBase::EFieldType Type() const override { return  FFieldNodeBase::EFieldType::EField_Float; }
};

/**
* FieldNodeVector
*/
UCLASS(MinimalAPI)
class UFieldNodeVector : public UFieldNodeBase
{
	GENERATED_BODY()
public:
	virtual ~UFieldNodeVector() = default;
	virtual FFieldNodeBase::EFieldType Type() const override { return  FFieldNodeBase::EFieldType::EField_FVector; }
};


/**
* UUniformInteger
**/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Set a uniform integer value independently of the sample position. The output is equal to magnitude"), ShowCategories = ("Field"), MinimalAPI)
class UUniformInteger : public UFieldNodeInt
{
	GENERATED_BODY()
public:

	UUniformInteger()
		: Super()
		, Magnitude(0)
	{}

	virtual ~UUniformInteger() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Set a uniform integer value independently of the sample position. The output is equal to magnitude
	 * @param    Magnitude The field output will be equal the magnitude
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (Magnitude = "0"))
	FIELDSYSTEMENGINE_API UUniformInteger* SetUniformInteger(UPARAM(DisplayName = "Field Magnitude") int32 Magnitude);

	/** The field output will be equal the magnitude */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Field Magnitude")
	int32 Magnitude;
};


/**
* URadialIntMask
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "This function first defines a radial integer field equal to Interior-value inside a sphere / Exterior-value outside. This field will be used alongside the particle input value and the mask condition to compute the particle output value."), ShowCategories = ("Field"), MinimalAPI)
class URadialIntMask : public UFieldNodeInt
{
	GENERATED_BODY()
public:

	URadialIntMask()
		: Super()
		, Radius(0)
		, Position(FVector(0, 0, 0))
		, InteriorValue(1)
		, ExteriorValue(0)
		, SetMaskCondition(ESetMaskConditionType::Field_Set_Always)
	{}
	virtual ~URadialIntMask() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	* This function first defines a radial integer field equal to Interior-value inside a sphere / Exterior-value outside. This field will be used alongside the particle input value and the mask condition to compute the particle output value.
	 * - If Mask-condition = set-always : the particle output value will be equal to Interior-value if the particle position is inside a sphere / Exterior-value otherwise. 
	 * - If Mask-condition = merge-interior : the particle output value will be equal to Interior-value if the particle position is inside the sphere or if the particle input value is already Interior-Value / Exterior-value otherwise.
	 * - If Mask-condition = merge-exterior : the particle output value will be equal to Exterior-value if the particle position is outside the sphere or if the particle input value is already Exterior-Value / Interior-value otherwise.
	 * @param    Radius Radius of the radial field
	 * @param    Position Center position of the radial field"
	 * @param    InteriorValue If the sample distance from the center is less than radius, the intermediate value will be set the interior value
	 * @param    ExteriorValue If the sample distance from the center is greater than radius, the intermediate value will be set the exterior value
	 * @param    SetMaskConditionIn If the mask condition is set to always the output value will be the intermediate one. If set to not interior/exterior the output value will be the intermediate one if the input is different from the interior/exterior value
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (InteriorValue = "1"), DisplayName = "Set Radial Mask")
	FIELDSYSTEMENGINE_API URadialIntMask* SetRadialIntMask(UPARAM(DisplayName = "Mask Radius") float Radius,
			UPARAM(DisplayName = "Center Position") FVector Position,
			UPARAM(DisplayName = "Interior Value") int32 InteriorValue,
			UPARAM(DisplayName = "Exterior Value") int32 ExteriorValue,
			UPARAM(DisplayName = "Mask Condition") ESetMaskConditionType SetMaskConditionIn);

	/** Radius of the radial mask field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Mask Radius")
	float Radius;

	/** Center position of the radial mask field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Center Position")
	FVector Position;

	/** If the sample distance from the center is less than radius, the intermediate value will be set the interior value */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Interior Value")
	int32 InteriorValue;

	/** If the sample distance from the center is greater than radius, the intermediate value will be set the exterior value */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Exterior Value")
	int32 ExteriorValue;

	/** If the mask condition is set to always the output value will be the intermediate one. If set to not interior/exterior the output value will be the intermediate one if the input is different from the interior/exterior value */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Mask Condition")
	TEnumAsByte<ESetMaskConditionType> SetMaskCondition;
};


/**
* UUniformScalar
**/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Set a uniform scalar value independently of the sample position. The output is equal to magnitude"), ShowCategories = ("Field"), MinimalAPI)
class UUniformScalar : public UFieldNodeFloat
{
	GENERATED_BODY()
public:

	UUniformScalar()
		: Super()
		, Magnitude(1.0)
	{}

	virtual ~UUniformScalar() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Set a uniform scalar value independently of the sample position. The output is equal to magnitude
	 * @param    Magnitude The field output will be equal the magnitude
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (Magnitude = "1.0"))
	FIELDSYSTEMENGINE_API UUniformScalar* SetUniformScalar(UPARAM(DisplayName = "Field Magnitude") float Magnitude);

	/** The field output will be equal the magnitude */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Field Magnitude")
	float Magnitude;

};

/**
* UWaveScalar
**/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Set a temporal wave scalar value according to the sample distance from the field position."), ShowCategories = ("Field"), MinimalAPI)
class UWaveScalar : public UFieldNodeFloat
{
	GENERATED_BODY()
public:

	UWaveScalar()
		: Super()
		, Magnitude(1.0)
		, Position(0, 0, 0)
		, Wavelength(10000)
		, Period(1.0)
		, Function(EWaveFunctionType::Field_Wave_Cosine)
		, Falloff(EFieldFalloffType::Field_Falloff_Linear)
	{}

	virtual ~UWaveScalar() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Set a temporal wave scalar value according to the sample distance from the field position.
	 * @param    Magnitude Magnitude of the wave function
	 * @param    Position Center position of the wave field
	 * @param    Wavelength Distance between 2 wave peaks
	 * @param    Period Time over which the wave will travel from one peak to another one. The wave velocity is proportional to wavelength/period
	 * @param    Function Wave function used for the field
	 * @param    Falloff Type of falloff function used if the falloff function is picked
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (Magnitude = "1.0", Wavelength = "1000", Period = "1", HidePin = "Time"), DisplayName = "Set Wave Scalar")
	FIELDSYSTEMENGINE_API UWaveScalar* SetWaveScalar(UPARAM(DisplayName = "Field Magnitude") float Magnitude,
			UPARAM(DisplayName = "Center Position") FVector Position,
			UPARAM(DisplayName = "Wave Length") float Wavelength,
			UPARAM(DisplayName = "Wave Period") float Period,
			UPARAM(DisplayName = "Time Offset") float Time,
			UPARAM(DisplayName = "Wave Function") EWaveFunctionType Function,
			UPARAM(DisplayName = "Falloff Type") EFieldFalloffType Falloff);

	/** Magnitude of the wave function */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Field Magnitude")
	float Magnitude;

	/** Center position of the wave field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Center Position")
	FVector Position;

	/** Distance between 2 wave peaks */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Wave Length")
	float Wavelength;

	/** Time over which the wave will travel from one peak to another one. The wave velocity is proportional to wavelength/period */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Wave Period")
	float Period;

	/** Wave function used for the field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Wave Function")
	TEnumAsByte<EWaveFunctionType> Function;

	/** Type of falloff function used if the falloff function is picked */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Falloff Type")
	TEnumAsByte<EFieldFalloffType> Falloff;
};



/**
* RadialFalloff
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Sphere scalar field that will be defined only within a sphere"), ShowCategories = ("Field"), MinimalAPI)
class URadialFalloff : public UFieldNodeFloat
{
	GENERATED_BODY()
public:

	URadialFalloff()
		: Super()
		, Magnitude(1.0)
		, MinRange(0.f)
		, MaxRange(1.f)
		, Default(0.f)
		, Radius(0)
		, Position(FVector(0, 0, 0))
		, Falloff(EFieldFalloffType::Field_Falloff_Linear)
	{}

	virtual ~URadialFalloff() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Sphere scalar field that will be defined only within a sphere
	 * @param    Magnitude Magnitude of the sphere falloff field
	 * @param    MinRange The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude
	 * @param    MaxRange The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude
	 * @param    Default The field value will be set to Default if the sample distance from the center is higher than the radius
	 * @param    Radius Radius of the sphere falloff field
	 * @param    Position Center position of the sphere falloff field
	 * @param    Falloff Type of falloff function used if the falloff function is picked
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (Magnitude = "1.0", MinRange = "0.0", MaxRange = "1.0"), DisplayName = "Set Radial Falloff")
	FIELDSYSTEMENGINE_API URadialFalloff* SetRadialFalloff(UPARAM(DisplayName = "Field Magnitude") float Magnitude,
			UPARAM(DisplayName = "Min Range") float MinRange,
			UPARAM(DisplayName = "Max Range") float MaxRange,
			UPARAM(DisplayName = "Default Value") float Default,
			UPARAM(DisplayName = "Sphere Radius") float Radius,
			UPARAM(DisplayName = "Center Position") FVector Position,
			UPARAM(DisplayName = "Falloff Type") EFieldFalloffType Falloff);

	/** Magnitude of the sphere falloff field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Field Magnitude")
	float Magnitude;

	/** The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Min Range")
	float MinRange;

	/** The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Max Range")
	float MaxRange;

	/** The field value will be set to Default if the sample distance from the center is higher than the radius */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Default Value")
	float Default;

	/** Radius of the sphere falloff field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Sphere Radius")
	float Radius;

	/** Center position of the sphere falloff field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Center Position")
	FVector Position;

	/** Type of falloff function used to model the evolution of the field from the sphere center to the sample position */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Falloff Type")
	TEnumAsByte<EFieldFalloffType> Falloff;
};

/**
* PlaneFalloff
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Plane scalar field that will be defined only within a distance from a plane"), ShowCategories = ("Field"), MinimalAPI)
class UPlaneFalloff : public UFieldNodeFloat
{
	GENERATED_BODY()

public:

	UPlaneFalloff()
		: Super()
		, Magnitude(1.0)
		, MinRange(0.f)
		, MaxRange(1.f)
		, Default(0.f)
		, Distance(0.f)
		, Position(FVector(0, 0, 0))
		, Normal(FVector(0, 0, 1))
		, Falloff(EFieldFalloffType::Field_Falloff_Linear)
	{}

	virtual ~UPlaneFalloff() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Plane scalar field that will be defined only within a distance from a plane
	 * @param    Magnitude Magnitude of the plane falloff field
	 * @param    MinRange The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude
	 * @param    MaxRange The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude
	 * @param    Default The field value will be set to default if the sample projected distance ((Sample Position - Center Position).dot(Plane Normal)) is higher than the Plane Distance 
	 * @param    Distance Distance limit for the plane falloff field starting from the center position and extending in the direction of the plane normal
	 * @param    Position Plane center position of the plane falloff field
	 * @param    Normal Plane normal of the plane falloff field
	 * @param    Falloff Type of falloff function used to model the evolution of the field from the plane surface to the distance isosurface
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (Magnitude = "1.0", MinRange = "0.0", MaxRange = "1.0"))
	FIELDSYSTEMENGINE_API UPlaneFalloff* SetPlaneFalloff(UPARAM(DisplayName = "Field Magnitude") float Magnitude,
			UPARAM(DisplayName = "Min Range") float MinRange,
			UPARAM(DisplayName = "Max Range") float MaxRange,
			UPARAM(DisplayName = "Default Value") float Default,
			UPARAM(DisplayName = "Plane Distance") float Distance,
			UPARAM(DisplayName = "Center Position") FVector Position,
			UPARAM(DisplayName = "Plane Normal") FVector Normal,
			UPARAM(DisplayName = "Falloff Type") EFieldFalloffType Falloff);

	/** Magnitude of the plane falloff field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Field Magnitude")
	float Magnitude;

	/** The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Min Range")
	float MinRange;

	/** The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Max Range")
	float MaxRange;

	/** The field value will be set to Default if the sample distance from the plane is higher than the distance*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Default Value")
	float Default;

	/** Distance limit for the plane falloff field*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Plane Distance")
	float Distance;

	/** Plane position of the plane falloff field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Center Position")
	FVector Position;

	/** Plane normal of the plane falloff field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Plane Normal")
	FVector Normal;

	/** Type of falloff function used to model the evolution of the field from the plane surface to the distance isosurface */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Falloff Type")
	TEnumAsByte<EFieldFalloffType> Falloff;
};

/**
* BoxFalloff
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Box scalar field that will be defined only within a box"), ShowCategories = ("Field"), MinimalAPI)
class UBoxFalloff : public UFieldNodeFloat
{
	GENERATED_BODY()

public:

	UBoxFalloff()
		: Super()
		, Magnitude(1.0)
		, MinRange(0.f)
		, MaxRange(1.f)
		, Default(0.0)
		, Transform(FTransform::Identity)
		, Falloff(EFieldFalloffType::Field_Falloff_Linear)
	{}

	virtual ~UBoxFalloff() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Box scalar field that will be defined only within a box
	 * @param    Magnitude Magnitude of the box falloff field
	 * @param    MinRange The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude
	 * @param    MaxRange The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude
	 * @param    Default The field value will be set to Default if the sample distance from the box is higher than the scale of the transform
	 * @param    Transform Translation, Rotation and Scale of the unit box
	 * @param    Falloff Type of falloff function used to model the evolution of the field from the box surface to the sample position
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (Magnitude = "1.0", MinRange = "0.0", MaxRange = "1.0"))
	FIELDSYSTEMENGINE_API UBoxFalloff* SetBoxFalloff(UPARAM(DisplayName = "Field Magnitude")  float Magnitude,
			UPARAM(DisplayName = "Min Range")  float MinRange,
			UPARAM(DisplayName = "Max Range")  float MaxRange,
			UPARAM(DisplayName = "Default Value")  float Default,
			UPARAM(DisplayName = "Box Transform")  FTransform Transform,
			UPARAM(DisplayName = "Falloff Type")  EFieldFalloffType Falloff);

	/** Magnitude of the box falloff field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Field Magnitude")
	float Magnitude;

	/** The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Min Range")
	float MinRange;

	/** The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Max Range")
	float MaxRange;

	/** The field value will be set to Default if the sample distance from the box is higher than the scale of the transform */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Default Value")
	float Default;

	/** Translation, Rotation and Scale of the unit box */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Box Transform")
	FTransform Transform;

	/** Type of falloff function used to model the evolution of the field from the box surface to the sample position */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Falloff Type")
	TEnumAsByte<EFieldFalloffType> Falloff;
};


/**
* NoiseField
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Defines a perlin noise scalar value if the sample is within a box"), ShowCategories = ("Field"), MinimalAPI)
class UNoiseField : public UFieldNodeFloat
{
	GENERATED_BODY()

public:

	UNoiseField()
		: Super()
		, MinRange(0.f)
		, MaxRange(1.f)
		, Transform()
	{}

	virtual ~UNoiseField() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Defines a perlin noise scalar value if the sample is within a box
	 * @param    MinRange The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude
	 * @param    MaxRange The initial function value between 0 and 1 will be scaled between MinRange and MaxRange before being multiplied by magnitude
	 * @param    Transform Transform of the box in which the perlin noise will be defined
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (MinRange = "0.0", MaxRange = "1.0"), DisplayName = "Set Noise Field")
	FIELDSYSTEMENGINE_API UNoiseField* SetNoiseField(UPARAM(DisplayName = "Min Range") float MinRange,
			UPARAM(DisplayName = "Max Range") float MaxRange,
			UPARAM(DisplayName = "Noise Transform")  FTransform Transform);

	/** The initial function value between 0 and 1 will be scaled between MinRange and MaxRange */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Min Range")
	float MinRange;

	/** The initial function value between 0 and 1 will be scaled between MinRange and MaxRange */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Max Range")
	float MaxRange;

	/** Transform of the box in which the perlin noise will be defined */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Noise Transform")
	FTransform Transform;
};

/**
* UniformVector
**/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Set a uniform vector value independently of the sample position.The output is equal to magnitude * direction"), ShowCategories = ("Field"), MinimalAPI)
class UUniformVector : public UFieldNodeVector
{
	GENERATED_BODY()
public:

	UUniformVector()
		: Super()
		, Magnitude(1.0)
		, Direction(FVector(0, 0, 0))
	{}

	virtual ~UUniformVector() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Set a uniform vector value independently of the sample position.The output is equal to magnitude * direction
	 * @param    Magnitude Magnitude of the uniform vector field
	 * @param    Direction Normalized direction of the uniform vector field
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (Magnitude = "1.0"))
	FIELDSYSTEMENGINE_API UUniformVector* SetUniformVector(UPARAM(DisplayName = "Field Magnitude") float Magnitude,
			UPARAM(DisplayName = "Uniform Direction") FVector Direction);

	/** Magnitude of the uniform vector field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Field Magnitude")
	float Magnitude;

	/** Normalized direction of the uniform vector field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Uniform Direction")
	FVector Direction;
};


/**
* RadialVector
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Set a radial vector value, the direction being the vector from the sample position to the field one. The output is equal to magnitude * direction"), ShowCategories = ("Field"), MinimalAPI)
class URadialVector : public UFieldNodeVector
{
	GENERATED_BODY()
public:

	URadialVector()
		: Super()
		, Magnitude(1.0)
		, Position(FVector(0, 0, 0))
	{}

	virtual ~URadialVector() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Set a radial vector value. The direction is the normalized vector from the field position to the sample one. The output is equal to this direction * magnitude.
	 * @param    Magnitude Magnitude of the radial vector field
	 * @param    Position Center position of the radial vector field
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (Magnitude = "1.0"))
	FIELDSYSTEMENGINE_API URadialVector* SetRadialVector(UPARAM(DisplayName = "Field Magnitude") float Magnitude,
								   UPARAM(DisplayName = "Center Position") FVector Position);

	/** Magnitude of the radial vector field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Field Magnitude")
	float Magnitude;

	/** Center position of the radial vector field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Center Position")
	FVector Position;
};

/**
* RotatedRadialVector
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Set a radial vector value rotated around an axis. The direction is the normalized vector from the field position to the sample, rotated by Angle degrees around Axis. The output is equal to magnitude * rotated direction"), ShowCategories = ("Field"), MinimalAPI)
class URotatedRadialVector : public UFieldNodeVector
{
	GENERATED_BODY()
public:

	URotatedRadialVector()
		: Super()
		, Magnitude(1.0)
		, Position(FVector(0, 0, 0))
		, Angle(0.0f)
		, Axis(FVector(0, 0, 1))
	{}

	virtual ~URotatedRadialVector() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Set a rotated radial vector value. The direction is the normalized vector from the field position to the sample, then rotated by Angle degrees around Axis.
	 * @param    Magnitude Magnitude of the radial vector field
	 * @param    Position Center position of the radial vector field
	 * @param    Angle Rotation angle in degrees
	 * @param    Axis Axis to rotate around
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (Magnitude = "1.0", Angle = "0.0"))
	FIELDSYSTEMENGINE_API URotatedRadialVector* SetRotatedRadialVector(
		UPARAM(DisplayName = "Field Magnitude") float Magnitude,
		UPARAM(DisplayName = "Center Position") FVector Position,
		UPARAM(DisplayName = "Rotation Angle") float Angle,
		UPARAM(DisplayName = "Rotation Axis") FVector Axis);

	/** Magnitude of the radial vector field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Field Magnitude")
	float Magnitude;

	/** Center position of the radial vector field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Center Position")
	FVector Position;

	/** Rotation angle in degrees applied to the radial direction */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Rotation Angle")
	float Angle;

	/** Axis around which the radial direction is rotated */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Rotation Axis")
	FVector Axis;
};

/**
* URandomVector
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Set a random vector value independently of the sample position. The output is equal to magnitude * random direction "), ShowCategories = ("Field"), MinimalAPI)
class URandomVector : public UFieldNodeVector
{
	GENERATED_BODY()
public:

	URandomVector()
		: Super()
		, Magnitude(1.0)
	{}

	virtual ~URandomVector() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Set a random vector value independently of the sample position. The output is equal to magnitude * random direction
	 * @param    Magnitude Magnitude of the random vector field
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (Magnitude = "1.0"))
	FIELDSYSTEMENGINE_API URandomVector* SetRandomVector(UPARAM(DisplayName = "Field Magnitude") float Magnitude);

	/** Magnitude of the random vector field*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Field Magnitude")
	float Magnitude;
};


/**
* UOperatorField
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Compute an operation between 2 incoming fields"), ShowCategories = ("Field"), MinimalAPI)
class UOperatorField : public UFieldNodeBase
{
	GENERATED_BODY()
public:

	UOperatorField()
		: Super()
		, Magnitude(1.0)
		, RightField(nullptr)
		, LeftField(nullptr)
		, Operation(EFieldOperationType::Field_Multiply)
	{}

	virtual ~UOperatorField() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase::EFieldType Type() const;
	virtual bool ResultsExpector() const override { return true; }

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Compute an operation between 2 incoming fields
	 * @param    Magnitude Magnitude of the operator field
	 * @param    LeftField Input field A to be processed
	 * @param    RightField Input field B to be processed
	 * @param    Operation Type of math operation you want to perform between the 2 fields
	 */
	UFUNCTION(BlueprintPure, Category = "Field", meta = (Magnitude = "1.0"))
	FIELDSYSTEMENGINE_API UOperatorField* SetOperatorField(UPARAM(DisplayName = "Field Magnitude") float Magnitude,
			UPARAM(DisplayName = "Left Field") const UFieldNodeBase* LeftField,
			UPARAM(DisplayName = "Right Field") const UFieldNodeBase* RightField,
			UPARAM(DisplayName = "Field Operation") EFieldOperationType Operation);

	/** Magnitude of the operator field */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Field Magnitude")
	float Magnitude;

	/** Right field to be processed */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Right Field")
	TObjectPtr<const UFieldNodeBase> RightField;

	/** Left field to be processed */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Left Field")
	TObjectPtr<const UFieldNodeBase> LeftField;

	/** Type of operation you want to perform between the 2 fields */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Field Operation")
	TEnumAsByte<EFieldOperationType> Operation;
};

/**
* UToIntegerField
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Convert a scalar field to a integer one"), ShowCategories = ("Field"), MinimalAPI)
class UToIntegerField : public UFieldNodeInt
{
	GENERATED_BODY()
public:

	UToIntegerField()
		: Super()
		, FloatField(nullptr)
	{}

	virtual ~UToIntegerField() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Convert a float field to a integer one
	 * @param    FloatField Float field to be converted to an an integer one
	 */
	UFUNCTION(BlueprintPure, Category = "Field", DisplayName = "Set To Integer Field")
	FIELDSYSTEMENGINE_API UToIntegerField* SetToIntegerField(UPARAM(DisplayName = "Float Field") const UFieldNodeFloat* FloatField);

	/** Scalar field to be converted to an an integer one */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Float Field")
	TObjectPtr<const UFieldNodeFloat> FloatField;
};

/**
* UToFloatField
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Convert an integer field to a scalar one"), ShowCategories = ("Field"), MinimalAPI)
class UToFloatField : public UFieldNodeFloat
{
	GENERATED_BODY()
public:

	UToFloatField()
		: Super()
		, IntField(nullptr)
	{}

	virtual ~UToFloatField() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	/**
	 * Convert an integer field to a float one
	 * @param    IntegerField Integer field to be converted to an a float one
	 */
	UFUNCTION(BlueprintPure, Category = "Field", DisplayName = "Set To Float Field")
	FIELDSYSTEMENGINE_API UToFloatField* SetToFloatField(UPARAM(DisplayName = "Integer Field") const UFieldNodeInt* IntegerField);

	/** Integer field to be converted to an a scalar one */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Integer Field")
	TObjectPtr<const UFieldNodeInt> IntField;
};

/**
* UCullingField
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Evaluate the input field according to the result of the culling field"), ShowCategories = ("Field"), MinimalAPI)
class UCullingField : public UFieldNodeBase
{
	GENERATED_BODY()
public:

	UCullingField()
		: Super()
		, Culling(nullptr)
		, Field(nullptr)
		, Operation(EFieldCullingOperationType::Field_Culling_Inside)
	{}

	virtual ~UCullingField() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase::EFieldType Type() const override;

	/**
	 * Evaluate the input field according to the result of the culling field.
	 * 
	 * @param    Culling Culling field to be used.
	 * @param    Field Input field that will be evaluated according to the culling field result.
	 * @param    Operation Evaluate the input field if the result of the culling field is equal to 0 (Inside) or different from 0 (Outside).
	 */
	UFUNCTION(BlueprintPure, Category = "Field")
	FIELDSYSTEMENGINE_API UCullingField* SetCullingField(UPARAM(DisplayName = "Culling Field") const UFieldNodeBase* Culling,
			UPARAM(DisplayName = "Input Field") const UFieldNodeBase* Field,
			UPARAM(DisplayName = "Culling Operation") EFieldCullingOperationType Operation);

	/** Culling field to be used */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Culling Field")
	TObjectPtr<const UFieldNodeBase> Culling;

	/** Input field that will be evaluated according to the culling field result */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Input Field")
	TObjectPtr<const UFieldNodeBase> Field;

	/** Evaluate the input field if the result of the culling field is equal to 0 (Inside) or different from 0 (Outside) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Culling Operation")
	TEnumAsByte<EFieldCullingOperationType> Operation;
};

/**
* UReturnResultsField
*/
UCLASS(ClassGroup = "Field", meta = (BlueprintSpawnableComponent, ToolTip = "Terminal field of a graph"), ShowCategories = ("Field"), MinimalAPI)
class UReturnResultsTerminal : public UFieldNodeBase
{
	GENERATED_BODY()
public:

	UReturnResultsTerminal() : Super()
	{}

	virtual ~UReturnResultsTerminal() = default;

	FIELDSYSTEMENGINE_API virtual FFieldNodeBase* NewEvaluationGraph(TArray<const UFieldNodeBase*>& Nodes) const override;
	virtual FFieldNodeBase::EFieldType Type() const override { return  FFieldNodeBase::EFieldType::EField_Results; }

	/** Terminal field of a graph */
	UFUNCTION(BlueprintPure, Category = "Field", DisplayName = "Set Terminal Field")
	FIELDSYSTEMENGINE_API UReturnResultsTerminal* SetReturnResultsTerminal();

};

/**
* Field Commands container that will be stored in the construction script
*/
USTRUCT(BlueprintType)
struct FFieldObjectCommands
{
	GENERATED_BODY()

		FFieldObjectCommands()
		: TargetNames()
		, RootNodes()
		, MetaDatas()
	{}

	~FFieldObjectCommands() = default;

	/** Add a command to the container */
	void AddFieldCommand(const FName& TargetName, UFieldNodeBase* RootNode, UFieldSystemMetaData* MetaData)
	{
		TargetNames.Add(TargetName);
		RootNodes.Add(RootNode);
		MetaDatas.Add(MetaData);
	}

	/** Reset the commands container to empty */
	void ResetFieldCommands()
	{
		TargetNames.Reset();
		RootNodes.Reset();
		MetaDatas.Reset();
	}

	/** Get the number of commands in the container */
	int32 GetNumCommands() const
	{
		return TargetNames.Num();
	}

	/** Create a FFieldCommand from a given target + Unode + metadata  */
	static FFieldSystemCommand CreateFieldCommand(const EFieldPhysicsType PhysicsType, UFieldNodeBase* RootNode, UFieldSystemMetaData* MetaData)
	{
		if (RootNode)
		{
			TArray<const UFieldNodeBase*> Nodes;
			FFieldSystemCommand Command = { PhysicsType, RootNode->NewEvaluationGraph(Nodes) };
			if (ensureMsgf(Command.RootNode,
				TEXT("Failed to generate physics field command for target attribute.")))
			{
				if (MetaData)
				{
					switch (MetaData->Type())
					{
					case FFieldSystemMetaData::EMetaType::ECommandData_ProcessingResolution:
						Command.MetaData.Add(FFieldSystemMetaData::EMetaType::ECommandData_ProcessingResolution).Reset(new FFieldSystemMetaDataProcessingResolution(static_cast<UFieldSystemMetaDataProcessingResolution*>(MetaData)->ResolutionType));
						break;
					case FFieldSystemMetaData::EMetaType::ECommandData_Iteration:
						Command.MetaData.Add(FFieldSystemMetaData::EMetaType::ECommandData_Iteration).Reset(new FFieldSystemMetaDataIteration(static_cast<UFieldSystemMetaDataIteration*>(MetaData)->Iterations));
						break;
					case FFieldSystemMetaData::EMetaType::ECommandData_Filter:
						Command.MetaData.Add(FFieldSystemMetaData::EMetaType::ECommandData_Filter).Reset(new FFieldSystemMetaDataFilter(static_cast<UFieldSystemMetaDataFilter*>(MetaData)->FilterType, 
							static_cast<UFieldSystemMetaDataFilter*>(MetaData)->ObjectType, static_cast<UFieldSystemMetaDataFilter*>(MetaData)->PositionType));
						break;
					}
				}
				ensure(Command.PhysicsType != EFieldPhysicsType::Field_None);
			}
			UPhysicsFieldComponent::BuildCommandBounds(Command);
			return Command;
		}
		else
		{
			return FFieldSystemCommand();
		}
	}

	/** Create a FFieldCommand from a given target + Fnode */
	static FFieldSystemCommand CreateFieldCommand(const EFieldPhysicsType PhysicsType, FFieldNodeBase* RootNode)
	{
		if (RootNode)
		{
			FFieldSystemCommand Command = { PhysicsType, RootNode };
			ensure(Command.PhysicsType != EFieldPhysicsType::Field_None);
			UPhysicsFieldComponent::BuildCommandBounds(Command);
			return Command;
		}
		else
		{
			return FFieldSystemCommand();
		}
	}

	/** Build the FFieldCommand from one item in the container */
	FFieldSystemCommand BuildFieldCommand(const int32 CommandIndex) const
	{
		return (CommandIndex < GetNumCommands()) ? CreateFieldCommand(GetFieldPhysicsType(TargetNames[CommandIndex]), RootNodes[CommandIndex], MetaDatas[CommandIndex]) : FFieldSystemCommand();
	}

	/**Commands Target Name */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Target Names")
	TArray<FName> TargetNames;

	/** Commands Root Node */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Root Nodes")
	TArray<TObjectPtr<UFieldNodeBase>> RootNodes;

	/** Commands Meta Data*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Field", DisplayName = "Meta Datas")
	TArray<TObjectPtr<UFieldSystemMetaData>> MetaDatas;
};





