// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace UnrealBuildTool.Rules
{
	public class GeometryCollectionEngine : ModuleRules
	{
        public GeometryCollectionEngine(ReadOnlyTargetRules Target) : base(Target)
		{
			SetupModulePhysicsSupport(Target);
			SetupIrisSupport(Target);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"ChaosSolverEngine",
					"Core",
					"CoreUObject",
					"DataflowCore",
					"DataflowEngine",
					"Engine",
					"FieldSystemEngine",
					"IntelISPC",
					"ISMPool",
					"MeshDescription",
					"NetCore",
					"Renderer",
					"RenderCore",
					"RHI",
					"SkeletalMeshDescription",
					"StaticMeshDescription",
				}
				);
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"MeshConversion",
					"GeometryCore",
					"PhysicsCore",
				}
				);

			PrivateIncludePathModuleNames.Add("DerivedDataCache");

			if (Target.bBuildEditor)
            {
				DynamicallyLoadedModuleNames.Add("NaniteBuilder");
				PrivateIncludePathModuleNames.Add("NaniteBuilder");

				PublicDependencyModuleNames.Add("EditorFramework");
                PublicDependencyModuleNames.Add("UnrealEd");

				PrivateDependencyModuleNames.Add("SkeletalMeshUtilitiesCommon");
			}

			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
		}
	}
}
