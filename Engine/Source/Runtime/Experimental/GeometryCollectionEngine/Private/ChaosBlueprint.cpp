// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosBlueprint.h"
#include "PhysicsSolver.h"
#include "Async/Async.h"
#include "Engine/World.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "GeometryCollection/GeometryCollectionActor.h"
#include "GeometryCollection/GeometryCollectionComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosBlueprint)

#define DISPATCH_BLUEPRINTS_IMMEDIATE 1

UChaosDestructionListener::UChaosDestructionListener(FObjectInitializer const& ObjectInitializer)
	: Super(ObjectInitializer), LastCollisionDataTimeStamp(-1.f), LastBreakingDataTimeStamp(-1.f), LastTrailingDataTimeStamp(-1.f), LastRemovalDataTimeStamp(-1.f)
{
	bUseAttachParentBound = true;
	bAutoActivate = true;	
	bNeverNeedsRenderUpdate = true;

#ifdef DISPATCH_BLUEPRINTS_IMMEDIATE
	PrimaryComponentTick.bCanEverTick = false;
#else
	PrimaryComponentTick.bCanEverTick = true;
#endif

	SetCollisionFilter(MakeShareable(new FChaosCollisionEventFilter(&CollisionEventRequestSettings)));
	SetBreakingFilter(MakeShareable(new FChaosBreakingEventFilter(&BreakingEventRequestSettings)));
	SetTrailingFilter(MakeShareable(new FChaosTrailingEventFilter(&TrailingEventRequestSettings)));
	SetRemovalFilter(MakeShareable(new FChaosRemovalEventFilter(&RemovalEventRequestSettings)));
}

void UChaosDestructionListener::ClearEvents()
{
#if 0 // solver actors no longer functional, using GetWorld()->GetPhysicsScene() instead
	for (AChaosSolverActor* ChaosSolverActorObject : ChaosSolverActors)
	{
		UnregisterChaosEvents(ChaosSolverActorObject->GetPhysicsScene());
	}
#else
	UnregisterChaosEvents(GetWorld()->GetPhysicsScene());
#endif
}

void UChaosDestructionListener::UpdateEvents()
{
	if (!ChaosSolverActors.Num() && !GeometryCollectionActors.Num())
	{
		if (FPhysScene* PhysScene = GetWorld()->GetPhysicsScene())
		{
			RegisterChaosEvents(PhysScene);
		}
	}
	else
	{

#if 0 // solver actors no longer functional, using GetWorld()->GetPhysicsScene() instead
		for (AChaosSolverActor* ChaosSolverActorObject : ChaosSolverActors)
		{
			if (ChaosSolverActorObject)
			{
				RegisterChaosEvents(ChaosSolverActorObject->GetPhysicsScene());
			}
		}
#else
		if (FPhysScene* PhysScene = GetWorld()->GetPhysicsScene())
		{
			RegisterChaosEvents(PhysScene);
		}
#endif

		for (AGeometryCollectionActor* GeometryCollectionActor : GeometryCollectionActors)
		{
			if (GeometryCollectionActor)
			{
				if (const UGeometryCollectionComponent* GeometryCollectionComponent = GeometryCollectionActor->GetGeometryCollectionComponent())
				{
					if (const FGeometryCollectionPhysicsProxy* GeometryCollectionPhysicsObject = GeometryCollectionComponent->GetPhysicsProxy())
					{
						RegisterChaosEvents(GeometryCollectionComponent->GetWorld()->GetPhysicsScene());
					}
				}
			}
		}
	}
}

#if 0 // #todo: No longer required?
void UChaosDestructionListener::UpdateGeometryCollectionPhysicsProxies()
{
	GeometryCollectionPhysicsProxies.Reset();

	if (GeometryCollectionActors.Num() > 0)
	{
		for (AGeometryCollectionActor* GeometryCollectionActorObject : GeometryCollectionActors)
		{
			if (GeometryCollectionActorObject)
			{
				// Get GeometryCollectionComponent
				if (const UGeometryCollectionComponent* GeometryCollectionComponent = GeometryCollectionActorObject->GetGeometryCollectionComponent())
				{
					// Get GeometryCollectionPhysicsProxies
					if (const FGeometryCollectionPhysicsProxy* GeometryCollectionPhysicsProxy = GeometryCollectionComponent->GetPhysicsProxy())
					{
						if (Chaos::FPhysicsSolver* Solver = GeometryCollectionPhysicsProxy->GetSolver())
						{
							if (!Solvers.Contains(Solver))
							{
								GeometryCollectionPhysicsProxies.Add(GeometryCollectionPhysicsProxy);
							}
						}
					}
				}
			}
		}
	}
}
#endif

#if WITH_EDITOR
void UChaosDestructionListener::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif 

bool UChaosDestructionListener::IsEventListening() const
{
	return  bIsCollisionEventListeningEnabled ||
			bIsBreakingEventListeningEnabled  ||
			bIsTrailingEventListeningEnabled  ||
			bIsRemovalEventListeningEnabled;
}

void UChaosDestructionListener::UpdateTransformSettings()
{
	// Only need to update the transform if anybody is listening at all and if any of the settings are sorting by nearest, otherwise, no need to get updates
	if (IsEventListening())
	{
		bWantsOnUpdateTransform = CollisionEventRequestSettings.SortMethod == EChaosCollisionSortMethod::SortByNearestFirst ||
								  BreakingEventRequestSettings.SortMethod == EChaosBreakingSortMethod::SortByNearestFirst ||
							      TrailingEventRequestSettings.SortMethod == EChaosTrailingSortMethod::SortByNearestFirst ||
								  RemovalEventRequestSettings.SortMethod == EChaosRemovalSortMethod::SortByNearestFirst;
	}
	else
	{
		bWantsOnUpdateTransform = false;
	}

	bChanged = true;
}

void UChaosDestructionListener::BeginPlay()
{
	Super::BeginPlay();
	UpdateEvents();
}

void UChaosDestructionListener::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	ClearEvents();
	Super::EndPlay(EndPlayReason);
}

void UChaosDestructionListener::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
#if 0
	bool bIsListening = IsEventListening();

	// if owning actor is disabled, don't listen
	AActor* Owner = GetOwner();
	if (Owner && !Owner->IsActorTickEnabled())
	{
		bIsListening = false;
	}

	// If we have a task and it isn't finished, let it do it's thing
	int32 TaskStateValue = TaskState.GetValue();
	if (TaskStateValue == (int32)ETaskState::Processing)
	{
		return;
	}
	// Note this could be "NoTask" if this is the first tick or if the event listener has been stopped
	else if (TaskStateValue == (int32)ETaskState::Finished)
	{
		// Notify the callbacks with the filtered destruction data results if they're being listened to
		// If the data was changed during the task, then bChanged will be true and we will avoid broadcasting this frame since it won't be valid.
		if (bIsListening && !bChanged)
		{
			if (ChaosCollisionFilter.IsValid())
			{
				if (bIsCollisionEventListeningEnabled && ChaosCollisionFilter->GetNumEvents() > 0 && OnCollisionEvents.IsBound())
				{
					OnCollisionEvents.Broadcast(ChaosCollisionFilter->GetFilteredResults());
				}
			}

			if (ChaosBreakingFilter.IsValid())
			{
				if (bIsBreakingEventListeningEnabled && ChaosBreakingFilter->GetNumEvents() > 0 && OnBreakingEvents.IsBound())
				{
					OnBreakingEvents.Broadcast(ChaosBreakingFilter->GetFilteredResults());
				}
			}

			if (ChaosTrailingFilter.IsValid())
			{
				if (bIsTrailingEventListeningEnabled && ChaosTrailingFilter->GetNumEvents() > 0 && OnTrailingEvents.IsBound())
				{
					OnTrailingEvents.Broadcast(ChaosTrailingFilter->GetFilteredResults());
				}
			}
		}
		else
		{
			TaskState.Set((int32)ETaskState::NoTask);
		}

		// Reset the changed bool so we can broadcast next tick if the settings haven't changed
		bChanged = false;
	}

	// Early exit if we're not listening anymore
	if (!bIsListening)
	{
		return;
	}

	// If we don't have solvers, call update to make sure we have built our solver array
	if (!Solvers.Num())
	{
		UpdateSolvers();
	}

	if (!GeometryCollectionPhysicsProxies.Num())
	{
		UpdateGeometryCollectionPhysicsProxies();
	}

	// Reset our cached data arrays for various destruction types
	RawCollisionDataArray.Reset();
	RawBreakingDataArray.Reset();
	RawTrailingDataArray.Reset();

	// Retrieve the raw data arrays from the solvers
	GetDataFromSolvers();

	// Retrieve the raw data arrays from the GeometryCollectionPhysicsProxy
	GetDataFromGeometryCollectionPhysicsProxies();

	TaskState.Set((int32)ETaskState::Processing);

	// Retreive a copy of the transform before kicking off the task
	ChaosComponentTransform = GetComponentTransform();

	AsyncTask(ENamedThreads::AnyBackgroundThreadNormalTask,

		[this]()
		{
			if (bIsCollisionEventListeningEnabled)
			{
				if (ChaosCollisionFilter.IsValid())
				{
					ChaosCollisionFilter->FilterEvents(ChaosComponentTransform, RawCollisionDataArray);
				}
			}

			if (ChaosBreakingFilter.IsValid())
			{
				if (bIsBreakingEventListeningEnabled)
				{
					ChaosBreakingFilter->FilterEvents(ChaosComponentTransform, RawBreakingDataArray);
				}
			}

			if (ChaosTrailingFilter.IsValid())
			{
				if (bIsTrailingEventListeningEnabled)
				{
					ChaosTrailingFilter->FilterEvents(ChaosComponentTransform, RawTrailingDataArray);
				}
			}
			TaskState.Set((int32)ETaskState::Finished);
		});
#endif // if 0
}

void UChaosDestructionListener::AddChaosSolverActor(AChaosSolverActor* ChaosSolverActor)
{
#if 0 // solver actors no longer functional, using GetWorld()->GetPhysicsScene() instead
	if (ChaosSolverActor && !ChaosSolverActors.Contains(ChaosSolverActor))
	{
		ChaosSolverActors.Add(ChaosSolverActor);
		RegisterChaosEvents(ChaosSolverActor->GetPhysicsScene());
	}
#endif
}

void UChaosDestructionListener::RemoveChaosSolverActor(AChaosSolverActor* ChaosSolverActor)
{
#if 0 // solver actors no longer functional, using GetWorld()->GetPhysicsScene() instead
	if (ChaosSolverActor)
	{
		ClearEvents();
		ChaosSolverActors.Remove(ChaosSolverActor);
		UpdateEvents();
	}
#endif
}

void UChaosDestructionListener::AddGeometryCollectionActor(AGeometryCollectionActor* GeometryCollectionActor)
{
	if (GeometryCollectionActor && !GeometryCollectionActors.Contains(GeometryCollectionActor))
	{
		GeometryCollectionActors.Add(GeometryCollectionActor);
		if (const UGeometryCollectionComponent* GeometryCollectionComponent = GeometryCollectionActor->GetGeometryCollectionComponent())
		{
			if (const FGeometryCollectionPhysicsProxy* GeometryCollectionPhysicsObject = GeometryCollectionComponent->GetPhysicsProxy())
			{
				RegisterChaosEvents(GeometryCollectionComponent->GetWorld()->GetPhysicsScene());
			}
		}
	}
}

void UChaosDestructionListener::RemoveGeometryCollectionActor(AGeometryCollectionActor* GeometryCollectionActor)
{
	if (GeometryCollectionActor)
	{
		ClearEvents();
		GeometryCollectionActors.Remove(GeometryCollectionActor);
		UpdateEvents();
	}
}

void UChaosDestructionListener::SetCollisionEventRequestSettings(const FChaosCollisionEventRequestSettings& InSettings)
{
	CollisionEventRequestSettings = InSettings;
	UpdateTransformSettings();
}

void UChaosDestructionListener::SetBreakingEventRequestSettings(const FChaosBreakingEventRequestSettings& InSettings)
{
	BreakingEventRequestSettings = InSettings;
	UpdateTransformSettings();
}

void UChaosDestructionListener::SetTrailingEventRequestSettings(const FChaosTrailingEventRequestSettings& InSettings)
{
	TrailingEventRequestSettings = InSettings;
	UpdateTransformSettings();
}

void UChaosDestructionListener::SetRemovalEventRequestSettings(const FChaosRemovalEventRequestSettings& InSettings)
{
	RemovalEventRequestSettings = InSettings;
	UpdateTransformSettings();
}

void UChaosDestructionListener::SetCollisionEventEnabled(bool bIsEnabled)
{
	bIsCollisionEventListeningEnabled = bIsEnabled;
	UpdateTransformSettings();
}

void UChaosDestructionListener::SetBreakingEventEnabled(bool bIsEnabled)
{
	bIsBreakingEventListeningEnabled = bIsEnabled;
	UpdateTransformSettings();
}

void UChaosDestructionListener::SetTrailingEventEnabled(bool bIsEnabled)
{
	bIsTrailingEventListeningEnabled = bIsEnabled;
	UpdateTransformSettings();
}

void UChaosDestructionListener::SetRemovalEventEnabled(bool bIsEnabled)
{
	bIsRemovalEventListeningEnabled = bIsEnabled;
	UpdateTransformSettings();
}

void UChaosDestructionListener::SortCollisionEvents(TArray<FChaosCollisionEventData>& CollisionEvents, EChaosCollisionSortMethod SortMethod)
{
	if (ChaosCollisionFilter.IsValid())
	{
		ChaosCollisionFilter->SortEvents(CollisionEvents, SortMethod, GetComponentTransform());
	}
}

void UChaosDestructionListener::SortBreakingEvents(TArray<FChaosBreakingEventData>& BreakingEvents, EChaosBreakingSortMethod SortMethod)
{
	if (ChaosBreakingFilter.IsValid())
	{
		ChaosBreakingFilter->SortEvents(BreakingEvents, SortMethod, GetComponentTransform());
	}
}

void UChaosDestructionListener::SortTrailingEvents(TArray<FChaosTrailingEventData>& TrailingEvents, EChaosTrailingSortMethod SortMethod)
{
	if (ChaosTrailingFilter.IsValid())
	{
		ChaosTrailingFilter->SortEvents(TrailingEvents, SortMethod, GetComponentTransform());
	}
}

void UChaosDestructionListener::SortRemovalEvents(TArray<FChaosRemovalEventData>& RemovalEvents, EChaosRemovalSortMethod SortMethod)
{
	if (ChaosRemovalFilter.IsValid())
	{
		ChaosRemovalFilter->SortEvents(RemovalEvents, SortMethod, GetComponentTransform());
	}
}

void UChaosDestructionListener::RegisterChaosEvents(FPhysScene* Scene)
{
	Chaos::FPhysicsSolver* Solver = Scene->GetSolver();
	Chaos::FEventManager* EventManager = Solver->GetEventManager();
	EventManager->RegisterHandler<Chaos::FCollisionEventData>(Chaos::EEventType::Collision, this, &UChaosDestructionListener::HandleCollisionEvents);
	EventManager->RegisterHandler<Chaos::FBreakingEventData>(Chaos::EEventType::Breaking, this, &UChaosDestructionListener::HandleBreakingEvents);
	EventManager->RegisterHandler<Chaos::FTrailingEventData>(Chaos::EEventType::Trailing, this, &UChaosDestructionListener::HandleTrailingEvents);
	EventManager->RegisterHandler<Chaos::FRemovalEventData>(Chaos::EEventType::Removal, this, &UChaosDestructionListener::HandleRemovalEvents);
}

void UChaosDestructionListener::UnregisterChaosEvents(FPhysScene* Scene)
{
	Chaos::FPhysicsSolver* Solver = Scene->GetSolver();
	Chaos::FEventManager* EventManager = Solver->GetEventManager();
	EventManager->UnregisterHandler(Chaos::EEventType::Collision, this);
	EventManager->UnregisterHandler(Chaos::EEventType::Breaking, this);
	EventManager->UnregisterHandler(Chaos::EEventType::Trailing, this);
	EventManager->UnregisterHandler(Chaos::EEventType::Removal, this);
}

void UChaosDestructionListener::RegisterChaosEvents(TSharedPtr<FPhysScene_Chaos> Scene)
{
	Chaos::FPhysicsSolver* Solver = Scene->GetSolver();
	Chaos::FEventManager* EventManager = Solver->GetEventManager();
	EventManager->RegisterHandler<Chaos::FCollisionEventData>(Chaos::EEventType::Collision, this, &UChaosDestructionListener::HandleCollisionEvents);
	EventManager->RegisterHandler<Chaos::FBreakingEventData>(Chaos::EEventType::Breaking, this, &UChaosDestructionListener::HandleBreakingEvents);
	EventManager->RegisterHandler<Chaos::FTrailingEventData>(Chaos::EEventType::Trailing, this, &UChaosDestructionListener::HandleTrailingEvents);
	EventManager->RegisterHandler<Chaos::FRemovalEventData>(Chaos::EEventType::Removal, this, &UChaosDestructionListener::HandleRemovalEvents);
}

void UChaosDestructionListener::UnregisterChaosEvents(TSharedPtr<FPhysScene_Chaos> Scene)
{
	Chaos::FPhysicsSolver* Solver = Scene->GetSolver();
	Chaos::FEventManager* EventManager = Solver->GetEventManager();
	EventManager->UnregisterHandler(Chaos::EEventType::Collision, this);
	EventManager->UnregisterHandler(Chaos::EEventType::Breaking, this);
	EventManager->UnregisterHandler(Chaos::EEventType::Trailing, this);
	EventManager->UnregisterHandler(Chaos::EEventType::Removal, this);
}


void UChaosDestructionListener::HandleCollisionEvents(const Chaos::FCollisionEventData& Event)
{
	if (bIsCollisionEventListeningEnabled)
	{
		int NumCollisions = Event.CollisionData.AllCollisionsArray.Num();

		RawCollisionDataArray.Append(Event.CollisionData.AllCollisionsArray.GetData(), NumCollisions);

#if DISPATCH_BLUEPRINTS_IMMEDIATE
		if (ChaosCollisionFilter.IsValid())
		{
			ChaosCollisionFilter->FilterEvents(ChaosComponentTransform, RawCollisionDataArray);

			if (ChaosCollisionFilter->GetNumEvents() > 0 && OnCollisionEvents.IsBound())
			{
				OnCollisionEvents.Broadcast(ChaosCollisionFilter->GetFilteredResults());
			}
		}
		RawCollisionDataArray.Reset();
#endif
	}
}

void UChaosDestructionListener::HandleBreakingEvents(const Chaos::FBreakingEventData& Event)
{
	if (bIsBreakingEventListeningEnabled)
	{
		int NumBreakings = Event.BreakingData.AllBreakingsArray.Num();
		RawBreakingDataArray.Append(Event.BreakingData.AllBreakingsArray.GetData(), NumBreakings);

#if DISPATCH_BLUEPRINTS_IMMEDIATE
		if (ChaosBreakingFilter.IsValid())
		{
			ChaosBreakingFilter->FilterEvents(ChaosComponentTransform, RawBreakingDataArray);

			if (ChaosBreakingFilter->GetNumEvents() > 0 && OnBreakingEvents.IsBound())
			{
				OnBreakingEvents.Broadcast(ChaosBreakingFilter->GetFilteredResults());
			}
		}
		RawBreakingDataArray.Reset();
#endif
	}
}

void UChaosDestructionListener::HandleTrailingEvents(const Chaos::FTrailingEventData& Event)
{
	if (bIsTrailingEventListeningEnabled)
	{
		int NumTrailings = Event.TrailingData.AllTrailingsArray.Num();
		RawTrailingDataArray.Append(Event.TrailingData.AllTrailingsArray.GetData(), NumTrailings);

#if DISPATCH_BLUEPRINTS_IMMEDIATE
		if (ChaosTrailingFilter.IsValid())
		{
			ChaosTrailingFilter->FilterEvents(ChaosComponentTransform, RawTrailingDataArray);

			if (ChaosTrailingFilter->GetNumEvents() > 0 && OnTrailingEvents.IsBound())
			{
				OnTrailingEvents.Broadcast(ChaosTrailingFilter->GetFilteredResults());
			}
		}
		RawTrailingDataArray.Reset();
#endif
	}
}

void UChaosDestructionListener::HandleRemovalEvents(const Chaos::FRemovalEventData& Event)
{
	if (bIsRemovalEventListeningEnabled)
	{
		int NumRemovals = Event.RemovalData.AllRemovalArray.Num();
		RawRemovalDataArray.Append(Event.RemovalData.AllRemovalArray.GetData(), NumRemovals);

#if DISPATCH_BLUEPRINTS_IMMEDIATE
		if (ChaosRemovalFilter.IsValid())
		{
			ChaosRemovalFilter->FilterEvents(ChaosComponentTransform, RawRemovalDataArray);

			if (ChaosRemovalFilter->GetNumEvents() > 0 && OnRemovalEvents.IsBound())
			{
				OnRemovalEvents.Broadcast(ChaosRemovalFilter->GetFilteredResults());
			}
		}
		RawRemovalDataArray.Reset();
#endif
	}
}

