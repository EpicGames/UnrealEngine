// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosBreakingEventFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosBreakingEventFilter)

void FChaosBreakingEventFilter::FilterEvents(const FTransform& ChaosComponentTransform, const Chaos::FBreakingDataArray& RawBreakingDataArray)
{
	FilteredDataArray.Reset();

	check(BreakingEventRequestSettings);

	// Handle filtering and sorting of breaking events
	{
		float MaxDistSquared = BreakingEventRequestSettings->MaxDistance * BreakingEventRequestSettings->MaxDistance;

		// First build the output array by filtering the raw array
		for (const Chaos::FBreakingData& BreakingData : RawBreakingDataArray)
		{
			if (BreakingEventRequestSettings->MinSpeed > 0.0f && BreakingData.Velocity.Size() < BreakingEventRequestSettings->MinSpeed)
			{
				continue;
			}

			if (BreakingEventRequestSettings->MinMass > 0.0f && BreakingData.Mass < BreakingEventRequestSettings->MinMass)
			{
				continue;
			}

			if (BreakingEventRequestSettings->MaxDistance > 0.0f)
			{
				float DistSquared = FVector::DistSquared(BreakingData.Location, ChaosComponentTransform.GetLocation());
				if (DistSquared > MaxDistSquared)
				{
					continue;
				}
			}

			FChaosBreakingEventData NewData;
			NewData.Location = BreakingData.Location;
			NewData.Velocity = BreakingData.Velocity;
			NewData.Mass = BreakingData.Mass;

			FilteredDataArray.Add(NewData);

			// Check max results settings and early exit if we've reached that threshold
			if (BreakingEventRequestSettings->MaxNumberOfResults > 0 && FilteredDataArray.Num() == BreakingEventRequestSettings->MaxNumberOfResults)
			{
				break;
			}
		}

		SortEvents(FilteredDataArray, BreakingEventRequestSettings->SortMethod, ChaosComponentTransform);
	}
}

void FChaosBreakingEventFilter::SortEvents(TArray<FChaosBreakingEventData>& InOutBreakingEvents, EChaosBreakingSortMethod SortMethod, const FTransform& InTransform)
{
	struct FSortBreakingByMassMaxToMin
	{
		FORCEINLINE bool operator()(const FChaosBreakingEventData& Lhs, const FChaosBreakingEventData& Rhs) const
		{
			return Lhs.Mass > Rhs.Mass;
		}
	};

	struct FSortBreakingBySpeedMaxToMin
	{
		FORCEINLINE bool operator()(const FChaosBreakingEventData& Lhs, const FChaosBreakingEventData& Rhs) const
		{
			return Lhs.Velocity.Size() > Rhs.Velocity.Size();
		}
	};

	struct FSortBreakingByNearestFirst
	{
		FSortBreakingByNearestFirst(const FVector& InListenerLocation)
			: ListenerLocation(InListenerLocation)
		{}


		FORCEINLINE bool operator()(const FChaosBreakingEventData& Lhs, const FChaosBreakingEventData& Rhs) const
		{
			float DistSquaredLhs = FVector::DistSquared(Lhs.Location, ListenerLocation);
			float DistSquaredRhs = FVector::DistSquared(Rhs.Location, ListenerLocation);

			return DistSquaredLhs < DistSquaredRhs;
		}

		FVector ListenerLocation;
	};

	// Apply the sort
	switch (SortMethod)
	{
	case EChaosBreakingSortMethod::SortNone:
		break;

	case EChaosBreakingSortMethod::SortByHighestMass:
		InOutBreakingEvents.Sort(FSortBreakingByMassMaxToMin());
		break;

	case EChaosBreakingSortMethod::SortByHighestSpeed:
		InOutBreakingEvents.Sort(FSortBreakingBySpeedMaxToMin());
		break;

	case EChaosBreakingSortMethod::SortByNearestFirst:
		InOutBreakingEvents.Sort(FSortBreakingByNearestFirst(InTransform.GetLocation()));
		break;
	}
}

