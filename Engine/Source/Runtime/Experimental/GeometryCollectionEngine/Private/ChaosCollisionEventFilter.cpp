// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosCollisionEventFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosCollisionEventFilter)

void FChaosCollisionEventFilter::FilterEvents(const FTransform& ChaosComponentTransform, const Chaos::FCollisionDataArray& RawCollisionDataArray)
{
	FilteredDataArray.Reset();

	check(CollisionEventRequestSettings);

	// Handle filtering and sorting of collision events
	{
		const float MaxDistSquared = FMath::Square(CollisionEventRequestSettings->MaxDistance);
		const float MinSpeedSquared = FMath::Square(CollisionEventRequestSettings->MinSpeed);
		const float MinImpulseSquared = FMath::Square(CollisionEventRequestSettings->MinImpulse);

		// First build the output array by filtering the raw array
		for (const Chaos::FCollidingData& CollisionData : RawCollisionDataArray)
		{
			if (CollisionEventRequestSettings->MinMass > 0.0f)
			{
				const float MinMass = FMath::Min(CollisionData.Mass1, CollisionData.Mass2);
				if (MinMass < CollisionEventRequestSettings->MinMass)
				{
					continue;
				}
			}

			if (CollisionEventRequestSettings->MinSpeed > 0.0f)
			{
				const float MinSpeedSq = FMath::Min(CollisionData.Velocity1.SizeSquared(), CollisionData.Velocity2.SizeSquared());
				if (MinSpeedSq < MinSpeedSquared)
				{
					continue;
				}
			}

			if (CollisionEventRequestSettings->MinImpulse > 0.0f)
			{
				if (CollisionData.AccumulatedImpulse.SizeSquared() < MinImpulseSquared)
				{
					continue;
				}
			}

			if (CollisionEventRequestSettings->MaxDistance > 0.0f)
			{
				const float DistSquared = FVector::DistSquared(CollisionData.Location, ChaosComponentTransform.GetLocation());
				if (DistSquared > MaxDistSquared)
				{
					continue;
				}
			}

			FChaosCollisionEventData NewData;
			NewData.Location = CollisionData.Location;
			NewData.Normal = CollisionData.Normal;
			NewData.Velocity1 = CollisionData.Velocity1;
			NewData.Velocity2 = CollisionData.Velocity2;
			NewData.Mass1 = CollisionData.Mass1;
			NewData.Mass2 = CollisionData.Mass2;
			NewData.Impulse = CollisionData.AccumulatedImpulse;

			FilteredDataArray.Add(NewData);

			// Check max results settings and early exit if we've reached that threshold
			if (CollisionEventRequestSettings->MaxNumberResults > 0 && FilteredDataArray.Num() == CollisionEventRequestSettings->MaxNumberResults)
			{
				break;
			}
		}

		SortEvents(FilteredDataArray, CollisionEventRequestSettings->SortMethod, ChaosComponentTransform);
	}
}

void FChaosCollisionEventFilter::SortEvents(TArray<FChaosCollisionEventData>& InOutCollisionEvents, EChaosCollisionSortMethod SortMethod, const FTransform& InTransform)
{
	struct FSortCollisionByMassMaxToMin
	{
		FORCEINLINE bool operator()(const FChaosCollisionEventData& Lhs, const FChaosCollisionEventData& Rhs) const
		{
			return FMath::Max(Lhs.Mass1, Lhs.Mass2) > FMath::Max(Rhs.Mass1, Rhs.Mass2);
		}
	};

	struct FSortCollisionBySpeedMaxToMin
	{
		FORCEINLINE bool operator()(const FChaosCollisionEventData& Lhs, const FChaosCollisionEventData& Rhs) const
		{
			return FMath::Max(Lhs.Velocity1.Size(), Lhs.Velocity2.Size()) > FMath::Max(Rhs.Velocity1.Size(), Rhs.Velocity2.Size());
		}
	};

	struct FSortCollisionByHighestImpulse
	{
		FORCEINLINE bool operator()(const FChaosCollisionEventData& Lhs, const FChaosCollisionEventData& Rhs) const
		{
			return Lhs.Impulse.Size() > Rhs.Impulse.Size();
		}
	};

	struct FSortCollisionByNearestFirst
	{
		FSortCollisionByNearestFirst(const FVector& InListenerLocation)
			: ListenerLocation(InListenerLocation)
		{}

		FORCEINLINE bool operator()(const FChaosCollisionEventData& Lhs, const FChaosCollisionEventData& Rhs) const
		{
			float DistSquaredLhs = FVector::DistSquared(Lhs.Location, ListenerLocation);
			float DistSquaredRhs = FVector::DistSquared(Rhs.Location, ListenerLocation);

			return DistSquaredLhs < DistSquaredRhs;
		}

		FVector ListenerLocation;
	};

	switch (SortMethod)
	{
	case EChaosCollisionSortMethod::SortNone:
		break;

	case EChaosCollisionSortMethod::SortByHighestMass:
		InOutCollisionEvents.Sort(FSortCollisionByMassMaxToMin());
		break;

	case EChaosCollisionSortMethod::SortByHighestSpeed:
		InOutCollisionEvents.Sort(FSortCollisionBySpeedMaxToMin());
		break;

	case EChaosCollisionSortMethod::SortByHighestImpulse:
		InOutCollisionEvents.Sort(FSortCollisionByHighestImpulse());
		break;

	case EChaosCollisionSortMethod::SortByNearestFirst:
		InOutCollisionEvents.Sort(FSortCollisionByNearestFirst(InTransform.GetLocation()));
		break;
	}
}

