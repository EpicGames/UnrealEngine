// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosRemovalEventFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosRemovalEventFilter)

void FChaosRemovalEventFilter::FilterEvents(const FTransform& ChaosComponentTransform, const Chaos::FRemovalDataArray& RawRemovalDataArray)
{
	FilteredDataArray.Reset();

	{
		check(RemovalEventRequestSettings);

		// First filter the data 

		float MaxDistanceSquared = RemovalEventRequestSettings->MaxDistance * RemovalEventRequestSettings->MaxDistance;

		for (const Chaos::FRemovalData& RemovalData : RawRemovalDataArray)
		{
			if (RemovalEventRequestSettings->MinMass > 0.0f && RemovalData.Mass < RemovalEventRequestSettings->MinMass)
			{
				continue;
			}

			if (MaxDistanceSquared > 0.0f)
			{
				float DistSquared = FVector::DistSquared(RemovalData.Location, ChaosComponentTransform.GetLocation());
				if (DistSquared > MaxDistanceSquared)
				{
					continue;
				}
			}

			FChaosRemovalEventData NewData;
			NewData.Location = RemovalData.Location;
			NewData.Mass = RemovalData.Mass;
#if TODO_MAKE_A_BLUEPRINT_WAY_TO_ACCESS_PARTICLES
			NewData.ParticleIndex = RemovalData.Particle;
#endif

			FilteredDataArray.Add(NewData);

			if (RemovalEventRequestSettings->MaxNumberOfResults > 0 && FilteredDataArray.Num() == RemovalEventRequestSettings->MaxNumberOfResults)
			{
				break;
			}
		}

		SortEvents(FilteredDataArray, RemovalEventRequestSettings->SortMethod, ChaosComponentTransform);
	}
}

void FChaosRemovalEventFilter::SortEvents(TArray<FChaosRemovalEventData>& InOutRemovalEvents, EChaosRemovalSortMethod SortMethod, const FTransform& InTransform)
{
	struct FSortRemovalByMassMaxToMin
	{
		FORCEINLINE bool operator()(const FChaosRemovalEventData& Lhs, const FChaosRemovalEventData& Rhs) const
		{
			return Lhs.Mass > Rhs.Mass;
		}
	};

	struct FSortRemovalByNearestFirst
	{
		FSortRemovalByNearestFirst(const FVector& InListenerLocation)
			: ListenerLocation(InListenerLocation)
		{}

		FORCEINLINE bool operator()(const FChaosRemovalEventData& Lhs, const FChaosRemovalEventData& Rhs) const
		{
			float DistSquaredLhs = FVector::DistSquared(Lhs.Location, ListenerLocation);
			float DistSquaredRhs = FVector::DistSquared(Rhs.Location, ListenerLocation);

			return DistSquaredLhs < DistSquaredRhs;
		}

		FVector ListenerLocation;
	};

	// Apply the sort
	switch (SortMethod)
	{
	case EChaosRemovalSortMethod::SortNone:
		break;

	case EChaosRemovalSortMethod::SortByHighestMass:
		InOutRemovalEvents.Sort(FSortRemovalByMassMaxToMin());
		break;

	case EChaosRemovalSortMethod::SortByNearestFirst:
		InOutRemovalEvents.Sort(FSortRemovalByNearestFirst(InTransform.GetLocation()));
		break;
	}
}

