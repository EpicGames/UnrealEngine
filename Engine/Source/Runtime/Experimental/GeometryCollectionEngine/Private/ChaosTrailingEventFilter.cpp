// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosTrailingEventFilter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosTrailingEventFilter)

void FChaosTrailingEventFilter::FilterEvents(const FTransform& ChaosComponentTransform, const Chaos::FTrailingDataArray& RawTrailingDataArray)
{
	FilteredDataArray.Reset();

	{
		check(TrailingEventRequestSettings);

		// First filter the data 

		float MaxDistanceSquared = TrailingEventRequestSettings->MaxDistance * TrailingEventRequestSettings->MaxDistance;

		for (const Chaos::FTrailingData& TrailingData : RawTrailingDataArray)
		{
			if (TrailingEventRequestSettings->MinMass > 0.0f && TrailingData.Mass < TrailingEventRequestSettings->MinMass)
			{
				continue;
			}

			if (TrailingEventRequestSettings->MinSpeed > 0.0f && TrailingData.Velocity.Size() < TrailingEventRequestSettings->MinSpeed)
			{
				continue;
			}

			if (TrailingEventRequestSettings->MinAngularSpeed > 0.0f && TrailingData.AngularVelocity.Size() < TrailingEventRequestSettings->MinAngularSpeed)
			{
				continue;
			}

			if (MaxDistanceSquared > 0.0f)
			{
				float DistSquared = FVector::DistSquared(TrailingData.Location, ChaosComponentTransform.GetLocation());
				if (DistSquared > MaxDistanceSquared)
				{
					continue;
				}
			}

			FChaosTrailingEventData NewData;
			NewData.Location = TrailingData.Location;
			NewData.Velocity = TrailingData.Velocity;
			NewData.AngularVelocity = TrailingData.AngularVelocity;
			NewData.Mass = TrailingData.Mass;
#if TODO_MAKE_A_BLUEPRINT_WAY_TO_ACCESS_PARTICLES
			NewData.ParticleIndex = TrailingData.Particle;
#endif

			FilteredDataArray.Add(NewData);

			if (TrailingEventRequestSettings->MaxNumberOfResults > 0 && FilteredDataArray.Num() == TrailingEventRequestSettings->MaxNumberOfResults)
			{
				break;
			}
		}

		SortEvents(FilteredDataArray, TrailingEventRequestSettings->SortMethod, ChaosComponentTransform);
	}
}

void FChaosTrailingEventFilter::SortEvents(TArray<FChaosTrailingEventData>& InOutTrailingEvents, EChaosTrailingSortMethod SortMethod, const FTransform& InTransform)
{
	struct FSortTrailingByMassMaxToMin
	{
		FORCEINLINE bool operator()(const FChaosTrailingEventData& Lhs, const FChaosTrailingEventData& Rhs) const
		{
			return Lhs.Mass > Rhs.Mass;
		}
	};

	struct FSortTrailingBySpeedMaxToMin
	{
		FORCEINLINE bool operator()(const FChaosTrailingEventData& Lhs, const FChaosTrailingEventData& Rhs) const
		{
			return Lhs.Velocity.Size() > Rhs.Velocity.Size();
		}
	};

	struct FSortTrailingByNearestFirst
	{
		FSortTrailingByNearestFirst(const FVector& InListenerLocation)
			: ListenerLocation(InListenerLocation)
		{}

		FORCEINLINE bool operator()(const FChaosTrailingEventData& Lhs, const FChaosTrailingEventData& Rhs) const
		{
			float DistSquaredLhs = FVector::DistSquared(Lhs.Location, ListenerLocation);
			float DistSquaredRhs = FVector::DistSquared(Rhs.Location, ListenerLocation);

			return DistSquaredLhs < DistSquaredRhs;
		}

		FVector ListenerLocation;
	};

	// Apply the sort
	switch (SortMethod)
	{
	case EChaosTrailingSortMethod::SortNone:
		break;

	case EChaosTrailingSortMethod::SortByHighestMass:
		InOutTrailingEvents.Sort(FSortTrailingByMassMaxToMin());
		break;

	case EChaosTrailingSortMethod::SortByHighestSpeed:
		InOutTrailingEvents.Sort(FSortTrailingBySpeedMaxToMin());
		break;

	case EChaosTrailingSortMethod::SortByNearestFirst:
		InOutTrailingEvents.Sort(FSortTrailingByNearestFirst(InTransform.GetLocation()));
		break;
	}
}

