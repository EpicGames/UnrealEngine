// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/DerivedDataGeometryCollectionCooker.h"
#include "Chaos/Core.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionRenderData.h"
#include "Serialization/MemoryWriter.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "Chaos/ChaosArchive.h"
#include "UObject/DestructionObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "Chaos/ErrorReporter.h"
#include "EngineUtils.h"

#if WITH_EDITOR

#include "NaniteBuilder.h"

FDerivedDataGeometryCollectionCooker::FDerivedDataGeometryCollectionCooker(UGeometryCollection& InGeometryCollection)
	: GeometryCollection(InGeometryCollection)
{
}

FString FDerivedDataGeometryCollectionCooker::GetDebugContextString() const
{
	return GeometryCollection.GetFullName();
}

bool FDerivedDataGeometryCollectionCooker::Build(TArray<uint8>& OutData)
{
	FMemoryWriter Ar(OutData, true);	// Must be persistent for BulkData to serialize
	Chaos::FChaosArchive ChaosAr(Ar);
	if (FGeometryCollection* Collection = GeometryCollection.GetGeometryCollection().Get())
	{
		FSharedSimulationParameters SharedParams;
		GeometryCollection.GetSharedSimulationParams(SharedParams);

		Chaos::FErrorReporter ErrorReporter(GeometryCollection.GetName());

		BuildSimulationData(ErrorReporter, *Collection, SharedParams);
		// important : this is necessary to make sure we compute mass scale on the instances properly
		// sadly we cannot call this in BuildSimulationData because we have no access to the asset
		GeometryCollection.CacheMaterialDensity();
		Collection->Serialize(ChaosAr);

		#if 0
		if (ErrorReporter.EncounteredAnyErrors())
		{
			bool bAllErrorsHandled = !ErrorReporter.ContainsUnhandledError();
			ErrorReporter.ReportError(*FString::Printf(TEXT("Could not cook content for Collection:%s"), *GeometryCollection.GetPathName()));
			if (bAllErrorsHandled)
			{
				ErrorReporter.HandleLatestError();
			}
			return false;	//Don't save into DDC if any errors found
		}
		#endif

		return true;
	}

	return false;
}

const TCHAR* FDerivedDataGeometryCollectionCooker::GetVersionString() const
{
	const TCHAR* VersionString = TEXT("315169AFDF0046FBB2A69B70E42378E9");
	return VersionString;
}

FString FDerivedDataGeometryCollectionCooker::GetPluginSpecificCacheKeySuffix() const
{
	FString KeySuffix = FString::Printf(
		TEXT("%s_%s_%s_%d_%d"),
		Chaos::ChaosVersionGUID,
		*GeometryCollection.GetIdGuid().ToString(),
		*GeometryCollection.GetStateGuid().ToString(),
		FDestructionObjectVersion::Type::LatestVersion,
		FUE5MainStreamObjectVersion::Type::LatestVersion
	);

#if PLATFORM_CPU_ARM_FAMILY
	// Separate out arm keys as x64 and arm64 clang do not generate the same data for a given
	// input. Add the arm specifically so that a) we avoid rebuilding the current DDC and
	// b) we can remove it once we get arm64 to be consistent.
	KeySuffix.Append(TEXT("_arm64"));
#endif

	return KeySuffix;
}


FString FDerivedDataGeometryCollectionRenderDataCooker::GetDebugContextString() const
{
	return GeometryCollection.GetFullName();
}

bool FDerivedDataGeometryCollectionRenderDataCooker::Build(TArray<uint8>& OutData)
{
	FMemoryWriter Ar(OutData, true);	// Must be persistent for BulkData to serialize
	Chaos::FChaosArchive ChaosAr(Ar);
	if (FGeometryCollection* Collection = GeometryCollection.GetGeometryCollection().Get())
	{
		Chaos::FErrorReporter ErrorReporter(GeometryCollection.GetName());

		FGeometryCollectionRenderData::FCreateDesc CreateDesc = {
			.bEnableNanite = GeometryCollection.EnableNanite,
			.bEnableNaniteFallback = GeometryCollection.bEnableNaniteFallback,
			.bUseFullPrecisionUVs = GeometryCollection.bUseFullPrecisionUVs, 
			.bConvertVertexColorsToSRGB = GeometryCollection.bConvertVertexColorsToSRGB,
			.NaniteMinimumResidencyInKB = GeometryCollection.NaniteMinimumResidencyInKB
		};

		TUniquePtr<FGeometryCollectionRenderData> RenderData = FGeometryCollectionRenderData::Create(*Collection, CreateDesc);
		RenderData->Serialize(ChaosAr, GeometryCollection);

		#if 0
		if (ErrorReporter.EncounteredAnyErrors())
		{
			bool bAllErrorsHandled = !ErrorReporter.ContainsUnhandledError();
			ErrorReporter.ReportError(*FString::Printf(TEXT("Could not cook content for Collection:%s"), *GeometryCollection.GetPathName()));
			if (bAllErrorsHandled)
			{
				ErrorReporter.HandleLatestError();
			}
			return false;	//Don't save into DDC if any errors found
		}
		#endif

		return true;
	}

	return false;
}

const TCHAR* FDerivedDataGeometryCollectionRenderDataCooker::VersionString = TEXT("2087D33DBF524754A1F8FD20DCB7BEAB");

const TCHAR* FDerivedDataGeometryCollectionRenderDataCooker::GetVersionString() const
{
	static FString CachedNaniteVersionString;
	if (CachedNaniteVersionString.IsEmpty())
	{
		CachedNaniteVersionString = FString::Printf(TEXT("%s_%s"), VersionString, *Nanite::IBuilderModule::Get().GetVersionString());
	}

	return GeometryCollection.EnableNanite ? *CachedNaniteVersionString : VersionString;
}

FString FDerivedDataGeometryCollectionRenderDataCooker::GetPluginSpecificCacheKeySuffix() const
{
	FString KeySuffix = FString::Printf(
		TEXT("%s_%s_%s_%d_%d"),
		Chaos::ChaosVersionGUID,
		*GeometryCollection.GetIdGuid().ToString(),
		*GeometryCollection.GetStateGuid().ToString(),
		FDestructionObjectVersion::Type::LatestVersion,
		FUE5MainStreamObjectVersion::Type::LatestVersion
	);

#if PLATFORM_CPU_ARM_FAMILY
	// Separate out arm keys as x64 and arm64 clang do not generate the same data for a given
	// input. Add the arm specifically so that a) we avoid rebuilding the current DDC and
	// b) we can remove it once we get arm64 to be consistent.
	KeySuffix.Append(TEXT("_arm64"));
#endif

	return KeySuffix;
}

#endif // WITH_EDITOR
