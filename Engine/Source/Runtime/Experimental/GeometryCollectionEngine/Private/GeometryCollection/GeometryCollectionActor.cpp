// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
GeometryCollectionActor.cpp: AGeometryCollectionActor methods.
=============================================================================*/

#include "GeometryCollection/GeometryCollectionActor.h"

#include "Chaos/ChaosSolverActor.h"
#include "Chaos/Utilities.h"
#include "Chaos/Plane.h"
#include "Chaos/Box.h"
#include "Chaos/Sphere.h"
#include "Chaos/PerParticleGravity.h"
#include "Chaos/ImplicitObject.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/World.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "Math/Box.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/PhysicsInterfaceCore.h"
#include "PhysicsSolver.h"
#include "GeometryCollection/GeometryCollectionDebugDrawComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionActor)


DEFINE_LOG_CATEGORY_STATIC(AGeometryCollectionActorLogging, Log, All);

AGeometryCollectionActor::AGeometryCollectionActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	UE_LOGF(AGeometryCollectionActorLogging, Verbose, "AGeometryCollectionActor::AGeometryCollectionActor()");

	GeometryCollectionComponent = CreateDefaultSubobject<UGeometryCollectionComponent>(TEXT("GeometryCollectionComponent0"));
	RootComponent = GeometryCollectionComponent;

	GeometryCollectionDebugDrawComponent_DEPRECATED = nullptr;

	PrimaryActorTick.bCanEverTick = true;

	bReplicates = true;
	NetDormancy = DORM_Initial;
}

const Chaos::FPhysicsSolver* GetSolver(const AGeometryCollectionActor& GeomCollectionActor)
{
	return GeomCollectionActor.GetGeometryCollectionComponent()->ChaosSolverActor != nullptr ? GeomCollectionActor.GetGeometryCollectionComponent()->ChaosSolverActor->GetSolver() : GeomCollectionActor.GetWorld()->PhysicsScene_Chaos->GetSolver();
}

bool AGeometryCollectionActor::RaycastSingle(FVector Start, FVector End, FHitResult& OutHit) const
{
	// this is a deprecated function (deprecated starting 5.8 ) and has been a no-op return false since 2021
	return false;

}

#if WITH_EDITOR
bool AGeometryCollectionActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);

	if (GeometryCollectionComponent)
	{
		FGeometryCollectionEdit GeometryCollectionEdit = GeometryCollectionComponent->EditRestCollection(GeometryCollection::EEditUpdate::None);
		if (UGeometryCollection* GeometryCollection = GeometryCollectionEdit.GetRestCollection())
		{
			Objects.Add(GeometryCollection);
		}
	}
	return true;
}
#endif

