// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
GeometryCollectionActor.cpp: AGeometryCollectionActor methods.
=============================================================================*/

#include "GeometryCollection/GeometryCollectionActor.h"

#if 0

#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "Engine/SkeletalMesh.h"
#include "Math/Box.h"

#include "Physics/PhysicsInterfaceCore.h"
#include "Physics/ImmediatePhysics/ImmediatePhysicsActorHandle.h"
#include "Chaos/Utilities.h"
#include "Chaos/Plane.h"
#include "Chaos/Box.h"
#include "Chaos/Sphere.h"

DEFINE_LOG_CATEGORY_STATIC(AGeometryCollectionActorLogging, Log, All);

FTransform TransformMatrix(const FTransform& A, const FTransform& B) { return B * A; }

AGeometryCollectionActor::AGeometryCollectionActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bInitializedState(false)
	, RigidBodyIdArray(new TManagedArray<int32>())
	, CenterOfMassArray(new TManagedArray<FVector3f>())
{
	GeometryCollectionComponent = CreateDefaultSubobject<UGeometryCollectionComponent>(TEXT("GeometryCollectionComponent0"));
	RootComponent = GeometryCollectionComponent;
	AGeometryCollectionActor::bInitializedState = false;

	PrimaryActorTick.bCanEverTick = true;
	SetActorTickEnabled(true);
}

void AGeometryCollectionActor::Tick(float DeltaTime)
{
	UE_LOGF(AGeometryCollectionActorLogging, Verbose, "AGeometryCollectionActor::Tick()");

#if 0	//we don't access dynamic collection directly anymore
	UGeometryCollection* Collection = GeometryCollectionComponent->GetDynamicCollection();
	if (Collection && !AGeometryCollectionActor::bInitializedState)
	{
		Collection->GetGeometryCollection()->AddAttribute<int32>("RigidBodyID", FGeometryCollection::TransformGroup, RigidBodyIdArray);
		Collection->GetGeometryCollection()->AddAttribute<FVector3f>("CenterOfMass", FGeometryCollection::TransformGroup, CenterOfMassArray);

		Scene.SetKinematicUpdateFunction([this](FSolverCallbacks::FParticlesType& Particles, const float Dt, const float Time, const int32 Index) {

		});

		Scene.SetStartFrameFunction([this](const float StartFrame) {
			StartFrameCallback(StartFrame);
		});

		Scene.SetEndFrameFunction([this](const float EndFrame) {
			EndFrameCallback(EndFrame);
		});

		Scene.SetCreateBodiesFunction([this](FSolverCallbacks::FParticlesType& Particles) {
			CreateRigidBodyCallback(Particles);
		});

		Scene.SetParameterUpdateFunction([this](FSolverCallbacks::FParticlesType& Particles, const float, const int32 Index) {

		});

		Scene.SetDisableCollisionsUpdateFunction([this](TSet<TTuple<int32, int32>>&) {
		});

		Scene.AddPBDConstraintFunction([this](FSolverCallbacks::FParticlesType&, const float) {
		});

		Scene.AddForceFunction([this](FSolverCallbacks::FParticlesType& Particles, const float, const int32 Index) {
			Particles[Index]->AddForce(FVector(0, 0, -980.f));
		});

		bInitializedState = true;

		Scene.Init();
		UGeometryCollection* RestCollection = const_cast<UGeometryCollection*>(GeometryCollectionComponent->GetRestCollection());
		if (GeometryCollectionComponent->bClearCache)
		{
			RestCollection->RecordedTracks.Records.Reset();
		}
	}

	int32 NumTimeSteps = 1.f;
	float dt = GWorld->DeltaTimeSeconds / (float)NumTimeSteps;
	for (int i = 0; i < NumTimeSteps; i++)
	{
		Scene.Tick(dt);
	}
#endif
}

// void AGeometryCollectionActor::StartFrameCallback(float EndFrame)
// {
// #if 0	//we don't access dynamic collection directly anymore
// 	UE_LOGF(AGeometryCollectionActorLogging, Verbose, "AGeometryCollectionActor::StartFrameCallback()");
// 	UGeometryCollection* Collection = GeometryCollectionComponent->GetDynamicCollection();
// 	if (!Scene.GetSimulation()->NumActors() && Collection->GetGeometryCollection()->HasAttribute("RigidBodyID", FGeometryCollection::TransformGroup))
// 	{
// 		TManagedArray<int32> & RigidBodyId = *RigidBodyIdArray;
// 		TManagedArray<FVector> & CenterOfMass = *CenterOfMassArray;
// 
// 		TManagedArray<FTransform> & Transform = *Collection->GetGeometryCollection()->Transform;
// 		TManagedArray<int32> & BoneMap = *Collection->GetGeometryCollection()->BoneMap;
// 		TManagedArray<FVector> & Vertex = Collection->GetGeometryCollection()->Vertex;
// 
// 		PxMaterial* NewMaterial = GPhysXSDK->createMaterial(0, 0, 0);
// 
// 		// floor
// 		FTransform FloorTransform;
// 		PxRigidStatic* FloorActor = GPhysXSDK->createRigidStatic(U2PTransform(FTransform::Identity));
// 		PxShape* FloorShape = PxRigidActorExt::createExclusiveShape(*FloorActor, PxBoxGeometry(U2PVector(FVector(10000.f, 10000.f, 10.f))), *NewMaterial);
// 		// This breaks threading correctness in a general sense but is needed until we can call this in create rigid bodies
// 		const_cast<ImmediatePhysics::FSimulation*>(Scene.GetSimulation())->CreateStaticActor(FloorActor, FloorTransform);
// 
// 		FVector Scale = GeometryCollectionComponent->GetComponentTransform().GetScale3D();
// 
// 		TArray<FBox> Bounds;
// 		Bounds.AddZeroed(Collection->GetGeometryCollection()->NumElements(FGeometryCollection::TransformGroup));
// 
// 		TArray<int32> SurfaceParticlesCount;
// 		SurfaceParticlesCount.AddZeroed(Collection->GetGeometryCollection()->NumElements(FGeometryCollection::TransformGroup));
// 
// 		TArray<FVector> SumOfMass;
// 		SumOfMass.AddZeroed(Collection->GetGeometryCollection()->NumElements(FGeometryCollection::TransformGroup));
// 
// 		for (int i = 0; i < Vertex.Num(); i++)
// 		{
// 			FVector ScaledVertex = Scale * Vertex[i];
// 			int32 ParticleIndex = BoneMap[i];
// 			Bounds[ParticleIndex] += ScaledVertex;
// 			SurfaceParticlesCount[ParticleIndex]++;
// 			SumOfMass[ParticleIndex] += ScaledVertex;
// 		}
// 
// 
// 		for (int32 i = 0; i < Collection->GetGeometryCollection()->TransformNum(); ++i)
// 		{
// 			if (SurfaceParticlesCount[i] && 0.f < Bounds[i].GetSize().SizeSquared())
// 			{
// 				CenterOfMass[i] = SumOfMass[i] / SurfaceParticlesCount[i];
// 				Bounds[i] = Bounds[i].InverseTransformBy(FTransform(CenterOfMass[i]));
// 
// 				RigidBodyId[i] = i;
// 				int32 RigidBodyIndex = RigidBodyId[i];
// 
// 				FTransform NewTransform = TransformMatrix(GeometryCollectionComponent->GetComponentTransform(), Transform[i]);
// 				float SideSquared = Bounds[i].GetSize()[0] * Bounds[i].GetSize()[0] / 6.f;
// 
// 				PxRigidDynamic* NewActor = GPhysXSDK->createRigidDynamic(U2PTransform(FTransform::Identity));
// 				NewActor->setLinearVelocity(U2PVector(FVector(0.f, 0.f, 0.f)));
// 				NewActor->setAngularVelocity(U2PVector(FVector(0.f, 0.f, 0.f)));
// 				NewActor->setMass(1.f);
// 				NewActor->setMassSpaceInertiaTensor(U2PVector(FVector(SideSquared, SideSquared, SideSquared)));
// 				PxShape* NewShape = PxRigidActorExt::createExclusiveShape(*NewActor, PxBoxGeometry(U2PVector((Bounds[i].Max - Bounds[i].Min) / 2.f)), *NewMaterial);
// 				const_cast<ImmediatePhysics::FSimulation*>(Scene.GetSimulation())->CreateDynamicActor(NewActor, NewTransform);
// 			}
// 		}
// 	}
// #endif
// }
// 
// void AGeometryCollectionActor::CreateRigidBodyCallback(FSolverCallbacks::FParticlesType& Particles)
// {
// }
// 
// void AGeometryCollectionActor::EndFrameCallback(float EndFrame)
// {
// #if 0	//we don't use dynamic collection directly anymore
// 	UE_LOGF(AGeometryCollectionActorLogging, Log, "AGeometryCollectionActor::EndFrameFunction()");
// 	UGeometryCollection* Collection = GeometryCollectionComponent->GetDynamicCollection();
// 	if (Collection->GetGeometryCollection()->HasAttribute("RigidBodyID", FGeometryCollection::TransformGroup))
// 	{
// 		TManagedArray<int32> & RigidBodyId = *RigidBodyIdArray;
// 		TManagedArray<FVector> & CenterOfMass = *CenterOfMassArray;
// 		TManagedArray<FTransform> & Transform = *Collection->GetGeometryCollection()->Transform;
// 
// 		FTransform InverseComponentTransform = GeometryCollectionComponent->GetComponentTransform().Inverse();
// 		for (int i = 0; i < Collection->GetGeometryCollection()->NumElements(FGeometryCollection::TransformGroup); i++)
// 		{
// 			int32 RigidBodyIndex = RigidBodyId[i];
// 			Transform[i] = TransformMatrix(InverseComponentTransform, Scene.GetSimulation()->GetActorHandle(RigidBodyIndex)->GetWorldTransform());
// 		}
// 
// 		GeometryCollectionComponent->SetRenderStateDirty();
// 	}
// #endif
// }

bool AGeometryCollectionActor::RaycastSingle(FVector Start, FVector End, FHitResult& OutHit) const
{
	OutHit = FHitResult();
	OutHit.TraceStart = Start;
	OutHit.TraceEnd = End;
	return false;
}

//void AGeometryCollectionActor::UpdateKinematicBodiesCallback(FSolverCallbacks::FParticlesType& Particles, const float Dt, const float Time, const int32 Index) {}
//
//void AGeometryCollectionActor::ParameterUpdateCallback(FSolverCallbacks::FParticlesType& Particles, const float Time) {}
//
//void AGeometryCollectionActor::DisableCollisionsCallback(TSet<TTuple<int32, int32>>& CollisionPairs) {}
//
//void AGeometryCollectionActor::AddForceCallback(FSolverCallbacks::FParticlesType& Particles, const float Dt, const int32 Index) {}

#if WITH_EDITOR
bool AGeometryCollectionActor::GetReferencedContentObjects(TArray<UObject*>& Objects) const
{
	Super::GetReferencedContentObjects(Objects);
	return true;
}
#endif

#endif
