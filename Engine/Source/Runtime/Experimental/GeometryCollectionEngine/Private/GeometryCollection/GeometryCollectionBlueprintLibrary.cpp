// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionBlueprintLibrary.h"

#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionExternalRenderInterface.h"
#if WITH_EDITOR
#include "Engine/SCS_Node.h"
#include "Editor/EditorEngine.h"
#include "Engine/SimpleConstructionScript.h"
#endif
#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionBlueprintLibrary)

static IGeometryCollectionCustomDataInterface* GetCustomRenderer(UGeometryCollectionComponent* GeometryCollectionComponent)
{
	return GeometryCollectionComponent ? Cast<IGeometryCollectionCustomDataInterface>(GeometryCollectionComponent->GetCustomRenderer()) : nullptr;
}

void UGeometryCollectionBlueprintLibrary::SetCustomInstanceDataByIndex(UGeometryCollectionComponent* GeometryCollectionComponent, int32 CustomDataIndex, float CustomDataValue)
{
	if (IGeometryCollectionCustomDataInterface* CustomRenderer = GetCustomRenderer(GeometryCollectionComponent))
	{
		CustomRenderer->SetCustomInstanceData(CustomDataIndex, CustomDataValue);
	}
}

void UGeometryCollectionBlueprintLibrary::SetCustomInstanceDataByName(UGeometryCollectionComponent* GeometryCollectionComponent, FName CustomDataName, float CustomDataValue)
{
	if (IGeometryCollectionCustomDataInterface* CustomRenderer = GetCustomRenderer(GeometryCollectionComponent))
	{
		CustomRenderer->SetCustomInstanceData(CustomDataName, CustomDataValue);
	}
}

void UGeometryCollectionBlueprintLibrary::SetISMPoolCustomInstanceData(UGeometryCollectionComponent* GeometryCollectionComponent, int32 CustomDataIndex, float CustomDataValue)
{
	SetCustomInstanceDataByIndex(GeometryCollectionComponent, CustomDataIndex, CustomDataValue);
}

void UGeometryCollectionBlueprintLibrary::SetCustomDepthStencil(UGeometryCollectionComponent* GeometryCollectionComponent, bool bEnable, int32 CustomStencilValue)
{
	if (IGeometryCollectionCustomDataInterface* CustomRenderer = GetCustomRenderer(GeometryCollectionComponent))
	{
		CustomRenderer->SetRenderCustomDepthStencil(bEnable, CustomStencilValue);
		GeometryCollectionComponent->RefreshCustomRenderer();
	}
}

bool UGeometryCollectionBlueprintLibrary::AddGeometryCollectionsToBlueprint(UBlueprint* Blueprint, const TArray<UObject*>& Collections)
{
#if WITH_EDITOR
	for (UObject* Collection : Collections)
	{
		USCS_Node* NewNode = Blueprint->SimpleConstructionScript->CreateNode(Collection->GetClass(), Collection->GetFName());
		TArray<USCS_Node*> AllNodes = Blueprint->SimpleConstructionScript->GetAllNodes();
		if (AllNodes.Num() == 0)
		{
			Blueprint->SimpleConstructionScript->AddNode(NewNode);
		}
		else 
		{
			AllNodes[0]->AddChildNode(NewNode);
		}
		UEditorEngine::CopyPropertiesForUnrelatedObjects(Collection, NewNode->ComponentTemplate);
	}
	Blueprint->Modify();
#endif
	return true;
}