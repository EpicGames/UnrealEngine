// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionCache.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "HAL/IConsoleManager.h"
#include "Serialization/ArchiveCountMem.h"
#include "Features/IModularFeatures.h"
#include "UObject/AssetRegistryTagsContext.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionCache)

DEFINE_LOG_CATEGORY(LogGeometryCollectionCache);

FName UGeometryCollectionCache::TagName_Name = FName("CollectionName");
FName UGeometryCollectionCache::TagName_IdGuid = FName("CollectionIdGuid");
FName UGeometryCollectionCache::TagName_StateGuid = FName("CollectionStateGuid");

void UGeometryCollectionCache::SetFromRawTrack(const FRecordedTransformTrack& InTrack)
{
	ProcessRawRecordedDataInternal(InTrack);
	CompatibleCollectionState = SupportedCollection ? SupportedCollection->GetStateGuid() : FGuid();
}

void UGeometryCollectionCache::SetFromTrack(const FRecordedTransformTrack& InTrack)
{
	RecordedData = InTrack;
	CompatibleCollectionState = SupportedCollection ? SupportedCollection->GetStateGuid() : FGuid();
}

void UGeometryCollectionCache::SetSupportedCollection(const UGeometryCollection* InCollection)
{
	if(InCollection != SupportedCollection)
	{
		// New collection. Set it and then clear out recorded data
		SupportedCollection = InCollection;
		RecordedData.Records.Reset();
	}
}

void UGeometryCollectionCache::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UGeometryCollectionCache::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	Super::GetAssetRegistryTags(Context);

	Context.AddTag(FAssetRegistryTag(TagName_Name, SupportedCollection ? SupportedCollection->GetName() : FString("None"), FAssetRegistryTag::TT_Alphabetical));
	Context.AddTag(FAssetRegistryTag(TagName_IdGuid, SupportedCollection ? SupportedCollection->GetIdGuid().ToString() : FString("INVALID"), FAssetRegistryTag::TT_Hidden));
	Context.AddTag(FAssetRegistryTag(TagName_StateGuid, SupportedCollection ? CompatibleCollectionState.ToString() : FString("INVALID"), FAssetRegistryTag::TT_Hidden));
}

UGeometryCollectionCache* UGeometryCollectionCache::CreateCacheForCollection(const UGeometryCollection* InCollection)
{
	UGeometryCollectionCache* ResultCache = nullptr;

	if(InCollection)
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		if(ModularFeatures.IsModularFeatureAvailable(ITargetCacheProvider::GetFeatureName()))
		{
			ITargetCacheProvider* Provider = &ModularFeatures.GetModularFeature<ITargetCacheProvider>(ITargetCacheProvider::GetFeatureName());
			check(Provider);

			ResultCache = Provider->GetCacheForCollection(InCollection);

			if(ResultCache)
			{
				ResultCache->SetSupportedCollection(InCollection);
			}
		}
	}

	return ResultCache;
}

bool UGeometryCollectionCache::CompatibleWithForRecord(const UGeometryCollection* InCollection)
{
	check(InCollection);

	return InCollection == SupportedCollection;
}

bool UGeometryCollectionCache::CompatibleWithForPlayback(const UGeometryCollection* InCollection)
{
	check(InCollection);

	const bool bStateGuidValid = GetCompatibleStateGuid().IsValid();
	const bool bCollectionMatch = InCollection == SupportedCollection;
	const bool bStatesMatch = bStateGuidValid && GetCompatibleStateGuid() == InCollection->GetStateGuid();

	return bCollectionMatch && bStatesMatch;
}

void UGeometryCollectionCache::ProcessRawRecordedDataInternal(const FRecordedTransformTrack& InTrack)
{
	RecordedData = FRecordedTransformTrack::ProcessRawRecordedData(InTrack);
}

