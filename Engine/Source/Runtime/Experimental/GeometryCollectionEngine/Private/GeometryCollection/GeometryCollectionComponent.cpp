// Copyright Epic Games, Inc. All Rights Reserved.
#include "GeometryCollection/GeometryCollectionComponent.h"

#include "AI/Navigation/NavCollisionBase.h"
#include "AI/NavigationSystemHelpers.h"
#include "Async/ParallelFor.h"
#include "Chaos/ChaosPhysicalMaterial.h"
#include "Chaos/ChaosScene.h"
#include "Chaos/PhysicsObjectCollisionInterface.h"
#include "ChaosSolversModule.h"
#include "ChaosStats.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Components/BoxComponent.h"
#include "Engine/CollisionProfile.h"
#include "Engine/Engine.h"
#include "Engine/InstancedStaticMesh.h"
#include "Engine/StaticMeshSocket.h"
#include "MaterialDomain.h"
#include "Field/FieldSystemComponent.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionActor.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionCache.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionComponentPluginPrivate.h"
#include "GeometryCollection/GeometryCollectionDebugDrawComponent.h"
#include "GeometryCollection/GeometryCollectionISMPoolRenderer.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionProximityUtility.h"
#include "GeometryCollection/GeometryCollectionSQAccelerator.h"
#include "GeometryCollection/GeometryCollectionSceneProxy.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "ISMPool/ISMPoolActor.h"
#include "ISMPool/ISMPoolComponent.h"
#include "Math/Sphere.h"
#include "Modules/ModuleManager.h"
#include "Net/Core/PushModel/PushModel.h"
#include "Net/UnrealNetwork.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Physics/Experimental/ChaosInterfaceUtils.h"
#include "Physics/PhysicsFiltering.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "PhysicsField/PhysicsFieldComponent.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "PhysicsSolver.h"
#include "UObject/FortniteValkyrieBranchObjectVersion.h"
#include "Chaos/PBDRigidClusteringAlgo.h"

#include "Algo/RemoveIf.h"

#if WITH_EDITOR
#include "AssetToolsModule.h"
#include "Editor.h"
#include "UObject/UObjectThreadContext.h"
#endif

#if ENABLE_DRAW_DEBUG
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/ChaosDebugDraw.h"
#endif

#include "PhysicsEngine/BodySetup.h"
#include "PhysicsEngine/BodyInstance.h"
#include "Chaos/ChaosGameplayEventDispatcher.h"

#include "Rendering/NaniteResources.h"
#include "NaniteVertexFactory.h"
#include "PrimitiveSceneInfo.h"
#include "GeometryCollection/GeometryCollectionEngineRemoval.h"
#include "GeometryCollection/Facades/CollectionAnchoringFacade.h"
#include "GeometryCollection/Facades/CollectionRemoveOnBreakFacade.h"
#include "GeometryCollection/Facades/CollectionInstancedMeshFacade.h"
#include "GeometryCollection/GeometryCollectionExternalRenderInterface.h"
#include "Dataflow/DataflowSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionComponent)

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#endif //!(UE_BUILD_SHIPPING || UE_BUILD_TEST)

#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"

#if INTEL_ISPC

#if USING_CODE_ANALYSIS
MSVC_PRAGMA(warning(push))
MSVC_PRAGMA(warning(disable : ALL_CODE_ANALYSIS_WARNINGS))
#endif    // USING_CODE_ANALYSIS

#include "GeometryCollectionComponent.ispc.generated.h"

#if USING_CODE_ANALYSIS
MSVC_PRAGMA(warning(pop))
#endif    // USING_CODE_ANALYSIS

static_assert(sizeof(ispc::FMatrix) == sizeof(FMatrix), "sizeof(ispc::FMatrix) != sizeof(FMatrix)");
static_assert(sizeof(ispc::FBox) == sizeof(FBox), "sizeof(ispc::FBox) != sizeof(FBox)");
#endif

#if !defined(CHAOS_BOX_CALC_BOUNDS_ISPC_ENABLED_DEFAULT)
#define CHAOS_BOX_CALC_BOUNDS_ISPC_ENABLED_DEFAULT 1
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bChaos_BoxCalcBounds_ISPC_Enabled = INTEL_ISPC && CHAOS_BOX_CALC_BOUNDS_ISPC_ENABLED_DEFAULT;
#else
static bool bChaos_BoxCalcBounds_ISPC_Enabled = CHAOS_BOX_CALC_BOUNDS_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarChaosBoxCalcBoundsISPCEnabled(TEXT("p.Chaos.BoxCalcBounds.ISPC"), bChaos_BoxCalcBounds_ISPC_Enabled, TEXT("Whether to use ISPC optimizations in calculating box bounds in geometry collections"));
#endif

bool bChaos_GC_UseCustomRenderer = true;
FAutoConsoleVariableRef CVarChaosGCUseCustomRenderer(TEXT("p.Chaos.GC.UseCustomRenderer"), bChaos_GC_UseCustomRenderer, TEXT("When enabled, use a custom renderer if specified"));

bool bChaos_GC_InitConstantDataUseParallelFor = true;
FAutoConsoleVariableRef CVarChaosGCInitConstantDataUseParallelFor(TEXT("p.Chaos.GC.InitConstantDataUseParallelFor"), bChaos_GC_InitConstantDataUseParallelFor, TEXT("When enabled, InitConstant data will use parallelFor for copying some of the data"));

int32 bChaos_GC_InitConstantDataParallelForBatchSize = 5000;
FAutoConsoleVariableRef CVarChaosGCInitConstantDataParallelForBatchSize(TEXT("p.Chaos.GC.InitConstantDataParallelForBatchSize"), bChaos_GC_InitConstantDataParallelForBatchSize, TEXT("When parallelFor is used in InitConstantData, defined the minimium size of a batch of vertex "));

int32 MaxGeometryCollectionAsyncPhysicsTickIdleTimeMs = 30;
FAutoConsoleVariableRef CVarMaxGeometryCollectionAsyncPhysicsTickIdleTimeMs(TEXT("p.Chaos.GC.MaxGeometryCollectionAsyncPhysicsTickIdleTimeMs"), MaxGeometryCollectionAsyncPhysicsTickIdleTimeMs, TEXT("Amount of time in milliseconds before the async tick turns off when it is otherwise not doing anything."));

float GeometryCollectionRemovalMultiplier = 1.0f;
FAutoConsoleVariableRef CVarGeometryCollectionRemovalTimerMultiplier(TEXT("p.Chaos.GC.RemovalTimerMultiplier"), GeometryCollectionRemovalMultiplier, TEXT("Multiplier for the removal time evaluation ( > 1 : faster removal , > 1 slower"));

// temporary cvar , should be removed when the root event is no longer no necessary
bool GeometryCollectionEmitRootBreakingEvent = false;
FAutoConsoleVariableRef CVarGeometryCollectionEmitRootBreakingEvent(TEXT("p.Chaos.GC.EmitRootBreakingEvent"), GeometryCollectionEmitRootBreakingEvent, TEXT("When true send a breaking event when root is breaking"));

bool GeometryCollectionCreatePhysicsStateInEditor = false;
FAutoConsoleVariableRef CVarGeometryCollectionCreatePhysicsStateInEditor(TEXT("p.Chaos.GC.CreatePhysicsStateInEditor"), GeometryCollectionCreatePhysicsStateInEditor, TEXT("when on , physics state for a GC will be create in editor ( non PIE )"));

bool GeometryCollectionUseReplicationV2 = true;
FAutoConsoleVariableRef CVarGeometryCollectionUseReplicationV2(TEXT("p.Chaos.GC.UseReplicationV2"), GeometryCollectionUseReplicationV2, TEXT("When true use new replication data model"));

int32 GeometryCollectionNetAwakeningMode = 1;
FAutoConsoleVariableRef CVarGeometryCollectionNetAwakeningMode(TEXT("p.Chaos.GC.NetAwakeningMode"), GeometryCollectionNetAwakeningMode, TEXT("Changes how GC components ensure that their owner is awake for replication. 0 = ForceDormancyAwake, 1 = Use Flush Net Dormancy"));

bool bGeometryCollectionCustomRendererHiddenActorFix = true;
FAutoConsoleVariableRef CVarGeometryCollectionCustomRendererHiddenActorFix(TEXT("p.Chaos.GC.CustomRendererHiddenActorFix"), bGeometryCollectionCustomRendererHiddenActorFix, TEXT("When true custom renderer will account for the actor hidden flag"));

bool bChaos_GC_DestroyProxyOnSetSimulatePhysicsFalse = false;
FAutoConsoleVariableRef CVarChaosGCDestroyProxyOnSetSimulatePhysicsFalse(TEXT("p.Chaos.GC.DestroyProxyOnSetSimulatePhysicsFalse"), bChaos_GC_DestroyProxyOnSetSimulatePhysicsFalse, TEXT("When true, calling SetSimulatePhysics(false) on a Geometry Collection component destroys its physics proxy. Default false preserves legacy behavior of keeping the proxy alive."));

// this cvar enables by default the fix for the render issue happening when a cluster with children breaks from a remove on break cluster crumbling event 
// this should remain true unless an issue is found with it ( note this was introduce right after 5.5 )
bool bGeometryCollectionRemoveOnBreakDecayFix = true;
FAutoConsoleVariableRef CVarGeometryCollectionRemoveOnBreakDecayFix(TEXT("p.Chaos.GC.RemoveOnBreakDecayFix"), bGeometryCollectionRemoveOnBreakDecayFix, TEXT("When true, fix the issue causing a remove on break cluster crumbling with simulating children from disappearing for several frames"));

bool bUpdateProxyWorldTransformWhenSkippingPhysicsUpdate = true;
FAutoConsoleVariableRef CVarUpdateProxyWorldTransformWhenSkippingPhysicsUpdate(
	TEXT("p.Chaos.GC.UpdateProxyWorldTransformWhenSkippingPhysicsUpdate"), 
	bUpdateProxyWorldTransformWhenSkippingPhysicsUpdate, 
	TEXT("When true, bSkipPhysicsUpdate is true in OnUpdateTransform, the physics proxy will be requested to update it's world transform data internally - this fix issues when bUpdateComponentTransformToRootBone is enabled"));

DEFINE_LOG_CATEGORY_STATIC(UGCC_LOG, Error, All);
DEFINE_LOG_CATEGORY_STATIC(LogGeometryCollectionComponent, Warning, All);

extern FGeometryCollectionDynamicDataPool GDynamicDataPool;

static void GetGeometryCollectionComponentDebugDebugName(const UGeometryCollectionComponent& Comp, FString& DebugName)
{
	// Setup names
	// Make the debug name for this geometry...
	DebugName.Reset();

#if WITH_EDITOR
	if (Comp.GetOwner())
	{
		DebugName += FString::Printf(TEXT("Actor: '%s' "), *Comp.GetOwner()->GetActorLabel(false));
	}
#endif
	DebugName += FString::Printf(TEXT("Component: '%s' "), *Comp.GetPathName());
}

FString NetModeToString(ENetMode InMode)
{
	switch(InMode)
	{
	case ENetMode::NM_Client:
		return FString("Client");
	case ENetMode::NM_DedicatedServer:
		return FString("DedicatedServer");
	case ENetMode::NM_ListenServer:
		return FString("ListenServer");
	case ENetMode::NM_Standalone:
		return FString("Standalone");
	default:
		break;
	}

	return FString("INVALID NETMODE");
}

FString RoleToString(ENetRole InRole)
{
	switch(InRole)
	{
	case ROLE_None:
		return FString("None");
	case ROLE_SimulatedProxy:
		return FString("SimProxy");
	case ROLE_AutonomousProxy:
		return FString("AutoProxy");
	case ROLE_Authority:
		return FString("Auth");
	default:
		break;
	}

	return FString("Invalid Role");
}

int32 GetClusterLevel(const FTransformCollection* Collection, int32 TransformGroupIndex)
{
	int32 Level = 0;
	while(Collection && Collection->Parent[TransformGroupIndex] != -1)
	{
		TransformGroupIndex = Collection->Parent[TransformGroupIndex];
		Level++;
	}
	return Level;
}

bool FGeometryCollectionRepData::Identical(const FGeometryCollectionRepData* Other, uint32 PortFlags) const
{
	return Other && (Version == Other->Version);
}

bool FGeometryCollectionRepData::HasChanged(const FGeometryCollectionRepData& BaseData) const
{
	if((Version == 0) || (bIsRootAnchored != BaseData.bIsRootAnchored))
	{
		return true;
	}
	for (const FGeometryCollectionActivatedCluster& Data : OneOffActivated)
	{
		if (!BaseData.OneOffActivated.Contains(Data))
		{
			return true;
		}
	}
	
	for (const FGeometryCollectionClusterRep& Data : Clusters)
	{
		const FGeometryCollectionClusterRep* ExistingData = BaseData.Clusters.FindByPredicate(
			[&TransformIndex = Data.ClusterIdx](const FGeometryCollectionClusterRep& OtherData) ->
			bool { return OtherData.ClusterIdx == TransformIndex; });

		if(!ExistingData || (ExistingData && Data.ClusterChanged(*ExistingData)))
		{
			return true;
		}
	}
	return false;
}

UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(GeometryCollectionRepData);

bool FGeometryCollectionRepData::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	bOutSuccess = true;

	Ar << Version;

	Ar << OneOffActivated;

	Ar << ServerFrame;

	int32 NumClusters = Clusters.Num();
	Ar << NumClusters;

	Ar << bIsRootAnchored;

	if(Ar.IsLoading())
	{
		Clusters.SetNum(NumClusters);

		// Resetting this received time signals that this is the first frame that this
		// RepData will be processed.
		RepDataReceivedTime.Reset();
	}

	for(FGeometryCollectionClusterRep& Cluster : Clusters)
	{
		Ar << Cluster.Position;
		Ar << Cluster.LinearVelocity;
		Ar << Cluster.AngularVelocity;
		Ar << Cluster.Rotation;
		Ar << Cluster.ClusterIdx;
		Ar << Cluster.ClusterState.Value;
	}

	return true;
}

bool FGeometryCollectionRepStateData::SetBroken(int32 TransformIndex, int32 NumTransforms, bool bDisabled, const FVector& LinV, const FVector& AngVInRadiansPerSecond)
{
	if (BrokenState.Num() != NumTransforms)
	{
		BrokenState.SetNum(NumTransforms, false);
	}
	if (!BrokenState[TransformIndex])
	{
		BrokenState[TransformIndex] = true;

		FReleasedData Data;
		Data.TransformIndex = TransformIndex;
		Data.LinearVelocity = LinV;
		Data.AngularVelocityInDegreesPerSecond = FMath::RadiansToDegrees(AngVInRadiansPerSecond);
		ReleasedData.Emplace(Data);

		return true;
	}

	// breaking state already recorded, if it's disabled we can remove it from the ReleasedData
	// because it certainly went throug the removal stage and clients do not need to care about the initial velocity anymore
	if (bDisabled)
	{
		const int32 FoundDataIndex = ReleasedData.IndexOfByPredicate(
			[&TransformIndex](const FReleasedData& Data) -> bool { return (Data.TransformIndex == TransformIndex); });
		if (FoundDataIndex != INDEX_NONE)
		{
			ReleasedData.RemoveAtSwap(FoundDataIndex, EAllowShrinking::No);
		}
		// this does not have to be reported as a state change to save bandwidth
		return false;
	}

	return false;
}

bool FGeometryCollectionRepStateData::Identical(const FGeometryCollectionRepStateData* Other, uint32 PortFlags) const
{
	return Other && (Version == Other->Version);
}

bool  FGeometryCollectionRepStateData::HasChanged(const FGeometryCollectionRepStateData& BaseData) const
{
	// We are not using the ReleasedData to check if something has changed since this array is fully controlled by the brokenstate one
	return (Version == 0) || (BrokenState != BaseData.BrokenState) || (bIsRootAnchored != BaseData.bIsRootAnchored);
}

UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(GeometryCollectionRepStateData);

bool FGeometryCollectionRepStateData::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	bOutSuccess = true;

	Ar << Version;

	Ar << BrokenState;

	Ar << bIsRootAnchored;

	// we only support up to 2^16 bones for replication ( bandwidth reasons )
	ensure(ReleasedData.Num() < TNumericLimits<uint16>::Max());
	uint16 NumReleasedData = static_cast<uint16>(ReleasedData.Num());
	Ar << NumReleasedData;

	if (Ar.IsLoading())
	{
		ReleasedData.SetNumUninitialized(static_cast<int32>(NumReleasedData));
	}

	for (FReleasedData& Data : ReleasedData)
	{
		Ar << Data.TransformIndex;
		Ar << Data.LinearVelocity;
		Ar << Data.AngularVelocityInDegreesPerSecond;
	}

	return true;
}

bool FGeometryCollectionRepDynamicData::FClusterData::IsEqualPositionsAndVelocities(const FGeometryCollectionRepDynamicData::FClusterData& Data) const
{
	return Position.Equals(Data.Position)
		&& EulerRotation.Equals(Data.EulerRotation)
		&& LinearVelocity.Equals(Data.LinearVelocity)
		&& AngularVelocityInDegreesPerSecond.Equals(Data.AngularVelocityInDegreesPerSecond)
		;
}

bool FGeometryCollectionRepDynamicData::SetData(const FGeometryCollectionRepDynamicData::FClusterData& Data)
{
	FClusterData* ExistingData = ClusterData.FindByPredicate(
		[&TransformIndex = Data.TransformIndex](const FClusterData& Data) -> bool { return Data.TransformIndex == TransformIndex; });
	if (ExistingData)
	{
		const bool bHasChanged = !ExistingData->IsEqualPositionsAndVelocities(Data);
		if (bHasChanged)
		{
			*ExistingData = Data;
			ExistingData->LastUpdatedVersion = (Version + 1);
		}
		return bHasChanged;
	}
	FClusterData& NewData = ClusterData.Emplace_GetRef(Data);
	NewData.LastUpdatedVersion = (Version + 1);
	return true;
}

bool FGeometryCollectionRepDynamicData::RemoveOutOfDateClusterData()
{
	const int32 NumRemoved = ClusterData.RemoveAllSwap(
		[this](const FClusterData& Data) -> bool
		{
			return (Data.LastUpdatedVersion <= Version);
		},
		EAllowShrinking::No);

	return (NumRemoved > 0);
}

bool FGeometryCollectionRepDynamicData::Identical(const FGeometryCollectionRepDynamicData* Other, uint32 PortFlags) const
{
	return Other && (Version == Other->Version);
}

bool FGeometryCollectionRepDynamicData::HasChanged(const FGeometryCollectionRepDynamicData& BaseData) const
{
	if (Version == 0)
	{
		return true;
	}
	for (const FGeometryCollectionRepDynamicData::FClusterData& Data : ClusterData)
	{
		const FGeometryCollectionRepDynamicData::FClusterData* ExistingData = BaseData.ClusterData.FindByPredicate(
			[&TransformIndex = Data.TransformIndex](const FGeometryCollectionRepDynamicData::FClusterData& OtherData) ->
			bool { return OtherData.TransformIndex == TransformIndex; });

		if (!ExistingData || (ExistingData && !ExistingData->IsEqualPositionsAndVelocities(Data)))
		{
			return true;
		}
	}
	return false;
}

UE_NET_IMPLEMENT_NAMED_STRUCT_LASTRESORT_NETSERIALIZER_AND_REGISTRY_DELEGATES(GeometryCollectionRepDynamicData);

bool FGeometryCollectionRepDynamicData::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	Ar << Version;

	// we only support up to 2^16 bones for replication ( bandwidth reasons )
	ensure(ClusterData.Num() < TNumericLimits<uint16>::Max());
	uint16 NumClusterData = static_cast<uint16>(ClusterData.Num());
	Ar << NumClusterData;

	if (Ar.IsLoading())
	{
		ClusterData.SetNumUninitialized(static_cast<int32>(NumClusterData));
	}

	for (FClusterData& Data : ClusterData)
	{
		Ar << Data.TransformIndex;
		Ar << Data.bIsInternalCluster;
		Ar << Data.Position;
		Ar << Data.EulerRotation; // as rotator
		Ar << Data.LinearVelocity;
		Ar << Data.AngularVelocityInDegreesPerSecond;
	}

	return true;
}

int32 GGeometryCollectionNanite = 1;
FAutoConsoleVariableRef CVarGeometryCollectionNanite(
	TEXT("r.GeometryCollection.Nanite"),
	GGeometryCollectionNanite,
	TEXT("Render geometry collections using Nanite."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
	{
		FGlobalComponentRecreateRenderStateContext Context;
	}),
	ECVF_RenderThreadSafe
);

// Size in CM used as a threshold for whether a geometry in the collection is collected and exported for
// navigation purposes. Measured as the diagonal of the leaf node bounds.
float GGeometryCollectionNavigationSizeThreshold = 20.0f;
FAutoConsoleVariableRef CVarGeometryCollectionNavigationSizeThreshold(TEXT("p.GeometryCollectionNavigationSizeThreshold"), GGeometryCollectionNavigationSizeThreshold, TEXT("Size in CM used as a threshold for whether a geometry in the collection is collected and exported for navigation purposes. Measured as the diagonal of the leaf node bounds."));

// Single-Threaded Bounds
bool bGeometryCollectionSingleThreadedBoundsCalculation = false;
FAutoConsoleVariableRef CVarGeometryCollectionSingleThreadedBoundsCalculation(TEXT("p.GeometryCollectionSingleThreadedBoundsCalculation"), bGeometryCollectionSingleThreadedBoundsCalculation, TEXT("[Debug Only] Single threaded bounds calculation. [def:false]"));

FName UGeometryCollectionComponent::DefaultCollisionProfileName("InternalGCDefaultCollision");

FGeomComponentCacheParameters::FGeomComponentCacheParameters()
	: CacheMode(EGeometryCollectionCacheType::None)
	, TargetCache(nullptr)
	, ReverseCacheBeginTime(0.0f)
	, SaveCollisionData(false)
	, DoGenerateCollisionData(false)
	, CollisionDataSizeMax(512)
	, DoCollisionDataSpatialHash(false)
	, CollisionDataSpatialHashRadius(50.f)
	, MaxCollisionPerCell(1)
	, SaveBreakingData(false)
	, DoGenerateBreakingData(false)
	, BreakingDataSizeMax(512)
	, DoBreakingDataSpatialHash(false)
	, BreakingDataSpatialHashRadius(50.f)
	, MaxBreakingPerCell(1)
	, SaveTrailingData(false)
	, DoGenerateTrailingData(false)
	, TrailingDataSizeMax(512)
	, TrailingMinSpeedThreshold(200.f)
	, TrailingMinVolumeThreshold(10000.f)
{
}

#undef COPY_ON_WRITE_ATTRIBUTE
#define COPY_ON_WRITE_ATTRIBUTE(Type, Name, Group)										\
const TManagedArray<Type>& UGeometryCollectionComponent::Get##Name##Array() const 		\
{																						\
	return RestCollection->GetGeometryCollection()->Name;								\
}																						\
TManagedArray<Type>& UGeometryCollectionComponent::Get##Name##ArrayCopyOnWrite()		\
{																						\
	static FName StaticName(#Name);														\
	if (!DynamicCollection->HasAttribute(StaticName, Group))							\
	{																					\
		DynamicCollection->AddAttribute<Type>(StaticName, Group);						\
		DynamicCollection->CopyAttribute(												\
		    *RestCollection->GetGeometryCollection(), StaticName, Group);				\
	}																					\
	return RestCollection->GetGeometryCollection()->Name;								\
}																						\
void UGeometryCollectionComponent::Reset##Name##ArrayDynamic()							\
{																						\
}																						\
const TManagedArray<Type>& UGeometryCollectionComponent::Get##Name##ArrayRest() const	\
{																						\
	return RestCollection->GetGeometryCollection()->Name;								\
}																						\

#if !(UE_BUILD_SHIPPING)
static TAutoConsoleVariable<int> CVarNumToForceBreak(
	TEXT("r.GeometryCollection.CustomRenderer.ForceBreak"),
	-1,
	TEXT("Force the specified number of pieces to render individually, replacing their root proxy mesh.")
);

struct _ForcedBroken
{
public:
	void Reset()
	{
		Components.Reset();
		TotalTransforms = 0;
	}

	bool Break(UGeometryCollectionComponent* t, const TArray<FTransform3f>& CompSpaceTransforms)
	{
		if (Components.Contains(t))
			return true;

		if (TotalTransforms + CompSpaceTransforms.Num() > (uint32)CVarNumToForceBreak->GetInt())
			return false;

		TotalTransforms += CompSpaceTransforms.Num();
		Components.Add(t);
		return true;
	}

private:
	TSet<UGeometryCollectionComponent*> Components;
	uint32 TotalTransforms = 0;

}static ForcedBroken;
#endif

// Define the methods
COPY_ON_WRITE_ATTRIBUTES

int32 UGeometryCollectionComponent::GetParent(int32 Index) const
{
	if (DynamicCollection)
	{
		return DynamicCollection->GetParent(Index);
	}
	return RestCollection->GetGeometryCollection()->Parent[Index];
}

const TManagedArray<int32>& UGeometryCollectionComponent::GetParentArrayRest() const
{
	return RestCollection->GetGeometryCollection()->Parent;
}

UGeometryCollectionComponent::UGeometryCollectionComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ChaosSolverActor(nullptr)
	, InitializationState(ESimulationInitializationState::Unintialized)
	, ObjectType(EObjectStateTypeEnum::Chaos_Object_Dynamic)
	, GravityGroupIndex(0)
	, OneWayInteractionLevel(INDEX_NONE)
	, bDensityFromPhysicsMaterial(false)
	, bForceMotionBlur()
	, EnableClustering(true)
	, ClusterGroupIndex(0)
	, MaxClusterLevel(100)
	, MaxSimulatedLevel(100)
	, DamageThreshold({ 500000.f, 50000.f, 5000.f })
	, bUseSizeSpecificDamageThreshold(false)
	, bUseMaterialDamageModifiers(false)
	, bEnableDamageFromCollision(true)
	, bAllowRemovalOnSleep(true)
	, bAllowRemovalOnBreak(true)
	, ClusterConnectionType_DEPRECATED(EClusterConnectionTypeEnum::Chaos_MinimalSpanningSubsetDelaunayTriangulation)
	, CollisionGroup(0)
	, CollisionSampleFraction(1.0)
	, InitialVelocityType(EInitialVelocityTypeEnum::Chaos_Initial_Velocity_User_Defined)
	, InitialLinearVelocity(0.f, 0.f, 0.f)
	, InitialAngularVelocity(0.f, 0.f, 0.f)
	, BaseRigidBodyIndex(INDEX_NONE)
	, NumParticlesAdded(0)
	, CachePlayback(false)
	, bNotifyBreaks(false)
	, bNotifyCollisions(false)
	, bNotifyRemovals(false)
	, bNotifyCrumblings(false)
	, bCrumblingEventIncludesChildren(false)
	, bNotifyGlobalBreaks(false)
	, bNotifyGlobalCollisions(false)
	, bNotifyGlobalRemovals(false)
	, bNotifyGlobalCrumblings(false)
	, bGlobalCrumblingEventIncludesChildren(false)
	, bNotifyGlobalTrailings(false)
	, bStoreVelocities(false)
	, bShowBoneColors(false)
	, bUpdateComponentTransformToRootBone(false)
	, bUseRootProxyForNavigation(false)
	, bUpdateNavigationInTick(true) 
#if WITH_EDITORONLY_DATA 
	, bEnableRunTimeDataCollection(false)
	, RunTimeDataCollectionGuid(FGuid::NewGuid())
#endif
	, bForceBrokenForCustomRenderer(false)
	, bUpdateCustomRendererOnPostPhysicsSync(true)
	, bCustomRendererCanUseNativeFallback(false)
	, bCustomRendererShouldUseNativeFallback(false)
	, bForceNativeRenderer(false)
	, bEnableReplication(false)
	, bEnableAbandonAfterLevel(true)
	, AbandonedCollisionProfileName(UCollisionProfile::CustomCollisionProfileName)
	, ReplicationAbandonClusterLevel_DEPRECATED(0)
	, ReplicationAbandonAfterLevel(0)
	, ReplicationMaxPositionAndVelocityCorrectionLevel(100)
	, bInitializedRemovalDynamicAttribute(false)
	, bEnableBoneSelection(false)
	, IsObjectDynamic(false)
	, IsObjectLoading(true)
	, ViewLevel(-1)
	, NavmeshInvalidationTimeSliceIndex(0)
	, ComponentSpaceTransforms(this)
	, PhysicsProxy(nullptr)
#if WITH_EDITOR && WITH_EDITORONLY_DATA
	, EditorActor(nullptr)
#endif
#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	, bIsTransformSelectionModeEnabled(false)
#endif  // #if GEOMETRYCOLLECTION_EDITOR_SELECTION
{
	// by default tick is registered but disabled, we only need it when we need to update the removal timers
	// tick will be then enabled only when the root is broken from OnPostPhysicsSync callback
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = false;
	bTickInEditor = true;

	static uint32 GlobalNavMeshInvalidationCounter = 0;
	//space these out over several frames (3 is arbitrary)
	GlobalNavMeshInvalidationCounter += 3;
	NavmeshInvalidationTimeSliceIndex = GlobalNavMeshInvalidationCounter;

	// default current cache time
	CurrentCacheTime = MAX_flt;

	SetGenerateOverlapEvents(false);

	// By default use the destructible object channel unless the user specifies otherwise
	BodyInstance.SetObjectType(ECC_Destructible);

	// By default, we initialize immediately. If this is set false, we defer initialization.
	BodyInstance.bSimulatePhysics = true;

	if (!HasAnyFlags(RF_ArchetypeObject | RF_ClassDefaultObject))
	{
		EventDispatcher = NewObject<UChaosGameplayEventDispatcher>(this, TEXT("GameplayEventDispatcher"), RF_Transient);
	}

	DynamicCollection = nullptr;
	bHasCustomNavigableGeometry = EHasCustomNavigableGeometry::Yes;

	bWantsInitializeComponent = true;

	// make sure older asset are using the default behaviour
	DamagePropagationData.bEnabled = false;

#if !(UE_BUILD_SHIPPING)
	static auto ForcedBrokenDelegate = CVarNumToForceBreak->OnChangedDelegate().Add(FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* CVar)
	{
		ForcedBroken.Reset();

		for (TObjectIterator<UGeometryCollectionComponent> It; It; ++It)
		{
			It->RefreshCustomRenderer();
		}
	}));
#endif
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS // Disable warnings for implicit destruction of deprecated members.
UGeometryCollectionComponent::~UGeometryCollectionComponent()
{
	if (!ensureMsgf(
			!PhysicsProxy,
			TEXT("Geometry collection %s destroyed while still attached to physics proxy. "
				"BeginDestroy() might be overridden without a base class call."),
			*GetName()))
	{
		// Clear callback to avoid use-after-destruction via dangling pointer.
		PhysicsProxy->SetPostPhysicsSyncCallback(nullptr);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

Chaos::FPhysicsSolver* UGeometryCollectionComponent::GetSolver(const UGeometryCollectionComponent& GeometryCollectionComponent)
{
	if(GeometryCollectionComponent.ChaosSolverActor)
	{
		return GeometryCollectionComponent.ChaosSolverActor->GetSolver();
	}
	else if(UWorld* CurrentWorld = GeometryCollectionComponent.GetWorld())
	{
		if(FPhysScene* Scene = CurrentWorld->GetPhysicsScene())
		{
			return Scene->GetSolver();
		}
	}
	return nullptr;
}

void UGeometryCollectionComponent::BeginPlay()
{
	Super::BeginPlay();

#if WITH_EDITOR
	if (RestCollection != nullptr)
	{
		if (RestCollection->GetGeometryCollection()->HasAttribute("ExplodedVector", FGeometryCollection::TransformGroup))
		{
			RestCollection->GetGeometryCollection()->RemoveAttribute("ExplodedVector", FGeometryCollection::TransformGroup);
		}
	}
#endif

	// default current cache time
	CurrentCacheTime = MAX_flt;	

	RefreshCustomRenderer();
}


void UGeometryCollectionComponent::EndPlay(const EEndPlayReason::Type ReasonEnd)
{
#if WITH_EDITOR && WITH_EDITORONLY_DATA
	// Track our editor component if needed for syncing simulations back from PIE on shutdown
	EditorActor = EditorUtilities::GetEditorWorldCounterpartActor(GetTypedOuter<AActor>());
#endif

	Super::EndPlay(ReasonEnd);

	CurrentCacheTime = MAX_flt;
}

void UGeometryCollectionComponent::OnVisibilityChanged()
{
	Super::OnVisibilityChanged();
	RefreshCustomRenderer();
}

void UGeometryCollectionComponent::OnActorVisibilityChanged()
{
	Super::OnVisibilityChanged();
	RefreshCustomRenderer();
}

void UGeometryCollectionComponent::OnHiddenInGameChanged()
{
	Super::OnHiddenInGameChanged();
	RefreshCustomRenderer();
}

void UGeometryCollectionComponent::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
	
	/*
	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	Params.RepNotifyCondition = REPNOTIFY_OnChanged;
	DOREPLIFETIME_WITH_PARAMS_FAST(UGeometryCollectionComponent, RepData, Params);*/
	DOREPLIFETIME(UGeometryCollectionComponent, RepData);

	FDoRepLifetimeParams Params;
	Params.bIsPushBased = true;
	Params.RepNotifyCondition = REPNOTIFY_OnChanged;
	DOREPLIFETIME_WITH_PARAMS_FAST(UGeometryCollectionComponent, RepStateData, Params);
	DOREPLIFETIME_WITH_PARAMS_FAST(UGeometryCollectionComponent, RepDynamicData, Params);
}

void UGeometryCollectionComponent::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	if (EventDispatcher)
	{
		EventDispatcher->DestroyComponent();
		EventDispatcher = nullptr;
	}

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

namespace
{
	template <typename TTransformType>
	void UpdateGlobalMatricesWithExplodedVectors(TArray<TTransformType>& GlobalMatricesInOut, FGeometryCollection& GeometryCollection)
	{
		int32 NumMatrices = GlobalMatricesInOut.Num();
		if (GlobalMatricesInOut.Num() > 0)
		{
			if (GeometryCollection.HasAttribute("ExplodedVector", FGeometryCollection::TransformGroup))
			{
				const TManagedArray<FVector3f>& ExplodedVectors = GeometryCollection.GetAttribute<FVector3f>("ExplodedVector", FGeometryCollection::TransformGroup);

				if (!ensure(NumMatrices == ExplodedVectors.Num()))
				{
					return;
				}
				constexpr bool bIsMatrixType = std::is_same<TTransformType, FMatrix>::value;
				if constexpr (bIsMatrixType)
				{
					for (int32 TransformIndex = 0; TransformIndex < ExplodedVectors.Num(); ++TransformIndex)
					{
						GlobalMatricesInOut[TransformIndex] = GlobalMatricesInOut[TransformIndex].ConcatTranslation((FVector)ExplodedVectors[TransformIndex]);
					}
				}
				else
				{
					for (int32 TransformIndex = 0; TransformIndex < ExplodedVectors.Num(); ++TransformIndex)
					{
						GlobalMatricesInOut[TransformIndex].AddToTranslation((FVector3f)ExplodedVectors[TransformIndex]);
					}
				}
			}
		}
	}

	/**
	* compute the bounding box from the bounding boxes stored in the geometry group
	*/
	template <typename TWorldTransformType, typename TLocalTransformType>
	inline FBox ComputeBoundsFromGeometryBoundingBoxes(
		const TManagedArray<int32>& TransformToGeometryIndex,
		const TManagedArray<int32>& TransformIndices,
		const TManagedArray<FBox>& BoundingBoxes,
		const TArray<TLocalTransformType>& GlobalMatrices,
		const TWorldTransformType& LocalToWorldWithScale)
	{
		FBox BoundingBox(ForceInit);
		// todo(chaos ) implement ISPC function using FTransform 
		constexpr bool bIsMatrixType = std::is_same<TLocalTransformType, FMatrix>::value && std::is_same<TWorldTransformType, FMatrix>::value;
		if constexpr (bIsMatrixType)
		{
			if (bChaos_BoxCalcBounds_ISPC_Enabled && !bGeometryCollectionSingleThreadedBoundsCalculation)
			{
				ensure(BoundingBoxes.Num() > 0);
				ensure(TransformIndices.Num() == BoundingBoxes.Num());
				ensure(TransformToGeometryIndex.Num() > 0);
				ensure(TransformToGeometryIndex.Num() == GlobalMatrices.Num());

#if INTEL_ISPC
				ispc::BoxCalcBoundsFromGeometryGroup(
					(int32*)&TransformToGeometryIndex[0],
					(int32*)&TransformIndices[0],
					(ispc::FMatrix*)&GlobalMatrices[0],
					(ispc::FBox*)&BoundingBoxes[0],
					(ispc::FMatrix&)LocalToWorldWithScale,
					(ispc::FBox&)BoundingBox,
					BoundingBoxes.Num());
#endif
				return BoundingBox;
			}
		}

		// non-ISPC path
		for (int32 BoxIdx = 0; BoxIdx < BoundingBoxes.Num(); ++BoxIdx)
		{
			const int32 TransformIndex = TransformIndices[BoxIdx];
			
			if (TransformToGeometryIndex[TransformIndex] != INDEX_NONE)
			{
				BoundingBox += BoundingBoxes[BoxIdx].TransformBy(TWorldTransformType(GlobalMatrices[TransformIndex]) * LocalToWorldWithScale);
			}
		}
		return BoundingBox;
	}

	/**
	* compute the bounding box from the bounding boxes stored in the transform group
	* (used for nanite or when the geometry group data has been stripped on cook )
	*/
	template <typename TWorldTransformType, typename TLocalTransformType>
	inline FBox ComputeBoundsFromTransformBoundingBoxes(
		const TManagedArray<int32>& TransformToGeometryIndex,
		const TManagedArray<FBox>& BoundingBoxes,
		const TArray<TLocalTransformType>& GlobalMatrices,
		const TWorldTransformType& LocalToWorldWithScale)
	{
		FBox BoundingBox(ForceInit);

		// todo(chaos ) implement ISPC function using FTransform 
		constexpr bool bIsMatrixType = std::is_same<TWorldTransformType, FMatrix>::value && std::is_same<TLocalTransformType, FMatrix>::value;
		if constexpr (bIsMatrixType)
		{
			if (bChaos_BoxCalcBounds_ISPC_Enabled && !bGeometryCollectionSingleThreadedBoundsCalculation)
			{
#if INTEL_ISPC
				ispc::BoxCalcBoundsFromTransformGroup(
					(int32*)&TransformToGeometryIndex[0],
					(ispc::FMatrix*)&GlobalMatrices[0],
					(ispc::FBox*)&BoundingBoxes[0],
					(ispc::FMatrix&)LocalToWorldWithScale,
					(ispc::FBox&)BoundingBox,
					BoundingBoxes.Num());
#endif
				return BoundingBox;
			}
		}

		// non-ISPC path 
		for (int32 TransformIndex = 0; TransformIndex < BoundingBoxes.Num(); ++TransformIndex)
		{
			if (TransformToGeometryIndex[TransformIndex] != INDEX_NONE)
			{
				BoundingBox += BoundingBoxes[TransformIndex].TransformBy(TWorldTransformType(GlobalMatrices[TransformIndex]) * LocalToWorldWithScale);
			}
		}
		return BoundingBox;
	}

	template <typename TWorldTransformType, typename TLocalTransformType>
	inline FBox ComputeBoundsFromTransforms(const UGeometryCollectionComponent& Component, const TWorldTransformType& LocalToWorldWithScale, const TArray<TLocalTransformType>& GlobalMatricesArray)
	{
		static FName BoundingBoxAttributeName = "BoundingBox";

		auto GeometryCollectionPtr = Component.GetRestCollection()->GetGeometryCollection();
		const TManagedArray<FBox>* TransformBoundingBoxes = GeometryCollectionPtr->FindAttribute<FBox>(BoundingBoxAttributeName, FGeometryCollection::TransformGroup);
		const TManagedArray<FBox>& GeometryBoundingBoxes =  GeometryCollectionPtr->BoundingBox;
		const TManagedArray<int32>& TransformToGeometryIndex = GeometryCollectionPtr->TransformToGeometryIndex;

		if (TransformBoundingBoxes)
		{
			return ComputeBoundsFromTransformBoundingBoxes(TransformToGeometryIndex, *TransformBoundingBoxes, GlobalMatricesArray, LocalToWorldWithScale);
		}

		const TManagedArray<int32>& TransformIndices = GeometryCollectionPtr->TransformIndex;
		return ComputeBoundsFromGeometryBoundingBoxes(TransformToGeometryIndex, TransformIndices, GeometryBoundingBoxes, GlobalMatricesArray, LocalToWorldWithScale);
	}
}

FBox UGeometryCollectionComponent::ComputeBoundsFromGlobalMatrices(const FMatrix& LocalToWorldWithScale, const TArray<FMatrix>& GlobalMatricesArray) const
{
	return ComputeBoundsFromTransforms<FMatrix, FMatrix>(*this, LocalToWorldWithScale, GlobalMatricesArray);
}

FBox UGeometryCollectionComponent::ComputeBoundsFromComponentSpaceTransforms(const FTransform& LocalToWorldWithScale, const TArray<FTransform>& ComponentSpaceTransformsArray) const
{
	return ComputeBoundsFromTransforms<FTransform, FTransform>(*this, LocalToWorldWithScale, ComponentSpaceTransformsArray);
}

FBox UGeometryCollectionComponent::ComputeBoundsFromComponentSpaceTransforms(const FTransform& LocalToWorldWithScale, const TArray<FTransform3f>& ComponentSpaceTransformsArray) const
{
	return ComputeBoundsFromTransforms<FTransform, FTransform3f>(*this, LocalToWorldWithScale, ComponentSpaceTransformsArray);
}


FBox UGeometryCollectionComponent::ComputeBounds(const FMatrix& LocalToWorldWithScale) const
{
	return ComputeBounds(FTransform(LocalToWorldWithScale));
}

FBox UGeometryCollectionComponent::ComputeBounds(const FTransform& LocalToWorldWithScale) const
{
	FBox BoundingBox(ForceInit);
	if (RestCollection)
	{
		if (IsRootBroken())
		{
			// root is broken we need to go through all the leafs and compute bounds from them 
			const TArray<FTransform3f>& CompSpaceTransforms = ComponentSpaceTransforms.RequestAllTransforms();
			if (CompSpaceTransforms.Num() > 0)
			{
				BoundingBox = ComputeBoundsFromComponentSpaceTransforms(LocalToWorldWithScale, CompSpaceTransforms);
			}
		}
		else
		{
			// fast path where we use the root space bounds top avoid paying the high cost of recomputing everything from scratch
			if (!RootSpaceBounds.IsValid)
			{
				const TArray<FTransform3f>& CompSpaceTransforms = ComponentSpaceTransforms.RequestAllTransforms();
				if (CompSpaceTransforms.Num() > 0)
				{
					const FTransform InverseRootTransform(ComponentSpaceTransforms.RequestRootTransform().Inverse());
					RootSpaceBounds = ComputeBoundsFromComponentSpaceTransforms(InverseRootTransform, CompSpaceTransforms);
				}
			}

			if (RootSpaceBounds.IsValid)
			{
				const FTransform RootToWorld = (FTransform(ComponentSpaceTransforms.RequestRootTransform()) * LocalToWorldWithScale);
				BoundingBox = RootSpaceBounds.TransformBy(RootToWorld);
			}
		}
	}
	return BoundingBox;
}

FBoxSphereBounds UGeometryCollectionComponent::CalcBounds(const FTransform& LocalToWorldIn) const
{	
	SCOPE_CYCLE_COUNTER(STAT_GCCUpdateBounds);

	const bool NeedBoundsUpdate = (!ComponentSpaceBounds.IsValid) || CachePlayback;
	if (NeedBoundsUpdate)
	{
		ComponentSpaceBounds = ComputeBounds(FTransform::Identity);
	}

	return ComponentSpaceBounds.TransformBy(LocalToWorldIn);
}

int32 UGeometryCollectionComponent::GetNumElements(FName Group) const
{
	int32 Size = RestCollection->NumElements(Group);	//assume rest collection has the group and is connected to dynamic.
	return Size > 0 ? Size : DynamicCollection->NumElements(Group);	//if not, maybe dynamic has the group
}

void UGeometryCollectionComponent::SetDamageModel(EDamageModelTypeEnum InDamageModel)
{
	DamageModel = InDamageModel;
	if (PhysicsProxy)
	{
		PhysicsProxy->SetDamageModel_External(DamageModel);
	}
}

void UGeometryCollectionComponent::SetGravityGroupIndex(int32 InGravityGroupIndex)
{
	GravityGroupIndex = InGravityGroupIndex;
	if (PhysicsProxy)
	{
		PhysicsProxy->SetGravityGroupIndex_External(GravityGroupIndex);
	}
}

void UGeometryCollectionComponent::SetOneWayInteractionLevel(int32 InOneWayInteractionLevel)
{
	OneWayInteractionLevel = InOneWayInteractionLevel;
	if (PhysicsProxy)
	{
		PhysicsProxy->SetOneWayInteractionLevel_External(InOneWayInteractionLevel);
	}
}

void UGeometryCollectionComponent::SetDamageThreshold(const TArray<float>& InDamageThreshold)
{
	// NOTE: Should only call this during construction, not during runtime

	DamageThreshold = InDamageThreshold;

	if (PhysicsProxy)
	{
		PhysicsProxy->SetDamageThresholds_External(DamageThreshold);
	}
}

void UGeometryCollectionComponent::SetDamagePropagationData(const FGeometryCollectionDamagePropagationData& InDamagePropagationData)
{
	DamagePropagationData = InDamagePropagationData;

	if (PhysicsProxy)
	{
		PhysicsProxy->SetDamagePropagationData_External(DamagePropagationData.bEnabled, DamagePropagationData.BreakDamagePropagationFactor, DamagePropagationData.ShockDamagePropagationFactor);
	}
}

void UGeometryCollectionComponent::SetUseMaterialDamageModifiers(bool bInUseMaterialDamageModifiers)
{
	bUseMaterialDamageModifiers = bInUseMaterialDamageModifiers;

	if (PhysicsProxy)
	{
		PhysicsProxy->SetUseMaterialDamageModifiers_External(bUseMaterialDamageModifiers);
	}
}

void UGeometryCollectionComponent::SetDensityFromPhysicsMaterial(bool bInDensityFromPhysicsMaterial)
{
	bDensityFromPhysicsMaterial = bInDensityFromPhysicsMaterial;

	if (PhysicsProxy)
	{
		PhysicsProxy->SetMaterialOverrideMassScaleMultiplier_External(ComputeMassScaleRelativeToAsset());
	}
}

void UGeometryCollectionComponent::UpdateCachedBounds()
{
	ComponentSpaceBounds = ComputeBounds(FTransform::Identity);
	UpdateBounds();
}

bool UGeometryCollectionComponent::ShouldCreateRenderState() const
{
	// Called once on registration
	// Returning true here means that we pay overhead of render thread tasks for maintaining render state.
	// Try and return false whenever possible, but ONLY if there is no chance that we will want to create a scene proxy.
	return 
		!IsCustomRendererAvailable() ||  // Custom renderer replaces need for a scene proxy.
		bCustomRendererCanUseNativeFallback || // Except if custom renderer can require native rendering according to state.
		!GetWorld()->IsGameWorld(); // Editor worlds need to support fracture editor which forces native rendering.
}

void UGeometryCollectionComponent::CreateRenderState_Concurrent(FRegisterComponentContext* Context)
{
	Super::CreateRenderState_Concurrent(Context);
}

FPrimitiveSceneProxy* UGeometryCollectionComponent::CreateSceneProxy()
{
	static const auto NaniteProxyRenderModeVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Nanite.ProxyRenderMode"));
	const int32 NaniteProxyRenderMode = (NaniteProxyRenderModeVar != nullptr) ? (NaniteProxyRenderModeVar->GetInt() != 0) : 0;

	FPrimitiveSceneProxy* LocalSceneProxy = nullptr;

	if (RestCollection && !IsUsingCustomRenderer())
	{		
		if (CheckPSOPrecachingAndBoostPriority() && GetPSOPrecacheProxyCreationStrategy() == EPSOPrecacheProxyCreationStrategy::DelayUntilPSOPrecached)
		{
			UE_LOGF(LogGeometryCollectionComponent, Verbose, "Skipping CreateSceneProxy for UGeometryCollectionComponent %ls (UGeometryCollectionComponent PSOs are still compiling)", *GetFullName());
			return nullptr;
		}

		if (UseNanite(GetScene()->GetShaderPlatform()) &&
			RestCollection->EnableNanite &&
			RestCollection->HasNaniteData() &&
			GGeometryCollectionNanite != 0)
		{
			LocalSceneProxy = new FNaniteGeometryCollectionSceneProxy(this);
		}
		else if (RestCollection->HasMeshData())
		{
			LocalSceneProxy = new FGeometryCollectionSceneProxy(this);
		}
	}

	return LocalSceneProxy;
}

bool UGeometryCollectionComponent::ShouldCreatePhysicsState() const
{
	// Geometry collections always create physics state, not relying on the
	// underlying implementation that requires the body instance to decide
	return true;
}

bool UGeometryCollectionComponent::HasValidPhysicsState() const
{
	return PhysicsProxy != nullptr;
}

void UGeometryCollectionComponent::SetNotifyBreaks(bool bNewNotifyBreaks)
{
	if (bNotifyBreaks != bNewNotifyBreaks)
	{
		if (PhysicsProxy)
		{
			PhysicsProxy->SetNotifyBreakings_External(bNewNotifyBreaks);
		}
		bNotifyBreaks = bNewNotifyBreaks;
		RegisterForEvents();
	}
}

void UGeometryCollectionComponent::SetNotifyRemovals(bool bNewNotifyRemovals)
{
	if (bNotifyRemovals != bNewNotifyRemovals)
	{
		if (PhysicsProxy)
		{
			PhysicsProxy->SetNotifyRemovals_External(bNewNotifyRemovals);
		}
		bNotifyRemovals = bNewNotifyRemovals;
		RegisterForEvents();
	}
}

void UGeometryCollectionComponent::SetNotifyCrumblings(bool bNewNotifyCrumblings, bool bNewCrumblingEventIncludesChildren)
{
	if (bNotifyCrumblings != bNewNotifyCrumblings || 
		bCrumblingEventIncludesChildren != bNewCrumblingEventIncludesChildren)
	{
		if (PhysicsProxy)
		{
			PhysicsProxy->SetNotifyCrumblings_External(bNewNotifyCrumblings, bNewCrumblingEventIncludesChildren);
		}
		bNotifyCrumblings = bNewNotifyCrumblings;
		bCrumblingEventIncludesChildren = bNewCrumblingEventIncludesChildren;
		RegisterForEvents();
	}
}

void UGeometryCollectionComponent::SetNotifyTrailings(bool bNewNotifyTrailings)
{
	if (bNotifyTrailing != bNewNotifyTrailings)
	{
		if (PhysicsProxy)
		{
			PhysicsProxy->SetNotifyTrailings_External(bNewNotifyTrailings);
		}
		bNotifyTrailing = bNewNotifyTrailings;
		RegisterForEvents();
	}
}


void UGeometryCollectionComponent::SetNotifyGlobalBreaks(bool bNewNotifyGlobalBreaks)
{
	if (bNotifyGlobalBreaks != bNewNotifyGlobalBreaks)
	{
		if (PhysicsProxy)
		{
			PhysicsProxy->SetNotifyGlobalBreakings_External(bNewNotifyGlobalBreaks);
		}
		bNotifyGlobalBreaks = bNewNotifyGlobalBreaks;
	}
}

void UGeometryCollectionComponent::SetNotifyGlobalCollision(bool bNewNotifyGlobalCollisions)
{
	if (bNotifyGlobalCollisions != bNewNotifyGlobalCollisions)
	{
		bNotifyGlobalCollisions = bNewNotifyGlobalCollisions;
		UpdateGlobalCollisionEventRegistration();
	}
}

void UGeometryCollectionComponent::SetNotifyGlobalRemovals(bool bNewNotifyGlobalRemovals)
{
	if (bNotifyGlobalRemovals != bNewNotifyGlobalRemovals)
	{
		if (PhysicsProxy)
		{
			PhysicsProxy->SetNotifyGlobalRemovals_External(bNewNotifyGlobalRemovals);
		}
		bNotifyGlobalRemovals = bNewNotifyGlobalRemovals;
		UpdateGlobalRemovalEventRegistration();
	}
}

void UGeometryCollectionComponent::SetNotifyGlobalCrumblings(bool bNewNotifyGlobalCrumblings, bool bGlobalNewCrumblingEventIncludesChildren)
{
	if (bNotifyGlobalCrumblings != bNewNotifyGlobalCrumblings ||
		bGlobalCrumblingEventIncludesChildren != bGlobalNewCrumblingEventIncludesChildren)
	{
		if (PhysicsProxy)
		{
			PhysicsProxy->SetNotifyGlobalCrumblings_External(bNewNotifyGlobalCrumblings, bGlobalNewCrumblingEventIncludesChildren);
		}
		bNotifyGlobalCrumblings = bNewNotifyGlobalCrumblings;
		bGlobalCrumblingEventIncludesChildren = bGlobalNewCrumblingEventIncludesChildren;
	}
}

void UGeometryCollectionComponent::SetNotifyGlobalTrailings(bool bNewNotifyGlobalTrailings)
{
	if (bNotifyGlobalTrailings != bNewNotifyGlobalTrailings)
	{
		if (PhysicsProxy)
		{
			PhysicsProxy->SetNotifyGlobalTrailings_External(bNewNotifyGlobalTrailings);
		}
		bNotifyGlobalTrailings = bNewNotifyGlobalTrailings;
	}
}

FBodyInstance* UGeometryCollectionComponent::GetBodyInstance(FName BoneName /*= NAME_None*/, bool bGetWelded /*= true*/, int32 Index /*=INDEX_NONE*/) const
{
	return nullptr;
}

void UGeometryCollectionComponent::SetNotifyRigidBodyCollision(bool bNewNotifyRigidBodyCollision)
{
	Super::SetNotifyRigidBodyCollision(bNewNotifyRigidBodyCollision);
	UpdateRBCollisionEventRegistration();
}

bool UGeometryCollectionComponent::CanEditSimulatePhysics()
{
	return (RestCollection != nullptr);
}

void UGeometryCollectionComponent::SetSimulatePhysics(bool bEnabled)
{
	// make sure owner component is set to null before calling Super::SetSimulatePhysics
	// this will prevent unwanted log warning to trigger in BodyInstance::SetInstanceSimulatePhysics() because
	// in geometry collection , body instance never holds a valid physics handle
	const TWeakObjectPtr<UPrimitiveComponent> PreviousOwnerCOmponent = BodyInstance.OwnerComponent;
	{
		BodyInstance.OwnerComponent = nullptr;
		Super::SetSimulatePhysics(bEnabled);
		BodyInstance.OwnerComponent = PreviousOwnerCOmponent;
	}

	if (bEnabled && !PhysicsProxy && RestCollection)
	{
		EnsurePhysicsStateCreated();
		if (!PhysicsProxy && !RestCollection->IsEmpty() && DynamicCollection)
		{
			RegisterAndInitializePhysicsProxy();
		}
	}
	else if (!bEnabled && PhysicsProxy && bChaos_GC_DestroyProxyOnSetSimulatePhysicsFalse)
	{
		DestroyPhysicsState();
	}
}

void UGeometryCollectionComponent::SetEnableGravity(bool bGravityEnabled)
{
	if (bGravityEnabled != BodyInstance.bEnableGravity)
	{
		BodyInstance.bEnableGravity = bGravityEnabled;

		if (PhysicsProxy)
		{
			PhysicsProxy->SetEnableGravity_External(bGravityEnabled);
		}
	}
}

void UGeometryCollectionComponent::AddForce(FVector Force, FName BoneName, bool bAccelChange)
{
	ensure(bAccelChange == false); // not supported
	
	const FVector Direction = Force.GetSafeNormal(); 
	const FVector::FReal Magnitude = Force.Size();
	const FFieldSystemCommand Command = FFieldObjectCommands::CreateFieldCommand(EFieldPhysicsType::Field_LinearForce, new FUniformVector(Magnitude, Direction));
	DispatchFieldCommand(Command);
}


void UGeometryCollectionComponent::AddForceAtLocation(FVector Force, FVector WorldLocation, FName BoneName)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->ApplyForceAt_External(Force, WorldLocation);
	}
}

void UGeometryCollectionComponent::AddImpulse(FVector Impulse, FName BoneName, bool bVelChange)
{
	const FVector Direction = Impulse.GetSafeNormal(); 
	const FVector::FReal Magnitude = Impulse.Size();
	const EFieldPhysicsType FieldType = bVelChange? EFieldPhysicsType::Field_LinearVelocity: EFieldPhysicsType::Field_LinearImpulse;
	
	const FFieldSystemCommand Command = FFieldObjectCommands::CreateFieldCommand(FieldType, new FUniformVector(Magnitude, Direction));
	DispatchFieldCommand(Command);
}

void UGeometryCollectionComponent::AddImpulseAtLocation(FVector Impulse, FVector WorldLocation, FName BoneName)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->ApplyImpulseAt_External(Impulse, WorldLocation);
	}
	
}

TUniquePtr<FFieldNodeBase> MakeRadialField(const FVector& Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff)
{
	TUniquePtr<FFieldNodeBase> Field;
	if (Falloff == ERadialImpulseFalloff::RIF_Constant)
	{
		Field.Reset(new FRadialVector(Strength, Origin));
	}
	else
	{
		FRadialFalloff * FalloffField = new FRadialFalloff(Strength,0.f, 1.f, 0.f, Radius, Origin, EFieldFalloffType::Field_Falloff_Linear);
		FRadialVector* VectorField = new FRadialVector(1.f, Origin);
		Field.Reset(new FSumVector(1.0, FalloffField, VectorField, nullptr, Field_Multiply));
	}
	return Field;
}

void UGeometryCollectionComponent::AddRadialForce(FVector Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff, bool bAccelChange)
{
	ensure(bAccelChange == false); // not supported
	if(bIgnoreRadialForce)
	{
		return;
	}

	if (TUniquePtr<FFieldNodeBase> Field = MakeRadialField(Origin, Radius, Strength, Falloff))
	{
		const FFieldSystemCommand Command = FFieldObjectCommands::CreateFieldCommand(EFieldPhysicsType::Field_LinearForce, Field.Release());
		DispatchFieldCommand(Command);
	}
}

void UGeometryCollectionComponent::AddRadialImpulse(FVector Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff, bool bVelChange)
{
	if(bIgnoreRadialImpulse)
	{
		return;
	}

	if (TUniquePtr<FFieldNodeBase> Field = MakeRadialField(Origin, Radius, Strength, Falloff))
	{
		const EFieldPhysicsType FieldType = bVelChange? EFieldPhysicsType::Field_LinearVelocity: EFieldPhysicsType::Field_LinearImpulse;
		const FFieldSystemCommand Command = FFieldObjectCommands::CreateFieldCommand(FieldType, Field.Release());
		DispatchFieldCommand(Command);
	}
}

void UGeometryCollectionComponent::AddTorqueInRadians(FVector Torque, FName BoneName, bool bAccelChange)
{
	ensure(bAccelChange == false); // not supported
	
	const FVector Direction = Torque.GetSafeNormal(); 
	const FVector::FReal Magnitude = Torque.Size();
	const FFieldSystemCommand Command = FFieldObjectCommands::CreateFieldCommand(EFieldPhysicsType::Field_AngularTorque, new FUniformVector(Magnitude, Direction));
	DispatchFieldCommand(Command);
}

bool UGeometryCollectionComponent::IsFullyDecayed() const
{
	return bAlreadyFullyDecayed;
}

bool UGeometryCollectionComponent::FindLeafTransformByLineTrace(const FVector Start, const FVector End, int32& OutLeafTransformIndex, FName& OutLeafTransformName) const
{
	OutLeafTransformIndex = INDEX_NONE;
	OutLeafTransformName = NAME_None;

	if (!PhysicsProxy)
	{
		return false;
	}

	TArray<Chaos::FPhysicsObjectHandle> Objects = GetAllPhysicsObjects();
	FLockedReadPhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockRead(Objects);
	Objects = Objects.FilterByPredicate(
		[&Interface](Chaos::FPhysicsObjectHandle Handle)
		{

			return !Interface.GetInterface().HasChildren(Handle);
		}
	);

	bool bHaveHit = false;

	Chaos::FPhysicsObjectCollisionInterface_External CollisionInterface{ Interface.GetInterface() };
	ChaosInterface::FRaycastHit BestHit;
	if (CollisionInterface.LineTrace(Objects, Start, End, /*bTraceComplex*/false, BestHit))
	{
		if (BestHit.Actor)
		{
			if (const Chaos::FPBDRigidParticle* RigiParticle = BestHit.Actor->CastToRigidParticle())
			{
				const FGeometryCollectionItemIndex ItemIndex = PhysicsProxy->GetItemIndexFromGTParticleNoInternalCluster_External(RigiParticle);
				OutLeafTransformIndex = ItemIndex.GetItemIndex();
				OutLeafTransformName = PhysicsProxy->GetTransformName_External(ItemIndex);
				bHaveHit = true;
			}
		}
	}
	return bHaveHit;
}

void UGeometryCollectionComponent::DispatchBreakEvent(const FChaosBreakEvent& Event)
{
	// native
	NotifyBreak(Event);

	// bp
	if (OnChaosBreakEvent.IsBound())
	{
		OnChaosBreakEvent.Broadcast(Event);
	}
}

void UGeometryCollectionComponent::DispatchRemovalEvent(const FChaosRemovalEvent& Event)
{
	// native
	NotifyRemoval(Event);

	// bp
	if (OnChaosRemovalEvent.IsBound())
	{
		OnChaosRemovalEvent.Broadcast(Event);
	}
}

void UGeometryCollectionComponent::DispatchCrumblingEvent(const FChaosCrumblingEvent& Event)
{
	// bp
	if (OnChaosCrumblingEvent.IsBound())
	{
		OnChaosCrumblingEvent.Broadcast(Event);
	}
}

bool UGeometryCollectionComponent::DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const
{
	if(!RestCollection)
	{
		// No geometry data so skip export - geometry collections don't have other geometry sources
		// so return false here to skip non-custom export for this component as well.
		return false;
	}

	if (bUseRootProxyForNavigation)
	{
		const FGeometryCollectionProxyMeshData& ProxyMeshData = RestCollection->RootProxyData;

		bool bHasData = false;
		for (int32 MeshIndex = 0; MeshIndex < ProxyMeshData.ProxyMeshes.Num(); MeshIndex++)
		{
			const TObjectPtr<UStaticMesh>& ProxyMesh = ProxyMeshData.ProxyMeshes[MeshIndex];
			if (ProxyMesh != nullptr)
			{
				const int32 RootIndex = GetRootIndex();
				const FTransform& CompToWorld = GetComponentToWorld(); 
				const FTransform FinalTransform = (DynamicCollection ? FTransform(DynamicCollection->GetTransform(RootIndex)) : FTransform::Identity) * CompToWorld;
				const FVector Scale3D = FinalTransform.GetScale3D();
				const FTransform MeshLocalTransform{ ProxyMeshData.GetMeshTransform(MeshIndex) };

				if (!Scale3D.IsZero())
				{
					if (const UNavCollisionBase* NavCollision = ProxyMesh->GetNavCollision())
					{
						if (NavCollision->IsDynamicObstacle())
						{
							continue;
						}
						
						bHasData = NavCollision->ExportGeometry(MeshLocalTransform * FinalTransform, GeomExport) || bHasData;
					}
				}
			}
		}

		if (bHasData)
		{
			// skip default export
			return false;
		}

		return true;
	}

	if (RestCollection->bStripOnCook)
	{
		UE_LOGF(LogGeometryCollectionComponent, Log,
			"Collection data is set to be stripped on cook so there is nothing left to export. Make sure to use bUseRootProxyForNavigation with bStripOnCook for %ls.", *GetFullNameSafe(GetOwner()));
	}
	
	TArray<FVector> OutVertexBuffer;
	TArray<int32> OutIndexBuffer;

	const FGeometryCollection* const Collection = RestCollection->GetGeometryCollection().Get();
	check(Collection);

	const float SizeThreshold = GGeometryCollectionNavigationSizeThreshold * GGeometryCollectionNavigationSizeThreshold;

	// for all geometry. inspect bounding box build int list of transform indices.
	int32 VertexCount = 0;
	int32 FaceCountEstimate = 0;
	TArray<int32> GeometryIndexBuffer;
	TArray<int32> TransformIndexBuffer;

	int32 NumGeometry = Collection->NumElements(FGeometryCollection::GeometryGroup);

	const TManagedArray<FBox>& BoundingBox = Collection->BoundingBox;
	const TManagedArray<int32>& TransformIndexArray = Collection->TransformIndex;
	const TManagedArray<int32>& VertexCountArray = Collection->VertexCount;
	const TManagedArray<int32>& FaceCountArray = Collection->FaceCount;
	const TManagedArray<int32>& VertexStartArray = Collection->VertexStart;
	const TManagedArray<FVector3f>& Vertex = Collection->Vertex;

	for(int32 GeometryGroupIndex = 0; GeometryGroupIndex < NumGeometry; GeometryGroupIndex++)
	{
		if(BoundingBox[GeometryGroupIndex].GetSize().SizeSquared() > SizeThreshold)
		{
			TransformIndexBuffer.Add(TransformIndexArray[GeometryGroupIndex]);
			GeometryIndexBuffer.Add(GeometryGroupIndex);
			VertexCount += VertexCountArray[GeometryGroupIndex];
			FaceCountEstimate += FaceCountArray[GeometryGroupIndex];
		}
	}

	// Get all the geometry transforms in component space (they are stored natively in parent-bone space)
	const TArray<FTransform3f>& GeomToComponent = ComponentSpaceTransforms.RequestAllTransforms();

	OutVertexBuffer.AddUninitialized(VertexCount);

	int32 DestVertex = 0;
	//for each "subset" we care about 
	for(int32 SubsetIndex = 0; SubsetIndex < GeometryIndexBuffer.Num(); ++SubsetIndex)
	{
		//find indices into the collection data
		int32 GeometryIndex = GeometryIndexBuffer[SubsetIndex];
		int32 TransformIndex = TransformIndexBuffer[SubsetIndex];
		
		int32 SourceGeometryVertexStart = VertexStartArray[GeometryIndex];
		int32 SourceGeometryVertexCount = VertexCountArray[GeometryIndex];

		ParallelFor(SourceGeometryVertexCount, [&OutVertexBuffer, &GeomToComponent, &Vertex, SourceGeometryVertexStart, TransformIndex, DestVertex](int32 PointIdx)
			{
				//extract vertex from source
				int32 SourceGeometryVertexIndex = SourceGeometryVertexStart + PointIdx;
				const FVector3f VertexInComponentSpace = GeomToComponent[TransformIndex].TransformPosition(Vertex[SourceGeometryVertexIndex]);
				int32 DestVertexIndex = DestVertex + PointIdx;
				OutVertexBuffer[DestVertexIndex].X = VertexInComponentSpace.X;
				OutVertexBuffer[DestVertexIndex].Y = VertexInComponentSpace.Y;
				OutVertexBuffer[DestVertexIndex].Z = VertexInComponentSpace.Z;
			});

		DestVertex += SourceGeometryVertexCount;
	}

	//gather data needed for indices
	const TManagedArray<int32>& FaceStartArray = Collection->FaceStart;
	const TManagedArray<FIntVector>& Indices = Collection->Indices;
	const TManagedArray<bool>& Visible = Collection->Visible;
	const TManagedArray<int32>& MaterialIndex = Collection->MaterialIndex;

	//pre-allocate enough room (assuming all faces are visible)
	OutIndexBuffer.AddUninitialized(3 * FaceCountEstimate);

	//reset vertex counter so that we base the indices off the new location rather than the global vertex list
	DestVertex = 0;
	int32 DestinationIndex = 0;

	//leaving index traversal in a different loop to help cache coherency of source data
	for(int32 SubsetIndex = 0; SubsetIndex < GeometryIndexBuffer.Num(); ++SubsetIndex)
	{
		int32 GeometryIndex = GeometryIndexBuffer[SubsetIndex];

		//for each index, subtract the starting vertex for that geometry to make it 0-based.  Then add the new starting vertex index for this geometry
		int32 SourceGeometryVertexStart = VertexStartArray[GeometryIndex];
		int32 SourceGeometryVertexCount = VertexCountArray[GeometryIndex];
		int32 IndexDelta = DestVertex - SourceGeometryVertexStart;

		int32 FaceStart = FaceStartArray[GeometryIndex];
		int32 FaceCount = FaceCountArray[GeometryIndex];

		//Copy the faces
		for(int FaceIdx = FaceStart; FaceIdx < FaceStart + FaceCount; FaceIdx++)
		{
			if(Visible[FaceIdx])
			{
				OutIndexBuffer[DestinationIndex++] = Indices[FaceIdx].X + IndexDelta;
				OutIndexBuffer[DestinationIndex++] = Indices[FaceIdx].Y + IndexDelta;
				OutIndexBuffer[DestinationIndex++] = Indices[FaceIdx].Z + IndexDelta;
			}
		}

		DestVertex += SourceGeometryVertexCount;
	}

	// Invisible faces make the index buffer smaller
	OutIndexBuffer.SetNum(DestinationIndex);

	// Push as a custom mesh to navigation system
	// #CHAOSTODO This is pretty inefficient as it copies the whole buffer transforming each vert by the component to world
	// transform. Investigate a move aware custom mesh for pre-transformed verts to speed this up.
	GeomExport.ExportCustomMesh(OutVertexBuffer.GetData(), OutVertexBuffer.Num(), OutIndexBuffer.GetData(), OutIndexBuffer.Num(), GetComponentToWorld());

	return true;
}

UPhysicalMaterial* UGeometryCollectionComponent::GetPhysicalMaterial() const
{
	// Pull material from first mesh element to grab physical material. Prefer an override if one exists
	UPhysicalMaterial* PhysMatToUse = BodyInstance.GetSimplePhysicalMaterial();

	if(!PhysMatToUse || PhysMatToUse->GetFName() == "DefaultPhysicalMaterial" || PhysMatToUse == GEngine->DefaultPhysMaterial)
	{
		// No override, try render materials
		const int32 NumMaterials = GetNumMaterials();

		if(NumMaterials > 0)
		{
			UMaterialInterface* FirstMatInterface = GetMaterial(0);

			if(FirstMatInterface && FirstMatInterface->GetPhysicalMaterial())
			{
				PhysMatToUse = FirstMatInterface->GetPhysicalMaterial();
			}
		}
	}

	if ((!PhysMatToUse || PhysMatToUse == GEngine->DefaultPhysMaterial) && RestCollection)
	{
		PhysMatToUse = RestCollection->PhysicsMaterial;
	}

	if(!PhysMatToUse)
	{
		// Still no material, fallback on default
		PhysMatToUse = GEngine->DefaultPhysMaterial;
	}

	// Should definitely have a material at this point.
	check(PhysMatToUse);
	return PhysMatToUse;
}

void UGeometryCollectionComponent::RefreshEmbeddedGeometry()
{
	const int32 ExemplarCount = EmbeddedGeometryComponents.Num();
	if (ExemplarCount == 0)
	{
		return;
	}

	const TManagedArray<int32>& ExemplarIndexArray = GetRestCollection()->GetGeometryCollection()->ExemplarIndex;
	const int32 TransformCount = ComponentSpaceTransforms.Num();
	if (!ensureMsgf(TransformCount == ExemplarIndexArray.Num(), TEXT("GlobalMatrices (Num=%d) cached on GeometryCollectionComponent are not in sync with ExemplarIndexArray (Num=%d) on underlying GeometryCollection; likely missed a dynamic data update"), TransformCount, ExemplarIndexArray.Num()))
	{
		return;
	}
	
	const TManagedArray<bool>* HideArray = nullptr;
	if (RestCollection->GetGeometryCollection()->HasAttribute("Hide", FGeometryCollection::TransformGroup))
	{
		HideArray = &RestCollection->GetGeometryCollection()->GetAttribute<bool>("Hide", FGeometryCollection::TransformGroup);
	}

#if WITH_EDITOR	
	EmbeddedInstanceIndex.Init(INDEX_NONE, RestCollection->GetGeometryCollection()->NumElements(FGeometryCollection::TransformGroup));
#endif

	const TArray<FTransform3f>& CompSpaceTransforms = ComponentSpaceTransforms.RequestAllTransforms();

	for (int32 ExemplarIndex = 0; ExemplarIndex < ExemplarCount; ++ExemplarIndex)
	{		
#if WITH_EDITOR
		EmbeddedBoneMaps[ExemplarIndex].Empty(TransformCount);
		EmbeddedBoneMaps[ExemplarIndex].Reserve(TransformCount); // Allocate for worst case
#endif
		
		TArray<FTransform> InstanceTransforms;
		InstanceTransforms.Reserve(TransformCount); // Allocate for worst case

		// Construct instance transforms for this exemplar
		for (int32 Idx = 0; Idx < TransformCount; ++Idx)
		{
			if (ExemplarIndexArray[Idx] == ExemplarIndex)
			{
				if (!HideArray || !(*HideArray)[Idx])
				{ 
					InstanceTransforms.Add(FTransform(CompSpaceTransforms[Idx]));
#if WITH_EDITOR
					int32 InstanceIndex = EmbeddedBoneMaps[ExemplarIndex].Add(Idx);
					EmbeddedInstanceIndex[Idx] = InstanceIndex;
#endif
				}
			}
		}

		if (EmbeddedGeometryComponents[ExemplarIndex])
		{
			const int32 InstanceCount = EmbeddedGeometryComponents[ExemplarIndex]->GetInstanceCount();

			// If the number of instances has changed, we rebuild the structure.
			if (InstanceCount != InstanceTransforms.Num())
			{
				EmbeddedGeometryComponents[ExemplarIndex]->ClearInstances();
				EmbeddedGeometryComponents[ExemplarIndex]->PreAllocateInstancesMemory(InstanceTransforms.Num());
				for (const FTransform& InstanceTransform : InstanceTransforms)
				{
					EmbeddedGeometryComponents[ExemplarIndex]->AddInstance(InstanceTransform);
				}
				EmbeddedGeometryComponents[ExemplarIndex]->MarkRenderStateDirty();
			}
			else
			{
				// #todo (bmiller) When ISMC has been changed to be able to update transforms in place, we need to switch this function call over.

				EmbeddedGeometryComponents[ExemplarIndex]->BatchUpdateInstancesTransforms(0, InstanceTransforms, false, true, false);

				// EmbeddedGeometryComponents[ExemplarIndex]->UpdateKinematicTransforms(InstanceTransforms);
			}	
		}
	}
}

#if WITH_EDITOR
void UGeometryCollectionComponent::SetEmbeddedGeometrySelectable(bool bSelectableIn)
{
	for (TObjectPtr<UInstancedStaticMeshComponent>& EmbeddedGeometryComponent : EmbeddedGeometryComponents)
	{
		if (EmbeddedGeometryComponent)
		{
			EmbeddedGeometryComponent->bSelectable = bSelectable;
			EmbeddedGeometryComponent->bHasPerInstanceHitProxies = bSelectable;
		}
	}
}

void UGeometryCollectionComponent::ForceNativeRendering(bool bForce)
{
	if (bForce != bForceNativeRenderer)
	{
		bForceNativeRenderer = bForce;
		UnregisterCustomRenderer();
		RegisterCustomRenderer();
		RefreshCustomRenderer();
		MarkRenderStateDirty();
	}
}

int32 UGeometryCollectionComponent::EmbeddedIndexToTransformIndex(const UInstancedStaticMeshComponent* ISMComponent, int32 InstanceIndex) const
{
	for (int32 ISMIdx = 0; ISMIdx < EmbeddedGeometryComponents.Num(); ++ISMIdx)
	{
		if (EmbeddedGeometryComponents[ISMIdx].Get() == ISMComponent)
		{
			return (EmbeddedBoneMaps[ISMIdx][InstanceIndex]);
		}
	}

	return INDEX_NONE;
}
#endif

void UGeometryCollectionComponent::ResetRestTransforms()
{
	const bool bWasOverriden = (RestTransforms.Num() > 0);
	if (bWasOverriden)
	{
		RestTransforms.Empty();

		if (RestCollection && RestCollection->GetGeometryCollection())
		{
			if (DynamicCollection)
			{
				DynamicCollection->ResetInitialTransforms();
			}
			RestTransformsChanged();
		}
	}
}

void UGeometryCollectionComponent::SetRestState(TArray<FTransform>&& InRestTransforms)
{
	RestTransforms = InRestTransforms;
	SetInitialTransforms(RestTransforms);
	RestTransformsChanged();
}

void UGeometryCollectionComponent::RestTransformsChanged()
{
	if (SceneProxy)
	{
		OnTransformsDirty();

		// This is needed to make sure geometry Collections driven by a Chaos Cache in Static Pose renders and replay properly
		UpdateCachedBounds();
		MarkRenderTransformDirty();

		// force update the dynamic data 
		SendDynamicDataToSceneProxy();
	}
	else
	{
		// only need to mark transform dirty and let whoever needs it to compute it on demand
		OnTransformsDirty();
	}
	RootSpaceBounds.Init();

	RefreshEmbeddedGeometry();
	RefreshCustomRenderer();
}

void UGeometryCollectionComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);
	
	int32 SizeBytes =
		InitializationFields.GetAllocatedSize()
		+ DamageThreshold.GetAllocatedSize()
		+ RestTransforms.GetAllocatedSize()
		+ DisabledFlags.GetAllocatedSize()
		+ CollisionProfilePerLevel.GetAllocatedSize()
		+ ComponentSpaceTransforms.GetAllocatedSize()
		+ EventsPlayed.GetAllocatedSize()
		+ EmbeddedGeometryComponents.GetAllocatedSize()
		+ (ClustersToRep ? ClustersToRep->GetAllocatedSize() : 0);

#if WITH_EDITORONLY_DATA
	SizeBytes +=
		+ SelectedBones.GetAllocatedSize()
		+ HighlightedBones.GetAllocatedSize()
		+ EmbeddedInstanceIndex.GetAllocatedSize()
		+ EmbeddedBoneMaps.GetAllocatedSize();

	for (TArray<int32>& BoneMap : EmbeddedBoneMaps)
	{
		SizeBytes += BoneMap.GetAllocatedSize();
	}
#endif

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(SizeBytes);
}

bool UGeometryCollectionComponent::IsNavigationRelevant() const
{
	return bIsCurrentlyNavigationRelevant && Super::IsNavigationRelevant();
}

#if WITH_EDITOR
FDelegateHandle UGeometryCollectionComponent::RegisterOnGeometryCollectionPropertyChanged(const FOnGeometryCollectionPropertyChanged& Delegate)
{
	return OnGeometryCollectionPropertyChanged.Add(Delegate);
}

void UGeometryCollectionComponent::UnregisterOnGeometryCollectionPropertyChanged(FDelegateHandle Handle)
{
	OnGeometryCollectionPropertyChanged.Remove(Handle);
}

void UGeometryCollectionComponent::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UGeometryCollectionComponent, bShowBoneColors))
	{
		FScopedColorEdit EditBoneColor(this, true /*bForceUpdate*/); // the property has already changed; this will trigger the color update + render state updates
	}
}

void UGeometryCollectionComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (OnGeometryCollectionPropertyChanged.IsBound())
	{
		OnGeometryCollectionPropertyChanged.Broadcast();
	}

	if (PropertyChangedEvent.Property && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UGeometryCollectionComponent, ChaosSolverActor))
	{
		// call the setter method to make sure everything is proeprly reset
		SetSolverActor(ChaosSolverActor);
	}
}
#endif

static void DispatchGeometryCollectionBreakEvent(const FChaosBreakEvent& Event)
{
	if (UGeometryCollectionComponent* const GC = Cast<UGeometryCollectionComponent>(Event.Component))
	{
		GC->DispatchBreakEvent(Event);
	}
}

static void DispatchGeometryCollectionRemovalEvent(const FChaosRemovalEvent& Event)
{
	if (UGeometryCollectionComponent* const GC = Cast<UGeometryCollectionComponent>(Event.Component))
	{
		GC->DispatchRemovalEvent(Event);
	}
}

static void DispatchGeometryCollectionCrumblingEvent(const FChaosCrumblingEvent& Event)
{
	if (UGeometryCollectionComponent* const GC = Cast<UGeometryCollectionComponent>(Event.Component))
	{
		GC->DispatchCrumblingEvent(Event);
	}
}

// DEPRECATED Use GetComponentSpaceTransforms3f instead
TArray<FTransform> UGeometryCollectionComponent::GetComponentSpaceTransforms()
{ 
	const TArray<FTransform3f>& Transforms3f = GetComponentSpaceTransforms3f();
	TArray<FTransform> Transforms;
	Transforms.Reserve(Transforms3f.Num());
	Algo::Transform(Transforms3f, Transforms, [](const FTransform3f& T) { return FTransform(T); });
	return Transforms;
}

const TArray<FTransform3f>& UGeometryCollectionComponent::GetComponentSpaceTransforms3f()
{
	return ComponentSpaceTransforms.RequestAllTransforms();
}

const FGeometryDynamicCollection* UGeometryCollectionComponent::GetDynamicCollection() const
{
	return DynamicCollection.Get();
}

FGeometryDynamicCollection* UGeometryCollectionComponent::GetDynamicCollection()
{
	return DynamicCollection.Get();
}

void UGeometryCollectionComponent::DispatchChaosPhysicsCollisionBlueprintEvents(const FChaosPhysicsCollisionInfo& CollisionInfo)
{
	ReceivePhysicsCollision(CollisionInfo);
	OnChaosPhysicsCollision.Broadcast(CollisionInfo);
}

// call when first registering
void UGeometryCollectionComponent::RegisterForEvents()
{
	UWorld* World = GetWorld();
	if (!World)
	{
		return;
	}

	FPhysScene* Scene = World->GetPhysicsScene();
	if (!Scene)
	{
		return;
	}

	Chaos::FPhysicsSolver* Solver = Scene->GetSolver();
	if (Solver)
	{
		if (EventDispatcher)
		{
			// The new interested proxy system relies on us having having a proxy registered with the scene at the time we register Chaos events.
			// Hence we need to do a double-take here to force the re-registration of chaos events. This is guaranteed to happen every time we
			// re-create the physics proxy (e.g. in the case of the construction script in PIE). In that case, UChaosGameplayEventDispatcher::OnRegister
			// will run *before* the new physics proxy is created.
			EventDispatcher->UnregisterChaosEvents();

			if (BodyInstance.bNotifyRigidBodyCollision || bNotifyBreaks || bNotifyCollisions || bNotifyRemovals || bNotifyCrumblings)
			{
				if (bNotifyCollisions || BodyInstance.bNotifyRigidBodyCollision)
				{
					EventDispatcher->RegisterForCollisionEvents(this, this);

					Solver->EnqueueCommandImmediate([Solver]()
						{
							Solver->SetGenerateCollisionData(true);
						});
				}

				if (bNotifyBreaks)
				{
					EventDispatcher->RegisterForBreakEvents(this, &DispatchGeometryCollectionBreakEvent);
					Solver->EnqueueCommandImmediate([Solver]()
						{
							Solver->SetGenerateBreakingData(true);
						});
				}

				if (bNotifyRemovals)
				{
					EventDispatcher->RegisterForRemovalEvents(this, &DispatchGeometryCollectionRemovalEvent);

					Solver->EnqueueCommandImmediate([Solver]()
						{
							Solver->SetGenerateRemovalData(true);
						});

				}

				if (bNotifyCrumblings)
				{
					EventDispatcher->RegisterForCrumblingEvents(this, &DispatchGeometryCollectionCrumblingEvent);

					Solver->EnqueueCommandImmediate([Solver]()
						{
							Solver->SetGenerateBreakingData(true);
						});
				}
			}

			// Needs to be after the RegisterForXYZEvents calls on the event dispatcher or else GetInterestedProxyOwnersForXYZEvents will not
			// properly allow our registration to go through.
			EventDispatcher->RegisterChaosEvents();
		}
		if (bNotifyGlobalBreaks || bNotifyGlobalCrumblings)
		{
			Solver->EnqueueCommandImmediate([Solver]()
				{
					Solver->SetGenerateBreakingData(true);
				});
		}
		if (bNotifyGlobalCollisions)
		{
			if (FPhysScene_Chaos* PhysicsScene = GetInnerChaosScene())
			{
				PhysicsScene->RegisterForGlobalCollisionEvents(this);
				Solver->EnqueueCommandImmediate([Solver]()
					{
						Solver->SetGenerateCollisionData(true);
					});
			}
		}
		if (bNotifyGlobalRemovals)
		{
			if (FPhysScene_Chaos* PhysicsScene = GetInnerChaosScene())
			{
				PhysicsScene->RegisterForGlobalRemovalEvents(this);
				Solver->EnqueueCommandImmediate([Solver]()
					{
						Solver->SetGenerateRemovalData(true);
					});
			}
		}
		if (bNotifyGlobalTrailings)
		{
			if (FPhysScene_Chaos* PhysicsScene = GetInnerChaosScene())
			{
				Solver->EnqueueCommandImmediate([Solver]()
					{
						Solver->SetGenerateTrailingData(true);
					});
			}
		}
	}
}

void UGeometryCollectionComponent::UpdateRBCollisionEventRegistration()
{
	if (EventDispatcher)
	{
		if (bNotifyCollisions || BodyInstance.bNotifyRigidBodyCollision)
		{
			EventDispatcher->RegisterForCollisionEvents(this, this);
		}
		else
		{
			EventDispatcher->UnRegisterForCollisionEvents(this, this);
		}
	}
}

void UGeometryCollectionComponent::UpdateGlobalCollisionEventRegistration()
{
	if (FPhysScene_Chaos* PhysScene = GetInnerChaosScene())
	{
		if (bNotifyGlobalCollisions)
		{
			PhysScene->RegisterForGlobalCollisionEvents(this);
		}
		else
		{
			PhysScene->UnRegisterForGlobalCollisionEvents(this);
		}
	}
}

void UGeometryCollectionComponent::UpdateGlobalRemovalEventRegistration()
{
	if (FPhysScene_Chaos* PhysicsScene = GetInnerChaosScene())
	{
		if (bNotifyGlobalRemovals)
		{
			PhysicsScene->RegisterForGlobalRemovalEvents(this);
		}
		else
		{
			PhysicsScene->UnRegisterForGlobalRemovalEvents(this);
		}
	}
}

void ActivateClusters(Chaos::FRigidClustering& Clustering, Chaos::TPBDRigidClusteredParticleHandle<Chaos::FReal, 3>* Cluster)
{
	if(!Cluster)
	{
		return;
	}

	if(Cluster->ClusterIds().Id)
	{
		ActivateClusters(Clustering, Cluster->Parent());
	}

	Clustering.DeactivateClusterParticle(Cluster);
}

void UGeometryCollectionComponent::ResetRepData()
{
	ClustersToRep.Reset();
	RepData.Reset();
	RepStateData.Reset();
	RepDynamicData.Reset();

	// Those following data are initialized here from the Game Thread, 
	// but they are read and written from the Physics Thread
	// This function ResetRepData is only called when the component is destroyed and so the PhysicsProxy become null
	// so at this point those data won't be changed on the Physics Thread. 
	OneOffActivatedProcessed = 0;
	VersionProcessed = INDEX_NONE;
	DynamicRepDataVersionProcessed = INDEX_NONE;
	LastHardsnapTimeInMs = 0;
}

void UGeometryCollectionComponent::UpdateRepData()
{
	if (GeometryCollectionUseReplicationV2)
	{
		return UpdateRepStateAndDynamicData();
	}
	
	check(GetNetMode() != ENetMode::NM_Client);
	using namespace Chaos;
	if(!bEnableReplication)
	{
		return;
	}

	if (PhysicsProxy == nullptr)
	{
		return;
	}

	AActor* Owner = GetOwner();
	
	// If we have no owner or our netmode means we never require replication then early out
	if(!Owner || Owner->GetNetMode() == ENetMode::NM_Standalone)
	{
		return;
	}
	
	if (Owner && GetIsReplicated() && Owner->GetLocalRole() == ROLE_Authority)
	{
		const bool bReplicateMovement = Owner->IsReplicatingMovement();
		bool bFirstUpdate = false;
		if(ClustersToRep == nullptr)
		{
			//we only allocate set if needed because it's pretty big to have per components that don't replicate
			ClustersToRep = MakeUnique<TSet<Chaos::FPBDRigidClusteredParticleHandle*>>();
			bFirstUpdate = true;
		}

		//We need to build a snapshot of the GC
		//We rely on the fact that clusters always fracture with one off pieces being removed.
		//This means we only need to record the one offs that broke and we get the connected components for free
		//The cluster properties are replicated with the first child of each connected component. These are always children that are known at author time and have a unique id per component
		//If the first child is disabled it means the properties apply to the parent (i.e. the cluster)
		//If the first child is enabled it means it's a one off and the cluster IS the first child
		
		//TODO: for now we have to iterate over all particles to find the clusters, would be better if we had the clusters and children already available
		//We are relying on the fact that we fracture one level per step. This means we will see all one offs here

		FGeometryCollectionRepData LocalRepData;

		LocalRepData.Version = RepData.Version;
		LocalRepData.RepDataReceivedTime = RepData.RepDataReceivedTime;
		LocalRepData.ServerFrame = RepData.ServerFrame;

		FPBDRigidsSolver* Solver = PhysicsProxy->GetSolver<Chaos::FPBDRigidsSolver>();
		const FRigidClustering& RigidClustering = Solver->GetEvolution()->GetRigidClustering();

		const TManagedArrayAccessor<int32> InitialLevels = PhysicsProxy->GetPhysicsCollection().GetInitialLevels();
		const TArray<FPBDRigidClusteredParticleHandle*>& UnorderedParticleHandles = PhysicsProxy->GetUnorderedParticles_Internal();

		// Replicate the anchored state of the root particle
		LocalRepData.bIsRootAnchored = RepData.bIsRootAnchored;
		if (FPBDRigidClusteredParticleHandle* RootHandle = PhysicsProxy->GetInitialRootHandle_Internal())
		{
			LocalRepData.bIsRootAnchored = RootHandle->IsAnchored();
		}
		
		//see if we have any new clusters that are enabled
		TSet<FPBDRigidClusteredParticleHandle*> Processed;

		// It feels weird to start from the previous one. Need to be checked
		LocalRepData.OneOffActivated = RepData.OneOffActivated;
		for (FPBDRigidClusteredParticleHandle* Particle : UnorderedParticleHandles)
		{
			// Particle can be null if we have embedded geometry 
			if (Particle)
			{
				bool bProcess = true;
				Processed.Add(Particle);
				FPBDRigidClusteredParticleHandle* Root = Particle;
				while (Root->Parent())
				{
					Root = Root->Parent();
	
					//TODO: set avoids n^2, would be nice if clustered particle cached its root
					if(Processed.Contains(Root))
					{
						bProcess = false;
						break;
					}
					else
					{
						Processed.Add(Root);
					}
				}
	
				// The additional physics proxy check is to make sure that we don't try to replicate a cluster union particle.
				if (bProcess && Root->Disabled() == false && ClustersToRep->Find(Root) == nullptr && Root->PhysicsProxy() == PhysicsProxy)
				{
					int32 TransformGroupIdx = INDEX_NONE;
					int32 Level = INDEX_NONE;
					if (Root->InternalCluster() == false)
					{
						TransformGroupIdx = PhysicsProxy->GetTransformGroupIndexFromHandle(Root);
						ensureMsgf(TransformGroupIdx >= 0, TEXT("Non-internal cluster should always have a group index"));
						ensureMsgf(TransformGroupIdx < TNumericLimits<uint16>::Max(), TEXT("Trying to replicate GC with more than 65k pieces. We assumed uint16 would suffice"));

						Level = InitialLevels.IsValid() && InitialLevels.Num() > 0 ? InitialLevels[TransformGroupIdx] : INDEX_NONE;
					}
					else
					{
						// Use internal cluster child's index to compute level.
						const TArray<FPBDRigidParticleHandle*>& Children = RigidClustering.GetChildrenMap()[Root];
						const int32 ChildTransformGroupIdx = PhysicsProxy->GetTransformGroupIndexFromHandle(Children[0]);
						Level = InitialLevels.IsValid() && InitialLevels.Num() > 0 ? (InitialLevels[ChildTransformGroupIdx] - 1) : INDEX_NONE;
					}
	
					if (!bEnableAbandonAfterLevel || Level <= ReplicationAbandonAfterLevel)
					{
						// not already replicated and not abandoned level, start replicating cluster
						if (Level <= ReplicationMaxPositionAndVelocityCorrectionLevel)
						{
							ClustersToRep->Add(Root);
						}
					}

					if(Root->InternalCluster() == false && Level > 0)
					{
						//a one off so record it
						ensureMsgf(TransformGroupIdx >= 0, TEXT("Non-internal cluster should always have a group index"));
						ensureMsgf(TransformGroupIdx < TNumericLimits<uint16>::Max(), TEXT("Trying to replicate GC with more than 65k pieces. We assumed uint16 would suffice"));
						ensureMsgf(PhysicsProxy->GetParticle_Internal(TransformGroupIdx) != nullptr, TEXT("Invalid particle index being replicated - Contact physics team"));

						// Because we cull ClustersToRep with abandoned level, we must make sure we don't add duplicates to one off activated.
						// TODO: avoid search for entry for perf
						// TODO: once we support deep fracture we should be able to remove one offs clusters that are now disabled, reducing the amount to be replicated
						FGeometryCollectionActivatedCluster OneOffActivated(TransformGroupIdx, Root->GetV(), Root->GetW());
						if(!LocalRepData.OneOffActivated.Contains(OneOffActivated))
						{
							LocalRepData.OneOffActivated.Add(OneOffActivated);
						}
					}

					// if we just hit the abandon level , let's disable all children 
					if (bEnableAbandonAfterLevel && Level >= (ReplicationAbandonAfterLevel + 1))
					{
						if (!Root->Disabled())
						{
							Solver->GetEvolution()->DisableParticle(Root);
							Solver->GetParticles().MarkTransientDirtyParticle(Root);
						}
					}
				}
			}
		}

		INC_DWORD_STAT_BY(STAT_GCReplicatedFractures, LocalRepData.OneOffActivated.Num());
		
		//Build up clusters to replicate and compare with previous frame
		if (bReplicateMovement)
		{
			LocalRepData.Clusters.Reserve(ClustersToRep->Num());
			//remove disabled clusters and update rep data if needed
			for (auto Itr = ClustersToRep->CreateIterator(); Itr; ++Itr)
			{
				FPBDRigidClusteredParticleHandle* Cluster = *Itr;

				const TArray<FPBDRigidParticleHandle*>& Children = RigidClustering.GetChildrenMap().FindRef(Cluster);
				const bool bIsInternalCluster = Cluster->InternalCluster();
				const bool bChildrenEmpty = Children.IsEmpty();
				if (Cluster->Disabled() || (bIsInternalCluster && bChildrenEmpty))
				{
					Itr.RemoveCurrent();
				}
				else
				{
					FGeometryCollectionClusterRep& ClusterRep = LocalRepData.Clusters.AddDefaulted_GetRef();

					ClusterRep.Position = Cluster->GetX();
					ClusterRep.Rotation = Cluster->GetR();
					ClusterRep.LinearVelocity = Cluster->GetV();
					ClusterRep.AngularVelocity = Cluster->GetW();
					ClusterRep.ClusterState.SetObjectState(Cluster->ObjectState());
					ClusterRep.ClusterState.SetInternalCluster(Cluster->InternalCluster());
					int32 TransformGroupIdx;
					if (Cluster->InternalCluster())
					{
						ensureMsgf(Children.Num(), TEXT("Internal cluster yet we have no children? [Num %d vs Cached Empty %d]"), Children.Num(), bChildrenEmpty);
						TransformGroupIdx = PhysicsProxy->GetTransformGroupIndexFromHandle(Children[0]);
					}
					else
					{
						// not internal so we can just use the cluster's ID. On client we'll know based on the parent whether to use this index or the parent
						TransformGroupIdx = PhysicsProxy->GetTransformGroupIndexFromHandle(Cluster);
					}

					ensureMsgf(TransformGroupIdx < TNumericLimits<uint16>::Max(), TEXT("Trying to replicate GC with more than 65k pieces. We assumed uint16 would suffice"));
					ClusterRep.ClusterIdx = TransformGroupIdx;
				}
			}
		}

		if (LocalRepData.HasChanged(RepData))
		{
			LocalRepData.ServerFrame = Solver->GetMarshallingManager().GetInternalStep_External();

			INC_DWORD_STAT_BY(STAT_GCReplicatedClusters, LocalRepData.Clusters.Num());

			FlushNetDormancyIfNeeded();

			RepData = MoveTemp(LocalRepData);
			MARK_PROPERTY_DIRTY_FROM_NAME(UGeometryCollectionComponent, RepData, this);
			++RepData.Version;

			const bool bSetDormancyToAwake = Owner->NetDormancy > DORM_Awake && GetDesiredNetAwakeningMode() == ENetAwakeningMode::ForceDormancyAwake;
			if (bSetDormancyToAwake)
			{
				if (Owner->NetDormancy > DORM_Awake)
				{
					//If net dormancy is Initial it must be for perf reasons, but since a cluster changed we need to replicate down	
					Owner->SetNetDormancy(DORM_Awake);
				}
			}
		}
	}
}


void UGeometryCollectionComponent::UpdateRepStateAndDynamicData()
{
	check(GetNetMode() != ENetMode::NM_Client);
	using namespace Chaos;
	if (!bEnableReplication || PhysicsProxy == nullptr)
	{
		return;
	}

	AActor* Owner = GetOwner();

	// If we have no owner or our netmode means we never require replication then early out
	if (!Owner || Owner->GetNetMode() == ENetMode::NM_Standalone)
	{
		return;
	}

	if (GetIsReplicated() && Owner->GetLocalRole() == ROLE_Authority)
	{
		if (RestCollection && RestCollection->GetGeometryCollection())
		{
			FPBDRigidsSolver* Solver = PhysicsProxy->GetSolver<Chaos::FPBDRigidsSolver>();

			// let go through all the transform and see which one has changed its state
			const TArray<FPBDRigidClusteredParticleHandle*>& UnorderedParticleHandles = PhysicsProxy->GetUnorderedParticles_Internal();
			const int32 NumParticles = UnorderedParticleHandles.Num();

			const TManagedArrayAccessor<int32> InitialLevels = PhysicsProxy->GetPhysicsCollection().GetInitialLevels();
			
			FGeometryCollectionRepStateData LocalRepStateData;
			FGeometryCollectionRepDynamicData LocalRepDynamicData;

			LocalRepStateData.Version = RepStateData.Version;
			LocalRepDynamicData.Version = RepDynamicData.Version;

			// root level anchor state
			LocalRepStateData.bIsRootAnchored = RepStateData.bIsRootAnchored;
			if (FPBDRigidClusteredParticleHandle* RootHandle = PhysicsProxy->GetInitialRootHandle_Internal())
			{
				LocalRepStateData.bIsRootAnchored = RootHandle->IsAnchored() ? 1 : 0;
			}

			struct FRootHandle
			{
				const FPBDRigidClusteredParticleHandle* Handle; // handle of the particle or internal cluster 
				const int32 TransformIndex; // transform index of the particle or the child of the internal cluster 

				bool operator==(const FRootHandle& Other) const	{ return Handle == Other.Handle; }
			};

			// contains all the dynamic root of the geometry collection
			// this includes non-clusterunion internal clusters 
			// using an array as the number of root should always be small 
			TArray<FRootHandle> RootsToTrack;

			const int32 RootIndex = GetRootIndex();

			// go through particles and send replicated data 
			const int32 NumTransforms = PhysicsProxy->GetNumTransforms();
			LocalRepStateData.BrokenState.SetNum(NumTransforms, false);
			LocalRepStateData.ReleasedData.Reserve(NumTransforms);
			for (int32 ParticleIndex = 0; ParticleIndex < NumParticles; ParticleIndex++)
			{
				const int32 TransformIndex = PhysicsProxy->GetFromParticleToTransformIndex(ParticleIndex);
				if (FPBDRigidClusteredParticleHandle* ParticleHandle = UnorderedParticleHandles[ParticleIndex])
				{
					const int32 Level = (InitialLevels.IsValid() && InitialLevels.IsValidIndex(TransformIndex)) ? InitialLevels[TransformIndex] : INDEX_NONE;

					const FPBDRigidClusteredParticleHandle* ParentHandle = ParticleHandle->Parent();

					// no parent and not root means the particle has been broken  
					const bool bIsRootIndex = (TransformIndex == RootIndex);
					const bool bIsBroken = (ParentHandle == nullptr && !bIsRootIndex);
					if (bIsBroken)
					{
						// only record the one at the abandon level and before ( child break of abandon level clusters must be recorded )  
						if (!bEnableAbandonAfterLevel || Level <= ReplicationAbandonAfterLevel)
						{
							LocalRepStateData.BrokenState[TransformIndex] = true;
							if(!ParticleHandle->Disabled())
							{
								FGeometryCollectionRepStateData::FReleasedData& Data = LocalRepStateData.ReleasedData.AddDefaulted_GetRef();
								Data.TransformIndex = TransformIndex;
								Data.LinearVelocity = ParticleHandle->GetV();
								Data.AngularVelocityInDegreesPerSecond = FMath::RadiansToDegrees(ParticleHandle->GetW());
							}
						}

						// anything beyond ReplicationAbandonAfterLevel must be disabled to save server CPU as they are now client authoritative
						if (bEnableAbandonAfterLevel && Level >= (ReplicationAbandonAfterLevel + 1))
						{
							if (!ParticleHandle->Disabled())
							{
								Solver->GetEvolution()->DisableParticle(ParticleHandle);
								Solver->GetParticles().MarkTransientDirtyParticle(ParticleHandle);
							}
						}
					}

					const bool bIsActiveBrokenOrOriginalRoot = (!ParticleHandle->Disabled()) && (bIsBroken || bIsRootIndex);
					const bool bHasInternalClusterParent = ParentHandle && (ParentHandle->InternalCluster()) && (ParentHandle->PhysicsProxy() == ParticleHandle->PhysicsProxy());

					const FPBDRigidClusteredParticleHandle* RootParticle = bIsActiveBrokenOrOriginalRoot ? ParticleHandle : (bHasInternalClusterParent ? ParentHandle : nullptr);
					const int32 RootParticleLevel = (bHasInternalClusterParent)? (Level - 1): (Level);
					const bool bTrackPosition = (RootParticleLevel <= this->ReplicationMaxPositionAndVelocityCorrectionLevel);
					if (bTrackPosition && RootParticle)
					{
						RootsToTrack.AddUnique(FRootHandle{ RootParticle, TransformIndex });
					}
					
				}
			}

			//reclaim unused slack
			LocalRepStateData.ReleasedData.Shrink();

			// let's update the data for the tracked clusters 
			LocalRepDynamicData.ClusterData.Reserve(RootsToTrack.Num());
			for (const FRootHandle& Root: RootsToTrack)
			{
				FGeometryCollectionRepDynamicData::FClusterData& Data = LocalRepDynamicData.ClusterData.AddDefaulted_GetRef();
				Data.TransformIndex = Root.TransformIndex;
				Data.bIsInternalCluster = Root.Handle->InternalCluster();
				Data.Position = Root.Handle->GetX();
				Data.EulerRotation = Root.Handle->GetR().Rotator().Euler();
				Data.LinearVelocity = Root.Handle->GetV();
				Data.AngularVelocityInDegreesPerSecond = FMath::RadiansToDegrees(Root.Handle->GetW());
				Data.LastUpdatedVersion = LocalRepDynamicData.Version + 1;
			}

			// Check to see if the replicated state and dynamic Data has changed
			const bool bStateChanged = LocalRepStateData.HasChanged(RepStateData);
			const bool bDynamicChanged = LocalRepDynamicData.HasChanged(RepDynamicData);

			if(bStateChanged || bDynamicChanged)
			{
				FlushNetDormancyIfNeeded();
			}

			if (bStateChanged)
			{
				RepStateData = MoveTemp(LocalRepStateData);
				MARK_PROPERTY_DIRTY_FROM_NAME(UGeometryCollectionComponent, RepStateData, this);
				++RepStateData.Version;
			}

			if (bDynamicChanged)
			{
				RepDynamicData = MoveTemp(LocalRepDynamicData);
				MARK_PROPERTY_DIRTY_FROM_NAME(UGeometryCollectionComponent, RepDynamicData, this);
				++RepDynamicData.Version;
			}

			if (bDynamicChanged || bStateChanged)
			{
				const bool bSetDormancyToAwake = Owner->NetDormancy > DORM_Awake && GetDesiredNetAwakeningMode() == ENetAwakeningMode::ForceDormancyAwake;
				if (bSetDormancyToAwake)
				{
					//If net dormancy is Initial it must be for perf reasons, but since a cluster changed we need to replicate down
					Owner->SetNetDormancy(DORM_Awake);
				}
			}
		}
	}
}

float GeometryCollectionRepLinearMatchStrength = 50;
FAutoConsoleVariableRef CVarGeometryCollectionRepLinearMatchStrength(TEXT("p.GeometryCollectionRepLinearMatchStrength"), GeometryCollectionRepLinearMatchStrength, TEXT("Units can be interpreted as %/s^2 - acceleration of percent linear correction"));

float GeometryCollectionRepAngularMatchTime = .5f;
FAutoConsoleVariableRef CVarGeometryCollectionRepAngularMatchTime(TEXT("p.GeometryCollectionRepAngularMatchTime"), GeometryCollectionRepAngularMatchTime, TEXT("In seconds, how quickly should the angle match the replicated target angle"));

float GeometryCollectionRepMaxExtrapolationTime = 3.f;
FAutoConsoleVariableRef CVarGeometryCollectionRepMaxExtrapolationTime(TEXT("p.GeometryCollectionRepMaxExtrapolationTime"), GeometryCollectionRepMaxExtrapolationTime, TEXT("Number of seconds that replicated physics data will persist for a GC, extrapolating velocities"));

bool bGeometryCollectionDebugDrawRep = 0;
FAutoConsoleVariableRef CVarGeometryCollectionDebugDrawRep(TEXT("p.Chaos.DebugDraw.GeometryCollectionReplication"), bGeometryCollectionDebugDrawRep,
	TEXT("If true debug draw deltas and corrections for geometry collection replication"));

bool bGeometryCollectionRepUseClusterVelocityMatch = true;
FAutoConsoleVariableRef CVarGeometryCollectionRepUseClusterVelocityMatch(TEXT("p.bGeometryCollectionRepUseClusterVelocityMatch"), bGeometryCollectionRepUseClusterVelocityMatch,
	TEXT("Use physical velocity to match cluster states"));

int32 GeometryCollectionHardMissingUpdatesSnapThreshold = 20;
FAutoConsoleVariableRef CVarGeometryCollectionHardMissingUpdatesSnapThreshold(TEXT("p.GeometryCollectionHardMissingUpdatesSnapThreshold"), GeometryCollectionHardMissingUpdatesSnapThreshold,
	TEXT("Determines how many missing updates before we trigger a hard snap"));

int32 GeometryCollectionHardsnapThresholdMs = 100; // 10 Hz
FAutoConsoleVariableRef CVarGeometryCollectionHardsnapThresholdMs(TEXT("p.GeometryCollectionHardsnapThresholdMs"), GeometryCollectionHardMissingUpdatesSnapThreshold,
	TEXT("Determines how many ms since the last hardsnap to trigger a new one"));

void UGeometryCollectionComponent::ProcessRepData()
{
	bGeometryCollectionRepUseClusterVelocityMatch = false;
	ProcessRepData(0.f, 0.f);
}

namespace 
{
	bool ReplicationShouldHardSnap(int32 RepDataVersion, int32 VersionProcessed, bool bReplicateMovement, double& LastHardsnapTimeInMs)
	{
		bool bHardSnap = false;
		if (bReplicateMovement)
		{
			const int64 CurrentTimeInMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());

			// Always hard snap on the very first version received
			if (VersionProcessed == 0)
			{
				bHardSnap = true;
			}
			else if (VersionProcessed < RepDataVersion)
			{
				//TODO: this will not really work if a fracture happens and then immediately goes to sleep without updating client enough times
				//A time method would work better here, but is limited to async mode. Maybe we can support both
				bHardSnap = (RepDataVersion - VersionProcessed) > GeometryCollectionHardMissingUpdatesSnapThreshold;

				if (!bGeometryCollectionRepUseClusterVelocityMatch)
				{
					// When not doing velocity match for clusters, instead we do periodic hard snapping
					bHardSnap |= (CurrentTimeInMs - LastHardsnapTimeInMs) > GeometryCollectionHardsnapThresholdMs;
				}
			}
			else if (VersionProcessed > RepDataVersion)
			{
				//rollover so just treat as hard snap - this case is extremely rare and a one off
				bHardSnap = true;
			}

			if (bHardSnap)
			{
				LastHardsnapTimeInMs = CurrentTimeInMs;
			}
		}
		return bHardSnap;
	}

	void UpdateRootStateFromReplication(const FGeometryCollectionPhysicsProxy& PhysicsProxy, const bool bNewAnchoredState)
	{
		using namespace Chaos;

		const int32 InitialRootIndex = PhysicsProxy.GetSimParameters().InitialRootIndex;
		if (FPBDRigidClusteredParticleHandle* RootHandle = PhysicsProxy.GetParticle_Internal(InitialRootIndex))
		{
			if (RootHandle->Parent() || !RootHandle->Disabled())
			{
				if (bNewAnchoredState != RootHandle->IsAnchored())
				{
					RootHandle->SetIsAnchored(bNewAnchoredState);

					if (FPBDRigidsSolver* Solver = PhysicsProxy.GetSolver<Chaos::FPBDRigidsSolver>())
					{
						if (FPBDRigidsEvolutionGBF* Evolution = Solver->GetEvolution())
						{
							FRigidClustering& RigidClustering = Solver->GetEvolution()->GetRigidClustering();
							// NOTE: We have to start out with the particle as kinematic and let UpdateKinematicProperties correct it if this is wrong
							Evolution->SetParticleObjectState(RootHandle, EObjectStateType::Kinematic);
							UpdateKinematicProperties(RootHandle, RigidClustering.GetChildrenMap(), *Evolution);
						}
					}
				}
			}
		}
	}

	// return the awaken state of the cluster 
	bool UpdateClusterPositionAndVelocitiesFromReplication(
		Chaos::FPBDRigidsSolver* Solver,
		Chaos::FPBDRigidParticleHandle& Cluster,
		const FVector& Position, const FQuat& Rotation, 
		const FVector& LinearVelocity, const FVector& AngularVelocity,
		const bool bHardSnap,
		const double RepExtrapTime,
		const double DeltaTime)
	{
		bool bWake = false;

		if (bHardSnap)
		{
			Cluster.SetX(Position);
			Cluster.SetR(Rotation);
			Cluster.SetV(LinearVelocity);
			Cluster.SetW(AngularVelocity);
			bWake = true;
		}
		else if (bGeometryCollectionRepUseClusterVelocityMatch)
		{
			//
			// Match linear velocity
			//
			const FVector RepVel = LinearVelocity;
			const FVector RepExtrapPos = Position + (RepVel * RepExtrapTime);
			const Chaos::FVec3 DeltaX = RepExtrapPos - Cluster.GetX();
			const float DeltaXMagSq = DeltaX.SizeSquared();
			if (DeltaXMagSq > SMALL_NUMBER && GeometryCollectionRepLinearMatchStrength > SMALL_NUMBER)
			{
				bWake = true;
				//
				// DeltaX * MatchStrength is an acceleration, m/s^2, which is integrated
				// by multiplying by DeltaTime.
				//
				// It's formulated this way to get a larger correction for a longer time
				// step, ie. correction velocities are framerate independent.
				//
				Cluster.SetV(RepVel + (DeltaX * FMath::Min(1.0f, GeometryCollectionRepLinearMatchStrength * DeltaTime)));
			}

			//
			// Match angular velocity
			//
			const FVector RepAngVel = AngularVelocity;
			const Chaos::FRotation3 RepExtrapAng = Chaos::FRotation3::IntegrateRotationWithAngularVelocity(Rotation, RepAngVel, RepExtrapTime);
			const FVector AngVel = Chaos::FRotation3::CalculateAngularVelocity(Cluster.GetR(), RepExtrapAng, GeometryCollectionRepAngularMatchTime);
			if (AngVel.SizeSquared() > SMALL_NUMBER)
			{
				Cluster.SetW(RepAngVel + AngVel);
				bWake = true;
			}
		}

		//
		// Wake up particle if it's sleeping and there's a delta to correct
		//
		if (bWake && Cluster.IsSleeping())
		{
			Solver->GetEvolution()->SetParticleObjectState(&Cluster, Chaos::EObjectStateType::Dynamic);
		}

		return bWake;
	}

	bool ProcessRepDataCommon(const float DeltaTime, const float SimTime, const FGeometryCollectionRepData& RepData, FGeometryCollectionPhysicsProxy* PhysicsProxy, int32& VersionProcessed, int32& OneOffActivatedProcessed, double& LastHardsnapTimeInMs, bool bReplicateMovement)
	{
		using namespace Chaos;

		if (!PhysicsProxy || !PhysicsProxy->IsInitializedOnPhysicsThread() || PhysicsProxy->GetReplicationMode() != FGeometryCollectionPhysicsProxy::EReplicationMode::Client)
		{
			return false;
		}

		float ReceivedTime = SimTime;

		// Track the sim time that this rep data was received on.
		if (RepData.RepDataReceivedTime.IsSet())
		{
			ReceivedTime = *RepData.RepDataReceivedTime;
		}
		
		// How far we must extrapolate from when we received the data
		const float RepExtrapTime = FMath::Max(0.f, SimTime - ReceivedTime);

		// If we've extrapolated past a threshold, then stop tracking
		// the last received rep data
		if (RepExtrapTime > GeometryCollectionRepMaxExtrapolationTime)
		{
			return false;
		}

		// Create a little little function for applying a lambda to each
		// corresponding pair of replicated and local clusters.
		const auto ForEachClusterPair = [&RepData, &PhysicsProxy](const auto& Lambda)
		{
			for (const FGeometryCollectionClusterRep& RepCluster : RepData.Clusters)
			{
				if (Chaos::FPBDRigidParticleHandle* Cluster = PhysicsProxy->GetParticle_Internal(RepCluster.ClusterIdx))
				{
					if (RepCluster.ClusterState.IsInternalCluster())
					{
						// internal cluster do not have an index so we rep data send one of the children's
						// let's find the parent
						Cluster = Cluster->CastToClustered()->Parent();
					}

					if (Cluster && Cluster->Disabled() == false)
					{
						Lambda(RepCluster, *Cluster);
					}
				}
			}
		};


		// If not doing velocity match, don't bother processing the same version twice.
		// Do this one after the debug draw so that we can still easily see the diff
		// between the position and the target position.
		if (VersionProcessed == RepData.Version && !bGeometryCollectionRepUseClusterVelocityMatch)
		{
			return false;
		}

		// Update the anchored state of the root particle
		UpdateRootStateFromReplication(*PhysicsProxy, RepData.bIsRootAnchored);

		const bool bHardSnap = ReplicationShouldHardSnap(RepData.Version, VersionProcessed, bReplicateMovement, LastHardsnapTimeInMs);

		FPBDRigidsSolver* Solver = PhysicsProxy->GetSolver<Chaos::FPBDRigidsSolver>();
		FRigidClustering& RigidClustering = Solver->GetEvolution()->GetRigidClustering();

		//First make sure all one off activations have been applied. This ensures our connectivity graph is the same and we have the same clusters as the server
		for (; OneOffActivatedProcessed < RepData.OneOffActivated.Num(); ++OneOffActivatedProcessed)
		{
			const FGeometryCollectionActivatedCluster& ActivatedCluster = RepData.OneOffActivated[OneOffActivatedProcessed];

#if !UE_BUILD_SHIPPING
			if(PhysicsProxy->GetParticle_Internal(ActivatedCluster.ActivatedIndex) == nullptr)
			{
				ensureMsgf(false, TEXT("Invalid activated cluster index processing replication data."));
				continue;
			}
#endif

			if (FPBDRigidParticleHandle* OneOff = PhysicsProxy->GetParticle_Internal(ActivatedCluster.ActivatedIndex))
			{
				if (FPBDRigidClusteredParticleHandle* ClusterParticle = OneOff->CastToClustered())
				{
					// Set initial velocities if not hard snapping
					if (!bHardSnap)
					{
						// TODO: we should get an update cluster position first so that when particles break off they get the right position 
						// TODO: should we invalidate?
						OneOff->SetV(ActivatedCluster.InitialLinearVelocity);
						OneOff->SetW(ActivatedCluster.InitialAngularVelocity);
					}

					// OneOffActivated is not guaranteed to be sorted by level, so we may need to release particles before their parent get released
					// so we make sure the full parent chain is properly released as well 
					RigidClustering.ForceReleaseChildParticleAndParents(ClusterParticle, /* bTriggerBreakEvents */true);
				}
			}
		}

		// Keep track of whether we did some "work" on this frame so we can turn off the async tick after
		// multiple frames of not doing anything.
		bool bProcessed = false;
		if (bReplicateMovement)
		{
			ForEachClusterPair([Solver, DeltaTime, RepExtrapTime, bHardSnap, &bProcessed](const FGeometryCollectionClusterRep& RepData, Chaos::FPBDRigidParticleHandle& Cluster)
			{
				bProcessed |= UpdateClusterPositionAndVelocitiesFromReplication(Solver, Cluster, RepData.Position, RepData.Rotation, RepData.LinearVelocity, RepData.AngularVelocity, bHardSnap, RepExtrapTime, DeltaTime);
			});
		}

		VersionProcessed = RepData.Version;
		return bProcessed;
	}

	bool ProcessRepStateDataCommon(FGeometryCollectionPhysicsProxy* PhysicsProxy, const FGeometryCollectionRepStateData& RepStateData, int32& VersionProcessed)
	{
		using namespace Chaos;

		if (!PhysicsProxy || !PhysicsProxy->IsInitializedOnPhysicsThread() || PhysicsProxy->GetReplicationMode() != FGeometryCollectionPhysicsProxy::EReplicationMode::Client)
		{
			return false;
		}

		if (VersionProcessed == RepStateData.Version)
		{
			return false;
		}

		const TArray<FPBDRigidClusteredParticleHandle*>& UnorderedParticleHandle = PhysicsProxy->GetUnorderedParticles_Internal();

		// Update the anchored state of the root particle
		UpdateRootStateFromReplication(*PhysicsProxy, (bool)RepStateData.bIsRootAnchored);
		
		
		// now sync the broken state of the particles
		if (RepStateData.BrokenState.Num() == PhysicsProxy->GetNumTransforms())
		{
			FPBDRigidsSolver* Solver = PhysicsProxy->GetSolver<Chaos::FPBDRigidsSolver>();
			FRigidClustering& RigidClustering = Solver->GetEvolution()->GetRigidClustering();

			for (int32 ParticleIndex = 0; ParticleIndex < UnorderedParticleHandle.Num(); ParticleIndex++)
			{
				if (FPBDRigidClusteredParticleHandle* ParticleHandle = UnorderedParticleHandle[ParticleIndex])
				{
					const int32 TransformIndex = PhysicsProxy->GetFromParticleToTransformIndex(ParticleIndex);
					// we only check for when thing goes from unbroken to broken 
					const bool bWasBroken = (ParticleHandle->Parent() == nullptr);
					const bool bIsBroken = RepStateData.BrokenState[TransformIndex];
					if (!bWasBroken && bIsBroken)
					{
						RigidClustering.ForceReleaseChildParticleAndParents(ParticleHandle, /* bTriggerBreakEvents */true);

						if (!ParticleHandle->Disabled())
						{
							// check if we have a release velocity to be applied 
							const FGeometryCollectionRepStateData::FReleasedData* ReleaseData = RepStateData.ReleasedData.FindByPredicate(
								[TransformIndex](const FGeometryCollectionRepStateData::FReleasedData& Data)
								{
									return Data.TransformIndex == TransformIndex;
								});
							if (ReleaseData)
							{
								ParticleHandle->SetV(ReleaseData->LinearVelocity);
								ParticleHandle->SetW(FMath::DegreesToRadians(ReleaseData->AngularVelocityInDegreesPerSecond));
							}
							else
							{
								// if we are a leaf we can disable this particle right away if there's no release data
								// because this means the server has already disabled the particle from a removal 
								if (ParticleHandle->ClusterIds().NumChildren == 0)
								{
									// todo we could probably disable all at once later to have only one call 
									Solver->GetEvolution()->DisableParticle(ParticleHandle);
									Solver->GetParticles().MarkTransientDirtyParticle(ParticleHandle);
								}
							}
						}
					}
				}
			}
		}

		VersionProcessed = RepStateData.Version;
		return true;
	}

	bool ProcessRepDynamicDataCommon(
		FGeometryCollectionPhysicsProxy* PhysicsProxy,
		const FGeometryCollectionRepDynamicData& RepDynamicData,
		int32& VersionProcessed,
		bool bReplicateMovement,
		double SimTime,
		double DeltaTime,
		double& LastHardsnapTimeInMs
	)
	{
		using namespace Chaos;

		if (!PhysicsProxy || !PhysicsProxy->IsInitializedOnPhysicsThread() || PhysicsProxy->GetReplicationMode() != FGeometryCollectionPhysicsProxy::EReplicationMode::Client)
		{
			return false;
		}

		if (VersionProcessed == RepDynamicData.Version)
		{
			return false;
		}

		// extrapolation time is 0, because we do have the sim time when we receive the data 
		// we keep this to be mimic the original implementation that was not running on the physics thread 
		// todo(chaos) shoul dwe fix this ? 
		const float RepExtrapTime = 0.f;

		// If we've extrapolated past a threshold, then stop tracking
		// the last received rep data
		if (RepExtrapTime > GeometryCollectionRepMaxExtrapolationTime)
		{
			return false;
		}

		const bool bHardSnap = ReplicationShouldHardSnap(RepDynamicData.Version, VersionProcessed, bReplicateMovement, LastHardsnapTimeInMs);

		// Keep track of whether we did some "work" on this frame so we can turn off the async tick after
		// multiple frames of not doing anything.
		bool bProcessed = false;
		if (bReplicateMovement)
		{
			FPBDRigidsSolver* Solver = PhysicsProxy->GetSolver<Chaos::FPBDRigidsSolver>();
			for (const FGeometryCollectionRepDynamicData::FClusterData& RepCluster : RepDynamicData.ClusterData)
			{
				if (Chaos::FPBDRigidParticleHandle* Cluster = PhysicsProxy->GetParticle_Internal(RepCluster.TransformIndex))
				{
					if (RepCluster.bIsInternalCluster)
					{
						// internal cluster do not have an index so we rep data send one of the children's, let's find the parent
						Cluster = Cluster->CastToClustered()->Parent();
					}

					if (Cluster && Cluster->Disabled() == false)
					{
						const FQuat RotationAsQuat = FQuat::MakeFromEuler(RepCluster.EulerRotation);
						const FVector AngularVelocityInRadiansPerSecond = FMath::DegreesToRadians(RepCluster.AngularVelocityInDegreesPerSecond);
						UpdateClusterPositionAndVelocitiesFromReplication(Solver, *Cluster, RepCluster.Position, RotationAsQuat, RepCluster.LinearVelocity, AngularVelocityInRadiansPerSecond, bHardSnap, RepExtrapTime, DeltaTime);
					}
				}
			};
		}

		VersionProcessed = RepDynamicData.Version;
		return true;
	}
}



bool UGeometryCollectionComponent::ProcessRepData(const float DeltaTime, const float SimTime)
{

	// Track the sim time that this rep data was received on.
	if (!RepData.RepDataReceivedTime.IsSet())
	{
		RepData.RepDataReceivedTime = SimTime;
	}

	AActor* Owner = GetOwner();
	const bool bReplicateMovement = Owner ? GetOwner()->IsReplicatingMovement() : true;

	bool ReturnValue = ::ProcessRepDataCommon(DeltaTime, SimTime, RepData, PhysicsProxy, VersionProcessed, OneOffActivatedProcessed, LastHardsnapTimeInMs, bReplicateMovement);


#if CHAOS_DEBUG_DRAW
	
	if (bGeometryCollectionDebugDrawRep)
	{
		// Track the sim time that this rep data was received on.
		if (!RepData.RepDataReceivedTime.IsSet())
		{
			RepData.RepDataReceivedTime = SimTime;
		}

		// How far we must extrapolate from when we received the data
		const float RepExtrapTime = FMath::Max(0.f, SimTime - *RepData.RepDataReceivedTime);

		// Create a little little function for applying a lambda to each
		// corresponding pair of replicated and local clusters.
		const auto ForEachClusterPair = [this](const auto& Lambda)
		{
			for (const FGeometryCollectionClusterRep& RepCluster : RepData.Clusters)
			{
				if (Chaos::FPBDRigidParticleHandle* Cluster = PhysicsProxy->GetParticle_Internal(RepCluster.ClusterIdx))
				{
					if (RepCluster.ClusterState.IsInternalCluster())
					{
						// internal cluster do not have an index so we rep data send one of the children's
						// let's find the parent
						Cluster = Cluster->CastToClustered()->Parent();
					}

					if (Cluster && Cluster->Disabled() == false)
					{
						Lambda(RepCluster, *Cluster);
					}
				}
			}
		};

		ForEachClusterPair([RepExtrapTime](const FGeometryCollectionClusterRep& RepCluster, Chaos::FPBDRigidParticleHandle& Cluster)
		{
			// Don't bother debug drawing if the delta is too small
			if ((Cluster.GetX() - RepCluster.Position).SizeSquared() < .1f)
			{
				FVector Axis;
				float Angle;
				(RepCluster.Rotation.Inverse() * Cluster.GetR()).ToAxisAndAngle(Axis, Angle);
				if (FMath::Abs(Angle) < .1f)
				{
					return;
				}
			}

			Chaos::FDebugDrawQueue& DrawQueue = Chaos::FDebugDrawQueue::GetInstance();
			DrawQueue.DrawDebugCoordinateSystem(Cluster.GetX(), FRotator(Cluster.GetR()), 100.f, false, -1, -1, 1.f);
			DrawQueue.DrawDebugBox(Cluster.GetX() + Cluster.LocalBounds().Center(), Cluster.LocalBounds().Extents(), Cluster.GetR(), FColor::White, false, -1, -1, 1.f);
			DrawQueue.DrawDebugBox(RepCluster.Position + Cluster.LocalBounds().Center(), Cluster.LocalBounds().Extents(), RepCluster.Rotation, FColor::Green, false, -1, -1, 1.f);

			if (bGeometryCollectionRepUseClusterVelocityMatch)
			{
				const FVector RepVel = RepCluster.LinearVelocity;
				const FVector RepAngVel = RepCluster.AngularVelocity;
				const FVector RepExtrapPos = RepCluster.Position + (RepVel * RepExtrapTime);
				const Chaos::FRotation3 RepExtrapAng = Chaos::FRotation3::IntegrateRotationWithAngularVelocity(RepCluster.Rotation, RepAngVel, RepExtrapTime);
				DrawQueue.DrawDebugCoordinateSystem(RepExtrapPos, FRotator(RepExtrapAng), 100.f, false, -1, -1, 1.f);
				DrawQueue.DrawDebugDirectionalArrow(Cluster.GetX(), RepExtrapPos, 10.f, FColor::White, false, -1, -1, 1.f);
				DrawQueue.DrawDebugBox(RepExtrapPos + Cluster.LocalBounds().Center(), Cluster.LocalBounds().Extents(), RepExtrapAng, FColor::Orange, false, -1, -1, 1.f);
			}
			else
			{
				DrawQueue.DrawDebugCoordinateSystem(RepCluster.Position, FRotator(RepCluster.Rotation), 100.f, false, -1, -1, 1.f);
			}
		});
	}
#endif

	return ReturnValue;
}

void UGeometryCollectionComponent::ProcessRepDataOnPT()
{
	using namespace Chaos;

	if (Chaos::FPhysicsSolver* CurrSolver = GetSolver(*this))
	{
		AActor* Owner = GetOwner();
		const bool bReplicateMovement = Owner ? GetOwner()->IsReplicatingMovement() : true;

		// Capture a weak component pointer to make sure we can check for validity when executing the lambda 
		// This is needed to access VersionProcessed, LastHardsnapTimeInMs, OneOffActivatedProcessed in Read/Write
		// Note : those properties are only used in that function and in the reset function and both are executed on the Physics Thread

		TWeakObjectPtr<UGeometryCollectionComponent> WeakThis(this);
		CurrSolver->EnqueueCommandImmediate([WeakThis, PhysicsProxy = PhysicsProxy, RepDataCopy = RepData, bReplicateMovement](Chaos::FReal DeltaTime, Chaos::FReal SimTime)
		{
			if (TStrongObjectPtr<UGeometryCollectionComponent> StrongThis = WeakThis.Pin())
			{
				::ProcessRepDataCommon(DeltaTime, SimTime, RepDataCopy, PhysicsProxy, StrongThis->VersionProcessed, StrongThis->OneOffActivatedProcessed, StrongThis->LastHardsnapTimeInMs, bReplicateMovement);
			}
		});
	}
}

void UGeometryCollectionComponent::ProcessRepStateDataOnPT()
{
	ensure(GeometryCollectionUseReplicationV2);

	if (Chaos::FPhysicsSolver* CurrSolver = GetSolver(*this))
	{
		// enqueue a callback to process the replicated state data on the physics thread 
		// Capture a weak component pointer to make sure we can check for validity when executing the lambda 
		// This is needed to access VersionProcessed in Read/Write
		// Note : this property is only used in that function and in the reset function and both are executed on the Physics Thread

		TWeakObjectPtr<UGeometryCollectionComponent> WeakThis(this);
		CurrSolver->EnqueueCommandImmediate(
			[WeakThis, PhysicsProxy = PhysicsProxy, RepStateDataCopy = RepStateData]
			(Chaos::FReal /*DeltaTime*/, Chaos::FReal /*SimTime*/)
			{
				if (TStrongObjectPtr<UGeometryCollectionComponent> StrongThis = WeakThis.Pin())
				{
					::ProcessRepStateDataCommon(PhysicsProxy, RepStateDataCopy, StrongThis->VersionProcessed);
				}
			});
	}
}

void UGeometryCollectionComponent::ProcessRepDynamicDataOnPT()
{
	ensure(GeometryCollectionUseReplicationV2);

	if (Chaos::FPhysicsSolver* CurrSolver = GetSolver(*this))
	{
		AActor* Owner = GetOwner();
		const bool bReplicateMovement = Owner ? GetOwner()->IsReplicatingMovement() : true;

		// enqueue a callback to process the replicated dynamic data on the physics thread 
		// Capture a weak component pointer to make sure we can check for validity when executing the lambda 
		// This is needed to access VersionProcessed, LastHardsnapTimeInMs, OneOffActivatedProcessed in Read/Write
		// Note : those properties are only used in that function and in the reset function and both are executed on the Physics Thread

		TWeakObjectPtr<UGeometryCollectionComponent> WeakThis(this);
		CurrSolver->EnqueueCommandImmediate(
			[WeakThis, PhysicsProxy = PhysicsProxy, RepDynamicDataCopy = RepDynamicData, bReplicateMovement]
			(Chaos::FReal DeltaTime, Chaos::FReal SimTime)
			{
				if (TStrongObjectPtr<UGeometryCollectionComponent> StrongThis = WeakThis.Pin())
				{
					::ProcessRepDynamicDataCommon(PhysicsProxy, RepDynamicDataCopy, StrongThis->DynamicRepDataVersionProcessed, bReplicateMovement, SimTime, DeltaTime, StrongThis->LastHardsnapTimeInMs);
				}
			});
	}
}

void UGeometryCollectionComponent::SetDynamicState(const Chaos::EObjectStateType& NewDynamicState)
{
	if (DynamicCollection)
	{
		TManagedArray<uint8>& DynamicState = DynamicCollection->DynamicState;
		for (int i = 0; i < DynamicState.Num(); i++)
		{
			DynamicState[i] = static_cast<uint8>(NewDynamicState);
		}
	}
}

void UGeometryCollectionComponent::SetInitialTransforms(const TArray<FTransform>& InitialTransforms)
{
	if (DynamicCollection)
	{
		int32 MaxIdx = FMath::Min(DynamicCollection->GetNumTransforms(), InitialTransforms.Num());
		for (int32 Idx = 0; Idx < MaxIdx; ++Idx)
		{
			DynamicCollection->SetTransform(Idx, FTransform3f(InitialTransforms[Idx]));
		}
	}
}

void UGeometryCollectionComponent::SetInitialClusterBreaks(const TArray<int32>& ReleaseIndices)
{
	if (DynamicCollection)
	{
		const int32 NumTransforms = DynamicCollection->GetNumTransforms();
		for (int32 ReleaseIndex : ReleaseIndices)
		{
			if (ReleaseIndex < NumTransforms)
			{
				if (int32 ParentIndex = DynamicCollection->GetParent(ReleaseIndex); ParentIndex > INDEX_NONE)
				{
					DynamicCollection->SetHasParent(ReleaseIndex, false);
				}
			}
		}
	}
}

#if WITH_EDITOR

void UGeometryCollectionComponent::GetBoneColors(TArray<FColor>& OutColors) const
{
	const FGeometryCollection* Collection = RestCollection->GetGeometryCollection().Get();
	const int32 NumPoints = Collection->NumElements(FGeometryCollection::VerticesGroup);
	const TManagedArray<int32>& BoneMap = Collection->BoneMap;
	const TManagedArray<FLinearColor>& BoneColors = Collection->BoneColor;

	OutColors.SetNumUninitialized(NumPoints);
	if (bChaos_GC_InitConstantDataUseParallelFor)
	{
		ParallelFor(TEXT("GC:InitBoneColors"), NumPoints, bChaos_GC_InitConstantDataParallelForBatchSize,
			[&](const int32 InPointIndex)
			{
				const int32 BoneIndex = BoneMap[InPointIndex];
				OutColors[InPointIndex] = BoneColors[BoneIndex].ToFColor(true);
			});
	}
	else
	{
		for (int32 PointIndex = 0; PointIndex < NumPoints; PointIndex++)
		{
			const int32 BoneIndex = BoneMap[PointIndex];
			OutColors[PointIndex] = BoneColors[BoneIndex].ToFColor(true);
		}
	}
}

void UGeometryCollectionComponent::GetHiddenTransforms(TArray<bool>& OutHiddenTransforms) const
{
	const FGeometryCollection* Collection = RestCollection->GetGeometryCollection().Get();
	check(Collection);

	OutHiddenTransforms.Reset();
	if (Collection->HasAttribute("Hide", FGeometryCollection::TransformGroup))
	{
		const TManagedArray<bool>& Hide = Collection->GetAttribute<bool>("Hide", FGeometryCollection::TransformGroup);
		const TManagedArray<int32>& TransformIndices = Collection->TransformIndex;
		const int32 NumGeom = Collection->NumElements(FGeometryCollection::GeometryGroup);
		const int32 NumTransforms = Collection->Transform.Num();

		OutHiddenTransforms.SetNumZeroed(NumTransforms);
		for (int32 GeometryIndex = 0; GeometryIndex < NumGeom; ++GeometryIndex)
		{
			const int32 TransformIndex = TransformIndices[GeometryIndex];
			if (Hide[TransformIndex])
			{
				OutHiddenTransforms[TransformIndex] = 1;
			}
		}
	}
}

#endif // WITH_EDITOR

void UGeometryCollectionComponent::GetRestTransforms(TArray<FMatrix44f>& OutRestTransforms) const
{
	TArray<FMatrix> RestMatrices;
	
	const int32 RestCollectionNum = RestCollection->GetGeometryCollection()->Transform.Num();
	if (RestTransforms.Num() == RestCollectionNum)
	{
		GeometryCollectionAlgo::GlobalMatrices(TManagedArray<FTransform>(RestTransforms), RestCollection->GetGeometryCollection()->Parent, RestMatrices);
	}
	else
	{
		GeometryCollectionAlgo::GlobalMatrices(RestCollection->GetGeometryCollection()->Transform, RestCollection->GetGeometryCollection()->Parent, RestMatrices);
	}
#if WITH_EDITOR
	UpdateGlobalMatricesWithExplodedVectors(RestMatrices, *(RestCollection->GetGeometryCollection()));
#endif
	CopyTransformsWithConversionWhenNeeded(OutRestTransforms, RestMatrices);
}

FGeometryCollectionDynamicData* UGeometryCollectionComponent::InitDynamicData(bool bInitialization)
{
	SCOPE_CYCLE_COUNTER(STAT_GCInitDynamicData);
	FGeometryCollectionDynamicData* DynamicData = GDynamicDataPool.Allocate();
	DynamicData->SetTransforms(ComponentSpaceTransforms.RequestAllTransforms());

#if WITH_EDITOR
	// zero out transfrom matrices if they are marked to be hidden 
	if (RestCollection && RestCollection->GetGeometryCollection())
	{
		static const FName HideAttribute{ "Hide" };

		const FGeometryCollection& Collection = *RestCollection->GetGeometryCollection();
		if (const TManagedArray<bool>* HideTransforms = Collection.FindAttribute<bool>(HideAttribute, FGeometryCollection::TransformGroup))
		{
			if (DynamicData->Transforms.Num() == HideTransforms->Num())
			{
				for (int32 TransformIndex = 0; TransformIndex < HideTransforms->Num(); TransformIndex++)
				{
					if ((*HideTransforms)[TransformIndex])
					{
						DynamicData->Transforms[TransformIndex] = FMatrix44f(EForceInit::ForceInitToZero);
					}
				}
			}
		}
	}
#endif

	return DynamicData;
}

bool UGeometryCollectionComponent::MoveComponentImpl(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult* Hit, EMoveComponentFlags MoveFlags, ETeleportType Teleport)
{
	const bool bResult = Super::MoveComponentImpl(Delta, NewRotation, bSweep, Hit, MoveFlags, Teleport);

#if WITH_EDITOR
	// Make sure that custom renderer is updated _after_ any move has been applied to the full component hierarchy.
	if (UWorld* World = GetWorld())
	{
		if (!World->IsGameWorld())
		{
			RefreshCustomRenderer();
		}
	}
#endif

	return bResult;
}

void UGeometryCollectionComponent::OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	// Don't allow the primitive component to update physics as that does processing to set a transform for
	// UPrimitiveComponent::BodyInstance as it is unecessary for geometry collections. Out update is handled below.
	Super::OnUpdateTransform(UpdateTransformFlags | EUpdateTransformFlags::SkipPhysicsUpdate, Teleport);

	const bool bSkipPhysicsUpdate = ((UpdateTransformFlags & EUpdateTransformFlags::SkipPhysicsUpdate) == EUpdateTransformFlags::SkipPhysicsUpdate);
	if (PhysicsProxy)
	{
		// Even when skipping the physics push, keep the proxy's cached world transforms in sync
		// with the actual component transform - the rebase that turns particle world XR into
		// component-local transforms depends on them being accurate.
		if (bSkipPhysicsUpdate)
		{
			if (bUpdateProxyWorldTransformWhenSkippingPhysicsUpdate)
			{
				PhysicsProxy->UpdateWorldTransformCaches_External(GetComponentTransform());
			}
		}
		else
		{
			PhysicsProxy->SetWorldTransform_External(GetComponentTransform());
		}
	}

	if (SceneProxy && SceneProxy->IsNaniteMesh())
	{
		// Nanite scene proxy requires an render update because it may fail to detect rotation and scale changes in the transform 
		MarkRenderTransformDirty();
		MarkRenderDynamicDataDirty();
	}
	
	if (bForceUpdateActiveTransforms)
	{
		RebaseDynamicCollectionTransformsOnNewWorldTransform();
	}
}

bool UGeometryCollectionComponent::HasAnySockets() const
{
	bool bHasAnySocket= false;
	if (RestCollection)
	{
		if (RestCollection->GetGeometryCollection())
		{
			if (RestCollection->GetGeometryCollection()->BoneName.Num() > 0)
			{
				bHasAnySocket = true;
			}
		}
		if (!bHasAnySocket)
		{
			for (const TObjectPtr<UStaticMesh>& ProxyMesh : RestCollection->RootProxyData.ProxyMeshes)
			{
				if (ProxyMesh && ProxyMesh->Sockets.Num() > 0)
				{
					bHasAnySocket = true;
					break;
				}
			}
		}
	}
	return bHasAnySocket;
}

bool UGeometryCollectionComponent::DoesSocketExist(FName InSocketName) const
{
	bool bFoundSocket = false;
	if (RestCollection)
	{
		if (RestCollection->GetGeometryCollection())
		{
			bFoundSocket |= RestCollection->GetGeometryCollection()->BoneName.Contains(InSocketName.ToString());
		}
		if (!bFoundSocket)
		{
			for (const TObjectPtr<UStaticMesh>& ProxyMesh : RestCollection->RootProxyData.ProxyMeshes)
			{
				if (ProxyMesh)
				{
					if (UStaticMeshSocket* MeshSocket = ProxyMesh->FindSocket(InSocketName))
					{
						bFoundSocket = true;
						break;
					}
				}
			}
		}
	}
	return bFoundSocket;
}

FTransform UGeometryCollectionComponent::GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace) const
{
	if (RestCollection)
	{
		bool bFoundSocket = false;
		FTransform SocketComponentSpaceTransform{ FTransform::Identity };
		FTransform ParentComponentSpaceTransform{ FTransform::Identity };
		if (RestCollection->GetGeometryCollection())
		{
			const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> Collection = RestCollection->GetGeometryCollection();
			if (Collection)
			{
				const int32 TransformIndex = Collection->BoneName.Find(InSocketName.ToString());
				if (TransformIndex != INDEX_NONE)
				{
					const TArray<FTransform3f>& CompSpaceTransforms = ComponentSpaceTransforms.RequestAllTransforms();
					if (CompSpaceTransforms.IsValidIndex(TransformIndex))
					{
						SocketComponentSpaceTransform = FTransform(CompSpaceTransforms[TransformIndex]);
						const int32 ParentTransformIndex = Collection->Parent[TransformIndex];
						if (CompSpaceTransforms.IsValidIndex(ParentTransformIndex))
						{
							ParentComponentSpaceTransform = FTransform(CompSpaceTransforms[ParentTransformIndex]);
						}
						bFoundSocket = true;
					}
				}
			}
		}
		if (!bFoundSocket)
		{
			const FGeometryCollectionProxyMeshData& ProxyMeshData = RestCollection->RootProxyData;
			for (int32 MeshIdx = 0; MeshIdx < ProxyMeshData.ProxyMeshes.Num(); ++MeshIdx)
			{
				const TObjectPtr<UStaticMesh>& ProxyMesh = ProxyMeshData.ProxyMeshes[MeshIdx];
				if (ProxyMesh)
				{
					if (UStaticMeshSocket* MeshSocket = ProxyMesh->FindSocket(InSocketName))
					{
						SocketComponentSpaceTransform = FTransform(MeshSocket->RelativeRotation, MeshSocket->RelativeLocation, MeshSocket->RelativeScale);
						SocketComponentSpaceTransform = SocketComponentSpaceTransform * FTransform(ProxyMeshData.GetMeshTransform(MeshIdx));
						bFoundSocket = true;
						break;
					}
				}
			}
		}
		
		if (bFoundSocket)
		{
			// convert to the right space 
			switch (TransformSpace)
			{
			case RTS_World:
				return SocketComponentSpaceTransform * GetComponentTransform();

			case RTS_Actor:
			{
				if (const AActor* Actor = GetOwner())
				{
					const FTransform SocketWorldSpaceTransform = SocketComponentSpaceTransform * GetComponentTransform();
					return SocketWorldSpaceTransform.GetRelativeTransform(Actor->GetTransform());
				}
				break;
			}

			case RTS_Component:
				return SocketComponentSpaceTransform;

			case RTS_ParentBoneSpace:
			{
				return SocketComponentSpaceTransform.GetRelativeTransform(ParentComponentSpaceTransform);
			}

			default:
				check(false);
			}
		}
	}
	return Super::GetSocketTransform(InSocketName, TransformSpace);
}

void UGeometryCollectionComponent::QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const
{
	if (RestCollection)
	{
		if (RestCollection->GetGeometryCollection())
		{
			for (const FString& BoneName : RestCollection->GetGeometryCollection()->BoneName)
			{
				FComponentSocketDescription& Desc = OutSockets.AddZeroed_GetRef();
				Desc.Name = *BoneName;
				Desc.Type = EComponentSocketType::Type::Bone;
			}
		}

		for (const TObjectPtr<UStaticMesh>& ProxyMesh : RestCollection->RootProxyData.ProxyMeshes)
		{
			if (ProxyMesh)
			{
				for (const TObjectPtr<class UStaticMeshSocket>& MeshSocket : ProxyMesh->Sockets)
				{
					if (MeshSocket)
					{
						FComponentSocketDescription& Desc = OutSockets.AddZeroed_GetRef();
						Desc.Name = MeshSocket->SocketName;
						Desc.Type = EComponentSocketType::Type::Socket;
					}
				}
			}
		}
	}
}

void UGeometryCollectionComponent::UpdateAttachedChildrenTransform() const
{
	// todo(chaos) : find a way to only update that of transform have changed
	// right now this does not work properly because the dirty flags may not be updated at the right time
	//if (PhysicsProxy && PhysicsProxy->IsGTCollectionDirty())
	{
	 	for (const TObjectPtr<USceneComponent>& AttachedChild: this->GetAttachChildren())
	 	{
			if (AttachedChild)
			{
				AttachedChild->UpdateComponentToWorld();
			}
	 	}
	}
}

bool UGeometryCollectionComponent::HasVisibleGeometry() const
{
	return CustomRenderer != nullptr || RestCollection->HasVisibleGeometry();
}

void UGeometryCollectionComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction)
{
	//UE_LOGF(UGCC_LOG, Log, "GeometryCollectionComponent[%p]::TickComponent()", this);
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

	if (BrokenAndDecayedStates.HasAnyDecaying())
	{
		InitializeRemovalDynamicAttributesIfNeeded();
		const float AdjustedDeltaTime = FMath::Max(GeometryCollectionRemovalMultiplier, 0.0001f) * DeltaTime;
		// todo(chaos) : move removal logic on the physics thread
		IncrementSleepTimer(AdjustedDeltaTime);
		IncrementBreakTimer(AdjustedDeltaTime);
	}
	// if there's no more decaying pieces, we can disable tick component
	if (!BrokenAndDecayedStates.HasAnyDecaying())
	{
		PrimaryComponentTick.SetTickFunctionEnable(false);
	}
}

void UGeometryCollectionComponent::CheckFullyDecayed()
{
	if (!bAlreadyFullyDecayed)
	{
		if (BrokenAndDecayedStates.HasFullyDecayed())
		{
			bAlreadyFullyDecayed = true;
			if (OnFullyDecayedEvent.IsBound())
			{
				SCOPE_CYCLE_COUNTER(STAT_GCFullyDecayedBroadcast);
				OnFullyDecayedEvent.Broadcast();
			}
		}
	}
}

void UGeometryCollectionComponent::AsyncPhysicsTickComponent(float DeltaTime, float SimTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(GeometryCollectionComponent_AsyncPhysicsTick);

	if (!PhysicsProxy || !PhysicsProxy->IsInitializedOnPhysicsThread())
	{
		return;
	}

#if WITH_EDITOR
	// A bit of a hack because the super async physics tick will crash if this is true.
	if (FUObjectThreadContext::Get().IsRoutingPostLoad)
	{
		return;
	}
#endif
	check(GetNetMode() != ENetMode::NM_Client);

	Super::AsyncPhysicsTickComponent(DeltaTime, SimTime);
	UpdateRepData();
}

void UGeometryCollectionComponent::OnRegister()
{
	//UE_LOGF(UGCC_LOG, Log, "GeometryCollectionComponent[%p]::OnRegister()[%p]", this,RestCollection );
	ResetDynamicCollection();

	bool bIsReplicated = false;
	const bool bHasClusterGroup = (ClusterGroupIndex != 0);
	if (bEnableReplication)
	{
		if (ensureMsgf(!bHasClusterGroup, TEXT("Replication with cluster groups is not supported - disabling replication")))
		{
			bIsReplicated = true; 
		}
	}
	SetIsReplicated(bIsReplicated);

	InitializeEmbeddedGeometry();

	InitializeCustomRenderer();

	Super::OnRegister();
}

void UGeometryCollectionComponent::OnUnregister()
{
	Super::OnUnregister();

	UnregisterCustomRenderer();
}

void UGeometryCollectionComponent::InitializeCustomRenderer()
{
	// Create a custom renderer object if a type is set on the component or the GC asset.
	UClass* Type = bOverrideCustomRenderer ? CustomRendererType : (RestCollection ? RestCollection->CustomRendererType : nullptr);
	if (Type && Type->ImplementsInterface(UGeometryCollectionExternalRenderInterface::StaticClass()))
	{
		if (!CustomRenderer || CustomRenderer.GetObject()->GetClass() != Type)
		{
			CustomRenderer = NewObject<UObject>(this, Type, NAME_None, RF_Transient | RF_DuplicateTransient);
		}
		RegisterCustomRenderer();

#if WITH_EDITOR
		// Make sure that custom renderer is updated at least once in editor.
		if (GetWorld() && !GetWorld()->IsGameWorld())
		{
			RefreshCustomRenderer();
		}
#endif
	}
	else
	{
		CustomRenderer = nullptr;
	}
}

void UGeometryCollectionComponent::RegisterCustomRenderer()
{
	bCustomRendererCanUseNativeFallback = false;

	if (IsCustomRendererAvailable())
	{
		if (IGeometryCollectionExternalRenderInterface* RendererInterface = CustomRenderer.GetInterface())
		{
			bCustomRendererCanUseNativeFallback = RendererInterface->CanEverUseNativeFallback();

			if (IsUsingCustomRenderer())
			{
				RendererInterface->OnRegisterGeometryCollection(*this);
			}
		}
	}
}

void UGeometryCollectionComponent::UnregisterCustomRenderer()
{
	if (IGeometryCollectionExternalRenderInterface* RendererInterface = CustomRenderer.GetInterface())
	{
		RendererInterface->OnUnregisterGeometryCollection();
	}
}

void UGeometryCollectionComponent::ReregisterAllCustomRenderers()
{
	for (TObjectIterator<UGeometryCollectionComponent> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
	{
		It->UnregisterCustomRenderer();
	}
	for (TObjectIterator<UGeometryCollectionComponent> It(RF_ClassDefaultObject, true, EInternalObjectFlags::Garbage); It; ++It)
	{
		It->RegisterCustomRenderer();
		It->RefreshCustomRenderer();
	}
}


void UGeometryCollectionComponent::InitializeRemovalDynamicAttributesIfNeeded()
{
	if (bInitializedRemovalDynamicAttribute == false)
	{
		FGeometryCollectionDecayDynamicFacade DecayDynamicFacade(*DynamicCollection);

		// we are not testing for bAllowRemovalOnSleep, so that we can enable it at runtime if necessary
		if (RestCollection->bRemoveOnMaxSleep)
		{
			DecayDynamicFacade.AddAttributes();

			FGeometryCollectionRemoveOnSleepDynamicFacade RemoveOnSleepDynamicFacade(*DynamicCollection);
			RemoveOnSleepDynamicFacade.DefineSchema();
			RemoveOnSleepDynamicFacade.SetAttributeValues(RestCollection->MaximumSleepTime, RestCollection->RemovalDuration);
		}

		// Remove on break feature related dynamic attribute arrays
		// we are not testing for bAllowRemovalOnBreak, so that we can enable it at runtime if necessary
		GeometryCollection::Facades::FCollectionRemoveOnBreakFacade RemoveOnBreakFacade(*RestCollection->GetGeometryCollection());
		if (RemoveOnBreakFacade.IsValid())
		{
			DecayDynamicFacade.AddAttributes();

			FGeometryCollectionRemoveOnBreakDynamicFacade RemoveOnBreakDynamicFacade(*DynamicCollection);
			RemoveOnBreakDynamicFacade.DefineSchema();
			RemoveOnBreakDynamicFacade.SetAttributeValues(RemoveOnBreakFacade);
		}
		bInitializedRemovalDynamicAttribute = true;
	}
}

void UGeometryCollectionComponent::ResetDynamicCollection()
{
	bool bCreateDynamicCollection = true;
#if WITH_EDITOR
	bCreateDynamicCollection = CanRunSimulationInEditor();
#endif
	if (bCreateDynamicCollection && RestCollection && RestCollection->GetGeometryCollection())
	{
		DynamicCollection = MakeShared<FGeometryDynamicCollection>(RestCollection->GetGeometryCollection());

		if (bStoreVelocities || bNotifyTrailing)
		{
			DynamicCollection->AddVelocitiesAttributes();
		}

		DynamicCollection->MakeDirty();
		MarkRenderStateDirty();
		MarkRenderDynamicDataDirty();
	}
	else
	{
		DynamicCollection.Reset();
	}

	const int32 NumRestCollectionTransforms = RestCollection ? RestCollection->GetGeometryCollection()->Transform.Num() : 0;
	if (RestTransforms.Num() != NumRestCollectionTransforms)
	{
		RestTransforms.Reset();
	}

	// if Reset transform have been overridden uses them to initialize the dynamic collection transforms
	if (RestTransforms.Num() > 0)
	{
		SetInitialTransforms(RestTransforms);
	}

	ComponentSpaceTransforms.Reset(NumRestCollectionTransforms, GetRootIndex());

	RootSpaceBounds.Init();
	UpdateCachedBounds();
	bInitializedRemovalDynamicAttribute = false;
	BrokenAndDecayedStates.Reset(NumRestCollectionTransforms);
}

void UGeometryCollectionComponent::OnCreatePhysicsState()
{
	// Skip the chain - don't care about body instance setup
	UActorComponent::OnCreatePhysicsState();
	if (!BodyInstance.bSimulatePhysics) IsObjectLoading = false; // just mark as loaded if we are simulating.

	// Static mesh uses an init framework that goes through FBodyInstance.  We
	// do the same thing, but through the geometry collection proxy and lambdas
	// defined below.  FBodyInstance doesn't work for geometry collections 
	// because FBodyInstance manages a single particle, where we have many.
	if (!PhysicsProxy && RestCollection)
	{
#if WITH_EDITOR && WITH_EDITORONLY_DATA
		EditorActor = nullptr;

		if (RestCollection)
		{
			//hack: find a better place for this
			UGeometryCollection* RestCollectionMutable = const_cast<UGeometryCollection*>(ToRawPtr(RestCollection));
			RestCollectionMutable->CreateSimulationDataIfNeeded();
		}
#endif
		const bool bValidWorld = CanRunSimulationInEditor() || (GetWorld() && GetWorld()->IsPreviewWorld());
		const bool bValidCollection = DynamicCollection && DynamicCollection->GetNumTransforms() > 0;
		if (bValidWorld && bValidCollection)
		{
			FChaosUserData::Set<UPrimitiveComponent>(&PhysicsUserData, this);

			// If the Component is set to Dynamic, we look to the RestCollection for initial dynamic state override per transform.
			TManagedArray<uint8> & DynamicState = DynamicCollection->DynamicState;

			// if this code is changed you may need to account for bStartAwake
			EObjectStateTypeEnum LocalObjectType = (ObjectType != EObjectStateTypeEnum::Chaos_Object_Sleeping) ? ObjectType : EObjectStateTypeEnum::Chaos_Object_Dynamic;
			if (LocalObjectType != EObjectStateTypeEnum::Chaos_Object_UserDefined)
			{
				if (RestCollection && (LocalObjectType == EObjectStateTypeEnum::Chaos_Object_Dynamic))
				{
					TManagedArray<int32>& InitialDynamicState = RestCollection->GetGeometryCollection()->InitialDynamicState;
					for (int i = 0; i < DynamicState.Num(); i++)
					{
						DynamicState[i] = (InitialDynamicState[i] == static_cast<int32>(Chaos::EObjectStateType::Uninitialized)) ? static_cast<uint8>(LocalObjectType) : InitialDynamicState[i];
					}
				}
				else
				{
					for (int i = 0; i < DynamicState.Num(); i++)
					{
						DynamicState[i] = static_cast<uint8>(LocalObjectType);
					}
				}
			}

			TManagedArray<bool>& Active = DynamicCollection->Active;
			if (RestCollection->GetGeometryCollection()->HasAttribute(FGeometryCollection::SimulatableParticlesAttribute, FTransformCollection::TransformGroup))
			{
				TManagedArray<bool>* SimulatableParticles = RestCollection->GetGeometryCollection()->FindAttribute<bool>(FGeometryCollection::SimulatableParticlesAttribute, FTransformCollection::TransformGroup);
				for (int i = 0; i < Active.Num(); i++)
				{
					Active[i] = (*SimulatableParticles)[i];
				}
			}
			else
			{
				// If no simulation data is available then default to the simulation of just the rigid geometry.
				for (int i = 0; i < Active.Num(); i++)
				{
					Active[i] = RestCollection->GetGeometryCollection()->IsRigid(i);
				}
			}
			
			// there's a code path where Level is not serialized and InitializeSharedCollisionStructures is not being called,
			// resulting in the attribute missing and causing a crash in CopyAttribute calls later in FGeometryCollectionPhysicsProxy::Initialize
			// @todo(chaos) we should better handle computation of dependent attribute like level
			// @todo(chaos) We should implement a facade for levels, (parent and child included ? )
			if (!RestCollection->GetGeometryCollection()->HasAttribute("Level", FTransformCollection::TransformGroup))
			{
				TManagedArray<int32>& Levels = RestCollection->GetGeometryCollection()->AddAttribute<int32>("Level", FTransformCollection::TransformGroup);
				for (int32 TransformIndex = 0; TransformIndex < Levels.Num(); ++TransformIndex)
				{
					FGeometryCollectionPhysicsProxy::CalculateAndSetLevel(TransformIndex, RestCollection->GetGeometryCollection()->Parent,  Levels);
				}
			}
			
			// let's copy anchored information if available
			const Chaos::Facades::FCollectionAnchoringFacade RestCollectionAnchoringFacade(*RestCollection->GetGeometryCollection());
			Chaos::Facades::FCollectionAnchoringFacade DynamicCollectionAnchoringFacade(*DynamicCollection);
			DynamicCollectionAnchoringFacade.CopyAnchoredAttribute(RestCollectionAnchoringFacade);
	
			// Set up initial filter data for our particles
			// #BGTODO We need a dummy body setup for now to allow the body instance to generate filter information. Change body instance to operate independently.
			DummyBodySetup = NewObject<UBodySetup>(this, UBodySetup::StaticClass());
			BodyInstance.BodySetup = DummyBodySetup;
			BodyInstance.OwnerComponent = this; // Required to make filter data include component/actor ID for ignored actors/components

			BuildInitialFilterData();

			if (BodyInstance.bSimulatePhysics)
			{
				RegisterAndInitializePhysicsProxy();

				// We're skipping over the primitive component so we need to make sure this event gets fired.
				OnComponentPhysicsStateChanged.Broadcast(this, EComponentPhysicsStateChange::Created);
			}
		}
	}
}


static FORCEINLINE_DEBUGGABLE int32 ComputeParticleLevel(Chaos::FPBDRigidClusteredParticleHandle* Particle)
{
	int32 Level = 0;
	if (Particle)
	{
		Chaos::FPBDRigidClusteredParticleHandle* Current = Particle;
		while (Current->Parent())
		{
			Current = Current->Parent();
			++Level;
		}
	}
	return Level;
};

void UGeometryCollectionComponent::RegisterAndInitializePhysicsProxy()
{
	check(DynamicCollection != nullptr);

	// CVar defined in BodyInstance but pertinent here as we will need to copy simplicials in the case that this is set.
	// Original CVar is read-only so taking a static ptr here is fine as the value cannot be changed
	static IConsoleVariable* AnalyticDisableCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("p.IgnoreAnalyticCollisionsOverride"));
	static const bool bAnalyticsDisabled = (AnalyticDisableCVar && AnalyticDisableCVar->GetBool());

	FSimulationParameters SimulationParameters;
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		GetGeometryCollectionComponentDebugDebugName(*this, SimulationParameters.Name);
#endif
		EClusterConnectionTypeEnum ClusterCollectionType = ClusterConnectionType_DEPRECATED;
		float ConnectionGraphBoundsFilteringMargin = 0;
		bool bUseSimplicialWhenAvailable = false;
		if (RestCollection)
		{
			RestCollection->GetSharedSimulationParams(SimulationParameters.Shared);
			SimulationParameters.RestCollectionShared = RestCollection->GetGeometryCollection();
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			// To be removed when RestCollection is removed post-deprecation. Here for back compat
			SimulationParameters.RestCollection = SimulationParameters.RestCollectionShared.Get();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			SimulationParameters.InitialRootIndex = RestCollection->GetRootIndex();
			ClusterCollectionType = RestCollection->ClusterConnectionType;
			ConnectionGraphBoundsFilteringMargin = RestCollection->ConnectionGraphBoundsFilteringMargin;
			bUseSimplicialWhenAvailable =
				FGeometryCollection::AreCollisionParticlesEnabled()
				&& SimulationParameters.RestCollectionShared
				&& SimulationParameters.RestCollectionShared->HasAttribute(FGeometryDynamicCollection::SimplicialsAttribute, FTransformCollection::TransformGroup)
				&& SimulationParameters.Shared.SizeSpecificData[0].CollisionShapesData.Num()
				&& (SimulationParameters.Shared.SizeSpecificData[0].CollisionShapesData[0].CollisionType == ECollisionTypeEnum::Chaos_Surface_Volumetric || bAnalyticsDisabled)
				;
		}
		SimulationParameters.Simulating = BodyInstance.bSimulatePhysics;
		SimulationParameters.EnableClustering = EnableClustering;
		SimulationParameters.ClusterGroupIndex = EnableClustering ? ClusterGroupIndex : 0;
		SimulationParameters.MaxClusterLevel = MaxClusterLevel;
		SimulationParameters.MaxSimulatedLevel = MaxSimulatedLevel;
		SimulationParameters.DamageModel = DamageModel;
		SimulationParameters.DamageEvaluationModel = GetDamageEvaluationModel(DamageModel);
		SimulationParameters.bUseSizeSpecificDamageThresholds = bUseSizeSpecificDamageThreshold;
		SimulationParameters.bUseMaterialDamageModifiers = bUseMaterialDamageModifiers;
		SimulationParameters.DamageThreshold = DamageThreshold;
		SimulationParameters.bUsePerClusterOnlyDamageThreshold = RestCollection? RestCollection->PerClusterOnlyDamageThreshold: false; 
		SimulationParameters.ClusterConnectionMethod = (Chaos::FClusterCreationParameters::EConnectionMethod)ClusterCollectionType;
		SimulationParameters.ConnectionGraphBoundsFilteringMargin = ConnectionGraphBoundsFilteringMargin; 
		SimulationParameters.bForceUpdateActiveTransforms = bForceUpdateActiveTransforms;
		SimulationParameters.CollisionGroup = CollisionGroup;
		SimulationParameters.CollisionSampleFraction = CollisionSampleFraction;
		SimulationParameters.InitialVelocityType = InitialVelocityType;
		SimulationParameters.InitialLinearVelocity = FVector3f(InitialLinearVelocity);
		SimulationParameters.InitialAngularVelocity = FVector3f(InitialAngularVelocity);
		SimulationParameters.ObjectType = ObjectType;
		SimulationParameters.StartAwake = BodyInstance.bStartAwake;
		SimulationParameters.bGenerateBreakingData = bNotifyBreaks;
		SimulationParameters.bGenerateCollisionData = bNotifyCollisions;
		SimulationParameters.bGenerateTrailingData = bNotifyTrailing;
		SimulationParameters.bGenerateCrumblingData = bNotifyCrumblings;
		SimulationParameters.bGenerateCrumblingChildrenData = bCrumblingEventIncludesChildren;
		SimulationParameters.bGenerateGlobalBreakingData = bNotifyGlobalBreaks;
		SimulationParameters.bGenerateGlobalCollisionData = bNotifyGlobalCollisions;
		SimulationParameters.bGenerateGlobalCrumblingData = bNotifyGlobalCrumblings;
		SimulationParameters.bGenerateGlobalTrailingData = bNotifyGlobalTrailings;
		SimulationParameters.bGenerateGlobalCrumblingChildrenData = bGlobalCrumblingEventIncludesChildren;
		SimulationParameters.EnableGravity = BodyInstance.bEnableGravity;
		SimulationParameters.GravityGroupIndex = GravityGroupIndex;
		SimulationParameters.OneWayInteractionLevel = OneWayInteractionLevel;
		SimulationParameters.UseInertiaConditioning = BodyInstance.IsInertiaConditioningEnabled();
		SimulationParameters.UseCCD = BodyInstance.bUseCCD;
		SimulationParameters.UseMACD = BodyInstance.GetUseMACD();
		SimulationParameters.AllowPartialIslandSleep = BodyInstance.GetAllowPartialIslandSleep();
		const int32 PositionSolverIterationCount = BodyInstance.GetPositionSolverIterationCount();
		SimulationParameters.PositionSolverIterations = (PositionSolverIterationCount >= 0) ? FMath::Min(PositionSolverIterationCount, 255) : 8;
		const int32 VelocitySolverIterationCount = BodyInstance.GetVelocitySolverIterationCount();
		SimulationParameters.VelocitySolverIterations = (VelocitySolverIterationCount >= 0) ? FMath::Min(VelocitySolverIterationCount, 255) : 1;
		const int32 ProjectionSolverIterationCount = BodyInstance.GetProjectionSolverIterationCount();
		SimulationParameters.ProjectionSolverIterations = (ProjectionSolverIterationCount >= 0) ? FMath::Min(ProjectionSolverIterationCount, 255) : 1;
		SimulationParameters.LinearDamping = BodyInstance.LinearDamping;
		SimulationParameters.AngularDamping = BodyInstance.AngularDamping;
		SimulationParameters.InitialOverlapDepenetrationVelocity = BodyInstance.GetMaxDepenetrationVelocity();
		SimulationParameters.SleepThresholdMultiplier = BodyInstance.GetSleepThresholdMultiplier();
		SimulationParameters.bUseDamagePropagation = DamagePropagationData.bEnabled;
		SimulationParameters.bGyroscopicTorqueEnabled = BodyInstance.bGyroscopicTorqueEnabled;
		SimulationParameters.BreakDamagePropagationFactor = DamagePropagationData.BreakDamagePropagationFactor;
		SimulationParameters.ShockDamagePropagationFactor = DamagePropagationData.ShockDamagePropagationFactor;
		SimulationParameters.WorldTransform = GetComponentToWorld();
		SimulationParameters.UserData = static_cast<void*>(&PhysicsUserData);
		SimulationParameters.bEnableStrainOnCollision = bEnableDamageFromCollision;
		SimulationParameters.bUseStaticMeshCollisionForTraces = bUseStaticMeshCollisionForTraces;
		SimulationParameters.bOptimizeConvexes = RestCollection ? RestCollection->bOptimizeConvexes : true;
		SimulationParameters.bUseSimplicialsWhenAvailable = bUseSimplicialWhenAvailable;

#if SIMULATIONPARAMETERS_CACHE_PARAMETERS
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		SimulationParameters.bClearCache = true;
		SimulationParameters.CacheType = CacheParameters.CacheMode;
		SimulationParameters.ReverseCacheBeginTime = CacheParameters.ReverseCacheBeginTime;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

		UPhysicalMaterial* EnginePhysicalMaterial = GetPhysicalMaterial();
		if (ensure(EnginePhysicalMaterial))
		{
			SimulationParameters.PhysicalMaterialHandle = EnginePhysicalMaterial->GetPhysicsMaterial();
		}

		// mass properties are cooked in the GC asset, but component can override physics material with a different density
		// if the user wants the density to be set from the physics material we need adjust by scaling mass by the ratio override/cooked defined properties
		SimulationParameters.MaterialOverrideMassScaleMultiplier = ComputeMassScaleRelativeToAsset();

		GetInitializationCommands(SimulationParameters.InitializationCommands);
	}

	FGuid CollectorGuid = FGuid::NewGuid();
#if WITH_EDITORONLY_DATA
	CollectorGuid = RunTimeDataCollectionGuid;
	if (bEnableRunTimeDataCollection && RestCollection)
	{
		FRuntimeDataCollector::GetInstance().AddCollector(CollectorGuid, RestCollection->NumElements(FGeometryCollection::TransformGroup));
	}
	else
	{
		FRuntimeDataCollector::GetInstance().RemoveCollector(CollectorGuid);
	}
#endif
	PhysicsProxy = new FGeometryCollectionPhysicsProxy(this, DynamicCollection.ToSharedRef(), SimulationParameters, InitialShapeFilterData, InitialFilterInstanceData, CollectorGuid);

	PhysicsProxy->SetPostPhysicsSyncCallback(
		[WeakThis = MakeWeakObjectPtr(this)]()
		{
			if (UGeometryCollectionComponent* This = WeakThis.Get())
			{
				This->OnPostPhysicsSync();
			}
		}
	);

	PhysicsProxy->SetPostParticlesCreatedCallback(
		[WeakThis = MakeWeakObjectPtr(this)]()
		{
			if (UGeometryCollectionComponent* This = WeakThis.Get())
			{
				This->OnPostCreateParticles();
			}
		}
	);

	if (GetIsReplicated())
	{
		// using net mode and not local role because at this time in the initialization client and server both have an authority local role
		const ENetMode NetMode = GetNetMode();
		if (NetMode != NM_Standalone)
		{
			const FGeometryCollectionPhysicsProxy::EReplicationMode ReplicationMode =
				(NetMode == ENetMode::NM_Client)
				? FGeometryCollectionPhysicsProxy::EReplicationMode::Client
				: FGeometryCollectionPhysicsProxy::EReplicationMode::Server;
				PhysicsProxy->SetReplicationMode(ReplicationMode);
		}
	}

	auto CreateTraceCollisionFromStaticMeshGeometry = [this](const FTransform& InToLocal, TArray<Chaos::FImplicitObjectPtr>& OutGeoms, Chaos::FShapesArray& OutShapes) {
		const FGeometryCollectionProxyMeshData& ProxyMeshData = RestCollection->RootProxyData;
		for (int32 MeshIdx = 0; MeshIdx < ProxyMeshData.ProxyMeshes.Num(); ++MeshIdx)
		{
			const TObjectPtr<UStaticMesh>& ProxyMesh = RestCollection->RootProxyData.ProxyMeshes[MeshIdx];
			if (const UStaticMesh* StaticMesh = ProxyMesh.Get())
			{
				if (UBodySetup* BodySetup = ProxyMesh->GetBodySetup())
				{
					FBodyCollisionData BodyCollisionData;
					BodyInstance.BuildBodyFilterData(BodyCollisionData.CollisionFilterData);
					FBodyInstance::BuildBodyCollisionFlags(BodyCollisionData.CollisionFlags, BodyInstance.GetCollisionEnabled(), BodySetup->GetCollisionTraceFlag() == CTF_UseComplexAsSimple);

					const FTransform MeshLocalTransform(ProxyMeshData.GetMeshTransform(MeshIdx));

					FGeometryAddParams AddParams;
					AddParams.bDoubleSided = BodySetup->bDoubleSidedGeometry;
					AddParams.CollisionData = BodyCollisionData;
					AddParams.CollisionTraceType = BodySetup->GetCollisionTraceFlag();
					AddParams.WorldTransform = PhysicsProxy->GetWorldTransform_External();
					AddParams.Scale = AddParams.WorldTransform.GetScale3D();
					AddParams.SimpleMaterial = GetPhysicalMaterial();
					AddParams.LocalTransform = InToLocal * MeshLocalTransform;

					AddParams.Geometry = &BodySetup->AggGeom;
					ChaosInterface::CreateGeometry(AddParams, OutGeoms, OutShapes);
				}
			}
		}
	};

	// todo(chaos): Remove this and move to a cook time approach of the SM data based on the GC property
	if (RestCollection != nullptr)
	{
		for (const TObjectPtr<UStaticMesh>& ProxyMesh : RestCollection->RootProxyData.ProxyMeshes)
		{
			if (const UStaticMesh* StaticMesh = ProxyMesh.Get())
			{
				if (UBodySetup* BodySetup = ProxyMesh->GetBodySetup())
				{
					// We have at least one valid mesh, so set the callback in case we are asked to use SM collision for traces
					PhysicsProxy->SetCreateTraceCollisionGeometryCallback(CreateTraceCollisionFromStaticMeshGeometry);
					break;
				}
			}
		}
	}

	FPhysScene_Chaos* PhysicsScene = GetInnerChaosScene();
	if (ensure(PhysicsScene))
	{
		PhysicsScene->AddObject(this, PhysicsProxy);

		// If we're replicating we need some extra setup - check netmode as we don't need this for standalone runtime where we aren't going to network the component
		// IMPORTANT this need to happen after the object is registered so this will guarantee that the particles are properly created by the time the callback below gets called
		if (GetIsReplicated() && PhysicsProxy->GetReplicationMode() == FGeometryCollectionPhysicsProxy::EReplicationMode::Client)
		{
			// Client side : geometry collection children of parents below the rep level need to be infinitely strong so that client cannot break it 
			if (Chaos::FPhysicsSolver* CurrSolver = GetSolver(*this))
			{
				CurrSolver->EnqueueCommandImmediate([Proxy = PhysicsProxy, AbandonAfterLevel = ReplicationAbandonAfterLevel, EnableAbandonAfterLevel = bEnableAbandonAfterLevel]()
					{
						// As we're not in control we make it so our simulated proxy cannot break clusters
						// We have to set the strain to a high value but be below the max for the data type
						// so releasing on authority demand works
						for (Chaos::FPBDRigidClusteredParticleHandle* ParticleHandle : Proxy->GetUnorderedParticles_Internal())
						{
							if (ParticleHandle)
							{
								const int32 Level = EnableAbandonAfterLevel ? ComputeParticleLevel(ParticleHandle) : -1;
								if (Level <= AbandonAfterLevel)	//we only replicate up until level X, but it means we should replicate the breaking event of level X+1 (but not X+1's positions)
								{
									ParticleHandle->SetUnbreakable(true);
								}
							}
						}
					});
			}
		}

		LoadCollisionProfiles();

		// We need to add the geometry collection into the external acceleration structure so that it's immediately available for queries instead of waiting for the sync from the physics thread (which could take awhile).
		// Just adding the root particle should be sufficient since that'll be the only particle we'd expect any collisions with right after initialization.
		if (Chaos::FPhysicsObjectHandle RootObject = GetPhysicsObjectByName(NAME_None))
		{
			TArrayView<Chaos::FPhysicsObjectHandle> Handles{ &RootObject, 1 };
			FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(Handles);
			Interface->AddToSpatialAcceleration(Handles, PhysicsScene->GetSpacialAcceleration());
		}
	}

	RegisterForEvents();
}

void UGeometryCollectionComponent::UpdateBrokenAndDecayedStates()
{
	if (PhysicsProxy && DynamicCollection && DynamicCollection->Active.Num() > 0)
	{
		const int32 RootIndex = GetRootIndex();
		if (RootIndex != INDEX_NONE)
		{
			const int32 NumTransforms = DynamicCollection->GetNumTransforms();
			
			const bool bIsRootBroken = !DynamicCollection->Active[RootIndex];
			BrokenAndDecayedStates.SetRootIsBroken(bIsRootBroken);
			// we mark it both broken and decayed
			BrokenAndDecayedStates.SetIsBroken(RootIndex);
			BrokenAndDecayedStates.SetHasDecayed(RootIndex);

			if (BrokenAndDecayedStates.GetIsRootBroken())
			{
				const FGeometryCollectionDynamicStateFacade DynamicStateFacade(*DynamicCollection);
				if (DynamicStateFacade.IsValid())
				{
					for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
					{
						if (BrokenAndDecayedStates.GetIsBroken(TransformIdx) || BrokenAndDecayedStates.GetHasDecayed(TransformIdx))
						{
							// already decayed nothing to do anymore 
							continue;
						}

						const Chaos::FPBDRigidParticle* Particle = PhysicsProxy->GetParticleByIndex_External(TransformIdx);
						if (!Particle)
						{
							// when no particle we mark it as a decayed one
							BrokenAndDecayedStates.SetHasDecayed(TransformIdx);
							continue;
						}

						if (!BrokenAndDecayedStates.GetIsBroken(TransformIdx))
						{
							const bool bInternalClusterParentBrokenOff = DynamicStateFacade.HasDynamicInternalClusterParent(TransformIdx) && !DynamicStateFacade.HasClusterUnionParent(TransformIdx);
							const bool bSelfBrokenOff = DynamicStateFacade.HasBrokenOff(TransformIdx) && !DynamicStateFacade.HasInternalClusterParent(TransformIdx);
							if (bSelfBrokenOff || bInternalClusterParentBrokenOff)
							{
								BrokenAndDecayedStates.SetIsBroken(TransformIdx);								
							}
						}

						if (BrokenAndDecayedStates.GetIsBroken(TransformIdx))
						{
							if (Particle->Disabled())
							{
								// Todo(chaos) in the future we shoud probably store a bitArray to know if the bone is a simulated leaf
								bool bHasAnyChildWithValidParticle = false;
								DynamicCollection->IterateThroughChildren(TransformIdx, 
									[&bHasAnyChildWithValidParticle, this] (int32 ChildTransformIdx) -> bool
									{
										if (PhysicsProxy->GetParticleByIndex_External(ChildTransformIdx))
										{
											bHasAnyChildWithValidParticle = true;
											return false; // do no continue iterating
										}
										return true; // continue iterating
									});

								if (!bHasAnyChildWithValidParticle)
								{
									BrokenAndDecayedStates.SetHasDecayed(TransformIdx);
								}
							}
						}
					}
				}
			}
		}
	}
}

bool UGeometryCollectionComponent::FBrokenAndDecayedStates::GetIsBroken(int32 TransformIndex) const
{
	return bIsRootBroken ? IsBroken[TransformIndex] : false;
}
bool UGeometryCollectionComponent::FBrokenAndDecayedStates::GetHasDecayed(int32 TransformIndex) const
{
	return bIsRootBroken ? HasDecayed[TransformIndex] : false;
}

double UGeometryCollectionComponent::FBrokenAndDecayedStates::GetRootBrokenEventTimeInMs() const
{ 
	return RootBrokenEventTimeInMs;
}

double UGeometryCollectionComponent::FBrokenAndDecayedStates::GetRootBrokenElapsedTimeInMs() const
{
	if (bIsRootBroken)
	{
		const double CurrentTimeInMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
		return (CurrentTimeInMs - RootBrokenEventTimeInMs);
	}
	return 0;
}

void UGeometryCollectionComponent::FBrokenAndDecayedStates::Reset(int32 InNumTransforms)
{
	NumTransforms = InNumTransforms;
	bIsRootBroken = false;
	NumDecaying = 0;
	IsBroken.Empty();
	HasDecayed.Empty();
	RootBrokenEventTimeInMs = 0;
}

void UGeometryCollectionComponent::FBrokenAndDecayedStates::SetRootIsBroken(bool bIsBroken)
{
	if (bIsBroken != bIsRootBroken)
	{
		bIsRootBroken = bIsBroken;
		NumDecaying = 0;
		if (bIsBroken)
		{
			IsBroken.Init(false, NumTransforms);
			HasDecayed.Init(false, NumTransforms);
			RootBrokenEventTimeInMs = FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64());
		}
		else
		{
			IsBroken.Empty();
			HasDecayed.Empty();
			RootBrokenEventTimeInMs = 0;
		}
	}
}

void UGeometryCollectionComponent::FBrokenAndDecayedStates::SetIsBroken(int32 TransformIndex)
{
	if (bIsRootBroken && !IsBroken[TransformIndex])
	{
		IsBroken[TransformIndex] = true;
		NumDecaying++;
	}
}

void UGeometryCollectionComponent::FBrokenAndDecayedStates::SetHasDecayed(int32 TransformIndex)
{
	if (bIsRootBroken && !HasDecayed[TransformIndex])
	{
		HasDecayed[TransformIndex] = true;
		if (IsBroken[TransformIndex])
		{
			NumDecaying--;
		}
	}
}

void UGeometryCollectionComponent::FBrokenAndDecayedStates::SetHasDecayedRecursive(int32 TransformIndex, const TArray<TSet<int32>>& Children)
{
	if (bIsRootBroken && ensure(Children.Num() == HasDecayed.Num()))
	{
		if (!HasDecayed[TransformIndex])
		{
			HasDecayed[TransformIndex] = true;
			if (IsBroken[TransformIndex])
			{
				NumDecaying--;
			}
			for (const int32 ChildIndex : Children[TransformIndex])
			{
				SetHasDecayedRecursive(ChildIndex, Children);
			}
		}
	}
}

bool UGeometryCollectionComponent::FBrokenAndDecayedStates::HasAnyDecaying() const
{
	return (NumDecaying > 0);
}

bool UGeometryCollectionComponent::FBrokenAndDecayedStates::HasFullyDecayed() const
{
	if (HasAnyDecaying())
	{
		return false;
	}
	return (HasDecayed.CountSetBits() == HasDecayed.Num());
}

void UGeometryCollectionComponent::OnTransformsDirty()
{
	ComponentSpaceTransforms.MarkDirty();
	ComponentSpaceBounds.Init();
}

void UGeometryCollectionComponent::OnPostCreateParticles()
{
	LoadCollisionProfiles();
}

void UGeometryCollectionComponent::OnPostPhysicsSync()
{
	SCOPE_CYCLE_COUNTER(STAT_GCPostPhysicsSync);

	// dirty the transform if the collection is
	bool bHasRootMoved = false;
	if (DynamicCollection && DynamicCollection->IsDirty())
	{
		// Can't be a const reference - we need to make a copy or else the next RequestRootTransform will change the value under our nose.
		const FTransform3f PreviousRootTransform = OnRootMovedEvent.IsBound() ? ComponentSpaceTransforms.RequestRootTransform() : FTransform3f::Identity;

		OnTransformsDirty();
		
		if (OnRootMovedEvent.IsBound() || OnRootMovedNativeEvent.IsBound())
		{
			const FTransform3f& NewRootTransform = ComponentSpaceTransforms.RequestRootTransform();
			bHasRootMoved = !PreviousRootTransform.EqualsNoScale(NewRootTransform);
		}
	}

	UpdateAttachedChildrenTransform();

	if (GetIsReplicated() && GetNetMode() != ENetMode::NM_Client)
	{
		// The GameThreadCollection dirty flag doesn't correspond to the "dirtiness" that should trigger replication.
		// So as long as the physics sync happens, check for potential replication updates.
		RequestUpdateRepData();
	}

	// Once the GC is broken, removal feature will need the tick to properly update the timers 
	// even if the physics does not get any updates
	UpdateBrokenAndDecayedStates();
	if (BrokenAndDecayedStates.GetIsRootBroken())
	{
		if (GeometryCollectionEmitRootBreakingEvent)
		{
			if (OnRootBreakEvent.IsBound())
			{
				FChaosBreakEvent Event;
				Event.Index = GetRootIndex();
				Event.Component = this;
				OnRootBreakEvent.Broadcast(Event);
				OnRootBreakEvent.Clear(); // one shot
			}
		}

		if (BrokenAndDecayedStates.HasAnyDecaying())
		{
			if (!PrimaryComponentTick.IsTickFunctionEnabled())
			{
				PrimaryComponentTick.SetTickFunctionEnable(true);
			}
		}
		else
		{
			InitializeRemovalDynamicAttributesIfNeeded();
		}

		UpdateRemovalIfNeeded();

		CheckFullyDecayed();
	}
	else if (bUpdateComponentTransformToRootBone)
	{
		MoveComponentToRootTransform();
	}

	const bool bDynamicDataIsDirty = (DynamicCollection && DynamicCollection->IsDirty() && HasVisibleGeometry());
	UpdateRenderSystemsIfNeeded(bDynamicDataIsDirty);
	UpdateNavigationDataIfNeeded(bDynamicDataIsDirty);

	// at this is called at the end as thos events may change some internal values of the component
	// and interfere with transforms required for naviagtion or rendering 
	if (bHasRootMoved)
	{
		if (OnRootMovedEvent.IsBound())
		{
			OnRootMovedEvent.Broadcast();
		}
		if (OnRootMovedNativeEvent.IsBound())
		{
			OnRootMovedNativeEvent.Broadcast(this);
		}
	}
}

void UGeometryCollectionComponent::MoveComponentToRootTransform()
{
	if (RestCollection && PhysicsProxy)
	{
		const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> AssetCollection = RestCollection->GetGeometryCollection();

		if (AssetCollection && DynamicCollection && DynamicCollection->IsDirty())
		{
			static FName MassToLocalAttributeName = "MassToLocal";
			const TManagedArray<FTransform>& MassToLocalAttribute = RestCollection->GetGeometryCollection()->GetAttribute<FTransform>(MassToLocalAttributeName, FGeometryCollection::TransformGroup);

			FGeometryCollectionDynamicStateFacade DynamicStateFacade(*DynamicCollection);

			const int32 RootIndex = GetRootIndex();

			// bUpdateComponentTransformToRootBone only supports a dynamically simulated root that is
			// not a cluster union member. Other configurations are out of scope for this feature.
			if (!DynamicStateFacade.IsDynamicOrSleeping(RootIndex) || DynamicStateFacade.HasClusterUnionParent(RootIndex))
			{
				return;
			}

			const bool bIsRootActive = DynamicStateFacade.IsDynamicOrSleeping(RootIndex);
			const bool bHasDynamicOPrClusterUnionParent = DynamicStateFacade.HasDynamicInternalClusterParent(RootIndex) || !DynamicStateFacade.HasClusterUnionParent(RootIndex);
			if (bIsRootActive || bHasDynamicOPrClusterUnionParent)
			{
				const FTransform3f& OriginalComponentSpaceRootTransformOffset = AssetCollection->Transform[RootIndex];
				DynamicCollection->SetTransform(RootIndex, OriginalComponentSpaceRootTransformOffset);
				const Chaos::FPBDRigidParticle* RootParticle = PhysicsProxy->GetParticleByIndex_External(RootIndex);
				const FTransform ParticleWorldPosition(RootParticle->R(), RootParticle->X());
				const FTransform MassToLocal = MassToLocalAttribute[RootIndex];
				const FTransform NewRootWorldPosition = MassToLocal.Inverse() * ParticleWorldPosition;

				const FTransform CurrentComponentTransform = GetComponentTransform();
				if (!NewRootWorldPosition.EqualsNoScale(CurrentComponentTransform))
				{
					const FVector NewPosition = NewRootWorldPosition.GetLocation();
					const FVector MoveBy = NewPosition - CurrentComponentTransform.GetLocation();
					const FRotator NewRotation = NewRootWorldPosition.Rotator();
					MoveComponent(MoveBy, NewRotation, /* bSweep */ false,/* Hit */ NULL, MOVECOMP_SkipPhysicsMove);
				}

				OnTransformsDirty();
			}
		}
	}
}

void UGeometryCollectionComponent::UpdateRenderSystemsIfNeeded(bool bDynamicCollectionDirty)
{
#if WITH_EDITOR
	if (IsRegistered() && SceneProxy && RestCollection)
	{
		const bool bWantNanite = UseNanite(GMaxRHIShaderPlatform) && RestCollection->EnableNanite && GGeometryCollectionNanite != 0;
		const bool bHaveNanite = SceneProxy->IsNaniteMesh();
		bool bRecreateProxy = bWantNanite != bHaveNanite;
		if (bRecreateProxy)
		{
			// Wait until resources are released
			FlushRenderingCommands();

			FComponentReregisterContext ReregisterContext(this);
			UpdateAllPrimitiveSceneInfosForSingleComponent(this);
		}
	}
#endif

	if (bDynamicCollectionDirty)
	{
		// #todo review: When we've made changes to ISMC, we need to move this function call to SetRenderDynamicData_Concurrent
		RefreshEmbeddedGeometry();

		if (bUpdateCustomRendererOnPostPhysicsSync)
		{
			RefreshCustomRenderer();
		}

		if (SceneProxy != nullptr)
		{
			if (SceneProxy->IsNaniteMesh())
			{
				FNaniteGeometryCollectionSceneProxy* NaniteProxy = static_cast<FNaniteGeometryCollectionSceneProxy*>(SceneProxy);
				NaniteProxy->FlushGPUSceneUpdate_GameThread();
			}

			MarkRenderTransformDirty();
			MarkRenderDynamicDataDirty();
		}
	}
}

void UGeometryCollectionComponent::UpdateNavigationDataIfNeeded(bool bDynamicCollectionDirty)
{
	if (bUpdateNavigationInTick && bDynamicCollectionDirty)
	{
		const UWorld* MyWorld = GetWorld();// If not doing velocity match, don't bother processing the same version twice.
		if (MyWorld && MyWorld->IsGameWorld())
		{
			//cycle every 0xff frames
			//@todo - Need way of seeing if the collection is actually changing
			if (bNavigationRelevant && bRegistered && (((GFrameCounter + NavmeshInvalidationTimeSliceIndex) & 0xff) == 0))
			{
				UpdateNavigationData();
			}
		}
	}
}

void UGeometryCollectionComponent::UpdateRemovalIfNeeded()
{
	static FName MassToLocalAttributeName = "MassToLocal";

	// if removal is enabled, update the dynamic collection transform based on the decay 
	// todo: we could optimize this using a list of transform to update from when we update the decay values
	const bool bAllowAnyRemoval = (bAllowRemovalOnBreak || bAllowRemovalOnSleep);
	if (DynamicCollection && bAllowAnyRemoval && PhysicsProxy)
	{
		FGeometryCollectionDecayDynamicFacade DecayFacade(*DynamicCollection);
		if (DecayFacade.IsValid())
		{
			const FTransform ZeroScaleTransform(FQuat::Identity, FVector::Zero(), FVector(0, 0, 0));

			const FTransform InverseComponentTransform = (RestCollection->bScaleOnRemoval) ? GetComponentTransform().Inverse() : FTransform::Identity;

			const TManagedArray<FTransform>* MassToLocal = nullptr;
			const TArray<FTransform3f>* CompSpaceTransform = nullptr;
			if (RestCollection->bScaleOnRemoval)
			{
				MassToLocal = RestCollection->GetGeometryCollection()->FindAttribute<FTransform>(MassToLocalAttributeName, FGeometryCollection::TransformGroup);
				CompSpaceTransform = &ComponentSpaceTransforms.RequestAllTransforms();
			}

			const int32 RootIndex = RestCollection->GetRootIndex();

			bool bTransformsChanged = false;

			const int32 NumTransforms = DecayFacade.GetDecayAttributeSize();
			for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
			{
				// only update values if the decay has changed 
				float Decay = DecayFacade.GetDecay(TransformIndex);

				// reconcile changes to the decay state with the decay value
				if (Decay < 1.f && TransformIndex != RootIndex)
				{
					if (PhysicsProxy->GetParticleByIndex_External(TransformIndex))
					{
						if (BrokenAndDecayedStates.GetHasDecayed(TransformIndex))
						{
							// this logic should only apply to end nodes that do not have simulated children 
							// otherwise this may cause clusters with children to disappear for several frames
							// ( see Cl 30290839 and UpdateBrokenAndDecayedStates() method logic )
							bool bHasAnyChildWithValidParticle = false;
							if (bGeometryCollectionRemoveOnBreakDecayFix)
							{
								DynamicCollection->IterateThroughChildren(TransformIndex,
									[&bHasAnyChildWithValidParticle, this](int32 ChildTransformIdx) -> bool
									{
										if (PhysicsProxy->GetParticleByIndex_External(ChildTransformIdx))
										{
											bHasAnyChildWithValidParticle = true;
											return false; // do no continue iterating
										}
										return true; // continue iterating
									});
							}
							if (!bHasAnyChildWithValidParticle)
							{
								Decay = 1.0f;
								DecayFacade.SetDecay(TransformIndex, Decay);
							}
						}
					}
				}

				if (Decay > 0.f && Decay <= 1.f)
				{
					const float Scale = 1.0 - Decay;
					if (Scale < UE_SMALL_NUMBER)
					{
						FTransform3f Transform = DynamicCollection->GetTransform(TransformIndex);
						Transform.SetScale3D(FVector3f::ZeroVector);
						DynamicCollection->SetTransform(TransformIndex, Transform);
						bTransformsChanged = true;
					}
					// do not try to get this condition out of the loop as this may cause some optimizer related issues
					else if (RestCollection->bScaleOnRemoval && MassToLocal && CompSpaceTransform)
					{
						float ShrinkRadius = 0.0f;
						UE::Math::TSphere<double> AccumulatedSphere;
						// todo(chaos) : find a faster way to do that ( precompute the data ? )
						if (CalculateInnerSphere(TransformIndex, AccumulatedSphere))
						{
							ShrinkRadius = -AccumulatedSphere.W;
						}

						const FTransform3f& CurrentTransform = DynamicCollection->GetTransform(TransformIndex);
						const float CurrentScale = CurrentTransform.GetScale3D().X; // always uniform scale

						const float ScaleToApply = (CurrentScale >  SMALL_NUMBER) ? (Scale / CurrentScale): Scale;

						// InverseComponentTransform calculation must in double precision for LWC
						const FQuat LocalRotation = (InverseComponentTransform * FTransform((*CompSpaceTransform)[TransformIndex].Inverse())).GetRotation();
						const FVector LocalDown = LocalRotation.RotateVector(FVector(0.f, 0.f, ShrinkRadius));
						const FVector CenterOfMass = (*MassToLocal)[TransformIndex].GetTranslation();
						const FVector3f ScaleCenter = FVector3f(LocalDown + CenterOfMass);
						const FTransform3f ScaleTransform(FQuat4f::Identity, ScaleCenter * FVector3f::FReal(1.f - ScaleToApply), FVector3f(ScaleToApply));
						const FTransform3f FinalTransform = ScaleTransform * CurrentTransform;
						DynamicCollection->SetTransform(TransformIndex, FinalTransform);
						bTransformsChanged = true;
					}
				}
			}

			if (bTransformsChanged)
			{
				DynamicCollection->MakeDirty();
				OnTransformsDirty();
			}
		}
	}
}

void UGeometryCollectionComponent::RequestUpdateRepData()
{
	FPhysScene* PhysScene = GetInnerChaosScene();
	if (PhysScene && PhysicsProxy) 
	{
		if (Chaos::FPBDRigidsSolver* Solver = PhysicsProxy->GetSolver<Chaos::FPBDRigidsSolver>())
		{
			const int32 CurrentFrame = Solver->GetCurrentFrame();

			PhysScene->EnqueueAsyncPhysicsCommand(CurrentFrame, this,
				[this]()
				{
					if (GetIsReplicated() && GetNetMode() != ENetMode::NM_Client)
					{
						UpdateRepData();
					}
				}, false
			);
		}
	}
}

void UGeometryCollectionComponent::OnRep_RepData()
{
	// We have new data that was replicated! Turn on the async tick to process instead of just requesting a one-off
	// since we may want to keep processing for extra time afterwards.
	check(IsInGameThread());
    check(GetNetMode() == ENetMode::NM_Client);

	ProcessRepDataOnPT();
}

void UGeometryCollectionComponent::OnRep_RepStateData()
{
	// We have new data that was replicated! Turn on the async tick to process instead of just requesting a one-off
	// since we may want to keep processing for extra time afterwards.
	check(IsInGameThread());
	check(GetNetMode() == ENetMode::NM_Client);

	ProcessRepStateDataOnPT();
}

void UGeometryCollectionComponent::OnRep_RepDynamicData()
{
	// We have new dynamic data that was replicated! Turn on the async tick to process instead of just requesting a one-off
	// since we may want to keep processing for extra time afterwards.
	check(IsInGameThread());
	check(GetNetMode() == ENetMode::NM_Client);

	ProcessRepDynamicDataOnPT();
}

void UGeometryCollectionComponent::SetAbandonedParticleCollisionProfileName(FName CollisionProfile)
{
	if (!bEnableAbandonAfterLevel || !GetIsReplicated())
	{
		return;
	}

	AbandonedCollisionProfileName = CollisionProfile;
	LoadCollisionProfiles();
}

void UGeometryCollectionComponent::SetPerLevelCollisionProfileNames(const TArray<FName>& ProfileNames)
{
	CollisionProfilePerLevel = ProfileNames;
	LoadCollisionProfiles();
}

// return true if anything was cahnged
template <typename TIteratableBoneContainerType>
static bool SetPerParticleCollisionProfileNameFromIterable(const UGeometryCollection* RestCollection, const TIteratableBoneContainerType& BoneIds, FName ProfileName, TArray<FName>& CollisionProfilePerParticle)
{
	if (!RestCollection)
	{
		return false;
	}

	bool bHasChanged = false;

	const int32 NumTransforms = RestCollection->GetGeometryCollection()->Transform.Num();
	if (CollisionProfilePerParticle.Num() != NumTransforms)
	{
		bHasChanged = true;
		CollisionProfilePerParticle.SetNumZeroed(NumTransforms);
	}

	for (int32 Id : BoneIds)
	{
		if (CollisionProfilePerParticle.IsValidIndex(Id))
		{
			if (CollisionProfilePerParticle[Id] != ProfileName)
			{
				bHasChanged = true;
				CollisionProfilePerParticle[Id] = ProfileName;
			}
		}
	}

	return bHasChanged;
}

void UGeometryCollectionComponent::SetPerParticleCollisionProfileName(const TSet<int32>& BoneIds, FName ProfileName)
{
	const bool bHasChanged = SetPerParticleCollisionProfileNameFromIterable(RestCollection, BoneIds, ProfileName, CollisionProfilePerParticle);
	if (bHasChanged)
	{
		LoadCollisionProfiles();
	}
}

bool UGeometryCollectionComponent::UpdatePerParticleCollisionProfilesNum()
{
	if (!RestCollection)
	{
		return false;
	}

	const int32 NumTransforms = RestCollection->GetGeometryCollection()->Transform.Num();
	const bool bHasChanged = CollisionProfilePerParticle.Num() != NumTransforms;

	if (bHasChanged)
	{
		CollisionProfilePerParticle.SetNumZeroed(NumTransforms);
	}

	return bHasChanged;
}

void UGeometryCollectionComponent::SetParticleCollisionProfileName(int32 BoneId, FName ProfileName, FGCCollisionProfileScopedTransaction& InProfileNameUpdateTransaction)
{
	if (!InProfileNameUpdateTransaction.IsValid())
	{
		return;
	}
	
	if (!CollisionProfilePerParticle.IsValidIndex(BoneId))
	{
		return;
	}

	FName& CurrentProfileName = CollisionProfilePerParticle[BoneId];

	if (CurrentProfileName !=  ProfileName)
	{
		CurrentProfileName = ProfileName;

		InProfileNameUpdateTransaction.MarkDirty();
	}
}

void UGeometryCollectionComponent::SetPerParticleCollisionProfileName(const TArray<int32>& BoneIds, FName ProfileName)
{
	const bool bHasChanged = SetPerParticleCollisionProfileNameFromIterable(RestCollection, BoneIds, ProfileName, CollisionProfilePerParticle);
	if (bHasChanged)
	{
		LoadCollisionProfiles();
	}
}

void UGeometryCollectionComponent::LoadCollisionProfiles()
{
	if (!PhysicsProxy)
	{
		return;
	}

	// Cache the FCollisionResponseTemplate as well as the query/sim collision filter data for a given collision profile name
	// so we don't have to recreate it every time.
	using FCollisionProfileDataCache = FGeometryCollectionPhysicsProxy::FParticleCollisionFilterData;
	TMap<FName, FCollisionProfileDataCache> CachedData;

	// Returns nullptr if we can't create or get the data.
	auto CreateOrGetCollisionProfileData = [this, &CachedData](const FName& ProfileName) -> const FCollisionProfileDataCache*
	{
		if (const FCollisionProfileDataCache* Data = CachedData.Find(ProfileName))
		{
			return Data;
		}

		FCollisionProfileDataCache Cache;
		FCollisionResponseTemplate Template;
		if (ProfileName == NAME_None)
		{
			return nullptr;
		}

		if (ProfileName == DefaultCollisionProfileName)
		{
			Cache.ShapeFilterData = InitialShapeFilterData;
			Cache.FilterInstanceData = InitialFilterInstanceData;

			Cache.bQueryEnabled = true;
			Cache.bSimEnabled = true;
		}
		else if (UCollisionProfile::Get()->GetProfileTemplate(ProfileName, Template))
		{
			AActor* Owner = GetOwner();
			const uint32 ActorID = Owner ? Owner->GetUniqueID() : 0;
			const uint32 CompID = GetUniqueID();

			// Maintain parity with the rest of the geometry collection filters.
			FPhysicsFilterBuilder FilterBuilder;
			FilterBuilder.SetOwnerID(ActorID);
			FilterBuilder.SetComponentID(CompID);
			FilterBuilder.SetCollisionChannelIndex(Template.ObjectType);
			FilterBuilder.SetMaskFilter(BodyInstance.GetMaskFilter());
			FilterBuilder.SetResponses(Template.ResponseToChannels);
			FilterBuilder.SetFlags(Chaos::EFilterFlags::SimpleCollision | Chaos::EFilterFlags::ComplexCollision, true);
			FilterBuilder.SetFlags(Chaos::EFilterFlags::CCD, BodyInstance.bUseCCD);
			FilterBuilder.SetFlags(Chaos::EFilterFlags::ContactNotify, bNotifyCollisions || bNotifyGlobalCollisions);

			Cache.FilterInstanceData = FilterBuilder.BuildInstanceData();
			Cache.ShapeFilterData = FilterBuilder.BuildShapeFilterData();
		
			Cache.bQueryEnabled = CollisionEnabledHasQuery(Template.CollisionEnabled);
			Cache.bSimEnabled = CollisionEnabledHasPhysics(Template.CollisionEnabled);
		}
		else
		{
			return nullptr;
		}

		Cache.bIsValid = true;
		return &CachedData.Add(ProfileName, Cache);
	};

	const FCollisionProfileDataCache* AbandonedData = (bEnableAbandonAfterLevel && GetIsReplicated()) ? CreateOrGetCollisionProfileData(AbandonedCollisionProfileName) : nullptr;

	Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*RestCollection->GetGeometryCollection());
	// Use GetAllPhysicsObjectIncludingNulls instead of GetAllPhysicsObjects if you need to use Level data or any data from HierarchyFacade
	TArray<Chaos::FPhysicsObjectHandle> PhysicsObjects = PhysicsProxy->GetAllPhysicsObjectIncludingNulls();
	FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(PhysicsObjects);

	TArray<FCollisionProfileDataCache> PerParticleData;
	PerParticleData.Reserve(PhysicsObjects.Num());

	for (int32 ParticleIndex = 0; ParticleIndex < PhysicsObjects.Num(); ++ParticleIndex)
	{
		if (PhysicsObjects[ParticleIndex] != nullptr)
		{
			const int32 Level = HierarchyFacade.GetInitialLevel(ParticleIndex);

			TArrayView<Chaos::FPhysicsObjectHandle> ParticleView{ &PhysicsObjects[ParticleIndex], 1 };
			const FCollisionProfileDataCache* Data = nullptr;
			if (!CollisionProfilePerParticle.IsEmpty() && CollisionProfilePerParticle[ParticleIndex] != NAME_None)
			{
				Data = CreateOrGetCollisionProfileData(CollisionProfilePerParticle[ParticleIndex]);
			}
			else if (!CollisionProfilePerLevel.IsEmpty())
			{
				Data = CreateOrGetCollisionProfileData(CollisionProfilePerLevel[FMath::Min(CollisionProfilePerLevel.Num() - 1, Level)]);
			}
			else if (AbandonedData && Level >= ReplicationAbandonAfterLevel + 1)
			{
				Data = AbandonedData;
			}
			else // last resort use the body instance set collision settings
			{
				Data = CreateOrGetCollisionProfileData(DefaultCollisionProfileName);
			}

			if (Data)
			{
				FCollisionProfileDataCache ParticleData = *Data;
				ParticleData.ParticleIndex = ParticleIndex;

				PerParticleData.Emplace(MoveTemp(ParticleData));

				// Need to update on the GT too (physics proxy will just enqueue to do it on the PT) or else the changes won't take effect for GT SQ queries.
				Interface->UpdateShapeCollisionFlags(ParticleView, ParticleData.bSimEnabled, ParticleData.bQueryEnabled);
				Interface->UpdateCombinedShapeFilterData(ParticleView, Chaos::Filter::FCombinedShapeFilterData(ParticleData.ShapeFilterData, ParticleData.FilterInstanceData));
			}
		}
	}

	PhysicsProxy->UpdatePerParticleFilterData_External(PerParticleData);
}

#if WITH_EDITORONLY_DATA
const FDamageCollector* UGeometryCollectionComponent::GetRunTimeDataCollector() const
{
	return FRuntimeDataCollector::GetInstance().Find(RunTimeDataCollectionGuid);
}
#endif

void UGeometryCollectionComponent::OnDestroyPhysicsState()
{
	UActorComponent::OnDestroyPhysicsState();

	// we need to unregister the events because it relies on the proxy internally 
	if (EventDispatcher)
	{
		EventDispatcher->UnregisterChaosEvents();
	}

	if(PhysicsProxy)
	{
		// Clear physics sync callback
		PhysicsProxy->SetPostPhysicsSyncCallback(nullptr); 

		if (FPhysScene* PhysScene = GetInnerChaosScene())
		{
			PhysScene->RemoveObject(PhysicsProxy);
		}
		InitializationState = ESimulationInitializationState::Unintialized;

		// clear the clusters to rep as the information hold by it is now invalid
		// we can still call this on the game thread because replication runs with the game thread frozen and will not run while the physics  state is being torned down
		ResetRepData();

		// Discard the pointer (cleanup happens through the scene or dedicated thread)
		PhysicsProxy = nullptr;
	}

	// We're skipping over the primitive component so we need to make sure this event gets fired.
	OnComponentPhysicsStateChanged.Broadcast(this, EComponentPhysicsStateChange::Destroyed);
}

void UGeometryCollectionComponent::SendDynamicDataToSceneProxy()
{
	FGeometryCollectionDynamicData* DynamicData = InitDynamicData();
	if (SceneProxy && DynamicData)
	{
		if (SceneProxy->IsNaniteMesh())
		{
			INC_DWORD_STAT_BY(STAT_GCTotalTransforms, DynamicData ? DynamicData->Transforms.Num() : 0);

			const FMatrix RenderMatrix{ GetRenderMatrix() };
			FNaniteGeometryCollectionSceneProxy* GeometryCollectionSceneProxy = static_cast<FNaniteGeometryCollectionSceneProxy*>(SceneProxy);
			ENQUEUE_RENDER_COMMAND(SendRenderDynamicData)(
				[GeometryCollectionSceneProxy, DynamicData, RenderMatrix](FRHICommandListBase& RHICmdList)
				{
					GeometryCollectionSceneProxy->SetDynamicData_RenderThread(RHICmdList, DynamicData, RenderMatrix);
				}
			);
		}
		else
		{
			FGeometryCollectionSceneProxy* GeometryCollectionSceneProxy = static_cast<FGeometryCollectionSceneProxy*>(SceneProxy);
			ENQUEUE_RENDER_COMMAND(SendRenderDynamicData)(
				[GeometryCollectionSceneProxy, DynamicData](FRHICommandListBase& RHICmdList)
				{
					GeometryCollectionSceneProxy->SetDynamicData_RenderThread(RHICmdList, DynamicData);
				}
			);
		}
	}
}

void UGeometryCollectionComponent::SendRenderDynamicData_Concurrent()
{
	//UE_LOGF(UGCC_LOG, Log, "GeometryCollectionComponent[%p]::SendRenderDynamicData_Concurrent()", this);
	Super::SendRenderDynamicData_Concurrent();

	SendDynamicDataToSceneProxy();
}

void UGeometryCollectionComponent::SetCollisionObjectType(ECollisionChannel Channel)
{
	Super::SetCollisionObjectType(Channel);

	BuildInitialFilterData();

	// Update filters stored on proxy
	if (PhysicsProxy)
	{
		PhysicsProxy->UpdateFilterData_External(Chaos::Filter::FCombinedShapeFilterData(InitialShapeFilterData, InitialFilterInstanceData));
	}
}

void UGeometryCollectionComponent::OnActorEnableCollisionChanged()
{
	// Update filters on BI
	BodyInstance.UpdatePhysicsFilterData();

	// Update Shape Filter and Instance data
	BuildInitialFilterData();

	// Update filters stored on proxy
	if (PhysicsProxy)
	{
		PhysicsProxy->UpdateFilterData_External(Chaos::Filter::FCombinedShapeFilterData(InitialShapeFilterData, InitialFilterInstanceData));
	}

}

void UGeometryCollectionComponent::BuildInitialFilterData()
{
	FBodyCollisionFilterData FilterData;
	BodyInstance.BuildBodyFilterData(FilterData);
	
	FPhysicsFilterBuilder Builder(FilterData.SimpleShapeFilterData, FilterData.FilterInstanceData);
	// Enable for complex and simple (no dual representation currently like other meshes)
	Builder.SetFlags(Chaos::EFilterFlags::SimpleCollision | Chaos::EFilterFlags::ComplexCollision, true);	
	// Should this actually clear the flag if the body is true but the geometry component is false?
	if (bNotifyCollisions || bNotifyGlobalCollisions)
	{
		Builder.SetFlags(Chaos::EFilterFlags::ContactNotify, true);
	}

	InitialFilterInstanceData = Builder.BuildInstanceData();
	InitialShapeFilterData = Builder.BuildShapeFilterData();
}

void UGeometryCollectionComponent::ApplyAssetDefaults()
{
	if (RestCollection)
	{
		// initialize the component per level damage threshold from the asset defaults 
		DamageModel = RestCollection->DamageModel;
		DamageThreshold = RestCollection->DamageThreshold;
		bUseSizeSpecificDamageThreshold = RestCollection->bUseSizeSpecificDamageThreshold;
		bUseMaterialDamageModifiers = RestCollection->bUseMaterialDamageModifiers;

		// initialize the component damage progataion data from the asset defaults 
		DamagePropagationData = RestCollection->DamagePropagationData;

		if (RestCollection->PhysicsMaterial)
		{
			BodyInstance.SetPhysMaterialOverride(RestCollection->PhysicsMaterial);
		}

		bDensityFromPhysicsMaterial = RestCollection->bDensityFromPhysicsMaterial;
	}
}

void UGeometryCollectionComponent::SetRestCollection(const UGeometryCollection* RestCollectionIn, bool bApplyAssetDefaults)
{
	if (CustomRenderer)
	{
		UnregisterCustomRenderer();
	}

	//UE_LOGF(UGCC_LOG, Log, "GeometryCollectionComponent[%p]::SetRestCollection()", this);
	if (RestCollectionIn)
	{
		RestCollection = RestCollectionIn;

		// if the geometry collection changes we need to clear the rest override rest transforms
		// otherwise this may cause mismatch issues and make the geometry collection look wrong 
		RestTransforms.Reset();

		// Reset the dynamic collection
		ResetDynamicCollection();

		// Rebuild the physics state since the dynamic collection has changed
		RecreatePhysicsState();

		if (!IsEmbeddedGeometryValid())
		{
			InitializeEmbeddedGeometry();
		}

		if (IsRegistered())
		{
			InitializeCustomRenderer();
		}

		if (CustomRenderer)
		{
			RegisterCustomRenderer();
			RefreshCustomRenderer();
		}

		ClearRootProxyLocalTransforms();

		if (bApplyAssetDefaults)
		{
			ApplyAssetDefaults();
		}
		
		PrecachePSOs();
	}
}

void UGeometryCollectionComponent::ResetState()
{
	// Reset the dynamic collection
	ResetDynamicCollection();

	// Rebuild the physics state since the dynamic collection has changed
	RecreatePhysicsState();

	// if we are using a custom render, make sure everything is properly reset 
	RefreshCustomRenderer();
}

FString UGeometryCollectionComponent::GetDebugInfo()
{
	// print the game thread side of things
	FString DebugInfo;
	DebugInfo += FString("RestCollection - ") + (RestCollection? RestCollection->GetName() : FString("None"));
	DebugInfo += "\n";
	if (RestCollection && RestCollection->GetGeometryCollection())
	{
		DebugInfo += RestCollection->GetGeometryCollection()->ToString();
	}
	DebugInfo += FString("DynamicCollection - ") + FString(DynamicCollection ? "Yes": "No");
	DebugInfo += "\n";
	if (DynamicCollection)
	{
		DebugInfo += DynamicCollection->ToString();
	}
	return DebugInfo;
}

FGeometryCollectionEdit::FGeometryCollectionEdit(UGeometryCollectionComponent* InComponent, GeometryCollection::EEditUpdate InEditUpdate, bool bShapeIsUnchanged, bool bPropagateToAllMatchingComponents)
	: Component(InComponent)
	, EditUpdate(InEditUpdate)
	, bShapeIsUnchanged(bShapeIsUnchanged)
	, bPropagateToAllMatchingComponents(bPropagateToAllMatchingComponents)
{
	if (!Component->RestCollection)
	{
		return;
	}

	auto ProcessComponent = [this](UGeometryCollectionComponent* ToProcess)
	{
		bool bHadPhysicsState = ToProcess->HasValidPhysicsState();
		if (EnumHasAnyFlags(EditUpdate, GeometryCollection::EEditUpdate::Physics) && bHadPhysicsState)
		{
			HadPhysicsState.Add(ToProcess);
			ToProcess->DestroyPhysicsState();
		}

		if (EnumHasAnyFlags(EditUpdate, GeometryCollection::EEditUpdate::Rest) && GetRestCollection())
		{
			ToProcess->Modify();
			GetRestCollection()->Modify(); // Note: If multiple components share the rest collection, its Modify() will be called multiple times, but this should be ok
		}
	};

	if (bPropagateToAllMatchingComponents)
	{
		for (TObjectIterator<UGeometryCollectionComponent> It(RF_ClassDefaultObject, false, EInternalObjectFlags::Garbage); It; ++It)
		{
			if (It->RestCollection == Component->RestCollection)
			{
				ProcessComponent(*It);
			}
		}
	}
	else
	{
		ProcessComponent(Component);
	}
}

FGeometryCollectionEdit::~FGeometryCollectionEdit()
{
#if WITH_EDITOR
	if (!!EditUpdate)
	{
		if (EnumHasAnyFlags(EditUpdate, GeometryCollection::EEditUpdate::Rest) && GetRestCollection())
		{
			if (!bShapeIsUnchanged)
			{
				GetRestCollection()->UpdateGeometryDependentProperties();
			}
			GetRestCollection()->InvalidateCollection();
		}

		auto ProcessComponent = [this](UGeometryCollectionComponent* ToProcess)
		{
			// Note: Reset dynamic collection for both rest and dynamic updates, since we don't want the dynamic to be out of sync with the rest
			if (EnumHasAnyFlags(EditUpdate, GeometryCollection::EEditUpdate::Dynamic | GeometryCollection::EEditUpdate::Rest))
			{
				ToProcess->ResetDynamicCollection();
			}

			if (EnumHasAnyFlags(EditUpdate, GeometryCollection::EEditUpdate::Physics) && HadPhysicsState.Contains(ToProcess))
			{
				ToProcess->RecreatePhysicsState();
			}

			ToProcess->RefreshCustomRenderer();
		};

		if (bPropagateToAllMatchingComponents)
		{
			for (TObjectIterator<UGeometryCollectionComponent> It(RF_ClassDefaultObject, false, EInternalObjectFlags::Garbage); It; ++It)
			{
				if (It->RestCollection == Component->RestCollection)
				{
					ProcessComponent(*It);
				}
			}
		}
		else
		{
			ProcessComponent(Component);
		}
	}
#endif
}

UGeometryCollection* FGeometryCollectionEdit::GetRestCollection()
{
	if (Component)
	{
		return const_cast<UGeometryCollection*>(ToRawPtr(Component->RestCollection));	//const cast is ok here since we are explicitly in edit mode. Should all this editor code be in an editor module?
	}
	return nullptr;
}

#if WITH_EDITOR
TArray<FLinearColor> FScopedColorEdit::RandomColors;

FScopedColorEdit::FScopedColorEdit(UGeometryCollectionComponent* InComponent, bool bForceUpdate) : bUpdated(bForceUpdate), Component(InComponent)
{
	if (RandomColors.Num() == 0)
	{
		// predictable colors based on the component
		FRandomStream Random(GetTypeHash(InComponent));
		for (int i = 0; i < 100; i++)
		{
			const uint8 R = static_cast<uint8>(Random.FRandRange(5, 105));
			const uint8 G = static_cast<uint8>(Random.FRandRange(5, 105));
			const uint8 B = static_cast<uint8>(Random.FRandRange(5, 105));
			RandomColors.Push(FLinearColor(FColor(R, G, B, 255)));
		}
	}
}

FScopedColorEdit::~FScopedColorEdit()
{
	if (bUpdated)
	{
		UpdateBoneColors();
	}
}
void FScopedColorEdit::SetShowBoneColors(bool ShowBoneColorsIn)
{
	if (Component->bShowBoneColors != ShowBoneColorsIn)
	{
		bUpdated = true;
		Component->bShowBoneColors = ShowBoneColorsIn;
	}
}

bool FScopedColorEdit::GetShowBoneColors() const
{
	return Component->bShowBoneColors;
}

void FScopedColorEdit::SetEnableBoneSelection(bool ShowSelectedBonesIn)
{
	if (Component->bEnableBoneSelection != ShowSelectedBonesIn)
	{
		bUpdated = true;
		Component->bEnableBoneSelection = ShowSelectedBonesIn;
	}
}

bool FScopedColorEdit::GetEnableBoneSelection() const
{
	return Component->bEnableBoneSelection;
}

bool FScopedColorEdit::IsBoneSelected(int BoneIndex) const
{
	return Component->SelectedBones.Contains(BoneIndex);
}

void FScopedColorEdit::Sanitize()
{
	const UGeometryCollection* GeometryCollection = Component->GetRestCollection();
	if (GeometryCollection)
	{
		TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollection->GetGeometryCollection();
		if (GeometryCollectionPtr)
		{
			const int32 NumTransforms = GeometryCollectionPtr->NumElements(FGeometryCollection::TransformGroup);
			const int32 NumSelectionRemoved = Component->SelectedBones.RemoveAll([this, NumTransforms](int32 Index) {
				return Index < 0 || Index >= NumTransforms;
				});
			const int32 NumHighlightRemoved = Component->HighlightedBones.RemoveAll([this, NumTransforms](int32 Index) {
				return Index < 0 || Index >= NumTransforms;
				});
			bUpdated = bUpdated || NumSelectionRemoved || NumHighlightRemoved;
		}
	}
}

void FScopedColorEdit::SetSelectedBones(const TArray<int32>& SelectedBonesIn)
{
	bUpdated = true;
	Component->SelectedBones = SelectedBonesIn;
	Component->SelectEmbeddedGeometry();
}

void FScopedColorEdit::AppendSelectedBones(const TArray<int32>& SelectedBonesIn)
{
	bUpdated = true;
	Component->SelectedBones.Append(SelectedBonesIn);
}

void FScopedColorEdit::ToggleSelectedBones(const TArray<int32>& SelectedBonesIn, bool bAdd, bool bSnapToLevel)
{
	bUpdated = true;
	
	const UGeometryCollection* GeometryCollection = Component->GetRestCollection();
	if (GeometryCollection)
	{ 
		TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollection->GetGeometryCollection();
		for (int32 BoneIndex : SelectedBonesIn)
		{
		
			int32 ContextBoneIndex = (bSnapToLevel && GetViewLevel() > -1) ? 
				FGeometryCollectionClusteringUtility::GetParentOfBoneAtSpecifiedLevel(GeometryCollectionPtr.Get(), BoneIndex, GetViewLevel(), true /*bSkipFiltered*/) 
				: BoneIndex;
			if (ContextBoneIndex == FGeometryCollection::Invalid)
			{
				continue;
			}
		
			if (bAdd) // shift select
			{
				Component->SelectedBones.Add(ContextBoneIndex);
			}
			else // ctrl select (toggle)
			{ 
				if (Component->SelectedBones.Contains(ContextBoneIndex))
				{
					Component->SelectedBones.Remove(ContextBoneIndex);
				}
				else
				{
					Component->SelectedBones.Add(ContextBoneIndex);
				}
			}
		}
	}
}

void FScopedColorEdit::AddSelectedBone(int32 BoneIndex)
{
	if (!Component->SelectedBones.Contains(BoneIndex))
	{
		bUpdated = true;
		Component->SelectedBones.Push(BoneIndex);
	}
}

void FScopedColorEdit::ClearSelectedBone(int32 BoneIndex)
{
	if (Component->SelectedBones.Contains(BoneIndex))
	{
		bUpdated = true;
		Component->SelectedBones.Remove(BoneIndex);
	}
}

const TArray<int32>& FScopedColorEdit::GetSelectedBones() const
{
	return Component->GetSelectedBones();
}

int32 FScopedColorEdit::GetMaxSelectedLevel(bool bOnlyRigid) const
{
	int32 MaxSelectedLevel = -1;
	const UGeometryCollection* GeometryCollection = Component->GetRestCollection();
	if (GeometryCollection && GeometryCollection->GetGeometryCollection()->HasAttribute("Level", FGeometryCollection::TransformGroup))
	{
		TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollection->GetGeometryCollection();
		const TManagedArray<int32>& Levels = GeometryCollectionPtr->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);
		const TManagedArray<int32>& SimTypes = GeometryCollectionPtr->SimulationType;
		for (int32 BoneIndex : Component->SelectedBones)
		{
			if (!bOnlyRigid || SimTypes[BoneIndex] == FGeometryCollection::ESimulationTypes::FST_Rigid)
			{
				MaxSelectedLevel = FMath::Max(MaxSelectedLevel, Levels[BoneIndex]);
			}
		}
	}
	return MaxSelectedLevel;
}

bool FScopedColorEdit::IsSelectionValidAtLevel(int32 TargetLevel) const
{
	if (TargetLevel == -1)
	{
		return true;
	}
	const UGeometryCollection* GeometryCollection = Component->GetRestCollection();
	if (GeometryCollection && GeometryCollection->GetGeometryCollection()->HasAttribute("Level", FGeometryCollection::TransformGroup))
	{
		TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollection->GetGeometryCollection();
		const TManagedArray<int32>& Levels = GeometryCollectionPtr->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);
		const TManagedArray<int32>& SimTypes = GeometryCollectionPtr->SimulationType;
		for (int32 BoneIndex : Component->SelectedBones)
		{
			if (SimTypes[BoneIndex] != FGeometryCollection::ESimulationTypes::FST_Clustered && // clusters are always shown in outliner
				Levels[BoneIndex] != TargetLevel && // nodes at the target level are shown in outliner
				// non-cluster parents are shown if they have children that are exact matches (i.e., a rigid parent w/ embedded at the target level)
				(GeometryCollectionPtr->Children[BoneIndex].Num() == 0 || Levels[BoneIndex] + 1 != TargetLevel))
			{
				return false;
			}
		}
	}
	return true;
}

void FScopedColorEdit::ResetBoneSelection()
{
	if (Component->SelectedBones.Num() > 0)
	{
		bUpdated = true;
	}

	Component->SelectedBones.Empty();
}

void FScopedColorEdit::FilterSelectionToLevel(bool bPreferLowestOnly)
{
	const UGeometryCollection* GeometryCollection = Component->GetRestCollection();
	int32 ViewLevel = GetViewLevel();
	bool bNeedsFiltering = ViewLevel >= 0 || bPreferLowestOnly;
	if (GeometryCollection && Component->SelectedBones.Num() > 0 && bNeedsFiltering && GeometryCollection->GetGeometryCollection()->HasAttribute("Level", FGeometryCollection::TransformGroup))
	{
		TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollection->GetGeometryCollection();

		const TManagedArray<int32>& Levels = GeometryCollectionPtr->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);
		const TManagedArray<int32>& SimTypes = GeometryCollectionPtr->SimulationType;

		TArray<int32> NewSelection;
		NewSelection.Reserve(Component->SelectedBones.Num());
		if (ViewLevel >= 0)
		{
			for (int32 BoneIdx : Component->SelectedBones)
			{
				bool bIsCluster = SimTypes[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Clustered;
				if (bPreferLowestOnly && bIsCluster && Levels[BoneIdx] < ViewLevel)
				{
					continue;
				}
				if (Levels[BoneIdx] == ViewLevel || (bIsCluster && Levels[BoneIdx] <= ViewLevel))
				{
					NewSelection.Add(BoneIdx);
				}
			}
		}
		else // bPreferLowestOnly && ViewLevel == -1
		{
			// If view level is "all" and we prefer lowest selection, just select any non-cluster nodes
			for (int32 BoneIdx : Component->SelectedBones)
			{
				bool bIsCluster = SimTypes[BoneIdx] == FGeometryCollection::ESimulationTypes::FST_Clustered;
				if (!bIsCluster)
				{
					NewSelection.Add(BoneIdx);
				}
			}
		}

		if (NewSelection.Num() != Component->SelectedBones.Num())
		{
			SetSelectedBones(NewSelection);
			SetHighlightedBones(NewSelection, true);
		}
	}
}

void FScopedColorEdit::SelectBones(GeometryCollection::ESelectionMode SelectionMode)
{
	check(Component);

	const UGeometryCollection* GeometryCollection = Component->GetRestCollection();
	if (GeometryCollection)
	{
		TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollection->GetGeometryCollection();

		switch (SelectionMode)
		{
		case GeometryCollection::ESelectionMode::None:
			ResetBoneSelection();
			break;

		case GeometryCollection::ESelectionMode::AllGeometry:
		{
			ResetBoneSelection();
			TArray<int32> BonesToSelect;
			FGeometryCollectionClusteringUtility::GetBonesToLevel(GeometryCollectionPtr.Get(), GetViewLevel(), BonesToSelect, true, true);
			AppendSelectedBones(BonesToSelect);
		}
		break;

		case GeometryCollection::ESelectionMode::Leaves:
		{
			ResetBoneSelection();
			int32 ViewLevel = GetViewLevel();
			TArray<int32> BonesToSelect;
			FGeometryCollectionClusteringUtility::GetBonesToLevel(GeometryCollectionPtr.Get(), GetViewLevel(), BonesToSelect, true, true);
			const TManagedArray<int32>& SimType = GeometryCollectionPtr->SimulationType;
			const TManagedArray<int32>* Levels = GeometryCollectionPtr->FindAttributeTyped<int32>("Level", FGeometryCollection::TransformGroup);
			BonesToSelect.SetNum(Algo::RemoveIf(BonesToSelect, [&](int32 BoneIdx)
				{
					return SimType[BoneIdx] != FGeometryCollection::ESimulationTypes::FST_Rigid
						|| (ViewLevel != -1 && Levels && (*Levels)[BoneIdx] != ViewLevel);
				}));
			AppendSelectedBones(BonesToSelect);
		}
		break;

		case GeometryCollection::ESelectionMode::Clusters:
		{
			ResetBoneSelection();
			int32 ViewLevel = GetViewLevel();
			TArray<int32> BonesToSelect;
			FGeometryCollectionClusteringUtility::GetBonesToLevel(GeometryCollectionPtr.Get(), ViewLevel, BonesToSelect, true, true);
			const TManagedArray<int32>& SimType = GeometryCollectionPtr->SimulationType;
			const TManagedArray<int32>* Levels = GeometryCollectionPtr->FindAttributeTyped<int32>("Level", FGeometryCollection::TransformGroup);
			BonesToSelect.SetNum(Algo::RemoveIf(BonesToSelect, [&](int32 BoneIdx)
			{
				return SimType[BoneIdx] != FGeometryCollection::ESimulationTypes::FST_Clustered
					|| (ViewLevel != -1 && Levels && (*Levels)[BoneIdx] != ViewLevel);
			}));
			AppendSelectedBones(BonesToSelect);
		}
		break;

		case GeometryCollection::ESelectionMode::InverseGeometry:
		{
			TArray<int32> Roots;
			FGeometryCollectionClusteringUtility::GetRootBones(GeometryCollectionPtr.Get(), Roots);
			TArray<int32> NewSelection;

			for (int32 RootElement : Roots)
			{
				if (GetViewLevel() == -1)
				{
					TArray<int32> LeafBones;
					FGeometryCollectionClusteringUtility::GetLeafBones(GeometryCollectionPtr.Get(), RootElement, true, LeafBones);

					for (int32 Element : LeafBones)
					{
						if (!IsBoneSelected(Element))
						{
							NewSelection.Push(Element);
						}
					}
				}
				else
				{
					TArray<int32> ViewLevelBones;
					FGeometryCollectionClusteringUtility::GetChildBonesAtLevel(GeometryCollectionPtr.Get(), RootElement, GetViewLevel(), ViewLevelBones);
					for (int32 ViewLevelBone : ViewLevelBones)
					{
						if (!IsBoneSelected(ViewLevelBone))
						{
							NewSelection.Push(ViewLevelBone);
						}
					}
				}
			}
			
			ResetBoneSelection();
			AppendSelectedBones(NewSelection);
		}
		break;


		case GeometryCollection::ESelectionMode::Neighbors:
		{
			FGeometryCollectionProximityUtility ProximityUtility(GeometryCollectionPtr.Get());
			ProximityUtility.RequireProximity();

			const TManagedArray<int32>& TransformIndex = GeometryCollectionPtr->TransformIndex;
			const TManagedArray<int32>& TransformToGeometryIndex = GeometryCollectionPtr->TransformToGeometryIndex;
			const TManagedArray<TSet<int32>>& Proximity = GeometryCollectionPtr->GetAttribute<TSet<int32>>("Proximity", FGeometryCollection::GeometryGroup);
			Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*GeometryCollectionPtr);

			const TArray<int32> SelectedBones = GetSelectedBones();

			TSet<int32> NewSelection;
			for (int32 Bone : SelectedBones)
			{
				NewSelection.Add(Bone);
				ProximityUtility.EnumerateNeighbors(HierarchyFacade, Bone, [&NewSelection](int32 NeighborTransformIdx)
				{
					NewSelection.Add(NeighborTransformIdx);
				}, true /*allow neighbors in parent levels*/, false /*bFilterDuplicates, not needed since we add to a set*/);
			}

			ResetBoneSelection();
			AppendSelectedBones(NewSelection.Array());
		}
		break;

		case GeometryCollection::ESelectionMode::Parent:
		{
			const TManagedArray<int32>& Parents = GeometryCollectionPtr->Parent;
			
			const TArray<int32> SelectedBones = GetSelectedBones();

			TSet<int32> NewSelection;
			for (int32 Bone : SelectedBones)
			{
				int32 ParentBone = Parents[Bone];
				if (ParentBone != FGeometryCollection::Invalid)
				{
					NewSelection.Add(ParentBone);
				}
			}

			ResetBoneSelection();
			AppendSelectedBones(NewSelection.Array());
		}
		break;

		case GeometryCollection::ESelectionMode::Children:
		{
			const TManagedArray<TSet<int32>>& Children = GeometryCollectionPtr->Children;

			const TArray<int32> SelectedBones = GetSelectedBones();

			TSet<int32> NewSelection;
			for (int32 Bone : SelectedBones)
			{
				if (Children[Bone].IsEmpty())
				{
					NewSelection.Add(Bone);
					continue;
				}
				for (int32 Child : Children[Bone])
				{
					NewSelection.Add(Child);
				}
			}

			ResetBoneSelection();
			AppendSelectedBones(NewSelection.Array());
		}
		break;

		case GeometryCollection::ESelectionMode::Siblings:
		{
			const TManagedArray<int32>& Parents = GeometryCollectionPtr->Parent;
			const TManagedArray<TSet<int32>>& Children = GeometryCollectionPtr->Children;

			const TArray<int32> SelectedBones = GetSelectedBones();

			TSet<int32> NewSelection;
			for (int32 Bone : SelectedBones)
			{
				int32 ParentBone = Parents[Bone];
				if (ParentBone != FGeometryCollection::Invalid)
				{
					for (int32 Child : Children[ParentBone])
					{
						NewSelection.Add(Child);
					}
				}

			}

			ResetBoneSelection();
			AppendSelectedBones(NewSelection.Array());
		}
		break;

		case GeometryCollection::ESelectionMode::Level:
		{
			if (GeometryCollectionPtr->HasAttribute("Level", FTransformCollection::TransformGroup))
			{
				const TManagedArray<int32>& Levels = GeometryCollectionPtr->GetAttribute<int32>("Level", FTransformCollection::TransformGroup);

				const TArray<int32> SelectedBones = GetSelectedBones();

				TSet<int32> NewSelection;
				for (int32 Bone : SelectedBones)
				{
					int32 Level = Levels[Bone];
					for (int32 TransformIdx = 0; TransformIdx < GeometryCollectionPtr->NumElements(FTransformCollection::TransformGroup); ++TransformIdx)
					{
						if (Levels[TransformIdx] == Level)
						{
							NewSelection.Add(TransformIdx);
						}
					}
				}

				ResetBoneSelection();
				AppendSelectedBones(NewSelection.Array());
			}	
		}
		break;

		default: 
			check(false); // unexpected selection mode
		break;
		}

		const TArray<int32>& SelectedBones = GetSelectedBones();
		TArray<int32> HighlightBones;
		for (int32 SelectedBone: SelectedBones)
		{ 
			FGeometryCollectionClusteringUtility::RecursiveAddAllChildren(GeometryCollectionPtr->Children, SelectedBone, HighlightBones);
		}
		SetHighlightedBones(HighlightBones);
	}
}

bool FScopedColorEdit::IsBoneHighlighted(int BoneIndex) const
{
	return Component->HighlightedBones.Contains(BoneIndex);
}

void FScopedColorEdit::SetHighlightedBones(const TArray<int32>& HighlightedBonesIn, bool bHighlightChildren)
{
	if (Component->HighlightedBones != HighlightedBonesIn)
	{
		const UGeometryCollection* GeometryCollection = Component->GetRestCollection();
		if (bHighlightChildren && GeometryCollection)
		{
			Component->HighlightedBones.Reset();
			TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollection->GetGeometryCollection();
			for (int32 SelectedBone : HighlightedBonesIn)
			{
				FGeometryCollectionClusteringUtility::RecursiveAddAllChildren(GeometryCollectionPtr->Children, SelectedBone, Component->HighlightedBones);
			}
		}
		else
		{
			Component->HighlightedBones = HighlightedBonesIn;
		}
		bUpdated = true;
	}
}

void FScopedColorEdit::AddHighlightedBone(int32 BoneIndex)
{
	Component->HighlightedBones.Push(BoneIndex);
}

const TArray<int32>& FScopedColorEdit::GetHighlightedBones() const
{
	return Component->GetHighlightedBones();
}

void FScopedColorEdit::ResetHighlightedBones()
{
	if (Component->HighlightedBones.Num() > 0)
	{
		bUpdated = true;
		Component->HighlightedBones.Empty();

	}
}

void FScopedColorEdit::SetLevelViewMode(int ViewLevelIn)
{
	if (Component->ViewLevel != ViewLevelIn)
	{
		bUpdated = true;
		Component->ViewLevel = ViewLevelIn;
	}
}

int FScopedColorEdit::GetViewLevel()
{
	return Component->ViewLevel;
}

void FScopedColorEdit::UpdateBoneColors()
{
	// @todo FractureTools - For large fractures updating colors this way is extremely slow because the render state (and thus all buffers) must be recreated.
	// It would be better to push the update to the proxy via a render command and update the existing buffer directly
	FGeometryCollectionEdit GeometryCollectionEdit = Component->EditRestCollection(GeometryCollection::EEditUpdate::None);
	UGeometryCollection* GeometryCollection = GeometryCollectionEdit.GetRestCollection();
	if(GeometryCollection)
	{
		const UDataflowSettings* DataflowSettings = GetDefault<UDataflowSettings>();

		FGeometryCollection* Collection = GeometryCollection->GetGeometryCollection().Get();

		const TManagedArray<int>& Parents = Collection->Parent;
		bool HasLevelAttribute = Collection->HasAttribute("Level", FTransformCollection::TransformGroup);
		const TManagedArray<int>* Levels = nullptr;
		if (HasLevelAttribute)
		{
			Levels = &Collection->GetAttribute<int32>("Level", FTransformCollection::TransformGroup);
		}
		TManagedArray<FLinearColor>& BoneColors = Collection->BoneColor;

		for (int32 BoneIndex = 0, NumBones = Parents.Num() ; BoneIndex < NumBones; ++BoneIndex)
		{
			FLinearColor BoneColor = FLinearColor(FColor::Black);

			if (Component->bShowBoneColors)
			{
				if (Component->ViewLevel == -1)
				{
					BoneColor = RandomColors[BoneIndex % RandomColors.Num()];
				}
				else
				{
					if (HasLevelAttribute && (*Levels)[BoneIndex] >= Component->ViewLevel)
					{
						// go up until we find parent at the required ViewLevel
						int32 Bone = BoneIndex;
						while (Bone != -1 && (*Levels)[Bone] > Component->ViewLevel)
						{
							Bone = Parents[Bone];
						}

						int32 ColorIndex = Bone + 1; // parent can be -1 for root, range [-1..n]
						BoneColor = RandomColors[ColorIndex % RandomColors.Num()];

						BoneColor = BoneColor.LinearRGBToHSV();
						BoneColor.B *= .5;
						BoneColor = BoneColor.HSVToLinearRGB();
					}
					else
					{
						BoneColor = DataflowSettings->TransformLevelColors.BlankColor;
					}
				}
			}
			else
			{
				BoneColor = FLinearColor(FColor::White);
				if (Component->ViewLevel != INDEX_NONE && HasLevelAttribute && (*Levels)[BoneIndex] < Component->ViewLevel)
				{
					BoneColor = FLinearColor(FColor(128U, 128U, 128U, 255U));
				}
				
			}
			// store the bone selected toggle in alpha so we can use it in the shader
			BoneColor.A = IsBoneHighlighted(BoneIndex) ? 1 : 0;

			BoneColors[BoneIndex] = BoneColor;
		}

		Component->MarkRenderStateDirty();
		Component->MarkRenderDynamicDataDirty();
	}
}
#endif

void UGeometryCollectionComponent::ApplyKinematicField(float Radius, FVector Position)
{
	FFieldSystemCommand Command = FFieldObjectCommands::CreateFieldCommand(EFieldPhysicsType::Field_DynamicState, new FRadialIntMask(Radius, Position, (int32)Chaos::EObjectStateType::Dynamic,
		(int32)Chaos::EObjectStateType::Kinematic, ESetMaskConditionType::Field_Set_IFF_NOT_Interior));
	DispatchFieldCommand(Command);
}

void UGeometryCollectionComponent::ApplyPhysicsField(bool Enabled, EGeometryCollectionPhysicsTypeEnum Target, UFieldSystemMetaData* MetaData, UFieldNodeBase* Field)
{
	if (Enabled && Field)
	{
		FFieldSystemCommand Command = FFieldObjectCommands::CreateFieldCommand(GetGeometryCollectionPhysicsType(Target), Field, MetaData);
		DispatchFieldCommand(Command);
	}
}

bool UGeometryCollectionComponent::GetIsObjectDynamic() const
{ 
	return PhysicsProxy ? PhysicsProxy->GetIsObjectDynamic() : IsObjectDynamic;
}

void UGeometryCollectionComponent::DispatchFieldCommand(const FFieldSystemCommand& InCommand)
{
	if (PhysicsProxy && InCommand.RootNode)
	{
		FChaosSolversModule* ChaosModule = FChaosSolversModule::GetModule();
		checkSlow(ChaosModule);

		auto Solver = PhysicsProxy->GetSolver<Chaos::FPBDRigidsSolver>();
		const FName Name = GetOwner() ? *GetOwner()->GetName() : TEXT("");

		FFieldSystemCommand LocalCommand = InCommand;
		LocalCommand.InitFieldNodes(Solver->GetSolverTime(), Name);

		PhysicsProxy->BufferFieldCommand_External(MoveTemp(LocalCommand));
	}
}

void UGeometryCollectionComponent::GetInitializationCommands(TArray<FFieldSystemCommand>& CombinedCommmands)
{
	CombinedCommmands.Reset();
	for (const AFieldSystemActor* FieldSystemActor : InitializationFields)
	{
		if (FieldSystemActor != nullptr)
		{
			if (FieldSystemActor->GetFieldSystemComponent())
			{
				const int32 NumCommands = FieldSystemActor->GetFieldSystemComponent()->ConstructionCommands.GetNumCommands();
				if (NumCommands > 0)
				{
					for (int32 CommandIndex = 0; CommandIndex < NumCommands; ++CommandIndex)
					{
						const FFieldSystemCommand NewCommand = FieldSystemActor->GetFieldSystemComponent()->ConstructionCommands.BuildFieldCommand(CommandIndex);
						if (NewCommand.RootNode)
						{
							CombinedCommmands.Emplace(NewCommand);

							UWorld* World = GetWorld();
							if (World && World->PhysicsField)
							{
								const FName Name = GetOwner() ? *GetOwner()->GetName() : TEXT("");

								FFieldSystemCommand LocalCommand = NewCommand;
								LocalCommand.InitFieldNodes(World->GetTimeSeconds(), Name);

								World->PhysicsField->AddConstructionCommand(LocalCommand);
							}
						}
					}
				}
				// Legacy path : only there for old levels. New ones will have the commands directly stored onto the component
				else if (FieldSystemActor->GetFieldSystemComponent()->GetFieldSystem())
				{
					const FName Name = GetOwner() ? *GetOwner()->GetName() : TEXT("");
					for (const FFieldSystemCommand& Command : FieldSystemActor->GetFieldSystemComponent()->GetFieldSystem()->Commands)
					{
						if (Command.RootNode)
						{
							FFieldSystemCommand NewCommand = { Command.TargetAttribute, Command.RootNode->NewCopy() };
							NewCommand.InitFieldNodes(0.0, Name);

							for (const TPair<FFieldSystemMetaData::EMetaType, TUniquePtr<FFieldSystemMetaData>>& Elem : Command.MetaData)
							{
								NewCommand.MetaData.Add(Elem.Key, TUniquePtr<FFieldSystemMetaData>(Elem.Value->NewCopy()));
							}
							CombinedCommmands.Emplace(NewCommand);
						}
					}
				}
			}
		}
	}
}

bool UGeometryCollectionComponent::GetSuppressSelectionMaterial() const
{
	return RestCollection->GetGeometryCollection()->HasAttribute("Hide", FGeometryCollection::TransformGroup);
}

FPhysScene_Chaos* UGeometryCollectionComponent::GetInnerChaosScene() const
{
	if (ChaosSolverActor)
	{
		return ChaosSolverActor->GetPhysicsScene().Get();
	}
	else
	{
		if (GetOwner() && GetOwner()->GetWorld())
		{
			return GetOwner()->GetWorld()->GetPhysicsScene();
		}

		if (GWorld)
		{
			return GWorld->GetPhysicsScene();
		}

		UE_LOGF(LogPhysics, Error, "Failed to find valid UWorld in UGeometryCollectionComponent::GetInnerChaosScene");
		return nullptr;
	}
}

AChaosSolverActor* UGeometryCollectionComponent::GetPhysicsSolverActor() const
{
	if (ChaosSolverActor)
	{
		return ChaosSolverActor;
	}
	else
	{
		FPhysScene_Chaos const* const Scene = GetInnerChaosScene();
		return Scene ? Cast<AChaosSolverActor>(Scene->GetSolverActor()) : nullptr;
	}
}

void UGeometryCollectionComponent::SetSolverActor(AChaosSolverActor* InSolverActor)
{
	ChaosSolverActor = InSolverActor;

	ResetState();
}

bool UGeometryCollectionComponent::CanRunSimulationInEditor() const
{
	if (UWorld* World = GetWorld())
	{
		Chaos::FPhysicsSolver* CurrSolver = GetSolver(*this);
		return World->IsGameWorld() || GeometryCollectionCreatePhysicsStateInEditor || (CurrSolver && CurrSolver->IsStandaloneSolver());
	}
	return false;
}

#define GEOMETRY_COLLECTION_CHECK_FOR_NANS_IN_TRANSFORMS 0

#if GEOMETRY_COLLECTION_CHECK_FOR_NANS_IN_TRANSFORMS
static void CheckForNaNs(const TArray<FTransform>& Transforms)
{
	for (const FTransform& Transform : Transforms)
	{
		ensureAlways(false == Transform.ContainsNaN());
	}
}
#endif

FTransform3f UGeometryCollectionComponent::GetCurrentTransform(int32 Index) const
{
	if (DynamicCollection)
	{
		return DynamicCollection->GetTransform(Index);
	}

	const bool bRestTransformsOverriden = (RestTransforms.Num() > 0);
	if (bRestTransformsOverriden)
	{
		return FTransform3f(RestTransforms[Index]);
	}
	return RestCollection->GetGeometryCollection()->Transform[Index];
}

void UGeometryCollectionComponent::ComputeCurrentGlobalsMatrices(TArray<FTransform3f>& OutTransforms) const
{
	if (DynamicCollection)
	{
		GeometryCollectionAlgo::Private::GlobalMatrices(*DynamicCollection, OutTransforms);
	}
	else
	{
		const bool bRestTransformsOverriden = (RestTransforms.Num() > 0);
		if (bRestTransformsOverriden)
		{
			GeometryCollectionAlgo::GlobalMatrices(TManagedArray<FTransform>(RestTransforms), GetParentArrayRest(), OutTransforms);
		}
		else
		{
			GeometryCollectionAlgo::GlobalMatrices(RestCollection->GetGeometryCollection()->Transform, GetParentArrayRest(), OutTransforms);
		}
	}
}

const FTransform3f& UGeometryCollectionComponent::FComponentSpaceTransforms::RequestRootTransform() const
{
	if (!Component)
	{
		ensure(false); // we should not be able to reach here 
		return FTransform3f::Identity;
	}

	if (Transforms.IsValidIndex(RootIndex))
	{
		if (bIsRootDirty)
		{
			Transforms[RootIndex] = Component->GetCurrentTransform(RootIndex);
			bIsRootDirty = false;
		}

		return Transforms[RootIndex];
	}

	return FTransform3f::Identity;
}

const TArray<FTransform3f>& UGeometryCollectionComponent::FComponentSpaceTransforms::RequestAllTransforms() const
{
	SCOPE_CYCLE_COUNTER(STAT_GCCUGlobalMatrices);

	static TArray<FTransform3f> EmptyArray;
	if (!Component || Transforms.Num() == 0)
	{
		return EmptyArray;
	}

	if (!bIsDirty)
	{
		return Transforms;
	}

	int32 CurrentTransformNum;
	if (Component->DynamicCollection)
	{
		CurrentTransformNum = Component->DynamicCollection->GetNumTransforms();
	}
	else
	{
		CurrentTransformNum = Component->RestCollection->GetGeometryCollection()->Transform.Num();
	}

	bool bFastPath = false;
	if (Component->RestCollection)
	{
		const TArray<int32>& BreadthFirstTransformIndices = Component->RestCollection->GetBreadthFirstTransformIndices();
		bFastPath = (BreadthFirstTransformIndices.Num() == CurrentTransformNum);
	}

	if (bFastPath)
	{
		const TArray<int32>& BreadthFirstTransformIndices = Component->RestCollection->GetBreadthFirstTransformIndices();

		for (int32 Index = 0; Index < BreadthFirstTransformIndices.Num(); Index++)
		{
			const int32 TransformIndex = BreadthFirstTransformIndices[Index];
			const int32 ParentTransformIndex = Component->GetParent(TransformIndex);


			FTransform3f CurrentTransform = Component->GetCurrentTransform(TransformIndex);
			if (ParentTransformIndex == INDEX_NONE)
			{
				
				Transforms[TransformIndex] = CurrentTransform;
			}
			else
			{
				const FTransform3f& ParentTransform = Transforms[ParentTransformIndex];
				Transforms[TransformIndex] = CurrentTransform * ParentTransform;
			}
		}
	}
	else
	{
		Component->ComputeCurrentGlobalsMatrices(Transforms);
	}

#if WITH_EDITOR
	UpdateGlobalMatricesWithExplodedVectors(Transforms, *(Component->RestCollection->GetGeometryCollection()));
#endif

#if GEOMETRY_COLLECTION_CHECK_FOR_NANS_IN_TRANSFORMS
	CheckForNaNs(Transforms);
#endif

	bIsDirty = false;
	bIsRootDirty = false;
	return Transforms;
}

int32 UGeometryCollectionComponent::GetNumMaterials() const
{
	return !RestCollection ? 0 : RestCollection->Materials.Num();
}

UMaterialInterface* UGeometryCollectionComponent::GetMaterial(int32 MaterialIndex) const
{
	// If we have a base materials array, use that
	if (OverrideMaterials.IsValidIndex(MaterialIndex) && OverrideMaterials[MaterialIndex])
	{
		return OverrideMaterials[MaterialIndex];
	}
	// Otherwise get from geom collection
	else
	{
		return RestCollection && RestCollection->Materials.IsValidIndex(MaterialIndex) ? RestCollection->Materials[MaterialIndex] : nullptr;
	}
}

void UGeometryCollectionComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	Super::GetUsedMaterials(OutMaterials, bGetDebugMaterials);

	if (GetRestCollection())
	{
		if (UMaterialInterface* BoneSelectedMaterial = GetRestCollection()->GetBoneSelectedMaterial())
		{
			OutMaterials.Add(BoneSelectedMaterial);
		}
	}
}

FMaterialRelevance UGeometryCollectionComponent::GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return GetMaterialRelevance(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

FMaterialRelevance UGeometryCollectionComponent::GetMaterialRelevance(EShaderPlatform InShaderPlatform) const
{
	FMaterialRelevance Result = Super::GetMaterialRelevance(InShaderPlatform);
	if (RestCollection)
	{
		if (UMaterialInterface* BoneSelectedMaterial = GetRestCollection()->GetBoneSelectedMaterial())
		{
			Result |= BoneSelectedMaterial->GetRelevance_Concurrent(InShaderPlatform);
		}
	}
	return Result;
}

#if WITH_EDITOR
void UGeometryCollectionComponent::SelectEmbeddedGeometry()
{
	// First reset the selections
	for (TObjectPtr<UInstancedStaticMeshComponent>& EmbeddedGeometryComponent : EmbeddedGeometryComponents)
	{
		EmbeddedGeometryComponent->ClearInstanceSelection();
	}
	
	const TManagedArray<int32>& ExemplarIndex = RestCollection->GetGeometryCollection()->ExemplarIndex;
	for (int32 SelectedBone : SelectedBones)
	{
		if (EmbeddedGeometryComponents.IsValidIndex(ExemplarIndex[SelectedBone]))
		{
			EmbeddedGeometryComponents[ExemplarIndex[SelectedBone]]->SelectInstance(true, EmbeddedInstanceIndex[SelectedBone], 1);
		}
	}
}
#endif

// #temp HACK for demo, When fracture happens (physics state changes to dynamic) then switch the visible render meshes in a blueprint/actor from static meshes to geometry collections
void UGeometryCollectionComponent::SwitchRenderModels(const AActor* Actor)
{
	// Don't touch visibility if the component is not visible
	if (!IsVisible())
	{
		return;
	}

	TInlineComponentArray<UPrimitiveComponent*> PrimitiveComponents;
	Actor->GetComponents(PrimitiveComponents);
	for (UPrimitiveComponent* PrimitiveComponent : PrimitiveComponents)
	{
		bool ValidComponent = false;

		if (UStaticMeshComponent* StaticMeshComp = Cast<UStaticMeshComponent>(PrimitiveComponent))
		{
			// unhacked.
			//StaticMeshComp->SetVisibility(false);
		}
		else if (UGeometryCollectionComponent* GeometryCollectionComponent = Cast<UGeometryCollectionComponent>(PrimitiveComponent))
		{
			if (!GeometryCollectionComponent->IsVisible())
			{
				continue;
			}

			GeometryCollectionComponent->SetVisibility(true);
		}
	}

	TInlineComponentArray<UChildActorComponent*> ChildActorComponents;
	Actor->GetComponents(ChildActorComponents);
	for (UChildActorComponent* ChildComponent : ChildActorComponents)
	{
		AActor* ChildActor = ChildComponent->GetChildActor();
		if (ChildActor)
		{
			SwitchRenderModels(ChildActor);
		}
	}

}

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
void UGeometryCollectionComponent::EnableTransformSelectionMode(bool bEnable)
{
	// TODO: Support for Nanite?
	bIsTransformSelectionModeEnabled = bEnable;
	MarkRenderStateDirty();
}
#endif  // #if GEOMETRYCOLLECTION_EDITOR_SELECTION

bool UGeometryCollectionComponent::IsEmbeddedGeometryValid() const
{
	// Check that the array of ISMCs that implement embedded geometry matches RestCollection Exemplar array.
	if (!RestCollection)
	{
		return false;
	}

	if (RestCollection->EmbeddedGeometryExemplar.Num() != EmbeddedGeometryComponents.Num())
	{
		return false;
	}

	for (int32 Idx = 0; Idx < EmbeddedGeometryComponents.Num(); ++Idx)
	{
		UStaticMesh* ExemplarStaticMesh = Cast<UStaticMesh>(RestCollection->EmbeddedGeometryExemplar[Idx].StaticMeshExemplar.TryLoad());
		if (!ExemplarStaticMesh)
		{
			return false;
		}

		if (ExemplarStaticMesh != EmbeddedGeometryComponents[Idx]->GetStaticMesh())
		{
			return false;
		}
	}

	return true;
}

void UGeometryCollectionComponent::ClearEmbeddedGeometry()
{
	if (AActor* OwningActor = GetOwner())
	{
		TArray<UActorComponent*> TargetComponents;
		OwningActor->GetComponents(TargetComponents, false);

		for (UActorComponent* TargetComponent : TargetComponents)
		{
			if ((TargetComponent->GetOuter() == this) || !IsValidChecked(TargetComponent->GetOuter()))
			{
				if (UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(TargetComponent))
				{
					ISMComponent->ClearInstances();
					ISMComponent->DestroyComponent();
				}
			}
		}
	}
	EmbeddedGeometryComponents.Empty();
}

void UGeometryCollectionComponent::InitializeEmbeddedGeometry()
{
	if (RestCollection)
	{
		ClearEmbeddedGeometry();
		
		if (AActor* ActorOwner = GetOwner())
		{

			// Construct an InstancedStaticMeshComponent for each exemplar
			for (const FGeometryCollectionEmbeddedExemplar& Exemplar : RestCollection->EmbeddedGeometryExemplar)
			{
				if (UStaticMesh* ExemplarStaticMesh = Cast<UStaticMesh>(Exemplar.StaticMeshExemplar.TryLoad()))
				{
					if (UInstancedStaticMeshComponent* ISMC = NewObject<UInstancedStaticMeshComponent>(this))
					{
						ISMC->SetStaticMesh(ExemplarStaticMesh);
						ISMC->SetCullDistances(Exemplar.StartCullDistance, Exemplar.EndCullDistance);
						ISMC->SetCanEverAffectNavigation(false);
						ISMC->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
						ISMC->SetCastShadow(false);
						ISMC->SetMobility(EComponentMobility::Stationary);
						ISMC->SetupAttachment(this);
						ActorOwner->AddInstanceComponent(ISMC);
						ISMC->RegisterComponent();

						EmbeddedGeometryComponents.Add(ISMC);
					}
				}
			}

#if WITH_EDITOR
			EmbeddedBoneMaps.SetNum(RestCollection->EmbeddedGeometryExemplar.Num());
			EmbeddedInstanceIndex.Init(INDEX_NONE, RestCollection->GetGeometryCollection()->NumElements(FGeometryCollection::TransformGroup));
#endif
		}
	}
}

void UGeometryCollectionComponent::ForceBrokenForCustomRenderer(bool bForceBroken)
{ 
	bForceBrokenForCustomRenderer = bForceBroken;
	RefreshCustomRenderer();
}

void UGeometryCollectionComponent::SetRootProxyComponentSpaceTransform(int32 Index, const FTransform& RootProxyTransform)
{
	SetRootProxyLocalTransform(Index, FTransform3f(RootProxyTransform));
	RefreshRootProxies();
}

void UGeometryCollectionComponent::SetRootProxyLocalTransform(int32 Index, const FTransform3f& RootProxyTransform)
{
	if (RootProxyLocalTransforms.IsEmpty() && RestCollection)
	{
		const int32 NumProxyMeshes = RestCollection->RootProxyData.ProxyMeshes.Num();
		RootProxyLocalTransforms.Reserve(NumProxyMeshes);
		RootProxyLocalTransforms.Append(RestCollection->RootProxyData.MeshTransforms);
		for (int32 TransformIndex = RootProxyLocalTransforms.Num(); TransformIndex < NumProxyMeshes; ++TransformIndex)
		{
			RootProxyLocalTransforms.Add(FTransform3f::Identity);
		}
	}
	if (RootProxyLocalTransforms.IsValidIndex(Index))
	{
		RootProxyLocalTransforms[Index] = RootProxyTransform;
	}
}

void UGeometryCollectionComponent::ClearRootProxyLocalTransforms()
{
	RootProxyLocalTransforms.Empty();
}

void UGeometryCollectionComponent::RefreshRootProxies()
{
	RefreshCustomRenderer();
}

bool UGeometryCollectionComponent::IsCustomRendererAvailable() const
{
	return bChaos_GC_UseCustomRenderer && CustomRenderer != nullptr && FApp::CanEverRender();
}

bool UGeometryCollectionComponent::IsUsingCustomRenderer() const
{
	return IsCustomRendererAvailable() && !(bCustomRendererCanUseNativeFallback && bCustomRendererShouldUseNativeFallback) && !bForceNativeRenderer;
}

void UGeometryCollectionComponent::RefreshCustomRenderer()
{
	if (RestCollection == nullptr || !IsCustomRendererAvailable())
	{
		return;
	}
	
	if (IGeometryCollectionExternalRenderInterface* RendererInterface = CustomRenderer.GetInterface())
	{
		bool bIsBroken = BrokenAndDecayedStates.GetIsRootBroken();

#if !(UE_BUILD_SHIPPING)
		if (CVarNumToForceBreak->GetInt() >= 0)
		{
			bIsBroken = ForcedBroken.Break(this, ComponentSpaceTransforms.RequestAllTransforms());
		}
#endif						

		const bool bRenderRootProxy = !bForceBrokenForCustomRenderer && !bIsBroken && (RestCollection->RootProxyData.ProxyMeshes.Num() > 0);

		const AActor* Owner = GetOwner();
		const bool bIsActorHidden = (Owner && bGeometryCollectionCustomRendererHiddenActorFix) ? Owner->IsHidden() : false;

		uint32 StateFlags = 0;
		StateFlags |= (bHiddenInGame || !IsVisible() || bIsActorHidden) ? 0 : IGeometryCollectionExternalRenderInterface::EState_Visible;
		StateFlags |= bRenderRootProxy ? 0 : IGeometryCollectionExternalRenderInterface::EState_Broken;
		StateFlags |= bForceBrokenForCustomRenderer ? IGeometryCollectionExternalRenderInterface::EState_ForcedBroken : 0;

		// Test if we should change the native rendering fallback state before updating custom renderer.
		const bool bShouldUseNativeFallback = bCustomRendererCanUseNativeFallback && RendererInterface->ShouldUseNativeFallback(StateFlags);
		if (bCustomRendererShouldUseNativeFallback != bShouldUseNativeFallback)
		{
			bCustomRendererShouldUseNativeFallback = bShouldUseNativeFallback;
			UnregisterCustomRenderer();
			RegisterCustomRenderer();
			MarkRenderStateDirty();
		}

		if (IsUsingCustomRenderer())
		{
			RendererInterface->UpdateState(*RestCollection, GetComponentTransform(), StateFlags);

			if (bRenderRootProxy)
			{
				const FTransform CompSpaceRootTransform(ComponentSpaceTransforms.RequestRootTransform());
				if (RootProxyLocalTransforms.IsEmpty())
				{
					if (RestCollection->RootProxyData.MeshTransforms.IsEmpty())
					{
						RendererInterface->UpdateRootTransform(*RestCollection, CompSpaceRootTransform);
					}
					else
					{
						RendererInterface->UpdateRootTransforms(*RestCollection, CompSpaceRootTransform, MakeArrayView(RestCollection->RootProxyData.MeshTransforms));
					}
				}
				else
				{
					RendererInterface->UpdateRootTransforms(*RestCollection, CompSpaceRootTransform, MakeArrayView(RootProxyLocalTransforms));
				}
			}
			else
			{
				const TArray<FTransform3f>& CompSpaceTransforms = ComponentSpaceTransforms.RequestAllTransforms();
				RendererInterface->UpdateTransforms(*RestCollection, CompSpaceTransforms);
			}
		}
	}
}

void UGeometryCollectionComponent::SetUseStaticMeshCollisionForTraces(const bool bInUseStaticMeshCollisionForTraces)
{
	if (bUseStaticMeshCollisionForTraces != bInUseStaticMeshCollisionForTraces)
	{
		if (PhysicsProxy != nullptr)
		{
			PhysicsProxy->SetUseStaticMeshCollisionForTraces_External(bInUseStaticMeshCollisionForTraces);
		}

		bUseStaticMeshCollisionForTraces = bInUseStaticMeshCollisionForTraces;
	}
}
struct FGeometryCollectionDecayContext
{
	FGeometryCollectionDecayContext(FGeometryCollectionPhysicsProxy& PhysicsProxyIn, FGeometryCollectionDecayDynamicFacade& DecayFacadeIn)
		: PhysicsProxy(PhysicsProxyIn)
		, DecayFacade(DecayFacadeIn)
		, DirtyDynamicCollection(false)
	{}

	FGeometryCollectionPhysicsProxy& PhysicsProxy;
	FGeometryCollectionDecayDynamicFacade& DecayFacade;
	
	bool DirtyDynamicCollection;
	TArray<int32> ToDisable;
	TArray<FGeometryCollectionItemIndex> ToCrumble;

	void Process(FGeometryDynamicCollection& DynamicCollection)
	{
		if (DirtyDynamicCollection)
		{
			DynamicCollection.MakeDirty();
		}
		if (ToCrumble.Num())
		{
			PhysicsProxy.BreakClusters_External(MoveTemp(ToCrumble));
		}
		if (ToDisable.Num())
		{
			PhysicsProxy.DisableParticles_External(MoveTemp(ToDisable));
		}
	}
};

void UGeometryCollectionComponent::UpdateDecay(int32 TransformIdx, float UpdatedDecay, bool bUseClusterCrumbling, bool bHasDynamicInternalClusterParent, FGeometryCollectionDecayContext& ContextInOut)
{
	float Decay = ContextInOut.DecayFacade.GetDecay(TransformIdx);
	if (UpdatedDecay > Decay)
	{
		ContextInOut.DirtyDynamicCollection = true;
		Decay = UpdatedDecay;

		if (bUseClusterCrumbling)
		{
			if (bHasDynamicInternalClusterParent)
			{
				FGeometryCollectionItemIndex InternalClusterItemindex = ContextInOut.PhysicsProxy.GetInternalClusterParentItemIndex_External(TransformIdx);
				if (InternalClusterItemindex.IsValid())
				{
					ContextInOut.ToCrumble.AddUnique(InternalClusterItemindex);
					Decay = 0.0f;
					// this is an internal cluster so we do not update the BrokenAndDecayedStates, the children will update their own state 
				}
			}
			else
			{
				ContextInOut.ToCrumble.AddUnique(FGeometryCollectionItemIndex::CreateTransformItemIndex(TransformIdx));
				Decay = 0.0f;
				BrokenAndDecayedStates.SetHasDecayed(TransformIdx);
			}
		}
		else if (Decay >= 1.0f)
		{
			// Disable the particle if it has decayed the requisite time
			Decay = 1.0f;
			ContextInOut.ToDisable.Add(TransformIdx);
			if (RestCollection && RestCollection->GetGeometryCollection())
			{
				BrokenAndDecayedStates.SetHasDecayedRecursive(TransformIdx, RestCollection->GetGeometryCollection()->Children.GetConstArray());
			}
		}

		// push back Decay in the attribute
		ContextInOut.DecayFacade.SetDecay(TransformIdx, Decay);
	}
}

void UGeometryCollectionComponent::IncrementSleepTimer(float DeltaTime)
{
	if (DeltaTime <= 0 || !RestCollection || !RestCollection->bRemoveOnMaxSleep || !bAllowRemovalOnSleep)
	{
		return;
	}
	
	// If a particle is sleeping, increment its sleep timer, otherwise reset it.
	if (DynamicCollection && PhysicsProxy)
	{
		FGeometryCollectionRemoveOnSleepDynamicFacade RemoveOnSleepFacade(*DynamicCollection);
		FGeometryCollectionDecayDynamicFacade DecayFacade(*DynamicCollection);
		FGeometryCollectionDynamicStateFacade DynamicStateFacade(*DynamicCollection);

		if (RemoveOnSleepFacade.IsValid()
			&& DecayFacade.IsValid()
			&& DynamicStateFacade.IsValid())
		{
			FGeometryCollectionDecayContext DecayContext(*PhysicsProxy, DecayFacade);

			const TManagedArray<int32>& OriginalParents = RestCollection->GetGeometryCollection()->Parent;
			const bool bAutomaticCrumblePartialClusters = RestCollection->bAutomaticCrumblePartialClusters;

			const int32 NumTransforms = OriginalParents.Num();
			for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
			{
				const bool HasInternalClusterParent = DynamicStateFacade.HasInternalClusterParent(TransformIdx);
				if (HasInternalClusterParent)
				{
					// this children has an dynamic internal cluster parent so it can't be removed but we need tyo process the internal cluster by looking at the original parent properties
					const int32 OriginalParentIdx = OriginalParents[TransformIdx];
					const bool HasDynamicInternalClusterParent = DynamicStateFacade.HasDynamicInternalClusterParent(TransformIdx);
					if (OriginalParentIdx > INDEX_NONE && HasDynamicInternalClusterParent && RemoveOnSleepFacade.IsRemovalActive(OriginalParentIdx))
					{
						const bool UseClusterCrumbling = bAutomaticCrumblePartialClusters;
						const float UpdatedBreakDecay = UE_SMALL_NUMBER; // since we crumble we can only pass a timy number since this will be ignore ( but need to be >0 to ake sure Update Decay works properly )
						UpdateDecay(TransformIdx, UpdatedBreakDecay, UseClusterCrumbling, HasDynamicInternalClusterParent, DecayContext);
					}
				}
				else if (RemoveOnSleepFacade.IsRemovalActive(TransformIdx) && DynamicStateFacade.HasBrokenOff(TransformIdx))
				{
					// root bone should not be affected by remove on sleep 
					if (OriginalParents[TransformIdx] > INDEX_NONE)
					{
						// if decay has started we do not need to check slow moving or sleeping state anymore  
						bool ShouldUpdateTimer = (DecayFacade.GetDecay(TransformIdx) > 0);
						if (!ShouldUpdateTimer && RestCollection->bSlowMovingAsSleeping)
						{
							const FVector3f CurrentPosition = DynamicCollection->GetTransform(TransformIdx).GetTranslation();
							ShouldUpdateTimer |= RemoveOnSleepFacade.ComputeSlowMovingState(TransformIdx, FVector(CurrentPosition), DeltaTime, RestCollection->SlowMovingVelocityThreshold);
						}
						if (ShouldUpdateTimer || DynamicStateFacade.IsSleeping(TransformIdx))
						{
							RemoveOnSleepFacade.UpdateSleepTimer(TransformIdx, DeltaTime);
						}

						// update the decay and disable the particle when decay has completed 
						const float UpdatedDecay = RemoveOnSleepFacade.ComputeDecay(TransformIdx);
						UpdateDecay(TransformIdx, UpdatedDecay, DynamicStateFacade.HasChildren(TransformIdx), false, DecayContext);
					}
				}
			}

			DecayContext.Process(*DynamicCollection);
		}
	}
}

void UGeometryCollectionComponent::IncrementBreakTimer(float DeltaTime)
{
	if (DeltaTime <= 0 || !bAllowRemovalOnBreak)
	{
		return;
	}
	
	if (RestCollection && DynamicCollection && PhysicsProxy)
	{
		FGeometryCollectionRemoveOnBreakDynamicFacade RemoveOnBreakFacade(*DynamicCollection);
		FGeometryCollectionDecayDynamicFacade DecayFacade(*DynamicCollection);
		FGeometryCollectionDynamicStateFacade DynamicStateFacade(*DynamicCollection);

		// if replication is on, client may not need to process this at all or only partially ( depending on the abandon cluster level )
		const bool bIsReplicatedClient = GetIsReplicated() && PhysicsProxy->GetReplicationMode() == FGeometryCollectionPhysicsProxy::EReplicationMode::Client;

		if (RemoveOnBreakFacade.IsValid()
			&& DecayFacade.IsValid()
			&& DynamicStateFacade.IsValid())
		{
			FGeometryCollectionDecayContext DecayContext(*PhysicsProxy, DecayFacade);
			const TManagedArray<int32>& OriginalParents = RestCollection->GetGeometryCollection()->Parent;

			const TManagedArrayAccessor<int32> InitialLevels = PhysicsProxy->GetPhysicsCollection().GetInitialLevels();

			const int32 NumTransforms = OriginalParents.Num();
			for (int32 TransformIdx = 0; TransformIdx < NumTransforms; ++TransformIdx)
			{
				const bool HasInternalClusterParent = DynamicStateFacade.HasInternalClusterParent(TransformIdx);
				if (HasInternalClusterParent)
				{
					// this children has an internal cluster parent so it can't be removed but we need tyo process the internal cluster by looking at the original parent properties
					const int32 OriginalParentIdx = OriginalParents[TransformIdx];
					const bool HasDynamicInternalClusterParent = DynamicStateFacade.HasDynamicInternalClusterParent(TransformIdx);

					if (OriginalParentIdx > INDEX_NONE && HasDynamicInternalClusterParent && RemoveOnBreakFacade.IsRemovalActive(OriginalParentIdx))
					{
						bool bIsAllowedClusterCrumbling = true;
						if (bIsReplicatedClient && InitialLevels.IsValid() && (InitialLevels.Num() > 0))
						{
							if (!bEnableAbandonAfterLevel || InitialLevels[OriginalParentIdx] <= ReplicationAbandonAfterLevel)
							{
								bIsAllowedClusterCrumbling = false;
							}
						}

						const bool UseClusterCrumbling = RemoveOnBreakFacade.UseClusterCrumbling(OriginalParentIdx);
						if (!UseClusterCrumbling || bIsAllowedClusterCrumbling)
						{
							const float UpdatedBreakDecay = RemoveOnBreakFacade.UpdateBreakTimerAndComputeDecay(TransformIdx, DeltaTime);
							UpdateDecay(TransformIdx, UpdatedBreakDecay, UseClusterCrumbling, HasDynamicInternalClusterParent, DecayContext);
						}
					}
				} 
				else if (RemoveOnBreakFacade.IsRemovalActive(TransformIdx) && DynamicStateFacade.HasBrokenOff(TransformIdx))
				{
					bool bIsAllowedClusterCrumbling = true;
					if (bIsReplicatedClient && InitialLevels.IsValid() && (InitialLevels.Num() > 0))
					{
						if (!bEnableAbandonAfterLevel || InitialLevels[TransformIdx] <= ReplicationAbandonAfterLevel)
						{
							bIsAllowedClusterCrumbling = false;
						}
					}

					const bool UseClusterCrumbling = RemoveOnBreakFacade.UseClusterCrumbling(TransformIdx);
					if (!UseClusterCrumbling || bIsAllowedClusterCrumbling)
					{
						const float UpdatedBreakDecay = RemoveOnBreakFacade.UpdateBreakTimerAndComputeDecay(TransformIdx, DeltaTime);
						UpdateDecay(TransformIdx, UpdatedBreakDecay, UseClusterCrumbling, false, DecayContext);
					}
				}
			}

			DecayContext.Process(*DynamicCollection);
		}
	}
}

void UGeometryCollectionComponent::ApplyExternalStrain(int32 ItemIndex, const FVector& Location, float Radius, int32 PropagationDepth, float PropagationFactor, float Strain)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->ApplyExternalStrain_External(FGeometryCollectionItemIndex::CreateFromExistingItemIndex(ItemIndex), Location, Radius, PropagationDepth, PropagationFactor, Strain);
	}
}

void UGeometryCollectionComponent::ApplyInternalStrain(int32 ItemIndex, const FVector& Location, float Radius, int32 PropagationDepth, float PropagationFactor, float Strain)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->ApplyInternalStrain_External(FGeometryCollectionItemIndex::CreateFromExistingItemIndex(ItemIndex), Location, Radius, PropagationDepth, PropagationFactor, Strain);
	}
}

void UGeometryCollectionComponent::CrumbleCluster(int32 ItemIndex)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->BreakClusters_External({FGeometryCollectionItemIndex::CreateFromExistingItemIndex(ItemIndex)});
	}
}

void UGeometryCollectionComponent::CrumbleActiveClusters()
{
	if (PhysicsProxy)
	{
		PhysicsProxy->BreakActiveClusters_External();
	}
}

void UGeometryCollectionComponent::SetAnchoredByIndex(int32 Index, bool bAnchored)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->SetAnchoredByIndex_External(Index, bAnchored);
	}
}

void UGeometryCollectionComponent::SetAnchoredByBox(FBox WorldSpaceBox, bool bAnchored, int32 MaxLevel)
{
	SetAnchoredByTransformedBox(WorldSpaceBox, FTransform::Identity, bAnchored, MaxLevel);
}

void UGeometryCollectionComponent::SetAnchoredByTransformedBox(FBox Box, FTransform Transform, bool bAnchored, int32 MaxLevel)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->SetAnchoredByTransformedBox_External(Box, Transform, bAnchored, MaxLevel);
	}
}
void UGeometryCollectionComponent::RemoveAllAnchors()
{
	if (PhysicsProxy)
	{
		PhysicsProxy->RemoveAllAnchors_External();
	}
}

void UGeometryCollectionComponent::ApplyBreakingLinearVelocity(int32 ItemIndex, const FVector& LinearVelocity)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->ApplyBreakingLinearVelocity_External(FGeometryCollectionItemIndex::CreateFromExistingItemIndex(ItemIndex), LinearVelocity);
	}

}

void UGeometryCollectionComponent::ApplyBreakingAngularVelocity(int32 ItemIndex, const FVector& AngularVelocity)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->ApplyBreakingAngularVelocity_External(FGeometryCollectionItemIndex::CreateFromExistingItemIndex(ItemIndex), AngularVelocity);
	}
}

void UGeometryCollectionComponent::ApplyLinearVelocity(int32 ItemIndex, const FVector& LinearVelocity)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->ApplyLinearVelocity_External(FGeometryCollectionItemIndex::CreateFromExistingItemIndex(ItemIndex), LinearVelocity);
	}
}

void UGeometryCollectionComponent::ApplyAngularVelocity(int32 ItemIndex, const FVector& AngularVelocity)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->ApplyAngularVelocity_External(FGeometryCollectionItemIndex::CreateFromExistingItemIndex(ItemIndex), AngularVelocity);
	}
}

int32 UGeometryCollectionComponent::GetInitialLevel(int32 ItemIndex)
{
	using FGeometryCollectionPtr = const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>; 

	int32 Level = INDEX_NONE;
	if (RestCollection && RestCollection->GetGeometryCollection())
	{
		const TManagedArray<int32>& Parent = RestCollection->GetGeometryCollection()->Parent;
		int32 TransformIndex = INDEX_NONE;

		FGeometryCollectionItemIndex GCItemIndex = FGeometryCollectionItemIndex::CreateFromExistingItemIndex(ItemIndex);

		if (GCItemIndex.IsInternalCluster())
		{
			if (const TArray<int32>* Children = PhysicsProxy->FindInternalClusterChildrenTransformIndices_External(GCItemIndex))
			{
				if (!Children->IsEmpty())
				{
					// find the original cluster index from first children
					TransformIndex = Parent[(*Children)[0]];
				}
			}
		}
		else
		{
			TransformIndex = GCItemIndex.GetTransformIndex();
		}

		// @todo(chaos) : use "Level" attribute when it will be properly serialized
		// for now climb back the hierarchy
		if (TransformIndex > INDEX_NONE)
		{
			Level = 0;
			int32 ParentTransformIndex =  Parent[TransformIndex];
			while (ParentTransformIndex != INDEX_NONE)
			{
				++Level;
				ParentTransformIndex =  Parent[ParentTransformIndex];
			}
		}
	}
	return Level;
}

int32 UGeometryCollectionComponent::GetRootIndex() const
{
	if (RestCollection)
	{
		return RestCollection->GetRootIndex();
	}
	return INDEX_NONE;
}

FTransform UGeometryCollectionComponent::GetRootInitialTransform() const
{
	FTransform RootInitialTransform{ FTransform::Identity };
	if (RestCollection && RestCollection->GetGeometryCollection())
	{
		const FGeometryCollection& RestGeometryCollection = *RestCollection->GetGeometryCollection();

		const int32 RootIndex = RestCollection->GetRootIndex();
		if (RestGeometryCollection.Transform.IsValidIndex(RootIndex))
		{
			const FTransform LocalSpaceTransform = GeometryCollectionAlgo::GlobalMatrix(RestGeometryCollection.Transform, RestGeometryCollection.Parent, RootIndex);
			RootInitialTransform = LocalSpaceTransform * GetComponentTransform();
		}
	}
	return RootInitialTransform;
}

FTransform UGeometryCollectionComponent::GetRootCurrentTransform() const
{
	FTransform RootInitialTransform{ FTransform::Identity };
	if (RestCollection)
	{
		const FTransform CompSpaceRootTransform(ComponentSpaceTransforms.RequestRootTransform());
		RootInitialTransform = CompSpaceRootTransform * GetComponentTransform();
	}
	return RootInitialTransform;
}

FTransform UGeometryCollectionComponent::GetRootCurrentComponentSpaceTransform() const
{
	return (RestCollection) ? FTransform(ComponentSpaceTransforms.RequestRootTransform()) : FTransform::Identity;
}

FTransform UGeometryCollectionComponent::GetRootParticleMassOffset() const
{
	FTransform MassOffset{ FTransform::Identity };
	if (RestCollection)
	{
		const int32 RootIndex = RestCollection->GetRootIndex();
		if (const FGeometryCollection* RestGeometryCollection = RestCollection->GetGeometryCollection().Get())
		{
			const TManagedArray<FTransform>* MassToLocal = RestGeometryCollection->FindAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);
			if (MassToLocal && MassToLocal->IsValidIndex(RootIndex))
			{
				MassOffset = (*MassToLocal)[RootIndex];
			}
		}
	}
	return MassOffset;
}

TArray<FTransform> UGeometryCollectionComponent::GetInitialLocalRestTransforms() const
{
	TArray<FTransform> InitialLocalTransforms;

	if (RestCollection && RestCollection->GetGeometryCollection())
	{
		const FGeometryCollection& RestGeometryCollection = *RestCollection->GetGeometryCollection();

		GeometryCollectionAlgo::GlobalMatrices(RestGeometryCollection.Transform, RestGeometryCollection.Parent, InitialLocalTransforms);

		const TManagedArray<FTransform>* MassToLocal = RestGeometryCollection.FindAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);
		if (MassToLocal && InitialLocalTransforms.Num() == MassToLocal->Num())
		{
			for (int32 TransformIndex = 0; TransformIndex < InitialLocalTransforms.Num(); TransformIndex++)
			{
				InitialLocalTransforms[TransformIndex] = (*MassToLocal)[TransformIndex] * InitialLocalTransforms[TransformIndex];
			}
		}
	}
	return InitialLocalTransforms;
}

TArray<FTransform> UGeometryCollectionComponent::GetLocalRestTransforms(bool bInitialTransforms) const
{
	TArray<FTransform> CurrentLocalTransforms;
	if (RestCollection && RestCollection->GetGeometryCollection())
	{
		const FGeometryCollection& RestGeometryCollection = *RestCollection->GetGeometryCollection();

		const bool bHasRestTransformOverride = (RestTransforms.Num() > 0) && (RestTransforms.Num() == RestGeometryCollection.Transform.Num());
		if (bInitialTransforms || !bHasRestTransformOverride)
		{
			GeometryCollectionAlgo::GlobalMatrices(RestCollection->GetGeometryCollection()->Transform, RestGeometryCollection.Parent, CurrentLocalTransforms);
		}
		else
		{
			GeometryCollectionAlgo::GlobalMatrices(TManagedArray<FTransform>(RestTransforms), RestGeometryCollection.Parent, CurrentLocalTransforms);
		}

		const TManagedArray<FTransform>* MassToLocal = RestGeometryCollection.FindAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);
		if (MassToLocal && CurrentLocalTransforms.Num() == MassToLocal->Num())
		{
			for (int32 TransformIndex = 0; TransformIndex < CurrentLocalTransforms.Num(); TransformIndex++)
			{
				CurrentLocalTransforms[TransformIndex] = (*MassToLocal)[TransformIndex] * CurrentLocalTransforms[TransformIndex];
			}
		}
	}
	return CurrentLocalTransforms;
}

void UGeometryCollectionComponent::SetLocalRestTransforms(const TArray<FTransform>& NewTransforms, bool bOnlyLeaves)
{
	if (RestCollection && RestCollection->GetGeometryCollection())
	{
		// get the current transform and will use this array to update
		TArray<FTransform> LocalTransforms = GetInitialLocalRestTransforms();
		if (LocalTransforms.Num() == NewTransforms.Num() && NewTransforms.Num() > 0)
		{
			FGeometryCollection& GeometryCollection = *RestCollection->GetGeometryCollection();
			const TManagedArray<FTransform>* MassToLocal = GeometryCollection.FindAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);
			const int32 NumTransforms = GeometryCollection.Parent.Num();

			TArray<int32> TransformsToProcess;
			TransformsToProcess.Reserve(NumTransforms);
			TransformsToProcess.Emplace(RestCollection->GetRootIndex());
			for (int32 ProcessIndex = 0; ProcessIndex < TransformsToProcess.Num(); ProcessIndex++)
			{
				const int32 TransformIndex = TransformsToProcess[ProcessIndex];
				const FTransform InvMassToLocal = MassToLocal ? (*MassToLocal)[TransformIndex].Inverse() : FTransform::Identity;

				const bool bIsLeaf = (GeometryCollection.SimulationType[TransformIndex] == FGeometryCollection::ESimulationTypes::FST_Rigid);
				const bool bNeedUpdate = !bOnlyLeaves || bIsLeaf;
				if (bNeedUpdate)
				{
					LocalTransforms[TransformIndex] = InvMassToLocal * NewTransforms[TransformIndex];

					// because we update top-down, the parent up-to-date transform is stored in LocalTransforms
					const int32 ParentTransformIndex = GeometryCollection.Parent[TransformIndex];
					if (ParentTransformIndex != INDEX_NONE)
					{
						LocalTransforms[TransformIndex] = LocalTransforms[TransformIndex].GetRelativeTransform(LocalTransforms[ParentTransformIndex]);
					}
				}
				else
				{
					LocalTransforms[TransformIndex] = InvMassToLocal * LocalTransforms[TransformIndex];
				}

				for (const int32 ChildTransformIndex : GeometryCollection.Children[TransformIndex])
				{
					TransformsToProcess.Emplace(ChildTransformIndex);
				}
			}

			// send the transforms for update
			SetRestState(MoveTemp(LocalTransforms));
		}
	}
}

TArray<FTransform> UGeometryCollectionComponent::GetCurrentTransforms()
{
	TArray<FTransform> ReturnedTransforms;

	const TArray<FTransform3f>& AllTransforms = ComponentSpaceTransforms.RequestAllTransforms();
	ReturnedTransforms.Reserve(AllTransforms.Num());

	const TManagedArray<FTransform>* MassToLocal = nullptr;
	if (RestCollection && RestCollection->GetGeometryCollection())
	{
		MassToLocal = RestCollection->GetGeometryCollection()->FindAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);
	}

	for (int32 TransformIndex = 0; TransformIndex < AllTransforms.Num(); ++TransformIndex)
	{
		const FTransform MassToLocalTransform = (MassToLocal && MassToLocal->IsValidIndex(TransformIndex)) ? (*MassToLocal)[TransformIndex] : FTransform::Identity;
		ReturnedTransforms.Emplace(MassToLocalTransform * FTransform(AllTransforms[TransformIndex]));
	}

	return ReturnedTransforms;
}

TArray<FTransform> UGeometryCollectionComponent::GetCurrentTransformsFromIndices(const TArray<int32>& TransformIndices)
{
	TArray<FTransform> ReturnedTransforms;
	ReturnedTransforms.Reserve(TransformIndices.Num());

	const TArray<FTransform3f>& AllTransforms = ComponentSpaceTransforms.RequestAllTransforms();

	const TManagedArray<FTransform>* MassToLocal = nullptr;
	if (RestCollection && RestCollection->GetGeometryCollection())
	{
		MassToLocal = RestCollection->GetGeometryCollection()->FindAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);
	}

	for (const int32 TransformIndex : TransformIndices)
	{
		const FTransform MassToLocalTransform = (MassToLocal && MassToLocal->IsValidIndex(TransformIndex)) ? (*MassToLocal)[TransformIndex] : FTransform::Identity;
		const FTransform Transform = (AllTransforms.IsValidIndex(TransformIndex)) ? FTransform(AllTransforms[TransformIndex]) : FTransform::Identity;
		ReturnedTransforms.Emplace(MassToLocalTransform * Transform);

	}

	return ReturnedTransforms;
}

TArray<FMatrix> UGeometryCollectionComponent::ComputeGlobalMatricesFromComponentSpaceTransforms() const
{
	TArray<FMatrix> ComponentSpaceMatrices;
	ComponentSpaceMatrices.SetNumUninitialized(ComponentSpaceTransforms.Num());

	const TArray<FTransform3f>& CompSpaceTransforms = ComponentSpaceTransforms.RequestAllTransforms();
	for (int32 TransformIndex = 0; TransformIndex < CompSpaceTransforms.Num(); TransformIndex++)
	{
		ComponentSpaceMatrices[TransformIndex] = FTransform(CompSpaceTransforms[TransformIndex]).ToMatrixWithScale();
	}
	return ComponentSpaceMatrices;
}

void UGeometryCollectionComponent::GetMassAndExtents(int32 ItemIndex, float& OutMass, FBox& OutExtents)
{
	using FGeometryCollectionPtr = const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>;

	static FName BoundingBoxAttributeName = "BoundingBox";
	static FName MassAttributeName = "Mass";

	OutMass = 0.0f;
	OutExtents = FBox(EForceInit::ForceInitToZero);

	if (RestCollection && RestCollection->GetGeometryCollection())
	{
		const FGeometryCollection& Collection = *RestCollection->GetGeometryCollection();
		if (const TManagedArray<float>* CollectionMass = Collection.FindAttribute<float>(MassAttributeName, FTransformCollection::TransformGroup))
		{
			const TManagedArray<FBox>* TransformBoundingBoxes = Collection.FindAttribute<FBox>(BoundingBoxAttributeName, FTransformCollection::TransformGroup);
			const TManagedArray<FBox>* GeoBoundingBoxes = Collection.FindAttribute<FBox>(BoundingBoxAttributeName, FGeometryCollection::GeometryGroup);

			const FGeometryCollectionItemIndex GCItemIndex = FGeometryCollectionItemIndex::CreateFromExistingItemIndex(ItemIndex);
			if (!ensure(CollectionMass->IsValidIndex(GCItemIndex.GetTransformIndex())))
			{
				return;
			}

			if (GCItemIndex.IsInternalCluster())
			{
				const TArray<int32>* Children = PhysicsProxy->FindInternalClusterChildrenTransformIndices_External(GCItemIndex);
				if (Children)
				{
					for (const int32 ChildTramsformIndex : *Children)
					{
						OutMass += (*CollectionMass)[ChildTramsformIndex];
						if (TransformBoundingBoxes)
						{
							OutExtents += (*TransformBoundingBoxes)[ChildTramsformIndex];
						}
						else if (GeoBoundingBoxes)
						{
							const int32 GeometryIndex = Collection.TransformToGeometryIndex[ChildTramsformIndex];
							if (GeoBoundingBoxes->IsValidIndex(GeometryIndex))
							{
								OutExtents += (*GeoBoundingBoxes)[GeometryIndex];
							}
						}
					}
				}
			}
			else
			{
				const int32 TransformIndex = GCItemIndex.GetTransformIndex();
				OutMass = (*CollectionMass)[TransformIndex];
				if (TransformBoundingBoxes)
				{
					OutExtents = (*TransformBoundingBoxes)[TransformIndex];
				}
				else if (GeoBoundingBoxes)
				{
					const int32 GeometryIndex = Collection.TransformToGeometryIndex[TransformIndex];
					if (GeoBoundingBoxes->IsValidIndex(GeometryIndex))
					{
						OutExtents = (*GeoBoundingBoxes)[GeometryIndex];
					}
				}
			}
		}
	}
}

float UGeometryCollectionComponent::ComputeMassScaleRelativeToAsset() const
{
	// note this only acount for material override density , in the future we could certainly take in account the scale of the component
	float MassScaleMultiplier = 1.0f;
	UPhysicalMaterial* EnginePhysicalMaterial = GetPhysicalMaterial();
	if (ensure(EnginePhysicalMaterial))
	{
		if (RestCollection && bDensityFromPhysicsMaterial)
		{
			bool bMassAsDensity = false;
			const float AssetMassOrDensity = RestCollection->GetMassOrDensity(bMassAsDensity);
			if (ensureMsgf(bMassAsDensity, TEXT("Need a density to be able to compute the mass scale multiplier, this may result in incorrect mass properties")))
			{
				if (ensureMsgf(AssetMassOrDensity > SMALL_NUMBER, TEXT("Asset density is set to a too small number, ignoring it for mass adjustment")))
				{
					const float OverrideMaterialDensity = Chaos::GCm3ToKgCm3(EnginePhysicalMaterial->Density);
					MassScaleMultiplier = OverrideMaterialDensity / AssetMassOrDensity;
				}
			}
		}
	}
	return MassScaleMultiplier;
}

float UGeometryCollectionComponent::GetMass() const
{
	float OutMass{ 0 };

	if (RestCollection && RestCollection->GetGeometryCollection())
	{
		const FGeometryCollection& Collection = *RestCollection->GetGeometryCollection();
		if (const TManagedArray<float>* CollectionMass = Collection.FindAttribute<float>(TEXT("Mass"), FTransformCollection::TransformGroup))
		{
			const int32 RootIndex = RestCollection->GetRootIndex();
			if (CollectionMass->IsValidIndex(RootIndex))
			{
				OutMass = (*CollectionMass)[RootIndex];
			}
		}
	}
	// need to multiply by the instance specific mass scale is any
	return OutMass * ComputeMassScaleRelativeToAsset();
}

float UGeometryCollectionComponent::CalculateMass(FName BoneName)
{
	float OutMass{ 0 };

	if (RestCollection && RestCollection->GetGeometryCollection())
	{
		const FGeometryCollection& Collection = *RestCollection->GetGeometryCollection();
		if (const TManagedArray<float>* CollectionMass = Collection.FindAttribute<float>(TEXT("Mass"), FTransformCollection::TransformGroup))
		{
			const int32 BoneIndex = (BoneName == NAME_None)? RestCollection->GetRootIndex(): Collection.BoneName.Find(BoneName.ToString());
			if (CollectionMass->IsValidIndex(BoneIndex))
			{
				OutMass = (*CollectionMass)[BoneIndex];
			}
		}
	}
	// need to multiply by the instance specific mass scale is any
	return OutMass * ComputeMassScaleRelativeToAsset();
}

bool UGeometryCollectionComponent::CalculateInnerSphere(int32 TransformIndex, UE::Math::TSphere<double>& SphereOut) const
{
	// Approximates the inscribed sphere. Returns false if no such sphere exists, if for instance the index is to an embedded geometry. 

	const TManagedArray<int32>& TransformToGeometryIndex = RestCollection->GetGeometryCollection()->TransformToGeometryIndex;
	const TManagedArray<TSet<int32>>& Children = RestCollection->GetGeometryCollection()->Children;
	const TManagedArray<FTransform>& MassToLocal = RestCollection->GetGeometryCollection()->GetAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);

	TManagedArrayAccessor<Chaos::FRealSingle> InnerRadiusAttribute(*RestCollection->GetGeometryCollection(), "InnerRadius", FGeometryCollection::GeometryGroup);
	if (InnerRadiusAttribute.IsValid() && InnerRadiusAttribute.IsValidIndex(TransformIndex))
	{
		if (RestCollection->GetGeometryCollection()->IsRigid(TransformIndex))
		{
			// Sphere in component space, centered on body's COM.
			FVector COM = MassToLocal[TransformIndex].GetLocation();
			SphereOut = UE::Math::TSphere<double>(COM, InnerRadiusAttribute[TransformToGeometryIndex[TransformIndex]]);
			return true;
		}
		else if (RestCollection->GetGeometryCollection()->IsClustered(TransformIndex))
		{
			// Recursively accumulate the cluster's child spheres. 
			bool bSphereFound = false;
			for (int32 ChildIndex : Children[TransformIndex])
			{
				UE::Math::TSphere<double> LocalSphere;
				if (CalculateInnerSphere(ChildIndex, LocalSphere))
				{
					if (!bSphereFound)
					{
						bSphereFound = true;
						SphereOut = LocalSphere;
					}
					else
					{
						SphereOut += LocalSphere;
					}
				}
			}
			return bSphereFound;
		}
	}
	// Likely an embedded geometry or missing inner radius attribute , which doesn't count towards volume.
	return false;
}

void UGeometryCollectionComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FFortniteValkyrieBranchObjectVersion::GUID);

#if WITH_EDITOR
	// Fix rest transforms
	// Prior to this change, it is possible for the RestTransforms to be set with the original rest collection transforms or have outdated rest transforms
	// this is a waste of memory and performance cost with serialization at runtime since FTransform can not be bulk serialized
	// so if they are no longer matching the rest collection let's clear them 
	if (Ar.IsLoading() && RestTransforms.Num() > 0)
	{
		if (Ar.CustomVer(FFortniteValkyrieBranchObjectVersion::GUID) < FFortniteValkyrieBranchObjectVersion::FixRestTransformsInGeometryCollectionComponent)
		{
			bool bNeedToRestRestTransform = false;
			bool bIsRestCollectionValid = (RestCollection && RestCollection->GetGeometryCollection());
			if (!bIsRestCollectionValid || RestTransforms.Num() != RestCollection->GetGeometryCollection()->Transform.Num())
			{
				bNeedToRestRestTransform = true;
			}
			else
			{
				const TArray<FTransform3f>& RestCollectionTransforms = RestCollection->GetGeometryCollection()->Transform.GetConstArray();
				for (int32 Index = 0; Index < RestTransforms.Num(); Index++)
				{
					// Do a float comparison otherwise, changing from float to double coulld cause transforms to be different
					if (!FTransform3f(RestTransforms[Index]).Equals(RestCollectionTransforms[Index]))
					{
						bNeedToRestRestTransform = true;
						break;
					}
				}
			}
			if (bNeedToRestRestTransform)
			{
				RestTransforms.Reset();
			}
		}
	}
#endif
}

void UGeometryCollectionComponent::PostLoad()
{
	Super::PostLoad();

	// If there is a rest collection and no custom renderer then precache PSOs required to render the mesh
	if (RestCollection && !IsCustomRendererAvailable())
	{
		PrecachePSOs();
	}

	//
	// The UGeometryCollectionComponent::PhysicalMaterial_DEPRECATED needs
	// to be transferred to the BodyInstance simple material. Going forward
	// the deprecated value will not be saved.
	//
	if (PhysicalMaterialOverride_DEPRECATED)
	{
		BodyInstance.SetPhysMaterialOverride(PhysicalMaterialOverride_DEPRECATED.Get());
		PhysicalMaterialOverride_DEPRECATED = nullptr;
	}

	// Convert deprecated ISMPool and bAutoAssignISMPool settings to use the a CustomRendererType.
	if ((ISMPool_DEPRECATED || bAutoAssignISMPool_DEPRECATED) && !(bOverrideCustomRenderer || CustomRendererType))
	{
		bOverrideCustomRenderer = true;
		CustomRendererType = UGeometryCollectionISMPoolRenderer::StaticClass();
		ISMPool_DEPRECATED = nullptr;
		bAutoAssignISMPool_DEPRECATED = false;
	}
}

void UGeometryCollectionComponent::CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams)
{
	if (!RestCollection)
	{
		return;
	}

	FPSOPrecacheVertexFactoryDataList VFDataList;
	const FVertexFactoryType* VFType = nullptr;
	if (UseNanite(GMaxRHIShaderPlatform) &&
		RestCollection->EnableNanite &&
		RestCollection->HasNaniteData() &&
		GGeometryCollectionNanite != 0)
	{
		VFDataList.Add(FPSOPrecacheVertexFactoryData(&FNaniteVertexFactory::StaticType));
	}	
	else if (RestCollection->HasMeshData())
	{
		VFDataList.Add(FPSOPrecacheVertexFactoryData(&FGeometryCollectionVertexFactory::StaticType));
	}

	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> RestGeomCollection = RestCollection->GetGeometryCollection();
	const TManagedArray<FGeometryCollectionSection>& SectionsArray = RestGeomCollection->Sections;
	for (int32 SectionIndex = 0; SectionIndex < SectionsArray.Num(); ++SectionIndex)
	{
		const FGeometryCollectionSection& MeshSection = SectionsArray[SectionIndex];
		const bool bValidMeshSection = MeshSection.MaterialID != INDEX_NONE;
		UMaterialInterface* MaterialInterface = bValidMeshSection ? GetMaterial(MeshSection.MaterialID) : UMaterial::GetDefaultMaterial(MD_Surface);

		FMaterialInterfacePSOPrecacheParams& ComponentParams = OutParams[OutParams.AddDefaulted()];
		ComponentParams.Priority = EPSOPrecachePriority::Medium;
		ComponentParams.MaterialInterface = MaterialInterface;
		ComponentParams.VertexFactoryDataList = VFDataList;
		ComponentParams.PSOPrecacheParams = BasePrecachePSOParams;
	}
}

void UGeometryCollectionComponent::SetPhysMaterialOverride(UPhysicalMaterial* NewPhysMaterial)
{
	Super::SetPhysMaterialOverride(NewPhysMaterial);

	if (PhysicsProxy)
	{
		UPhysicalMaterial* EnginePhysicalMaterial = GetPhysicalMaterial();
		if (ensure(EnginePhysicalMaterial))
		{
			PhysicsProxy->SetPhysicsMaterial_External(EnginePhysicalMaterial->GetPhysicsMaterial());
		}
	}
}

void UGeometryCollectionComponent::SetLinearDamping(float InDamping)
{
	Super::SetLinearDamping(InDamping);
	if (PhysicsProxy)
	{
		PhysicsProxy->SetLinearDamping_External(InDamping);
	}
}

void UGeometryCollectionComponent::SetLinearDampingPerTransform(const TArray<int32>& TransformIndices, float InDamping)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->SetLinearDampingPerTransform_External(TransformIndices, InDamping);
	}
}

void UGeometryCollectionComponent::SetAngularDamping(float InDamping)
{
	Super::SetAngularDamping(InDamping);
	if (PhysicsProxy)
	{
		PhysicsProxy->SetAngularDamping_External(InDamping);
	}
}

void UGeometryCollectionComponent::SetAngularDampingPerTransform(const TArray<int32>& TransformIndices, float InDamping)
{
	if (PhysicsProxy)
	{
		PhysicsProxy->SetAngularDampingPerTransform_External(TransformIndices, InDamping);
	}
}

void UGeometryCollectionComponent::FlushNetDormancyIfNeeded() const
{
	if (GetDesiredNetAwakeningMode() != ENetAwakeningMode::FlushNetDormancy)
	{
		return;
	}

	if (AActor* Owner = GetOwner())
	{

		Owner->FlushNetDormancy();
	}
}

UGeometryCollectionComponent::ENetAwakeningMode UGeometryCollectionComponent::GetDesiredNetAwakeningMode() const
{
	constexpr int32 MinNetAwakeningMode = 0;
	constexpr int32 MaxNetAwakeningMode = 1;
	ENetAwakeningMode Mode;
	if (GeometryCollectionNetAwakeningMode < MinNetAwakeningMode || GeometryCollectionNetAwakeningMode > MaxNetAwakeningMode)
	{
		ensureMsgf(false, TEXT("Invalid NetAwakening mode configured, falling back to the default mode"));
		Mode = ENetAwakeningMode::ForceDormancyAwake;
	}
	else
	{
		Mode = static_cast<ENetAwakeningMode>(GeometryCollectionNetAwakeningMode);
	}

	return Mode;
}

Chaos::FPhysicsObject* UGeometryCollectionComponent::GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const
{
	if (!PhysicsProxy)
	{
		return nullptr;
	}
	return PhysicsProxy->GetPhysicsObjectByIndex(Id);
}

Chaos::FPhysicsObject* UGeometryCollectionComponent::GetPhysicsObjectByName(const FName& Name) const
{
	if (!RestCollection)
	{
		return nullptr;
	}

	if (Name == NAME_None)
	{
		// Special case where it's more convenient for us to return the root bone instead.
		TArray<int32> Roots;
		FGeometryCollectionClusteringUtility::GetRootBones(RestCollection->GetGeometryCollection().Get(), Roots);

		if (Roots.IsEmpty())
		{
			return nullptr;
		}

		// More convenient just to assume there's one root for this special case here.
		return GetPhysicsObjectById(Roots[0]);
	}

	const int32 Index = RestCollection->GetGeometryCollection()->BoneName.Find(Name.ToString());
	return GetPhysicsObjectById(Index);
}

TArray<Chaos::FPhysicsObject*> UGeometryCollectionComponent::GetAllPhysicsObjects() const
{
	if (!PhysicsProxy)
	{
		return {};
	}
	TArray<Chaos::FPhysicsObject*> Objects;
	Objects.Reserve(PhysicsProxy->GetNumTransforms());
	
	for (int32 Index = 0; Index < PhysicsProxy->GetNumTransforms(); ++Index)
	{
		Objects.Add(GetPhysicsObjectById(Index));
	}
	return Objects;
}

Chaos::FPhysicsObjectId UGeometryCollectionComponent::GetIdFromGTParticle(Chaos::FGeometryParticle* Particle) const
{
	if (!PhysicsProxy || !Particle)
	{
		return INDEX_NONE;
	}
	FGeometryCollectionItemIndex Index = PhysicsProxy->GetItemIndexFromGTParticleNoInternalCluster_External(Particle->CastToRigidParticle());
	if (Index.IsValid())
	{
		return Index.GetTransformIndex();
	}
	else
	{
		return INDEX_NONE;
	}
}

void UGeometryCollectionComponent::SetEnableDamageFromCollision(bool bValue)
{
	bEnableDamageFromCollision = bValue;
	if (PhysicsProxy)
	{
		PhysicsProxy->SetEnableDamageFromCollision_External(bValue);
	}
}

void UGeometryCollectionComponent::OnComponentCollisionSettingsChanged(bool bUpdateOverlaps)
{
	Super::OnComponentCollisionSettingsChanged(bUpdateOverlaps);

	BuildInitialFilterData();
	LoadCollisionProfiles();
}

bool UGeometryCollectionComponent::CanBeUsedInPhysicsReplication(const FName BoneName) const
{
	return false;
}

const FTransform& UGeometryCollectionComponent::GetPreviousComponentToWorld() const
{
	if (!PhysicsProxy)
	{
		return FTransform::Identity;
	}

	return PhysicsProxy->GetPreviousWorldTransform_External();
}

#if WITH_EDITOR

bool UGeometryCollectionComponent::IsHLODRelevant() const
{
	if (!CanBeHLODRelevant(this))
	{
		return false;
	}

	if (RestCollection == nullptr || !IsCustomRendererAvailable())
	{
		return false;
	}

	if (!IsVisible())
	{
		return false;
	}

#if WITH_EDITORONLY_DATA
	if (!bEnableAutoLODGeneration)
	{
		return false;
	}
#endif

	if (GetHLODProxyComponents().IsEmpty())
	{
		return false;
	}

	return true;
}

TArray<UActorComponent*> UGeometryCollectionComponent::GetHLODProxyComponents() const
{
	TArray<UActorComponent*> HLODProxyComponents;

	ForEachObjectWithOuter(this, [&HLODProxyComponents](UObject* InObject)
	{
		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(InObject))
		{
			HLODProxyComponents.Add(StaticMeshComponent);
		}
	});

	return HLODProxyComponents;
}

#endif // #if WITH_EDITOR

void UGeometryCollectionComponent::RebaseDynamicCollectionTransformsOnNewWorldTransform()
{
	if (!PhysicsProxy)
	{
		return;
	}

	PhysicsProxy->RebaseAllGameThreadCollectionTransformsOnNewWorldTransform_External();
	ComponentSpaceTransforms.MarkDirty();
}
