// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Vector.isph"
#include "Math/Matrix.isph"

#define INDEX_NONE -1

struct FBox
{
	FVector Min;
	FVector Max;
	unsigned int8 IsValid;
};

inline static FBox BoxDefaultInit()
{
	FBox Box;
	Box.Min = Box.Max = ZeroVector;
	Box.IsValid = 0;
	return Box;
}

inline static FBox operator+(const FBox &A, const FBox &B)
{
	if(A.IsValid && B.IsValid)
	{
		FBox Box;
		Box.Min = VectorMin(A.Min, B.Min);
		Box.Max = VectorMax(A.Max, B.Max);
		Box.IsValid = A.IsValid;
		return Box;
	}
	else if(B.IsValid)
	{
		return B;
	}
	else
	{
		return A;
	}
}

inline static uniform FBox operator+(const uniform FBox &A, const uniform FBox &B)
{
	if(A.IsValid && B.IsValid)
	{
		uniform FBox Box;
		Box.Min = VectorMin(A.Min, B.Min);
		Box.Max = VectorMax(A.Max, B.Max);
		Box.IsValid = A.IsValid;
		return Box;
	}
	else if(B.IsValid)
	{
		return B;
	}
	else
	{
		return A;
	}
}

inline static FBox BoxTransformByMatrix(const FBox &Box, const FMatrix &M)
{
	FBox NewBox;

	const FVector4 m0 = SetVector4(M.M[0], M.M[1], M.M[2], M.M[3]);
	const FVector4 m1 = SetVector4(M.M[4], M.M[5], M.M[6], M.M[7]);
	const FVector4 m2 = SetVector4(M.M[8], M.M[9], M.M[10], M.M[11]);
	const FVector4 m3 = SetVector4(M.M[12], M.M[13], M.M[14], M.M[15]);

	const FVector Origin = (Box.Max + Box.Min) * HalfVector;
	const FVector Extent = (Box.Max - Box.Min) * HalfVector;

	FVector4 NewOrigin = SetVector4(Origin.V[0], Origin.V[0], Origin.V[0], Origin.V[0]) * m0;
	NewOrigin = VectorMultiplyAdd(SetVector4(Origin.V[1], Origin.V[1], Origin.V[1], Origin.V[1]), m1, NewOrigin);
	NewOrigin = VectorMultiplyAdd(SetVector4(Origin.V[2], Origin.V[2], Origin.V[2], Origin.V[2]), m2, NewOrigin);
	NewOrigin = NewOrigin + m3;

	FVector4 NewExtent = VectorAbs(SetVector4(Extent.V[0], Extent.V[0], Extent.V[0], Extent.V[0]) * m0);
	NewExtent = NewExtent + VectorAbs(SetVector4(Extent.V[1], Extent.V[1], Extent.V[1], Extent.V[1]) * m1);
	NewExtent = NewExtent + VectorAbs(SetVector4(Extent.V[2], Extent.V[2], Extent.V[2], Extent.V[2]) * m2);

	const FVector4 NewVecMin = NewOrigin - NewExtent;
	const FVector4 NewVecMax = NewOrigin + NewExtent;
	
	NewBox.Min = SetVector(NewVecMin.V[0], NewVecMin.V[1], NewVecMin.V[2]);
	NewBox.Max = SetVector(NewVecMax.V[0], NewVecMax.V[1], NewVecMax.V[2]);
	NewBox.IsValid = 1;

	return NewBox;
}

export void BoxCalcBoundsFromGeometryGroup(
				const uniform int TransformToGeometryIndex[],
				const uniform int TransformIndices[],
				const uniform FMatrix GlobalMatrices[],
				const uniform FBox BoundingBoxes[],
				const uniform FMatrix &LocalToWorldWithScale,
				uniform FBox &BoundingBox,
				const uniform int NumBoxes)
{
	FBox BoundingBoxSum = BoxDefaultInit();

	foreach(BoxIdx = 0 ... NumBoxes)
	{
		const int TransformIndex = TransformIndices[BoxIdx];

		#pragma ignore warning(perf)
		const FBox B = BoundingBoxes[BoxIdx];

		#pragma ignore warning(perf)
		if (TransformToGeometryIndex[TransformIndex] != INDEX_NONE && B.IsValid)
		{
			#pragma ignore warning(perf)
			const FMatrix M = GlobalMatrices[TransformIndex];
			BoundingBoxSum = BoundingBoxSum + BoxTransformByMatrix(B, M * LocalToWorldWithScale);
		}
	}

	// Custom reduce for large arrays
	uniform FBox ReducedBox[programCount];
	#pragma ignore warning(perf)
	ReducedBox[programIndex] = BoundingBoxSum;

	if (programIndex < NumBoxes)
	{
		foreach_active(j)
		{
			unmasked
			{
				BoundingBox = BoundingBox + ReducedBox[j];
			}
		}
	}
}

export void BoxCalcBoundsFromTransformGroup(
				const uniform int TransformToGeometryIndex[],
				const uniform FMatrix GlobalMatrices[],
				const uniform FBox BoundingBoxes[],
				const uniform FMatrix &LocalToWorldWithScale,
				uniform FBox &BoundingBox,
				const uniform int NumBoxes)
{
	FBox BoundingBoxSum = BoxDefaultInit();

	foreach(TransformIndex = 0 ... NumBoxes)
	{
		#pragma ignore warning(perf)
		const FBox B = BoundingBoxes[TransformIndex];

		#pragma ignore warning(perf)
		if (TransformToGeometryIndex[TransformIndex] != INDEX_NONE && B.IsValid)
		{
			#pragma ignore warning(perf)
			const FMatrix M = GlobalMatrices[TransformIndex];
			BoundingBoxSum = BoundingBoxSum + BoxTransformByMatrix(B, M * LocalToWorldWithScale);
		}
	}

	// Custom reduce for large arrays
	uniform FBox ReducedBox[programCount];
	#pragma ignore warning(perf)
	ReducedBox[programIndex] = BoundingBoxSum;

	if (programIndex < NumBoxes)
	{
		foreach_active(j)
		{
			unmasked
			{
				BoundingBox = BoundingBox + ReducedBox[j];
			}
		}
	}
}