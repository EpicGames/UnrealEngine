// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionComponentPlugin.h"

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"




class FGeometryCollectionComponentPlugin : public IGeometryCollectionComponentPlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FGeometryCollectionComponentPlugin, GeometryCollectionEngine )



void FGeometryCollectionComponentPlugin::StartupModule()
{
	
}


void FGeometryCollectionComponentPlugin::ShutdownModule()
{
	
}



