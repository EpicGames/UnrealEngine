// Copyright Epic Games, Inc. All Rights Reserved.


#include "GeometryCollection/GeometryCollectionCreationParameters.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionEngineUtility.h"

FGeometryCollectionCreationParameters::FGeometryCollectionCreationParameters(FGeometryCollection& GeometryCollectioIn, bool ReCalculateNormalsIn /*= false*/, bool ReCalculateTangetsIn /*= false*/)
	: ReCalculateNormals(ReCalculateNormalsIn)
	, ReCalculateTangents(ReCalculateTangetsIn),
	GeometryCollection(GeometryCollectioIn)
{
}

FGeometryCollectionCreationParameters::~FGeometryCollectionCreationParameters()
{
	if (ReCalculateNormals)
	{
		GeometryCollectionEngineUtility::ComputeNormals(&GeometryCollection);
	}

	if (ReCalculateTangents)
	{
		GeometryCollectionEngineUtility::ComputeTangents(&GeometryCollection);
	}
}

