// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionDebugDraw.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollectionProxyData.h"

#include "PrimitiveDrawingUtils.h"
#include "PhysicsEngine/SphereElem.h"
#include "PhysicsEngine/SphylElem.h"
#include "PhysicsEngine/BoxElem.h"
#include "PhysicsEngine/ConvexElem.h"
#include "PhysicsEngine/AggregateGeom.h"

#include "Chaos/Sphere.h"
#include "Chaos/Capsule.h"
#include "Chaos/Box.h"
#include "Chaos/Convex.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Chaos/HeightField.h"

namespace GeometryCollectionDebugDraw
{
	void AddSphere(FKAggregateGeom& AggGeom, const Chaos::FImplicitSphere3& Sphere, const Chaos::FRigidTransform3& Transform)
	{
		FKSphereElem& SphereElem = AggGeom.SphereElems.AddDefaulted_GetRef();
		SphereElem.Center = Transform.TransformPosition(FVector3d(Sphere.GetCenterf()));
		SphereElem.Radius = (Transform.GetScale3D().Z * Sphere.GetRadiusf());
	}

	void AddCapsule(FKAggregateGeom& AggGeom, const Chaos::FImplicitCapsule3& Capsule, const Chaos::FRigidTransform3& Transform)
	{
		FKSphylElem& CapsuleElem = AggGeom.SphylElems.AddDefaulted_GetRef();
		CapsuleElem.Center = Transform.TransformPosition(FVector3d(Capsule.GetCenterf()));
		CapsuleElem.Rotation = (Transform.GetRotation() * FRotationMatrix::MakeFromZ(Capsule.GetAxis())).Rotator();
		CapsuleElem.Length = (Transform.GetScale3D().Z * Capsule.GetHeightf());
		CapsuleElem.Radius = (Transform.GetScale3D().X * Capsule.GetRadiusf());
	}

	void AddBox(FKAggregateGeom& AggGeom, const Chaos::FImplicitBox3& Box, const Chaos::FRigidTransform3& Transform)
	{
		FKBoxElem& BoxElem = AggGeom.BoxElems.AddDefaulted_GetRef();
		BoxElem.Center = Transform.TransformPosition(Box.Center());
		BoxElem.Rotation = Transform.Rotator();
		BoxElem.X = Transform.GetScale3D().X * Box.Extents().X;
		BoxElem.Y = Transform.GetScale3D().Y * Box.Extents().Y;
		BoxElem.Z = Transform.GetScale3D().Z * Box.Extents().Z;
	}

	void AddConvex(FKAggregateGeom& AggGeom, const Chaos::FImplicitConvex3& Convex, const Chaos::FRigidTransform3& Transform)
	{
		FKConvexElem& ConvexElem = AggGeom.ConvexElems.AddDefaulted_GetRef();
		ConvexElem.VertexData.Reserve(Convex.GetVertices().Num());
		for (const FVector3f& Vtx : Convex.GetVertices())
		{
			ConvexElem.VertexData.Emplace(Transform.TransformPosition(FVector(Vtx)));
		}
		for (int32 PlaneIndex = 0; PlaneIndex < Convex.NumPlanes(); PlaneIndex++)
		{
			const int32 NumVertices = Convex.NumPlaneVertices(PlaneIndex);
			if (NumVertices >= 3)
			{
				const int32 NumTriangles = (NumVertices - 2);
				ConvexElem.IndexData.Reserve(ConvexElem.IndexData.Num() + (NumTriangles * 3));
				for (int32 VtxIndex = 2; VtxIndex < NumVertices; VtxIndex++)
				{
					ConvexElem.IndexData.Emplace(Convex.GetPlaneVertex(PlaneIndex, 0));
					ConvexElem.IndexData.Emplace(Convex.GetPlaneVertex(PlaneIndex, VtxIndex-1));
					ConvexElem.IndexData.Emplace(Convex.GetPlaneVertex(PlaneIndex, VtxIndex));
				}
			}
		}
	}

	void AddImplicitObject(FKAggregateGeom& AggGeom, const Chaos::FImplicitObject* Implicit, const Chaos::FRigidTransform3& Transform)
	{
		using namespace Chaos;

		if (const FImplicitSphere3* Sphere = Implicit->AsA<FImplicitSphere3>())
		{
			AddSphere(AggGeom, *Sphere, Transform);
		}
		else if (const FImplicitCapsule3* Capsule = Implicit->AsA<FImplicitCapsule3>())
		{
			AddCapsule(AggGeom, *Capsule, Transform);
		}
		else if (const FImplicitBox3* Box = Implicit->AsA<FImplicitBox3>())
		{
			AddBox(AggGeom, *Box, Transform);
		}
		else if (const FImplicitConvex3* Convex = Implicit->AsA<FImplicitConvex3>())
		{
			AddConvex(AggGeom, *Convex, Transform);
		}
		else if (const FTriangleMeshImplicitObject* TriMesh = Implicit->AsA<FTriangleMeshImplicitObject>())
		{
			ensure(false); // unsupported for geometry collection
		}
		else if (const FHeightField* HeightField = Implicit->AsA<FHeightField>())
		{
			ensure(false); // unsupported for geometry collection
		}
		else if (const FImplicitObjectInstanced* Instanced = Implicit->AsA<FImplicitObjectInstanced>())
		{
			AddImplicitObject(AggGeom, Instanced->GetInnerObject().Get(), Transform);
		}
		else if (const FImplicitObjectScaled* Scaled = Implicit->AsA<FImplicitObjectScaled>())
		{
			const FRigidTransform3 ScaledTransform = FRigidTransform3(Transform.GetTranslation(), Transform.GetRotation(), Transform.GetScale3D() * Scaled->GetScale());
			AddImplicitObject(AggGeom, Scaled->GetInnerObject().Get(), ScaledTransform);
		}
		else if (const FImplicitObjectUnion* Union = Implicit->AsA<FImplicitObjectUnion>())
		{
			ensure(false);
		}
	}


	void Draw(const FGeometryCollection& Collection, const FTransform& CollectionWorldTransform, FMeshElementCollector& MeshCollector, int32 ViewIndex, const FMaterialRenderProxy* MaterialProxy, const FColor& Color, bool bDrawSolid)
	{
		const FName MassToLocalAttributeName = "MassToLocal";

		const TManagedArray<FTransform>* MassToLocalPtr = Collection.FindAttribute<FTransform>(MassToLocalAttributeName, FTransformCollection::TransformGroup);
		const TManagedArray<Chaos::FImplicitObjectPtr>* ImplicitsPtr = Collection.FindAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
		if (MassToLocalPtr && ImplicitsPtr)
		{
			const TManagedArray<FTransform>& MassToLocal = *MassToLocalPtr;
			const TManagedArray<Chaos::FImplicitObjectPtr>& Implicits = *ImplicitsPtr;

			// let's take advantage of the existing method for rendering Aggregate geometry 
			FKAggregateGeom AggGeom;

			const GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(Collection);
			const Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(Collection);
			TArray<int32> TransformIndices = TransformFacade.GetRootIndices();

			for (int32 Index = 0; Index < TransformIndices.Num(); Index++)
			{
				const int32 TransformIndex = TransformIndices[Index];
				if (const Chaos::FImplicitObjectPtr ImplicitPtr = Implicits[TransformIndex])
				{
					// TODO : use the root bone transform * component transform
					const Chaos::FRigidTransform3 CollectionSpaceTransform = TransformFacade.ComputeCollectionSpaceTransform(TransformIndex);
					const Chaos::FRigidTransform3 CollectionSpaceParticleTransform = MassToLocal[TransformIndex] * CollectionSpaceTransform;

					ImplicitPtr->VisitLeafObjects(
						[&](const Chaos::FImplicitObject* LeafImplicitObject, const Chaos::FRigidTransform3& LeafRelativeTransform, const int32 UnusedRootObjectIndex, const int32 UnusedObjectIndex, const int32 UnusedLeafObjectIndex)
						{
							const Chaos::FRigidTransform3 LeafTransform = LeafRelativeTransform * CollectionSpaceParticleTransform;
							AddImplicitObject(AggGeom, LeafImplicitObject, LeafTransform);
						});
				}
				else
				{
					// no implicit , so we need to use the children ones 
					TransformIndices.Append(HierarchyFacade.GetChildrenAsArray(TransformIndex));
				}

			}

			// collect the rendering data through the collector 
			AggGeom.GetAggGeom(CollectionWorldTransform, Color, MaterialProxy, /*bPerHullColor*/false, bDrawSolid, /*bOutputVelocity*/false, ViewIndex, MeshCollector);
		}
	}

	void DrawSolid(const FGeometryCollection& Collection, const FTransform& CollectionWorldTransform, FMeshElementCollector& MeshCollector, int32 ViewIndex, const FMaterialRenderProxy* MaterialProxy)
	{
		Draw(Collection, CollectionWorldTransform, MeshCollector, ViewIndex, MaterialProxy, /*Color*/FColor::White, /*bDrawSolid*/true);
	}

	void DrawWireframe(const FGeometryCollection& Collection, const FTransform& CollectionWorldTransform, FMeshElementCollector& MeshCollector, int32 ViewIndex, const FColor& Color)
	{
		Draw(Collection, CollectionWorldTransform, MeshCollector, ViewIndex, /*MaterialProxy*/nullptr, Color, /*bDrawSolid*/false);
	}

}
