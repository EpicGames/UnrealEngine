// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionDebugDrawActor.h"

#include "Chaos/ChaosSolverActor.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionParticlesData.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionActor.h"

#include "DrawDebugHelpers.h"
#include "Debug/DebugDrawService.h"
#include "Engine/Engine.h"
#include "Engine/Canvas.h"
#include "CanvasItem.h"
#include "HAL/IConsoleManager.h"
#include "EngineUtils.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/BillboardComponent.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "HAL/IConsoleManager.h"
#include "PBDRigidsSolver.h"
#include "PhysicsSolver.h"  // #if TODO_REIMPLEMENT_GET_RIGID_PARTICLES
#include "GeometryCollection/GeometryCollectionDebugDrawComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionDebugDrawActor)

DEFINE_LOG_CATEGORY_STATIC(LogGeometryCollectionDebugDrawActor, Log, All);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FString FGeometryCollectionDebugDrawActorSelectedRigidBody::GetSolverName() const
{
	return !Solver ? FName(NAME_None).ToString() : Solver->GetName();
}

AGeometryCollectionDebugDrawActor::AGeometryCollectionDebugDrawActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryActorTick.bCanEverTick = false;
	PrimaryActorTick.bTickEvenWhenPaused = false;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
