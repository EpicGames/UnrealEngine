// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionDebugDrawComponent.h"

#if GEOMETRYCOLLECTION_DEBUG_DRAW
#include "GeometryCollection/GeometryCollectionRenderLevelSetActor.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionDebugDrawActor.h"
#include "GeometryCollection/GeometryCollection.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#endif  // #if GEOMETRYCOLLECTION_DEBUG_DRAW
#include "HAL/IConsoleManager.h"
#include "PBDRigidsSolver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionDebugDrawComponent)

DEFINE_LOG_CATEGORY_STATIC(LogGeometryCollectionDebugDraw, Log, All);

UGeometryCollectionDebugDrawComponent::UGeometryCollectionDebugDrawComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, GeometryCollectionDebugDrawActor_DEPRECATED(nullptr)
	, GeometryCollectionRenderLevelSetActor(nullptr)
	, GeometryCollectionComponent(nullptr)
{
	bNavigationRelevant = false;
	bTickInEditor = false;
	PrimaryComponentTick.bCanEverTick = false;
}



