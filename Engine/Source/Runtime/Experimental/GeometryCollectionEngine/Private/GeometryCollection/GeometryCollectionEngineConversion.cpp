// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionEngineConversion.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AnimationRuntime.h"
#include "Async/ParallelFor.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/Selection.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Engine/StaticMesh.h"
#include "GameFramework/Actor.h"
#include "GeometryCollection/Facades/CollectionInstancedMeshFacade.h"
#include "GeometryCollection/Facades/CollectionTransformFacade.h"
#include "GeometryCollection/Facades/CollectionTransformSourceFacade.h"
#include "GeometryCollection/Facades/CollectionMeshFacade.h"
#include "GeometryCollection/Facades/CollectionTransformSelectionFacade.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/Facades/CollectionBoundsFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionActor.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionClusteringUtility.h"
#include "GeometryCollection/GeometryCollectionEngineUtility.h"
#include "GeometryCollection/GeometryCollectionUtility.h"
#include "GeometryCollectionProxyData.h"
#include "IndexTypes.h"
#include "Logging/LogMacros.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "MeshDescription.h"
#include "MeshDescriptionBuilder.h"
#include "Misc/ScopedSlowTask.h"
#include "Physics/Experimental/ChaosInterfaceUtils.h"
#include "PhysicsEngine/BodySetup.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Rendering/SkeletalMeshModel.h"
#include "ReferenceSkeleton.h"
#include "SkeletalMeshAttributes.h"
#include "StaticMeshAttributes.h"
#include "StaticMeshOperations.h"
#include "VertexConnectedComponents.h"
#include "Util/ColorConstants.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"

#if WITH_EDITOR
#include "StaticToSkeletalMeshConverter.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(UGeometryCollectionConversionLogging, Log, All);

#define LOCTEXT_NAMESPACE "GeometryCollectionConversion"

struct FUniqueVertex
{
	FVector3f Normal;
	FVector3f Tangent;
	TArray<FVector2f> UVs;

	bool operator==(const FUniqueVertex& Other) const
	{
		if (this->UVs.Num() != Other.UVs.Num())
		{
			return false;
		}
		
		bool bEquality = true;
		bEquality &= (this->Normal == Other.Normal);
		bEquality &= (this->Tangent == Other.Tangent);
		for (int32 UVLayerIdx = 0; UVLayerIdx < UVs.Num(); ++UVLayerIdx)
		{
			bEquality &= (this->UVs[UVLayerIdx] == Other.UVs[UVLayerIdx]);
		}
		
		return bEquality;
	}
};

FORCEINLINE uint32 GetTypeHash(const FUniqueVertex& UniqueVertex)
{
	uint32 VertexHash = GetTypeHash(UniqueVertex.Normal);
	VertexHash = HashCombine(VertexHash, GetTypeHash(UniqueVertex.Tangent));
	for (int32 UVLayerIdx = 0; UVLayerIdx < UniqueVertex.UVs.Num(); ++UVLayerIdx)
	{
		VertexHash = HashCombine(VertexHash, GetTypeHash(UniqueVertex.UVs[UVLayerIdx]));
	}
	
	return VertexHash;
}

static bool IsImportableImplicitObjectType(const Chaos::FImplicitObject& ImplicitObject)
{
	const Chaos::EImplicitObjectType InnerType = ImplicitObject.GetType() & (~(Chaos::ImplicitObjectType::IsScaled | Chaos::ImplicitObjectType::IsInstanced));
	if (InnerType == Chaos::ImplicitObjectType::Transformed)
	{
		const Chaos::FImplicitObjectTransformed& TransformedImplicitObject = static_cast<const Chaos::FImplicitObjectTransformed&>(ImplicitObject);
		if (const Chaos::FImplicitObject* SubObject = TransformedImplicitObject.GetTransformedObject())
		{
			return IsImportableImplicitObjectType(*SubObject);
		}
	}
	return (InnerType == Chaos::ImplicitObjectType::Box || InnerType == Chaos::ImplicitObjectType::Sphere || InnerType == Chaos::ImplicitObjectType::Capsule || InnerType == Chaos::ImplicitObjectType::Convex);
}

static FVector GetMeshBuildScale3D(const UStaticMesh& StaticMesh)
{
#if WITH_EDITOR
	const TArray<FStaticMeshSourceModel>& SourceModels = StaticMesh.GetSourceModels();
	if (SourceModels.Num() > 0)
	{
		return SourceModels[0].BuildSettings.BuildScale3D;
	}
#endif
	return FVector::One();
}

static void SetExternalCollisions(FVector3d MeshScale, UBodySetup* BodySetup, FGeometryCollection* GeometryCollection, int32 TransformIndex)
{
	if (BodySetup)
	{
		TArray<Chaos::FImplicitObjectPtr> Geoms;
		Chaos::FShapesArray Shapes;

		FGeometryAddParams CreateGeometryParams;
		CreateGeometryParams.bDoubleSided = false;
		CreateGeometryParams.CollisionData.CollisionFlags.bEnableQueryCollision = true;
		CreateGeometryParams.CollisionData.CollisionFlags.bEnableSimCollisionComplex = false; // no support for trimesh in destruction
		CreateGeometryParams.CollisionData.CollisionFlags.bEnableSimCollisionSimple = true;
		CreateGeometryParams.CollisionTraceType = ECollisionTraceFlag::CTF_UseSimpleAsComplex;
		CreateGeometryParams.Scale = MeshScale;
		CreateGeometryParams.LocalTransform = Chaos::FRigidTransform3::Identity;
		CreateGeometryParams.WorldTransform = Chaos::FRigidTransform3::Identity;
		CreateGeometryParams.Geometry = &BodySetup->AggGeom;
		CreateGeometryParams.TriMeshGeometries = MakeArrayView(BodySetup->TriMeshGeometries);

		// todo(chaos) : this currently also create the shape array which is unnecessary ,this could be optimized by having a common function to create only the implicits 
		ChaosInterface::CreateGeometry(CreateGeometryParams, Geoms, Shapes);

		TManagedArray<Chaos::FImplicitObjectPtr>& ExternalCollisions = GeometryCollection->AddAttribute<Chaos::FImplicitObjectPtr>(FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup);

		ExternalCollisions[TransformIndex] = nullptr;
		for (int32 GeomIndex = 0; GeomIndex < Geoms.Num();)
		{
			// make sure we only import box, sphere, capsule or convex
			if (Geoms[GeomIndex] && IsImportableImplicitObjectType(*Geoms[GeomIndex]))
			{
				GeomIndex++;
			}
			else
			{
				Geoms.RemoveAtSwap(GeomIndex);
			}
		}
		if (Geoms.Num() > 0)
		{
			ExternalCollisions[TransformIndex] = MakeImplicitObjectPtr<Chaos::FImplicitObjectUnion>(MoveTemp(Geoms));
		}
	}
}

void FGeometryCollectionEngineConversion::AppendMeshDescription(
	const FMeshDescription* MeshDescription, const FString& Name, int32 MaterialStartIndex, const FTransform& StaticMeshTransform, 
	FGeometryCollection* GeometryCollection, UBodySetup* BodySetup, bool ReindexMaterials, bool bAddInternalMaterials, bool bSetInternalFromMaterialIndex)
{
#if WITH_EDITORONLY_DATA

	if (!MeshDescription)
	{
		return;
	}

	check(GeometryCollection);

	// prepare to tick progress per 100k vertices
	const int32 ReportProgressSpacing = 100000;
	int32 NumVertProgressSteps = int32(MeshDescription->Vertices().GetArraySize() / ReportProgressSpacing);

	FScopedSlowTask AppendMeshDescriptionTask(6 + 2*NumVertProgressSteps, LOCTEXT("AppendMeshDescriptionTask", "Appending Mesh Description Data"));
	AppendMeshDescriptionTask.EnterProgressFrame(1);

	// source vertex information
	FStaticMeshConstAttributes Attributes(*MeshDescription);
	TArrayView<const FVector3f> SourcePosition = Attributes.GetVertexPositions().GetRawArray();
	TArrayView<const FVector3f> SourceTangent = Attributes.GetVertexInstanceTangents().GetRawArray();
	TArrayView<const float> SourceBinormalSign = Attributes.GetVertexInstanceBinormalSigns().GetRawArray();
	TArrayView<const FVector3f> SourceNormal = Attributes.GetVertexInstanceNormals().GetRawArray();
	TArrayView<const FVector4f> SourceColor = Attributes.GetVertexInstanceColors().GetRawArray();

	TVertexInstanceAttributesConstRef<FVector2f> InstanceUVs = Attributes.GetVertexInstanceUVs();
	const int32 NumUVLayers = InstanceUVs.GetNumChannels();
	TArray<TArrayView<const FVector2f>> SourceUVArrays;
	SourceUVArrays.SetNum(NumUVLayers);
	for (int32 UVLayerIdx = 0; UVLayerIdx < NumUVLayers; ++UVLayerIdx)
	{
		SourceUVArrays[UVLayerIdx] = InstanceUVs.GetRawArray(UVLayerIdx);
	}
		
	// target vertex information
	TManagedArray<FVector3f>& TargetVertex = GeometryCollection->Vertex;
	TManagedArray<FVector3f>& TargetTangentU = GeometryCollection->TangentU;
	TManagedArray<FVector3f>& TargetTangentV = GeometryCollection->TangentV;
	TManagedArray<FVector3f>& TargetNormal = GeometryCollection->Normal;
	TManagedArray<FLinearColor>& TargetColor = GeometryCollection->Color;
	TManagedArray<int32>& TargetBoneMap = GeometryCollection->BoneMap;
	TManagedArray<FLinearColor>& TargetBoneColor = GeometryCollection->BoneColor;
	TManagedArray<FString>& TargetBoneName = GeometryCollection->BoneName;

	if (GeometryCollection->NumUVLayers() < NumUVLayers)
	{
		GeometryCollection->SetNumUVLayers(NumUVLayers);
	}

	const int32 VertexStart = GeometryCollection->NumElements(FGeometryCollection::VerticesGroup);
	int32 VertexCount = 0;
		
	FVector Scale = StaticMeshTransform.GetScale3D();
		
	// We'll need to re-introduce UV seams, etc. by splitting vertices.
	// A new mapping of MeshDescription vertex instances to the split vertices is maintained.
	TMap<FVertexInstanceID, int32> VertexInstanceToGeometryCollectionVertex;
	VertexInstanceToGeometryCollectionVertex.Reserve(Attributes.GetVertexInstanceNormals().GetNumElements());


	int32 LastProgress = 0;		
	for (const FVertexID VertexIndex : MeshDescription->Vertices().GetElementIDs())
	{
		int32 Progress = int32(VertexIndex / ReportProgressSpacing);
		if (Progress > LastProgress)
		{
			AppendMeshDescriptionTask.EnterProgressFrame(Progress - LastProgress);
			LastProgress = Progress;
		}
		TArrayView<const FVertexInstanceID> ReferencingVertexInstances = MeshDescription->GetVertexVertexInstanceIDs(VertexIndex);

		// Generate per instance hash of splittable attributes.
		TMap<FUniqueVertex, TArray<FVertexInstanceID>> SplitVertices;
		for (const FVertexInstanceID& InstanceID : ReferencingVertexInstances)
		{
			TArray<FVector2f> SourceUVs;
			SourceUVs.SetNum(NumUVLayers);
			for (int32 UVLayerIdx = 0; UVLayerIdx < NumUVLayers; ++UVLayerIdx)
			{
				SourceUVs[UVLayerIdx] = SourceUVArrays[UVLayerIdx][InstanceID];
			}
				
			FUniqueVertex UniqueVertex{ SourceNormal[InstanceID], SourceTangent[InstanceID], SourceUVs };
			TArray<FVertexInstanceID>& SplitVertex = SplitVertices.FindOrAdd(UniqueVertex);
			SplitVertex.Add(InstanceID);
		}

		int32 CurrentVertex = GeometryCollection->AddElements(SplitVertices.Num(), FGeometryCollection::VerticesGroup);
			
		// Create a new vertex for each split vertex and map the mesh description instance to it.
		for (const TTuple<FUniqueVertex,TArray<FVertexInstanceID>>& SplitVertex : SplitVertices)
		{
			const TArray<FVertexInstanceID>& InstanceIDs = SplitVertex.Value;
			const FVertexInstanceID& ExemplarInstanceID = InstanceIDs[0];

			TargetVertex[CurrentVertex] = SourcePosition[VertexIndex] * (FVector3f)Scale;
			TargetBoneMap[CurrentVertex] = GeometryCollection->NumElements(FGeometryCollection::TransformGroup);

			TargetNormal[CurrentVertex] = SourceNormal[ExemplarInstanceID];
			TargetTangentU[CurrentVertex] = SourceTangent[ExemplarInstanceID];
			TargetTangentV[CurrentVertex] = (FVector3f)SourceBinormalSign[ExemplarInstanceID] * FVector3f::CrossProduct(TargetNormal[CurrentVertex], TargetTangentU[CurrentVertex]);

			GeometryCollection::UV::SetUVs(*GeometryCollection, CurrentVertex, SplitVertex.Key.UVs);

			if (SourceColor.Num() > 0)
			{
				TargetColor[CurrentVertex] = FLinearColor(SourceColor[ExemplarInstanceID]);
			}
			else
			{
				TargetColor[CurrentVertex] = FLinearColor::White;
			}

			for (const FVertexInstanceID& InstanceID : InstanceIDs)
			{
				VertexInstanceToGeometryCollectionVertex.Add(InstanceID, CurrentVertex);
			}

			++CurrentVertex;
			++VertexCount;
		}
	}

	if (LastProgress < NumVertProgressSteps)
	{
		AppendMeshDescriptionTask.EnterProgressFrame(NumVertProgressSteps - LastProgress);
		LastProgress = NumVertProgressSteps;
	}

	// enter a progress frame for triangle processing w/ size equivalent to the vertex processing (as a heuristic)
	// (note: could instead tick this per 100k triangles as we do with vertices above, if more responsive progress tracking is desired)
	AppendMeshDescriptionTask.EnterProgressFrame(NumVertProgressSteps);

	// target triangle indices
	TManagedArray<FIntVector>& TargetIndices = GeometryCollection->Indices;
	TManagedArray<bool>& TargetVisible = GeometryCollection->Visible;
	TManagedArray<int32>& TargetMaterialID = GeometryCollection->MaterialID;
	TManagedArray<int32>& TargetMaterialIndex = GeometryCollection->MaterialIndex;
	TManagedArray<bool>& TargetInternal = GeometryCollection->Internal;

	const int32 IndicesCount = MeshDescription->Triangles().Num();
	const int32 InitialNumIndices = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);
	const int32 IndicesStart = GeometryCollection->AddElements(IndicesCount, FGeometryCollection::FacesGroup);
	int32 TargetIndex = IndicesStart;
	for (const int32 TriangleIndex : MeshDescription->Triangles().GetElementIDs())
	{
		TArrayView<const FVertexInstanceID> TriangleVertices = MeshDescription->GetTriangleVertexInstances(TriangleIndex);

		TargetIndices[TargetIndex] = FIntVector(
			VertexInstanceToGeometryCollectionVertex[TriangleVertices[0]],
			VertexInstanceToGeometryCollectionVertex[TriangleVertices[1]],
			VertexInstanceToGeometryCollectionVertex[TriangleVertices[2]]
		);

		TargetVisible[TargetIndex] = true;

		// bAddInternalMaterials and bSetInternalFromMaterialIndex support the legacy system of odd-numbered materials indicating internal surfaces
		// bSetInternalFromMaterialIndex can be used to 'round trip' which faces are internal when using the 'ToMesh' tool to temporarily convert to/from static mesh.
		int32 MaterialIndexScale = 1 + int32(bAddInternalMaterials);
		int32 MaterialSourceID = MeshDescription->GetTrianglePolygonGroup(TriangleIndex);
		TargetMaterialID[TargetIndex] = MaterialStartIndex + (MaterialSourceID * MaterialIndexScale);
		bool bIsInternal = false;
		if (bSetInternalFromMaterialIndex && !bAddInternalMaterials)
		{
			bIsInternal = (MaterialSourceID % 2) == 1;
		}
		TargetInternal[TargetIndex] = bIsInternal;

		// Is this right?
		TargetMaterialIndex[TargetIndex] = TargetIndex;

		++TargetIndex;
	}

	AppendMeshDescriptionTask.EnterProgressFrame(1);

	// Geometry transform
	TManagedArray<FTransform3f>& Transform = GeometryCollection->Transform;

	int32 TransformIndex1 = GeometryCollection->AddElements(1, FGeometryCollection::TransformGroup);
	Transform[TransformIndex1] = FTransform3f(StaticMeshTransform);
	Transform[TransformIndex1].SetScale3D(FVector3f(1.f, 1.f, 1.f));

	SetExternalCollisions(Scale, BodySetup, GeometryCollection, TransformIndex1);
		
	// Bone Hierarchy - Added at root with no common parent
	TManagedArray<int32>& Parent = GeometryCollection->Parent;
	TManagedArray<int32>& SimulationType = GeometryCollection->SimulationType;
	Parent[TransformIndex1] = FGeometryCollection::Invalid;
	SimulationType[TransformIndex1] = FGeometryCollection::ESimulationTypes::FST_Rigid;

	const FColor RandBoneColor(FMath::Rand() % 100 + 5, FMath::Rand() % 100 + 5, FMath::Rand() % 100 + 5, 255);
	TargetBoneColor[TransformIndex1] = FLinearColor(RandBoneColor);
	TargetBoneName[TransformIndex1] = Name;

	// GeometryGroup
	int GeometryIndex = GeometryCollection->AddElements(1, FGeometryCollection::GeometryGroup);

	TManagedArray<int32>& TransformIndex = GeometryCollection->TransformIndex;
	TManagedArray<FBox>& BoundingBox = GeometryCollection->BoundingBox;
	TManagedArray<float>& InnerRadius = GeometryCollection->InnerRadius;
	TManagedArray<float>& OuterRadius = GeometryCollection->OuterRadius;
	TManagedArray<int32>& VertexStartArray = GeometryCollection->VertexStart;
	TManagedArray<int32>& VertexCountArray = GeometryCollection->VertexCount;
	TManagedArray<int32>& FaceStartArray = GeometryCollection->FaceStart;
	TManagedArray<int32>& FaceCountArray = GeometryCollection->FaceCount;

	TransformIndex[GeometryIndex] = TargetBoneMap[VertexStart];
	VertexStartArray[GeometryIndex] = VertexStart;
	VertexCountArray[GeometryIndex] = VertexCount;
	FaceStartArray[GeometryIndex] = InitialNumIndices;
	FaceCountArray[GeometryIndex] = IndicesCount;

	// TransformGroup
	TManagedArray<int32>& TransformToGeometryIndexArray = GeometryCollection->TransformToGeometryIndex;
	TransformToGeometryIndexArray[TransformIndex1] = GeometryIndex;

	FVector Center(0);
	for (int32 VertexIndex = VertexStart; VertexIndex < VertexStart + VertexCount; VertexIndex++)
	{
		Center += (FVector)TargetVertex[VertexIndex];
	}
	if (VertexCount) Center /= VertexCount;

	AppendMeshDescriptionTask.EnterProgressFrame(1);

	// Inner/Outer edges, bounding box
	BoundingBox[GeometryIndex] = FBox(ForceInitToZero);
	InnerRadius[GeometryIndex] = FLT_MAX;
	OuterRadius[GeometryIndex] = -FLT_MAX;
	for (int32 VertexIndex = VertexStart; VertexIndex < VertexStart + VertexCount; VertexIndex++)
	{
		BoundingBox[GeometryIndex] += (FVector)TargetVertex[VertexIndex];

		float Delta = (Center - (FVector)TargetVertex[VertexIndex]).Size();
		InnerRadius[GeometryIndex] = FMath::Min(InnerRadius[GeometryIndex], Delta);
		OuterRadius[GeometryIndex] = FMath::Max(OuterRadius[GeometryIndex], Delta);
	}

	AppendMeshDescriptionTask.EnterProgressFrame(1);

	// Inner/Outer centroid
	for (int fdx = IndicesStart; fdx < IndicesStart + IndicesCount; fdx++)
	{
		FVector Centroid(0);
		for (int e = 0; e < 3; e++)
		{
			Centroid += (FVector)TargetVertex[TargetIndices[fdx][e]];
		}
		Centroid /= 3;

		float Delta = (Center - Centroid).Size();
		InnerRadius[GeometryIndex] = FMath::Min(InnerRadius[GeometryIndex], Delta);
		OuterRadius[GeometryIndex] = FMath::Max(OuterRadius[GeometryIndex], Delta);
	}

	AppendMeshDescriptionTask.EnterProgressFrame(1);

	// Inner/Outer edges
	for (int fdx = IndicesStart; fdx < IndicesStart + IndicesCount; fdx++)
	{
		for (int e = 0; e < 3; e++)
		{
			int i = e, j = (e + 1) % 3;
			FVector Edge = (FVector)TargetVertex[TargetIndices[fdx][i]] + 0.5 * FVector(TargetVertex[TargetIndices[fdx][j]] - TargetVertex[TargetIndices[fdx][i]]);
			float Delta = (Center - Edge).Size();
			InnerRadius[GeometryIndex] = FMath::Min(InnerRadius[GeometryIndex], Delta);
			OuterRadius[GeometryIndex] = FMath::Max(OuterRadius[GeometryIndex], Delta);
		}
	}

	AppendMeshDescriptionTask.EnterProgressFrame(1);

	if (ReindexMaterials) {
		GeometryCollection->ReindexMaterials();
	}
#endif //WITH_EDITORONLY_DATA

}

namespace
{
	// Based on FStaticMeshOperations::AreNormalsAndTangentsValid but tests if *any* are valid instead of if *all* are valid to avoid forcing a recomputes for any small degen tri
	void HasValidNormalsAndTangents(const FMeshDescription& MeshDescription, bool& bHasValidNormals, bool& bHasValidTangents)
	{
		bHasValidNormals = false;
		bHasValidTangents = false;

		FStaticMeshConstAttributes Attributes(MeshDescription);
		TArrayView<const FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals().GetRawArray();
		TArrayView<const FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents().GetRawArray();

		for (const FVertexInstanceID VertexInstanceID : MeshDescription.VertexInstances().GetElementIDs())
		{
			bHasValidNormals |= (!VertexInstanceNormals[VertexInstanceID].IsNearlyZero() && !VertexInstanceNormals[VertexInstanceID].ContainsNaN());
			bHasValidTangents |= (!VertexInstanceTangents[VertexInstanceID].IsNearlyZero() && !VertexInstanceTangents[VertexInstanceID].ContainsNaN());
			if (bHasValidNormals && bHasValidTangents)
			{
				break;
			}
		}
	}

	// Note: This is similar to the InitializeAutoGeneratedAttributes ModelingComponents AssetUtils function, in the MeshModelingToolset plugin,
	// but we cannot use a plugin function from here, so have our own version. This version forces a recompute if Normals or Tangents are fully invalid, 
	// even if the corresponding Recompute flag is not set.
	void InitializeNormalsAndTangentsIfNeededOrRequested(FMeshDescription& Mesh, const FMeshBuildSettings* BuildSettings)
	{
		check(BuildSettings);

		// Recompute according to build settings
		bool bShouldRecomputeNormals = BuildSettings->bRecomputeNormals;
		bool bShouldRecomputeTangents = BuildSettings->bRecomputeTangents;
		// Also recompute if normals or tangents are fully invalid
		// Note: We don't force a recompute if only some elements are invalid as these may just be a degenerate element on a mesh where the normals/tangents should otherwise be preserved.
		if (!BuildSettings->bRecomputeNormals || !BuildSettings->bRecomputeTangents)
		{
			bool bHasValidNormals = false, bHasValidTangents = false;
			HasValidNormalsAndTangents(Mesh, bHasValidNormals, bHasValidTangents);
			bShouldRecomputeNormals |= !bHasValidNormals;
			bShouldRecomputeTangents |= !bHasValidTangents;
		}
		

		// run recompute function if either normals or tangents need recompute
		if (bShouldRecomputeNormals || bShouldRecomputeTangents)
		{
			FStaticMeshAttributes Attributes(Mesh);
			if (!Attributes.GetTriangleNormals().IsValid() || !Attributes.GetTriangleTangents().IsValid())
			{
				// If these attributes don't exist, create them and compute their values for each triangle
				FStaticMeshOperations::ComputeTriangleTangentsAndNormals(Mesh);
			}

			EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::BlendOverlappingNormals;
			ComputeNTBsOptions |= (bShouldRecomputeNormals) ? EComputeNTBsFlags::Normals : EComputeNTBsFlags::None;
			ComputeNTBsOptions |= (bShouldRecomputeTangents) ? EComputeNTBsFlags::Tangents : EComputeNTBsFlags::None;
			ComputeNTBsOptions |= BuildSettings->bUseMikkTSpace ? EComputeNTBsFlags::UseMikkTSpace : EComputeNTBsFlags::None;
			ComputeNTBsOptions |= BuildSettings->bComputeWeightedNormals ? EComputeNTBsFlags::WeightedNTBs : EComputeNTBsFlags::None;
			ComputeNTBsOptions |= BuildSettings->bRemoveDegenerates ? EComputeNTBsFlags::IgnoreDegenerateTriangles : EComputeNTBsFlags::None;

			FStaticMeshOperations::ComputeTangentsAndNormals(Mesh, ComputeNTBsOptions);
		}
	}
}

FMeshDescription* FGeometryCollectionEngineConversion::GetMaxResMeshDescriptionWithNormalsAndTangents(const UStaticMesh* StaticMesh)
{
	if (StaticMesh == nullptr)
	{
		return nullptr;
	}

	FMeshDescription* MeshDescription = nullptr;
	const FStaticMeshSourceModel* SourceModel = nullptr;
#if WITH_EDITORONLY_DATA
	// Prefer the HiRes description, although this isn't always available.
	if (StaticMesh->IsHiResMeshDescriptionValid())
	{
		MeshDescription = StaticMesh->GetHiResMeshDescription();
		SourceModel = &StaticMesh->GetHiResSourceModel();
	}
	else
	{
		MeshDescription = StaticMesh->GetMeshDescription(0);
		SourceModel = &StaticMesh->GetSourceModel(0);
	}

	InitializeNormalsAndTangentsIfNeededOrRequested(*MeshDescription, &SourceModel->BuildSettings);
#endif //WITH_EDITORONLY_DATA
	return MeshDescription;
}

int32 FGeometryCollectionEngineConversion::AppendMaterials(const TArray<UMaterialInterface*>& Materials, UGeometryCollection* GeometryCollectionObject, bool bAddInteriorCopy)
{
	// for each material, add a reference in our GeometryCollectionObject
	const int32 MaterialStart = GeometryCollectionObject->Materials.Num();
	const int32 NumMeshMaterials = Materials.Num();
	GeometryCollectionObject->Materials.Reserve(MaterialStart + NumMeshMaterials);

	for (int32 Index = 0; Index < NumMeshMaterials; ++Index)
	{
		UMaterialInterface* CurrMaterial = Materials[Index];

		// Possible we have a null entry - replace with default
		if (CurrMaterial == nullptr)
		{
			CurrMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
		}

		// We add the material twice, once for interior and again for exterior.
		GeometryCollectionObject->Materials.Add(CurrMaterial);
		if (bAddInteriorCopy)
		{
			GeometryCollectionObject->Materials.Add(CurrMaterial);
		}
	}
	return MaterialStart;
}

void FGeometryCollectionEngineConversion::AppendAutoInstanceMeshIndices(UGeometryCollection* GeometryCollectionObject, int32 FromTransformIndex, const UStaticMesh* StaticMesh, const TArray<UMaterialInterface*>& Materials)
{
	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollectionObject->GetGeometryCollection();
	if (GeometryCollectionPtr)
	{
		using namespace GeometryCollection::Facades;
		FCollectionInstancedMeshFacade InstancedMeshFacade(*GeometryCollectionPtr);

		const int32 NewNumOfTransforms = GeometryCollectionPtr->NumElements(FGeometryCollection::TransformGroup);
		if (NewNumOfTransforms > FromTransformIndex)
		{
			// create the schema if necessary
			InstancedMeshFacade.DefineSchema();
	
			const int32 AutoInstanceMeshIndex = GeometryCollectionObject->FindOrAddAutoInstanceMesh(StaticMesh, Materials);
			for (int32 TransformIndex = FromTransformIndex; TransformIndex < NewNumOfTransforms; TransformIndex++)
			{
				InstancedMeshFacade.SetIndex(TransformIndex, AutoInstanceMeshIndex);
			}
		}
	}
}

bool FGeometryCollectionEngineConversion::AppendStaticMesh(const UStaticMesh* StaticMesh, const TArray<UMaterialInterface*>& Materials,
	const FTransform& StaticMeshTransform, UGeometryCollection* GeometryCollectionObject, bool bReindexMaterials,
	bool bAddInternalMaterials, bool bSplitComponents, bool bSetInternalFromMaterialIndex)
{
#if WITH_EDITORONLY_DATA

	int32 StartMaterialIndex = GeometryCollectionObject->Materials.Num();

	check(GeometryCollectionObject);
	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollectionObject->GetGeometryCollection();
	FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get();
	check(GeometryCollection);

	const int32 OriginalNumOfTransforms = GeometryCollection->NumElements(FGeometryCollection::TransformGroup);

	if (AppendStaticMesh(StaticMesh, StartMaterialIndex, StaticMeshTransform, GeometryCollection, bReindexMaterials, bAddInternalMaterials, bSplitComponents, bSetInternalFromMaterialIndex))
	{
		AppendMaterials(Materials, GeometryCollectionObject, bAddInternalMaterials);

		AppendAutoInstanceMeshIndices(GeometryCollectionObject, OriginalNumOfTransforms, StaticMesh, Materials);

		return true;
	}

#endif //WITH_EDITORONLY_DATA
	return false;
}

bool FGeometryCollectionEngineConversion::AppendStaticMesh(const UStaticMesh* StaticMesh, int32 StartMaterialIndex,  const FTransform& StaticMeshTransform,
	FGeometryCollection* GeometryCollection, bool bReindexMaterials, bool bAddInternalMaterials, bool bSplitComponents, bool bSetInternalFromMaterialIndex)
{
#if WITH_EDITORONLY_DATA
	FScopedSlowTask AppendStaticMeshTask(bSplitComponents ? 3 : 2, LOCTEXT("AppendStaticMeshTask", "Appending Static Mesh"));

	if (StaticMesh)
	{
		AppendStaticMeshTask.EnterProgressFrame(1);
		FMeshDescription* MeshDescription = GetMaxResMeshDescriptionWithNormalsAndTangents(StaticMesh);

		check(GeometryCollection);

		if (MeshDescription)
		{
			const FVector MeshBuildScale3D = GetMeshBuildScale3D(*StaticMesh);
			const FTransform MeshTransform(
					StaticMeshTransform.GetRotation(),
					StaticMeshTransform.GetTranslation(),
					StaticMeshTransform.GetScale3D() * MeshBuildScale3D
			);

			if (bSplitComponents)
			{
				AppendStaticMeshTask.EnterProgressFrame(1);

				int32 MaxVID = MeshDescription->Vertices().Num();
				UE::Geometry::FVertexConnectedComponents Components(MaxVID);
				for (const FTriangleID TriangleID : MeshDescription->Triangles().GetElementIDs())
				{
					TArrayView<const FVertexID> TriangleIDs = MeshDescription->GetTriangleVertices(TriangleID);
					Components.ConnectVertices(TriangleIDs[0].GetValue(), TriangleIDs[1].GetValue());
					Components.ConnectVertices(TriangleIDs[1].GetValue(), TriangleIDs[2].GetValue());
				}
				if (Components.HasMultipleComponents(MaxVID, 2))
				{
					// look up vertex positions
					TVertexAttributesConstRef<FVector3f> VertexPositions = MeshDescription->GetVertexPositions();

					// vertex instance attributes
					FStaticMeshConstAttributes Attributes(*MeshDescription);
					TVertexInstanceAttributesConstRef<FVector2f> InstanceUVs = Attributes.GetVertexInstanceUVs();
					TVertexInstanceAttributesConstRef<FVector3f> InstanceNormals = Attributes.GetVertexInstanceNormals();
					TVertexInstanceAttributesConstRef<FVector3f> InstanceTangents = Attributes.GetVertexInstanceTangents();
					TVertexInstanceAttributesConstRef<float> InstanceBiTangentSign = Attributes.GetVertexInstanceBinormalSigns();
					TVertexInstanceAttributesConstRef<FVector4f> InstanceColors = Attributes.GetVertexInstanceColors();
					const int NumUVLayers = InstanceUVs.GetNumChannels();

					TMap<int32, int32> Map = Components.MakeComponentMap(MaxVID, 2);
					int32 NumIslands = Map.Num();
					
					TArray<FMeshDescription> Descriptions;
					Descriptions.SetNum(NumIslands);
					TArray<FMeshDescriptionBuilder> Builders;
					Builders.SetNum(NumIslands);
					for (int32 MeshIdx = 0; MeshIdx < NumIslands; ++MeshIdx)
					{
						FStaticMeshAttributes MeshAttributes(Descriptions[MeshIdx]);
						MeshAttributes.Register();

						Builders[MeshIdx].SetMeshDescription(&Descriptions[MeshIdx]);
						Builders[MeshIdx].SuspendMeshDescriptionIndexing();
						Builders[MeshIdx].SetNumUVLayers(NumUVLayers);
					}
					for (TPair<int32, int32> IDToIdx : Map)
					{
						int32 ID = IDToIdx.Key;
						int32 Idx = IDToIdx.Value;
						int32 NumVertices = Components.GetComponentSize(ID);
						Builders[Idx].ReserveNewVertices(NumVertices);
					}
					TArray<int32> VertexIDMap;
					VertexIDMap.Init(INDEX_NONE, MeshDescription->Vertices().Num());

					for (const FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
					{
						int32 MeshID = Components.GetComponent(VertexID.GetValue());
						int32* MeshIdx = Map.Find(MeshID);
						if (MeshIdx)
						{
							FVector Position = (FVector)VertexPositions.Get(VertexID);
							VertexIDMap[VertexID.GetValue()] = Builders[*MeshIdx].AppendVertex(Position);
						}
					}
					for (const FTriangleID TriangleID : MeshDescription->Triangles().GetElementIDs())
					{
						TArrayView<const FVertexID> TriangleVerts = MeshDescription->GetTriangleVertices(TriangleID);
						TArrayView<const FVertexInstanceID> SourceInstanceTri = MeshDescription->GetTriangleVertexInstances(TriangleID);
						int32 MeshID = Components.GetComponent(TriangleVerts[0].GetValue());
						int32 MeshIdx = Map[MeshID];
						FMeshDescriptionBuilder& Builder = Builders[MeshIdx];

						// create new vtx instances for each triangle
						FVertexInstanceID DestInstanceTri[3];
						for (int32 j = 0; j < 3; ++j)
						{
							const FVertexID TriVertex = VertexIDMap[TriangleVerts[j].GetValue()];
							DestInstanceTri[j] = Builder.AppendInstance(TriVertex);
						}
						// add the triangle to MeshDescription
						FPolygonGroupID MaterialID = MeshDescription->GetTrianglePolygonGroup(TriangleID);
						FTriangleID NewTriangleID = Builder.AppendTriangle(DestInstanceTri[0], DestInstanceTri[1], DestInstanceTri[2], MaterialID);
						// transfer UVs.  Note the Builder sets both the shared and per-instance UVs from this
						for (int32 UVLayer = 0; UVLayer < NumUVLayers; ++UVLayer)
						{
							FUVID UVIDs[3] = { FUVID(-1), FUVID(-1), FUVID(-1) };
							for (int32 j = 0; j < 3; ++j)
							{
								FVector2D UV = (FVector2D)InstanceUVs.Get(SourceInstanceTri[j], UVLayer);
								UVIDs[j] = Builder.AppendUV(UV, UVLayer);
							}

							// append the UV triangle - builder takes care of the rest
							Builder.AppendUVTriangle(NewTriangleID, UVIDs[0], UVIDs[1], UVIDs[2], UVLayer);
						}

						// Set instance attributes: normal/tangent/bitangent frame and color
						for (int32 j = 0; j < 3; ++j)
						{
							const FVertexInstanceID SourceInstanceID = SourceInstanceTri[j];
							const FVertexInstanceID DestInstanceID = DestInstanceTri[j];
							FVector TriVertNormal = (FVector)InstanceNormals.Get(SourceInstanceID);
							FVector TriVertTangent = (FVector)InstanceTangents.Get(SourceInstanceID);
							float BiTangentSign = (float)InstanceBiTangentSign.Get(SourceInstanceID);
							Builder.SetInstanceTangentSpace(DestInstanceID, TriVertNormal, TriVertTangent, BiTangentSign);
							FVector4f InstColor = InstanceColors.Get(SourceInstanceID);
							Builder.SetInstanceColor(DestInstanceID, InstColor);
						}
					}

					for (int32 MeshIdx = 0; MeshIdx < NumIslands; ++MeshIdx)
					{
						Builders[MeshIdx].ResumeMeshDescriptionIndexing();
					}

					for (FMeshDescription& MD : Descriptions)
					{
						AppendMeshDescription(&MD, StaticMesh->GetName(), StartMaterialIndex, MeshTransform, GeometryCollection, nullptr /*Body Setup to be set later, on root*/, 
							false, bAddInternalMaterials, bSetInternalFromMaterialIndex);
					}

					if (bReindexMaterials)
					{
						GeometryCollection->ReindexMaterials();
					}

					return true;
				}
				// else only one component -- fall back to just using the original mesh description
			}

			AppendStaticMeshTask.EnterProgressFrame(1);
			AppendMeshDescription(MeshDescription, StaticMesh->GetName(), StartMaterialIndex, MeshTransform, GeometryCollection, StaticMesh->GetBodySetup(), bReindexMaterials, bAddInternalMaterials, bSetInternalFromMaterialIndex);
			return true;
		}
	}
#endif //WITH_EDITORONLY_DATA
	return false;
}


bool FGeometryCollectionEngineConversion::AppendGeometryCollection(const FGeometryCollection* SourceGeometryCollectionPtr, int32 AssetMaterialStart, const FTransform& GeometryCollectionTransform, FGeometryCollection* TargetGeometryCollection, bool bReindexMaterials)
{
	if (SourceGeometryCollectionPtr == nullptr)
	{
		return false;
	}

	// Assemble offsets and add elements
	const int32 VertexCount = SourceGeometryCollectionPtr->Vertex.Num();
	const int32 FaceCount = SourceGeometryCollectionPtr->Indices.Num();
	const int32 TransformCount = SourceGeometryCollectionPtr->Transform.Num();
	const int32 GeometryCount = SourceGeometryCollectionPtr->TransformIndex.Num();
	const int32 SectionCount = SourceGeometryCollectionPtr->Sections.Num();

	FVector3f Scale = FVector3f(GeometryCollectionTransform.GetScale3D());
	FTransform3f AppliedTransform = FTransform3f(GeometryCollectionTransform);
	AppliedTransform.RemoveScaling();

	const int32 VertexStart = TargetGeometryCollection->AddElements(VertexCount, FGeometryCollection::VerticesGroup);
	const int32 FaceStart = TargetGeometryCollection->AddElements(FaceCount, FGeometryCollection::FacesGroup);
	const int32 TransformStart = TargetGeometryCollection->AddElements(TransformCount, FGeometryCollection::TransformGroup);
	const int32 GeometryStart = TargetGeometryCollection->AddElements(GeometryCount, FGeometryCollection::GeometryGroup);
	const int32 SectionStart = TargetGeometryCollection->AddElements(SectionCount, FGeometryCollection::MaterialGroup);

	// source vertex information
	const TManagedArray<FVector3f>& SourceVertex = SourceGeometryCollectionPtr->Vertex;
	const TManagedArray<FVector3f>& SourceTangentU = SourceGeometryCollectionPtr->TangentU;
	const TManagedArray<FVector3f>& SourceTangentV = SourceGeometryCollectionPtr->TangentV;
	const TManagedArray<FVector3f>& SourceNormal = SourceGeometryCollectionPtr->Normal;
	const TManagedArray<FLinearColor>& SourceColor = SourceGeometryCollectionPtr->Color;
	const TManagedArray<int32>& SourceBoneMap = SourceGeometryCollectionPtr->BoneMap;

	// target vertex information
	TManagedArray<FVector3f>& TargetVertex = TargetGeometryCollection->Vertex;
	TManagedArray<FVector3f>& TargetTangentU = TargetGeometryCollection->TangentU;
	TManagedArray<FVector3f>& TargetTangentV = TargetGeometryCollection->TangentV;
	TManagedArray<FVector3f>& TargetNormal = TargetGeometryCollection->Normal;
	TManagedArray<FLinearColor>& TargetColor = TargetGeometryCollection->Color;
	TManagedArray<int32>& TargetBoneMap = TargetGeometryCollection->BoneMap;

	TargetGeometryCollection->SetNumUVLayers(FMath::Max(TargetGeometryCollection->NumUVLayers(), SourceGeometryCollectionPtr->NumUVLayers()));
	GeometryCollection::UV::FUVLayers TargetUVLayers = GeometryCollection::UV::FindActiveUVLayers(*TargetGeometryCollection);
	GeometryCollection::UV::FConstUVLayers SourceUVLayers = GeometryCollection::UV::FindActiveUVLayers(*SourceGeometryCollectionPtr);

	// append vertices
	for (int32 VertexIndex = 0; VertexIndex < VertexCount; VertexIndex++)
	{
		const int32 VertexOffset = VertexStart + VertexIndex;
		TargetVertex[VertexOffset] = SourceVertex[VertexIndex] * (FVector3f)Scale;

		TargetTangentU[VertexOffset] = SourceTangentU[VertexIndex];
		TargetTangentV[VertexOffset] = SourceTangentV[VertexIndex];
		TargetNormal[VertexOffset] = SourceNormal[VertexIndex];

		for (int32 UVLayer = 0; UVLayer < SourceUVLayers.Num(); ++UVLayer)
		{
			TargetUVLayers[UVLayer][VertexOffset] = SourceUVLayers[UVLayer][VertexIndex];
		}
		TargetColor[VertexOffset] = SourceColor[VertexIndex];

		TargetBoneMap[VertexOffset] = SourceBoneMap[VertexIndex] + TransformStart;
	}

	// source face information
	const TManagedArray<FIntVector>& SourceIndices = SourceGeometryCollectionPtr->Indices;
	const TManagedArray<bool>& SourceVisible = SourceGeometryCollectionPtr->Visible;
	const TManagedArray<int32>& SourceMaterialID = SourceGeometryCollectionPtr->MaterialID;
	const TManagedArray<int32>& SourceMaterialIndex = SourceGeometryCollectionPtr->MaterialIndex;
	const TManagedArray<bool>& SourceInternal = SourceGeometryCollectionPtr->Internal;

	// target face information
	TManagedArray<FIntVector>& TargetIndices = TargetGeometryCollection->Indices;
	TManagedArray<bool>& TargetVisible = TargetGeometryCollection->Visible;
	TManagedArray<int32>& TargetMaterialID = TargetGeometryCollection->MaterialID;
	TManagedArray<int32>& TargetMaterialIndex = TargetGeometryCollection->MaterialIndex;
	TManagedArray<bool>& TargetInternal = TargetGeometryCollection->Internal;

	// append faces
	for (int32 FaceIndex = 0; FaceIndex < FaceCount; ++FaceIndex)
	{
		const FIntVector& SourceFace = SourceIndices[FaceIndex];
		const int32 FaceOffset = FaceStart + FaceIndex;
		TargetIndices[FaceOffset] = FIntVector(
			SourceFace[0] + VertexStart,
			SourceFace[1] + VertexStart,
			SourceFace[2] + VertexStart);
		TargetVisible[FaceOffset] = SourceVisible[FaceIndex];

		TargetMaterialID[FaceOffset] = AssetMaterialStart + SourceMaterialID[FaceIndex];
		TargetMaterialIndex[FaceOffset] = FaceOffset;
		TargetInternal[FaceOffset] = SourceInternal[FaceIndex];
	}

	// source transform information
	const TManagedArray<FTransform3f>& SourceTransform = SourceGeometryCollectionPtr->Transform;
	const TManagedArray<FString>& SourceBoneName = SourceGeometryCollectionPtr->BoneName;
	const TManagedArray<FLinearColor>& SourceBoneColor = SourceGeometryCollectionPtr->BoneColor;
	const TManagedArray<int32>& SourceParent = SourceGeometryCollectionPtr->Parent;
	const TManagedArray<TSet<int32>>& SourceChildren = SourceGeometryCollectionPtr->Children;
	const TManagedArray<int32>& SourceTransformToGeometryIndex = SourceGeometryCollectionPtr->TransformToGeometryIndex;
	const TManagedArray<int32>& SourceSimulationType = SourceGeometryCollectionPtr->SimulationType;
	const TManagedArray<int32>& SourceStatusFlags = SourceGeometryCollectionPtr->StatusFlags;
	const TManagedArray<int32>& SourceInitialDynamicState = SourceGeometryCollectionPtr->InitialDynamicState;
	const TManagedArray<Chaos::FImplicitObjectPtr>* SourceExternalCollisions = SourceGeometryCollectionPtr->FindAttribute<Chaos::FImplicitObjectPtr>(FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup);

	// target transform information
	TManagedArray<FTransform3f>& TargetTransform = TargetGeometryCollection->Transform;
	TManagedArray<FString>& TargetBoneName = TargetGeometryCollection->BoneName;
	TManagedArray<FLinearColor>& TargetBoneColor = TargetGeometryCollection->BoneColor;
	TManagedArray<int32>& TargetParent = TargetGeometryCollection->Parent;
	TManagedArray<TSet<int32>>& TargetChildren = TargetGeometryCollection->Children;
	TManagedArray<int32>& TargetTransformToGeometryIndex = TargetGeometryCollection->TransformToGeometryIndex;
	TManagedArray<int32>& TargetSimulationType = TargetGeometryCollection->SimulationType;
	TManagedArray<int32>& TargetStatusFlags = TargetGeometryCollection->StatusFlags;
	TManagedArray<int32>& TargetInitialDynamicState = TargetGeometryCollection->InitialDynamicState;
	TManagedArray<Chaos::FImplicitObjectPtr>& TargetExternalCollisions = TargetGeometryCollection->AddAttribute<Chaos::FImplicitObjectPtr>(FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup);

	// append transform hierarchy
	for (int32 TransformIndex = 0; TransformIndex < TransformCount; ++TransformIndex)
	{
		const int32 TransformOffset = TransformStart + TransformIndex;

		// Only apply the transform to the parent node. Child nodes only need scaling applied to translation offsets.
		if (SourceParent[TransformIndex] == INDEX_NONE)
		{
			TargetTransform[TransformOffset] = SourceTransform[TransformIndex] * AppliedTransform;
		}
		else
		{
			FTransform3f ScaledTranslation = SourceTransform[TransformIndex];
			ScaledTranslation.ScaleTranslation(Scale);
			TargetTransform[TransformOffset] = ScaledTranslation;
		}

		// #todo Get this Bone name to be unique
		TargetBoneName[TransformOffset] = SourceBoneName[TransformIndex];

		const FColor RandBoneColor(FMath::Rand() % 100 + 5, FMath::Rand() % 100 + 5, FMath::Rand() % 100 + 5, 255);
		TargetBoneColor[TransformOffset] = FLinearColor(RandBoneColor);

		TargetParent[TransformOffset] = (SourceParent[TransformIndex] == INDEX_NONE) ? INDEX_NONE : SourceParent[TransformIndex] + TransformStart;

		const TSet<int32>& SourceChildrenSet = SourceChildren[TransformIndex];
		for (int32 ChildIndex : SourceChildrenSet)
		{
			TargetChildren[TransformOffset].Add(ChildIndex + TransformStart);
		}

		TargetTransformToGeometryIndex[TransformOffset] = SourceTransformToGeometryIndex[TransformIndex] + GeometryStart;
		TargetSimulationType[TransformOffset] = SourceSimulationType[TransformIndex];
		TargetStatusFlags[TransformOffset] = SourceStatusFlags[TransformIndex];
		TargetInitialDynamicState[TransformOffset] = SourceInitialDynamicState[TransformIndex];

		TargetExternalCollisions[TransformOffset] = nullptr;
		if (SourceExternalCollisions)
		{
			TargetExternalCollisions[TransformOffset] = (*SourceExternalCollisions)[TransformIndex];
		}
	}

	// source geometry information
	const TManagedArray<int32>& SourceTransformIndex = SourceGeometryCollectionPtr->TransformIndex;
	const TManagedArray<int32>& SourceVertexStart = SourceGeometryCollectionPtr->VertexStart;
	const TManagedArray<int32>& SourceVertexCount = SourceGeometryCollectionPtr->VertexCount;
	const TManagedArray<int32>& SourceFaceStart = SourceGeometryCollectionPtr->FaceStart;
	const TManagedArray<int32>& SourceFaceCount = SourceGeometryCollectionPtr->FaceCount;

	// target geometry information
	TManagedArray<int32>& TargetTransformIndex = TargetGeometryCollection->TransformIndex;
	TManagedArray<FBox>& TargetBoundingBox = TargetGeometryCollection->BoundingBox;
	TManagedArray<float>& TargetInnerRadius = TargetGeometryCollection->InnerRadius;
	TManagedArray<float>& TargetOuterRadius = TargetGeometryCollection->OuterRadius;
	TManagedArray<int32>& TargetVertexStart = TargetGeometryCollection->VertexStart;
	TManagedArray<int32>& TargetVertexCount = TargetGeometryCollection->VertexCount;
	TManagedArray<int32>& TargetFaceStart = TargetGeometryCollection->FaceStart;
	TManagedArray<int32>& TargetFaceCount = TargetGeometryCollection->FaceCount;

	// append geometry
	for (int32 GeometryIndex = 0; GeometryIndex < GeometryCount; ++GeometryIndex)
	{
		const int32 GeometryOffset = GeometryStart + GeometryIndex;

		TargetTransformIndex[GeometryOffset] = SourceTransformIndex[GeometryIndex] + TransformStart;

		TargetVertexStart[GeometryOffset] = SourceVertexStart[GeometryIndex] + VertexStart;
		TargetVertexCount[GeometryOffset] = SourceVertexCount[GeometryIndex];
		TargetFaceStart[GeometryOffset] = SourceFaceStart[GeometryIndex] + FaceStart;
		TargetFaceCount[GeometryOffset] = SourceFaceCount[GeometryIndex];

		// Find centroid of geometry for inner/outer radius calculations
		FVector Center(0);
		for (int32 VertexIndex = TargetVertexStart[GeometryOffset]; VertexIndex < TargetVertexStart[GeometryOffset] + TargetVertexCount[GeometryOffset]; ++VertexIndex)
		{
			Center += (FVector)TargetVertex[VertexIndex];
		}
		if (TargetVertexCount[GeometryOffset]) Center /= TargetVertexCount[GeometryOffset];

		TargetBoundingBox[GeometryOffset] = FBox(ForceInitToZero);
		TargetInnerRadius[GeometryOffset] = FLT_MAX;
		TargetOuterRadius[GeometryOffset] = -FLT_MAX;
		for (int32 VertexIndex = TargetVertexStart[GeometryOffset]; VertexIndex < TargetVertexStart[GeometryOffset] + TargetVertexCount[GeometryOffset]; ++VertexIndex)
		{
			TargetBoundingBox[GeometryOffset] += (FVector)TargetVertex[VertexIndex];

			float Delta = (Center - (FVector)TargetVertex[VertexIndex]).Size();
			TargetInnerRadius[GeometryOffset] = FMath::Min(TargetInnerRadius[GeometryOffset], Delta);
			TargetOuterRadius[GeometryOffset] = FMath::Max(TargetOuterRadius[GeometryOffset], Delta);
		}
	}

	// source material information
	const TManagedArray<FGeometryCollectionSection>& SourceSections = SourceGeometryCollectionPtr->Sections;

	// target material information
	TManagedArray<FGeometryCollectionSection>& TargetSections = TargetGeometryCollection->Sections;

	// append sections
	for (int32 SectionIndex = 0; SectionIndex < SectionCount; ++SectionIndex)
	{
		int32 SectionOffset = SectionStart + SectionIndex;

		TargetSections[SectionOffset].MaterialID = AssetMaterialStart + SourceSections[SectionIndex].MaterialID;

		TargetSections[SectionOffset].FirstIndex = SourceSections[SectionIndex].FirstIndex + FaceStart * 3;
		TargetSections[SectionOffset].MinVertexIndex = VertexStart + SourceSections[SectionIndex].MinVertexIndex;

		TargetSections[SectionOffset].NumTriangles = SourceSections[SectionIndex].NumTriangles;
		TargetSections[SectionOffset].MaxVertexIndex = VertexStart + SourceSections[SectionIndex].MaxVertexIndex;
	}

	if (bReindexMaterials)
	{
		TargetGeometryCollection->ReindexMaterials();
	}

	return true;

}

void FGeometryCollectionEngineConversion::AppendGeometryCollection(const UGeometryCollection* SourceGeometryCollection, const TArray<UMaterialInterface*>& Materials, const FTransform& GeometryCollectionTransform, UGeometryCollection* TargetGeometryCollectionObject, bool bReindexMaterials)
{
	if (SourceGeometryCollection == nullptr)
	{
		return;
	}
	const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> SourceGeometryCollectionPtr = SourceGeometryCollection->GetGeometryCollection();

	check(TargetGeometryCollectionObject);
	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = TargetGeometryCollectionObject->GetGeometryCollection();
	FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get();
	check(GeometryCollection);

	int32 MaterialStart = AppendMaterials(Materials, TargetGeometryCollectionObject, false);

	const int32 TargetTransformStart = GeometryCollectionPtr->NumElements(FGeometryCollection::TransformGroup);

	if (AppendGeometryCollection(SourceGeometryCollectionPtr.Get(), MaterialStart, GeometryCollectionTransform, GeometryCollection, bReindexMaterials))
	{
		AppendGeometryCollectionInstancedMeshes(SourceGeometryCollection, TargetGeometryCollectionObject, TargetTransformStart);
	}
}


void FGeometryCollectionEngineConversion::AppendStaticMesh(const UStaticMesh* StaticMesh, const UStaticMeshComponent* StaticMeshComponent, const FTransform& StaticMeshTransform, UGeometryCollection* GeometryCollectionObject, 
	bool ReindexMaterials, bool bAddInternalMaterials, bool bSplitComponents, bool bSetInternalFromMaterialIndex)
{
	if (StaticMesh == nullptr)
	{
		return;
	}

	TArray<UMaterialInterface*> Materials;
	Materials.Reserve(StaticMesh->GetStaticMaterials().Num());

	for (int32 Index = 0; Index < StaticMesh->GetStaticMaterials().Num(); ++Index)
	{
		UMaterialInterface* CurrMaterial = StaticMeshComponent ? StaticMeshComponent->GetMaterial(Index) : StaticMesh->GetMaterial(Index);
		Materials.Add(CurrMaterial);
	}

	// Geometry collections usually carry the selection material, which we'll delete before appending
	UMaterialInterface* BoneSelectedMaterial = LoadObject<UMaterialInterface>(nullptr, UGeometryCollection::GetSelectedMaterialPath(), nullptr, LOAD_None, nullptr);
	GeometryCollectionObject->Materials.Remove(BoneSelectedMaterial);
	Materials.Remove(BoneSelectedMaterial);
	
	(void)AppendStaticMesh(StaticMesh, Materials, StaticMeshTransform, GeometryCollectionObject, ReindexMaterials, bAddInternalMaterials, bSplitComponents, bSetInternalFromMaterialIndex);
}


int32 FGeometryCollectionEngineConversion::AppendGeometryCollectionMaterials(const UGeometryCollection* SourceGeometryCollection, const UGeometryCollectionComponent* GeometryCollectionComponent, UGeometryCollection* TargetGeometryCollectionObject)
{
	check(SourceGeometryCollection);
	check(GeometryCollectionComponent);
	check(TargetGeometryCollectionObject);

	TArray<UMaterialInterface*> Materials;
	Materials.Reserve(SourceGeometryCollection->Materials.Num());

	for (int32 Index = 0; Index < SourceGeometryCollection->Materials.Num(); ++Index)
	{
		UMaterialInterface* CurrMaterial = GeometryCollectionComponent ? GeometryCollectionComponent->GetMaterial(Index) : SourceGeometryCollection->Materials[Index].Get();
		Materials.Add(CurrMaterial);
	}

	// Geometry collections usually carry the selection material, which we'll delete before appending
	UMaterialInterface* BoneSelectedMaterial = LoadObject<UMaterialInterface>(nullptr, UGeometryCollection::GetSelectedMaterialPath(), nullptr, LOAD_None, nullptr);
	TargetGeometryCollectionObject->Materials.Remove(BoneSelectedMaterial);
	Materials.Remove(BoneSelectedMaterial);

	return AppendMaterials(Materials, TargetGeometryCollectionObject, false);
}

void FGeometryCollectionEngineConversion::AppendGeometryCollectionInstancedMeshes(const UGeometryCollection* SourceGeometryCollectionObject, UGeometryCollection* TargetGeometryCollectionObject, int32 TargetTransformStartIndex)
{
	TSharedPtr<const FGeometryCollection, ESPMode::ThreadSafe> SourceGeometryCollectionPtr = SourceGeometryCollectionObject->GetGeometryCollection();
	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> TargetGeometryCollectionPtr = TargetGeometryCollectionObject->GetGeometryCollection();

	using namespace GeometryCollection::Facades;

	if (SourceGeometryCollectionPtr && TargetGeometryCollectionPtr)
	{
		const FCollectionInstancedMeshFacade SourceInstancedMeshFacade(*SourceGeometryCollectionPtr);
		FCollectionInstancedMeshFacade TargetInstancedMeshFacade(*TargetGeometryCollectionPtr);

		if (SourceInstancedMeshFacade.IsValid())
		{
			TargetInstancedMeshFacade.DefineSchema();

			const int32 NumSourceIndices = SourceInstancedMeshFacade.GetNumIndices();
			for (int32 SourceTransformIndex = 0; SourceTransformIndex < NumSourceIndices; SourceTransformIndex++)
			{
				int32 TargetInstancedMeshIndex = INDEX_NONE;

				const int32 SourceAutoInstanceIndex = SourceInstancedMeshFacade.GetIndex(SourceTransformIndex);
				if (SourceGeometryCollectionObject->AutoInstanceMeshes.IsValidIndex(SourceAutoInstanceIndex))
				{
					const FGeometryCollectionAutoInstanceMesh& SourceAutoInstanceMesh = SourceGeometryCollectionObject->GetAutoInstanceMesh(SourceAutoInstanceIndex);
					TargetInstancedMeshIndex = TargetGeometryCollectionObject->FindOrAddAutoInstanceMesh(SourceAutoInstanceMesh);
				}

				const int32 TargetTransformIndex = TargetTransformStartIndex + SourceTransformIndex;
				TargetInstancedMeshFacade.SetIndex(TargetTransformIndex, TargetInstancedMeshIndex);
			}
		}
	}
}

void FGeometryCollectionEngineConversion::AppendGeometryCollection(const UGeometryCollection* SourceGeometryCollection, const UGeometryCollectionComponent* GeometryCollectionComponent, const FTransform& GeometryCollectionTransform, UGeometryCollection* TargetGeometryCollectionObject, bool bReindexMaterials)
{
	if (SourceGeometryCollection == nullptr)
	{
		return;
	}

	int32 MaterialStartIndex = AppendGeometryCollectionMaterials(SourceGeometryCollection, GeometryCollectionComponent, TargetGeometryCollectionObject);

	const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> SourceGeometryCollectionPtr = SourceGeometryCollection->GetGeometryCollection();

	check(TargetGeometryCollectionObject);
	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = TargetGeometryCollectionObject->GetGeometryCollection();
	FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get();
	check(GeometryCollection);

	const int32 TargetTransformStart = GeometryCollectionPtr->NumElements(FGeometryCollection::TransformGroup);

	if (AppendGeometryCollection(SourceGeometryCollectionPtr.Get(), MaterialStartIndex, GeometryCollectionTransform, GeometryCollection, bReindexMaterials))
	{
		AppendGeometryCollectionInstancedMeshes(SourceGeometryCollection, TargetGeometryCollectionObject, TargetTransformStart);
	}
}




bool FGeometryCollectionEngineConversion::AppendSkeletalMesh(const USkeletalMesh* InSkeletalMesh, int32 MaterialStartIndex, const FTransform& SkeletalMeshTransform, 
	FManagedArrayCollection* InManagedArrayCollection, bool bReindexMaterials, bool bImportTransformOnly)
{
	//UE_LOGF(UGeometryCollectionConversionLogging, Log, "FGeometryCollectionEngineConversion::AppendSkeletalMesh()");
#if WITH_EDITOR
	int LODIndex = 0;
	if (!InManagedArrayCollection || !InSkeletalMesh)
	{
		return false;
	}

	FGeometryCollection GeometryCollection;

	// Transform Attributes
	TManagedArray<FTransform3f>& LocalSpaceTransform = GeometryCollection.ModifyAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);
	TManagedArray<int32>& Parent = GeometryCollection.ModifyAttribute<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup);
	TManagedArray<TSet<int32>>& Children = GeometryCollection.ModifyAttribute<TSet<int32>>(FTransformCollection::ChildrenAttribute, FTransformCollection::TransformGroup);
	TManagedArray<FLinearColor>& BoneColor = GeometryCollection.ModifyAttribute<FLinearColor>("BoneColor", FTransformCollection::TransformGroup);
	TManagedArray<FString>& BoneName = GeometryCollection.ModifyAttribute<FString>("BoneName", FTransformCollection::TransformGroup);
	TManagedArray<int32>& SimulationType = GeometryCollection.ModifyAttribute<int32>("SimulationType", FTransformCollection::TransformGroup);

	//
	// Convert the transform hierarchy
	//
	int32 RootIndex = INDEX_NONE;
	int32 TransformBaseIndex = INDEX_NONE;
	const USkeleton* Skeleton = InSkeletalMesh->GetSkeleton();
	const FReferenceSkeleton& ReferenceSkeleton = InSkeletalMesh->GetRefSkeleton();

	if (Skeleton && ReferenceSkeleton.GetNum())
	{
		const TArray<FTransform>& RestArray = ReferenceSkeleton.GetRefBonePose();
		TransformBaseIndex = GeometryCollection.AddElements(ReferenceSkeleton.GetNum(), FGeometryCollection::TransformGroup);
		RootIndex = TransformBaseIndex;

		for (int32 BoneIndex = 0; BoneIndex < ReferenceSkeleton.GetNum(); BoneIndex++)
		{
			// For validation against the component space position use
			// FTransform ComponentSpaceTransform =FAnimationRuntime::GetComponentSpaceTransformRefPose(ReferenceSkeleton, SkeletalBoneMap[BoneIndex]);

			LocalSpaceTransform[TransformBaseIndex + BoneIndex] = FTransform3f(RestArray[BoneIndex]);
			BoneName[TransformBaseIndex + BoneIndex] = ReferenceSkeleton.GetRefBoneInfo()[BoneIndex].Name.ToString();
			Parent[TransformBaseIndex + BoneIndex] = ReferenceSkeleton.GetRefBoneInfo()[BoneIndex].ParentIndex;

			TArray<int32> ChildrenArr;
			if (ReferenceSkeleton.GetDirectChildBones(BoneIndex, ChildrenArr))
			{
				Children[TransformBaseIndex + BoneIndex].Append(ChildrenArr);
			}

			SimulationType[TransformBaseIndex + BoneIndex] = FGeometryCollection::ESimulationTypes::FST_None;
			BoneColor[TransformBaseIndex + BoneIndex] = FLinearColor::MakeRandomColor();

			if (Parent[TransformBaseIndex + BoneIndex] == INDEX_NONE)
			{
				RootIndex = TransformBaseIndex + BoneIndex;
			}
		}
		GeometryCollection::Facades::FTransformSource TransformSourceFacade(*InManagedArrayCollection);
		TSet<int32> Roots;
		Roots.Add(RootIndex);
		TransformSourceFacade.AddTransformSource(Skeleton->GetName(), Skeleton->GetGuid().ToString(), Roots, InSkeletalMesh->GetName());
	}

	if (bImportTransformOnly)
	{
		GeometryCollection.CopyTo(InManagedArrayCollection);
		return true;
	}

	FMeshDescription MeshDescription;
	if (!InSkeletalMesh->CloneMeshDescription(LODIndex, MeshDescription))
	{
		return false;
	}

	// Vertices Attributes
	TManagedArray<FVector3f>& Vertex = GeometryCollection.ModifyAttribute<FVector3f>("Vertex", FGeometryCollection::VerticesGroup);
	TManagedArray<FVector3f>& Normal = GeometryCollection.ModifyAttribute<FVector3f>("Normal", FGeometryCollection::VerticesGroup);
	TManagedArray<FLinearColor>& Color = GeometryCollection.ModifyAttribute<FLinearColor>("Color", FGeometryCollection::VerticesGroup);
	TManagedArray<FVector3f>& TangentU = GeometryCollection.ModifyAttribute<FVector3f>("TangentU", FGeometryCollection::VerticesGroup);
	TManagedArray<FVector3f>& TangentV = GeometryCollection.ModifyAttribute<FVector3f>("TangentV", FGeometryCollection::VerticesGroup);
	TManagedArray<int32>& BoneMap = GeometryCollection.ModifyAttribute<int32>("BoneMap", FGeometryCollection::VerticesGroup);
	// Index Attributes
	TManagedArray<FIntVector>& Indices = GeometryCollection.ModifyAttribute<FIntVector>("Indices", FGeometryCollection::FacesGroup);
	TManagedArray<bool>& Visible = GeometryCollection.ModifyAttribute<bool>("Visible", FGeometryCollection::FacesGroup);
	TManagedArray<int32>& MaterialIndex = GeometryCollection.ModifyAttribute<int32>("MaterialIndex", FGeometryCollection::FacesGroup);
	TManagedArray<int32>& MaterialID = GeometryCollection.ModifyAttribute<int32>("MaterialID", FGeometryCollection::FacesGroup);

	//
	// Identify disconnected geoemtry
	//
	int32 VertexCount = 0, TriangleCount = 0;
	TArray<int32> SourceVertexToComponentMap;				// Map from mesh vertex index to target vertex index
	TArray<TArray<FIntVector>> ComponentsSourceIndices;		// Mesh triangle indices of each component.
	TArray<TArray<FIntVector2>> SourceToTargetTriangleMap;	// Mesh triangle index of each triangle in the component. 
	GeometryCollectionEngineUtility::GenerateConnectedComponents(InSkeletalMesh, ComponentsSourceIndices,
		SourceToTargetTriangleMap, SourceVertexToComponentMap, TriangleCount, VertexCount);

	//
	// Vertex Attributes
	//
	int VertexBaseIndex = GeometryCollection.AddElements(VertexCount, FGeometryCollection::VerticesGroup);
	int NumTargetVertices = GeometryCollection.NumElements(FGeometryCollection::VerticesGroup);



	// 
	// Transform Attributes 
	// 
	auto MakeUnique = [&BoneName](FString& NewName, int32& CurrentIndex)
	{
		FString TestName = FString::Printf(TEXT("%s%d"), *NewName, CurrentIndex);
		while (BoneName.Contains(TestName))
		{
			TestName = FString::Printf(TEXT("%s%d"), *NewName, CurrentIndex);
			CurrentIndex++;
		}
		return TestName;
	};

	// add transforms for the separated geometry components. 
	int32 SplitMeshIndex = 1;
	TArray<int32> ComponentToTransformGroupIndex;
	ComponentToTransformGroupIndex.Init(INDEX_NONE, ComponentsSourceIndices.Num());
	int32 ComponentTransformBaseIndex = GeometryCollection.AddElements(ComponentsSourceIndices.Num(), FTransformCollection::TransformGroup);
	for (int ComponentIndex = 0; ComponentIndex < ComponentsSourceIndices.Num(); ComponentIndex++)
	{
		FString BaseName("SplitMesh");

		int32 ComponentTransformIndex = ComponentTransformBaseIndex + ComponentIndex;
		Parent[ComponentTransformIndex] = RootIndex;
		if (RootIndex != INDEX_NONE)
		{
			Children[RootIndex].Add(ComponentTransformIndex);
			BaseName = FString::Printf(TEXT("%s_SplitMesh"), *InSkeletalMesh->GetName());
		}

		BoneName[ComponentTransformIndex] = MakeUnique(BaseName, SplitMeshIndex);
		LocalSpaceTransform[ComponentTransformIndex] = FTransform3f::Identity;
		SimulationType[ComponentTransformIndex] = FGeometryCollection::ESimulationTypes::FST_None;
		BoneColor[ComponentTransformIndex] = FLinearColor::MakeRandomColor();
		ComponentToTransformGroupIndex[ComponentIndex] = ComponentTransformIndex;
		SplitMeshIndex++;
	}

	TArray<FTransform> ComponentTransform;
	GeometryCollectionAlgo::GlobalMatrices(LocalSpaceTransform, Parent, ComponentTransform);

	FSkeletalMeshConstAttributes SkeletalMeshConstAttributes(MeshDescription);

	// normal detection
	const bool bHasNormals = MeshDescription.VertexInstanceAttributes().HasAttribute(MeshAttribute::VertexInstance::Normal);

	// color attribute detection 
	bool bHasVertexColors = false;
	FLinearColor DefaultColor = FLinearColor::White;
	TVertexInstanceAttributesConstRef<FVector4f> InstanceColors = SkeletalMeshConstAttributes.GetVertexInstanceColors();
	if (InstanceColors.IsValid())
	{
		bHasVertexColors = true;
		DefaultColor = InstanceColors.GetDefaultValue();
	}

	// vertex weight attribute detection
	FSkinWeightsVertexAttributesConstRef VertexSkinWeights = SkeletalMeshConstAttributes.GetVertexSkinWeights();
	const bool bHasVertexSkinWeights = VertexSkinWeights.IsValid();
	GeometryCollection::Facades::FVertexBoneWeightsFacade VertexBoneWeightsFacade(GeometryCollection);
	TArray<int32> VertexBoneIndex;
	TArray<float> VertexBoneWeight;

	// @todo(GeometryCollectionConversion) : Add support for UV's, Normals  
	//const int32 NumUVLayers = VertexBuffers.StaticMeshVertexBuffer.GetNumTexCoords();
	//GeometryCollection::UV::SetNumUVLayers(*GeometryCollection, NumUVLayers);
	//GeometryCollection::UV::FUVLayers UVLayers = GeometryCollection::UV::FindActiveUVLayers(*GeometryCollection);
	TArray<bool> TargetVertexVisited; TargetVertexVisited.Init(false, NumTargetVertices);
	for (int32 ComponentIndex = 0;ComponentIndex<ComponentsSourceIndices.Num(); ComponentIndex++)
	{
		for (int32 TriangleIndex = 0; TriangleIndex < ComponentsSourceIndices[ComponentIndex].Num(); TriangleIndex++)
		{
			const int32 SourceTriangleIndex = SourceToTargetTriangleMap[ComponentIndex][TriangleIndex][0];
			const FIntVector& Triangle = ComponentsSourceIndices[ComponentIndex][TriangleIndex];
			for (int TriVtxIdx = 0; TriVtxIdx < 3; TriVtxIdx++)
			{
				int SourceVertexIndex = Triangle[TriVtxIdx];
				if (ensure(0 <= SourceVertexIndex && SourceVertexIndex < SourceVertexToComponentMap.Num()))
				{
					int TargetVertexIndex = SourceVertexToComponentMap[SourceVertexIndex] + VertexBaseIndex;
					if (ensure(VertexBaseIndex <= TargetVertexIndex && TargetVertexIndex < TargetVertexVisited.Num()))
					{
						if (!TargetVertexVisited[TargetVertexIndex])
						{
							TargetVertexVisited[TargetVertexIndex] = true;

							// Vertex positions
							BoneMap[TargetVertexIndex] = ComponentToTransformGroupIndex[ComponentIndex];

							FVector SourceVertex = (FVector)MeshDescription.GetVertexPosition(SourceVertexIndex);
							FMatrix M = ComponentTransform[BoneMap[TargetVertexIndex]].ToInverseMatrixWithScale();
							Vertex[TargetVertexIndex] = (FVector4f)M.TransformPosition(SourceVertex);

							// Get Avg Vertex Normals
							// TODO(chaos) - once we support UV and split vertices accordingly, we can use the actual normal instead of averaging it 
							if (bHasNormals)
							{
								Normal[TargetVertexIndex] = FVector3f::ZeroVector;

								const TArrayView<const FVertexInstanceID> VertexInstances = MeshDescription.GetVertexVertexInstanceIDs(SourceVertexIndex);
								for (const FVertexInstanceID VertexInstanceID : VertexInstances)
								{
									const FVector3f VertexInstanceNormal = MeshDescription.VertexInstanceAttributes().GetAttribute<FVector3f>(VertexInstanceID, MeshAttribute::VertexInstance::Normal);
									Normal[TargetVertexIndex] += VertexInstanceNormal / VertexInstances.Num();
								}
								Normal[TargetVertexIndex].Normalize();
							}

							// Vertex Colors
							Color[TargetVertexIndex] = DefaultColor;
							if (bHasVertexColors)
							{
								TArrayView<const FVertexInstanceID> SourceInstanceTri = MeshDescription.GetTriangleVertexInstances(SourceTriangleIndex);
								FVector4f InstColor = InstanceColors[SourceInstanceTri[TriVtxIdx]];
								UE::Geometry::LinearColors::SRGBToLinear(InstColor);
								Color[TargetVertexIndex] = FLinearColor(InstColor);
							}

							// Vertex skin weights
							if (bHasVertexSkinWeights)
							{
								const FVertexBoneWeightsConst BoneWeights = VertexSkinWeights.Get(FVertexID(SourceVertexIndex));
								const int32 InfluenceCount = BoneWeights.Num();
								VertexBoneIndex.SetNum(InfluenceCount);
								VertexBoneWeight.SetNum(InfluenceCount);
								for (int32 InfluenceIndex = 0; InfluenceIndex < InfluenceCount; ++InfluenceIndex)
								{
									VertexBoneIndex[InfluenceIndex] = BoneWeights[InfluenceIndex].GetBoneIndex();
									VertexBoneWeight[InfluenceIndex] = BoneWeights[InfluenceIndex].GetWeight();
								}
								VertexBoneWeightsFacade.ModifyBoneWeight(TargetVertexIndex, VertexBoneIndex, VertexBoneWeight);
							}
							// @todo(GeometryCollectionConversion) : Add support for UV's  
							//TangentU[VertexOffset] = VertexBuffers.StaticMeshVertexBuffer.VertexTangentX(VertexIndex);
							//TangentV[VertexOffset] = VertexBuffers.StaticMeshVertexBuffer.VertexTangentY(VertexIndex);
							//Normal[VertexOffset] = VertexBuffers.StaticMeshVertexBuffer.VertexTangentZ(VertexIndex);
							//for (int32 UVLayerIdx = 0; UVLayerIdx < NumUVLayers; ++UVLayerIdx)
							//{
							//	UVLayers[UVLayerIdx][VertexOffset] = VertexBuffers.StaticMeshVertexBuffer.GetVertexUV(VertexIndex, UVLayerIdx);
							//}
						}
					}
				}
			}
		}
	}

	//
	// Add the Triangles to the Geometry Collection
	//.. ensure all component vertices are contigious in the array 
	//
	int IndicesBaseIndex = GeometryCollection.AddElements(TriangleCount, FGeometryCollection::FacesGroup);

	int CurrentIndex = IndicesBaseIndex;
	for (int ComponentIndex = 0; ComponentIndex < ComponentsSourceIndices.Num(); ComponentIndex++)
	{
		for (int32 TriangleIndex = 0; TriangleIndex < ComponentsSourceIndices[ComponentIndex].Num(); TriangleIndex++)
		{
			SourceToTargetTriangleMap[ComponentIndex][TriangleIndex][1] = CurrentIndex;

			FIntVector& Triangle = ComponentsSourceIndices[ComponentIndex][TriangleIndex];
			for (int TriVtxIdx = 0; TriVtxIdx < 3; TriVtxIdx++)
			{
				Indices[CurrentIndex][TriVtxIdx] = SourceVertexToComponentMap[Triangle[TriVtxIdx]] + VertexBaseIndex;
			}

			Visible[CurrentIndex] = true;
			MaterialID[CurrentIndex] = 0;
			MaterialIndex[CurrentIndex] = CurrentIndex;
			CurrentIndex++;
		}
	}

	// Geometry Group
	TArray<int32> GeometryIndices;
	FGeometryCollection::DefineGeometrySchema(GeometryCollection);
	GeometryCollectionAlgo::ContiguousArray(GeometryIndices, GeometryCollection.NumElements(FGeometryCollection::GeometryGroup));
	GeometryCollection.RemoveDependencyFor(FGeometryCollection::GeometryGroup);
	GeometryCollection.RemoveElements(FGeometryCollection::GeometryGroup, GeometryIndices);
	::GeometryCollection::AddGeometryProperties(&GeometryCollection);

	// copy to the target collection
	GeometryCollection.CopyTo(InManagedArrayCollection);
	return true;
#else
	return false;
#endif
}

void FGeometryCollectionEngineConversion::AppendSkeleton(const USkeleton* InSkeleton, const FTransform& SkeletalMeshTransform, FManagedArrayCollection* InCollection)
{
	//UE_LOGF(UGeometryCollectionConversionLogging, Log, "FGeometryCollectionEngineConversion::AppendSkeletalMesh()");
	if (!InCollection || !InSkeleton)
	{
		return;
	}
	FGeometryCollection::DefineTransformSchema(*InCollection);
	GeometryCollection::Facades::FTransformSource TransformSourceFacade(*InCollection);

	TManagedArray<FTransform3f>& Transform = InCollection->ModifyAttribute<FTransform3f>(FTransformCollection::TransformAttribute, FTransformCollection::TransformGroup);
	TManagedArray<FLinearColor>& BoneColor = InCollection->ModifyAttribute<FLinearColor>("BoneColor", FTransformCollection::TransformGroup);
	TManagedArray<FString>& BoneName = InCollection->ModifyAttribute<FString>("BoneName", FTransformCollection::TransformGroup);
	TManagedArray<int32>& Parent = InCollection->ModifyAttribute<int32>(FTransformCollection::ParentAttribute, FTransformCollection::TransformGroup);
	TManagedArray< TSet<int32> >& Child = InCollection->ModifyAttribute< TSet<int32> >(FTransformCollection::ChildrenAttribute, FTransformCollection::TransformGroup);

	const FReferenceSkeleton& Skeleton = InSkeleton->GetReferenceSkeleton();
	int32 NumBones = Skeleton.GetNum();
	if (NumBones)
	{
		const TArray<FTransform>& RestTransform = Skeleton.GetRefBonePose();
		const TArray<FMeshBoneInfo>& BoneInfo = Skeleton.GetRefBoneInfo();

		TSet<int32> Roots;
		int32 TransformBaseIndex = InCollection->AddElements(NumBones, FGeometryCollection::TransformGroup);
		for (int i = 0, Idx = TransformBaseIndex; i < NumBones; i++, Idx++)
		{
			Transform[Idx] = FTransform3f(RestTransform[i]);
			BoneColor[Idx] = FLinearColor(FColor(FMath::Rand() % 100 + 5, FMath::Rand() % 100 + 5, FMath::Rand() % 100 + 5, 255));
			BoneName[Idx] = BoneInfo[i].Name.ToString();
			Parent[Idx] = BoneInfo[i].ParentIndex;
			if (Parent[Idx] != INDEX_NONE)
			{
				Child[Parent[Idx]].Add(Idx);
			}
			else
			{
				Roots.Add(Idx);
			}
		}

		ensure(Roots.Num());
		TransformSourceFacade.AddTransformSource(InSkeleton->GetName(), InSkeleton->GetGuid().ToString(), Roots, FString() /*SKMName*/);
	}
}

const FSkeletalMeshLODRenderData* FGeometryCollectionEngineConversion::GetSkeletalMeshLOD(const USkeletalMesh* SkeletalMesh, int32 LOD)
{
	if (const USkeleton* Skeleton = SkeletalMesh->GetSkeleton())
	{
		if (const FSkeletalMeshRenderData* SkelMeshRenderData = SkeletalMesh->GetResourceForRendering())
		{
			if (SkelMeshRenderData->LODRenderData.IsValidIndex(LOD))
			{
				return &SkelMeshRenderData->LODRenderData[LOD];
			}
		}
	}
	return nullptr;
}

void FGeometryCollectionEngineConversion::AppendSkeletalMesh(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, const FTransform& SkeletalMeshTransform, UGeometryCollection* GeometryCollectionObject, bool bReindexMaterials)
{
	//UE_LOGF(UGeometryCollectionConversionLogging, Log, "FGeometryCollectionEngineConversion::AppendSkeletalMesh()");
	check(SkeletalMesh);
	if (GeometryCollectionObject)
	{
		TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionPtr = GeometryCollectionObject->GetGeometryCollection();
		if (FGeometryCollection* GeometryCollection = GeometryCollectionPtr.Get())
		{
			int32 MaterialStart = GeometryCollectionObject->Materials.Num();
			constexpr bool bImportTransformOnly = false;
			if (AppendSkeletalMesh(SkeletalMesh, MaterialStart, SkeletalMeshTransform, GeometryCollection, bReindexMaterials, bImportTransformOnly))
			{
				AppendSkeletalMeshMaterials(SkeletalMesh, SkeletalMeshComponent, GeometryCollectionObject);
			}
		}
	}
}

int32 FGeometryCollectionEngineConversion::AppendSkeletalMeshMaterials(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, UGeometryCollection* GeometryCollectionObject)
{
	check(SkeletalMesh);
	check(SkeletalMeshComponent);
	check(GeometryCollectionObject);

	const TArray<FSkeletalMaterial>& SkeletalMeshMaterials = SkeletalMesh->GetMaterials();

	int32 CurrIdx = 0;
	UMaterialInterface* CurrMaterial = SkeletalMeshComponent ? SkeletalMeshComponent->GetMaterial(CurrIdx) : ToRawPtr(SkeletalMeshMaterials[CurrIdx].MaterialInterface);

	int MaterialStart = GeometryCollectionObject->Materials.Num();
	while (CurrMaterial)
	{
		GeometryCollectionObject->Materials.Add(CurrMaterial);
		CurrMaterial = SkeletalMeshComponent ? SkeletalMeshComponent->GetMaterial(++CurrIdx) : ToRawPtr(SkeletalMeshMaterials[++CurrIdx].MaterialInterface);
	}

	return MaterialStart;
}

void FGeometryCollectionEngineConversion::AppendGeometryCollectionSource(const FGeometryCollectionSource& GeometryCollectionSource, FGeometryCollection& GeometryCollectionInOut, TArray<UMaterial*>& MaterialsInOut, bool ReindexMaterials)
{
	const int32 StartMaterialIndex = MaterialsInOut.Num();
	if (AppendGeometryCollectionSourceNoMaterial(GeometryCollectionSource, GeometryCollectionInOut, StartMaterialIndex, ReindexMaterials))
	{
		MaterialsInOut.Append(GeometryCollectionSource.SourceMaterial);
	}
}
void FGeometryCollectionEngineConversion::AppendGeometryCollectionSource(const FGeometryCollectionSource& GeometryCollectionSource, FGeometryCollection& GeometryCollectionInOut, TArray<UMaterialInterface*>& MaterialInstancesInOut, bool ReindexMaterials)
{
	const int32 StartMaterialIndex = MaterialInstancesInOut.Num();
	if (AppendGeometryCollectionSourceNoMaterial(GeometryCollectionSource, GeometryCollectionInOut, StartMaterialIndex, ReindexMaterials))
	{
		MaterialInstancesInOut.Append(GeometryCollectionSource.SourceMaterial);
	}
}
bool FGeometryCollectionEngineConversion::AppendGeometryCollectionSourceNoMaterial(const FGeometryCollectionSource& GeometryCollectionSource, FGeometryCollection& GeometryCollectionInOut, int32 StartMaterialIndex, bool ReindexMaterials)
{
	if (const UObject* SourceObject = GeometryCollectionSource.SourceGeometryObject.TryLoad())
	{
		if (const UStaticMesh* SourceStaticMesh = Cast<UStaticMesh>(SourceObject))
		{
			bool bLegacyAddInternal = GeometryCollectionSource.bAddInternalMaterials;
			(void)AppendStaticMesh(
				SourceStaticMesh,
				StartMaterialIndex,
				GeometryCollectionSource.LocalTransform,
				&GeometryCollectionInOut,
				ReindexMaterials,
				bLegacyAddInternal,
				GeometryCollectionSource.bSplitComponents,
				GeometryCollectionSource.bSetInternalFromMaterialIndex
				);
			return true;
		}
		else if (const USkeletalMesh* SourceSkeletalMesh = Cast<USkeletalMesh>(SourceObject))
		{
			(void)AppendSkeletalMesh(
				SourceSkeletalMesh,
				StartMaterialIndex,
				GeometryCollectionSource.LocalTransform,
				&GeometryCollectionInOut,
				ReindexMaterials,
				false
				);
			return true;
		}
		else if (const UGeometryCollection* SourceGeometryCollection = Cast<UGeometryCollection>(SourceObject))
		{
			AppendGeometryCollection(
				SourceGeometryCollection->GetGeometryCollection().Get(),
				StartMaterialIndex,
				GeometryCollectionSource.LocalTransform,
				&GeometryCollectionInOut,
				ReindexMaterials
				);
			return true;
		}
	}
	return false;
}

void FGeometryCollectionEngineConversion::ConvertStaticMeshToGeometryCollection(const TObjectPtr<UStaticMesh> StaticMesh, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterial>>& OutMaterials, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes, bool bSetInternalFromMaterialIndex, bool bSplitComponents)
{
	TArray<TObjectPtr<UMaterialInterface>> OutMaterialInstances;
	ConvertStaticMeshToGeometryCollection(StaticMesh, OutCollection, OutMaterialInstances, OutInstancedMeshes, bSetInternalFromMaterialIndex, bSplitComponents);
	GetMaterialsFromInstances(OutMaterialInstances, OutMaterials);
}

void FGeometryCollectionEngineConversion::ConvertStaticMeshToGeometryCollection(const TObjectPtr<UStaticMesh> StaticMesh, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterialInterface>>& OutMaterialInstances, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes, bool bSetInternalFromMaterialIndex, bool bSplitComponents)
{
	ConvertStaticMeshToGeometryCollection(StaticMesh, FTransform::Identity, OutCollection, OutMaterialInstances, OutInstancedMeshes, bSetInternalFromMaterialIndex, bSplitComponents);
}

void FGeometryCollectionEngineConversion::ConvertStaticMeshToGeometryCollection(const TObjectPtr<UStaticMesh> StaticMesh, const FTransform& MeshTransform, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterialInterface>>& OutMaterialInstances, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes, bool bSetInternalFromMaterialIndex, bool bSplitComponents)
{
#if WITH_EDITORONLY_DATA
	if (UGeometryCollection* NewGeometryCollection = NewObject<UGeometryCollection>())
	{
		// If any of the static meshes have Nanite enabled, also enable on the new geometry collection asset for convenience.
		NewGeometryCollection->EnableNanite |= StaticMesh->IsNaniteEnabled();

		// Record the contributing source on the asset.
		FSoftObjectPath SourceSoftObjectPath(StaticMesh);

		// Materials		
		TArray<TObjectPtr<UMaterialInterface>> MatArr;
		for (auto& StaticMaterial : StaticMesh->GetStaticMaterials())
		{
			MatArr.Emplace(StaticMaterial.MaterialInterface);
		}
		TArray<TObjectPtr<UMaterialInterface>> SourceMaterials(MatArr);

		// InstanceMeshes
		FGeometryCollectionAutoInstanceMesh NewInstanceMesh;
		NewInstanceMesh.Mesh = StaticMesh;
		NewInstanceMesh.Materials = SourceMaterials;
		OutInstancedMeshes.Emplace(NewInstanceMesh);

		bool bAddInternalMaterials = false;

		NewGeometryCollection->GeometrySource.Emplace(SourceSoftObjectPath, MeshTransform, SourceMaterials, bSplitComponents, bSetInternalFromMaterialIndex);
		FGeometryCollectionEngineConversion::AppendStaticMesh(StaticMesh, SourceMaterials, MeshTransform, NewGeometryCollection, false, bAddInternalMaterials, bSplitComponents, bSetInternalFromMaterialIndex);

		// make sure we have only one root if we split components
		TSharedPtr<FGeometryCollection> OutCollectionPtr = NewGeometryCollection->GetGeometryCollection();
		if (bSplitComponents && FGeometryCollectionClusteringUtility::ContainsMultipleRootBones(OutCollectionPtr.Get()))
		{
			FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(OutCollectionPtr.Get());
			// If we split components, set external collision (if any) on the new root
			TArray<int32> RootBones;
			FGeometryCollectionClusteringUtility::GetRootBones(OutCollectionPtr.Get(), RootBones);
			if (ensure(RootBones.Num() == 1))
			{
				SetExternalCollisions(GetMeshBuildScale3D(*StaticMesh) * MeshTransform.GetScale3D(), StaticMesh->GetBodySetup(), OutCollectionPtr.Get(), RootBones[0]);
			}
		}

		NewGeometryCollection->InitializeMaterials();

		// Materials
		OutMaterialInstances.Append(NewGeometryCollection->Materials);

		OutCollectionPtr->CopyTo(&OutCollection);
	}
#endif //WITH_EDITORONLY_DATA
}

void FGeometryCollectionEngineConversion::ConvertGeometryCollectionToGeometryCollection(const TObjectPtr<UGeometryCollection> InGeometryCollectionAssetPtr, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterial>>& OutMaterials, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes)
{
	TArray<TObjectPtr<UMaterialInterface>> OutMaterialInstances;
	ConvertGeometryCollectionToGeometryCollection(InGeometryCollectionAssetPtr, OutCollection, OutMaterialInstances, OutInstancedMeshes);
	GetMaterialsFromInstances(OutMaterialInstances, OutMaterials);
}

void FGeometryCollectionEngineConversion::ConvertGeometryCollectionToGeometryCollection(const TObjectPtr<UGeometryCollection> InGeometryCollectionAssetPtr, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterialInterface>>& OutMaterialInstances, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes)
{
	if (InGeometryCollectionAssetPtr)
	{
		const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> NewGeometryCollectionPtr = InGeometryCollectionAssetPtr->GetGeometryCollection();

		// Materials
		OutMaterialInstances = InGeometryCollectionAssetPtr->Materials;

		// InstanceMeshes
		OutInstancedMeshes = InGeometryCollectionAssetPtr->AutoInstanceMeshes;
			
		if (NewGeometryCollectionPtr)
		{
			NewGeometryCollectionPtr->CopyTo(&OutCollection);
		}
	}
}


void FGeometryCollectionEngineConversion::ConvertActorToGeometryCollection(const AActor* Actor, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterial>>& OutMaterials, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes, const FSkeletalMeshToCollectionConversionParameters& ConversionParameters, bool bSplitComponents)
{
	TArray<TObjectPtr<UMaterialInterface>> OutMaterialInstances;
	ConvertActorToGeometryCollection(Actor, OutCollection, OutMaterialInstances, OutInstancedMeshes, ConversionParameters, bSplitComponents);
	GetMaterialsFromInstances(OutMaterialInstances, OutMaterials);
}

void FGeometryCollectionEngineConversion::ConvertActorToGeometryCollection(const AActor* Actor, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterialInterface>>& OutMaterialInstances, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes, const FSkeletalMeshToCollectionConversionParameters& ConversionParameters, bool bSplitComponents)
{
#if WITH_EDITORONLY_DATA
	const FTransform ActorTransform(Actor->GetTransform());

	if (UGeometryCollection* NewGeometryCollection = NewObject<UGeometryCollection>())
	{
		TInlineComponentArray<UStaticMeshComponent*> StaticMeshComponents(Actor);
		for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
		{
			if (StaticMeshComponent)
			{
				if (const UStaticMesh* ComponentStaticMesh = StaticMeshComponent->GetStaticMesh())
				{
					NewGeometryCollection->EnableNanite |= ComponentStaticMesh->IsNaniteEnabled();

					FTransform ComponentTransform(StaticMeshComponent->GetComponentTransform());
					ComponentTransform.SetTranslation((ComponentTransform.GetTranslation() - ActorTransform.GetTranslation()));

					// Record the contributing source on the asset.
					FSoftObjectPath SourceSoftObjectPath(ComponentStaticMesh);
					TArray<TObjectPtr<UMaterialInterface>> SourceMaterials(StaticMeshComponent->GetMaterials());

					NewGeometryCollection->GeometrySource.Emplace(SourceSoftObjectPath, ComponentTransform, SourceMaterials, bSplitComponents, true/*bSetInternalFromMaterialIndex*/);

					FGeometryCollectionEngineConversion::AppendStaticMesh(ComponentStaticMesh, SourceMaterials, ComponentTransform, NewGeometryCollection, false/*bReindexMaterials*/, false/*bAddInternalMaterials*/, bSplitComponents, true/*bSetInternalFromMaterialIndex*/);
				}
			}
		}

		TInlineComponentArray<UGeometryCollectionComponent*> GeometryCollectionComponents(Actor);
		for (UGeometryCollectionComponent* GeometryCollectionComponent : GeometryCollectionComponents)
		{
			if (GeometryCollectionComponent)
			{
				if (const UGeometryCollection* RestCollection = GeometryCollectionComponent->GetRestCollection())
				{
					NewGeometryCollection->EnableNanite |= RestCollection->EnableNanite;

					FTransform ComponentTransform(GeometryCollectionComponent->GetComponentTransform());
					ComponentTransform.SetTranslation((ComponentTransform.GetTranslation() - ActorTransform.GetTranslation()));

					// Record the contributing source on the asset.
					FSoftObjectPath SourceSoftObjectPath(RestCollection);

					int32 NumMaterials = GeometryCollectionComponent->GetNumMaterials();
					TArray<TObjectPtr<UMaterialInterface>> SourceMaterials;
					SourceMaterials.SetNum(NumMaterials);
					for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
					{
						SourceMaterials[MaterialIndex] = GeometryCollectionComponent->GetMaterial(MaterialIndex);
					}
					NewGeometryCollection->GeometrySource.Emplace(SourceSoftObjectPath, ComponentTransform, SourceMaterials, bSplitComponents, true/*bSetInternalFromMaterialIndex*/);

					FGeometryCollectionEngineConversion::AppendGeometryCollection(RestCollection, GeometryCollectionComponent, ComponentTransform, NewGeometryCollection, false /*bReindexMaterials*/);
				}
			}
		}

		NewGeometryCollection->InitializeMaterials();

		if (ConversionParameters.bParentAllBonesUnderNewRoot && FGeometryCollectionClusteringUtility::ContainsMultipleRootBones(NewGeometryCollection->GetGeometryCollection().Get()))
		{
			UE_LOGF(UGeometryCollectionConversionLogging, Log, "FGeometryCollectionEngineConversion::ConvertActorToGeometryCollection() - All bones were parented under new root.");

			FGeometryCollectionClusteringUtility::ClusterAllBonesUnderNewRoot(NewGeometryCollection->GetGeometryCollection().Get(), FName("root"), false);
		}

		// InstanceMeshes
		OutInstancedMeshes.Append(NewGeometryCollection->AutoInstanceMeshes);

		// Materials
		OutMaterialInstances = NewGeometryCollection->Materials;

		TSharedPtr<FGeometryCollection> OutCollectionPtr = NewGeometryCollection->GetGeometryCollection();
		OutCollectionPtr->CopyTo(&OutCollection);
	}
#endif //WITH_EDITORONLY_DATA
}


void FGeometryCollectionEngineConversion::ConvertCollectionToSkeleton(const FManagedArrayCollection& InCollection, USkeleton* OutSkeleton, TArray<int32>& OutIndexRemap, bool bCenterBonesOnGeometry)
{
	//clear the existing skeleton data 
	const TArray<FMeshBoneInfo> BoneInfos = OutSkeleton->GetReferenceSkeleton().GetRefBoneInfo();
	if (BoneInfos.Num())
	{
		int32 RootIndex = 0;
		while (BoneInfos[RootIndex].ParentIndex != INDEX_NONE)
		{
			RootIndex = BoneInfos[RootIndex].ParentIndex;
		}
		if (RootIndex != INDEX_NONE)
		{
			FReferenceSkeletonModifier Edit(OutSkeleton);
			Edit.Remove(BoneInfos[RootIndex].Name, /* bRemoveChildren */ true);
		}
	}

	FManagedArrayCollection LocalCollection = InCollection;
	GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(LocalCollection);
	
	const GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
	if (!BoundsFacade.IsValid())
	{
		UE_LOGF(UGeometryCollectionConversionLogging, Error, "FGeometryCollectionEngineConversion::ConvertCollectionToSkeleton() - Bounds Facade not valid - check the schema of the collection");
		return;
	}

	const TManagedArrayAccessor<int32> SimulationTypeAttribute(InCollection, "SimulationType", FTransformCollection::TransformGroup);
	const TManagedArray<int32>& TransformToGeometryIndexAttribute = BoundsFacade.GetTransformToGeometryIndex();

	auto GetGeometryCenter = [&TransformToGeometryIndexAttribute, &SimulationTypeAttribute, &BoundsFacade, &bCenterBonesOnGeometry](int32 TransformIndex)
		{
			FVector3f GeometryCenter = FVector3f::ZeroVector;
			if (bCenterBonesOnGeometry)
			{
				const bool bIsLeaf = (SimulationTypeAttribute.IsValid() && SimulationTypeAttribute.IsValidIndex(TransformIndex)) ? SimulationTypeAttribute[TransformIndex] == FGeometryCollection::ESimulationTypes::FST_Rigid : true;
				if (bIsLeaf && TransformToGeometryIndexAttribute.IsValidIndex(TransformIndex))
				{
					const int32 GeometryIndex = TransformToGeometryIndexAttribute[TransformIndex];
					if (GeometryIndex != INDEX_NONE)
					{
						GeometryCenter = FVector3f(BoundsFacade.GetBoundingBoxes()[GeometryIndex].GetCenter());
					}
				}
			}
			return GeometryCenter;
		};

	if (TransformFacade.IsValid() && TransformFacade.HasBoneNameAttribute() && OutSkeleton)
	{
		TransformFacade.EnforceSingleRoot("root");

		const TManagedArray<FTransform3f>* Transforms = TransformFacade.FindTransforms();
		const TManagedArray<FString>* BoneNames = TransformFacade.FindBoneNames();
		ensure(Transforms && BoneNames);

		OutIndexRemap.Init(INDEX_NONE, TransformFacade.Num());
		
		auto AddBoneToSkeleton = [&BoneNames, &Transforms, &OutSkeleton, &OutIndexRemap, GetGeometryCenter](int32 TransformIndex, int32 ParentBoneIndex)
			{
				const FName BoneName((*BoneNames)[TransformIndex]);
				const FTransform3f GeomtryCenterOffsetTransform(GetGeometryCenter(TransformIndex));
				const FTransform Transform(GeomtryCenterOffsetTransform * (*Transforms)[TransformIndex]);
				FMeshBoneInfo Info(BoneName, BoneName.ToString(), ParentBoneIndex);
				{
					FReferenceSkeletonModifier Edit(OutSkeleton);
					if (Edit.FindBoneIndex(BoneName) == INDEX_NONE) //Bone does not exist
					{
						Edit.Add(Info, Transform, true /*bAllowMultipleRoots*/);
					}
				}
				OutIndexRemap[TransformIndex] = OutSkeleton->GetReferenceSkeleton().GetNum();
			};


		auto AddChildren = [&TransformFacade, &OutSkeleton, &AddBoneToSkeleton](const TArray<int32>& CollectionChildren)
		{
			TQueue<int32> Children;
			auto Enqueue = [&Children](const TArray<int32>& List)
			{
				for (int32 Elem : List)
					Children.Enqueue(Elem);
			};

			int CurrentIndex = INDEX_NONE;
			Enqueue(CollectionChildren);
			while (!Children.IsEmpty())
			{
				Children.Dequeue(CurrentIndex);

				int32 CollectionParentIndex = (*TransformFacade.GetParents())[CurrentIndex];
				int32 SkeletionParentIndex = OutSkeleton->GetReferenceSkeleton().FindBoneIndex(FName((*TransformFacade.FindBoneNames())[CollectionParentIndex]));

				AddBoneToSkeleton(CurrentIndex, SkeletionParentIndex);

				Enqueue((*TransformFacade.FindChildren())[CurrentIndex].Array());
			}
		};


		// must insert in decending order from parent to child. 
		TArray<FString> BoneNameStrings = TransformFacade.FindBoneNames()->GetConstArray();
		for (int i = 0; i < TransformFacade.GetParents()->Num(); i++)
		{
			if ((*TransformFacade.GetParents())[i] == INDEX_NONE) // No parent
			{
				AddBoneToSkeleton(i, INDEX_NONE);

				if ((*TransformFacade.FindChildren())[i].Num())
				{
					AddChildren((*TransformFacade.FindChildren())[i].Array());
				}
			}
		}

		TransformFacade.EnforceSingleRoot("root");
		UE_LOGF(UGeometryCollectionConversionLogging, Log, "FGeometryCollectionEngineConversion::ConvertCollectionToSkeleton(NumTransforms:%d)", OutSkeleton->GetReferenceSkeleton().GetRawBoneNum());
	}
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
void FGeometryCollectionEngineConversion::ConvertCollectionToSkeletalMesh(const FManagedArrayCollection& InCollection, const TArray<TObjectPtr<UMaterialInterface>> InMaterials, USkeletalMesh& OutSkeletalMesh, USkeleton& OutSkeleton)
{
#if WITH_EDITOR
	// make sure we have all the necessary facade supported 
	const GeometryCollection::Facades::FCollectionTransformFacade TransformFacade(InCollection);
	if (!TransformFacade.IsValid())
	{
		UE_LOGF(UGeometryCollectionConversionLogging, Error, "FGeometryCollectionEngineConversion::ConvertCollectionToSkeletalMesh() - Transform Facade not valid - check the schema of the collection");
		return;
	}

	const GeometryCollection::Facades::FCollectionMeshFacade MeshFacade(InCollection);
	if (!MeshFacade.IsValid())
	{
		UE_LOGF(UGeometryCollectionConversionLogging, Error, "FGeometryCollectionEngineConversion::ConvertCollectionToSkeletalMesh() - Mesh Facade not valid - check the schema of the collection");
		return;
	}

	const GeometryCollection::Facades::FBoundsFacade BoundsFacade(InCollection);
	if (!BoundsFacade.IsValid())
	{
		UE_LOGF(UGeometryCollectionConversionLogging, Error, "FGeometryCollectionEngineConversion::ConvertCollectionToSkeletalMesh() - Bounds Facade not valid - check the schema of the collection");
		return;
	}

	const TManagedArrayAccessor<int32> SimulationTypeAttribute(InCollection, "SimulationType", FTransformCollection::TransformGroup);
	const TManagedArray<FString>* Names = TransformFacade.FindBoneNames();
	const TManagedArray<int32>* Parents = TransformFacade.GetParents();
	const TManagedArray<FTransform3f>* Transforms = TransformFacade.FindTransforms();

	// Reset the skeletal mesh
	OutSkeletalMesh.SetNumSourceModels(0);
	OutSkeletalMesh.GetImportedModel()->LODModels.Reset();
	OutSkeletalMesh.ReleaseResources();

	// first extract the skeleton 
	TArray<int32> TransformToBoneIndexMapping;
	FGeometryCollectionEngineConversion::ConvertCollectionToSkeleton(InCollection, &OutSkeleton, TransformToBoneIndexMapping, /* bCenterBonesOnGeometry */true);

	// convert the collection to a mesh description including skeletal mesh attributes
	FMeshDescription MeshDescription;

	FSkeletalMeshAttributes Attributes(MeshDescription);
	Attributes.Register();

	const int32 NumUVChannels = MeshFacade.GetNumUVLayers();

	MeshDescription.SetNumUVChannels(MeshFacade.GetNumUVLayers());
	MeshDescription.VertexInstanceAttributes().SetAttributeChannelCount(MeshAttribute::VertexInstance::TextureCoordinate, NumUVChannels);

	TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
	FSkinWeightsVertexAttributesRef VertexSkinWeights = Attributes.GetVertexSkinWeights();

	TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = Attributes.GetVertexInstanceNormals();
	TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = Attributes.GetVertexInstanceTangents();
	TVertexInstanceAttributesRef<float> VertexInstanceBinormalSigns = Attributes.GetVertexInstanceBinormalSigns();
	TVertexInstanceAttributesRef<FVector4f> VertexInstanceColors = Attributes.GetVertexInstanceColors();
	TVertexInstanceAttributesRef<FVector2f> VertexInstanceUVs = Attributes.GetVertexInstanceUVs();

	FSkeletalMeshAttributes::FBoneNameAttributesRef BoneNames = Attributes.GetBoneNames();
	FSkeletalMeshAttributes::FBoneParentIndexAttributesRef BoneParentIndices = Attributes.GetBoneParentIndices();
	FSkeletalMeshAttributes::FBonePoseAttributesRef BonePoses = Attributes.GetBonePoses();

	// make sure we create a polygroup per material and that they have properly defined slot names
	int32 MaxMaterialID = 0;
	for (int32 FaceIndex = 0; FaceIndex < MeshFacade.MaterialIDAttribute.Num(); ++FaceIndex)
	{
		MaxMaterialID = FMath::Max(MaxMaterialID, MeshFacade.MaterialIDAttribute[FaceIndex]);
	}
	const int32 MaterialCount = (MaxMaterialID + 1);

	TPolygonGroupAttributesRef<FName> PolygonGroupMaterialSlotNames = Attributes.GetPolygonGroupMaterialSlotNames();

	TArray<FSkeletalMaterial> SkeletalMaterials;
	TArray<FPolygonGroupID> MatIDToPolyGroupID;
	MatIDToPolyGroupID.SetNum(MaterialCount);
	for (int32 MatIndex = 0; MatIndex < MaterialCount; ++MatIndex)
	{ 
		const FPolygonGroupID PolygonGroupID = MeshDescription.CreatePolygonGroup();
		MatIDToPolyGroupID[MatIndex] = PolygonGroupID;
		const FName SlotName = FName("Material", MatIndex);
		PolygonGroupMaterialSlotNames.Set(PolygonGroupID, SlotName);

		FSkeletalMaterial NewMaterial;
		NewMaterial.MaterialInterface = InMaterials.IsValidIndex(MatIndex) ? InMaterials[MatIndex] : nullptr;
		NewMaterial.MaterialSlotName = SlotName;
		NewMaterial.ImportedMaterialSlotName = SlotName;
		SkeletalMaterials.Add(NewMaterial);
	}

	int32 NewVtxId = 0;

	// convert collection geometry to mesh decsription
	const int32 NumTransforms = InCollection.NumElements(FTransformCollection::TransformGroup);
	for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
	{
		const int32 GeometryIndex = MeshFacade.TransformToGeometryIndexAttribute[TransformIndex];
		const bool bIsLeaf = (SimulationTypeAttribute.IsValid() && SimulationTypeAttribute.IsValidIndex(TransformIndex)) ? SimulationTypeAttribute[TransformIndex] == FGeometryCollection::ESimulationTypes::FST_Rigid : true;
		if (bIsLeaf && GeometryIndex != INDEX_NONE)
		{
			// make sure we operate in the right space
			const FTransform3f Transform = FTransform3f(TransformFacade.ComputeCollectionSpaceTransform(TransformIndex));

			const int32 VertexStart = MeshFacade.VertexStartAttribute[GeometryIndex];
			const int32 VertexCount = MeshFacade.VertexCountAttribute[GeometryIndex];

			MeshDescription.ReserveNewVertices(VertexCount);
			MeshDescription.ReserveNewVertexInstances(VertexCount);
			for (int32 UVChannelIndex = 0; UVChannelIndex < NumUVChannels; ++UVChannelIndex)
			{
				MeshDescription.ReserveNewUVs(VertexCount, UVChannelIndex);
			}

			const int32 VtxIndexToVtxId = (NewVtxId - VertexStart);

			for (int32 VtxIndex = VertexStart; VtxIndex < (VertexStart + VertexCount); ++VtxIndex)
			{
				const FVector3f VtxPos = Transform.TransformPosition(MeshFacade.VertexAttribute[VtxIndex]);
				const FVector3f VtxNormal = Transform.TransformVector(MeshFacade.NormalAttribute[VtxIndex]);
				const FVector3f VtxTangentU = Transform.TransformVector(MeshFacade.TangentUAttribute[VtxIndex]);
				const FVector3f VtxTangentV = Transform.TransformVector(MeshFacade.TangentVAttribute[VtxIndex]);
				const FLinearColor& VtxColor = MeshFacade.ColorAttribute[VtxIndex];

				const FVertexID VertexID(NewVtxId);
				const FVertexInstanceID VertexInstanceID(NewVtxId);

				MeshDescription.CreateVertexWithID(VertexID);
				MeshDescription.CreateVertexInstanceWithID(VertexInstanceID, VertexID);

				VertexPositions.Set(VertexID, VtxPos);
				VertexInstanceNormals.Set(VertexInstanceID, VtxNormal);
				VertexInstanceTangents.Set(VertexInstanceID, VtxTangentU);
				const float BinormalSign = GetBasisDeterminantSign(FVector(VtxTangentU), FVector(VtxTangentV), FVector(VtxNormal));
				VertexInstanceBinormalSigns.Set(VertexInstanceID, BinormalSign);
				VertexInstanceColors.Set(VertexInstanceID, FVector4f(VtxColor));
				for (int32 UVChannelIndex = 0; UVChannelIndex < NumUVChannels; ++UVChannelIndex)
				{
					const FVector2f& UV = MeshFacade.GetUV(VtxIndex, UVChannelIndex);
					VertexInstanceUVs.Set(VertexInstanceID, UVChannelIndex, UV);
				}

				// geometry collection are rigidly skinned to bones
				FBoneIndexType InfluenceBones[MAX_TOTAL_INFLUENCES];
				uint16 InfluenceWeights[MAX_TOTAL_INFLUENCES] = { 0 };

				const int32 BoneIndex = (TransformToBoneIndexMapping.IsValidIndex(TransformIndex)) ? TransformToBoneIndexMapping[TransformIndex] - 1 : 0;
				InfluenceBones[0] = BoneIndex;
				InfluenceWeights[0] = std::numeric_limits<uint16>::max();

				VertexSkinWeights.Set(VertexID, UE::AnimationCore::FBoneWeights::Create(InfluenceBones, InfluenceWeights));

				++NewVtxId;
			}

			const int32 FaceStart = MeshFacade.FaceStartAttribute[GeometryIndex];
			const int32 FaceCount = MeshFacade.FaceCountAttribute[GeometryIndex];

			MeshDescription.ReserveNewTriangles(FaceCount);

			for (int32 TriIndex = FaceStart; TriIndex < FaceStart + FaceCount; ++TriIndex)
			{
				const FIntVector& VtxIndices = MeshFacade.IndicesAttribute[TriIndex];
				const int32 Vtx0 = VtxIndices.X + VtxIndexToVtxId;
				const int32 Vtx1 = VtxIndices.Y + VtxIndexToVtxId;
				const int32 Vtx2 = VtxIndices.Z + VtxIndexToVtxId;

				const FVertexInstanceID TriVertInstanceIDs[] = { FVertexInstanceID(Vtx0), FVertexInstanceID(Vtx1), FVertexInstanceID(Vtx2) };
				const int32 MaterialID = MeshFacade.MaterialIDAttribute[TriIndex];
				MeshDescription.CreateTriangle(MatIDToPolyGroupID[MaterialID], MakeConstArrayView(TriVertInstanceIDs, 3));
			}
		}
	
		// find corresponding bone from skeleton 
		const FName BoneName = FName((*Names)[TransformIndex]);
		const int32 BoneIndex = OutSkeleton.GetReferenceSkeleton().FindBoneIndex(BoneName);
		if (BoneIndex != INDEX_NONE)
		{
			const int32 ParentBoneIndex = OutSkeleton.GetReferenceSkeleton().GetParentIndex(BoneIndex);

			const TArray<FTransform> BoneTransforms = OutSkeleton.GetReferenceSkeleton().GetRefBonePose();
			const FTransform BoneTransform = (BoneTransforms.IsValidIndex(BoneIndex)) ? BoneTransforms[BoneIndex] : FTransform::Identity;

			const FBoneID BoneID(BoneIndex);

			Attributes.CreateBone(BoneIndex);
			{
				BoneNames.Set(BoneID, BoneName);
				BoneParentIndices.Set(BoneID, ParentBoneIndex);
				BonePoses.Set(BoneID, BoneTransform);
			}
		}
	}

	TArray<const FMeshDescription*> MeshDescriptions = { &MeshDescription };

	// Update the skeletal mesh from the description
	FStaticToSkeletalMeshConverter::FInitializationParams Parameters;
	Parameters.Materials = SkeletalMaterials;
	Parameters.bRecomputeNormals = false;
	Parameters.bRecomputeTangents = false;
	
	FStaticToSkeletalMeshConverter::InitializeSkeletalMeshFromMeshDescriptions(
		&OutSkeletalMesh,
		MeshDescriptions,
		OutSkeleton.GetReferenceSkeleton(),
		Parameters);

	// finalize skeletal mesh binding to skeleton 
	OutSkeletalMesh.SetSkeleton(&OutSkeleton);
	OutSkeleton.SetPreviewMesh(&OutSkeletalMesh);
	OutSkeleton.RecreateBoneTree(&OutSkeletalMesh);

#else
	UE_LOGF(UGeometryCollectionConversionLogging, Error, "FGeometryCollectionEngineConversion::ConvertCollectionToSkeletalMesh() - conversion only supported in editor.");
#endif // WITH_EDITOR
}

void FGeometryCollectionEngineConversion::GetMaterialsFromInstances(const TArray<TObjectPtr<UMaterialInterface>>& MaterialInstances, TArray<TObjectPtr<UMaterial>>& OutMaterials)
{
	// somehow append does not like appending to a TArray<ObjectPtr<>>
	// so we need to store in a array of raw pointers and then transfer over 
	TArray<UMaterial*> MaterialArray;
	MaterialArray.Append(MaterialInstances);
	OutMaterials.Reserve(MaterialInstances.Num());
	for (UMaterial* Material : MaterialArray)
	{
		OutMaterials.Add(Material);
	}
}

#undef LOCTEXT_NAMESPACE 
