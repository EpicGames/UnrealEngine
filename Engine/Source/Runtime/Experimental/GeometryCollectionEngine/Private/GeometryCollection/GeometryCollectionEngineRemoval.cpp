// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionEngineRemoval.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/Facades/CollectionRemoveOnBreakFacade.h"
#include "GeometryCollectionProxyData.h"

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

namespace
{
	const FName BreakTimerAttributeName = "BreakTimer";
	const FName PostBreakDurationAttributeName = "PostBreakDuration";
	const FName BreakRemovalDurationAttributeName = "BreakRemovalDuration";

	const FName SleepTimerAttributeName = "SleepTimer";
	const FName MaxSleepTimeAttributeName = "MaxSleepTime";
	const FName SleepRemovalDurationAttributeName = "SleepRemovalDuration";
	const FName LastPositionAttributeName = "LastPosition";

	const FName DecayAttributeName = "Decay";
}

FGeometryCollectionRemoveOnBreakDynamicFacade::FGeometryCollectionRemoveOnBreakDynamicFacade(FGeometryDynamicCollection& InCollection)
	: BreakTimerAttribute(InCollection, BreakTimerAttributeName, FGeometryCollection::TransformGroup)
	, PostBreakDurationAttribute(InCollection, PostBreakDurationAttributeName, FGeometryCollection::TransformGroup)
	, BreakRemovalDurationAttribute(InCollection, BreakRemovalDurationAttributeName, FGeometryCollection::TransformGroup)
	, DynamicCollection(InCollection)
{
}

bool FGeometryCollectionRemoveOnBreakDynamicFacade::IsValid() const
{
	return BreakTimerAttribute.IsValid()
		&& PostBreakDurationAttribute.IsValid()
		&& BreakRemovalDurationAttribute.IsValid();
}

bool FGeometryCollectionRemoveOnBreakDynamicFacade::IsConst() const
{
	return BreakTimerAttribute.IsConst();
}

void FGeometryCollectionRemoveOnBreakDynamicFacade::DefineSchema()
{
	check(!IsConst());
	BreakTimerAttribute.AddAndFill(BreakTimerStartValue);
	PostBreakDurationAttribute.Add();
	BreakRemovalDurationAttribute.Add();
}

void FGeometryCollectionRemoveOnBreakDynamicFacade::SetAttributeValues(const GeometryCollection::Facades::FCollectionRemoveOnBreakFacade& RemoveOnBreakFacade)
{
	check(!IsConst());
	if (IsValid())
	{
		BreakTimerAttribute.Fill(BreakTimerStartValue);

		// make sure we generate random value consistently between client and server 
		// we can use the length of the transform group for that
		const int32 RandomSeed = DynamicCollection.GetNumTransforms();
		FRandomStream Random(RandomSeed);

		TManagedArray<float>& PostBreakDuration = PostBreakDurationAttribute.Modify();
		for (int32 Idx = 0; Idx < PostBreakDuration.Num(); ++Idx)
		{
			const GeometryCollection::Facades::FRemoveOnBreakData RemoveOnBreakData{ RemoveOnBreakFacade.GetData(Idx) };
			const FVector2f PostBreakTimer = RemoveOnBreakData.GetBreakTimer();
			const float MinBreakTime = FMath::Max(0.0f, PostBreakTimer.X);
			const float MaxBreakTime = FMath::Max(MinBreakTime, PostBreakTimer.Y);
			PostBreakDuration[Idx] = RemoveOnBreakData.IsEnabled() ? Random.FRandRange(MinBreakTime, MaxBreakTime) : DisabledPostBreakDuration;
		}

		TManagedArray<float>& BreakRemovalDuration = BreakRemovalDurationAttribute.Modify();
		for (int32 Idx = 0; Idx < PostBreakDuration.Num(); ++Idx)
		{
			const GeometryCollection::Facades::FRemoveOnBreakData RemoveOnBreakData{ RemoveOnBreakFacade.GetData(Idx) };
			const FVector2f RemovalTimer = RemoveOnBreakData.GetRemovalTimer();
			const float MinRemovalTime = FMath::Max(0.0f, RemovalTimer.X);
			const float MaxRemovalTime = FMath::Max(MinRemovalTime, RemovalTimer.Y);
			const bool bIsCluster = DynamicCollection.IsCluster(Idx);
			const bool bUseClusterCrumbling = (bIsCluster && RemoveOnBreakData.GetClusterCrumbling());
			BreakRemovalDuration[Idx] = bUseClusterCrumbling ? CrumblingRemovalTimer : Random.FRandRange(MinRemovalTime, MaxRemovalTime);
		}
	}
}

bool FGeometryCollectionRemoveOnBreakDynamicFacade::IsRemovalActive(int32 TransformIndex) const
{
	return (PostBreakDurationAttribute.Get()[TransformIndex] > DisabledPostBreakDuration);
}

bool FGeometryCollectionRemoveOnBreakDynamicFacade::UseClusterCrumbling(int32 TransformIndex) const
{
	return (BreakRemovalDurationAttribute.Get()[TransformIndex] <= CrumblingRemovalTimer);
}

float FGeometryCollectionRemoveOnBreakDynamicFacade::UpdateBreakTimerAndComputeDecay(int32 TransformIndex, float DeltaTime)
{
	float& BreakTimerRef = BreakTimerAttribute.Modify()[TransformIndex];
	const float PostBreakDuration = PostBreakDurationAttribute.Get()[TransformIndex];
	const float BreakRemovalDuration = BreakRemovalDurationAttribute.Get()[TransformIndex];

	float UpdatedBreakDecay = 0;
	if (PostBreakDuration >= 0)
	{
		BreakTimerRef += DeltaTime;
	
		const bool bPostBreakTimeExpired = (BreakTimerRef >= PostBreakDuration) && (PostBreakDuration >= 0);
		const bool bZeroRemovalDuration = (BreakRemovalDuration < UE_SMALL_NUMBER);
		if (bPostBreakTimeExpired)
		{
			UpdatedBreakDecay = bZeroRemovalDuration? 1.f: FMath::Clamp<float>((BreakTimerRef - PostBreakDuration) / BreakRemovalDuration, 0.f, 1.f);
		}
	}
	return UpdatedBreakDecay;
}

///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FGeometryCollectionRemoveOnSleepDynamicFacade::FGeometryCollectionRemoveOnSleepDynamicFacade(FManagedArrayCollection& InCollection)
	: SleepTimerAttribute(InCollection, SleepTimerAttributeName, FGeometryCollection::TransformGroup)
	, MaxSleepTimeAttribute(InCollection, MaxSleepTimeAttributeName, FGeometryCollection::TransformGroup)
	, SleepRemovalDurationAttribute(InCollection, SleepRemovalDurationAttributeName, FGeometryCollection::TransformGroup)
	, LastPositionAttribute(InCollection, LastPositionAttributeName, FGeometryCollection::TransformGroup)
{
}

bool FGeometryCollectionRemoveOnSleepDynamicFacade::IsValid() const
{
	return SleepTimerAttribute.IsValid()
		&& MaxSleepTimeAttribute.IsValid()
		&& SleepRemovalDurationAttribute.IsValid()
		&& LastPositionAttribute.IsValid()
	;
}

bool FGeometryCollectionRemoveOnSleepDynamicFacade::IsConst() const
{
	return SleepTimerAttribute.IsConst();
}

void FGeometryCollectionRemoveOnSleepDynamicFacade::DefineSchema()
{
	check(!IsConst());
	SleepTimerAttribute.Add();
	MaxSleepTimeAttribute.Add();
	SleepRemovalDurationAttribute.Add();
	LastPositionAttribute.AddAndFill(FVector::ZeroVector);
}

float FGeometryCollectionRemoveOnSleepDynamicFacade::GetSleepRemovalDuration(int32 TransformIndex) const
{
	return SleepRemovalDurationAttribute[TransformIndex];
}

float FGeometryCollectionRemoveOnSleepDynamicFacade::GetMaxSleepTime(int32 TransformIndex) const
{
	return MaxSleepTimeAttribute[TransformIndex];
}

void FGeometryCollectionRemoveOnSleepDynamicFacade::SetSleepRemovalDuration(int32 TransformIndex, float SleepRemovalDuration)
{
	SleepRemovalDurationAttribute.ModifyAt(TransformIndex, SleepRemovalDuration);
}

void FGeometryCollectionRemoveOnSleepDynamicFacade::SetMaxSleepTime(int32 TransformIndex, float MaxSleepTime)
{
	MaxSleepTimeAttribute.ModifyAt(TransformIndex, MaxSleepTime);
}

void FGeometryCollectionRemoveOnSleepDynamicFacade::SetAttributeValues(const FVector2D& MaximumSleepTime, const FVector2D& RemovalDuration)
{
	check(!IsConst());
	if (IsValid())
	{
		SleepTimerAttribute.Fill(0.0f);
		LastPositionAttribute.Fill(FVector::ZeroVector);

		TManagedArray<float>& MaxSleepTime = MaxSleepTimeAttribute.Modify();
		const float MinTime = FMath::Max(0.0f, MaximumSleepTime.X);
		const float MaxTime = FMath::Max(MinTime, MaximumSleepTime.Y);
		for (int32 Idx = 0; Idx < MaxSleepTime.Num(); ++Idx)
		{
			MaxSleepTime[Idx] = FMath::RandRange(MinTime, MaxTime);
		}

		TManagedArray<float>& SleepRemovalDuration = SleepRemovalDurationAttribute.Modify();
		float MinRemovalTime = FMath::Max(0.0f, RemovalDuration.X);
		float MaxRemovalTime = FMath::Max(MinRemovalTime, RemovalDuration.Y);
		for (int32 Idx = 0; Idx < SleepRemovalDuration.Num(); ++Idx)
		{
			SleepRemovalDuration[Idx] = FMath::RandRange(MinRemovalTime, MaxRemovalTime);
		}
	}
}

bool FGeometryCollectionRemoveOnSleepDynamicFacade::IsRemovalActive(int32 TransformIndex) const
{
	return (MaxSleepTimeAttribute[TransformIndex] >= 0);
}

bool FGeometryCollectionRemoveOnSleepDynamicFacade::ComputeSlowMovingState(int32 TransformIndex, const FVector& Position, float DeltaTime, FVector::FReal VelocityThreshold)
{
	bool IsSlowMoving = false;
	FVector& LastPositionRef = LastPositionAttribute.Modify()[TransformIndex];
	if (DeltaTime > 0)
	{
		const FVector::FReal InstantVelocity = (Position-LastPositionRef).Size() / (FVector::FReal)DeltaTime; 
		IsSlowMoving = (InstantVelocity < VelocityThreshold);
	}
	LastPositionRef = Position;
	return IsSlowMoving;
}

void FGeometryCollectionRemoveOnSleepDynamicFacade::UpdateSleepTimer(int32 TransformIndex, float DeltaTime)
{
	check(SleepTimerAttribute.IsValid());
	SleepTimerAttribute.Modify()[TransformIndex] += DeltaTime;
}

float FGeometryCollectionRemoveOnSleepDynamicFacade::ComputeDecay(int32 TransformIndex) const
{
	const float SleepRemovalDuration = SleepRemovalDurationAttribute.Get()[TransformIndex];
	const float MaxSleepTime = MaxSleepTimeAttribute.Get()[TransformIndex];
	const float SleepTimer = SleepTimerAttribute.Get()[TransformIndex];
	const bool bZeroRemovalDuration = (SleepRemovalDuration < UE_SMALL_NUMBER);
	return bZeroRemovalDuration? 1.f: FMath::Clamp<float>((SleepTimer -MaxSleepTime) / SleepRemovalDuration, 0.f, 1.f);
}


///////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////

FGeometryCollectionDecayDynamicFacade::FGeometryCollectionDecayDynamicFacade(FManagedArrayCollection& InCollection)
	: DecayAttribute(InCollection, DecayAttributeName, FGeometryCollection::TransformGroup)
{
}

bool FGeometryCollectionDecayDynamicFacade::IsValid() const
{
	// on check the decay since the uniform attribute is optional ( see bScaleOnRemoval )
	return DecayAttribute.IsValid();
}

void FGeometryCollectionDecayDynamicFacade::AddAttributes()
{
	DecayAttribute.AddAndFill(0.0f);
}

float FGeometryCollectionDecayDynamicFacade::GetDecay(int32 TransformIndex) const
{
	return DecayAttribute[TransformIndex];
}

void FGeometryCollectionDecayDynamicFacade::SetDecay(int32 TransformIndex, float DecayValue)
{
	DecayAttribute.ModifyAt(TransformIndex, DecayValue);
}

int32 FGeometryCollectionDecayDynamicFacade::GetDecayAttributeSize() const
{
	return DecayAttribute.Num();
}