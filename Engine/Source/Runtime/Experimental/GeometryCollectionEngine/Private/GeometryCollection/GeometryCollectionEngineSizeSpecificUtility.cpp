// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionEngineSizeSpecificUtility.h"

namespace GeometryCollection
{
	namespace SizeSpecific
	{
		bool UsesImplicitCollisionType(const TArray<FGeometryCollectionSizeSpecificData>& SizeSpecificData, EImplicitTypeEnum ImplicitType)
		{
			for (int32 Idx = SizeSpecificData.Num() - 1; Idx >= 0; --Idx)
			{
				for (int Cdx = 0; Cdx < SizeSpecificData[Idx].CollisionShapes.Num(); Cdx++)
				{
					if (SizeSpecificData[Idx].CollisionShapes[Cdx].ImplicitType == ImplicitType)
					{
						return true;
					}
				}
			}
			return false;
		}

		void SetImplicitCollisionType(TArray<FGeometryCollectionSizeSpecificData>& SizeSpecificData, EImplicitTypeEnum ImplicitTypeTo, EImplicitTypeEnum ImplicitTypeFrom)
		{
			for (int32 Idx = SizeSpecificData.Num() - 1; Idx >= 0; --Idx)
			{
				for (int Cdx = 0; Cdx < SizeSpecificData[Idx].CollisionShapes.Num(); Cdx++)
				{
					if (SizeSpecificData[Idx].CollisionShapes[Cdx].ImplicitType == ImplicitTypeFrom ||
						ImplicitTypeFrom == EImplicitTypeEnum::Chaos_Implicit_None)
					{
						SizeSpecificData[Idx].CollisionShapes[Cdx].ImplicitType = ImplicitTypeTo;
					}
				}
			}
		}

	}
}