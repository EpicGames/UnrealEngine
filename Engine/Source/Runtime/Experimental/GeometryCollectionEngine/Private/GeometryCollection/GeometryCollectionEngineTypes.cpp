// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionEngineTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionEngineTypes)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FName GetCollectionAttributeName(ECollectionAttributeEnum TypeIn)
{
	switch (TypeIn)
	{
	case ECollectionAttributeEnum::Chaos_Active:
		return FGeometryDynamicCollection::ActiveAttribute;
	case ECollectionAttributeEnum::Chaos_DynamicState:
		return FGeometryDynamicCollection::DynamicStateAttribute;
	case ECollectionAttributeEnum::Chaos_CollisionGroup:
		return FGeometryDynamicCollection::CollisionGroupAttribute;
	}
	check(false);
	return FName();
}

FName GetCollectionGroupName(ECollectionGroupEnum TypeIn)
{
	switch (TypeIn)
	{
	case ECollectionGroupEnum::Chaos_Traansform:
		return FGeometryCollection::TransformGroup;
	}
	check(false);
	return FName();
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
