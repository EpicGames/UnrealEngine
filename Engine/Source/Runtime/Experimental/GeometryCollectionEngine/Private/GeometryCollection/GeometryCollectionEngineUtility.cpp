// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionEngineUtility.h"

#include "Chaos/Utilities.h"
#include "Engine/SkeletalMesh.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionCache.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "MeshDescription.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


#if WITH_EDITOR
#include "MeshUtilities.h"
#endif

DEFINE_LOG_CATEGORY_STATIC(LogGeoemtryCollectionClean, Verbose, All);


void GeometryCollectionEngineUtility::PrintDetailedStatistics(const FGeometryCollection* GeometryCollection, const UGeometryCollectionCache* InCache)
{
	check(GeometryCollection);

	TSharedPtr< TManagedArray<FTransform> >   Transform;

	int32 NumTransforms = GeometryCollection->NumElements(FGeometryCollection::TransformGroup);
	int32 NumVertices = GeometryCollection->NumElements(FGeometryCollection::VerticesGroup);
	int32 NumFaces = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);
	int32 NumGeometries = GeometryCollection->NumElements(FGeometryCollection::GeometryGroup);
	int32 NumBreakings = GeometryCollection->NumElements(FGeometryCollection::BreakingGroup);

	const TManagedArray<FVector3f>& VertexArray = GeometryCollection->Vertex;
	const TManagedArray<int32>& BoneMapArray = GeometryCollection->BoneMap;

	TArray<FTransform> GlobalTransformArray;
	GeometryCollectionAlgo::GlobalMatrices(GeometryCollection->Transform, GeometryCollection->Parent, GlobalTransformArray);

	FBox BoundingBox(ForceInitToZero);
	for (int32 IdxVertex = 0; IdxVertex < NumVertices; ++IdxVertex)
	{
		FTransform GlobalTransform = GlobalTransformArray[BoneMapArray[IdxVertex]];
		FVector VertexInWorld = GlobalTransform.TransformPosition((FVector)VertexArray[IdxVertex]);

		BoundingBox += VertexInWorld;
	}

	FString Buffer;
	Buffer += FString::Printf(TEXT("\n\n"));
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("TRANSFORM GROUP\n"));
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("Number of transforms = %d\n"), NumTransforms);
	// Print Transform array
	// Print BoneHierarchy array
	GeometryCollectionAlgo::PrintParentHierarchy(GeometryCollection);
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("VERTICES GROUP\n"));
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("Number of vertices = %d\n"), NumVertices);
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("FACES GROUP\n"));
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("Number of faces = %d\n"), NumFaces);
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("GEOMETRY GROUP\n"));
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("Number of geometries = %d\n"), NumGeometries);
	// Print TransformIndex array
	// Print Proximity array
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("BREAKING GROUP\n"));
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("Number of breakings = %d\n"), NumBreakings);
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("BOUNDING BOX\n"));
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("Min = (%f, %f, %f)\n"), BoundingBox.Min.X, BoundingBox.Min.Y, BoundingBox.Min.Z);
	Buffer += FString::Printf(TEXT("Max = (%f, %f, %f)\n"), BoundingBox.Max.X, BoundingBox.Max.Y, BoundingBox.Max.Z);
	Buffer += FString::Printf(TEXT("Center = (%f, %f, %f)\n"), BoundingBox.GetCenter().X, BoundingBox.GetCenter().Y, BoundingBox.GetCenter().Z);
	Buffer += FString::Printf(TEXT("Size = (%f, %f, %f)\n"), 2.f * BoundingBox.GetExtent().X, 2.f * BoundingBox.GetExtent().Y, 2.f * BoundingBox.GetExtent().Z);
	Buffer += FString::Printf(TEXT("Volume = %f\n"), BoundingBox.GetVolume());

	
	if(InCache && InCache->GetData())
	{
		Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
		Buffer += FString::Printf(TEXT("CACHE INFO\n"));
		Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
		const FRecordedTransformTrack* Track = InCache->GetData();
		int32 NumRecords = Track->Records.Num();
		if(NumRecords == 0)
		{
			Buffer += FString::Printf(TEXT("Cache is empty\n"));
		}
		else
		{
			float FirstRecordTimestamp = Track->Records[0].Timestamp;
			float LastRecordTimestamp = Track->Records[NumRecords - 1].Timestamp;

			TMultiMap<int32, int32> TimestampRecordMap;
			for(int32 IdxRecord = 0; IdxRecord < NumRecords; ++IdxRecord)
			{
				TimestampRecordMap.Add(FMath::FloorToInt(Track->Records[IdxRecord].Timestamp), IdxRecord);
			}

			TArray<int32> NumRecordsPerSecond;
			NumRecordsPerSecond.SetNum(FMath::CeilToInt(LastRecordTimestamp));
			for(int32 Idx = 0; Idx < FMath::CeilToInt(LastRecordTimestamp); ++Idx)
			{
				NumRecordsPerSecond[Idx] = TimestampRecordMap.Num(Idx);
			}

			int32 NumRecordsMin = INT_MAX, NumRecordsMax = INT_MIN;
			float NumRecordsAverage = 0.f;
			for(int32 Idx = 0; Idx < NumRecordsPerSecond.Num(); ++Idx)
			{
				if(NumRecordsPerSecond[Idx] < NumRecordsMin)
				{
					NumRecordsMin = NumRecordsPerSecond[Idx];
				}
				if(NumRecordsPerSecond[Idx] > NumRecordsMax)
				{
					NumRecordsMax = NumRecordsPerSecond[Idx];
				}
				NumRecordsAverage += (float)NumRecordsPerSecond[Idx];
			}
			NumRecordsAverage /= (float)NumRecordsPerSecond.Num();

			Buffer += FString::Printf(TEXT("Cache length [%f - %f]\n"), FirstRecordTimestamp, LastRecordTimestamp);
			Buffer += FString::Printf(TEXT("Number of recorded frames = %d\n"), NumRecords);
			for(int32 Idx = 0; Idx < NumRecordsPerSecond.Num(); ++Idx)
			{
				Buffer += FString::Printf(TEXT("Number of recorded frames at %ds = %d\n"), Idx, NumRecordsPerSecond[Idx]);
			}
			Buffer += FString::Printf(TEXT("Minimum number of recorded frames per second = %d\n"), NumRecordsMin);
			Buffer += FString::Printf(TEXT("Maximum number of recorded frames per second = %d\n"), NumRecordsMax);
			Buffer += FString::Printf(TEXT("Average number of recorded frames per second = %f\n"), NumRecordsAverage);

			TArray<int32> NumCollisionMinPerSecond;
			NumCollisionMinPerSecond.Init(0, FMath::CeilToInt(LastRecordTimestamp));
			TArray<int32> NumCollisionMaxPerSecond;
			NumCollisionMaxPerSecond.Init(0, FMath::CeilToInt(LastRecordTimestamp));
			TArray<float> NumCollisionAveragePerSecond;
			NumCollisionAveragePerSecond.Init(0.f, FMath::CeilToInt(LastRecordTimestamp));
			int32 NumTotalCollisions = 0;
			TArray<int32> RecordIdxForOneSecond;
			for(int32 IdxSeconds = 0; IdxSeconds < NumRecordsPerSecond.Num(); ++IdxSeconds)
			{
				RecordIdxForOneSecond.Empty();
				TimestampRecordMap.MultiFind(IdxSeconds, RecordIdxForOneSecond);

				for(int32 IdxRecord = 0; IdxRecord < RecordIdxForOneSecond.Num(); ++IdxRecord)
				{
					int32 NumCollisions = Track->Records[RecordIdxForOneSecond[IdxRecord]].Collisions.Num();
					if(NumCollisions > 0)
					{
						if(NumCollisions < NumCollisionMinPerSecond[IdxSeconds])
						{
							NumCollisionMinPerSecond[IdxSeconds] = NumCollisions;
						}
						if(NumCollisions > NumCollisionMaxPerSecond[IdxSeconds])
						{
							NumCollisionMaxPerSecond[IdxSeconds] = NumCollisions;
						}
						NumCollisionAveragePerSecond[IdxSeconds] += NumCollisions;
						NumTotalCollisions += NumCollisions;
					}
				}
				NumCollisionAveragePerSecond[IdxSeconds] /= (float)RecordIdxForOneSecond.Num();
			}

			Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
			for(int32 Idx = 0; Idx < NumRecordsPerSecond.Num(); ++Idx)
			{
				Buffer += FString::Printf(TEXT("Number of min collisions at %ds = %d\n"), Idx, NumCollisionMinPerSecond[Idx]);
				Buffer += FString::Printf(TEXT("Number of max collisions at %ds = %d\n"), Idx, NumCollisionMaxPerSecond[Idx]);
				Buffer += FString::Printf(TEXT("Number of average collisions at %ds = %f\n"), Idx, NumCollisionAveragePerSecond[Idx]);
			}
			Buffer += FString::Printf(TEXT("Number of total collisions = %d\n"), NumTotalCollisions);
		}
	}
	

	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("BONE HIERARCHY TRANSFORM COUNTS\n"));
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));

	const TManagedArray<int32>& Levels = GeometryCollection->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);

	TArray<int32> LevelTransforms;
	for (int32 Element = 0, NumElement = Levels.Num(); Element < NumElement; ++Element)
	{
		const int32 NodeLevel = Levels[Element];
		while (LevelTransforms.Num() <= NodeLevel)
		{
			LevelTransforms.SetNum(NodeLevel + 1);
			LevelTransforms[NodeLevel] = 0;
		}
		++LevelTransforms[NodeLevel];
	}

	for (int32 Level = 0 ; Level < LevelTransforms.Num() ; ++Level)
	{
		Buffer += FString::Printf(TEXT("Level: %d = %d\n"), Level, LevelTransforms[Level]);
	}

	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
	Buffer += FString::Printf(TEXT("MESH QUALITY\n"));
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));

	TSet<int32> VertexToDeleteSet;
	TMap<int32, int32> CoincidentVerticesMap;
	GeometryCollectionAlgo::ComputeCoincidentVertices(GeometryCollection, 1e-2f, CoincidentVerticesMap, VertexToDeleteSet);
	int32 NumCoincidentVertices = VertexToDeleteSet.Num();

	TSet<int32> FaceToDeleteSet;
	GeometryCollectionAlgo::ComputeZeroAreaFaces(GeometryCollection, 1e-4f, FaceToDeleteSet);
	int32 NumZeroAreaFaces = FaceToDeleteSet.Num();

	GeometryCollectionAlgo::ComputeHiddenFaces(GeometryCollection, FaceToDeleteSet);
	int32 NumHiddenFaces = FaceToDeleteSet.Num();

	GeometryCollectionAlgo::ComputeStaleVertices(GeometryCollection, VertexToDeleteSet);
	int32 NumStaleVertices = VertexToDeleteSet.Num();

	TMap<GeometryCollectionAlgo::FFaceEdge, int32> FaceEdgeMap;
	GeometryCollectionAlgo::ComputeEdgeInFaces(GeometryCollection, FaceEdgeMap);

	int32 NumBoundaryEdges = 0;
	int32 NumDegenerateEdges = 0;
	for (auto& Edge : FaceEdgeMap)
	{
		if (FaceEdgeMap[Edge.Key] == 0)
		{
			NumBoundaryEdges++;
		}
		else if (FaceEdgeMap[Edge.Key] > 2)
		{
			NumDegenerateEdges++;
		}
	}

	Buffer += FString::Printf(TEXT("Number of coincident vertices = %d\n"), NumCoincidentVertices);
	Buffer += FString::Printf(TEXT("Number of zero area faces = %d\n"), NumZeroAreaFaces);
	Buffer += FString::Printf(TEXT("Number of hidden faces = %d\n"), NumHiddenFaces);
	Buffer += FString::Printf(TEXT("Number of stale vertices = %d\n"), NumStaleVertices);
	Buffer += FString::Printf(TEXT("Number of boundary edges = %d\n"), NumBoundaryEdges);
	Buffer += FString::Printf(TEXT("Number of degenerate edges (included in more than 2 faces) = %d\n"), NumDegenerateEdges);
	Buffer += FString::Printf(TEXT("------------------------------------------------------------\n\n"));



	UE_LOGF(LogGeoemtryCollectionClean, Log, "%ls", *Buffer);
}

void GeometryCollectionEngineUtility::PrintDetailedStatisticsSummary(const TArray<const FGeometryCollection*> GeometryCollectionArray)
{
	if (GeometryCollectionArray.Num() > 0)
	{
		FString Buffer;

		TArray<int32> LevelTransformsAll;
		LevelTransformsAll.SetNumZeroed(10);
		int32 LevelMax = INT_MIN;

		for (int32 Idx = 0; Idx < GeometryCollectionArray.Num(); ++Idx)
		{
			const FGeometryCollection* GeometryCollection = GeometryCollectionArray[Idx];

			check(GeometryCollection);


			Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
			Buffer += FString::Printf(TEXT("BONE HIERARCHY TRANSFORM COUNTS\n"));
			Buffer += FString::Printf(TEXT("------------------------------------------------------------\n"));
			Buffer += FString::Printf(TEXT("Sum for all the selected GCs\n\n"));

			const TManagedArray<int32>& Levels = GeometryCollection->GetAttribute<int32>("Level", FGeometryCollection::TransformGroup);

			TArray<int32> LevelTransforms;
			for (int32 Element = 0, NumElement = Levels.Num(); Element < NumElement; ++Element)
			{
				const int32 NodeLevel = Levels[Element];
				while (LevelTransforms.Num() <= NodeLevel)
				{
					LevelTransforms.SetNum(NodeLevel + 1);
					LevelTransforms[NodeLevel] = 0;
				}
				++LevelTransforms[NodeLevel];
			}

			for (int32 Level = 0; Level < LevelTransforms.Num(); ++Level)
			{
				LevelTransformsAll[Level] += LevelTransforms[Level];
			}

			if (LevelTransforms.Num() > LevelMax)
			{
				LevelMax = LevelTransforms.Num();
			}
		}

		for (int32 Level = 0; Level < LevelMax; ++Level)
		{
			Buffer += FString::Printf(TEXT("Level: %d = %d\n"), Level, LevelTransformsAll[Level]);
		}

		UE_LOGF(LogGeoemtryCollectionClean, Log, "%ls", *Buffer);
	}	
}

void GeometryCollectionEngineUtility::ComputeNormals(FGeometryCollection* GeometryCollection)
{
#if WITH_EDITOR
	// recompute tangents for geometry collection
	IMeshUtilities& MeshUtilities = FModuleManager::LoadModuleChecked<IMeshUtilities>("MeshUtilities");

	int32 NumVertices = GeometryCollection->NumElements(FGeometryCollection::VerticesGroup);
	int32 NumIndices = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);

	// #todo(dmp): Deal with smoothing groups
	TArray<uint32> SmoothingGroup;
	SmoothingGroup.Init(0, NumIndices);
	int32 TangentOptions = 0;

	// Create index and uvs in flat arrays per face vertex
	TArray<uint32> TmpIndices;
	TArray<FVector2f> TmpUV;

	TManagedArray<FIntVector>& Indices = GeometryCollection->Indices;
	const int32 UVChannelIndex = 0;
	const TManagedArray<FVector2f>& UV0 = *GeometryCollection->FindUVLayer(UVChannelIndex);
	for (int i = 0; i < NumIndices; ++i)
	{
		FIntVector CurrTri = Indices[i];

		TmpIndices.Add(CurrTri.X);
		TmpIndices.Add(CurrTri.Y);
		TmpIndices.Add(CurrTri.Z);

		TmpUV.Add(UV0[CurrTri.X]);
		TmpUV.Add(UV0[CurrTri.Y]);
		TmpUV.Add(UV0[CurrTri.Z]);
	}

	// Make a copy of the vertex array
	// #todo(dmp): can we avoid copying?
	TArray<FVector3f> Vertex((GeometryCollection->Vertex).GetData(), NumVertices);

	// Compute new tangents and normals
	TArray<FVector3f> TmpTangentU;
	TArray<FVector3f> TmpTangentV;
	TArray<FVector3f> TmpNormal;

	// #todo(dmp): Create our own implementation to avoid all the copying
	MeshUtilities.CalculateNormals(Vertex, TmpIndices, TmpUV, SmoothingGroup, TangentOptions, TmpNormal);

	TManagedArray<FVector3f>& Normals = GeometryCollection->Normal;

	// Reset Normals
	for (int i = 0; i < NumVertices; ++i)
	{
		Normals[i] = TmpNormal[i];
	}
#else
	UE_LOGF(LogGeoemtryCollectionClean, Fatal, "GeometryCollectionEngineUtility::ComputeNormals not supported in non-editor builds");
#endif
}

void GeometryCollectionEngineUtility::ComputeTangents(FGeometryCollection* GeometryCollection)
{
#if WITH_EDITOR
	// recompute tangents for geometry collection
	IMeshUtilities& MeshUtilities = FModuleManager::LoadModuleChecked<IMeshUtilities>("MeshUtilities");

	int32 NumVertices = GeometryCollection->NumElements(FGeometryCollection::VerticesGroup);
	int32 NumIndices = GeometryCollection->NumElements(FGeometryCollection::FacesGroup);

	// #todo(dmp): Deal with smoothing groups
	TArray<uint32> SmoothingGroup;
	SmoothingGroup.Init(0, NumIndices);
	int32 TangentOptions = 0;

	// Create index and uvs in flat arrays per face vertex
	TArray<uint32> TmpIndices;
	TArray<FVector2f> TmpUV;

	TManagedArray<FIntVector>& Indices = GeometryCollection->Indices;
	const int32 UVChannelIndex = 0;
	TManagedArray<FVector2f>& UV0 = *GeometryCollection->FindUVLayer(UVChannelIndex);
	for (int i = 0; i < NumIndices; ++i)
	{
		FIntVector CurrTri = Indices[i];

		TmpIndices.Add(CurrTri.X);
		TmpIndices.Add(CurrTri.Y);
		TmpIndices.Add(CurrTri.Z);

		TmpUV.Add(UV0[CurrTri.X]);
		TmpUV.Add(UV0[CurrTri.Y]);
		TmpUV.Add(UV0[CurrTri.Z]);
	}

	// Make a copy of the vertex array
	// #todo(dmp): can we avoid copying?
	TArray<FVector3f> Vertex((GeometryCollection->Vertex).GetData(), NumVertices);

	// Compute new tangents and normals
	TArray<FVector3f> TmpTangentU;
	TArray<FVector3f> TmpTangentV;
	TArray<FVector3f> TmpNormal;

	// #todo(dmp): Create our own implementation to avoid all the copying
	MeshUtilities.CalculateTangents(Vertex, TmpIndices, TmpUV, SmoothingGroup, TangentOptions, TmpTangentU, TmpTangentV, TmpNormal);

	// we only use the tangents along with the original normals
	TManagedArray<FVector3f>& TangentU = GeometryCollection->TangentU;
	TManagedArray<FVector3f>& TangentV = GeometryCollection->TangentV;

	// Reset  tangents
	for (int i = 0; i < NumVertices; ++i)
	{
		TangentU[i] = FVector3f(0, 0, 0);
		TangentV[i] = FVector3f(0, 0, 0);
		// Normal[i] = FVector3f(0, 0, 0);
	}

	// Sum all face vertex tangents to the smaller vertex array
	for (int i = 0; i < TmpIndices.Num(); ++i)
	{
		int32 VertIndex = TmpIndices[i];
		TangentU[VertIndex] += TmpTangentU[i];
		TangentV[VertIndex] += TmpTangentV[i];
	}

	// normalize tangents
	for (int i = 0; i < NumVertices; ++i)
	{
		TangentU[i].Normalize();
		TangentV[i].Normalize();
	}
#else
	UE_LOGF(LogGeoemtryCollectionClean, Fatal, "GeometryCollectionEngineUtility::ComputeTangents not supported in non-editor builds");
#endif
}

class FSortableIntVector2 : public UE::Math::TIntVector2<int32>
{
public:
	FSortableIntVector2()
	{
		X = INT_MAX;
		Y = -INT_MAX;
	}

	bool operator < (const FIntVector2& Other) const
	{
		return X < Other.X;
	}
};

void GeometryCollectionEngineUtility::GenerateConnectedComponents(const USkeletalMesh* InSkeletalMesh,
	TArray<TArray<FIntVector>>& SourceTriangleVertices,
	TArray<TArray<FIntVector2>>& SourceTriangleIndex,
	TArray<int32>& VertexComponentMap,
	int32& TriangleCount, int32& VertexCount)
{
#if WITH_EDITOR
	constexpr int32 LODIndex = 0;
	if (InSkeletalMesh->HasMeshDescription(LODIndex))
	{
		FMeshDescription MeshDescription;
		InSkeletalMesh->CloneMeshDescription(LODIndex, MeshDescription);

		TArray<int32> VertexComponentID;
		VertexComponentID.Init(INDEX_NONE, MeshDescription.Vertices().Num());

		int32 MaxComponentID = INDEX_NONE;
		for (int VisitedVertexIdx = 0; VisitedVertexIdx < VertexComponentID.Num(); VisitedVertexIdx++)
		{
			if (VertexComponentID[VisitedVertexIdx] == INDEX_NONE)
			{
				MaxComponentID++;
				TArray<int32> Neighbors;
				Neighbors.Push(VisitedVertexIdx);

				while (!Neighbors.IsEmpty())
				{
					int32 CurrentVertex = Neighbors.Pop();
					for (int32 EdgeID : MeshDescription.GetVertexConnectedEdgeIDs(CurrentVertex))
					{
						const TArrayView<const FVertexID>& Edge = MeshDescription.GetEdgeVertices(EdgeID);
						int NewVertex = Edge[0] == CurrentVertex ? Edge[1] : Edge[0];
						if (VertexComponentID[NewVertex] == INDEX_NONE)
						{
							Neighbors.Push(NewVertex);
						}
					}
					VertexComponentID[CurrentVertex] = MaxComponentID;
				}
			}
		}

		// Build Component Triangles
		TriangleCount = 0;
		SourceTriangleIndex.Init(TArray< FIntVector2 >(), MaxComponentID + 1);
		SourceTriangleVertices.Init(TArray<FIntVector>(), MaxComponentID + 1);
		for (FTriangleID TriangleID : MeshDescription.Triangles().GetElementIDs())
		{
			const TArrayView<const FVertexID>& Triangle = MeshDescription.GetTriangleVertices(TriangleID);
			int32 ComponentID = VertexComponentID[Triangle[0]];
			for (int k = 1; k < 3; k++)
			{
				if (!ensure(VertexComponentID[Triangle[0]] == ComponentID))
				{
					return;
				}
			}
			SourceTriangleVertices[ComponentID].Add(FIntVector3(Triangle[0], Triangle[1], Triangle[2]));
			SourceTriangleIndex[ComponentID].Add(FIntVector2(TriangleID, INDEX_NONE));
			TriangleCount++;
		}

		// Build Remapping
		int32 CurrentRemapIndex = 0;
		VertexComponentMap.Init(INDEX_NONE, MeshDescription.Vertices().Num());
		for (TArray<FIntVector>& Component : SourceTriangleVertices)
		{
			for (FIntVector& Triangle : Component)
			{
				for (int k = 0; k < 3; k++)
				{
					int32 Index = Triangle[k];
					if (VertexComponentMap[Index] == INDEX_NONE)
					{
						VertexComponentMap[Index] = CurrentRemapIndex++;
					}
				}
			}
		}
		VertexCount = CurrentRemapIndex;
		if (!ensure(CurrentRemapIndex <= VertexComponentMap.Num()))
		{
			return;
		}
		return;
	}
#endif
}
