// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionHitProxy.h"

#if WITH_EDITOR
IMPLEMENT_HIT_PROXY(HGeometryCollection, HHitProxy);
#endif 
