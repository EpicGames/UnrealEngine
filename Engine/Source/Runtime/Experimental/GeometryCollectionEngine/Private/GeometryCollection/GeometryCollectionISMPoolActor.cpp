// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionISMPoolActor.h"
#include "GeometryCollection/GeometryCollectionISMPoolComponent.h"
#include "GeometryCollection/GeometryCollectionISMPoolDebugDrawComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionISMPoolActor)

AGeometryCollectionISMPoolActor::AGeometryCollectionISMPoolActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ISMPoolComp = CreateDefaultSubobject<UGeometryCollectionISMPoolComponent>(TEXT("ISMPoolComp"));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	RootComponent = ISMPoolComp;

#if UE_ENABLE_DEBUG_DRAWING
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ISMPoolDebugDrawComp = CreateDefaultSubobject<UGeometryCollectionISMPoolDebugDrawComponent>(TEXT("ISMPoolDebug"));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	ISMPoolDebugDrawComp->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	ISMPoolDebugDrawComp->SetCanEverAffectNavigation(false);
	ISMPoolDebugDrawComp->CanCharacterStepUpOn = ECanBeCharacterBase::ECB_No;
	ISMPoolDebugDrawComp->SetGenerateOverlapEvents(false);
	ISMPoolDebugDrawComp->SetupAttachment(ISMPoolComp);
#endif
}
