// Copyright Epic Games, Inc. All Rights Reserved.
#include "GeometryCollection/GeometryCollectionISMPoolRenderer.h"

#include "Engine/Level.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "GeometryCollection/Facades/CollectionInstancedMeshFacade.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "ISMPool/ISMPoolActor.h"
#include "ISMPool/ISMPoolComponent.h"
#include "ISMPool/ISMPoolSubSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionISMPoolRenderer)

void UGeometryCollectionISMPoolRenderer::OnRegisterGeometryCollection(UGeometryCollectionComponent& InComponent)
{
	OwningLevel = MakeWeakObjectPtr(InComponent.GetComponentLevel());

	// In editor we create our own ISMPool. 
	// This guarantees the same look in editor/game, and allows editor hit proxies to continue working.
	if (UWorld* World = InComponent.GetWorld())
	{
		if (!World->IsGameWorld())
		{
			if (LocalISMPoolComponent)
			{
				LocalISMPoolComponent->DestroyComponent();
			}

			LocalISMPoolComponent = NewObject<UISMPoolComponent>(this, NAME_None, RF_Transient | RF_DuplicateTransient);
			LocalISMPoolComponent->SetTickablePoolManagement(false);
			LocalISMPoolComponent->SetupAttachment(&InComponent);
			LocalISMPoolComponent->RegisterComponent();
		}
	}

	bIsRegistered = true;
}

void UGeometryCollectionISMPoolRenderer::OnUnregisterGeometryCollection()
{
	ReleaseGroup(MergedMeshGroup);
	ReleaseGroup(InstancesGroup);

	if (LocalISMPoolComponent)
	{
		LocalISMPoolComponent->DestroyComponent();
		LocalISMPoolComponent = nullptr;
	}

	CachedISMPoolComponent = nullptr;

	bIsRegistered = false;
}

void UGeometryCollectionISMPoolRenderer::UpdateState(UGeometryCollection const& InGeometryCollection, FTransform const& InComponentTransform, uint32 InStateFlags)
{
	ComponentTransform = InComponentTransform;

	const bool bIsVisible = (InStateFlags & EState_Visible) != 0;
	const bool bIsBroken = (InStateFlags & EState_Broken) != 0;

	if (bIsVisible == false)
	{
		ReleaseGroup(InstancesGroup);
		ReleaseGroup(MergedMeshGroup);
	}
	else
	{
		if (!bIsBroken && MergedMeshGroup.GroupIndex == INDEX_NONE)
		{
			// Remove broken primitives.
			ReleaseGroup(InstancesGroup);

			// Add merged mesh.
			InitMergedMeshFromGeometryCollection(InGeometryCollection);
		}

		if (bIsBroken && InstancesGroup.GroupIndex == INDEX_NONE)
		{
			// Remove merged mesh.
			ReleaseGroup(MergedMeshGroup);

			// Add broken primitives.
			InitInstancesFromGeometryCollection(InGeometryCollection);
		}
	}
}

void UGeometryCollectionISMPoolRenderer::UpdateRootTransform(UGeometryCollection const& InGeometryCollection, FTransform const& InRootTransform)
{
	UpdateMergedMeshTransforms(InRootTransform * ComponentTransform, {});
}

void UGeometryCollectionISMPoolRenderer::UpdateRootTransforms(UGeometryCollection const& InGeometryCollection, FTransform const& InRootTransform, TArrayView<const FTransform3f> InRootLocalTransforms)
{
	UpdateMergedMeshTransforms(InRootTransform * ComponentTransform, InRootLocalTransforms);
}

void UGeometryCollectionISMPoolRenderer::UpdateTransforms(UGeometryCollection const& InGeometryCollection, TArrayView<const FTransform3f> InTransforms)
{
	UpdateInstanceTransforms(InGeometryCollection, ComponentTransform, InTransforms);
}

UISMPoolComponent* UGeometryCollectionISMPoolRenderer::GetISMPoolComponent() const
{
	if (!bIsRegistered)
	{
		return nullptr;
	}
	return LocalISMPoolComponent ? LocalISMPoolComponent : CachedISMPoolComponent;
}

UISMPoolComponent* UGeometryCollectionISMPoolRenderer::GetOrCreateISMPoolComponent()
{
	if (!bIsRegistered)
	{
		return nullptr;
	}
	if (LocalISMPoolComponent)
	{
		return LocalISMPoolComponent;
	}
	if (!CachedISMPoolComponent)
	{
		if (UISMPoolSubSystem* ISMPoolSubSystem = UWorld::GetSubsystem<UISMPoolSubSystem>(GetWorld()))
		{
			if (ULevel* Level = OwningLevel.Get())
			{
				if (AISMPoolActor* ISMPoolActor = ISMPoolSubSystem->FindISMPoolActor(Level))
				{
					CachedISMPoolComponent = ISMPoolActor->GetISMPoolComp();
				}
			}
		}
	}

	return CachedISMPoolComponent;
}

void UGeometryCollectionISMPoolRenderer::InitMergedMeshFromGeometryCollection(UGeometryCollection const& InGeometryCollection)
{
	const FGeometryCollectionProxyMeshData& RootProxyData = InGeometryCollection.RootProxyData;
	if (RootProxyData.ProxyMeshes.Num() == 0)
	{
		return;
	}

	UISMPoolComponent* ISMPoolComponent = GetOrCreateISMPoolComponent();
	MergedMeshGroup.GroupIndex = ISMPoolComponent != nullptr ? ISMPoolComponent->CreateMeshGroup() : INDEX_NONE;

	if (MergedMeshGroup.GroupIndex == INDEX_NONE)
	{
		return;
	}

	const int32 NumProxyMesh = RootProxyData.ProxyMeshes.Num();
	for (int32 MeshIndex = 0; MeshIndex < NumProxyMesh; ++MeshIndex)
	{
		UStaticMesh* StaticMesh = RootProxyData.ProxyMeshes[MeshIndex];
		if (StaticMesh == nullptr)
		{
			continue;
		}

		FISMPoolStaticMeshInstance StaticMeshInstance;
		StaticMeshInstance.StaticMesh = StaticMesh;

		if (RootProxyData.MeshOverrideMaterials.IsValidIndex(MeshIndex))
		{
			const FGeometryCollectionProxyMeshMaterials& ProxyMaterials = RootProxyData.MeshOverrideMaterials[MeshIndex];
			for (int32 MatIndex = 0; MatIndex < ProxyMaterials.Materials.Num(); ++MatIndex)
			{
				StaticMeshInstance.MaterialsOverrides.Add(ProxyMaterials.Materials[MatIndex]);
			}
		}

		TArray<float> DummyCustomData;
		MergedMeshGroup.MeshIds.Add(ISMPoolComponent->AddMeshToGroup(MergedMeshGroup.GroupIndex, StaticMeshInstance, 1, DummyCustomData));
	}
}

void UGeometryCollectionISMPoolRenderer::InitInstancesFromGeometryCollection(UGeometryCollection const& InGeometryCollection)
{
	const int32 NumMeshes = InGeometryCollection.AutoInstanceMeshes.Num();
	if (NumMeshes == 0)
	{
		return;
	}

	UISMPoolComponent* ISMPoolComponent = GetOrCreateISMPoolComponent();
	InstancesGroup.GroupIndex = ISMPoolComponent != nullptr ? ISMPoolComponent->CreateMeshGroup() : INDEX_NONE;

	if (InstancesGroup.GroupIndex == INDEX_NONE)
	{
		return;
	}

	InstancesGroup.MeshIds.Reserve(NumMeshes);

	for (const FGeometryCollectionAutoInstanceMesh& AutoInstanceMesh : InGeometryCollection.AutoInstanceMeshes)
	{
		if (const UStaticMesh* StaticMesh = AutoInstanceMesh.Mesh)
		{
			bool bMaterialOverride = false;
			for (int32 MatIndex = 0; MatIndex < AutoInstanceMesh.Materials.Num(); MatIndex++)
			{
				const UMaterialInterface* OriginalMaterial = StaticMesh->GetMaterial(MatIndex);
				if (OriginalMaterial != AutoInstanceMesh.Materials[MatIndex])
				{
					bMaterialOverride = true;
					break;
				}
			}
			FISMPoolStaticMeshInstance StaticMeshInstance;
			StaticMeshInstance.StaticMesh = const_cast<UStaticMesh*>(StaticMesh);
			StaticMeshInstance.Desc.NumCustomDataFloats = AutoInstanceMesh.GetNumDataPerInstance();
			if (bMaterialOverride)
			{
				StaticMeshInstance.MaterialsOverrides.Reset();
				StaticMeshInstance.MaterialsOverrides.Append(AutoInstanceMesh.Materials);
			}

			InstancesGroup.MeshIds.Add(ISMPoolComponent->AddMeshToGroup(InstancesGroup.GroupIndex, StaticMeshInstance, AutoInstanceMesh.NumInstances, AutoInstanceMesh.CustomData));
		}
	}
}

void UGeometryCollectionISMPoolRenderer::UpdateMergedMeshTransforms(FTransform const& InBaseTransform, TArrayView<const FTransform3f> InLocalTransforms)
{
	if (MergedMeshGroup.GroupIndex == INDEX_NONE)
	{
		return;
	}

	UISMPoolComponent* ISMPoolComponent = GetOrCreateISMPoolComponent();
	if (ISMPoolComponent == nullptr)
	{
		return;
	}

	TArrayView<const FTransform> InstanceTransforms(&InBaseTransform, 1);
	for (int32 MeshIndex = 0; MeshIndex < MergedMeshGroup.MeshIds.Num(); MeshIndex++)
	{
		if (InLocalTransforms.IsValidIndex(MeshIndex))
		{
			const FTransform CombinedTransform{ FTransform(InLocalTransforms[MeshIndex]) * InBaseTransform };
			ISMPoolComponent->BatchUpdateInstancesTransforms(MergedMeshGroup.GroupIndex, MergedMeshGroup.MeshIds[MeshIndex], 0, MakeArrayView(&CombinedTransform, 1), true/*bWorldSpace*/, false/*bMarkRenderStateDirty*/, false/*bTeleport*/);
		}
		else
		{
			ISMPoolComponent->BatchUpdateInstancesTransforms(MergedMeshGroup.GroupIndex, MergedMeshGroup.MeshIds[MeshIndex], 0, InstanceTransforms, true/*bWorldSpace*/, false/*bMarkRenderStateDirty*/, false/*bTeleport*/);
		}
	}
}

void UGeometryCollectionISMPoolRenderer::UpdateInstanceTransforms(UGeometryCollection const& InGeometryCollection, FTransform const& InBaseTransform, TArrayView<const FTransform3f> InTransforms)
{
	if (InstancesGroup.GroupIndex == INDEX_NONE)
	{
		return;
	}

	UISMPoolComponent* ISMPoolComponent = GetOrCreateISMPoolComponent();
	if (ISMPoolComponent == nullptr)
	{
		return;
	}

	const GeometryCollection::Facades::FCollectionInstancedMeshFacade InstancedMeshFacade(*InGeometryCollection.GetGeometryCollection());
	if (!InstancedMeshFacade.IsValid())
	{
		return;
	}

	const int32 NumTransforms = InGeometryCollection.NumElements(FGeometryCollection::TransformAttribute);
	const TManagedArray<TSet<int32>>& Children = InGeometryCollection.GetGeometryCollection()->Children;

	TArray<FTransform> InstanceTransforms;
	for (int32 MeshIndex = 0; MeshIndex < InstancesGroup.MeshIds.Num(); MeshIndex++)
	{
		InstanceTransforms.Reset(NumTransforms); // Allocate for worst case
		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; TransformIndex++)
		{
			const int32 AutoInstanceMeshIndex = InstancedMeshFacade.GetIndex(TransformIndex);
			if (AutoInstanceMeshIndex == MeshIndex && Children[TransformIndex].Num() == 0)
			{
				InstanceTransforms.Add(FTransform(InTransforms[TransformIndex]) * InBaseTransform);
			}
		}
		
		ISMPoolComponent->BatchUpdateInstancesTransforms(InstancesGroup.GroupIndex, InstancesGroup.MeshIds[MeshIndex], 0, MakeArrayView(InstanceTransforms), true/*bWorldSpace*/, false/*bMarkRenderStateDirty*/, false/*bTeleport*/);
	}
}

void UGeometryCollectionISMPoolRenderer::ReleaseGroup(FISMPoolGroup& InOutGroup)
{
	if (InOutGroup.GroupIndex == INDEX_NONE)
	{
		return;
	}

	// Component and owning actor may already be released safely by a level unload.
	// Don't want to create a new component here, so don't use GetOrCreateISMPoolComponent().
	UISMPoolComponent* ISMPoolComponent = GetISMPoolComponent();
	if (ISMPoolComponent != nullptr)
	{
		ISMPoolComponent->DestroyMeshGroup(InOutGroup.GroupIndex);
	}

	InOutGroup.GroupIndex = INDEX_NONE;
	InOutGroup.MeshIds.Empty();
}
