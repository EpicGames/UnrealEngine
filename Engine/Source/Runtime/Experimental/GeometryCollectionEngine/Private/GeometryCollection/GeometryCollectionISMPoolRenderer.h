// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/GeometryCollectionExternalRenderInterface.h"

#include "GeometryCollectionISMPoolRenderer.generated.h"

class UGeometryCollectionComponent;
class UISMPoolComponent;
class ULevel;

/** Implementation of a geometry collection custom renderer that pushes AutoInstanceMeshes to an ISMPool. */
UCLASS()
class UGeometryCollectionISMPoolRenderer : public UObject, public IGeometryCollectionExternalRenderInterface
{
	GENERATED_BODY()

	/** Instanced Static Mesh Pool component that is used to render our meshes. */
	UPROPERTY(Transient)
	TObjectPtr<UISMPoolComponent> CachedISMPoolComponent;

	/** Set if we have an Instanced Static Mesh Pool component owned by this renderer (ie when in Editor mode). Non-transient to behave correctly under actor duplication. */
	UPROPERTY()
	TObjectPtr<UISMPoolComponent> LocalISMPoolComponent;

public:
	//~ Begin IGeometryCollectionExternalRenderInterface Interface.
	virtual void OnRegisterGeometryCollection(UGeometryCollectionComponent& InComponent) override;
	virtual void OnUnregisterGeometryCollection() override;
	virtual void UpdateState(UGeometryCollection const& InGeometryCollection, FTransform const& InComponentTransform, uint32 InStateFlags) override;
	virtual void UpdateRootTransform(UGeometryCollection const& InGeometryCollection, FTransform const& InRootTransform) override;
	virtual void UpdateRootTransforms(UGeometryCollection const& InGeometryCollection, FTransform const& InRootTransform, TArrayView<const FTransform3f> InRootLocalTransforms) override;
	virtual void UpdateTransforms(UGeometryCollection const& InGeometryCollection, TArrayView<const FTransform3f> InTransforms) override;
	//~ End IGeometryCollectionExternalRenderInterface Interface.

protected:
	/** Description for a group of meshes that are added/updated together. */
	struct FISMPoolGroup
	{
		int32 GroupIndex = INDEX_NONE;
		TArray<int32> MeshIds;
	};

	/** Cached component transform. */
	FTransform ComponentTransform = FTransform::Identity;

	/** ISM pool groups per rendering element type. */
	FISMPoolGroup MergedMeshGroup;
	FISMPoolGroup InstancesGroup;

	/** Level of the owning component of this renderer */
	TWeakObjectPtr<ULevel> OwningLevel;

	/** Registered flag set true in between calls to OnRegister() and OnUnregister(). */
	bool bIsRegistered = false;

private:
	UISMPoolComponent* GetISMPoolComponent() const;
	UISMPoolComponent* GetOrCreateISMPoolComponent();
	void InitMergedMeshFromGeometryCollection(UGeometryCollection const& InGeometryCollection);
	void InitInstancesFromGeometryCollection(UGeometryCollection const& InGeometryCollection);
	void UpdateMergedMeshTransforms(FTransform const& InBaseTransform, TArrayView<const FTransform3f> InLocalTransforms);
	void UpdateInstanceTransforms(UGeometryCollection const& InGeometryCollection, FTransform const& InBaseTransform, TArrayView<const FTransform3f> InTransforms);
	void ReleaseGroup(FISMPoolGroup& InOutGroup);
};
