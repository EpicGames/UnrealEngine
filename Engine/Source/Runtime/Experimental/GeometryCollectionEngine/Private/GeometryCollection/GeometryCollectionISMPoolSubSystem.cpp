// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionISMPoolSubSystem.h"
#include "GeometryCollection/GeometryCollectionISMPoolActor.h"
#include "Engine/World.h"

//----------------------------------------------------------------------//
// UGeometryCollectionISMPoolSubSystem
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionISMPoolSubSystem)
UGeometryCollectionISMPoolSubSystem::UGeometryCollectionISMPoolSubSystem()
{
}

void UGeometryCollectionISMPoolSubSystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	Collection.InitializeDependency<UGeometryCollectionISMPoolSubSystem>();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UGeometryCollectionISMPoolSubSystem::Deinitialize()
{
	PerLevelISMPoolActors.Reset();
	Super::Deinitialize();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
AGeometryCollectionISMPoolActor* UGeometryCollectionISMPoolSubSystem::FindISMPoolActor(ULevel* Level)
{
	AGeometryCollectionISMPoolActor* ISMPoolActor = nullptr;

	// on demand creation of the actor based on level
	TObjectPtr<AGeometryCollectionISMPoolActor>* ISMPoolActorInLevel = PerLevelISMPoolActors.Find(Level);
	if (ISMPoolActorInLevel)
	{
		ISMPoolActor = ISMPoolActorInLevel->Get();
	}
	else
	{
		// we keep it as transient to avoid accumulation of those actors in saved levels
		FActorSpawnParameters Params;
		Params.ObjectFlags = EObjectFlags::RF_DuplicateTransient | EObjectFlags::RF_Transient;
		Params.OverrideLevel = Level;
		ISMPoolActor = GetWorld()->SpawnActor<AGeometryCollectionISMPoolActor>(Params);
		// spawn can still fail if we are in the middle or tearing down the world
		if (ISMPoolActor)
		{
			PerLevelISMPoolActors.Add(Level, ISMPoolActor);
			// make sure we capture when the actor get removed so we can update our internal structure accordingly 
			ISMPoolActor->OnEndPlay.AddDynamic(this, &UGeometryCollectionISMPoolSubSystem::OnActorEndPlay);
		}
	}
	return ISMPoolActor;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UGeometryCollectionISMPoolSubSystem::OnActorEndPlay(AActor* InSource, EEndPlayReason::Type Reason)
{
	if (ULevel* ActorLevel = InSource->GetLevel())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		PerLevelISMPoolActors.Remove(ActorLevel);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UGeometryCollectionISMPoolSubSystem::GetISMPoolActors(TArray<AGeometryCollectionISMPoolActor*>& OutActors) const
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	for (const auto& MapEntry : PerLevelISMPoolActors)
	{
		if (MapEntry.Value != nullptr)
		{
			OutActors.Add(MapEntry.Value);
		}
	}
}
