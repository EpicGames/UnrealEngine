// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	GeometryCollection.cpp: UGeometryCollection methods.
=============================================================================*/
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionCache.h"
#include "GeometryCollection/GeometryCollectionRenderData.h"
#include "Materials/Material.h"
#include "UObject/DestructionObjectVersion.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/FortniteMainBranchObjectVersion.h"
#include "UObject/ObjectSaveContext.h"
#include "Serialization/ArchiveCountMem.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "UObject/Package.h"
#include "Materials/MaterialInstance.h"
#include "ProfilingDebugging/CookStats.h"
#include "EngineUtils.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "EditorFramework/AssetImportData.h"
#include "Rendering/NaniteResources.h"
#include "Engine/AssetUserData.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "Chaos/ErrorReporter.h"

#if WITH_EDITOR
#include "GeometryCollection/DerivedDataGeometryCollectionCooker.h"
#include "GeometryCollection/GeometryCollectionConvexUtility.h"
#include "GeometryCollection/GeometryCollectionEngineSizeSpecificUtility.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "DerivedDataCacheInterface.h"
#include "Serialization/MemoryReader.h"
#include "NaniteBuilder.h"
#include "Rendering/NaniteResources.h"
// TODO: Temp until new asset-agnostic builder API
#include "StaticMeshResources.h"
#endif

#include "GeometryCollection/GeometryCollectionSimulationCoreTypes.h"
#include "Chaos/ChaosArchive.h"
#include "Chaos/MassProperties.h"
#include "GeometryCollectionProxyData.h"
#include "Dataflow/DataflowObject.h"
#include "GeometryCollection/Facades/CollectionHierarchyFacade.h"
#include "GeometryCollection/Facades/CollectionInstancedMeshFacade.h"
#include "GeometryCollection/Facades/CollectionVertexBoneWeightsFacade.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionObject)

DEFINE_LOG_CATEGORY_STATIC(LogGeometryCollectionInternal, Log, All);

bool GeometryCollectionAssetForceStripOnCook = false;
FAutoConsoleVariableRef CVarGeometryCollectionBypassPhysicsAttributes(
	TEXT("p.GeometryCollectionAssetForceStripOnCook"),
	GeometryCollectionAssetForceStripOnCook,
	TEXT("Bypass the construction of simulation properties when all bodies are simply cached for playback."));

bool bGeometryCollectionEnableForcedConvexGenerationInSerialize = true;
FAutoConsoleVariableRef CVarGeometryCollectionEnableForcedConvexGenerationInSerialize(
	TEXT("p.GeometryCollectionEnableForcedConvexGenerationInSerialize"),
	bGeometryCollectionEnableForcedConvexGenerationInSerialize,
	TEXT("Enable generation of convex geometry on older destruction files.[def:true]"));

bool bGeometryCollectionAlwaysRecreateSimulationData = false;
FAutoConsoleVariableRef CVarGeometryCollectionAlwaysRecreateSimulationData(
	TEXT("p.GeometryCollectionAlwaysRecreateSimulationData"),
	bGeometryCollectionAlwaysRecreateSimulationData,
	TEXT("always recreate the simulation data even if the simulation data is not marked as dirty - this has runtime cost in editor - only use as a last resort if default has issues [def:false]"));

namespace Chaos
{
	namespace CVars
	{
		extern CHAOS_API bool bChaosConvexSimplifyUnion;
	}
}


#if ENABLE_COOK_STATS
namespace GeometryCollectionCookStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("GeometryCollection.Usage"), TEXT(""));
	});
}
#endif

static constexpr float DefaultMaxSizeValue = 99999.9f;
static const FName GeometryCollectionDataflowTerminalNodeName("GeometryCollectionTerminal");

UGeometryCollection::UGeometryCollection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	#if WITH_EDITOR
	, bManualDataCreate(false)
	#endif
	, EnableClustering(true)
	, ClusterGroupIndex(0)
	, MaxClusterLevel(100)
	, DamageModel(EDamageModelTypeEnum::Chaos_Damage_Model_UserDefined_Damage_Threshold)
	, DamageThreshold({ 500000.f, 50000.f, 5000.f })
	, bUseSizeSpecificDamageThreshold(false)
	, bUseMaterialDamageModifiers(false)
	, PerClusterOnlyDamageThreshold(false)
	, ClusterConnectionType(EClusterConnectionTypeEnum::Chaos_MinimalSpanningSubsetDelaunayTriangulation)
	, ConnectionGraphBoundsFilteringMargin(0)
	, bUseFullPrecisionUVs(false)
	, bStripOnCook(false)
	, bStripRenderDataOnCook(false)
	, bSupportRayTracing(true)
	, EnableNanite(false)
	, bEnableNaniteFallback(false)
	, NaniteMinimumResidencyInKB(0)
#if WITH_EDITORONLY_DATA
	, CollisionType_DEPRECATED(ECollisionTypeEnum::Chaos_Volumetric)
	, ImplicitType_DEPRECATED(EImplicitTypeEnum::Chaos_Implicit_Convex)
	, MinLevelSetResolution_DEPRECATED(10)
	, MaxLevelSetResolution_DEPRECATED(10)
	, MinClusterLevelSetResolution_DEPRECATED(50)
	, MaxClusterLevelSetResolution_DEPRECATED(50)
	, CollisionObjectReductionPercentage_DEPRECATED(0.0f)
#endif
	, bDensityFromPhysicsMaterial(false)
	, CachedDensityFromPhysicsMaterialInGCm3(0) // <=0 value means not cached yet 
	, bMassAsDensity(true)
	, Mass(2500.0f)
	, MinimumMassClamp(0.1f)
	, bImportCollisionFromSource(false)
	, bOptimizeConvexes(Chaos::CVars::bChaosConvexSimplifyUnion)
	, bScaleOnRemoval(true)
	, bRemoveOnMaxSleep(false)
	, bAutomaticCrumblePartialClusters(true)
	, MaximumSleepTime(5.0, 10.0)
	, RemovalDuration(2.5, 5.0)
	, bSlowMovingAsSleeping(true)
	, SlowMovingVelocityThreshold(1)
	, EnableRemovePiecesOnFracture_DEPRECATED(false)
	, DataflowInstance(this)
	, GeometryCollection(new FGeometryCollection())
{
	if(!IsTemplate())
	{
		PersistentGuid = FGuid::NewGuid();
	}
	InvalidateCollection();
#if WITH_EDITOR
	SimulationDataGuid = StateGuid;
	RenderDataGuid = StateGuid;
	ValidRenderDataGuid = StateGuid;
	bStripOnCook = GeometryCollectionAssetForceStripOnCook;
#endif
	PhysicsMaterial = GEngine? GEngine->DefaultPhysMaterial: nullptr;

	// make sure we have at least one size specific entry
	SizeSpecificData.AddDefaulted();

	// set the default name for the terminal node 
	DataflowInstance.SetDataflowTerminal(GeometryCollectionDataflowTerminalNodeName);
}

FGeometryCollectionLevelSetData::FGeometryCollectionLevelSetData()
	: MinLevelSetResolution(5)
	, MaxLevelSetResolution(10)
	, MinClusterLevelSetResolution(25)
	, MaxClusterLevelSetResolution(50)
{
}

FGeometryCollectionCollisionParticleData::FGeometryCollectionCollisionParticleData()
	: CollisionParticlesFraction(1.0f)
	, MaximumCollisionParticles(60)
{
}



FGeometryCollectionCollisionTypeData::FGeometryCollectionCollisionTypeData()
	: CollisionType(ECollisionTypeEnum::Chaos_Volumetric)
	, ImplicitType(EImplicitTypeEnum::Chaos_Implicit_Convex)
	, LevelSet()
	, CollisionParticles()
	, CollisionObjectReductionPercentage(0.0f)
	, CollisionMarginFraction(0.f)
{
}

FGeometryCollectionSizeSpecificData::FGeometryCollectionSizeSpecificData()
	: MaxSize(DefaultMaxSizeValue)
	, CollisionShapes({ FGeometryCollectionCollisionTypeData()})
#if WITH_EDITORONLY_DATA
	, CollisionType_DEPRECATED(ECollisionTypeEnum::Chaos_Volumetric)
	, ImplicitType_DEPRECATED(EImplicitTypeEnum::Chaos_Implicit_Convex)
	, MinLevelSetResolution_DEPRECATED(5)
	, MaxLevelSetResolution_DEPRECATED(10)
	, MinClusterLevelSetResolution_DEPRECATED(25)
	, MaxClusterLevelSetResolution_DEPRECATED(50)
	, CollisionObjectReductionPercentage_DEPRECATED(0)
	, CollisionParticlesFraction_DEPRECATED(1.f)
	, MaximumCollisionParticles_DEPRECATED(60)
#endif
	, DamageThreshold(5000.0)
{
}


bool FGeometryCollectionSizeSpecificData::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FPhysicsObjectVersion::GUID);
	return false;	//We only have this function to mark custom GUID. Still want serialize tagged properties
}

#if WITH_EDITORONLY_DATA
void FGeometryCollectionSizeSpecificData::PostSerialize(const FArchive& Ar)
{
	const int32 PhysicsObjectVersion = Ar.CustomVer(FPhysicsObjectVersion::GUID);
	const int32 StreamObjectVersion = Ar.CustomVer(FUE5MainStreamObjectVersion::GUID);
	// make sure to load back the deprecated values in the new structure if necessary
	// IMPORTANT : this was merge backed in UE4 and PhysicsObjectVersion had to be used,
	//		that's why we need to test both version to make sure backward asset compatibility is maintained
	if (Ar.IsLoading() && (
		StreamObjectVersion < FUE5MainStreamObjectVersion::GeometryCollectionUserDefinedCollisionShapes &&
		PhysicsObjectVersion < FPhysicsObjectVersion::GeometryCollectionUserDefinedCollisionShapes
		)) 
	{
		if (CollisionShapes.Num())
		{
			// @todo(chaos destruction collisions) : Add support for many
			CollisionShapes[0].CollisionType = CollisionType_DEPRECATED;
			CollisionShapes[0].ImplicitType = ImplicitType_DEPRECATED;
			CollisionShapes[0].CollisionObjectReductionPercentage = CollisionObjectReductionPercentage_DEPRECATED;
			CollisionShapes[0].CollisionMarginFraction = UPhysicsSettings::Get()->SolverOptions.CollisionMarginFraction;
			CollisionShapes[0].CollisionParticles.CollisionParticlesFraction = CollisionParticlesFraction_DEPRECATED;
			CollisionShapes[0].CollisionParticles.MaximumCollisionParticles = MaximumCollisionParticles_DEPRECATED;
			CollisionShapes[0].LevelSet.MinLevelSetResolution = MinLevelSetResolution_DEPRECATED;
			CollisionShapes[0].LevelSet.MaxLevelSetResolution = MaxLevelSetResolution_DEPRECATED;
			CollisionShapes[0].LevelSet.MinClusterLevelSetResolution = MinClusterLevelSetResolution_DEPRECATED;
			CollisionShapes[0].LevelSet.MaxClusterLevelSetResolution = MaxClusterLevelSetResolution_DEPRECATED;
		}
	}
}
#endif

const FTransform3f& FGeometryCollectionProxyMeshData::GetMeshTransform(int32 MeshIndex) const
{
	if (MeshTransforms.IsValidIndex(MeshIndex))
	{
		return MeshTransforms[MeshIndex];
	}
	return FTransform3f::Identity;
}

void FGeometryCollectionProxyMeshData::Reset()
{
	ProxyMeshes.Reset();
	MeshTransforms.Reset();
	MeshOverrideMaterials.Reset();
}

void FillSharedSimulationSizeSpecificData(FSharedSimulationSizeSpecificData& ToData, const FGeometryCollectionSizeSpecificData& FromData)
{
	ToData.MaxSize = FromData.MaxSize;

	ToData.CollisionShapesData.SetNumUninitialized(FromData.CollisionShapes.Num());

	if (FromData.CollisionShapes.Num())
	{
		for (int i = 0; i < FromData.CollisionShapes.Num(); i++)
		{
			ToData.CollisionShapesData[i].CollisionType = FromData.CollisionShapes[i].CollisionType;
			ToData.CollisionShapesData[i].ImplicitType = FromData.CollisionShapes[i].ImplicitType;
			ToData.CollisionShapesData[i].LevelSetData.MinLevelSetResolution = FromData.CollisionShapes[i].LevelSet.MinLevelSetResolution;
			ToData.CollisionShapesData[i].LevelSetData.MaxLevelSetResolution = FromData.CollisionShapes[i].LevelSet.MaxLevelSetResolution;
			ToData.CollisionShapesData[i].LevelSetData.MinClusterLevelSetResolution = FromData.CollisionShapes[i].LevelSet.MinClusterLevelSetResolution;
			ToData.CollisionShapesData[i].LevelSetData.MaxClusterLevelSetResolution = FromData.CollisionShapes[i].LevelSet.MaxClusterLevelSetResolution;
			ToData.CollisionShapesData[i].CollisionObjectReductionPercentage = FromData.CollisionShapes[i].CollisionObjectReductionPercentage;
			ToData.CollisionShapesData[i].CollisionMarginFraction = FromData.CollisionShapes[i].CollisionMarginFraction;
			ToData.CollisionShapesData[i].CollisionParticleData.CollisionParticlesFraction = FromData.CollisionShapes[i].CollisionParticles.CollisionParticlesFraction;
			ToData.CollisionShapesData[i].CollisionParticleData.MaximumCollisionParticles = FromData.CollisionShapes[i].CollisionParticles.MaximumCollisionParticles;
		}
	}
	ToData.DamageThreshold = FromData.DamageThreshold;
}

FGeometryCollectionSizeSpecificData UGeometryCollection::GeometryCollectionSizeSpecificDataDefaults() 
{
	FGeometryCollectionSizeSpecificData Data;

	Data.MaxSize = DefaultMaxSizeValue;
	if (Data.CollisionShapes.Num())
	{
		Data.CollisionShapes[0].CollisionType = ECollisionTypeEnum::Chaos_Volumetric;
		Data.CollisionShapes[0].ImplicitType = EImplicitTypeEnum::Chaos_Implicit_Capsule;
		Data.CollisionShapes[0].LevelSet.MinLevelSetResolution = 5;
		Data.CollisionShapes[0].LevelSet.MaxLevelSetResolution = 10;
		Data.CollisionShapes[0].LevelSet.MinClusterLevelSetResolution = 25;
		Data.CollisionShapes[0].LevelSet.MaxClusterLevelSetResolution = 50;
		Data.CollisionShapes[0].CollisionObjectReductionPercentage = 1.0;
		Data.CollisionShapes[0].CollisionMarginFraction = UPhysicsSettings::Get()->SolverOptions.CollisionMarginFraction;
		Data.CollisionShapes[0].CollisionParticles.CollisionParticlesFraction = 1.0;
		Data.CollisionShapes[0].CollisionParticles.MaximumCollisionParticles = 60;
	}
	Data.DamageThreshold = 5000.0f;
	return Data;
}


void UGeometryCollection::ValidateSizeSpecificDataDefaults()
{
	auto HasDefault = [](const TArray<FGeometryCollectionSizeSpecificData>& DatasIn)
	{
		for (const FGeometryCollectionSizeSpecificData& Data : DatasIn)
		{
			if (Data.MaxSize >= DefaultMaxSizeValue)
			{
				return true;
			}
		}
		return false;
	};

	if (!SizeSpecificData.Num() || !HasDefault(SizeSpecificData))
	{
		FGeometryCollectionSizeSpecificData Data = GeometryCollectionSizeSpecificDataDefaults();
		if (Data.CollisionShapes.Num())
		{
#if WITH_EDITORONLY_DATA
			Data.CollisionShapes[0].CollisionType = CollisionType_DEPRECATED;
			Data.CollisionShapes[0].ImplicitType = ImplicitType_DEPRECATED;
			Data.CollisionShapes[0].LevelSet.MinLevelSetResolution = MinLevelSetResolution_DEPRECATED;
			Data.CollisionShapes[0].LevelSet.MaxLevelSetResolution = MaxLevelSetResolution_DEPRECATED;
			Data.CollisionShapes[0].LevelSet.MinClusterLevelSetResolution = MinClusterLevelSetResolution_DEPRECATED;
			Data.CollisionShapes[0].LevelSet.MaxClusterLevelSetResolution = MaxClusterLevelSetResolution_DEPRECATED;
			Data.CollisionShapes[0].CollisionObjectReductionPercentage = CollisionObjectReductionPercentage_DEPRECATED;
			Data.CollisionShapes[0].CollisionMarginFraction = UPhysicsSettings::Get()->SolverOptions.CollisionMarginFraction;
#endif
			if (Data.CollisionShapes[0].ImplicitType == EImplicitTypeEnum::Chaos_Implicit_LevelSet)
			{
				Data.CollisionShapes[0].CollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
			}
		}
		SizeSpecificData.Add(Data);
	}
	check(SizeSpecificData.Num());
}


// update cachedroot index using the current hierarchy setup
void UGeometryCollection::UpdateRootIndex()
{
	RootIndex = INDEX_NONE;
	if (GeometryCollection)
	{
		Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*GeometryCollection);
		RootIndex = HierarchyFacade.GetRootIndex();
	}
}

void UGeometryCollection::CacheBreadthFirstTransformIndices()
{
	BreadthFirstTransformIndices.Reset();
	if (GeometryCollection)
	{
		Chaos::Facades::FCollectionHierarchyFacade HierarchyFacade(*GeometryCollection);
		BreadthFirstTransformIndices = HierarchyFacade.ComputeTransformIndicesInBreadthFirstOrder();
	}
}

void UGeometryCollection::CacheAutoInstanceTransformRemapIndices()
{
	AutoInstanceTransformRemapIndices.Reset();
	if (GeometryCollection == nullptr)
	{
		return;
	}
	const GeometryCollection::Facades::FCollectionInstancedMeshFacade InstancedMeshFacade(*GeometryCollection);
	if (!InstancedMeshFacade.IsValid())
	{
		return;
	}

	const int32 NumMeshes = AutoInstanceMeshes.Num();
	if(NumMeshes!=0)
	{
		TArray<int32> TransformGroups;
		TransformGroups.AddZeroed(NumMeshes);
		TArray<int32> TransformStarts;
		TransformStarts.AddUninitialized(NumMeshes);
		TArray<int32> InstanceCounts;
		InstanceCounts.AddUninitialized(NumMeshes);
		TArray<int32> WrittenTransformCounts;
		WrittenTransformCounts.AddZeroed(NumMeshes);


		for (int32 MeshIndex = 0; MeshIndex < NumMeshes; MeshIndex++)
		{
			const int32 NumInstances = AutoInstanceMeshes[MeshIndex].NumInstances;
			TransformStarts[MeshIndex] = MeshIndex == 0 ? 0 : TransformStarts[MeshIndex - 1] + InstanceCounts[MeshIndex - 1];
			InstanceCounts[MeshIndex] = NumInstances;
		}

		AutoInstanceTransformRemapIndices.AddUninitialized(TransformStarts.Last() + InstanceCounts.Last());

		const int32 NumTransforms = InstancedMeshFacade.GetNumIndices();
		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; TransformIndex++)
		{
			if (GeometryCollection->Children[TransformIndex].Num() == 0)
			{
				const int32 AutoInstanceMeshIndex = InstancedMeshFacade.GetIndex(TransformIndex);
				const int32 WriteIndex = WrittenTransformCounts[AutoInstanceMeshIndex];
				if (WriteIndex < InstanceCounts[AutoInstanceMeshIndex])
				{
					const int32 TransformArrayIndex = TransformStarts[AutoInstanceMeshIndex] + WriteIndex;
					if(AutoInstanceTransformRemapIndices.IsValidIndex(TransformArrayIndex))
					{
						AutoInstanceTransformRemapIndices[TransformArrayIndex] = TransformIndex;
						WrittenTransformCounts[AutoInstanceMeshIndex]++;
					}
				}
			}
		}
	}
}

void UGeometryCollection::UpdateGeometryDependentProperties()
{
#if WITH_EDITOR
	// Note: Currently, computing convex hulls also always computes proximity (if missing) as well as volumes and size.
	// If adding a condition where we do not compute convex hulls, make sure to still compute proximity, volumes and size here
	UpdateConvexGeometry();
#endif
}

void UGeometryCollection::UpdateConvexGeometryIfMissing()
{
	const bool bConvexAttributeMissing = !GeometryCollection->HasAttribute(FGeometryCollection::ConvexHullAttribute, FGeometryCollection::ConvexGroup);
	if (GeometryCollection && bConvexAttributeMissing)
	{
		UpdateConvexGeometry();
	}
}

void UGeometryCollection::UpdateConvexGeometry()
{
#if WITH_EDITOR
	if (GeometryCollection)
	{
		FGeometryCollectionConvexPropertiesInterface::FConvexCreationProperties ConvexProperties = GeometryCollection->GetConvexProperties();
		FGeometryCollectionConvexUtility::CreateNonOverlappingConvexHullData(GeometryCollection.Get(), ConvexProperties.FractionRemove, 
			ConvexProperties.SimplificationThreshold, ConvexProperties.CanExceedFraction, ConvexProperties.RemoveOverlaps, ConvexProperties.OverlapRemovalShrinkPercent);
		InvalidateCollection();
	}
#endif
}

void UGeometryCollection::PostInitProperties()
{
#if WITH_EDITORONLY_DATA
	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		AssetImportData = NewObject<UAssetImportData>(this, TEXT("AssetImportData"));
	}
#endif
	Super::PostInitProperties();
}

void UGeometryCollection::CacheMaterialDensity()
{
	CachedDensityFromPhysicsMaterialInGCm3 = 0;

	UPhysicalMaterial* PhysicsMaterialForDensity = PhysicsMaterial;
	if (!PhysicsMaterialForDensity)
	{
		PhysicsMaterialForDensity = GEngine ? GEngine->DefaultPhysMaterial.Get() : nullptr;
	}
	if (PhysicsMaterialForDensity)
	{
		CachedDensityFromPhysicsMaterialInGCm3 = PhysicsMaterialForDensity->Density;
	}
}

float UGeometryCollection::GetMassOrDensity(bool& bOutIsDensity) const
{
	return GetMassOrDensityInternal(bOutIsDensity, /* bCached */ true);
}

float UGeometryCollection::GetMassOrDensityInternal(bool& bOutIsDensity, bool bCached) const
{
	bOutIsDensity = bMassAsDensity;
	float MassOrDensity = bMassAsDensity ? Chaos::KgM3ToKgCm3(Mass) : Mass;
	
	if (bDensityFromPhysicsMaterial)
	{
		if (bCached && CachedDensityFromPhysicsMaterialInGCm3 > 0)
		{
			bOutIsDensity = true;
			MassOrDensity = Chaos::GCm3ToKgCm3(CachedDensityFromPhysicsMaterialInGCm3);
		}
		else
		{
			UPhysicalMaterial* PhysicsMaterialForDensity = PhysicsMaterial;
			if (!PhysicsMaterialForDensity)
			{
				PhysicsMaterialForDensity = GEngine ? GEngine->DefaultPhysMaterial.Get() : nullptr;
			}
			if (ensureMsgf(PhysicsMaterialForDensity, TEXT("bDensityFromPhysicsMaterial is true but no physics material has been set (and engine default cannot be found )")))
			{
				// materials only provide density
				bOutIsDensity = true;
				MassOrDensity = Chaos::GCm3ToKgCm3(PhysicsMaterialForDensity->Density);
			}
		}
	}
	return MassOrDensity;
}

void UGeometryCollection::GetSharedSimulationParams(FSharedSimulationParameters& OutParams) const
{
	const FGeometryCollectionSizeSpecificData& SizeSpecificDefault = GetDefaultSizeSpecificData();

	// we grab the non cached version because this is going to be used to generate the mass attribute which will eventually cache the density value if necessary
	bool bUseMassAsDensity = false;
	OutParams.Mass = GetMassOrDensityInternal(bUseMassAsDensity, false);
	OutParams.bMassAsDensity = bUseMassAsDensity;
	OutParams.MinimumMassClamp = MinimumMassClamp;

	FGeometryCollectionSizeSpecificData InfSize;
	if (SizeSpecificDefault.CollisionShapes.Num())
	{
		InfSize.CollisionShapes.SetNum(1); // @todo(chaos destruction collisions) : Add support for multiple shapes. 
		OutParams.MaximumCollisionParticleCount = SizeSpecificDefault.CollisionShapes[0].CollisionParticles.MaximumCollisionParticles;

		ECollisionTypeEnum SelectedCollisionType = SizeSpecificDefault.CollisionShapes[0].CollisionType;

		if (SelectedCollisionType == ECollisionTypeEnum::Chaos_Volumetric && SizeSpecificDefault.CollisionShapes[0].ImplicitType == EImplicitTypeEnum::Chaos_Implicit_LevelSet)
		{
			UE_LOGF(LogGeometryCollectionInternal, Verbose, "LevelSet geometry selected but non-particle collisions selected. Forcing particle-implicit collisions for %ls", *GetPathName());
			SelectedCollisionType = ECollisionTypeEnum::Chaos_Surface_Volumetric;
		}

		InfSize.CollisionShapes[0].CollisionType = SelectedCollisionType;
		InfSize.CollisionShapes[0].ImplicitType = SizeSpecificDefault.CollisionShapes[0].ImplicitType;
		InfSize.CollisionShapes[0].LevelSet.MinLevelSetResolution = SizeSpecificDefault.CollisionShapes[0].LevelSet.MinLevelSetResolution;
		InfSize.CollisionShapes[0].LevelSet.MaxLevelSetResolution = SizeSpecificDefault.CollisionShapes[0].LevelSet.MaxLevelSetResolution;
		InfSize.CollisionShapes[0].LevelSet.MinClusterLevelSetResolution = SizeSpecificDefault.CollisionShapes[0].LevelSet.MinClusterLevelSetResolution;
		InfSize.CollisionShapes[0].LevelSet.MaxClusterLevelSetResolution = SizeSpecificDefault.CollisionShapes[0].LevelSet.MaxClusterLevelSetResolution;
		InfSize.CollisionShapes[0].CollisionObjectReductionPercentage = SizeSpecificDefault.CollisionShapes[0].CollisionObjectReductionPercentage;
		InfSize.CollisionShapes[0].CollisionMarginFraction = SizeSpecificDefault.CollisionShapes[0].CollisionMarginFraction;
		InfSize.CollisionShapes[0].CollisionParticles.CollisionParticlesFraction = SizeSpecificDefault.CollisionShapes[0].CollisionParticles.CollisionParticlesFraction;
		InfSize.CollisionShapes[0].CollisionParticles.MaximumCollisionParticles = SizeSpecificDefault.CollisionShapes[0].CollisionParticles.MaximumCollisionParticles;
	}
	InfSize.MaxSize = TNumericLimits<float>::Max();
	OutParams.SizeSpecificData.SetNum(SizeSpecificData.Num() + 1);
	FillSharedSimulationSizeSpecificData(OutParams.SizeSpecificData[0], InfSize);

	for (int32 Idx = 0; Idx < SizeSpecificData.Num(); ++Idx)
	{
		FillSharedSimulationSizeSpecificData(OutParams.SizeSpecificData[Idx+1], SizeSpecificData[Idx]);
	}

	OutParams.bUseImportedCollisionImplicits = bImportCollisionFromSource;
	
	OutParams.SizeSpecificData.Sort();	//can we do this at editor time on post edit change?
}

bool UGeometryCollection::IsEmpty() const
{
	return (NumElements(FGeometryCollection::TransformGroup) == 0);
}

void UGeometryCollection::Reset()
{
	if (GeometryCollection.IsValid())
	{
		Modify(); 
		GeometryCollection->Reset();
		Materials.Empty();
		EmbeddedGeometryExemplar.Empty();
		AutoInstanceMeshes.Empty();
		InvalidateCollection();
	}
}

namespace UE::Dataflow::Private
{
	static void SetRandomBoneColor(TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>& InGeometryCollection)
	{
		TManagedArray<FLinearColor>& BoneColors = InGeometryCollection->BoneColor;

		const int32 NumBones = BoneColors.Num();
		FRandomStream RandomStream(NumBones);

		for (int32 Idx = 0; Idx < NumBones; ++Idx)
		{
			const uint8 R = static_cast<uint8>(RandomStream.FRandRange(5, 105));
			const uint8 G = static_cast<uint8>(RandomStream.FRandRange(5, 105));
			const uint8 B = static_cast<uint8>(RandomStream.FRandRange(5, 105));

			BoneColors[Idx] = FLinearColor(FColor(R, G, B, 255));
		}
	}
}

void UGeometryCollection::ResetFrom(const FManagedArrayCollection& InCollection, const TArray<UMaterial*>& InMaterials, bool bHasInternalMaterials)
{
	Reset();
	if (GeometryCollection.IsValid())
	{
		InCollection.CopyTo(GeometryCollection.Get());

		// todo(Chaos) : we could certainly run a "dependent attribute update method here instead of having to known about convex specifically 
		UpdateConvexGeometryIfMissing();
				
		Materials.Append(InMaterials);
		InitializeMaterials(bHasInternalMaterials);

		// Randomize BoneColor
		UE::Dataflow::Private::SetRandomBoneColor(GeometryCollection);
	}
}

void UGeometryCollection::ResetFrom(const FManagedArrayCollection& InCollection, const TArray<UMaterialInterface*>& InMaterialInstances, bool bHasInternalMaterials)
{
	Reset();
	if (GeometryCollection.IsValid())
	{
		InCollection.CopyTo(GeometryCollection.Get());
		// todo(Chaos) : we could certainly run a "dependent attribute update method here instead of having to known about convex specifically 
		UpdateConvexGeometryIfMissing();
		Materials.Append(InMaterialInstances);
		InitializeMaterials(bHasInternalMaterials);

		// Randomize BoneColor
		UE::Dataflow::Private::SetRandomBoneColor(GeometryCollection);
	}
}

/** AppendGeometry */
int32 UGeometryCollection::AppendGeometry(const UGeometryCollection & Element, bool ReindexAllMaterials, const FTransform& TransformRoot)
{
	Modify();
	InvalidateCollection();

	// add all materials
	// if there are none, we assume all material assignments in Element are shared by this GeometryCollection
	// otherwise, we assume all assignments come from the contained materials
	int32 MaterialIDOffset = 0;
	if (Element.Materials.Num() > 0)
	{
		MaterialIDOffset = Materials.Num();
		Materials.Append(Element.Materials);
	}	

	return GeometryCollection->AppendGeometry(*Element.GetGeometryCollection(), MaterialIDOffset, ReindexAllMaterials, TransformRoot);
}

/** NumElements */
int32 UGeometryCollection::NumElements(const FName& Group) const
{
	return GeometryCollection? GeometryCollection->NumElements(Group): 0;
}

/** RemoveElements */
void UGeometryCollection::RemoveElements(const FName & Group, const TArray<int32>& SortedDeletionList)
{
	Modify();
	GeometryCollection->RemoveElements(Group, SortedDeletionList);
	InvalidateCollection();
}

int UGeometryCollection::GetDefaultSizeSpecificDataIndex() const
{
	int LargestIndex = INDEX_NONE;
	float MaxSize = TNumericLimits<float>::Lowest();
	for (int i = 0; i < SizeSpecificData.Num(); i++)
	{
		const float SizeSpecificDataMaxSize = SizeSpecificData[i].MaxSize;
		if (MaxSize < SizeSpecificDataMaxSize)
		{
			MaxSize = SizeSpecificDataMaxSize;
			LargestIndex = i;
		}
	}
	check(LargestIndex != INDEX_NONE && LargestIndex < SizeSpecificData.Num());
	return LargestIndex;
}

/** Size Specific Data Access */
FGeometryCollectionSizeSpecificData& UGeometryCollection::GetDefaultSizeSpecificData()
{
	if (!SizeSpecificData.Num())
	{
		SizeSpecificData.Add(GeometryCollectionSizeSpecificDataDefaults());
	}
	const int DefaultSizeIndex = GetDefaultSizeSpecificDataIndex();
	return SizeSpecificData[DefaultSizeIndex];
}

const FGeometryCollectionSizeSpecificData& UGeometryCollection::GetDefaultSizeSpecificData() const
{
	ensure(SizeSpecificData.Num());
	const int DefaultSizeIndex = GetDefaultSizeSpecificDataIndex();
	return SizeSpecificData[DefaultSizeIndex];
}

/** ReindexMaterialSections */
void UGeometryCollection::ReindexMaterialSections()
{
	Modify();
	GeometryCollection->ReindexMaterials();
	InvalidateCollection();
}

UMaterialInterface* UGeometryCollection::GetBoneSelectedMaterial()
{
#if WITH_EDITORONLY_DATA
	return LoadObject<UMaterialInterface>(nullptr, GetSelectedMaterialPath(), nullptr, LOAD_None, nullptr);
#else
	return nullptr;
#endif
}

void UGeometryCollection::InitializeMaterials(bool bHasLegacyInternalMaterialsPairs)
{
	Modify();

	if (Materials.IsEmpty())
	{
		Materials.Add({}); // add an empty material is none exist
	}

	// Initialize the BoneSelectedMaterial separate from the materials on the collection
	UMaterialInterface* BoneSelectedMaterial = GetBoneSelectedMaterial();

	TManagedArray<int32>& MaterialIDs = GeometryCollection->MaterialID;

	// normally we filter out instances of the selection material ID, but if it's actually used on any face we have to keep it
	bool bBoneSelectedMaterialIsUsed = false;
	for (int32 FaceIdx = 0; FaceIdx < MaterialIDs.Num(); ++FaceIdx)
	{
		int32 FaceMaterialID = MaterialIDs[FaceIdx];
		if (FaceMaterialID < Materials.Num() && Materials[FaceMaterialID] == BoneSelectedMaterial)
		{
			bBoneSelectedMaterialIsUsed = true;
			break;
		}
	}

	TArray<UMaterialInterface*> FinalMaterials;
	if (bHasLegacyInternalMaterialsPairs)
	{
		// We're assuming that all materials are arranged in pairs, so first we collect these.
		using FMaterialPair = TPair<UMaterialInterface*, UMaterialInterface*>;
		TSet<FMaterialPair> MaterialSet;
		for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
		{
			UMaterialInterface* ExteriorMaterial = Materials[MaterialIndex];
			if (ExteriorMaterial == BoneSelectedMaterial && !bBoneSelectedMaterialIsUsed) // skip unused bone selected material
			{
				continue;
			}

			// If we have an odd number of materials, the last material duplicates itself.
			UMaterialInterface* InteriorMaterial = Materials[MaterialIndex];
			while (++MaterialIndex < Materials.Num())
			{
				if (Materials[MaterialIndex] == BoneSelectedMaterial && !bBoneSelectedMaterialIsUsed) // skip bone selected material
				{
					continue;
				}
				InteriorMaterial = Materials[MaterialIndex];
				break;
			}

			MaterialSet.Add(FMaterialPair(ExteriorMaterial, InteriorMaterial));
		}

		// create the final material array only containing unique materials
		// alternating exterior and interior materials
		TMap<UMaterialInterface*, int32> ExteriorMaterialPtrToArrayIndex;
		TMap<UMaterialInterface*, int32> InteriorMaterialPtrToArrayIndex;
		for (const FMaterialPair& Curr : MaterialSet)
		{
			// Add base material
			TTuple< UMaterialInterface*, int32> BaseTuple(Curr.Key, FinalMaterials.Add(Curr.Key));
			ExteriorMaterialPtrToArrayIndex.Add(BaseTuple);

			// Add interior material
			TTuple< UMaterialInterface*, int32> InteriorTuple(Curr.Value, FinalMaterials.Add(Curr.Value));
			InteriorMaterialPtrToArrayIndex.Add(InteriorTuple);
		}

		// Reassign material ID for each face given the new consolidated array of materials
		for (int32 Material = 0; Material < MaterialIDs.Num(); ++Material)
		{
			if (MaterialIDs[Material] < Materials.Num())
			{
				UMaterialInterface* OldMaterialPtr = Materials[MaterialIDs[Material]];
				if (MaterialIDs[Material] % 2 == 0)
				{
					MaterialIDs[Material] = *ExteriorMaterialPtrToArrayIndex.Find(OldMaterialPtr);
				}
				else
				{
					MaterialIDs[Material] = *InteriorMaterialPtrToArrayIndex.Find(OldMaterialPtr);
				}
			}
		}
	}
	else
	{
		// simple deduping process
		for (int32 MaterialIndex = 0; MaterialIndex < Materials.Num(); ++MaterialIndex)
		{
			UMaterialInterface* Material = Materials[MaterialIndex];
			if (Material == BoneSelectedMaterial && !bBoneSelectedMaterialIsUsed) // skip unused bone selected material
			{
				continue;
			}
			FinalMaterials.AddUnique(Material);
		}

		// Reassign material ID for each face given the new consolidated array of materials
		for (int32 MaterialIDIndex = 0; MaterialIDIndex < MaterialIDs.Num(); MaterialIDIndex++)
		{
			const int32 OldMaterialID = MaterialIDs[MaterialIDIndex];
			if (Materials.IsValidIndex(OldMaterialID))
			{
				UMaterialInterface* Material = Materials[OldMaterialID];
				MaterialIDs[MaterialIDIndex] = FinalMaterials.Find(Material);
			}
		}
	}

	// Set new material array on the collection
	Materials = FinalMaterials;

	// BoneSelectedMaterial is no longer stored in the general Materials array
	BoneSelectedMaterialIndex = INDEX_NONE;

	GeometryCollection->ReindexMaterials();
	InvalidateCollection();
}

int32 UGeometryCollection::AddNewMaterialSlot(bool bCopyLastMaterial)
{
	Modify();
	int32 NewIdx = Materials.Emplace();
	if (NewIdx > 0 && bCopyLastMaterial)
	{
		Materials[NewIdx] = Materials[NewIdx - 1];
	}

	InvalidateCollection();

	return NewIdx;
}

bool UGeometryCollection::RemoveLastMaterialSlot()
{
	if (Materials.Num() > 1)
	{
		Modify();
		Materials.Pop();
		InvalidateCollection();
		return true;
	}

	return false;
}

/** Returns true if there is anything to render */
bool UGeometryCollection::HasVisibleGeometry() const
{
	if(ensureMsgf(GeometryCollection.IsValid(), TEXT("Geometry Collection has an invalid internal collection")))
	{
		return ( (EnableNanite && RenderData && RenderData->bHasNaniteData) || GeometryCollection->HasVisibleGeometry());
	}

	return false;
}

struct FPackedHierarchyNode_Old
{
	FSphere		LODBounds[64];
	FSphere		Bounds[64];
	struct
	{
		uint32	MinLODError_MaxParentLODError;
		uint32	ChildStartReference;
		uint32	ResourcePageIndex_NumPages_GroupPartSize;
	} Misc[64];
};

FArchive& operator<<(FArchive& Ar, FPackedHierarchyNode_Old& Node)
{
	for (uint32 i = 0; i < 64; i++)
	{
		Ar << Node.LODBounds[i];
		Ar << Node.Bounds[i];
		Ar << Node.Misc[i].MinLODError_MaxParentLODError;
		Ar << Node.Misc[i].ChildStartReference;
		Ar << Node.Misc[i].ResourcePageIndex_NumPages_GroupPartSize;
	}

	return Ar;
}


struct FPageStreamingState_Old
{
	uint32			BulkOffset;
	uint32			BulkSize;
	uint32			PageUncompressedSize;
	uint32			DependenciesStart;
	uint32			DependenciesNum;
};

FArchive& operator<<(FArchive& Ar, FPageStreamingState_Old& PageStreamingState)
{
	Ar << PageStreamingState.BulkOffset;
	Ar << PageStreamingState.BulkSize;
	Ar << PageStreamingState.PageUncompressedSize;
	Ar << PageStreamingState.DependenciesStart;
	Ar << PageStreamingState.DependenciesNum;
	return Ar;
}

// Parse old Nanite data and throw it away. We need this to not crash when parsing old files.
static void SerializeOldNaniteData(FArchive& Ar, UGeometryCollection* Owner)
{
	check(Ar.IsLoading());

	int32 NumNaniteResources = 0;
	Ar << NumNaniteResources;

	for (int32 i = 0; i < NumNaniteResources; ++i)
	{
		FStripDataFlags StripFlags(Ar, 0);
		if (!StripFlags.IsAudioVisualDataStripped())
		{
			bool bLZCompressed;
			TArray< uint8 >						RootClusterPage;
			FByteBulkData						StreamableClusterPages;
			TArray< uint16 >					ImposterAtlas;
			TArray< FPackedHierarchyNode_Old >	HierarchyNodes;
			TArray< FPageStreamingState_Old >	PageStreamingStates;
			TArray< uint32 >					PageDependencies;

			Ar << bLZCompressed;
			Ar << RootClusterPage;
			StreamableClusterPages.Serialize(Ar, Owner, 0);
			Ar << PageStreamingStates;

			Ar << HierarchyNodes;
			Ar << PageDependencies;
			Ar << ImposterAtlas;
		}
	}
}

/** Serialize */
void UGeometryCollection::Serialize(FArchive& Ar)
{
	bool bCreateSimulationData = false;
	Ar.UsingCustomVersion(FDestructionObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Ar.UsingCustomVersion(FPhysicsObjectVersion::GUID);
	Ar.UsingCustomVersion(FFortniteMainBranchObjectVersion::GUID);
	
	Chaos::FChaosArchive ChaosAr(Ar);

	// The Geometry Collection we will be archiving. This may be replaced with a transient, stripped back Geometry Collection if we are cooking.
	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> ArchiveGeometryCollection = GeometryCollection;

	TObjectPtr<UDataflow> StrippedDataflowAsset = nullptr;
	FDataflowInstance StrippedDataflowInstance;
	bool bStrippedDataflowData = false;

	bool bIsCookedOrCooking = Ar.IsCooking();
	if ((bIsCookedOrCooking && Ar.IsSaving()) || (Ar.IsCountingMemory() && Ar.IsFilterEditorOnly()))
	{
#if WITH_EDITOR
		if (bStripOnCook)
		{
			// TODO: Since non-nanite path now stores mesh data in cooked build we may be able to unify 
			// the simplification of the Geometry Collection for both nanite and non-nanite cases.
			if (EnableNanite && HasNaniteData())
			{
				// If this is a cooked archive, we strip unnecessary data from the Geometry Collection to keep the memory footprint as small as possible.
				ArchiveGeometryCollection = GenerateMinimalGeometryCollection();
			}
			else
			{
				// non-nanite path where it may be necessary to remove geometry if the geometry collection is rendered using ISMPool or an external rendering system 
				ArchiveGeometryCollection = CopyCollectionAndRemoveGeometry(GeometryCollection);
			}
		}
		else
		{
			// do we need to remove the simplicial attribute ? 
			if (false == FGeometryCollection::AreCollisionParticlesEnabled())
			{
				ArchiveGeometryCollection = TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>(new FGeometryCollection);
				const TArray<FName> NoGroupsToSkip;
				const TArray<TTuple<FName, FName>> AttributesToSkip{ { FGeometryDynamicCollection::SimplicialsAttribute, FTransformCollection::TransformGroup } };
				GeometryCollection->CopyTo(ArchiveGeometryCollection.Get(), NoGroupsToSkip, AttributesToSkip);
			}
		}

		// The dataflow asset is only needed for the editor, so we just remove it when cooking 
		StrippedDataflowAsset = DataflowAsset;
		DataflowAsset = nullptr;
		StrippedDataflowInstance = DataflowInstance;
		DataflowInstance = FDataflowInstance();
		bStrippedDataflowData = true;
#endif
	}

#if WITH_EDITOR
	//Early versions did not have tagged properties serialize first
	if (Ar.CustomVer(FDestructionObjectVersion::GUID) < FDestructionObjectVersion::GeometryCollectionInDDC)
	{
		if (Ar.IsLoading())
		{
			GeometryCollection->Serialize(ChaosAr);
		}
		else
		{
			ArchiveGeometryCollection->Serialize(ChaosAr);
		}
	}

	if (Ar.CustomVer(FDestructionObjectVersion::GUID) < FDestructionObjectVersion::AddedTimestampedGeometryComponentCache)
	{
		if (Ar.IsLoading())
		{
			// Strip old recorded cache data
			int32 DummyNumFrames;
			TArray<TArray<FTransform>> DummyTransforms;

			Ar << DummyNumFrames;
			DummyTransforms.SetNum(DummyNumFrames);
			for (int32 Index = 0; Index < DummyNumFrames; ++Index)
			{
				Ar << DummyTransforms[Index];
			}
		}
	}
	else
#endif
	{
		// Push up the chain to hit tagged properties too
		// This should have always been in here but because we have saved assets
		// from before this line was here it has to be gated
		Super::Serialize(Ar);
	}

	// Important : this needs to remain after the call to Super::Serialize
	if (bStrippedDataflowData)
	{
#if WITH_EDITORONLY_DATA
		DataflowAsset = StrippedDataflowAsset;
#endif
		DataflowInstance = StrippedDataflowInstance;
	}

	if ((Ar.IsLoading() || Ar.IsSaving()) && !SizeSpecificData.Num())
	{
		// Validation is necessary when loading old version and when saving newly created version
		// that might not have created the defaults yet; the defaults are used during EnsureDataIsCooked.
		ValidateSizeSpecificDataDefaults();
	}

	if (Ar.CustomVer(FDestructionObjectVersion::GUID) < FDestructionObjectVersion::DensityUnitsChanged)
	{
		if (bMassAsDensity)
		{
			Mass = Chaos::KgCm3ToKgM3(Mass);
		}
	}

	if (Ar.CustomVer(FDestructionObjectVersion::GUID) >= FDestructionObjectVersion::GeometryCollectionInDDC)
	{
		Ar << bIsCookedOrCooking;
	}

	//new versions serialize geometry collection after tagged properties
	if (Ar.CustomVer(FDestructionObjectVersion::GUID) >= FDestructionObjectVersion::GeometryCollectionInDDCAndAsset)
	{
#if WITH_EDITOR
		if (Ar.IsSaving() && !Ar.IsTransacting())
		{
			constexpr bool bAllowCopyFromDDC = false;
			constexpr bool bIsTransacting = false; // the surrounding if statement garantees that 
			EnsureSimulationDataIsCooked(bIsTransacting, bAllowCopyFromDDC);
		}
#endif
		if (Ar.IsLoading() || (Ar.IsCountingMemory() && !Ar.IsFilterEditorOnly()))
		{
			GeometryCollection->Serialize(ChaosAr);

			// Legacy clean-up: previous versions persisted FVertexBoneWeightsFacade's
			// SkeletalMesh attribute (TManagedArray<TObjectPtr<UObject>>). It is Dataflow
			// editor-time scratch and the DDC simulation cooker's FMemoryWriter cannot
			// serialize FObjectPtr. Drop it before EnsureSimulationDataIsCooked runs below.
			if (Ar.IsLoading() && GeometryCollection)
			{
				const FName SkelMeshAttr = GeometryCollection::Facades::FVertexBoneWeightsFacade::SkeletalMeshAttributeName;
				if (GeometryCollection->HasAttribute(SkelMeshAttr, FGeometryCollection::GeometryGroup))
				{
					GeometryCollection->RemoveAttribute(SkelMeshAttr, FGeometryCollection::GeometryGroup);
				}
			}
		}
		else
		{
			ArchiveGeometryCollection->Serialize(ChaosAr);
		}

		TManagedArray<Chaos::FImplicitObjectPtr>* NewAttr = ArchiveGeometryCollection->FindAttributeTyped<Chaos::FImplicitObjectPtr>(
		FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
		if(!NewAttr && Ar.IsLoading())
		{
			const int32 NumElems = GeometryCollection->NumElements(FTransformCollection::TransformGroup);

			TArray<Chaos::FImplicitObjectPtr> ImplicitObjects;
			ImplicitObjects.SetNum(NumElems);
			
			const TManagedArray<TUniquePtr<Chaos::FImplicitObject>>* OldAttrA = ArchiveGeometryCollection->FindAttributeTyped<TUniquePtr<Chaos::FImplicitObject>>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
			const TManagedArray<TSharedPtr<Chaos::FImplicitObject, ESPMode::ThreadSafe>>* OldAttrB = ArchiveGeometryCollection->FindAttributeTyped<TSharedPtr<Chaos::FImplicitObject, ESPMode::ThreadSafe>>(FGeometryDynamicCollection::SharedImplicitsAttribute, FTransformCollection::TransformGroup);
			const TManagedArray<TSharedPtr<Chaos::FImplicitObject, ESPMode::ThreadSafe>>* OldAttrC = ArchiveGeometryCollection->FindAttributeTyped<TSharedPtr<Chaos::FImplicitObject, ESPMode::ThreadSafe>>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);

			// Some geometry collection can still store several of those arrays
			// We need to make sure to remove all of them and keep the last good one 
			if (OldAttrA)
			{
				for (int32 Index = 0; Index < NumElems; ++Index)
				{
					if( (*OldAttrA)[Index] != nullptr)
					{
						ImplicitObjects[Index] = Chaos::FImplicitObjectPtr((*OldAttrA)[Index]->DeepCopyGeometry());
					};
				}
				ArchiveGeometryCollection->RemoveAttribute(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
			}
			if (OldAttrB)
			{
				for (int32 Index = 0; Index < NumElems; ++Index)
				{
					if( (*OldAttrB)[Index] != nullptr)
                	{
						ImplicitObjects[Index] = Chaos::FImplicitObjectPtr((*OldAttrB)[Index]->DeepCopyGeometry());
					}
				}
				ArchiveGeometryCollection->RemoveAttribute(FGeometryDynamicCollection::SharedImplicitsAttribute, FTransformCollection::TransformGroup);
			}
			if (OldAttrC)
			{
				for (int32 Index = 0; Index < NumElems; ++Index)
				{
					if( (*OldAttrC)[Index] != nullptr)
					{
						ImplicitObjects[Index] = Chaos::FImplicitObjectPtr((*OldAttrC)[Index]->DeepCopyGeometry());
					}
				}
				ArchiveGeometryCollection->RemoveAttribute(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
			}
			NewAttr = &ArchiveGeometryCollection->AddAttribute<Chaos::FImplicitObjectPtr>(FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);
			for (int32 Index = 0; Index < NumElems; ++Index)
			{
				(*NewAttr)[Index] = ImplicitObjects[Index];
			}
		}
	}

	if (Ar.CustomVer(FDestructionObjectVersion::GUID) < FDestructionObjectVersion::GroupAndAttributeNameRemapping)
	{
		ArchiveGeometryCollection->UpdateOldAttributeNames();
		InvalidateCollection();
		bCreateSimulationData = true;
	}

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) == FUE5MainStreamObjectVersion::GeometryCollectionNaniteData || 
		(Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::GeometryCollectionNaniteCooked &&
		Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::GeometryCollectionNaniteTransient))
	{
		// This legacy version serialized structure information into archive, but the data is transient.
		// Just load it and throw away here, it will be rebuilt later and resaved past this point.
		SerializeOldNaniteData(ChaosAr, this);
	}

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) >= FUE5MainStreamObjectVersion::GeometryCollectionNaniteTransient)
	{
		bool bCooked = Ar.IsCooking();
		Ar << bCooked;
		if (bCooked)
		{
			if (RenderData == nullptr)
			{
				RenderData = MakeUnique<FGeometryCollectionRenderData>();
			}

			RenderData->Serialize(ChaosAr, *this);
		}
	}

	{
		TManagedArray<Chaos::FConvexPtr>* NewAttr = ArchiveGeometryCollection->FindAttributeTyped<Chaos::FConvexPtr>(
			FTransformCollection::ConvexHullAttribute, FTransformCollection::ConvexGroup);
		if(!NewAttr && Ar.IsLoading())
		{
			const int32 NumElems = GeometryCollection->NumElements(FTransformCollection::ConvexGroup);
			
			TArray<Chaos::FConvexPtr> ImplicitObjects;
			ImplicitObjects.SetNum(NumElems);
			
			if( TManagedArray<TUniquePtr<Chaos::FConvex>>* OldAttr = ArchiveGeometryCollection->FindAttributeTyped<TUniquePtr<Chaos::FConvex>>(
				FTransformCollection::ConvexHullAttribute, FTransformCollection::ConvexGroup))
			{
				for (int32 Index = 0; Index < NumElems; ++Index)
				{
					if((*OldAttr)[Index] != nullptr)
					{
						ImplicitObjects[Index] = Chaos::FConvexPtr((*OldAttr)[Index].Release());
					}
				}
				ArchiveGeometryCollection->RemoveAttribute(FTransformCollection::ConvexHullAttribute, FTransformCollection::ConvexGroup);
			}
			NewAttr = &ArchiveGeometryCollection->AddAttribute<Chaos::FConvexPtr>(FTransformCollection::ConvexHullAttribute, FTransformCollection::ConvexGroup);
			for (int32 Index = 0; Index < NumElems; ++Index)
			{
				(*NewAttr)[Index] = ImplicitObjects[Index];
			}
		}
	}
	{
		TManagedArray<Chaos::FImplicitObjectPtr>* NewAttr = ArchiveGeometryCollection->FindAttributeTyped<Chaos::FImplicitObjectPtr>(
			FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup);
			
		if(!NewAttr && Ar.IsLoading())
		{
			const int32 NumElems = GeometryCollection->NumElements(FGeometryCollection::TransformGroup);
			
			TArray<Chaos::FImplicitObjectPtr> ImplicitObjects;
			ImplicitObjects.SetNum(NumElems);
			
			if( TManagedArray<TSharedPtr<Chaos::FImplicitObject, ESPMode::ThreadSafe>>* OldAttr = ArchiveGeometryCollection->FindAttributeTyped<TSharedPtr<Chaos::FImplicitObject, ESPMode::ThreadSafe>>(
				FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup))
			{
				for (int32 Index = 0; Index < NumElems; ++Index)
				{
					if((*OldAttr)[Index] != nullptr)
					{
						ImplicitObjects[Index] = Chaos::FImplicitObjectPtr((*OldAttr)[Index]->DeepCopyGeometry());
					}
				}
				ArchiveGeometryCollection->RemoveAttribute(FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup);
			}
			NewAttr = &ArchiveGeometryCollection->AddAttribute<Chaos::FImplicitObjectPtr>(FGeometryCollection::ExternalCollisionsAttribute, FGeometryCollection::TransformGroup);
			for (int32 Index = 0; Index < NumElems; ++Index)
			{
				(*NewAttr)[Index] = ImplicitObjects[Index];
			}
		}
	}

	// will generate convex bodies when they dont exist. 
	if (Ar.CustomVer(FUE5ReleaseStreamObjectVersion::GUID) < FUE5ReleaseStreamObjectVersion::GeometryCollectionConvexDefaults
		&& Ar.CustomVer(FPhysicsObjectVersion::GUID) < FPhysicsObjectVersion::GeometryCollectionConvexDefaults)
	{
#if WITH_EDITOR
		if (bGeometryCollectionEnableForcedConvexGenerationInSerialize)
		{
			if (!FGeometryCollectionConvexUtility::HasConvexHullData(GeometryCollection.Get()) &&
				GeometryCollection::SizeSpecific::UsesImplicitCollisionType(SizeSpecificData, EImplicitTypeEnum::Chaos_Implicit_Convex))
			{
				GeometryCollection::SizeSpecific::SetImplicitCollisionType(SizeSpecificData, EImplicitTypeEnum::Chaos_Implicit_Box, EImplicitTypeEnum::Chaos_Implicit_Convex);
				bCreateSimulationData = true;
				InvalidateCollection();
			}
		}
#endif
	}

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::GeometryCollectionPerChildDamageThreshold)
	{
		// prior this version, damage threshold were computed per cluster and propagated to children
		PerClusterOnlyDamageThreshold = true; 
	}

	if (Ar.CustomVer(FUE5MainStreamObjectVersion::GUID) < FUE5MainStreamObjectVersion::GeometryCollectionDamagePropagationData)
	{
		// prior this version, damage propagation was not enabled by default
		DamagePropagationData.bEnabled = false;
	}

	if (Ar.IsLoading() && !bIsCookedOrCooking && BoneSelectedMaterialIndex != INDEX_NONE)
	{
		if (Materials.IsValidIndex(BoneSelectedMaterialIndex))
		{
			// Remove the material assuming it's the last in the list (otherwise, leave it, as it's not clear why it would be in that state)
			if (BoneSelectedMaterialIndex == Materials.Num() - 1)
			{
				Materials.RemoveAt(BoneSelectedMaterialIndex);
			}
		}
		BoneSelectedMaterialIndex = INDEX_NONE;
	}

	// Make sure the root index is properly set 
	if (RootIndex == INDEX_NONE)
	{
		UpdateRootIndex();
	}

	// Generate root to leave order lookup
	CacheBreadthFirstTransformIndices();
	// Generate transform remap for AutoInstanceMeshes instances
	CacheAutoInstanceTransformRemapIndices();

	if (Ar.IsLoading())
	{
		FillAutoInstanceMeshesInstancesIfNeeded();
	}

#if WITH_EDITORONLY_DATA
	if (bCreateSimulationData)
	{
		CreateSimulationData();
	}

	//for all versions loaded, make sure loaded content is built
 	if (Ar.IsLoading())
	{
		// note: don't allow copy from DDC here, since we've already loaded the data above, and the DDC data does not include any data migrations performed by the load
		constexpr bool bAllowCopyFromDDC = false;
		EnsureSimulationDataIsCooked(Ar.IsTransacting(), bAllowCopyFromDDC);
	}
#endif

	if (Ar.IsLoading() && Ar.CustomVer(FFortniteMainBranchObjectVersion::GUID) < FFortniteMainBranchObjectVersion::GeometryCollectionConvertVertexColorToSRGB)
	{
		// disable sRGB conversion for old assets to keep the default behavior from before this setting existed
		// (new assets will default-enable the conversion, because that matches static meshes and is the more-expected behavior)
		bConvertVertexColorsToSRGB = false;
	}
}

const TCHAR* UGeometryCollection::GetSelectedMaterialPath()
{
	return TEXT("/Engine/EditorMaterials/GeometryCollection/SelectedGeometryMaterial.SelectedGeometryMaterial");
}

void UGeometryCollection::SetEnableNanite(bool bValue)
{
	if (EnableNanite != bValue)
	{
		EnableNanite = bValue;

#if WITH_EDITOR
		RebuildRenderData();
#endif
	}
}

void UGeometryCollection::SetConvertVertexColorsToSRGB(bool bValue)
{
	if (bConvertVertexColorsToSRGB != bValue)
	{
		bConvertVertexColorsToSRGB = bValue;

#if WITH_EDITOR
		RebuildRenderData();
#endif
	}
}

void UGeometryCollection::FillAutoInstanceMeshesInstancesIfNeeded()
{
	// make sure the instanced meshes have their instance count properly set 
	if (GeometryCollection && AutoInstanceMeshes.Num() > 0 && AutoInstanceMeshes[0].NumInstances == 0)
	{
		//  make sure to rest all of it first 
		for (FGeometryCollectionAutoInstanceMesh& AutoInstanceMesh : AutoInstanceMeshes)
		{
			AutoInstanceMesh.NumInstances = 0;
		}

		const GeometryCollection::Facades::FCollectionInstancedMeshFacade InstancedMeshFacade(*GeometryCollection);
		if (InstancedMeshFacade.IsValid())
		{
			const int32 NumTransforms = GeometryCollection->Children.Num();
			for (int32 TransformIndex = 0; TransformIndex < NumTransforms; TransformIndex++)
			{
				// only applies to leaves nodes
				if (GeometryCollection->Children[TransformIndex].Num() == 0)
				{
					const int32 AutoInstanceMeshIndex = InstancedMeshFacade.GetIndex(TransformIndex);
					if (AutoInstanceMeshes.IsValidIndex(AutoInstanceMeshIndex))
					{
						AutoInstanceMeshes[AutoInstanceMeshIndex].NumInstances++;
					}
				}
			}
		}
		else
		{
			UE_LOGF(LogGeometryCollectionInternal, Warning, "[%ls] Could not find AutoInstanceMeshIndex attribute but the asset as instanced meshes assigned, you may need to regenerate this asset", *GetPathName());
		}
	}
}

#if WITH_EDITOR

void UGeometryCollection::CreateSimulationDataImp(bool bCopyFromDDC)
{
	COOK_STAT(auto Timer = GeometryCollectionCookStats::UsageStats.TimeSyncWork());

	// Skips the DDC fetch entirely for testing the builder without adding to the DDC
	const static bool bSkipDDC = false;

	//Use the DDC to build simulation data. If we are loading in the editor we then serialize this data into the geometry collection
	TArray<uint8> DDCData;
	FDerivedDataGeometryCollectionCooker* GeometryCollectionCooker = new FDerivedDataGeometryCollectionCooker(*this);

	if (GeometryCollectionCooker->CanBuild())
	{
		if (bSkipDDC)
		{
			GeometryCollectionCooker->Build(DDCData);
			COOK_STAT(Timer.AddMiss(DDCData.Num()));
		}
		else
		{
			bool bBuilt = false;
			const bool bSuccess = GetDerivedDataCacheRef().GetSynchronous(GeometryCollectionCooker, DDCData, &bBuilt);
			COOK_STAT(Timer.AddHitOrMiss(!bSuccess || bBuilt ? FCookStats::CallStats::EHitOrMiss::Miss : FCookStats::CallStats::EHitOrMiss::Hit, DDCData.Num()));
		}

		if (bCopyFromDDC)
		{
			FMemoryReader Ar(DDCData, true);	// Must be persistent for BulkData to serialize
			Chaos::FChaosArchive ChaosAr(Ar);
			GeometryCollection->Serialize(ChaosAr);
		}
	}
}

void UGeometryCollection::CreateSimulationData()
{
	CreateSimulationDataImp(/*bCopyFromDDC=*/false);
	SimulationDataGuid = StateGuid;
}

void UGeometryCollection::CreateSimulationDataIfNeeded()
{
	if (IsSimulationDataDirty() || bGeometryCollectionAlwaysRecreateSimulationData)
	{
		CreateSimulationData();
	}
}


void UGeometryCollection::CreateRenderDataImp(bool bCopyFromDDC)
{
	COOK_STAT(auto Timer = GeometryCollectionCookStats::UsageStats.TimeSyncWork());

	// Skips the DDC fetch entirely for testing the builder without adding to the DDC
	const static bool bSkipDDC = false;

	//Use the DDC to build simulation data. If we are loading in the editor we then serialize this data into the geometry collection
	TArray<uint8> DDCData;
	FDerivedDataGeometryCollectionRenderDataCooker* GeometryCollectionCooker = new FDerivedDataGeometryCollectionRenderDataCooker(*this);

	if (GeometryCollectionCooker->CanBuild())
	{
		if (bSkipDDC)
		{
			GeometryCollectionCooker->Build(DDCData);
			COOK_STAT(Timer.AddMiss(DDCData.Num()));
		}
		else
		{
			bool bBuilt = false;
			const bool bSuccess = GetDerivedDataCacheRef().GetSynchronous(GeometryCollectionCooker, DDCData, &bBuilt);
			COOK_STAT(Timer.AddHitOrMiss(!bSuccess || bBuilt ? FCookStats::CallStats::EHitOrMiss::Miss : FCookStats::CallStats::EHitOrMiss::Hit, DDCData.Num()));
		}

		if (bCopyFromDDC)
		{
			FMemoryReader Ar(DDCData, true);	// Must be persistent for BulkData to serialize
			Chaos::FChaosArchive ChaosAr(Ar);

			RenderData = MakeUnique<FGeometryCollectionRenderData>();
			RenderData->Serialize(ChaosAr, *this);
		}
	}
}

void UGeometryCollection::RebuildRenderData()
{
	if (RenderDataGuid != StateGuid)
	{
		// Release RenderData render resources (happens on render thread) and defer deletion to the render thread for after ReleaseResources.
		if (FGeometryCollectionRenderData* RenderDataToDelete = RenderData.Release())
		{
			RenderDataToDelete->ReleaseResources();
			ENQUEUE_RENDER_COMMAND(SetResourceRenderThread)([RenderDataToDelete](FRHICommandListImmediate& RHICmdList)
			{
				delete RenderDataToDelete;
			});
		}

		// Create new RenderData and initialize render resources (happens on render thread).
		FGeometryCollectionRenderData::FCreateDesc CreateDesc = {
			.bEnableNanite = EnableNanite,
			.bEnableNaniteFallback = bEnableNaniteFallback,
			.bUseFullPrecisionUVs = bUseFullPrecisionUVs,
			.bConvertVertexColorsToSRGB = bConvertVertexColorsToSRGB,
			.NaniteMinimumResidencyInKB = NaniteMinimumResidencyInKB
		};

		RenderData = FGeometryCollectionRenderData::Create(*GetGeometryCollection(), CreateDesc);
		InitResources();

		PropagateMarkDirtyToComponents();
		RenderDataGuid = StateGuid;
		ValidRenderDataGuid = StateGuid;
	}
}

void UGeometryCollection::PropagateMarkDirtyToComponents() const
{
	for (TObjectIterator<UGeometryCollectionComponent> It(RF_ClassDefaultObject, false, EInternalObjectFlags::Garbage); It; ++It)
	{
		if (It->RestCollection == this)
		{
			It->MarkRenderStateDirty();
			It->MarkRenderDynamicDataDirty();
		}
	}
}

void  UGeometryCollection::PropagateTransformUpdateToComponents() const
{
	for (TObjectIterator<UGeometryCollectionComponent> It(RF_ClassDefaultObject, false, EInternalObjectFlags::Garbage); It; ++It)
	{
		if (It->RestCollection == this)
		{
			// make sure to reset the rest collection to make sure the internal state of the components is up to date 
			// but we do not apply asset default to avoid overriding the existing overrides
			It->SetRestCollection(this, false /* bApplyAssetDefaults */);
		}
	}
}

void UGeometryCollection::SetRootProxiesFromGeometrySources()
{
	RootProxyData.Reset();

	// make sure the root index is properly cached
	if (RootIndex == INDEX_NONE)
	{
		UpdateRootIndex();
	}

	FTransform3f RootTransform = FTransform3f::Identity;
	if (GeometryCollection && GeometryCollection->Transform.IsValidIndex(RootIndex))
	{
		RootTransform = GeometryCollection->Transform[RootIndex];
	}

	for (const FGeometryCollectionSource& Source : GeometrySource)
	{
		UObject* SourceMesh = Source.SourceGeometryObject.TryLoad();
		if (UStaticMesh* SourceStaticMesh = Cast<UStaticMesh>(SourceMesh))
		{
			RootProxyData.ProxyMeshes.Add(SourceStaticMesh);

			const FTransform3f MeshTransform = FTransform3f(Source.LocalTransform).GetRelativeTransform(RootTransform);
			RootProxyData.MeshTransforms.Add(MeshTransform);

			FGeometryCollectionProxyMeshMaterials& MeshMaterials = RootProxyData.MeshOverrideMaterials.AddDefaulted_GetRef();
			MeshMaterials.Materials = Source.SourceMaterial;
		}
	}
}

TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> UGeometryCollection::GenerateMinimalGeometryCollection() const
{
	TMap<FName, TSet<FName>> SkipList;
	static TSet<FName> GeometryGroups{ FGeometryCollection::GeometryGroup, FGeometryCollection::VerticesGroup, FGeometryCollection::FacesGroup };
	if (bStripOnCook)
	{
		for (const FName& GeometryGroup : GeometryGroups)
		{
			TSet<FName>& SkipAttributes = SkipList.Add(GeometryGroup);
			SkipAttributes.Append(GeometryCollection->AttributeNames(GeometryGroup));
		}
	}

	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> DuplicateGeometryCollection(new FGeometryCollection());
	DuplicateGeometryCollection->AddAttribute<bool>(FGeometryCollection::SimulatableParticlesAttribute, FTransformCollection::TransformGroup);
	DuplicateGeometryCollection->AddAttribute<FVector3f>("InertiaTensor", FGeometryCollection::TransformGroup);
	DuplicateGeometryCollection->AddAttribute<float>("Mass", FGeometryCollection::TransformGroup);
	DuplicateGeometryCollection->AddAttribute<FTransform>("MassToLocal", FGeometryCollection::TransformGroup);
	DuplicateGeometryCollection->AddAttribute<Chaos::FImplicitObjectPtr>(
		FGeometryDynamicCollection::ImplicitsAttribute, FTransformCollection::TransformGroup);

	// TODO(chaos) : for now, let's fix up the missing attributes and we should consolidate GenerateMinimalGeometryCollection and CopyCollectionAndRemoveGeometry for later releases
	// we should also probably have a way to specify which attributes are needed in the final collection instead of hardcoding it here
	static const FName AnchoredAttributeName("Anchored");
	if (GeometryCollection->HasAttribute(AnchoredAttributeName, FTransformCollection::TransformGroup))
	{
		DuplicateGeometryCollection->AddAttribute<bool>(AnchoredAttributeName, FTransformCollection::TransformGroup);
	}
	static const FName AutoInstanceMeshIndexAttributeName("AutoInstanceMeshIndex");
	if (GeometryCollection->HasAttribute(AutoInstanceMeshIndexAttributeName, FTransformCollection::TransformGroup))
	{
		DuplicateGeometryCollection->AddAttribute<int32>(AutoInstanceMeshIndexAttributeName, FTransformCollection::TransformGroup);
	}
	static const FName LevelAttributeName("Level");
	if (GeometryCollection->HasAttribute(LevelAttributeName, FTransformCollection::TransformGroup))
	{
		DuplicateGeometryCollection->AddAttribute<int32>(LevelAttributeName, FTransformCollection::TransformGroup);
	}
	static const FName VolumeAttributeName("Volume");
	if (GeometryCollection->HasAttribute(VolumeAttributeName, FTransformCollection::TransformGroup))
	{
		DuplicateGeometryCollection->AddAttribute<float>(VolumeAttributeName, FTransformCollection::TransformGroup);
	}
	static const FName SizeAttributeName("Size");
	if (GeometryCollection->HasAttribute(SizeAttributeName, FTransformCollection::TransformGroup))
	{
		DuplicateGeometryCollection->AddAttribute<float>(SizeAttributeName, FTransformCollection::TransformGroup);
	}

	DuplicateGeometryCollection->CopyMatchingAttributesFrom(*GeometryCollection, &SkipList);

	// If we've removed all geometry, we need to make sure any references to that geometry are removed.
	// We also need to resize geometry groups to ensure that they are empty.
	if (bStripOnCook)
	{
		const TManagedArray<int32>& TransformToGeometryIndex = DuplicateGeometryCollection->GetAttribute<int32>("TransformToGeometryIndex", FTransformCollection::TransformGroup);

		//
		// Copy the bounds to the TransformGroup.
		//  @todo(nanite.bounds) : Rely on Nanite bounds in the component instead and dont copy here
		//
		if (!DuplicateGeometryCollection->HasAttribute("BoundingBox", "Transform"))
		{
			DuplicateGeometryCollection->AddAttribute<FBox>("BoundingBox", "Transform");
		}

		const int32 NumTransforms = GeometryCollection->NumElements(FGeometryCollection::TransformGroup);
		TManagedArray<FBox>& TransformBounds = DuplicateGeometryCollection->ModifyAttribute<FBox>("BoundingBox", "Transform");
		const TManagedArray<FBox>& GeometryBounds = GeometryCollection->GetAttribute<FBox>("BoundingBox", "Geometry");

		for (int TransformIndex = 0; TransformIndex < NumTransforms; TransformIndex++)
		{
			const int32 GeometryIndex = TransformToGeometryIndex[TransformIndex];
			if (GeometryIndex != INDEX_NONE)
			{
				TransformBounds[TransformIndex] = GeometryBounds[GeometryIndex];
			}
			else
			{
				TransformBounds[TransformIndex].Init();
			}
		}

		//
		//  Clear the geometry and the transforms connection to it. 
		//
		//TransformToGeometryIndex.Fill(INDEX_NONE);
		for (const FName& GeometryGroup : GeometryGroups)
		{
			DuplicateGeometryCollection->EmptyGroup(GeometryGroup);
		}
	}
	return DuplicateGeometryCollection;
}

TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> UGeometryCollection::CopyCollectionAndRemoveGeometry(const TSharedPtr<const FGeometryCollection, ESPMode::ThreadSafe>& CollectionToCopy)
{
	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionToReturn(new FGeometryCollection());

	const TArray<FName> GroupsToSkip{ FGeometryCollection::GeometryGroup, FGeometryCollection::VerticesGroup, FGeometryCollection::FacesGroup };
	const TArray<TTuple<FName, FName>> AttributesToSkip{ { FGeometryDynamicCollection::SimplicialsAttribute, FTransformCollection::TransformGroup } };

	CollectionToCopy->CopyTo(GeometryCollectionToReturn.Get(), GroupsToSkip, AttributesToSkip);

	if (FGeometryCollection::AreCollisionParticlesEnabled())
	{
		// recreate the simplicial attribute since we cannot copy it and we skipped it 
		using FSimplicialUniquePtr = TUniquePtr<FCollisionStructureManager::FSimplicial>;
		if (const TManagedArray<FSimplicialUniquePtr>* SourceSimplicials = CollectionToCopy->FindAttribute<FSimplicialUniquePtr>(FGeometryDynamicCollection::SimplicialsAttribute, FTransformCollection::TransformGroup))
		{
			TManagedArray<FSimplicialUniquePtr>& SimplicialsToWrite = GeometryCollectionToReturn->AddAttribute<FSimplicialUniquePtr>(FGeometryDynamicCollection::SimplicialsAttribute, FTransformCollection::TransformGroup);
			for (int32 Index = SourceSimplicials->Num() - 1; 0 <= Index; Index--)
			{
				SimplicialsToWrite[Index].Reset((*SourceSimplicials)[Index] ? (*SourceSimplicials)[Index]->NewCopy() : nullptr);
			}
		}
	}

	// since we are removing the bounding box attribute from the geometry group we need to move it to the transform group 
	const TManagedArray<FBox>& GeometryBounds = CollectionToCopy->GetAttribute<FBox>("BoundingBox", "Geometry");
	const TManagedArray<int32>& TransformToGeometryIndexArray = CollectionToCopy->TransformToGeometryIndex;

	TManagedArray<FBox>& TransformBounds = GeometryCollectionToReturn->AddAttribute<FBox>("BoundingBox", "Transform");
	
	for (int TransformIndex = 0; TransformIndex < TransformBounds.Num(); TransformIndex++)
	{
		const int32 GeometryIndex = TransformToGeometryIndexArray[TransformIndex];
		if (GeometryIndex != INDEX_NONE)
		{
			TransformBounds[TransformIndex] = GeometryBounds[GeometryIndex];
		}
		else
		{
			TransformBounds[TransformIndex].Init();
		}
	}

	return GeometryCollectionToReturn;
}

#endif

FGeometryCollectionRenderResourceSizeInfo UGeometryCollection::GetRenderResourceSizeInfo() const
{
	FGeometryCollectionRenderResourceSizeInfo InfoOut;
	const FGeometryCollectionMeshResources& MeshResources = RenderData->MeshResource;
	InfoOut.MeshResourcesSize += MeshResources.IndexBuffer.GetIndexDataSize();
	InfoOut.MeshResourcesSize += MeshResources.PositionVertexBuffer.GetAllocatedSize();
	InfoOut.MeshResourcesSize += MeshResources.StaticMeshVertexBuffer.GetResourceSize();
	InfoOut.MeshResourcesSize += MeshResources.ColorVertexBuffer.GetAllocatedSize();
	InfoOut.MeshResourcesSize += MeshResources.BoneMapVertexBuffer.GetAllocatedSize();

	InfoOut.NaniteResourcesSize += GetNaniteResourcesSize(*RenderData->NaniteResourcesPtr);

#if RHI_RAYTRACING
	for (const FRayTracingGeometry& PartRayTracingGeometry : RenderData->RayTracingResource.PartGeometries)
	{
		// TODO: Have a dedicated size for ray tracing?
		InfoOut.MeshResourcesSize += PartRayTracingGeometry.RawData.GetAllocatedSize();
	}
#endif

	return InfoOut;
}

void UGeometryCollection::InitResources()
{
	if (RenderData)
	{
		RenderData->InitResources(*this);
	}
}

void UGeometryCollection::ReleaseResources()
{
	if (RenderData)
	{
		RenderData->ReleaseResources();
	}
}

void UGeometryCollection::InvalidateCollection()
{
	if(!IsTemplate())
	{
		StateGuid = FGuid::NewGuid();
	}
	UpdateRootIndex();
	CacheBreadthFirstTransformIndices();
}

#if WITH_EDITOR
bool UGeometryCollection::IsSimulationDataDirty() const
{
	return StateGuid != SimulationDataGuid;
}
#endif

int32 UGeometryCollection::AttachEmbeddedGeometryExemplar(const UStaticMesh* Exemplar)
{
	FSoftObjectPath NewExemplarPath(Exemplar);
	
	// Check first if the exemplar is already attached
	for (int32 ExemplarIndex = 0; ExemplarIndex < EmbeddedGeometryExemplar.Num(); ++ExemplarIndex)
	{
		if (NewExemplarPath == EmbeddedGeometryExemplar[ExemplarIndex].StaticMeshExemplar)
		{
			return ExemplarIndex;
		}
	}

	return EmbeddedGeometryExemplar.Emplace( NewExemplarPath );
}

void UGeometryCollection::RemoveExemplars(const TArray<int32>& SortedRemovalIndices)
{
	if (SortedRemovalIndices.Num() > 0)
	{
		for (int32 Index = SortedRemovalIndices.Num() - 1; Index >= 0; --Index)
		{
			EmbeddedGeometryExemplar.RemoveAt(Index);
		}
	}
}

int32 FGeometryCollectionAutoInstanceMesh::GetNumDataPerInstance() const
{
	return NumInstances? (CustomData.Num() / NumInstances): 0;
}

bool FGeometryCollectionAutoInstanceMesh::operator ==(const FGeometryCollectionAutoInstanceMesh& Other) const
{
	return (Mesh == Other.Mesh) && (Materials == Other.Materials);
}

/** find or add a auto instance mesh and return its index */
const FGeometryCollectionAutoInstanceMesh& UGeometryCollection::GetAutoInstanceMesh(int32 AutoInstanceMeshIndex) const
{
	return AutoInstanceMeshes[AutoInstanceMeshIndex];
}

/**  find or add a auto instance mesh from another one and return its index */
int32 UGeometryCollection::FindOrAddAutoInstanceMesh(const FGeometryCollectionAutoInstanceMesh& AutoInstanceMesh)
{
	int32 AutoInstanceMeshIndex = AutoInstanceMeshes.AddUnique(AutoInstanceMesh);
	FGeometryCollectionAutoInstanceMesh& Instance = AutoInstanceMeshes[AutoInstanceMeshIndex];
	Instance.NumInstances++;
	return AutoInstanceMeshIndex;
}

int32 UGeometryCollection::FindOrAddAutoInstanceMesh(const UStaticMesh* StaticMesh, const TArray<UMaterialInterface*>& MeshMaterials)
{
	FGeometryCollectionAutoInstanceMesh NewMesh;
	NewMesh.Mesh = StaticMesh;
	NewMesh.Materials = MeshMaterials;

	return FindOrAddAutoInstanceMesh(NewMesh);
}

void UGeometryCollection::SetAutoInstanceMeshes(const TArray<FGeometryCollectionAutoInstanceMesh>& InAutoInstanceMeshes)
{
	AutoInstanceMeshes = InAutoInstanceMeshes;

	// dedup array and reassign indices
	if (AutoInstanceMeshes.Num() > 0)
	{
		TArray<FGeometryCollectionAutoInstanceMesh> UniqueAutoInstanceMeshes;
		TArray<int32> InstanceMeshIndexRemap;

		UniqueAutoInstanceMeshes.Reserve(AutoInstanceMeshes.Num());
		InstanceMeshIndexRemap.Reserve(AutoInstanceMeshes.Num());

		// now we may have two similar entries  we need to consolidate them 
		for (int32 InstanceMeshIndex = 0; InstanceMeshIndex < AutoInstanceMeshes.Num(); InstanceMeshIndex++)
		{
			const FGeometryCollectionAutoInstanceMesh& InstanceMesh = AutoInstanceMeshes[InstanceMeshIndex];
			int32 UniqueInstanceMeshIndex = UniqueAutoInstanceMeshes.Find(InstanceMesh);
			if (UniqueInstanceMeshIndex == INDEX_NONE)
			{
				FGeometryCollectionAutoInstanceMesh UniqueInstanceMesh = InstanceMesh;
				UniqueInstanceMesh.NumInstances = 0;
				UniqueInstanceMesh.CustomData.Reset();
				UniqueInstanceMeshIndex = UniqueAutoInstanceMeshes.Add(UniqueInstanceMesh);
			}
			// make sure num instances are aggregated
			UniqueAutoInstanceMeshes[UniqueInstanceMeshIndex].NumInstances += InstanceMesh.NumInstances;
			InstanceMeshIndexRemap.Add(UniqueInstanceMeshIndex);
		}

		GeometryCollection::Facades::FCollectionInstancedMeshFacade InstancedMeshFacade(*GetGeometryCollection());

		const TManagedArray<TSet<int32>>& Children = GetGeometryCollection()->Children;

		// relocate custom data : we cannot just aggregate them because we may have interleaved transform indices with alternating colors
		// also adjust the transform index to instance mesh index via the facade 
		TArray<int32> DataReadOffsets;
		DataReadOffsets.SetNumZeroed(AutoInstanceMeshes.Num());
		for (int32 TransformIndex = 0; TransformIndex < InstancedMeshFacade.GetNumIndices(); TransformIndex++)
		{
			// only for leaves
			if (Children[TransformIndex].Num() == 0)
			{
				const int32 OldIndex = InstancedMeshFacade.GetIndex(TransformIndex);
				if (InstanceMeshIndexRemap.IsValidIndex(OldIndex))
				{
					const FGeometryCollectionAutoInstanceMesh& OldInstanceMesh = AutoInstanceMeshes[OldIndex];

					const int32 NewIndex = InstanceMeshIndexRemap[OldIndex];
					FGeometryCollectionAutoInstanceMesh& NewInstanceMesh = UniqueAutoInstanceMeshes[NewIndex];

					InstancedMeshFacade.SetIndex(TransformIndex, NewIndex);

					const int32 NumDataPerInstance = OldInstanceMesh.GetNumDataPerInstance();
					if (NumDataPerInstance > 0)
					{
						const int32 DataReadOffset = DataReadOffsets[OldIndex];
						for (int32 DataIndex = 0; DataIndex < NumDataPerInstance; DataIndex++)
						{
							const float OldData = OldInstanceMesh.CustomData[DataReadOffset + DataIndex];
							NewInstanceMesh.CustomData.Add(OldData);
						}
						DataReadOffsets[OldIndex] += NumDataPerInstance;
					}
				}
			}
		}
		AutoInstanceMeshes = MoveTemp(UniqueAutoInstanceMeshes);
	}
}

FGuid UGeometryCollection::GetIdGuid() const
{
	return PersistentGuid;
}

FGuid UGeometryCollection::GetStateGuid() const
{
	return StateGuid;
}

#if WITH_EDITOR

void UGeometryCollection::PostEditUndo()
{
	PropagateTransformUpdateToComponents();
	Super::PostEditUndo();
}

void UGeometryCollection::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	bool bDoInvalidateCollection = false;
	bool bValidateSizeSpecificDataDefaults = false;
	bool bDoUpdateConvexGeometry = false;
	bool bRebuildSimulationData = false;
	bool bRebuildRenderData = false;

	if (PropertyChangedEvent.Property)
	{
		FName PropertyName = PropertyChangedEvent.Property->GetFName();

		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UGeometryCollection, EnableNanite))
		{
			bDoInvalidateCollection = true;
			bRebuildRenderData = true;
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UGeometryCollection, bEnableNaniteFallback))
		{
			bDoInvalidateCollection = true;
			bRebuildRenderData = true;
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UGeometryCollection, NaniteMinimumResidencyInKB))
		{
			bDoInvalidateCollection = true;
			bRebuildRenderData = true;
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UGeometryCollection, bSupportRayTracing))
		{
			bDoInvalidateCollection = true;
			bRebuildRenderData = true;
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UGeometryCollection, bUseFullPrecisionUVs))
		{
			bDoInvalidateCollection = true;
			bRebuildRenderData = true;
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UGeometryCollection, bConvertVertexColorsToSRGB))
		{
			bRebuildRenderData = true;
		}
		else if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UGeometryCollection, SizeSpecificData))
		{
			bDoInvalidateCollection = true;
			bDoUpdateConvexGeometry = true;
			bValidateSizeSpecificDataDefaults = true;
			bRebuildSimulationData = true;
		}		
		else if (PropertyName.ToString().Contains(FString("ImplicitType")))
			//SizeSpecificData.Num() && SizeSpecificData[0].CollisionShapes.Num() &&
			//	PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UGeometryCollection, SizeSpecificData[0].CollisionShapes[0].ImplicitType))
		{
				bDoInvalidateCollection = true;
				bDoUpdateConvexGeometry = true;
				bRebuildSimulationData = true;
		}
		else if (PropertyChangedEvent.Property->GetFName() != GET_MEMBER_NAME_CHECKED(UGeometryCollection, Materials))
		{
			bDoInvalidateCollection = true;
			bRebuildSimulationData = true;
		}
	}
	else if (PropertyChangedEvent.ChangeType == EPropertyChangeType::Unspecified)
	{
		// We get here on undo/redo operations.
		// Make sure that render data rebuilds.
		bRebuildRenderData = true;
	}

	if (bDoInvalidateCollection)
	{
		InvalidateCollection();
	}

	if (bValidateSizeSpecificDataDefaults)
	{
		ValidateSizeSpecificDataDefaults();
	}

	if (bDoUpdateConvexGeometry)
	{
		UpdateConvexGeometry();
	}

	if (bRebuildSimulationData)
	{
		if (!bManualDataCreate)
		{
			CreateSimulationData();
		}
	}

	if (bRebuildRenderData)
	{
		RebuildRenderData();
	}
	InvalidateDataflowContents();

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

bool UGeometryCollection::Modify(bool bAlwaysMarkDirty /*= true*/)
{
	bool bSuperResult = Super::Modify(bAlwaysMarkDirty);

	UPackage* Package = GetOutermost();
	if (Package->IsDirty())
	{
		InvalidateCollection();
	}

	return bSuperResult;
}

void UGeometryCollection::AppendToClassSchema(FAppendToClassSchemaContext& Context)
{
	Super::AppendToClassSchema(Context);

	FStringView RenderDataVersionString(FDerivedDataGeometryCollectionRenderDataCooker::VersionString);
	Context.Update(RenderDataVersionString.GetData(), RenderDataVersionString.Len() * sizeof(RenderDataVersionString[0]));
}

void UGeometryCollection::EnsureDataIsCooked(bool bInitResources, bool bIsTransacting, bool bIsPersistent, bool bAllowCopyFromDDC)
{
	EnsureSimulationDataIsCooked(bIsTransacting, bAllowCopyFromDDC);

	EnsureRenderDataIsCooked(bInitResources);
}

void UGeometryCollection::EnsureSimulationDataIsCooked(bool bIsTransacting, bool bAllowCopyFromDDC = true)
{
	if (StateGuid != LastBuiltSimulationDataGuid)
	{
		CreateSimulationDataImp(/*bCopyFromDDC=*/ bAllowCopyFromDDC && !bIsTransacting);

		LastBuiltSimulationDataGuid = StateGuid;
	}

	// todo(chaos) - this is temporary solution to make sure the data is computed accordingly if the attribute are missing
	//				 in the future we should probably get rid of this all cooker logic and have a proper dependent attribute system 
	if (GeometryCollection)
	{
		if (FGeometryCollectionPhysicsProxy::NeedToInitializeSharedCollisionStructures(*GeometryCollection))
		{
			FSharedSimulationParameters SharedParams;
			GetSharedSimulationParams(SharedParams);

			Chaos::FErrorReporter ErrorReporter(GetName());
			BuildSimulationData(ErrorReporter, *GeometryCollection, SharedParams);
			// important : this is necessary to make sure we compute mass scale on the instances properly
			// sadly we cannot call this in BuildSimulationData because we have no access to the asset
			CacheMaterialDensity();
		}
	}
}

void UGeometryCollection::EnsureRenderDataIsCooked(bool bInitResources)
{
	// Render data only goes through DDC when loading and saving ( called from OnPostLoad / OnSave  )
	// Using DDC during edits isn't worth it especially as we use a continually mutating guid instead of a state hash.
	// That ensures that all edits are cache misses (slow) and unnecessarily fill up DDC disk space.
	if (StateGuid != LastBuiltRenderDataGuid)
	{
		CreateRenderDataImp(/*bCopyFromDDC=*/ bInitResources);

		if (FApp::CanEverRender() && bInitResources)
		{
			if (RenderData)
			{
				RenderData->InitResources(*this);
			}
		}

		LastBuiltRenderDataGuid = StateGuid;
		RenderDataGuid = StateGuid;
		ValidRenderDataGuid = StateGuid;
	}
}

#endif

void UGeometryCollection::PreSave(FObjectPreSaveContext SaveContext)
{
#if WITH_EDITOR
	constexpr bool bInitResources = false; 
	constexpr bool bIsTransacting = false;
	constexpr bool bIsPersistent = false; // note that this has no effect on the call below  
	constexpr bool bAllowCopyFromDDC = false;
	EnsureDataIsCooked(bInitResources, bIsTransacting, bIsPersistent, bAllowCopyFromDDC);
#endif

	Super::PreSave(SaveContext);
}

void UGeometryCollection::PostLoad()
{
	Super::PostLoad();

	DataflowInstance.PostLoad();

#if WITH_EDITOR
	constexpr bool bInitResources = true;
	EnsureRenderDataIsCooked(bInitResources);
#else
	if (FApp::CanEverRender())
	{
		InitResources();
	}
#endif

	// migrate deprecated data if necessary 
	MigrateDeprecatedRootProxyData();
	MigrateDeprecatedDataflowData();
}

void UGeometryCollection::MigrateDeprecatedRootProxyData()
{
#if WITH_EDITORONLY_DATA
	if (!RootProxy_DEPRECATED.IsNull())
	{
		if (UStaticMesh* ProxyMesh = Cast<UStaticMesh>(RootProxy_DEPRECATED.TryLoad()))
		{
			RootProxyData.ProxyMeshes.Add(TObjectPtr<UStaticMesh>(ProxyMesh));
		}
		RootProxy_DEPRECATED = nullptr;
	}

	for (int32 MeshIndex = 0; MeshIndex < AutoInstanceMeshes.Num(); MeshIndex++)
	{
		FGeometryCollectionAutoInstanceMesh& AutoInstanceMesh = AutoInstanceMeshes[MeshIndex];
		if (!AutoInstanceMesh.StaticMesh_DEPRECATED.IsNull())
		{
			if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(AutoInstanceMesh.StaticMesh_DEPRECATED.TryLoad()))
			{
				AutoInstanceMesh.Mesh = TObjectPtr<UStaticMesh>(StaticMesh);
			}
			AutoInstanceMesh.StaticMesh_DEPRECATED = nullptr;
		}
	}
#endif
}

void UGeometryCollection::MigrateDeprecatedDataflowData()
{
#if WITH_EDITORONLY_DATA
	if (DataflowAsset != nullptr)
	{
		DataflowInstance.SetDataflowAsset(DataflowAsset);
		DataflowInstance.SetDataflowTerminal(FName(DataflowTerminal_DEPRECATED));

		DataflowAsset = nullptr;
		DataflowTerminal_DEPRECATED.Empty();

		// todo(ccaillaud) do not migrate the overrides until we have a solid plan to make sure they won't sync and risk of loosing them 
	
		//FInstancedPropertyBag& Variables = DataflowInstance.GetMutableVariables();
		//// convert all override to be string variables 
		//for (const TPair<FString, FString>& Pair : Overrides)
		//{
		//	const FName VariableName = FName(Pair.Key);
		//	const FString& VariableValue = Pair.Value;

		//	Variables.AddProperty(VariableName, EPropertyBagPropertyType::String);
		//	Variables.SetValueString(VariableName, VariableValue);
		//}
		// clear all the deprecated values

		// Overrides.Empty();
	}
#endif
}


void UGeometryCollection::BeginDestroy()
{
	Super::BeginDestroy();
	ReleaseResources();

	// Render fence to ensure that we complete resource release before destroy.
	RenderResourceReleaseFence.BeginFence();
}

bool UGeometryCollection::IsReadyForFinishDestroy()
{
	return Super::IsReadyForFinishDestroy() && RenderResourceReleaseFence.IsFenceComplete();
}

bool UGeometryCollection::HasMeshData() const
{
	return RenderData != nullptr &&
#if WITH_EDITOR
		RenderDataGuid == ValidRenderDataGuid &&
#endif
		RenderData->bHasMeshData;
}

bool UGeometryCollection::HasNaniteData() const
{
	return RenderData != nullptr && 
#if WITH_EDITOR
		RenderDataGuid == ValidRenderDataGuid &&
#endif
		RenderData->bHasNaniteData;
}

uint32 UGeometryCollection::GetNaniteResourceID() const
{
	return RenderData->NaniteResourcesPtr->RuntimeResourceID;
}

uint32 UGeometryCollection::GetNaniteHierarchyOffset() const
{
	return RenderData->NaniteResourcesPtr->HierarchyOffset;
}

uint32 UGeometryCollection::GetNaniteHierarchyOffset(int32 GeometryIndex, bool bFlattened) const
{
	const Nanite::FResources& NaniteResources = *RenderData->NaniteResourcesPtr;
	check(GeometryIndex >= 0 && GeometryIndex < NaniteResources.HierarchyRootOffsets.Num());
	uint32 HierarchyOffset = NaniteResources.HierarchyRootOffsets[GeometryIndex];

	// Translate the root offset into dwords here
	HierarchyOffset *= NANITE_HIERARCHY_NODE_SLICE_SIZE_DWORDS;

	if (bFlattened)
	{
		HierarchyOffset += NaniteResources.HierarchyOffset;
	}
	return HierarchyOffset;
}

void UGeometryCollection::AddAssetUserData(UAssetUserData* InUserData)
{
	if (InUserData != nullptr)
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass(InUserData->GetClass());
		if (ExistingData != nullptr)
		{
			AssetUserData.Remove(ExistingData);
		}
		AssetUserData.Add(InUserData);
	}
}

UAssetUserData* UGeometryCollection::GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			return Datum;
		}
	}
	return nullptr;
}

void UGeometryCollection::RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass)
{
	for (int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++)
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if (Datum != nullptr && Datum->IsA(InUserDataClass))
		{
			AssetUserData.RemoveAt(DataIdx);
			return;
		}
	}
}

const TArray<UAssetUserData*>* UGeometryCollection::GetAssetUserDataArray() const
{
	return &ToRawPtrTArrayUnsafe(AssetUserData);
}

void UGeometryCollection::SetDataflowAsset(UDataflow* InDataflowAsset)
{
	DataflowInstance.SetDataflowAsset(InDataflowAsset);
}

UDataflow* UGeometryCollection::GetDataflowAsset() const
{
	return DataflowInstance.GetDataflowAsset();
}

TObjectPtr<UDataflowBaseContent> UGeometryCollection::CreateDataflowContent()
{
	TObjectPtr<UDataflowBaseContent> BaseContent = NewObject<UDataflowBaseContent>();

	BaseContent->SetDataflowOwner(this);
	BaseContent->SetTerminalAsset(this);
	
	WriteDataflowContent(BaseContent);
	
	return BaseContent;
}

void UGeometryCollection::WriteDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) const
{
	if(const TObjectPtr<UDataflowBaseContent> BaseContent = Cast<UDataflowBaseContent>(DataflowContent))
	{
		BaseContent->SetDataflowAsset(DataflowInstance.GetDataflowAsset());
		BaseContent->SetDataflowTerminal(DataflowInstance.GetDataflowTerminal().ToString());
	}
}

void UGeometryCollection::ReadDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent)
{}

#if WITH_EDITOR
bool UGeometryCollection::CanEditChange(const FProperty* InProperty) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	const FName& Name = InProperty->GetFName();

	if (Name == GET_MEMBER_NAME_CHECKED(ThisClass, bOptimizeConvexes))
	{
		return Chaos::CVars::bChaosConvexSimplifyUnion == true;
	}

	return true;
}
#endif

void UGeometryCollection::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
	// Dataflow specific tags
	UE::Dataflow::InstanceUtils::GetAssetRegistryTags(this, Context);

	// Geometry collection specific tags
	const int32 NumTransforms = GeometryCollection? GeometryCollection->Transform.Num(): 0;
	Context.AddTag(FAssetRegistryTag(TEXT("NumTransforms"), FString::FromInt(NumTransforms), FAssetRegistryTag::TT_Numerical));

	Super::GetAssetRegistryTags(Context);
}