// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionParticlesData.h"

#if GEOMETRYCOLLECTION_DEBUG_DRAW

#include "PhysicsSolver.h"
#include "Chaos/Sphere.h"
#include "Chaos/Box.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ImplicitObjectUnion.h"
#include "Chaos/Levelset.h"
#include "ChaosSolversModule.h"
#include "GeometryCollection/ManagedArray.h"

DEFINE_LOG_CATEGORY_STATIC(LogGeometryCollectionParticlesData, Log, All);

FGeometryCollectionParticlesData::FGeometryCollectionParticlesData()
	: ChaosModule(FChaosSolversModule::GetModule())
	, BufferedData()
	, PhysicsSyncCount(0)
	, GameSyncCount(MAX_int32)
	, SyncFrame(MAX_uint64)
{
}

void FGeometryCollectionParticlesData::Sync(Chaos::FPhysicsSolver* Solver, const TManagedArray<FGuid>& RigidBodyIds)
{
	// No point in calling twice the sync function within the same frame
	if (!ensureMsgf(SyncFrame != GFrameCounter, TEXT("Sync should not happen twice during the same tick.")))
	{
		return;
	}
	SyncFrame = GFrameCounter;

	// Check for newly transferred data
	const int32 SyncCount = PhysicsSyncCount.GetValue();
	if(GameSyncCount != SyncCount)
	{
		// Received some new data, so safe to flip the buffers
		BufferedData.Flip();

		// Is a new command worth sending?
		const FData& ConstData = BufferedData.GetPhysicsDataForRead();
		if (ConstData.RequiredDataFlags.FindFirstSetBit() != INDEX_NONE && Solver && RigidBodyIds.Num() > 0)
		{
			// Update sync count so that the next flip will only happen once the sync command has completed
			GameSyncCount = SyncCount;

			// Send sync command

			Solver->EnqueueCommandImmediate([this, &RigidBodyIds, InSolver=Solver]()
			{
				// Iterate through all data
				FData& Data = BufferedData.GetPhysicsDataForWrite();
				for (int32 DataIndex = 0; DataIndex < uint32(EGeometryCollectionParticlesData::Count); ++DataIndex)
				{
					// Only sync required infos
					if (Data.RequiredDataFlags[DataIndex])
					{
						Data.Copy(EGeometryCollectionParticlesData(DataIndex), InSolver, RigidBodyIds);
					}
					else
					{
						Data.Reset(EGeometryCollectionParticlesData(DataIndex));
					}
				}

				// Update synced flags
				Data.SyncedDataFlags = Data.RequiredDataFlags;

				// Signal the new data has arrived and that it's safe to flip the buffers
				PhysicsSyncCount.Increment();
			});
		}
	}

	// Clear flags ready for next set of requests
	FData& Data = BufferedData.GetGameDataForWrite();
	Data.RequiredDataFlags = FDataFlags();

	UE_LOGF(LogGeometryCollectionParticlesData, Verbose, "Synced particles data at frame %llu: \n%ls.", SyncFrame, *ToString(0));
}


void FGeometryCollectionParticlesData::FData::SetAllDataSyncFlag() const
{
	for (int32 DataIndex = 0; DataIndex < uint32(EGeometryCollectionParticlesData::Count); ++DataIndex)
	{
		RequiredDataFlags[DataIndex] = true;
	}
}

void FGeometryCollectionParticlesData::FData::Reset(EGeometryCollectionParticlesData Data)
{
	switch (Data)
	{
	default: break;
	case EGeometryCollectionParticlesData::X                     : X                     .Resize(0); break;
	case EGeometryCollectionParticlesData::R                     : R                     .Resize(0); break;
	case EGeometryCollectionParticlesData::Geometry              : Geometry              .Resize(0); break;
	case EGeometryCollectionParticlesData::GeometryType          : GeometryType          .Resize(0); break;
	case EGeometryCollectionParticlesData::GeometryIsConvex      : GeometryIsConvex      .Resize(0); break;
	case EGeometryCollectionParticlesData::GeometryHasBoundingBox: GeometryHasBoundingBox.Resize(0); break;
	case EGeometryCollectionParticlesData::GeometryBoxMin        : GeometryBoxMin        .Resize(0); break;
	case EGeometryCollectionParticlesData::GeometryBoxMax        : GeometryBoxMax        .Resize(0); break;
	case EGeometryCollectionParticlesData::GeometrySphereCenter  : GeometrySphereCenter  .Resize(0); break;
	case EGeometryCollectionParticlesData::GeometrySphereRadius  : GeometrySphereRadius  .Resize(0); break;
	case EGeometryCollectionParticlesData::GeometryLevelSetGrid  : GeometryLevelSetGrid  .Resize(0); break;
	case EGeometryCollectionParticlesData::V                     : V                     .Resize(0); break;
	case EGeometryCollectionParticlesData::W                     : W                     .Resize(0); break;
	case EGeometryCollectionParticlesData::F                     : F                     .Resize(0); break;
	case EGeometryCollectionParticlesData::Torque                : Torque                .Resize(0); break;
	case EGeometryCollectionParticlesData::I                     : I                     .Resize(0); break;
	case EGeometryCollectionParticlesData::InvI                  : InvI                  .Resize(0); break;
	case EGeometryCollectionParticlesData::M                     : M                     .Resize(0); break;
	case EGeometryCollectionParticlesData::InvM                  : InvM                  .Resize(0); break;
	case EGeometryCollectionParticlesData::CollisionParticlesSize: CollisionParticlesSize.Resize(0); break;
	case EGeometryCollectionParticlesData::Disabled              : Disabled              .Resize(0); break;
	case EGeometryCollectionParticlesData::Sleeping              : Sleeping              .Resize(0); break;
	case EGeometryCollectionParticlesData::Island                : Island                .Resize(0); break;
	case EGeometryCollectionParticlesData::P                     : P                     .Resize(0); break;
	case EGeometryCollectionParticlesData::Q                     : Q                     .Resize(0); break;
	case EGeometryCollectionParticlesData::PreV                  : PreV                  .Resize(0); break;
	case EGeometryCollectionParticlesData::PreW                  : PreW                  .Resize(0); break;
	case EGeometryCollectionParticlesData::ConnectivityEdges     : ConnectivityEdges     .Resize(0); break;
	case EGeometryCollectionParticlesData::ChildToParentMap      : ChildToParentMap      .Resize(0); break;
	}
}

void FGeometryCollectionParticlesData::FData::Copy(EGeometryCollectionParticlesData Data, const Chaos::FPhysicsSolver* Solver, const TManagedArray<FGuid>& RigidBodyIds)
{
	check(Solver);
#if TODO_REIMPLEMENT_GET_RIGID_PARTICLES
	// 10.31.2019 Ryan - This code uses the RigidBodyIds as indices, and that's not going to work anymore.
	// We need to figure that out...

	const Chaos::FPBDRigidParticles& Particles = Solver->GetRigidParticles();
	const Chaos::TPBDRigidClustering<FPBDRigidsEvolution, FPBDCollisionConstraints, Chaos::FReal, 3>& Clustering = Solver->GetRigidClustering();

	// Lambdas used to flatten the particle structure
	// Can't rely on FImplicitObject::GetType, as it returns Unknown
	// TODO: Would there be something to fix in TImplicitObject so we can use this simpler line of code instead?:
	//    auto GetType = [](const Chaos::FImplicitObject* Geometry) { return Geometry ? Geometry->GetType(): Chaos::ImplicitObjectType::Unknown; };
	auto GetType = [](Chaos::TSerializablePtr<Chaos::FImplicitObject> ImplicitObject)
	{
		if (ImplicitObject)
		{
			if      (ImplicitObject->template GetObject<Chaos::TSphere                   <Chaos::FReal, 3>>()) { return Chaos::ImplicitObjectType::Sphere     ; }
			else if (ImplicitObject->template GetObject<Chaos::TAABB                     <Chaos::FReal, 3>>()) { return Chaos::ImplicitObjectType::Box        ; }
			else if (ImplicitObject->template GetObject<Chaos::TPlane                    <Chaos::FReal, 3>>()) { return Chaos::ImplicitObjectType::Plane      ; }
			else if (ImplicitObject->template GetObject<Chaos::TImplicitObjectTransformed<Chaos::FReal, 3>>()) { return Chaos::ImplicitObjectType::Transformed; }
			else if (ImplicitObject->template GetObject<Chaos::FImplicitObjectUnion                       >()) { return Chaos::ImplicitObjectType::Union      ; }
			else if (ImplicitObject->template GetObject<Chaos::FLevelSet                                  >()) { return Chaos::ImplicitObjectType::LevelSet   ; }
		}
		return Chaos::ImplicitObjectType::Unknown;
	};
	auto IsConvex       = [](Chaos::TSerializablePtr<Chaos::FImplicitObject> ImplicitObject) { return ImplicitObject ? ImplicitObject->IsConvex      (): false; };
	auto HasBoundingBox = [](Chaos::TSerializablePtr<Chaos::FImplicitObject> ImplicitObject) { return ImplicitObject ? ImplicitObject->HasBoundingBox(): false; };
	auto BoxMin         = [](Chaos::TSerializablePtr<Chaos::FImplicitObject> ImplicitObject) { const Chaos::TAABB     <Chaos::FReal, 3>* Box     ; return ImplicitObject && (Box      = ImplicitObject->template GetObject<Chaos::TAABB     <Chaos::FReal, 3>>()) != nullptr ? Box     ->Min       (): Chaos::FVec3(0); };
	auto BoxMax         = [](Chaos::TSerializablePtr<Chaos::FImplicitObject> ImplicitObject) { const Chaos::TAABB     <Chaos::FReal, 3>* Box     ; return ImplicitObject && (Box      = ImplicitObject->template GetObject<Chaos::TAABB     <Chaos::FReal, 3>>()) != nullptr ? Box     ->Max       (): Chaos::FVec3(0); };
	auto SphereCenter   = [](Chaos::TSerializablePtr<Chaos::FImplicitObject> ImplicitObject) { const Chaos::TSphere   <Chaos::FReal, 3>* Sphere  ; return ImplicitObject && (Sphere   = ImplicitObject->template GetObject<Chaos::TSphere   <Chaos::FReal, 3>>()) != nullptr ? Sphere  ->GetCenter (): Chaos::FVec3(0); };
	auto SphereRadius   = [](Chaos::TSerializablePtr<Chaos::FImplicitObject> ImplicitObject) { const Chaos::TSphere   <Chaos::FReal, 3>* Sphere  ; return ImplicitObject && (Sphere   = ImplicitObject->template GetObject<Chaos::TSphere   <Chaos::FReal, 3>>()) != nullptr ? Sphere  ->GetRadius (): Chaos::FReal(0); };
	auto LevelSetGrid   = [](Chaos::TSerializablePtr<Chaos::FImplicitObject> ImplicitObject) { const Chaos::FLevelSet*                   LevelSet; return ImplicitObject && (LevelSet = ImplicitObject->template GetObject<Chaos::FLevelSet                  >()) != nullptr ? LevelSet->GetGrid   (): Chaos::TUniformGrid<Chaos::FReal, 3>(); };

	// Data type copy for all particles
	const int32 Count = RigidBodyIds.Num();
	switch (Data)
	{
	default: break;
	case EGeometryCollectionParticlesData::X                     : X                     .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { X                     [i] = Particles.X                      (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::R                     : R                     .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { R                     [i] = Particles.R                      (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::Geometry              : Geometry              .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { Geometry              [i] = Particles.Geometry               (RigidBodyIds[i]).Get() ; } } break;
	case EGeometryCollectionParticlesData::GeometryType          : GeometryType          .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { GeometryType          [i] = GetType       (Particles.Geometry(RigidBodyIds[i])); } } break;
	case EGeometryCollectionParticlesData::GeometryIsConvex      : GeometryIsConvex      .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { GeometryIsConvex      [i] = IsConvex      (Particles.Geometry(RigidBodyIds[i])); } } break;
	case EGeometryCollectionParticlesData::GeometryHasBoundingBox: GeometryHasBoundingBox.Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { GeometryHasBoundingBox[i] = HasBoundingBox(Particles.Geometry(RigidBodyIds[i])); } } break;
	case EGeometryCollectionParticlesData::GeometryBoxMin        : GeometryBoxMin        .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { GeometryBoxMin        [i] = BoxMin        (Particles.Geometry(RigidBodyIds[i])); } } break;
	case EGeometryCollectionParticlesData::GeometryBoxMax        : GeometryBoxMax        .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { GeometryBoxMax        [i] = BoxMax        (Particles.Geometry(RigidBodyIds[i])); } } break;
	case EGeometryCollectionParticlesData::GeometrySphereCenter  : GeometrySphereCenter  .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { GeometrySphereCenter  [i] = SphereCenter  (Particles.Geometry(RigidBodyIds[i])); } } break;
	case EGeometryCollectionParticlesData::GeometrySphereRadius  : GeometrySphereRadius  .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { GeometrySphereRadius  [i] = SphereRadius  (Particles.Geometry(RigidBodyIds[i])); } } break;
	case EGeometryCollectionParticlesData::GeometryLevelSetGrid  : GeometryLevelSetGrid  .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { GeometryLevelSetGrid  [i] = LevelSetGrid  (Particles.Geometry(RigidBodyIds[i])); } } break;
	case EGeometryCollectionParticlesData::V                     : V                     .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { V                     [i] = Particles.V                      (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::W                     : W                     .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { W                     [i] = Particles.W                      (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::F                     : F                     .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { F                     [i] = Particles.F                      (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::Torque                : Torque                .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { Torque                [i] = Particles.Torque                 (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::I                     : I                     .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { I                     [i] = Particles.I                      (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::InvI                  : InvI                  .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { InvI                  [i] = Particles.InvI                   (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::M                     : M                     .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { M                     [i] = Particles.M                      (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::InvM                  : InvM                  .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { InvM                  [i] = Particles.InvM                   (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::CollisionParticlesSize: CollisionParticlesSize.Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { CollisionParticlesSize[i] = Particles.CollisionParticlesSize (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::Disabled              : Disabled              .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { Disabled              [i] = Particles.Disabled               (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::Sleeping              : Sleeping              .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { Sleeping              [i] = Particles.Sleeping               (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::Island                : Island                .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { Island                [i] = Particles.Island                 (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::P                     : P                     .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { P                     [i] = Particles.P                      (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::Q                     : Q                     .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { Q                     [i] = Particles.Q                      (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::PreV                  : PreV                  .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { PreV                  [i] = Particles.PreV                   (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::PreW                  : PreW                  .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { PreW                  [i] = Particles.PreW                   (RigidBodyIds[i]) ; } } break;
	case EGeometryCollectionParticlesData::ConnectivityEdges     : ConnectivityEdges     .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { ConnectivityEdges     [i] = Clustering.GetConnectivityEdges()[RigidBodyIds[i]] ; } } break;
	case EGeometryCollectionParticlesData::ChildToParentMap      : ChildToParentMap      .Resize(Count); for (int32 i = 0; i < Count; ++i) { if (RigidBodyIds[i] != INDEX_NONE) { ChildToParentMap      [i] = Clustering.GetChildToParentMap ()[RigidBodyIds[i]] ; } } break;
	}
#endif
}

FString FGeometryCollectionParticlesData::FData::ToString(int32 Index, const TCHAR* Separator) const
{
	auto TypeText = [](Chaos::EImplicitObjectType Type)
	{
		switch (Type)
		{
		default:
		case Chaos::ImplicitObjectType::Unknown    : return TEXT("Unknown"    );
		case Chaos::ImplicitObjectType::Box        : return TEXT("Box"        );
		case Chaos::ImplicitObjectType::LevelSet   : return TEXT("LevelSet"   );
		case Chaos::ImplicitObjectType::Plane      : return TEXT("Plane"      );
		case Chaos::ImplicitObjectType::Sphere     : return TEXT("Sphere"     );
		case Chaos::ImplicitObjectType::Transformed: return TEXT("Transformed");
		case Chaos::ImplicitObjectType::Union      : return TEXT("Union"      );
		}
	};

	auto GridString = [](const Chaos::TUniformGrid<Chaos::FReal, 3>& Grid)
	{
		return FString::Printf(TEXT("Counts X=%d Y=%d Z=%d, Dx %s, MinCorner %s, MaxCorner %s"), Grid.Counts().X, Grid.Counts().Y, Grid.Counts().Z, *Grid.Dx().ToString(), *Grid.MinCorner().ToString(), *Grid.MaxCorner().ToString());
	};

	FString String;
	const TCHAR* Sep = TEXT("");
	if (HasSyncedData(EGeometryCollectionParticlesData::X                     )) { String += FString::Printf(TEXT(  "X: %s"                     ),      *           X                     [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::R                     )) { String += FString::Printf(TEXT("%sR: %s"                     ), Sep, *           R                     [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::GeometryType          )) { String += FString::Printf(TEXT("%sGeometryType: %s"          ), Sep,  TypeText  (GeometryType          [Index])          ); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::GeometryIsConvex      )) { String += FString::Printf(TEXT("%sGeometryIsConvex: %d"      ), Sep,             GeometryIsConvex      [Index]           ); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::GeometryHasBoundingBox)) { String += FString::Printf(TEXT("%sGeometryHasBoundingBox: %d"), Sep,             GeometryHasBoundingBox[Index]           ); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::GeometryBoxMin        )) { String += FString::Printf(TEXT("%sGeometryBoxMin: %s"        ), Sep, *           GeometryBoxMin        [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::GeometryBoxMax        )) { String += FString::Printf(TEXT("%sGeometryBoxMax: %s"        ), Sep, *           GeometryBoxMax        [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::GeometrySphereCenter  )) { String += FString::Printf(TEXT("%sGeometrySphereCenter: %s"  ), Sep, *           GeometrySphereCenter  [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::GeometrySphereRadius  )) { String += FString::Printf(TEXT("%sGeometrySphereRadius: %f"  ), Sep,             GeometrySphereRadius  [Index]           ); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::GeometryLevelSetGrid  )) { String += FString::Printf(TEXT("%sGeometryLevelSetGrid: %s"  ), Sep, *GridString(GeometryLevelSetGrid  [Index])          ); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::V                     )) { String += FString::Printf(TEXT("%sV: %s"                     ), Sep, *           V                     [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::W                     )) { String += FString::Printf(TEXT("%sW: %s"                     ), Sep, *           W                     [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::F                     )) { String += FString::Printf(TEXT("%sF: %s"                     ), Sep, *           F                     [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::Torque                )) { String += FString::Printf(TEXT("%sTorque: %s"                ), Sep, *           Torque                [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::I                     )) { String += FString::Printf(TEXT("%sI: %s"                     ), Sep, *           I                     [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::InvI                  )) { String += FString::Printf(TEXT("%sInvI: %s"                  ), Sep, *           InvI                  [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::M                     )) { String += FString::Printf(TEXT("%sM: %f"                     ), Sep,             M                     [Index]           ); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::InvM                  )) { String += FString::Printf(TEXT("%sInvM: %f"                  ), Sep,             InvM                  [Index]           ); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::CollisionParticlesSize)) { String += FString::Printf(TEXT("%sCollisionParticlesSize: %d"), Sep,             CollisionParticlesSize[Index]           ); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::Disabled              )) { String += FString::Printf(TEXT("%sDisabled: %d"              ), Sep,             Disabled              [Index]           ); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::Sleeping              )) { String += FString::Printf(TEXT("%sSleeping: %d"              ), Sep,             Sleeping              [Index]           ); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::Island                )) { String += FString::Printf(TEXT("%sIsland: %d"                ), Sep,             Island                [Index]           ); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::P                     )) { String += FString::Printf(TEXT("%sP: %s"                     ), Sep, *           P                     [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::Q                     )) { String += FString::Printf(TEXT("%sQ: %s"                     ), Sep, *           Q                     [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::PreV                  )) { String += FString::Printf(TEXT("%sPreV: %s"                  ), Sep, *           PreV                  [Index].ToString()); Sep = Separator; }
	if (HasSyncedData(EGeometryCollectionParticlesData::PreW                  )) { String += FString::Printf(TEXT("%sPreW: %s"                  ), Sep, *           PreW                  [Index].ToString()); }

	return String;
}

#endif  // #if GEOMETRYCOLLECTION_DEBUG_DRAW

