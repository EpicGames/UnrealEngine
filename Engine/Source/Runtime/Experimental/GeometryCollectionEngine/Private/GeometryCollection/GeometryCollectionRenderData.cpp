// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionRenderData.h"

#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/ManagedArray.h"

#include "Rendering/RayTracingGeometryManager.h"

#if WITH_EDITOR
#include "NaniteBuilder.h"
#endif


FBoneMapVertexBuffer::~FBoneMapVertexBuffer()
{
	CleanUp();
}

void FBoneMapVertexBuffer::CleanUp()
{
	delete BoneMapData;
	BoneMapData = nullptr;
}

void FBoneMapVertexBuffer::Init(TArray<uint16> const& InBoneMap, bool bInNeedsCPUAccess)
{
	AllocateData(bInNeedsCPUAccess);
	ResizeBuffer(InBoneMap.Num());
	FMemory::Memcpy(Data, InBoneMap.GetData(), PixelFormatStride * NumVertices);
}

void FBoneMapVertexBuffer::ResizeBuffer(uint32 InNumVertices)
{
	NumVertices = InNumVertices;
	BoneMapData->ResizeBuffer(NumVertices);
	Data = NumVertices != 0 ? BoneMapData->GetDataPointer() : nullptr;
}

void FBoneMapVertexBuffer::AllocateData(bool bInNeedsCPUAccess)
{
	CleanUp();
	BoneMapData = new TStaticMeshVertexData<uint16>(bInNeedsCPUAccess);
}

void FBoneMapVertexBuffer::InitRHI(FRHICommandListBase& RHICmdList)
{
	const bool bHadData = BoneMapData != nullptr;
	VertexBufferRHI = FRenderResource::CreateRHIBuffer(RHICmdList, BoneMapData, NumVertices, BUF_Static | BUF_ShaderResource | BUF_SourceCopy, TEXT("FBoneMapVertexBuffer")); 
	if (VertexBufferRHI != nullptr)
	{
		VertexBufferSRV = RHICmdList.CreateShaderResourceView(FShaderResourceViewInitializer(bHadData ? VertexBufferRHI : nullptr, PixelFormat));
	}
}

void FBoneMapVertexBuffer::ReleaseRHI()
{
	VertexBufferSRV.SafeRelease();
	FVertexBuffer::ReleaseRHI();
}

void FBoneMapVertexBuffer::Serialize(FArchive& Ar, bool bInNeedsCPUAccess)
{
	bNeedsCPUAccess = bInNeedsCPUAccess;

	Ar << NumVertices;

	if (Ar.IsLoading())
	{
		// Allocate the vertex data storage type.
		AllocateData(bInNeedsCPUAccess);
	}

	if (BoneMapData != nullptr)
	{
		// Serialize the vertex data.
		BoneMapData->Serialize(Ar);

		// Make a copy of the vertex data pointer.
		Data = NumVertices != 0 ? BoneMapData->GetDataPointer() : nullptr;
	}
}


void FGeometryCollectionMeshResources::InitResources(UGeometryCollection const& Owner)
{
	FName OwnerName = Owner.GetFName();
	IndexBuffer.SetOwnerName(OwnerName);
	PositionVertexBuffer.SetOwnerName(OwnerName);
	StaticMeshVertexBuffer.SetOwnerName(OwnerName);
	ColorVertexBuffer.SetOwnerName(OwnerName);
	BoneMapVertexBuffer.SetOwnerName(OwnerName);

	BeginInitResource(&IndexBuffer);
	BeginInitResource(&PositionVertexBuffer);
	BeginInitResource(&StaticMeshVertexBuffer);
	BeginInitResource(&ColorVertexBuffer);
	BeginInitResource(&BoneMapVertexBuffer);
}

void FGeometryCollectionMeshResources::ReleaseResources()
{
	BeginReleaseResource(&IndexBuffer);
	BeginReleaseResource(&PositionVertexBuffer);
	BeginReleaseResource(&StaticMeshVertexBuffer);
	BeginReleaseResource(&ColorVertexBuffer);
	BeginReleaseResource(&BoneMapVertexBuffer);
}

void FGeometryCollectionMeshResources::Serialize(FArchive& Ar)
{
	// If platform doesn't have manual vertex fetch then we will need position and 
	// bone map CPU access so that we can skin positions on CPU.
	const bool bCPUAccessForBoneTransform = Ar.IsLoading() && GMaxRHIFeatureLevel <= ERHIFeatureLevel::ES3_1;
	// Otherwise we don't need CPU access.
	const bool bNeedsCPUAccess = false;

	IndexBuffer.Serialize(Ar, bNeedsCPUAccess);
	PositionVertexBuffer.Serialize(Ar, bCPUAccessForBoneTransform);
	StaticMeshVertexBuffer.Serialize(Ar, bNeedsCPUAccess);
	ColorVertexBuffer.Serialize(Ar, bNeedsCPUAccess);
	BoneMapVertexBuffer.Serialize(Ar, bCPUAccessForBoneTransform);
}


void FGeometryCollectionMeshDescription::Serialize(FArchive& Ar)
{
	Ar << NumVertices << NumTriangles << Sections << SectionsNoInternal;
	
	// need subsection data on cooked builds when using ray tracing
	// to be able to build one BLAS per subsection
#if !RHI_RAYTRACING
	if (Ar.IsCooking())
	{
		// No need for SubSections in cooked build.
		TArray<FGeometryCollectionMeshElement> Dummy;
		Ar << Dummy;
	}
	else
#endif
	{
		Ar << SubSections;
	}
}


#if WITH_EDITOR

/** Data to return from BuildMeshDataFromGeometryCollection(). */
struct FGeometryCollectionBuiltMeshData
{
	uint32 NumTexCoords = 0;
	TArray<uint32> Indices;
	FMeshBuildVertexData Vertices;
	TArray<uint16> BonesPerVertex;
	TArray<int32> MaterialsPerTriangle;
	TArray<bool> InternalPerTriangle;
	TArray<uint32> MeshTriangleCounts;
	FBounds3f VertexBounds;
};

/** Get the data needed to build render data from a Geometry Collection. */
bool BuildMeshDataFromGeometryCollection(FGeometryCollection& InCollection, FGeometryCollectionRenderData::FCreateDesc const& InDesc, FGeometryCollectionBuiltMeshData& OutMeshData)
{
	// Vertices Group
	const TManagedArray<FVector3f>& VertexArray = InCollection.Vertex;
	GeometryCollection::UV::FUVLayers UVsLayers = GeometryCollection::UV::FindActiveUVLayers(InCollection);
	const TManagedArray<FLinearColor>& ColorArray = InCollection.Color;
	const TManagedArray<FVector3f>& TangentUArray = InCollection.TangentU;
	const TManagedArray<FVector3f>& TangentVArray = InCollection.TangentV;
	const TManagedArray<FVector3f>& NormalArray = InCollection.Normal;
	const TManagedArray<int32>& BoneMapArray = InCollection.BoneMap;

	// Faces Group
	const TManagedArray<FIntVector>& IndicesArray = InCollection.Indices;
	const TManagedArray<bool>& VisibleArray = InCollection.Visible;
	const TManagedArray<int32>& MaterialIDArray = InCollection.MaterialID;
	const TManagedArray<bool>& Internal = InCollection.Internal;

	// Geometry Group
	const int32 NumGeometry = InCollection.NumElements(FGeometryCollection::GeometryGroup);
	const TManagedArray<int32>& VertexStartArray = InCollection.VertexStart;
	const TManagedArray<int32>& VertexCountArray = InCollection.VertexCount;
	const TManagedArray<int32>& FaceStartArray = InCollection.FaceStart;
	const TManagedArray<int32>& FaceCountArray = InCollection.FaceCount;

	const int32 NumTexCoords = InCollection.NumUVLayers();
	const bool bHasColors = ColorArray.Num() > 0;

	// Iterate geometry groups and collect vertices.
	FMeshBuildVertexData BuildVertexData;
	BuildVertexData.UVs.SetNum(NumTexCoords);

	TArray<uint16> BonesPerVertex;
	TArray<int32> DestVertexStarts;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCollectionRenderData::BuildVertices);

		// Gather destination vertex locations so that we can fill them in parallel.
		int32 NumBuildVertices = 0;
		DestVertexStarts.SetNumUninitialized(NumGeometry);
		for (int32 GeometryGroupIndex = 0; GeometryGroupIndex < NumGeometry; GeometryGroupIndex++)
		{
			DestVertexStarts[GeometryGroupIndex] = GeometryGroupIndex > 0 ? DestVertexStarts[GeometryGroupIndex - 1] + VertexCountArray[GeometryGroupIndex - 1] : 0;
			NumBuildVertices += VertexCountArray[GeometryGroupIndex];
		}

		BuildVertexData.Position.AddUninitialized(NumBuildVertices);
		BuildVertexData.TangentX.AddUninitialized(NumBuildVertices);
		BuildVertexData.TangentY.AddUninitialized(NumBuildVertices);
		BuildVertexData.TangentZ.AddUninitialized(NumBuildVertices);

		for (int32 TexCoord = 0; TexCoord < NumTexCoords; ++TexCoord)
		{
			BuildVertexData.UVs[TexCoord].AddUninitialized(NumBuildVertices);
		}

		if (bHasColors)
		{
			BuildVertexData.Color.AddUninitialized(NumBuildVertices);
		}

		BonesPerVertex.AddUninitialized(NumBuildVertices);

		TArray<FBounds3f> Bounds;

		ParallelForWithTaskContext(Bounds, NumGeometry, [&](FBounds3f& BatchBounds, int32 GeometryGroupIndex)
		{
			const int32 VertexStart = VertexStartArray[GeometryGroupIndex];
			const int32 VertexCount = VertexCountArray[GeometryGroupIndex];
			const int32 DestVertexStart = DestVertexStarts[GeometryGroupIndex];

			TArray<FBounds3f> GeometryBounds;

			ParallelForWithTaskContext(TEXT("GC:BuildVertices"), GeometryBounds, VertexCount, 500, [&](FBounds3f& BatchGeometryBounds, int32 VertexIndex)
			{
				BatchGeometryBounds += VertexArray[VertexStart + VertexIndex];

				BuildVertexData.Position[DestVertexStart + VertexIndex] = VertexArray[VertexStart + VertexIndex];
				BuildVertexData.TangentX[DestVertexStart + VertexIndex] = TangentUArray[VertexStart + VertexIndex];
				BuildVertexData.TangentY[DestVertexStart + VertexIndex] = TangentVArray[VertexStart + VertexIndex];
				BuildVertexData.TangentZ[DestVertexStart + VertexIndex] = NormalArray[VertexStart + VertexIndex];

				if (bHasColors)
				{
					BuildVertexData.Color[DestVertexStart + VertexIndex] = ColorArray[VertexStart + VertexIndex].ToFColor(InDesc.bConvertVertexColorsToSRGB /* sRGB */);
				}

				for (int32 TexCoord = 0; TexCoord < NumTexCoords; ++TexCoord)
				{
					FVector2f UV = UVsLayers[TexCoord][VertexStart + VertexIndex];
					if (UV.ContainsNaN())
					{
						UV = FVector2f::ZeroVector;
					}
					BuildVertexData.UVs[TexCoord][DestVertexStart + VertexIndex] = UV;
				}

				const int32 BoneIndex = BoneMapArray[VertexStart + VertexIndex];
				BonesPerVertex[DestVertexStart + VertexIndex] = (uint16)BoneIndex;
			});

			for (const FBounds3f& B : GeometryBounds)
				BatchBounds += B;
		});

		for (const FBounds3f& B : Bounds)
			OutMeshData.VertexBounds += B;
	}

	bool bAllOpaqueWhite = true;
	for (const FColor& Color : BuildVertexData.Color)
	{
		if (Color != FColor::White)
		{
			bAllOpaqueWhite = false;
			break;
		}
	}

	if (bAllOpaqueWhite)
	{
		BuildVertexData.Color.Empty();
	}

	// Iterate geometry groups and collect indices.
	// Can't go parallel here because we remove hidden faces as we go.
	TArray<uint32> BuildIndices;
	TArray<int32> MaterialsPerFace;
	TArray<bool> InternalPerFace;
	TArray<uint32> MeshTriangleCounts;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCollectionRenderData::GatherBuildIndices);

		BuildIndices.Reserve(IndicesArray.Num() * 3);
		MaterialsPerFace.Reserve(IndicesArray.Num());
		InternalPerFace.Reserve(IndicesArray.Num());
		MeshTriangleCounts.Reserve(NumGeometry);

		for (int32 GeometryGroupIndex = 0; GeometryGroupIndex < NumGeometry; ++GeometryGroupIndex)
		{
			const int32 FaceStart = FaceStartArray[GeometryGroupIndex];
			const int32 FaceCount = FaceCountArray[GeometryGroupIndex];
			const int32 DestFaceStart = MaterialsPerFace.Num();
			const int32 DestVertexOffset = DestVertexStarts[GeometryGroupIndex] - VertexStartArray[GeometryGroupIndex];

			for (int32 FaceIndex = 0; FaceIndex < FaceCount; ++FaceIndex)
			{
				// Remove hidden.
				if (!VisibleArray[FaceStart + FaceIndex])
				{
					continue;
				}

				FIntVector FaceIndices = IndicesArray[FaceStart + FaceIndex];
				FaceIndices = FaceIndices + FIntVector(DestVertexOffset);

				// Remove degenerates.
				if (BuildVertexData.Position[FaceIndices[0]] == BuildVertexData.Position[FaceIndices[1]] ||
					BuildVertexData.Position[FaceIndices[1]] == BuildVertexData.Position[FaceIndices[2]] ||
					BuildVertexData.Position[FaceIndices[2]] == BuildVertexData.Position[FaceIndices[0]])
				{
					continue;
				}

				BuildIndices.Add(FaceIndices.X);
				BuildIndices.Add(FaceIndices.Y);
				BuildIndices.Add(FaceIndices.Z);

				const int32 MaterialIndex = MaterialIDArray[FaceStart + FaceIndex];
				MaterialsPerFace.Add(MaterialIndex);
				const bool bInternal = Internal[FaceStart + FaceIndex];
				InternalPerFace.Add(bInternal);
			}

			MeshTriangleCounts.Add(MaterialsPerFace.Num() - DestFaceStart);
		}
	}

	OutMeshData.NumTexCoords = NumTexCoords;
	OutMeshData.Indices = MoveTemp(BuildIndices);
	OutMeshData.Vertices = MoveTemp(BuildVertexData);
	OutMeshData.BonesPerVertex = MoveTemp(BonesPerVertex);
	OutMeshData.MaterialsPerTriangle = MoveTemp(MaterialsPerFace);
	OutMeshData.InternalPerTriangle = MoveTemp(InternalPerFace);
	OutMeshData.MeshTriangleCounts = MoveTemp(MeshTriangleCounts);

	// Return false if we found no mesh data.
	return OutMeshData.Indices.Num() > 0 && OutMeshData.Vertices.Position.Num() > 0;
}

struct FGeometryCollectionBuiltMeshSectionData
{
	TArray<uint32> SortedIndices;
	TArray<FGeometryCollectionMeshElement> Sections;
	TArray<FGeometryCollectionMeshElement> SectionsNoInternal;
	TArray<FGeometryCollectionMeshElement> SubSections;
};

void BuildSectionDataFromMeshData(FGeometryCollectionBuiltMeshData const& InMeshData, FGeometryCollectionBuiltMeshSectionData& OutSectionData)
{
	// Find spans within the mesh data.
	TArray<FGeometryCollectionMeshElement> Spans;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCollectionRenderData::FindSpans);

		int32 LastMaterialIndex = -1;
		int32 LastIsInternal = -1;
		int32 LastBoneIndex = -1;

		for (int32 TriangleIndex = 0; TriangleIndex < InMeshData.MaterialsPerTriangle.Num(); ++TriangleIndex)
		{
			const int32 MaterialIndex = InMeshData.MaterialsPerTriangle[TriangleIndex];
			const int32 IsInternal = InMeshData.InternalPerTriangle[TriangleIndex] ? 1 : 0;
			const uint32 VertexIndex0 = InMeshData.Indices[TriangleIndex * 3];
			const uint32 VertexIndex1 = InMeshData.Indices[TriangleIndex * 3 + 1];
			const uint32 VertexIndex2 = InMeshData.Indices[TriangleIndex * 3 + 2];
			const uint16 BoneIndex = InMeshData.BonesPerVertex[VertexIndex0];
			const uint32 MinVertexIndex = FMath::Min(VertexIndex0, FMath::Min(VertexIndex1, VertexIndex2));
			const uint32 MaxVertexIndex = FMath::Max(VertexIndex0, FMath::Max(VertexIndex1, VertexIndex2));

			// Add a new span if the properties change.
			if (MaterialIndex != LastMaterialIndex || IsInternal != LastIsInternal || BoneIndex != LastBoneIndex)
			{
				FGeometryCollectionMeshElement& Span = Spans.AddZeroed_GetRef();
				Span.TransformIndex = BoneIndex;
				Span.MaterialIndex = MaterialIndex;
				Span.bIsInternal = IsInternal;
				Span.TriangleStart = TriangleIndex;
				Span.TriangleCount = 0;
				Span.VertexStart = MinVertexIndex;
				Span.VertexEnd = MaxVertexIndex;
			}

			// Update the current span.
			FGeometryCollectionMeshElement& Span = Spans.Last();
			Span.TriangleCount++;
			Span.VertexStart = FMath::Min(Span.VertexStart, MinVertexIndex);
			Span.VertexEnd = FMath::Max(Span.VertexEnd, MaxVertexIndex);

			LastMaterialIndex = MaterialIndex;
			LastIsInternal = IsInternal;
			LastBoneIndex = BoneIndex;
		}
	}

	// Sort spans by material and bone index.
	TArray<int32> SortedSpanIndices;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCollectionRenderData::SortSpans);

		TArray<uint64> SortKeys;
		SortKeys.SetNum(Spans.Num());
		SortedSpanIndices.SetNum(Spans.Num());
		for (int32 SpanIndex = 0; SpanIndex < SortedSpanIndices.Num(); SpanIndex++)
		{
			FGeometryCollectionMeshElement const& Span = Spans[SpanIndex];
			SortKeys[SpanIndex] = ((uint64)Span.MaterialIndex << 56) | ((uint64)Span.bIsInternal << 48) | ((uint64)Span.TransformIndex << 32) | (uint64)Span.TriangleStart;
			SortedSpanIndices[SpanIndex] = SpanIndex;
		}
		SortedSpanIndices.StableSort([&SortKeys](int32 A, int32 B)
		{
			return SortKeys[A] < SortKeys[B];
		});
	}

	// Now copy spans to final section arrays.
	TArray<FGeometryCollectionMeshElement> Sections;
	TArray<FGeometryCollectionMeshElement> SectionsNoInternal;
	TArray<FGeometryCollectionMeshElement> SubSections;

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCollectionRenderData::BuildSections);

		Sections.Reserve(Spans.Num());
		SectionsNoInternal.Reserve(Spans.Num());
		SubSections.Reserve(Spans.Num());

		int32 LastMaterialIndex = -1;
		uint8 LastIsInternal = 0xff;
		int32 LastBoneIndex = -1;
		int32 TriangleIndex = 0;

		for (int32 SortedSpanIndex = 0; SortedSpanIndex < SortedSpanIndices.Num(); SortedSpanIndex++)
		{
			const int32 SpanIndex = SortedSpanIndices[SortedSpanIndex];
			FGeometryCollectionMeshElement const& Span = Spans[SpanIndex];

			// Add new section if required.
			if (Span.MaterialIndex != LastMaterialIndex)
			{
				FGeometryCollectionMeshElement& Section = Sections.AddZeroed_GetRef();
				Section.TransformIndex = -1;
				Section.MaterialIndex = Span.MaterialIndex;
				Section.bIsInternal = 0;
				Section.TriangleStart = TriangleIndex;
				Section.TriangleCount = 0;
				Section.VertexStart = Span.VertexStart;
				Section.VertexEnd = Span.VertexEnd;
			}
			{
				// Accumulate this triangle to the current section.
				FGeometryCollectionMeshElement& Section = Sections.Last();
				Section.TriangleCount += Span.TriangleCount;
				Section.VertexStart = FMath::Min(Section.VertexStart, Span.VertexStart);
				Section.VertexEnd = FMath::Max(Section.VertexEnd, Span.VertexEnd);
			}

			// Add new no-internal section if required.
			// Note that internal face spans have been sorted to the end of each full section.
			if (Span.MaterialIndex != LastMaterialIndex && !Span.bIsInternal)
			{
				FGeometryCollectionMeshElement& Section = SectionsNoInternal.AddZeroed_GetRef();
				Section.TransformIndex = -1;
				Section.MaterialIndex = Span.MaterialIndex;
				Section.bIsInternal = 0;
				Section.TriangleStart = TriangleIndex;
				Section.TriangleCount = 0;
				Section.VertexStart = Span.VertexStart;
				Section.VertexEnd = Span.VertexEnd;
			}
			// Accumulate this triangle if this is non-internal section.
			if (!Span.bIsInternal)
			{
				FGeometryCollectionMeshElement& Section = SectionsNoInternal.Last();
				Section.TriangleCount += Span.TriangleCount;
				Section.VertexStart = FMath::Min(Section.VertexStart, Span.VertexStart);
				Section.VertexEnd = FMath::Max(Section.VertexEnd, Span.VertexEnd);
			}

			// Add new subsection if required.
			// Note that subsections don't care about the internal flag, since they are only ever used
			// in editor in modes where we want to render internal faces.
			if (Span.MaterialIndex != LastMaterialIndex || Span.TransformIndex != LastBoneIndex)
			{
				FGeometryCollectionMeshElement& Section = SubSections.AddZeroed_GetRef();
				Section.TransformIndex = Span.TransformIndex;
				Section.MaterialIndex = Span.MaterialIndex;
				Section.bIsInternal = 0;
				Section.TriangleStart = TriangleIndex;
				Section.TriangleCount = 0;
				Section.VertexStart = Span.VertexStart;
				Section.VertexEnd = Span.VertexEnd;
			}
			{
				// Accumulate this triangle to the current subsection.
				FGeometryCollectionMeshElement& Section = SubSections.Last();
				Section.TriangleCount += Span.TriangleCount;
				Section.VertexStart = FMath::Min(Section.VertexStart, Span.VertexStart);
				Section.VertexEnd = FMath::Max(Section.VertexEnd, Span.VertexEnd);
			}

			TriangleIndex += Span.TriangleCount;
			LastMaterialIndex = Span.MaterialIndex;
			LastIsInternal = Span.bIsInternal;
			LastBoneIndex = Span.TransformIndex;
		}

		//TODO: Clear SectionsNoInternal if it is just a duplicate of Sections.

		Sections.Shrink();
		SectionsNoInternal.Shrink();
		SubSections.Shrink();
	}

	// Copy indices to sorted index buffer.
	TArray<uint32> SortedIndices;
	SortedIndices.SetNumUninitialized(InMeshData.Indices.Num());

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCollectionRenderData::CopySortedIndices);

		// Gather destination index locations so that we can fill them in parallel.
		TArray<int32> DestIndexStarts;
		DestIndexStarts.SetNumUninitialized(SortedSpanIndices.Num());
		for (int32 SortedSpanIndex = 0; SortedSpanIndex < SortedSpanIndices.Num(); SortedSpanIndex++)
		{
			DestIndexStarts[SortedSpanIndex] = SortedSpanIndex > 0 ? DestIndexStarts[SortedSpanIndex - 1] + Spans[SortedSpanIndices[SortedSpanIndex - 1]].TriangleCount * 3 : 0;
		}

		ParallelFor(SortedSpanIndices.Num(), [&](int32 SortedSpanIndex)
		{
			const int32 SpanIndex = SortedSpanIndices[SortedSpanIndex];
			FGeometryCollectionMeshElement const& Span = Spans[SpanIndex];

			const int32 SourceIndexStart = Span.TriangleStart * 3;
			const int32 DestIndexStart = DestIndexStarts[SortedSpanIndex];

			ParallelFor(TEXT("GC:CopySortedIndices"), Span.TriangleCount, 500, [&](int32 TriangleIndex)
			{
				const int32 SourceIndex = SourceIndexStart + TriangleIndex * 3;
				const int32 DestIndex = DestIndexStart + TriangleIndex * 3;
				SortedIndices[DestIndex] = InMeshData.Indices[SourceIndex];
				SortedIndices[DestIndex + 1] = InMeshData.Indices[SourceIndex + 1];
				SortedIndices[DestIndex + 2] = InMeshData.Indices[SourceIndex + 2];
			});
		});
	}

	// Move to output.
	OutSectionData.SortedIndices = MoveTemp(SortedIndices);
	OutSectionData.Sections = MoveTemp(Sections);
	OutSectionData.SectionsNoInternal = MoveTemp(SectionsNoInternal);
	OutSectionData.SubSections = MoveTemp(SubSections);
}

FBoxSphereBounds BuildPreSkinnedBounds(FGeometryCollection& InCollection)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCollectionRenderData::BuildBounds);

	const TManagedArray<FBox>& BoundingBoxes = InCollection.BoundingBox;
	const TManagedArray<FTransform3f>& Transform = InCollection.Transform;
	const TManagedArray<int32>& Parent = InCollection.Parent;
	const TManagedArray<int32>& TransformIndices = InCollection.TransformIndex;
	const int32 NumBoxes = BoundingBoxes.Num();

	TArray<FMatrix> GlobalMatrices;
	GeometryCollectionAlgo::GlobalMatrices(Transform, Parent, GlobalMatrices);

	bool bBoundsInit = false;
	FBox PreSkinnedBounds(ForceInit);

	for (int32 BoxIndex = 0; BoxIndex < NumBoxes; ++BoxIndex)
	{
		const int32 TransformIndex = TransformIndices[BoxIndex];
		if (InCollection.IsGeometry(TransformIndex))
		{
			if (!bBoundsInit)
			{
				PreSkinnedBounds = BoundingBoxes[BoxIndex].TransformBy(GlobalMatrices[TransformIndex]);
				bBoundsInit = true;
			}
			else
			{
				PreSkinnedBounds += BoundingBoxes[BoxIndex].TransformBy(GlobalMatrices[TransformIndex]);
			}
		}
	}

	return bBoundsInit ? FBoxSphereBounds(PreSkinnedBounds) : FBoxSphereBounds(ForceInit);
}

void CreateMeshData(FGeometryCollectionBuiltMeshData const& InMeshData, FGeometryCollectionRenderData::FCreateDesc const& InDesc, FGeometryCollectionRenderData& OutRenderData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCollectionRenderData::CreateMeshData);

	OutRenderData.bHasMeshData = true;

	OutRenderData.MeshDescription.NumVertices = InMeshData.Vertices.Position.Num();
	OutRenderData.MeshDescription.NumTriangles = InMeshData.Indices.Num() / 3;

	const FConstMeshBuildVertexView VertexView = MakeConstMeshBuildVertexView(InMeshData.Vertices);

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCollectionRenderData::BufferInit);
		OutRenderData.MeshResource.PositionVertexBuffer.Init(VertexView);
		OutRenderData.MeshResource.StaticMeshVertexBuffer.SetUseFullPrecisionUVs(InDesc.bUseFullPrecisionUVs);
		OutRenderData.MeshResource.StaticMeshVertexBuffer.Init(VertexView);
		OutRenderData.MeshResource.ColorVertexBuffer.Init(VertexView);
		OutRenderData.MeshResource.BoneMapVertexBuffer.Init(InMeshData.BonesPerVertex);
	}

	FGeometryCollectionBuiltMeshSectionData SectionData;
	BuildSectionDataFromMeshData(InMeshData, SectionData);

	// Store the sorted indices that match the section data.
	OutRenderData.MeshResource.IndexBuffer.SetIndices(SectionData.SortedIndices, EIndexBufferStride::AutoDetect);

	OutRenderData.MeshDescription.Sections = MoveTemp(SectionData.Sections);
	OutRenderData.MeshDescription.SectionsNoInternal = MoveTemp(SectionData.SectionsNoInternal);
	OutRenderData.MeshDescription.SubSections = MoveTemp(SectionData.SubSections);
}

void CreateNaniteData(FGeometryCollectionBuiltMeshData&& InMeshData, FGeometryCollectionRenderData::FCreateDesc const& InDesc, FGeometryCollectionRenderData& OutRenderData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCollectionRenderData::CreateNaniteData);

	OutRenderData.bHasNaniteData = true;

	FMeshNaniteSettings NaniteSettings = {};
	NaniteSettings.bEnabled = true;
	NaniteSettings.TargetMinimumResidencyInKB = InDesc.NaniteMinimumResidencyInKB;
	NaniteSettings.KeepPercentTriangles = 1.0f;
	NaniteSettings.TrimRelativeError = 0.0f;
	NaniteSettings.FallbackPercentTriangles = 1.0f; // 100% - no reduction
	NaniteSettings.FallbackRelativeError = 0.0f;

	ClearNaniteResources(OutRenderData.NaniteResourcesPtr);

	Nanite::IBuilderModule& NaniteBuilderModule = Nanite::IBuilderModule::Get();

	// Sections are left empty because they are not used (not building a coarse representation)

	Nanite::IBuilderModule::FInputMeshData InputMeshData;
	InputMeshData.Vertices = MoveTemp(InMeshData.Vertices);
	InputMeshData.TriangleIndices = MoveTemp(InMeshData.Indices);
	InputMeshData.MaterialIndices = MoveTemp(InMeshData.MaterialsPerTriangle);
	InputMeshData.TriangleCounts = MoveTemp(InMeshData.MeshTriangleCounts);
	InputMeshData.NumTexCoords = InMeshData.NumTexCoords;
	InputMeshData.VertexBounds = InMeshData.VertexBounds;

	if (!NaniteBuilderModule.Build(
		*OutRenderData.NaniteResourcesPtr.Get(),
		InputMeshData,
		nullptr, // OutFallbackMeshData
		nullptr, // OutRayTracingFallbackMeshData
		nullptr, // RayTracingFallbackBuildSettings
		NaniteSettings))
	{
		UE_LOGF(LogStaticMesh, Error, "Failed to build Nanite for geometry collection. See previous line(s) for details.");
	}
}

TUniquePtr<FGeometryCollectionRenderData> FGeometryCollectionRenderData::Create(FGeometryCollection& InCollection, FCreateDesc const& InDesc)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FGeometryCollectionRenderData::Create);

	TUniquePtr<FGeometryCollectionRenderData> RenderData = MakeUnique<FGeometryCollectionRenderData>();

	FGeometryCollectionBuiltMeshData MeshBuildData;
	if (BuildMeshDataFromGeometryCollection(InCollection, InDesc, MeshBuildData))
	{
		if (!InDesc.bEnableNanite || InDesc.bEnableNaniteFallback)
		{
			CreateMeshData(MeshBuildData, InDesc, *RenderData.Get());
		}

		if (InDesc.bEnableNanite)
		{
			CreateNaniteData(MoveTemp(MeshBuildData), InDesc, *RenderData.Get());
		}

		RenderData->PreSkinnedBounds = BuildPreSkinnedBounds(InCollection);
	}

	return RenderData;
}

#endif // WITH_EDITOR


FGeometryCollectionRenderData::~FGeometryCollectionRenderData()
{
	// We should have released any resources before now.
	check(!bIsInitialized);
}

void FGeometryCollectionRenderData::Serialize(FArchive& Ar, UGeometryCollection& Owner)
{
#if WITH_EDITOR
	// Optionally don't cook the rendering data.
	// This is used if we expect to use a custom GC render path such as the ISM pool.
	const bool bIsArchiveValidCandidateForStrip = (Ar.IsCooking() || (Ar.IsCountingMemory() && Ar.IsFilterEditorOnly()));
	const bool bStripAllRenderDataFromCook = Owner.bStripRenderDataOnCook && bIsArchiveValidCandidateForStrip;
	const bool bStripMeshDataFromCook = bStripAllRenderDataFromCook;
	// Also strip nanite data for non-nanite platforms.
	const bool bStripNaniteDataFromCook = bStripAllRenderDataFromCook || (Ar.IsCooking() && !DoesTargetPlatformSupportNanite(Ar.CookingTarget()));
#else
	const bool bStripMeshDataFromCook = false;
	const bool bStripNaniteDataFromCook = false;
#endif

	if (bStripMeshDataFromCook)
	{
		bool bDummy = false;
		Ar << bDummy;
	}
	else
	{
		Ar << bHasMeshData;
	}

	if (bStripNaniteDataFromCook)
	{
		bool bDummy = false;
		Ar << bDummy;
	}
	else
	{
		Ar << bHasNaniteData;
	}

	if (bHasMeshData && !bStripMeshDataFromCook)
	{
		MeshResource.Serialize(Ar);
		MeshDescription.Serialize(Ar);
	}
	else if (Ar.IsLoading())
	{
		MeshResource = {};
		MeshDescription = {};
	}

	if (bHasNaniteData && !bStripNaniteDataFromCook)
	{
		InitNaniteResources(NaniteResourcesPtr);

		if (Ar.IsSaving())
		{
			// Nanite data is 1:1 with each geometry group in the collection.
			const int32 NumGeometryGroups = Owner.NumElements(FGeometryCollection::GeometryGroup);
			if (!Owner.EnableNanite || NumGeometryGroups != NaniteResourcesPtr->HierarchyRootOffsets.Num())
			{
				Ar.SetError();
			}
		}

		NaniteResourcesPtr->Serialize(Ar, &Owner, true);
	}
	else if (Ar.IsLoading())
	{
		ClearNaniteResources(NaniteResourcesPtr);
	}

	// Always write bounds.
	Ar << PreSkinnedBounds;
}

void FGeometryCollectionRenderData::InitResources(UGeometryCollection const& Owner)
{
	if (bIsInitialized)
	{
		ReleaseResources();
	}

	if (bHasMeshData)
	{
		MeshResource.InitResources(Owner);

#if RHI_RAYTRACING
		if (IsRayTracingAllowed() && Owner.bSupportRayTracing && MeshDescription.NumVertices > 0)
		{
			FName OwnerName = Owner.GetFName();

			ENQUEUE_RENDER_COMMAND(InitGeometryCollectiomRayTracingGeometry)(
				[this, OwnerName](FRHICommandListImmediate& RHICmdList)
				{
					// Create a single ray tracing geometry group to be used by parts

					check(RayTracingResource.GroupHandle.IsNull());
					RayTracingResource.GroupHandle = GRayTracingGeometryManager->RegisterRayTracingGeometryGroup(/*NumLODs*/ 0, /*CurrentFirstLODIdx*/ 0);

					// TODO: Combine sections using the same transform into a single BLAS

					RayTracingResource.PartGeometries.SetNum(MeshDescription.SubSections.Num());

					for (int32 SectionIndex = 0; SectionIndex < MeshDescription.SubSections.Num(); ++SectionIndex)
					{
						FRayTracingGeometryInitializer Initializer;
						Initializer.DebugName = FDebugName(OwnerName, SectionIndex);
						Initializer.GeometryType = RTGT_Triangles;
						Initializer.bFastBuild = true;
						Initializer.bAllowUpdate = false;
						Initializer.TotalPrimitiveCount = 0;
						Initializer.IndexBuffer = MeshResource.IndexBuffer.IndexBufferRHI;

						{
							const FGeometryCollectionMeshElement& Section = MeshDescription.SubSections[SectionIndex];

							FRayTracingGeometrySegment Segment;
							Segment.FirstPrimitive = Section.TriangleStart;
							Segment.VertexBuffer = MeshResource.PositionVertexBuffer.VertexBufferRHI;
							Segment.NumPrimitives = Section.TriangleCount;
							Segment.MaxVertices = MeshResource.PositionVertexBuffer.GetNumVertices(); // TODO: investigate why Section.VertexEnd doesn't work here when using SubSections

							Initializer.Segments.Add(Segment);

							Initializer.TotalPrimitiveCount += Section.TriangleCount;
						}

						RayTracingResource.PartGeometries[SectionIndex].GroupHandle = RayTracingResource.GroupHandle;
						RayTracingResource.PartGeometries[SectionIndex].LODIndex = FRayTracingGeometry::AdditionalGeometryLODIndex;
						RayTracingResource.PartGeometries[SectionIndex].SetInitializer(MoveTemp(Initializer));
						RayTracingResource.PartGeometries[SectionIndex].InitResource(RHICmdList);
					}
				}
				);
		}
#endif // RHI_RAYTRACING
	}

	if (bHasNaniteData)
	{
		NaniteResourcesPtr->InitResources(&Owner);
	}

	bIsInitialized = true;
}

void FGeometryCollectionRenderData::ReleaseResources()
{
	if (!bIsInitialized)
	{
		return;
	}

	if (bHasMeshData)
	{
#if RHI_RAYTRACING
		ENQUEUE_RENDER_COMMAND(CmdReleaseRayTracingGeometryGroup)(
			[this](FRHICommandListImmediate&)
			{
				for (FRayTracingGeometry& PartRayTracingGeometry : RayTracingResource.PartGeometries)
				{
					PartRayTracingGeometry.ReleaseResource();
				}
				RayTracingResource.PartGeometries.Empty();

				if (!RayTracingResource.GroupHandle.IsNull())
				{
					GRayTracingGeometryManager->ReleaseRayTracingGeometryGroup(RayTracingResource.GroupHandle);
					RayTracingResource.GroupHandle = {};
				}
			});
#endif

		MeshResource.ReleaseResources();
	}

	if (bHasNaniteData)
	{
		NaniteResourcesPtr->ReleaseResources();
	}

	bIsInitialized = false;
}
