// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RawIndexBuffer.h"
#include "Rendering/ColorVertexBuffer.h"
#include "Rendering/NaniteResources.h"
#include "Rendering/PositionVertexBuffer.h"
#include "Rendering/StaticMeshVertexBuffer.h"
#include "RayTracingGeometry.h"

class FGeometryCollection;
class UGeometryCollection;


/** Vertex Buffer for Bone Map. */
class FBoneMapVertexBuffer : public FVertexBuffer
{
public:
	~FBoneMapVertexBuffer();

	void Init(TArray<uint16> const& InBoneMap, bool bInNeedsCPUAccess = true);
	void Serialize(FArchive& Ar, bool bInNeedsCPUAccess);
	FRHIShaderResourceView* GetSRV() const { return VertexBufferSRV; }
	
	FORCEINLINE uint16& BoneIndex(uint32 VertexIndex) const
	{
		checkSlow(VertexIndex < NumVertices);
		return *(uint16*)(Data + VertexIndex * PixelFormatStride);
	}

	SIZE_T GetAllocatedSize() const { return (BoneMapData != nullptr) ? BoneMapData->GetResourceSize() : 0; }

protected:
	void CleanUp();
	void AllocateData(bool bInNeedsCPUAccess = true);
	void ResizeBuffer(uint32 InNumVertices);

	void InitRHI(FRHICommandListBase& RHICmdList) override;
	void ReleaseRHI() override;

	static const EPixelFormat PixelFormat = PF_R16_UINT;
	static const uint32 PixelFormatStride = 2;

private:
	uint32 NumVertices = 0;
	bool bNeedsCPUAccess = true;
	FShaderResourceViewRHIRef VertexBufferSRV = nullptr;
	FStaticMeshVertexDataInterface* BoneMapData = nullptr;
	uint8* Data = nullptr;
};

/** Set of GPU vertex and index buffers required to render a geometry collection. */
struct FGeometryCollectionMeshResources
{
	FRawStaticIndexBuffer IndexBuffer;
	FPositionVertexBuffer PositionVertexBuffer;
	FStaticMeshVertexBuffer StaticMeshVertexBuffer;
	FColorVertexBuffer ColorVertexBuffer;
	FBoneMapVertexBuffer BoneMapVertexBuffer;

	/** Initialize the render resources. */
	void InitResources(UGeometryCollection const& Owner);
	/** Release the render resources. */
	void ReleaseResources();
	/** Serialization. */
	void Serialize(FArchive& Ar);
};

/** Description of a single mesh element within a set of vertex and vertex buffers. */
struct FGeometryCollectionMeshElement
{
	int16 TransformIndex;
	uint8 MaterialIndex;
	uint8 bIsInternal;
	uint32 TriangleStart;
	uint32 TriangleCount;
	uint32 VertexStart;
	uint32 VertexEnd;

	/** Serialization. */
	friend FArchive& operator<<(FArchive& Ar, FGeometryCollectionMeshElement& Item)
	{
		Ar << Item.TransformIndex << Item.MaterialIndex << Item.bIsInternal << Item.TriangleStart << Item.TriangleCount << Item.VertexStart << Item.VertexEnd;
		return Ar;
	}
};

/** CPU side data required to render a geometry collection. */
struct FGeometryCollectionMeshDescription
{
	uint32 NumVertices = 0;
	uint32 NumTriangles = 0;

	// Info on grouped material sections within vertex buffers. 
	// A section has a uniform material index and can contain multiple per-bone subsections.
	TArray<FGeometryCollectionMeshElement> Sections;
	// Info on grouped material sections within vertex buffers but with internal faces added by the fracture tool removed.
	TArray<FGeometryCollectionMeshElement> SectionsNoInternal;
	// Info on individual subsections within vertex buffers. 
	// A subsection has a uniform bone and material index.
	TArray<FGeometryCollectionMeshElement> SubSections;

	/** Serialization. */
	void Serialize(FArchive& Ar);
};

#if RHI_RAYTRACING
/** Set of resources required to ray trace a geometry collection. */
struct FGeometryCollectionRayTracingResources
{
	RayTracing::FGeometryGroupHandle GroupHandle;
	TArray<FRayTracingGeometry> PartGeometries;
};
#endif

/** Built render data compiled and ready for use by the geometry collection scene proxy. */
class FGeometryCollectionRenderData
{
public:
	FGeometryCollectionRenderData() = default;
	~FGeometryCollectionRenderData();

#if WITH_EDITOR
	struct FCreateDesc
	{
		bool bEnableNanite = false;
		bool bEnableNaniteFallback = false;
		bool bUseFullPrecisionUVs = false;
		bool bConvertVertexColorsToSRGB = false;
		uint32 NaniteMinimumResidencyInKB = 0;
	};

	static TUniquePtr<FGeometryCollectionRenderData> Create(FGeometryCollection& InCollection, FCreateDesc const& InDesc);
#endif

	bool IsInitialized()
	{
		return bIsInitialized;
	}

	/** Serialization. */
	void Serialize(FArchive& Ar, UGeometryCollection& Owner);

	/** Initialize the render resources. */
	void InitResources(UGeometryCollection const& Owner);

	/** Releases the render resources. */
	void ReleaseResources();

	/** False if we don't store valid non-nanite mesh data. */
	bool bHasMeshData = false;
	/** False if we don't store valid nanite data. */
	bool bHasNaniteData = false;

	/** Mesh GPU resources. */
	FGeometryCollectionMeshResources MeshResource;

	/** Mesh CPU description. */
	FGeometryCollectionMeshDescription MeshDescription;

	/** Nanite resources. */
	TPimplPtr<Nanite::FResources> NaniteResourcesPtr;

	/** Object local bounds before any animation. */
	FBoxSphereBounds PreSkinnedBounds = FBoxSphereBounds(ForceInitToZero);

#if RHI_RAYTRACING
	FGeometryCollectionRayTracingResources RayTracingResource;
#endif

private:
	bool bIsInitialized = false;
};
