// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionRenderLevelSetActor.h"

#include "EngineUtils.h"

#include "Chaos/ArrayND.h"
#include "Chaos/Vector.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionRenderLevelSetActor)

DEFINE_LOG_CATEGORY_STATIC(LSR_LOG, Log, All);

AGeometryCollectionRenderLevelSetActor* AGeometryCollectionRenderLevelSetActor::FindOrCreate(UWorld* World)
{
	AGeometryCollectionRenderLevelSetActor* Actor = nullptr;
	if (!World)
	{
		UE_LOGF(LSR_LOG, Warning, "No valid World where to search for an existing GeometryCollectionRenderLevelSetActor singleton actor.");
	}
	else
	{
		const TActorIterator<AGeometryCollectionRenderLevelSetActor> ActorIterator(World);
		if (ActorIterator)
		{
			Actor = *ActorIterator;
		}
		else
		{
			FActorSpawnParameters SpawnInfo;
			SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
			Actor = World->SpawnActor<AGeometryCollectionRenderLevelSetActor>(SpawnInfo);
		}
		if (!Actor)
		{
			UE_LOGF(LSR_LOG, Warning, "No GeometryCollectionRenderLevelSetActor singleton actor could be found or created.");
		}
	}
	return Actor;
}

AGeometryCollectionRenderLevelSetActor::AGeometryCollectionRenderLevelSetActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), SurfaceTolerance(0.01f), Isovalue(0.f), Enabled(true), RenderVolumeBoundingBox(false), DynRayMarchMaterial(NULL), StepSizeMult(1.f)
{
	PostProcessComponent = CreateDefaultSubobject<UPostProcessComponent>(TEXT("PostProcessComponent0"));
	RootComponent = PostProcessComponent;

	// set initial values
	TargetVolumeTexture = LoadObject<UVolumeTexture>(NULL, TEXT("/Engine/EngineDebugMaterials/VolumeToRender"), NULL, LOAD_None, NULL);
	UMaterialInterface* MaterialInterface = LoadObject<UMaterialInterface>(NULL, TEXT("/Engine/EngineDebugMaterials/M_VolumeRenderSphereTracePP"), NULL, LOAD_None, NULL);
	RayMarchMaterial = MaterialInterface ? MaterialInterface->GetBaseMaterial() : nullptr;
}

void AGeometryCollectionRenderLevelSetActor::BeginPlay()
{
	Super::BeginPlay();

	// make sure to set enabled on the post process
	PostProcessComponent->bEnabled = Enabled;
	PostProcessComponent->bUnbound = true;
}

#if WITH_EDITOR
void AGeometryCollectionRenderLevelSetActor::PostEditChangeProperty(struct FPropertyChangedEvent& e)
{
	Super::PostEditChangeProperty(e);

	// sync all rendering properties each time a param changes.
	// @todo: optimize to only update parameters when rendering-specific ones are edited
	SyncMaterialParameters();
}
#endif

void AGeometryCollectionRenderLevelSetActor::SyncMaterialParameters()
{
	if (!RayMarchMaterial)
	{
		return;
	}

	// make dynamic material instance if it hasn't been created yet
	if (!DynRayMarchMaterial) {
		DynRayMarchMaterial = UMaterialInstanceDynamic::Create(RayMarchMaterial, this);

		// add the blendable with our post process material
		PostProcessComponent->AddOrUpdateBlendable(DynRayMarchMaterial);
	}

	// Sync all render parameters to our material
	DynRayMarchMaterial->SetScalarParameterValue("Surface Tolerance", SurfaceTolerance);
	DynRayMarchMaterial->SetScalarParameterValue("Isovalue", Isovalue);
	
	
	DynRayMarchMaterial->SetScalarParameterValue("Step Size Mult", StepSizeMult);
	DynRayMarchMaterial->SetScalarParameterValue("Voxel Size", VoxelSize);

	DynRayMarchMaterial->SetVectorParameterValue("Min Bounds", MinBBoxCorner);
	DynRayMarchMaterial->SetVectorParameterValue("Max Bounds", MaxBBoxCorner);

	DynRayMarchMaterial->SetVectorParameterValue("WorldToLocalc0", FLinearColor(WorldToLocal.GetColumn(0)));
	DynRayMarchMaterial->SetVectorParameterValue("WorldToLocalc1", FLinearColor(WorldToLocal.GetColumn(1)));
	DynRayMarchMaterial->SetVectorParameterValue("WorldToLocalc2", FLinearColor(WorldToLocal.GetColumn(2)));
	DynRayMarchMaterial->SetVectorParameterValue("WorldToLocalTranslation", FLinearColor(WorldToLocal.GetOrigin()));	

	DynRayMarchMaterial->SetTextureParameterValue("Volume To Render", TargetVolumeTexture);

	DynRayMarchMaterial->SetScalarParameterValue("Debug BBox", (float) RenderVolumeBoundingBox);

	PostProcessComponent->bEnabled = Enabled;
}

void AGeometryCollectionRenderLevelSetActor::SyncLevelSetTransform(const FTransform &LocalToWorld)
{
	if (!RayMarchMaterial)
	{
		return;
	}

	WorldToLocal = LocalToWorld.Inverse().ToMatrixWithScale();
	DynRayMarchMaterial->SetVectorParameterValue("WorldToLocalc0", FLinearColor(WorldToLocal.GetColumn(0)));
	DynRayMarchMaterial->SetVectorParameterValue("WorldToLocalc1", FLinearColor(WorldToLocal.GetColumn(1)));
	DynRayMarchMaterial->SetVectorParameterValue("WorldToLocalc2", FLinearColor(WorldToLocal.GetColumn(2)));
	DynRayMarchMaterial->SetVectorParameterValue("WorldToLocalTranslation", FLinearColor(WorldToLocal.GetOrigin()));
}

bool AGeometryCollectionRenderLevelSetActor::SetLevelSetToRender(const Chaos::FLevelSet &LevelSet, const FTransform &LocalToWorld)
{
	using namespace Chaos;

	// error case when the target volume texture isn't set
	if (TargetVolumeTexture == NULL)
	{
		UE_LOGF(LSR_LOG, Warning, "Target UVolumeTexture is null on %ls", *GetFullName());
		return false;
	}

	// get refs to the grid structures
	const TArrayND<FReal, 3>& LevelSetPhiArray = LevelSet.GetPhiArray();
	const TArrayND<FVec3, 3>& LevelSetNormalsArray = LevelSet.GetNormalsArray();
	const TUniformGrid<FReal, 3>& LevelSetGrid = LevelSet.GetGrid();

	const TVec3<int32>& Counts = LevelSetGrid.Counts();
	
	// set bounding box
	MinBBoxCorner = LevelSetGrid.MinCorner();
	MaxBBoxCorner = LevelSetGrid.MaxCorner();
	WorldToLocal = LocalToWorld.Inverse().ToMatrixWithScale();

	// @todo: do we need to deal with non square voxels?
	VoxelSize = LevelSetGrid.Dx().X;

	// Error case when the voxel size is sufficiently small
	if (VoxelSize < KINDA_SMALL_NUMBER)
	{
		UE_LOGF(LSR_LOG, Warning, "Voxel size is too small on %ls", *GetFullName());
		return false;
	}

	// lambda for querying the level set information
	// @todo: we could encode voxel ordering more nicely in the UVolumeTexture
	auto QueryVoxel = [&LevelSetPhiArray, &LevelSetNormalsArray](int32 PosX, int32 PosY, int32 PosZ, void* Value)
	{
		const FVec3 Normal = LevelSetNormalsArray(TVec3<int32>(PosX, PosY, PosZ)).GetSafeNormal();
		const float Phi = LevelSetPhiArray(TVec3<int32>(PosX, PosY, PosZ));

		FFloat16* const Voxel = static_cast<FFloat16*>(Value);  // TSF_RGBA16F
		Voxel[0] = Normal.X;
		Voxel[1] = Normal.Y;
		Voxel[2] = Normal.Z;
		Voxel[3] = Phi;
	};

	// fill volume texture from level set
	const bool success = TargetVolumeTexture->UpdateSourceFromFunction(QueryVoxel, Counts.X, Counts.Y, Counts.Z, TSF_RGBA16F);

	if (!success)
	{
		UE_LOGF(LSR_LOG, Warning, "Couldn't create target volume texture from FLevelSet with %ls", *GetFullName());
		return false;
	}

	// set all parameters on our dynamic material instance to sync state
	SyncMaterialParameters();

	UE_LOGF(LSR_LOG, Log, "Volume Bounds: %ls - %ls -- Volume Dims: %d %d %d -- Voxel Size: %f -- World To Local: %ls", *MinBBoxCorner.ToString(), *MaxBBoxCorner.ToString(), Counts.X, Counts.Y, Counts.Z, VoxelSize, *WorldToLocal.ToString());

	return true;
}
