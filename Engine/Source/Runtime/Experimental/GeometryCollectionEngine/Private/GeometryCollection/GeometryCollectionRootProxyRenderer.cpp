// Copyright Epic Games, Inc. All Rights Reserved.
#include "GeometryCollection/GeometryCollectionRootProxyRenderer.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/CollisionProfile.h"
#include "Engine/StaticMesh.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionObject.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GeometryCollectionRootProxyRenderer)

void UGeometryCollectionRootProxyRenderer::OnRegisterGeometryCollection(UGeometryCollectionComponent& InComponent)
{
	CreateRootProxyComponents(InComponent);
}

void UGeometryCollectionRootProxyRenderer::OnUnregisterGeometryCollection()
{
	ClearRootProxyComponents();
}

void UGeometryCollectionRootProxyRenderer::UpdateState(UGeometryCollection const& InGeometryCollection, FTransform const& InComponentTransform, uint32 InStateFlags)
{
	const bool bIsStateVisible = (InStateFlags & EState_Visible) != 0;
	const bool bIsStateBroken = (InStateFlags & EState_Broken) != 0;
	const bool bSetVisible = !bIsStateBroken && bIsStateVisible;
	
	if (bIsVisible != bSetVisible)
	{
		bIsVisible = bSetVisible;

		for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
		{
			if (StaticMeshComponent)
			{
				StaticMeshComponent->SetVisibility(bIsVisible);
			}
		}
	}
}

void UGeometryCollectionRootProxyRenderer::UpdateRootTransform(UGeometryCollection const& InGeometryCollection, FTransform const& InRootTransform)
{
	UpdateRootProxyTransforms(InGeometryCollection, InRootTransform, {});
}

void UGeometryCollectionRootProxyRenderer::UpdateRootTransforms(UGeometryCollection const& InGeometryCollection, FTransform const& InRootTransform, TArrayView<const FTransform3f> InRootTransforms)
{
	UpdateRootProxyTransforms(InGeometryCollection, InRootTransform, InRootTransforms);
}

void UGeometryCollectionRootProxyRenderer::UpdateTransforms(UGeometryCollection const& InGeometryCollection, TArrayView<const FTransform3f> InTransforms)
{
	// Don't support non root proxy mesh.
}

void UGeometryCollectionRootProxyRenderer::CreateRootProxyComponents(UGeometryCollectionComponent& InComponent)
{
	UGeometryCollection const* GeometryCollection = InComponent.GetRestCollection();
	if (GeometryCollection == nullptr)
	{
		return;
	}
	ClearRootProxyComponents();

	const FGeometryCollectionProxyMeshData& ProxyData = GeometryCollection->RootProxyData;
	const int32 NumProxyMesh = ProxyData.ProxyMeshes.Num();
	StaticMeshComponents.Reserve(NumProxyMesh);

	for (int32 MeshIndex = 0; MeshIndex < NumProxyMesh; ++MeshIndex)
	{
		UStaticMeshComponent* MeshComponent = nullptr;
		
		if (UStaticMesh* ProxyMesh = ProxyData.ProxyMeshes[MeshIndex])
		{
			MeshComponent = NewObject<UStaticMeshComponent>(this, NAME_None, RF_DuplicateTransient | RF_Transient);

			MeshComponent->SetStaticMesh(ProxyMesh);
			MeshComponent->SetCanEverAffectNavigation(false);
			MeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			MeshComponent->SetMobility(InComponent.Mobility);
			MeshComponent->SetupAttachment(&InComponent);
			MeshComponent->RegisterComponent();

			if (ProxyData.MeshOverrideMaterials.IsValidIndex(MeshIndex))
			{
				const FGeometryCollectionProxyMeshMaterials& ProxyMaterials = ProxyData.MeshOverrideMaterials[MeshIndex];
				for (int32 MatIndex = 0; MatIndex < ProxyMaterials.Materials.Num(); ++MatIndex)
				{
					MeshComponent->SetMaterial(MatIndex, ProxyMaterials.Materials[MatIndex]);
				}
			}
		}

		StaticMeshComponents.Add(MeshComponent);
	}
}

void UGeometryCollectionRootProxyRenderer::UpdateRootProxyTransforms(UGeometryCollection const& InGeometryCollection, FTransform const& InRootTransform, TArrayView<const FTransform3f> InLocalRootTransforms)
{
	for (int32 MeshIndex = 0; MeshIndex < StaticMeshComponents.Num(); MeshIndex++)
	{
		if (UStaticMeshComponent* MeshComponent = StaticMeshComponents[MeshIndex])
		{
			if (InLocalRootTransforms.IsValidIndex(MeshIndex))
			{
				MeshComponent->SetRelativeTransform(FTransform(InLocalRootTransforms[MeshIndex]) * InRootTransform);
			}
			else
			{
				MeshComponent->SetRelativeTransform(InRootTransform);
			}
		}
	}
}

void UGeometryCollectionRootProxyRenderer::ClearRootProxyComponents()
{
	for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
	{
		if (StaticMeshComponent)
		{
			StaticMeshComponent->DestroyComponent();
		}
	}
	StaticMeshComponents.Reset();
}
