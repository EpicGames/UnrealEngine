// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCollection/GeometryCollectionSceneProxy.h"

#include "Async/ParallelFor.h"
#include "Engine/Engine.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "MaterialDomain.h"
#include "MaterialShaderType.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "CommonRenderResources.h"
#include "Rendering/NaniteResources.h"
#include "PrimitiveSceneInfo.h"
#include "GeometryCollection/GeometryCollectionComponent.h"
#include "GeometryCollection/GeometryCollectionAlgo.h"
#include "GeometryCollection/GeometryCollectionHitProxy.h"
#include "GeometryCollection/GeometryCollectionDebugDraw.h"
#include "RHIDefinitions.h"
#include "ComponentReregisterContext.h"
#include "ComponentRecreateRenderStateContext.h"
#include "RenderGraphBuilder.h"
#include "MeshPaintVisualize.h"
#include "SceneView.h"
#include "PSOPrecacheSettings.h"

#if RHI_RAYTRACING
#include "RayTracingInstance.h"
#endif

#if INTEL_ISPC
#if USING_CODE_ANALYSIS
    MSVC_PRAGMA( warning( push ) )
    MSVC_PRAGMA( warning( disable : ALL_CODE_ANALYSIS_WARNINGS ) )
#endif    // USING_CODE_ANALYSIS

#ifdef __clang__
#pragma clang diagnostic push
#pragma clang diagnostic ignored "-Wnonportable-include-path"
#endif

#include "GeometryCollectionSceneProxy.ispc.generated.h"

#ifdef __clang__
#pragma clang diagnostic pop
#endif

#if USING_CODE_ANALYSIS
    MSVC_PRAGMA( warning( pop ) )
#endif    // USING_CODE_ANALYSIS

static_assert(sizeof(ispc::FMatrix44f) == sizeof(FMatrix44f), "sizeof(ispc::FMatrix44f) != sizeof(FMatrix44f)");
static_assert(sizeof(ispc::FVector3f) == sizeof(FVector3f), "sizeof(ispc::FVector3f) != sizeof(FVector3f)");
#endif

static bool GGeometryCollectionUseCachedMDCs = true;
static FAutoConsoleVariableRef CVarGeometryCollectionUseCachedMDCs(
	TEXT("r.GeometryCollection.UseCachedMDCs"),
	GGeometryCollectionUseCachedMDCs,
	TEXT("Whether geometry collections will take the cached MDC path."),
	ECVF_RenderThreadSafe);

static int32 GParallelGeometryCollectionBatchSize = 1024;
static TAutoConsoleVariable<int32> CVarParallelGeometryCollectionBatchSize(
	TEXT("r.ParallelGeometryCollectionBatchSize"),
	GParallelGeometryCollectionBatchSize,
	TEXT("The number of vertices per thread dispatch in a single collection. \n"),
	ECVF_Default
);

static int32 GGeometryCollectionTripleBufferUploads = 1;
FAutoConsoleVariableRef CVarGeometryCollectionTripleBufferUploads(
	TEXT("r.GeometryCollectionTripleBufferUploads"),
	GGeometryCollectionTripleBufferUploads,
	TEXT("Whether to triple buffer geometry collection uploads, which allows Lock_NoOverwrite uploads which are much faster on the GPU with large amounts of data."),
	ECVF_Default
);

static int32 GRayTracingGeometryCollection = 0;
FAutoConsoleVariableRef CVarRayTracingGeometryCollection(
	TEXT("r.RayTracing.Geometry.GeometryCollection"),
	GRayTracingGeometryCollection,
	TEXT("Include geometry collection proxy meshes in ray tracing effects (default = 0 (Geometry collection meshes disabled in ray tracing))"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			FGlobalComponentRecreateRenderStateContext Context;
		}),
	ECVF_RenderThreadSafe
);

static bool GRayTracingGeometryCollectionWPO = true;
static FAutoConsoleVariableRef CVarRayTracingGeometryCollectionWPO(
	TEXT("r.RayTracing.Geometry.GeometryCollection.WPO"),
	GRayTracingGeometryCollectionWPO,
	TEXT("Whether to update geometry collection ray tracing representation based on material World Position Offset."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			FGlobalComponentRecreateRenderStateContext Context;
		}),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingGeometryCollectionForceUpdate(
	TEXT("r.RayTracing.Geometry.GeometryCollection.ForceUpdate"),
	0,
	TEXT("Forces ray tracing representation for geometry collections meshes to be updated every frame."),
	ECVF_RenderThreadSafe
);

static TAutoConsoleVariable<int32> CVarRayTracingGeometryCollectionCombinedBLAS(
	TEXT("r.RayTracing.Geometry.GeometryCollection.CombinedBLAS"),
	0,
	TEXT("Whether to always use a combined BLAS instead of one instance per collection part.\n")
	TEXT("A combined BLAS needs to be fully rebuilt whenever any transform changes.\n")
	TEXT("This is automatically enabled for geometry collections using WPO since BLAS must be updated anyway."),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable)
		{
			FGlobalComponentRecreateRenderStateContext Context;
		}),
	ECVF_RenderThreadSafe
);


#if !defined(CHAOS_GEOMETRY_COLLECTION_SET_DYNAMIC_DATA_ISPC_ENABLED_DEFAULT)
#define CHAOS_GEOMETRY_COLLECTION_SET_DYNAMIC_DATA_ISPC_ENABLED_DEFAULT 1
#endif

// Support run-time toggling on supported platforms in non-shipping configurations
#if !INTEL_ISPC || UE_BUILD_SHIPPING
static constexpr bool bGeometryCollection_SetDynamicData_ISPC_Enabled = INTEL_ISPC && CHAOS_GEOMETRY_COLLECTION_SET_DYNAMIC_DATA_ISPC_ENABLED_DEFAULT;
#else
static bool bGeometryCollection_SetDynamicData_ISPC_Enabled = CHAOS_GEOMETRY_COLLECTION_SET_DYNAMIC_DATA_ISPC_ENABLED_DEFAULT;
static FAutoConsoleVariableRef CVarGeometryCollectionSetDynamicDataISPCEnabled(TEXT("r.GeometryCollectionSetDynamicData.ISPC"), bGeometryCollection_SetDynamicData_ISPC_Enabled, TEXT("Whether to use ISPC optimizations to set dynamic data in geometry collections"));
#endif

DEFINE_LOG_CATEGORY_STATIC(FGeometryCollectionSceneProxyLogging, Log, All);

FGeometryCollectionDynamicDataPool GDynamicDataPool;

static void UpdateLooseParameter(
	FRHICommandListBase& RHICmdList,
	FGeometryCollectionVertexFactory& VertexFactory,
	FRHIShaderResourceView* BoneTransformSRV,
	FRHIShaderResourceView* BonePrevTransformSRV,
	FRHIShaderResourceView* BoneMapSRV)
{
	FGCBoneLooseParameters LooseParameters;

	LooseParameters.VertexFetch_BoneTransformBuffer = BoneTransformSRV;
	LooseParameters.VertexFetch_BonePrevTransformBuffer = BonePrevTransformSRV;
	LooseParameters.VertexFetch_BoneMapBuffer = BoneMapSRV;

	RHICmdList.UpdateUniformBuffer(VertexFactory.LooseParameterUniformBuffer, &LooseParameters);
}

FGeometryCollectionSceneProxyBase::FGeometryCollectionSceneProxyBase(UGeometryCollectionComponent* Component, bool bInIsNanite)
	: bIsNanite(bInIsNanite)
	, FeatureLevel(Component->GetScene()->GetFeatureLevel())
	, MeshResource(Component->GetRestCollection()->RenderData->MeshResource)
	, MaterialRelevance(Component->GetMaterialRelevance(Component->GetScene()->GetShaderPlatform()))
	, VertexFactory(FeatureLevel)
	, bUseShaderBoneTransform(VertexFactory.UseShaderBoneTransform(Component->GetScene()->GetShaderPlatform()))
	, bSupportsTripleBufferVertexUpload(GRHISupportsMapWriteNoOverwrite)
#if RHI_RAYTRACING
	, bHasRayTracingRepresentation(GRayTracingGeometryCollection && IsRayTracingEnabled() && Component->GetRestCollection()->bSupportRayTracing && Component->GetRestCollection()->RenderData->MeshDescription.NumVertices > 0)
	, RayTracingResources(Component->GetRestCollection()->RenderData->RayTracingResource)
#endif
{
	if (!bIsNanite || bHasRayTracingRepresentation)
	{
		MeshDescription = Component->GetRestCollection()->RenderData->MeshDescription; // TODO: use const-reference instead?

		const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> Collection = Component->GetRestCollection()->GetGeometryCollection();

		NumTransforms = Collection ? Collection->NumElements(FTransformCollection::TransformGroup) : 0;

		Materials.Empty();
		const int32 NumMaterials = Component->GetNumMaterials();
		for (int MaterialIndex = 0; MaterialIndex < NumMaterials; ++MaterialIndex)
		{
			Materials.Push(Component->GetMaterial(MaterialIndex));

			if (Materials[MaterialIndex] == nullptr || !Materials[MaterialIndex]->CheckMaterialUsage_Concurrent(MATUSAGE_GeometryCollections))
			{
				Materials[MaterialIndex] = UMaterial::GetDefaultMaterial(MD_Surface);
			}
			else if (Component->GetPSOPrecacheComponentData().UsePSOPrecacheFallbackMaterial())
			{
				Materials[MaterialIndex] = UPSOPrecacheSettingsManager::GetFallbackMaterial() ? UPSOPrecacheSettingsManager::GetFallbackMaterial() : UMaterial::GetDefaultMaterial(MD_Surface);
			}
		}
	}

	PreSkinnedBounds = Component->GetRestCollection()->RenderData->PreSkinnedBounds;

	DynamicData = Component->InitDynamicData(true);
}

FGeometryCollectionSceneProxyBase::~FGeometryCollectionSceneProxyBase()
{
	if (DynamicData != nullptr)
	{
		GDynamicDataPool.Release(DynamicData);
		DynamicData = nullptr;
	}
}

void FGeometryCollectionSceneProxyBase::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
#if RHI_RAYTRACING
	if (IsRayTracingAllowed() && bHasRayTracingRepresentation)
	{
		// copy RayTracingGeometryGroupHandle from FGeometryCollectionRayTracingResources since FGeometryCollectionRenderData can be released before the proxy is destroyed
		RayTracingGeometryGroupHandle = RayTracingResources.GroupHandle;
	}
#endif

	if (!bIsNanite || bHasRayTracingRepresentation)
	{
		if (bUseShaderBoneTransform)
		{
			// Initialize transform buffers and upload rest transforms.
			TransformBuffers.AddDefaulted(1);

			TransformBuffers[0].NumTransforms = NumTransforms;
			TransformBuffers[0].InitResource(RHICmdList);

			const bool bLocalGeometryCollectionTripleBufferUploads = (GGeometryCollectionTripleBufferUploads != 0) && bSupportsTripleBufferVertexUpload;
			const EResourceLockMode LockMode = bLocalGeometryCollectionTripleBufferUploads ? RLM_WriteOnly_NoOverwrite : RLM_WriteOnly;

			FGeometryCollectionTransformBuffer& TransformBuffer = GetCurrentTransformBuffer();
			TransformBuffer.UpdateDynamicData(RHICmdList, DynamicData->Transforms, LockMode);
		}
		else
		{
			// Initialize CPU skinning buffer with rest transforms.
			SkinnedPositionVertexBuffer.Init(MeshResource.PositionVertexBuffer.GetNumVertices(), false);
			SkinnedPositionVertexBuffer.InitResource(RHICmdList);
			UpdateSkinnedPositions(RHICmdList, DynamicData->Transforms);
		}

		SetupVertexFactory(RHICmdList, VertexFactory);
	}

	bRenderResourcesCreated = true;
}

void FGeometryCollectionSceneProxyBase::DestroyRenderThreadResources()
{
	bRenderResourcesCreated = false;

	if (!bIsNanite || bHasRayTracingRepresentation)
	{
		if (bUseShaderBoneTransform)
		{
			for (int32 i = 0; i < TransformBuffers.Num(); i++)
			{
				TransformBuffers[i].ReleaseResource();
			}
			TransformBuffers.Reset();
		}
		else
		{
			SkinnedPositionVertexBuffer.ReleaseResource();
		}
	}

	VertexFactory.ReleaseResource();

#if RHI_RAYTRACING
	if (bHasRayTracingRepresentation)
	{
		RayTracingGeometry.ReleaseResource();
		RayTracingDynamicVertexBuffer.Release();
	}
#endif
}

void FGeometryCollectionSceneProxyBase::SetupVertexFactory(FRHICommandListBase& RHICmdList, FGeometryCollectionVertexFactory& GeometryCollectionVertexFactory, FColorVertexBuffer* ColorOverride) const
{
	checkf(GeometryCollectionVertexFactory.SupportsManualVertexFetch(FeatureLevel) == VertexFactory.SupportsManualVertexFetch(FeatureLevel), TEXT("Setting up vertex factory for manual vertex fetch but provided type doesn't support it."));

	FGeometryCollectionVertexFactory::FDataType Data;

	FPositionVertexBuffer const& PositionVB = bUseShaderBoneTransform ? MeshResource.PositionVertexBuffer : SkinnedPositionVertexBuffer;
	PositionVB.BindPositionVertexBuffer(&GeometryCollectionVertexFactory, Data);

	MeshResource.StaticMeshVertexBuffer.BindTangentVertexBuffer(&GeometryCollectionVertexFactory, Data);
	MeshResource.StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(&GeometryCollectionVertexFactory, Data);
	MeshResource.StaticMeshVertexBuffer.BindLightMapVertexBuffer(&GeometryCollectionVertexFactory, Data, 0);

	FColorVertexBuffer const& ColorVB = ColorOverride ? *ColorOverride : MeshResource.ColorVertexBuffer;
	ColorVB.BindColorVertexBuffer(&GeometryCollectionVertexFactory, Data);

	if (bUseShaderBoneTransform)
	{
		Data.BoneMapSRV = MeshResource.BoneMapVertexBuffer.GetSRV();
		Data.BoneTransformSRV = GetCurrentTransformBuffer().VertexBufferSRV;
		Data.BonePrevTransformSRV = GetCurrentPrevTransformBuffer().VertexBufferSRV;
	}
	else
	{
		// Make sure these are not null to pass UB validation
		Data.BoneMapSRV = GNullColorVertexBuffer.VertexBufferSRV;
		Data.BoneTransformSRV = GNullColorVertexBuffer.VertexBufferSRV;
		Data.BonePrevTransformSRV = GNullColorVertexBuffer.VertexBufferSRV;
	}

	GeometryCollectionVertexFactory.SetData(RHICmdList, Data);

	if (!GeometryCollectionVertexFactory.IsInitialized())
	{
		GeometryCollectionVertexFactory.InitResource(RHICmdList);
	}
	else
	{
		GeometryCollectionVertexFactory.UpdateRHI(RHICmdList);
	}
}

/** Called on render thread to setup dynamic geometry for rendering */
void FGeometryCollectionSceneProxyBase::SetDynamicData_RenderThread(FRHICommandListBase& RHICmdList, FGeometryCollectionDynamicData* NewDynamicData)
{
	if (NewDynamicData != DynamicData)
	{
		if (DynamicData)
		{
			GDynamicDataPool.Release(DynamicData);
			DynamicData = nullptr;
		}
		DynamicData = NewDynamicData;
	}

	if (MeshDescription.NumVertices == 0 || !DynamicData || !bRenderResourcesCreated)
	{
		return;
	}

	if (!bIsNanite || bHasRayTracingRepresentation)
	{
		if (bUseShaderBoneTransform)
		{
			const bool bLocalGeometryCollectionTripleBufferUploads = (GGeometryCollectionTripleBufferUploads != 0) && bSupportsTripleBufferVertexUpload;

			if (bLocalGeometryCollectionTripleBufferUploads && TransformBuffers.Num() == 1)
			{
				TransformBuffers.AddDefaulted(3);
				check(TransformBuffers.Num() == 4);

				for (int32 i = 1; i < TransformBuffers.Num(); i++)
				{
					TransformBuffers[i].NumTransforms = NumTransforms;
					TransformBuffers[i].InitResource(RHICmdList);
				}
			}

			// Copy the transform data over to the vertex buffer	
			{
				const EResourceLockMode LockMode = bLocalGeometryCollectionTripleBufferUploads ? RLM_WriteOnly_NoOverwrite : RLM_WriteOnly;

				CycleTransformBuffers(bLocalGeometryCollectionTripleBufferUploads);

				FGeometryCollectionTransformBuffer& TransformBuffer = GetCurrentTransformBuffer();
				const FGeometryCollectionTransformBuffer& PrevTransformBuffer = GetCurrentPrevTransformBuffer();

				VertexFactory.SetBoneTransformSRV(TransformBuffer.VertexBufferSRV);
				VertexFactory.SetBonePrevTransformSRV(PrevTransformBuffer.VertexBufferSRV);

				TransformBuffer.UpdateDynamicData(RHICmdList, DynamicData->Transforms, LockMode);

				UpdateLooseParameter(RHICmdList, VertexFactory, TransformBuffer.VertexBufferSRV, PrevTransformBuffer.VertexBufferSRV, MeshResource.BoneMapVertexBuffer.GetSRV());
			}
		}
		else
		{
			UpdateSkinnedPositions(RHICmdList, DynamicData->Transforms);
		}
	}

#if RHI_RAYTRACING
	if (RayTracingGeometry.IsInitialized())
	{
		RayTracingGeometry.SetRequiresBuild(true);
	}
#endif
}

void FGeometryCollectionSceneProxyBase::UpdateSkinnedPositions(FRHICommandListBase& RHICmdList, TArray<FMatrix44f> const& Transforms)
{
	const int32 VertexStride = SkinnedPositionVertexBuffer.GetStride();
	const int32 VertexCount = SkinnedPositionVertexBuffer.GetNumVertices();
	check(VertexCount == MeshDescription.NumVertices)

		void* VertexBufferData = RHICmdList.LockBuffer(SkinnedPositionVertexBuffer.VertexBufferRHI, 0, VertexCount * VertexStride, RLM_WriteOnly);
	check(VertexBufferData != nullptr);

	FPositionVertexBuffer const& SourcePositionVertexBuffer = MeshResource.PositionVertexBuffer;
	FBoneMapVertexBuffer const& SourceBoneMapVertexBuffer = MeshResource.BoneMapVertexBuffer;

	bool bParallelGeometryCollection = true;
	int32 ParallelGeometryCollectionBatchSize = CVarParallelGeometryCollectionBatchSize.GetValueOnRenderThread();

	int32 NumBatches = (VertexCount / ParallelGeometryCollectionBatchSize);

	if (VertexCount != ParallelGeometryCollectionBatchSize)
	{
		NumBatches++;
	}

	// Batch too small, don't bother with parallel
	if (ParallelGeometryCollectionBatchSize > VertexCount)
	{
		bParallelGeometryCollection = false;
		ParallelGeometryCollectionBatchSize = VertexCount;
	}

	auto GeometryCollectionBatch([&](int32 BatchNum)
		{
			uint32 IndexOffset = ParallelGeometryCollectionBatchSize * BatchNum;
			uint32 ThisBatchSize = ParallelGeometryCollectionBatchSize;

			// Check for final batch
			if (IndexOffset + ParallelGeometryCollectionBatchSize > MeshDescription.NumVertices)
			{
				ThisBatchSize = VertexCount - IndexOffset;
			}

			if (ThisBatchSize > 0)
			{
				const FMatrix44f* RESTRICT BoneTransformsPtr = Transforms.GetData();

				if (bGeometryCollection_SetDynamicData_ISPC_Enabled)
				{
#if INTEL_ISPC
					uint8* VertexBufferOffset = (uint8*)VertexBufferData + (IndexOffset * VertexStride);
					ispc::SetDynamicData_RenderThread(
						(ispc::FVector3f*)VertexBufferOffset,
						ThisBatchSize,
						VertexStride,
						&SourceBoneMapVertexBuffer.BoneIndex(IndexOffset),
						(ispc::FMatrix44f*)BoneTransformsPtr,
						(ispc::FVector3f*)&SourcePositionVertexBuffer.VertexPosition(IndexOffset));
#endif
				}
				else
				{
					for (uint32 i = IndexOffset; i < IndexOffset + ThisBatchSize; i++)
					{
						FVector3f Transformed = BoneTransformsPtr[SourceBoneMapVertexBuffer.BoneIndex(i)].TransformPosition(SourcePositionVertexBuffer.VertexPosition(i));
						FMemory::Memcpy((uint8*)VertexBufferData + (i * VertexStride), &Transformed, sizeof(FVector3f));
					}
				}
			}
		});

	ParallelFor(NumBatches, GeometryCollectionBatch, !bParallelGeometryCollection);

	RHICmdList.UnlockBuffer(SkinnedPositionVertexBuffer.VertexBufferRHI);
}

TConstArrayView<FGeometryCollectionMeshElement> FGeometryCollectionSceneProxyBase::GetSectionArray(bool bUsesSubSections, bool bRemoveInternalFaces) const
{
	return bUsesSubSections
		? MeshDescription.SubSections
		: bRemoveInternalFaces ? MeshDescription.SectionsNoInternal : MeshDescription.Sections;
}

void FGeometryCollectionSceneProxyBase::GetMeshElement(
	int32 LODIndex,
	int32 SectionIndex,
	const FGeometryCollectionMeshElement& Section,
	const FVertexFactory* InVertexFactory,
	FMaterialRenderProxy* MaterialProxy,
	uint8 InDepthPriorityGroup,
	FRHIUniformBuffer* UniformBuffer,
	bool bWireframe,
	bool bReverseCulling,
	FMeshBatch& OutMeshBatch) const
{
	OutMeshBatch.bWireframe = bWireframe;
	OutMeshBatch.SegmentIndex = SectionIndex;
	OutMeshBatch.VertexFactory = InVertexFactory;
	OutMeshBatch.MaterialRenderProxy = MaterialProxy;
	OutMeshBatch.ReverseCulling = bReverseCulling;
	OutMeshBatch.LODIndex = LODIndex;
	OutMeshBatch.Type = PT_TriangleList;
	OutMeshBatch.DepthPriorityGroup = InDepthPriorityGroup;

	FMeshBatchElement& BatchElement = OutMeshBatch.Elements[0];
	BatchElement.IndexBuffer = &MeshResource.IndexBuffer;
	BatchElement.PrimitiveUniformBuffer = UniformBuffer;
	BatchElement.FirstIndex = Section.TriangleStart * 3;
	BatchElement.NumPrimitives = Section.TriangleCount;
	BatchElement.MinVertexIndex = Section.VertexStart;
	BatchElement.MaxVertexIndex = Section.VertexEnd;
}

void FGeometryCollectionSceneProxyBase::GetSectionMaterialProxies(TConstArrayView<FGeometryCollectionMeshElement> InSectionArray, FMaterialArrayType& OutMaterialProxies) const
{
	// Grab the material proxies we'll be using for each section
	for (int32 SectionIndex = 0; SectionIndex < InSectionArray.Num(); ++SectionIndex)
	{
		const FGeometryCollectionMeshElement& Section = InSectionArray[SectionIndex];

		FMaterialRenderProxy* MaterialProxy = Materials.IsValidIndex(Section.MaterialIndex) ? Materials[Section.MaterialIndex]->GetRenderProxy() : nullptr;

		if (MaterialProxy == nullptr)
		{
			MaterialProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
		}

		OutMaterialProxies.Add(MaterialProxy);
	}
}

#if RHI_RAYTRACING
void FGeometryCollectionSceneProxyBase::UpdatingRayTracingGeometry_RenderingThread(FRHICommandListBase& RHICmdList, TConstArrayView<FGeometryCollectionMeshElement> InSectionArray, bool bRayTracingGeometryPerSection)
{
	// TODO: Could use SectionsNoInternal when geometry collection is undamaged?

	if (bRayTracingGeometryPerSection)
	{
		// release combined geometry since will use part geometries
		RayTracingGeometry.ReleaseResource();
	}
	else
	{
		// initialize combined geometry if necessary
		if (!RayTracingGeometry.IsInitialized())
		{
			FRayTracingGeometryInitializer Initializer;
			Initializer.DebugName = FName("GeometryCollection");
			Initializer.GeometryType = RTGT_Triangles;
			Initializer.bFastBuild = true;
			Initializer.bAllowUpdate = false;
			Initializer.TotalPrimitiveCount = 0;
			Initializer.IndexBuffer = MeshResource.IndexBuffer.IndexBufferRHI;

			RayTracingGeometry.SetInitializer(Initializer);

			// InitResource before initializing segments to avoid requesting an unnecessary build
			RayTracingGeometry.InitResource(RHICmdList);

			for (int32 SectionIndex = 0; SectionIndex < InSectionArray.Num(); ++SectionIndex)
			{
				const FGeometryCollectionMeshElement& Section = InSectionArray[SectionIndex];

				FRayTracingGeometrySegment Segment;
				Segment.FirstPrimitive = Section.TriangleStart;
				Segment.VertexBuffer = MeshResource.PositionVertexBuffer.VertexBufferRHI;
				Segment.NumPrimitives = Section.TriangleCount;
				Segment.MaxVertices = Section.VertexEnd + 1;

				Initializer.Segments.Add(Segment);

				Initializer.TotalPrimitiveCount += Section.TriangleCount;
			}

			RayTracingGeometry.SetInitializer(MoveTemp(Initializer));

			// Build will be requested later using dynamic geometry update path
			RayTracingGeometry.CreateRayTracingGeometry(RHICmdList, ERTAccelerationStructureBuildPriority::Skip);
		}
	}
}

void FGeometryCollectionSceneProxyBase::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector, const FMatrix& LocalToWorld, FRHIUniformBuffer* UniformBuffer, bool bAnyMaterialHasWorldPositionOffset)
{
	checkf(bHasRayTracingRepresentation, TEXT("Shouldn't try to get ray tracing instances from proxy that doesn't have a ray tracing representation."));

	if (MeshDescription.Sections.IsEmpty())
	{
		return;
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_GeometryCollectionSceneProxyBase_GetDynamicRayTracingInstances);

	TConstArrayView<const FSceneView*> Views = Collector.GetViews();
	const uint32 VisibilityMap = Collector.GetVisibilityMap();

	// RT geometry will be generated based on first active view and then reused for all other views
	// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
	const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
	checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

	const FSceneView* FirstActiveView = Views[FirstActiveViewIndex];

	if (!GRayTracingGeometryCollectionWPO)
	{
		bAnyMaterialHasWorldPositionOffset = false;
	}

	const uint32 LODIndex = 0;
	const bool bWireframe = false;
	const bool bReverseCulling = false; // DXR doesn't need the flipping implied by the transform

	const bool bUseSubSections = MeshDescription.SubSections.Num() && !bAnyMaterialHasWorldPositionOffset && CVarRayTracingGeometryCollectionCombinedBLAS.GetValueOnRenderThread() == 0;

	TConstArrayView<FGeometryCollectionMeshElement> SectionArray = GetSectionArray(bUseSubSections, false);

	UpdatingRayTracingGeometry_RenderingThread(Collector.GetRHICommandList(), SectionArray, bUseSubSections);

	// Grab the material proxies we'll be using for each section
	// TODO: Add BoneColor support in Path/Ray tracing?
	FMaterialArrayType MaterialProxies;
	GetSectionMaterialProxies(SectionArray, MaterialProxies);

	if (RayTracingGeometry.IsValid())
	{
		check(!bUseSubSections);

		// Render dynamic objects
		if (!VertexFactory.GetType()->SupportsRayTracingDynamicGeometry())
		{
			return;
		}

		TArray<FMeshBatch>& CollectorMeshBatches = Collector.AllocateMeshBatchArray();

		for (int32 SectionIndex = 0; SectionIndex < SectionArray.Num(); ++SectionIndex)
		{
			const FGeometryCollectionMeshElement& Section = SectionArray[SectionIndex];

			FMeshBatch& Mesh = CollectorMeshBatches.AddDefaulted_GetRef();
			GetMeshElement(LODIndex, SectionIndex, Section, &VertexFactory, MaterialProxies[SectionIndex], SDPG_World, UniformBuffer, bWireframe, bReverseCulling, Mesh);

			//#TODO: bone color, bone selection and render bound?
		}

		FRayTracingInstance RayTracingInstance;
		RayTracingInstance.Geometry = &RayTracingGeometry;
		RayTracingInstance.InstanceTransforms.Emplace(LocalToWorld);
		RayTracingInstance.MaterialsView = CollectorMeshBatches;

		const bool bAlwaysUpdate = bAnyMaterialHasWorldPositionOffset || (CVarRayTracingGeometryCollectionForceUpdate.GetValueOnRenderThread() != 0);
		const bool bNeedsUpdate = bAlwaysUpdate
			|| (!bAlwaysUpdate && RayTracingGeometry.DynamicGeometrySharedBufferGenerationID != FRayTracingGeometry::NonSharedVertexBuffers) // was using shared VB but won't use it anymore so update once
			|| RayTracingGeometry.IsEvicted()
			|| RayTracingGeometry.GetRequiresBuild();

		FRWBuffer* VertexBuffer = &RayTracingDynamicVertexBuffer;

		if (bAlwaysUpdate)
		{
			// if updating every frame release memory and use shared VB
			VertexBuffer->Release();
			VertexBuffer = nullptr;
		}

		if (bNeedsUpdate)
		{
			FRayTracingDynamicGeometryUpdateParams UpdateParams;
			UpdateParams.MeshBatchesView = CollectorMeshBatches;
			UpdateParams.bUsingIndirectDraw = false;
			UpdateParams.NumVertices = MeshDescription.NumVertices;
			UpdateParams.VertexBufferSize = MeshDescription.NumVertices * (uint32)sizeof(FVector3f);
			UpdateParams.NumTriangles = RayTracingGeometry.GetInitializer().TotalPrimitiveCount;
			UpdateParams.Geometry = &RayTracingGeometry;
			UpdateParams.Buffer = VertexBuffer;
			UpdateParams.bApplyWorldPositionOffset = true;

			Collector.AddRayTracingGeometryUpdate(FirstActiveViewIndex, MoveTemp(UpdateParams));
		}

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if ((VisibilityMap & (1 << ViewIndex)) == 0)
			{
				continue;
			}

			Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
		}
	}
	else
	{
		check(bUseSubSections);
		checkf(!bIsNanite, TEXT("Nanite geometry collections are expected to use static ray tracing instances."));

		TArray<FMatrix, SceneRenderingAllocator>& PartInstanceTransforms = Collector.AllocateOneFrameResource<TArray<FMatrix, SceneRenderingAllocator>>();
		PartInstanceTransforms.AddUninitialized(DynamicData->Transforms.Num());

		for (int32 TransformIndex = 0; TransformIndex < DynamicData->Transforms.Num(); ++TransformIndex)
		{
			PartInstanceTransforms[TransformIndex] = FMatrix(DynamicData->Transforms[TransformIndex]) * LocalToWorld;
		}

		bool bInstanceMaskAndFlagsDirty = false;

		if (SectionArray.Num() != CachedPartRayTracingMaterials.Num())
		{
			bInstanceMaskAndFlagsDirty = true;

			CachedPartRayTracingMaterials.Reset();
			CachedPartRayTracingMaterials.Reserve(SectionArray.Num());

			for (int32 SectionIndex = 0; SectionIndex < SectionArray.Num(); ++SectionIndex)
			{
				const FGeometryCollectionMeshElement& Section = SectionArray[SectionIndex];

				FMeshBatch& Mesh = CachedPartRayTracingMaterials.AddDefaulted_GetRef();
				GetMeshElement(LODIndex, 0, Section, &VertexFactory, MaterialProxies[SectionIndex], SDPG_World, UniformBuffer, bWireframe, bReverseCulling, Mesh);

				// #TODO: bone color, bone selection and render bound?
			}
		}

		checkf(RayTracingResources.PartGeometries.Num() == SectionArray.Num(), TEXT("Expected to have one part per section (%d vs %d)."), RayTracingResources.PartGeometries.Num(), SectionArray.Num());

		for (int32 SectionIndex = 0; SectionIndex < RayTracingResources.PartGeometries.Num(); ++SectionIndex)
		{
			const FGeometryCollectionMeshElement& Section = SectionArray[SectionIndex];

			FRayTracingInstance RayTracingInstance;
			RayTracingInstance.Geometry = &RayTracingResources.PartGeometries[SectionIndex];
			RayTracingInstance.InstanceTransformsView = MakeConstArrayView(&PartInstanceTransforms[Section.TransformIndex], 1);
			RayTracingInstance.MaterialsView = MakeConstArrayView(&CachedPartRayTracingMaterials[SectionIndex], 1);
			RayTracingInstance.bInstanceMaskAndFlagsDirty = bInstanceMaskAndFlagsDirty;

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if ((VisibilityMap & (1 << ViewIndex)) == 0)
				{
					continue;
				}

				Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
			}
		}
	}
}

void FGeometryCollectionSceneProxyBase::GetStaticRayTracingInstances(FStaticRayTracingInstances& OutRayTracingInstances, FRHIUniformBuffer* UniformBuffer)
{
	checkf(bHasRayTracingRepresentation, TEXT("Shouldn't try to get ray tracing instances from proxy that doesn't have a ray tracing representation."));

	const uint32 LODIndex = 0;
	const bool bWireframe = false;
	const bool bReverseCulling = false; // DXR doesn't need the flipping implied by the transform
	const bool bUseSubSections = true;

	TConstArrayView<FGeometryCollectionMeshElement> SectionArray = GetSectionArray(bUseSubSections, false);

	// Grab the material proxies we'll be using for each section
	// TODO: Add BoneColor support in Path/Ray tracing?
	FMaterialArrayType MaterialProxies;
	GetSectionMaterialProxies(SectionArray, MaterialProxies);

	checkf(RayTracingResources.PartGeometries.Num() == SectionArray.Num(), TEXT("Expected to have one part per section (%d vs %d)."), RayTracingResources.PartGeometries.Num(), SectionArray.Num());

	PrimitiveInstanceIndices.Reset();
	PrimitiveInstanceIndices.SetNumUninitialized(RayTracingResources.PartGeometries.Num());

	// One ray tracing LOD with N instances (one per part)

	FStaticRayTracingInstancesLOD& StaticRayTracingInstancesLOD = OutRayTracingInstances.LODs.AddDefaulted_GetRef();

	for (int32 SectionIndex = 0; SectionIndex < RayTracingResources.PartGeometries.Num(); ++SectionIndex)
	{
		if (!RayTracingResources.PartGeometries[SectionIndex].IsValid() || RayTracingResources.PartGeometries[SectionIndex].IsEvicted())
		{
			continue;
		}

		const FGeometryCollectionMeshElement& Section = SectionArray[SectionIndex];

		const int32 InstanceIndex = TransformToInstanceMapping[Section.TransformIndex];

		if (InstanceIndex < 0)
		{
			continue;
		}

		PrimitiveInstanceIndices[SectionIndex] = InstanceIndex;

		FRayTracingInstance& RayTracingInstance = StaticRayTracingInstancesLOD.Instances.AddDefaulted_GetRef();
		RayTracingInstance.Geometry = &RayTracingResources.PartGeometries[SectionIndex];
		RayTracingInstance.NumTransforms = 1;
		RayTracingInstance.PrimitiveInstanceIndicesView = MakeConstArrayView(&PrimitiveInstanceIndices[SectionIndex], 1); // parts are stored in GPU Scene as instances

		FMeshBatch& Mesh = RayTracingInstance.Materials.AddDefaulted_GetRef();
		GetMeshElement(LODIndex, 0, Section, &VertexFactory, MaterialProxies[SectionIndex], SDPG_World, UniformBuffer, bWireframe, bReverseCulling, Mesh);
	}
}

RayTracing::FGeometryGroupHandle FGeometryCollectionSceneProxyBase::GetRayTracingGeometryGroupHandle() const
{
	check(IsInRenderingThread() || IsInParallelRenderingThread());
	return RayTracingGeometryGroupHandle;
}

#endif

uint32 FGeometryCollectionSceneProxyBase::GetAllocatedSize() const
{
	return Materials.GetAllocatedSize()
		+ MeshDescription.Sections.GetAllocatedSize()
		+ MeshDescription.SubSections.GetAllocatedSize()
		+ (SkinnedPositionVertexBuffer.GetAllowCPUAccess() ? SkinnedPositionVertexBuffer.GetStride() * SkinnedPositionVertexBuffer.GetNumVertices() : 0)
#if RHI_RAYTRACING
		+ RayTracingGeometry.RawData.GetAllocatedSize()
#endif
		;
}

FGeometryCollectionSceneProxy::FGeometryCollectionSceneProxy(UGeometryCollectionComponent* Component)
	: FPrimitiveSceneProxy(Component)
	, FGeometryCollectionSceneProxyBase(Component, false)
#if WITH_EDITOR
	, bShowBoneColors(Component->GetShowBoneColors())
	, bSuppressSelectionMaterial(Component->GetSuppressSelectionMaterial())
	, VertexFactoryDebugColor(GetScene().GetFeatureLevel())
#endif
{
	if (Component->GetRestCollection())
	{
		GeometryCollection = Component->GetRestCollection()->GetGeometryCollection();
	}

	EnableGPUSceneSupportFlags();

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	// Render by SubSection if we are in the rigid body picker.
	bUsesSubSections = Component->GetIsTransformSelectionMode() && MeshDescription.SubSections.Num();
	// Enable bone hit selection proxies if we are in the rigid body picker or in the fracture modes.
	bEnableBoneSelection = Component->GetEnableBoneSelection();

	if (bEnableBoneSelection || bUsesSubSections)
	{
		for (int32 TransformIndex = 0; TransformIndex < NumTransforms; ++TransformIndex)
		{
			HGeometryCollection* HitProxy = new HGeometryCollection(Component, TransformIndex);
			HitProxies.Add(HitProxy);
		}
	}
#endif

#if WITH_EDITOR
	if (bShowBoneColors || bEnableBoneSelection)
	{
		Component->GetBoneColors(BoneColors);
		ColorVertexBuffer.InitFromColorArray(BoneColors);

		if (Component->GetRestCollection())
		{
			BoneSelectedMaterial = Component->GetRestCollection()->GetBoneSelectedMaterial();
		}
		if (BoneSelectedMaterial && !BoneSelectedMaterial->CheckMaterialUsage_Concurrent(MATUSAGE_GeometryCollections))
		{
			// If we have an invalid BoneSelectedMaterial, switch it back to null to skip its usage in GetDynamicMeshElements below
			BoneSelectedMaterial = nullptr;
		}

		// Make sure the vertex color material has the usage flag for rendering geometry collections
		if (GEngine->VertexColorMaterial)
		{
			GEngine->VertexColorMaterial->CheckMaterialUsage_Concurrent(MATUSAGE_GeometryCollections);
		}
	}

#endif

	// #todo(dmp): This flag means that when motion blur is turned on, it will always render geometry collections into the
	// velocity buffer.  Note that the way around this is to loop through the global matrices and test whether they have
	// changed from the prev to curr frame, but this is expensive.  We should revisit this if the draw calls for velocity
	// rendering become a problem. One solution could be to use internal solver sleeping state to drive motion blur.
	bAlwaysHasVelocity = true;

	SetWireframeColor(Component->GetWireframeColorForSceneProxy());
	CollisionResponse = Component->GetCollisionResponseToChannels();
}

FGeometryCollectionSceneProxy::~FGeometryCollectionSceneProxy()
{
}

SIZE_T FGeometryCollectionSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FGeometryCollectionSceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	FGeometryCollectionSceneProxyBase::CreateRenderThreadResources(RHICmdList);

#if WITH_EDITOR
	if (bShowBoneColors || bEnableBoneSelection)
	{
		// Initialize debug color buffer and associated vertex factory.
		ColorVertexBuffer.InitResource(RHICmdList);
		SetupVertexFactory(RHICmdList, VertexFactoryDebugColor, &ColorVertexBuffer);
	}
#endif

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	if (MeshDescription.NumVertices && HitProxies.Num())
	{
		// Create buffer containing per vertex hit proxy IDs.
		HitProxyIdBuffer.Init(MeshDescription.NumVertices);
		HitProxyIdBuffer.InitResource(RHICmdList);

		uint16 const* BoneMapData = &MeshResource.BoneMapVertexBuffer.BoneIndex(0);
		ParallelFor(MeshDescription.NumVertices, [&](int32 i)
		{
			// Note that some fracture undo/redo operations can: recreate scene proxy, then update render data, then recreate proxy again.
			// In that case we can come here the first time with too few hit proxy objects for the bone map which hasn't updated.
			// But we then enter here a second time with the render data correct.
			int16 ProxyIndex = BoneMapData[i];
			ProxyIndex = HitProxies.IsValidIndex(ProxyIndex) ? ProxyIndex : 0;
			HitProxyIdBuffer.VertexColor(i) = HitProxies[ProxyIndex]->Id.GetColor();
		});

		void* VertexBufferData = RHICmdList.LockBuffer(HitProxyIdBuffer.VertexBufferRHI, 0, HitProxyIdBuffer.GetNumVertices() * HitProxyIdBuffer.GetStride(), RLM_WriteOnly);
		FMemory::Memcpy(VertexBufferData, HitProxyIdBuffer.GetVertexData(), HitProxyIdBuffer.GetNumVertices() * HitProxyIdBuffer.GetStride());
		RHICmdList.UnlockBuffer(HitProxyIdBuffer.VertexBufferRHI);
	}
#endif

	bRenderResourcesCreated = true;
	SetDynamicData_RenderThread(RHICmdList, DynamicData);
}

void FGeometryCollectionSceneProxy::DestroyRenderThreadResources()
{
	FGeometryCollectionSceneProxyBase::DestroyRenderThreadResources();

#if WITH_EDITOR
	VertexFactoryDebugColor.ReleaseResource();
	ColorVertexBuffer.ReleaseResource();
#endif

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	HitProxyIdBuffer.ReleaseResource();
#endif
}

void FGeometryCollectionSceneProxy::SetDynamicData_RenderThread(FRHICommandListBase& RHICmdList, FGeometryCollectionDynamicData* NewDynamicData)
{
	FGeometryCollectionSceneProxyBase::SetDynamicData_RenderThread(RHICmdList, NewDynamicData);

	if (MeshDescription.NumVertices == 0 || !DynamicData || !bRenderResourcesCreated)
	{
		return;
	}
		
	if (bUseShaderBoneTransform)
	{
		const FGeometryCollectionTransformBuffer& TransformBuffer = GetCurrentTransformBuffer();
		const FGeometryCollectionTransformBuffer& PrevTransformBuffer = GetCurrentPrevTransformBuffer();

#if WITH_EDITOR
		if (bShowBoneColors || bEnableBoneSelection)
		{
			VertexFactoryDebugColor.SetBoneTransformSRV(TransformBuffer.VertexBufferSRV);
			VertexFactoryDebugColor.SetBonePrevTransformSRV(PrevTransformBuffer.VertexBufferSRV);
			UpdateLooseParameter(RHICmdList, VertexFactoryDebugColor, TransformBuffer.VertexBufferSRV, PrevTransformBuffer.VertexBufferSRV, MeshResource.BoneMapVertexBuffer.GetSRV());
		}
#endif
	}
}

FMaterialRenderProxy* FGeometryCollectionSceneProxy::GetMaterial(FMeshElementCollector& Collector, int32 MaterialIndex) const
{
	FMaterialRenderProxy* MaterialProxy = nullptr;

#if WITH_EDITOR
	if (bShowBoneColors && GEngine->VertexColorMaterial)
	{
		// Material for colored bones
		UMaterial* VertexColorVisualizationMaterial = GEngine->VertexColorMaterial;
		FMaterialRenderProxy* VertexColorVisualizationMaterialInstance = new FColoredMaterialRenderProxy(
			VertexColorVisualizationMaterial->GetRenderProxy(),
			GetSelectionColor(FLinearColor::White, false, false)
		);
		Collector.RegisterOneFrameMaterialProxy(VertexColorVisualizationMaterialInstance);
		MaterialProxy = VertexColorVisualizationMaterialInstance;
	}
	else 
#endif
	if(Materials.IsValidIndex(MaterialIndex))
	{
		MaterialProxy = Materials[MaterialIndex]->GetRenderProxy();
	}

	if (MaterialProxy == nullptr)
	{
		MaterialProxy = UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
	}

	return MaterialProxy;
}

FVertexFactory const* FGeometryCollectionSceneProxy::GetVertexFactory() const
{
#if WITH_EDITOR
	return bShowBoneColors ? &VertexFactoryDebugColor : &VertexFactory;
#else
	return &VertexFactory;
#endif
}

bool FGeometryCollectionSceneProxy::ShowCollisionMeshes(const FEngineShowFlags& EngineShowFlags) const
{
	if (IsCollisionEnabled())
	{
		if (EngineShowFlags.CollisionPawn && CollisionResponse.GetResponse(ECC_Pawn) != ECR_Ignore)
		{
			return true;
		}
		if (EngineShowFlags.CollisionVisibility && CollisionResponse.GetResponse(ECC_Visibility) != ECR_Ignore)
		{
			return true;
		}
		if (EngineShowFlags.Collision)
		{
			return true;
		}
	}
	return false;
}

void FGeometryCollectionSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_GeometryCollectionSceneProxy_GetDynamicMeshElements);
	if (MeshDescription.NumVertices == 0)
	{
		return;
	}
		
	const FEngineShowFlags& EngineShowFlags = ViewFamily.EngineShowFlags;
	const bool bWireframe = AllowDebugViewmodes() && EngineShowFlags.Wireframe;
	const bool bProxyIsSelected = IsSelected();
	const bool bDrawOnlyCollisionMeshes = EngineShowFlags.CollisionPawn || EngineShowFlags.CollisionVisibility;
	const bool bDrawWireframeCollision = EngineShowFlags.Collision && IsCollisionEnabled();

	const uint32 LODIndex = 0;
	const bool bReverseCulling = IsLocalToWorldDeterminantNegative();

	auto SetDebugMaterial = [this, &Collector, &EngineShowFlags, bProxyIsSelected](FMeshBatch& Mesh) -> void
	{
#if UE_ENABLE_DEBUG_DRAWING

		// flag to indicate whether we've set a debug material yet
		// Note: Will be used if we add more debug material options
		// (compare to variable of same name in StaticMeshSceneProxy.cpp)
		bool bDebugMaterialRenderProxySet = false;

		if (!bDebugMaterialRenderProxySet && bProxyIsSelected && EngineShowFlags.VertexColors && AllowDebugViewmodes())
		{
			// Note: static mesh renderer does something more complicated involving per-section selection, but whole component selection seems ok for now.
			if (FMaterialRenderProxy* VertexColorVisualizationMaterialInstance = MeshPaintVisualize::GetMaterialRenderProxy(bProxyIsSelected, IsHovered()))
			{
				Collector.RegisterOneFrameMaterialProxy(VertexColorVisualizationMaterialInstance);
				Mesh.MaterialRenderProxy = VertexColorVisualizationMaterialInstance;
				bDebugMaterialRenderProxySet = true;
			}
		}
#endif
	};

	const bool bDrawGeometryCollectionMesh = !bDrawOnlyCollisionMeshes;

	if (bDrawGeometryCollectionMesh)
	{
		// If hiding geometry in editor then we don't remove hidden faces.
		const bool bRemoveInternalFaces = false;

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
		// If using subsections then use the subsection array. 
		TConstArrayView<FGeometryCollectionMeshElement> SectionArray = GetSectionArray(bUsesSubSections, bRemoveInternalFaces);
#else
		TConstArrayView<FGeometryCollectionMeshElement> SectionArray = GetSectionArray(false, bRemoveInternalFaces);
#endif

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if ((VisibilityMap & (1 << ViewIndex)) == 0)
			{
				continue;
			}

			// Grab the material proxies we'll be using for each section.
			FMaterialArrayType MaterialProxies;
			for (int32 SectionIndex = 0; SectionIndex < SectionArray.Num(); ++SectionIndex)
			{
				const FGeometryCollectionMeshElement& Section = SectionArray[SectionIndex];
				FMaterialRenderProxy* MaterialProxy = GetMaterial(Collector, Section.MaterialIndex);
				MaterialProxies.Add(MaterialProxy);
			}

			// Draw the meshes.
			for (int32 SectionIndex = 0; SectionIndex < SectionArray.Num(); ++SectionIndex)
			{
				const FGeometryCollectionMeshElement& Section = SectionArray[SectionIndex];

				FMeshBatch& Mesh = Collector.AllocateMesh();
				GetMeshElement(LODIndex, SectionIndex, Section, GetVertexFactory(), MaterialProxies[SectionIndex], SDPG_World, GetUniformBuffer(), bWireframe, bReverseCulling, Mesh);
				Mesh.bCanApplyViewModeOverrides = true;
				SetDebugMaterial(Mesh);

				Collector.AddMesh(ViewIndex, Mesh);
			}

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
			// Highlight selected bone using specialized material.
			// #note: This renders the geometry again but with the bone selection material.  Ideally we'd have one render pass and one material.
			if (bEnableBoneSelection && !bSuppressSelectionMaterial && BoneSelectedMaterial)
			{
				FMaterialRenderProxy* MaterialRenderProxy = BoneSelectedMaterial->GetRenderProxy();

				FMeshBatch& Mesh = Collector.AllocateMesh();
				Mesh.bWireframe = bWireframe;
				Mesh.VertexFactory = &VertexFactoryDebugColor;
				Mesh.MaterialRenderProxy = MaterialRenderProxy;
				Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
				Mesh.Type = PT_TriangleList;
				Mesh.DepthPriorityGroup = SDPG_World;
				Mesh.bCanApplyViewModeOverrides = false;

				FMeshBatchElement& BatchElement = Mesh.Elements[0];
				BatchElement.IndexBuffer = &MeshResource.IndexBuffer;
				BatchElement.PrimitiveUniformBuffer = GetUniformBuffer();
				BatchElement.FirstIndex = 0;
				BatchElement.NumPrimitives = MeshDescription.NumTriangles;
				BatchElement.MinVertexIndex = 0;
				BatchElement.MaxVertexIndex = MeshDescription.NumVertices;

				Collector.AddMesh(ViewIndex, Mesh);
			}
#endif // GEOMETRYCOLLECTION_EDITOR_SELECTION
		}
	}

	// draw extra stuff ( collision , bounds ... )
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			// collision modes
			if (ShowCollisionMeshes(EngineShowFlags) && GeometryCollection && AllowDebugViewmodes())
			{
				FTransform GeomTransform(GetLocalToWorld());
				if (bDrawWireframeCollision)
				{
					GeometryCollectionDebugDraw::DrawWireframe(*GeometryCollection, GeomTransform, Collector, ViewIndex, GetWireframeColor().ToFColor(true));
				}
				else
				{
					FMaterialRenderProxy* CollisionMaterialInstance = new FColoredMaterialRenderProxy(GEngine->ShadedLevelColorationUnlitMaterial->GetRenderProxy(), GetWireframeColor());
					Collector.RegisterOneFrameMaterialProxy(CollisionMaterialInstance);
					GeometryCollectionDebugDraw::DrawSolid(*GeometryCollection, GeomTransform, Collector, ViewIndex, CollisionMaterialInstance);
				}
			}

			// render bounds
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			RenderBounds(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
#endif
		}
	}
}

FPrimitiveViewRelevance FGeometryCollectionSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	const FEngineShowFlags& EngineShowFlags = View->Family->EngineShowFlags;
	const bool bWireframe = AllowDebugViewmodes() && EngineShowFlags.Wireframe;
	const bool bProxyIsSelected = IsSelected();
	const bool bDrawOnlyCollisionMeshes = EngineShowFlags.CollisionPawn || EngineShowFlags.CollisionVisibility;
	const bool bDrawWireframeCollision = EngineShowFlags.Collision && IsCollisionEnabled();

	const bool bDynamic = !GGeometryCollectionUseCachedMDCs
		|| HasViewDependentDPG()
#if WITH_EDITOR
		|| bShowBoneColors
#endif
#if GEOMETRYCOLLECTION_EDITOR_SELECTION
		|| bEnableBoneSelection
#endif
		|| bWireframe
		|| bProxyIsSelected
		|| bDrawOnlyCollisionMeshes
		|| bDrawWireframeCollision
		|| View->Family->EngineShowFlags.Bounds;

	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View);
	Result.bShadowRelevance = IsShadowCast(View);
	if (bDynamic)
	{
		Result.bDynamicRelevance = true;
	}
	else
	{
		Result.bStaticRelevance = true;
	}
	Result.bRenderInMainPass = ShouldRenderInMainPass();
	Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
	Result.bRenderCustomDepth = ShouldRenderCustomDepth();
	Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
	MaterialRelevance.SetPrimitiveViewRelevance(Result);

	Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;

	return Result;
}

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
HHitProxy* FGeometryCollectionSceneProxy::CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies)
{
	HHitProxy* DefaultHitProxy = FPrimitiveSceneProxy::CreateHitProxies(Component, OutHitProxies);
	OutHitProxies.Append(HitProxies);
	return DefaultHitProxy;
}
#endif

void FGeometryCollectionSceneProxy::DrawStaticElements(FStaticPrimitiveDrawInterface* PDI)
{
	checkSlow(IsInParallelRenderingThread());

	if (HasViewDependentDPG())
	{
		return;
	}

	// Determine the DPG the primitive should be drawn in.
	ESceneDepthPriorityGroup PrimitiveDPG = GetStaticDepthPriorityGroup();

	const uint32 LODIndex = 0;
	const bool bReverseCulling = IsLocalToWorldDeterminantNegative();
	const bool bWireframe = false;

	// If hiding geometry in editor then we don't remove hidden faces.
	const bool bRemoveInternalFaces = false;

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	// If using subsections then use the subsection array. 
	TConstArrayView<FGeometryCollectionMeshElement> SectionArray = GetSectionArray(bUsesSubSections, bRemoveInternalFaces);
#else
	TConstArrayView<FGeometryCollectionMeshElement> SectionArray = GetSectionArray(false, bRemoveInternalFaces);
#endif

	// Grab the material proxies we'll be using for each section.
	FMaterialArrayType MaterialProxies;
	GetSectionMaterialProxies(SectionArray, MaterialProxies);

	// Draw the meshes.
	for (int32 SectionIndex = 0; SectionIndex < SectionArray.Num(); ++SectionIndex)
	{
		const FGeometryCollectionMeshElement& Section = SectionArray[SectionIndex];

		FMeshBatch Mesh;
		GetMeshElement(LODIndex, SectionIndex, Section, GetVertexFactory(), MaterialProxies[SectionIndex], PrimitiveDPG, GetUniformBuffer(), bWireframe, bReverseCulling, Mesh);

		PDI->DrawMesh(Mesh, 1.0f);
	}
}

void FGeometryCollectionSceneProxy::GetPreSkinnedLocalBounds(FBoxSphereBounds& OutBounds) const
{
	OutBounds = PreSkinnedBounds; 
}

uint32 FGeometryCollectionSceneProxy::GetAllocatedSize() const
{
	return FPrimitiveSceneProxy::GetAllocatedSize()
		+ FGeometryCollectionSceneProxyBase::GetAllocatedSize()
#if WITH_EDITOR
		+ BoneColors.GetAllocatedSize()
		+ (ColorVertexBuffer.GetAllowCPUAccess() ? ColorVertexBuffer.GetStride() * ColorVertexBuffer.GetNumVertices() : 0)
#endif
#if GEOMETRYCOLLECTION_EDITOR_SELECTION
		+ HitProxies.GetAllocatedSize()
		+ (HitProxyIdBuffer.GetAllowCPUAccess() ? HitProxyIdBuffer.GetStride() * HitProxyIdBuffer.GetNumVertices() : 0)
#endif
		;
}


FNaniteGeometryCollectionSceneProxy::FNaniteGeometryCollectionSceneProxy(UGeometryCollectionComponent* Component)
: Nanite::FSceneProxyBase(Component)
, FGeometryCollectionSceneProxyBase(Component, true)
, GeometryCollection(Component->GetRestCollection())
, bRequiresGPUSceneUpdate(false)
, bEnableBoneSelection(false)
{
	LLM_SCOPE_BYTAG(Nanite);

	// Nanite requires GPUScene
	checkSlow(UseGPUScene(GetScene().GetShaderPlatform()));
	checkSlow(DoesPlatformSupportNanite(GMaxRHIShaderPlatform));
	checkSlow(GeometryCollection->HasNaniteData());

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	bEnableBoneSelection = Component->GetEnableBoneSelection();
#endif

	FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
	FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffersImpl.BeginWriteAccess(AccessTag);
	ProxyData.Flags.bHasPerInstanceHierarchyOffset = true;
	ProxyData.Flags.bHasPerInstanceLocalBounds = true;
	ProxyData.Flags.bHasPerInstanceDynamicData = true;
	ProxyData.Flags.bHasPerInstanceEditorData = bEnableBoneSelection;
	InstanceSceneDataBuffersImpl.EndWriteAccess(AccessTag);

	// Note: ideally this would be picked up from the Flags.bHasPerInstanceDynamicData above, but that path is not great at the moment.
	bAlwaysHasVelocity = true;

	// Nanite supports the GPUScene instance data buffer.
	SetupInstanceSceneDataBuffers(&InstanceSceneDataBuffersImpl);

	bSupportsDistanceFieldRepresentation = false;

	// Dynamic draw path without Nanite isn't supported by Lumen
	bVisibleInLumenScene = false;

	// Use fast path that does not update static draw lists.
	bStaticElementsAlwaysUseProxyPrimitiveUniformBuffer = true;

	// Nanite always uses GPUScene, so we can skip expensive primitive uniform buffer updates.
	bVFRequiresPrimitiveUniformBuffer = false;

	const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> Collection = GeometryCollection->GetGeometryCollection();
	const TManagedArray<int32>& TransformToGeometryIndices = Collection->TransformToGeometryIndex;
	const TManagedArray<int32>& SimulationType = Collection->SimulationType;
	const TManagedArray<FGeometryCollectionSection>& SectionsArray = Collection->Sections;

	MaterialSections.SetNum(SectionsArray.Num());

	for (int32 SectionIndex = 0; SectionIndex < SectionsArray.Num(); ++SectionIndex)
	{
		const FGeometryCollectionSection& MeshSection = SectionsArray[SectionIndex];
		const bool bValidMeshSection = MeshSection.MaterialID != INDEX_NONE;

		// Keep track of highest observed material index.
		MaterialMaxIndex = FMath::Max(MeshSection.MaterialID, MaterialMaxIndex);

		UMaterialInterface* MaterialInterface = bValidMeshSection ? Component->GetMaterial(MeshSection.MaterialID) : nullptr;

		// TODO: PROG_RASTER (Implement programmable raster support)
		const bool bInvalidMaterial = !MaterialInterface || !IsOpaqueOrMaskedBlendMode(*MaterialInterface) || MaterialInterface->GetShadingModels().HasShadingModel(MSM_SingleLayerWater);
		if (bInvalidMaterial)
		{
			if (MaterialInterface)
			{
				UE_LOGF(
					LogStaticMesh, Warning,
					"Invalid material [%ls] used on Nanite geometry collection [%ls] - forcing default material instead. Only opaque blend mode and a shading model that is not SingleLayerWater is currently supported, [%ls] blend mode and [%ls] shading model was specified.",
					*MaterialInterface->GetName(),
					*GeometryCollection->GetName(),
					*GetBlendModeString(MaterialInterface->GetBlendMode()),
					*GetShadingModelFieldString(MaterialInterface->GetShadingModels())
				);
			}
		}

		if (bInvalidMaterial)
		{
			// force default material 
			MaterialInterface = UMaterial::GetDefaultMaterial(MD_Surface);
		}		
		else if (Component->GetPSOPrecacheComponentData().UsePSOPrecacheFallbackMaterial())
		{
			MaterialInterface = UPSOPrecacheSettingsManager::GetFallbackMaterial() ? UPSOPrecacheSettingsManager::GetFallbackMaterial() : UMaterial::GetDefaultMaterial(MD_Surface);
		}

		// Should never be null here
		check(MaterialInterface != nullptr);

		// Should always be opaque blend mode here.
		check(IsOpaqueOrMaskedBlendMode(*MaterialInterface));

		MaterialSections[SectionIndex].ShadingMaterialProxy = MaterialInterface->GetRenderProxy();
		MaterialSections[SectionIndex].RasterMaterialProxy  = MaterialInterface->GetRenderProxy(); // TODO: PROG_RASTER (Implement programmable raster support)
		MaterialSections[SectionIndex].MaterialIndex = MeshSection.MaterialID;
		MaterialSections[SectionIndex].bCastShadow = true;
	}

	OnMaterialsUpdated();

	const bool bHasGeometryBoundingBoxes = 
		Collection->HasAttribute("BoundingBox", FGeometryCollection::GeometryGroup) &&
		Collection->NumElements(FGeometryCollection::GeometryGroup);
	
	const bool bHasTransformBoundingBoxes = 
		Collection->NumElements(FGeometryCollection::TransformGroup) && 
		Collection->HasAttribute("BoundingBox", FGeometryCollection::TransformGroup) &&
		Collection->HasAttribute("TransformToGeometryIndex", FGeometryCollection::TransformGroup);

	int32 NumGeometry = 0;
	if (bHasGeometryBoundingBoxes)
	{
		NumGeometry = Collection->NumElements(FGeometryCollection::GeometryGroup);
		GeometryNaniteData.SetNumUninitialized(NumGeometry);

		const TManagedArray<FBox>& BoundingBoxes = Collection->GetAttribute<FBox>("BoundingBox", FGeometryCollection::GeometryGroup);
		for (int32 GeometryIndex = 0; GeometryIndex < NumGeometry; ++GeometryIndex)
		{
			FGeometryNaniteData& Instance = GeometryNaniteData[GeometryIndex];
			Instance.HierarchyOffset = GeometryCollection->GetNaniteHierarchyOffset(GeometryIndex);
			Instance.LocalBounds = BoundingBoxes[GeometryIndex];
		}
	}
	else if (bHasTransformBoundingBoxes)
	{
		NumGeometry = GeometryCollection->RenderData->NaniteResourcesPtr->HierarchyRootOffsets.Num();
		GeometryNaniteData.SetNumUninitialized(NumGeometry);
		
		const TManagedArray<FBox>& BoundingBoxes = Collection->GetAttribute<FBox>("BoundingBox", FGeometryCollection::TransformGroup);
		const TManagedArray<int32>& TransformToGeometry = Collection->GetAttribute<int32>("TransformToGeometryIndex", FGeometryCollection::TransformGroup);
		const int32 NumTransformToGeometry = TransformToGeometry.Num();
		for (int32 TransformToGeometryIndex = 0; TransformToGeometryIndex < NumTransformToGeometry; ++TransformToGeometryIndex)
		{
			const int32 GeometryIndex = TransformToGeometry[TransformToGeometryIndex];
			if (GeometryIndex > INDEX_NONE)
			{
				FGeometryNaniteData& Instance = GeometryNaniteData[GeometryIndex];
				Instance.HierarchyOffset = GeometryCollection->GetNaniteHierarchyOffset(GeometryIndex);
				Instance.LocalBounds = BoundingBoxes[TransformToGeometryIndex];
			}
		}
	}

	SetWireframeColor(Component->GetWireframeColorForSceneProxy());

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	if (bEnableBoneSelection)
	{
		// Generate a hit proxy per geometry section so that we can perform per bone hit tests.
		HitProxyMode = EHitProxyMode::PerInstance;
		HitProxies.Reserve(NumGeometry);
		for (int32 GeometryIndex = 0; GeometryIndex < NumGeometry; ++GeometryIndex)
		{
			HGeometryCollection* HitProxy = new HGeometryCollection(Component, GeometryIndex);
			HitProxies.Add(HitProxy);
		}
	}
	else if (AActor* Actor = Component->GetOwner())
	{
		// Generate default material hit proxies for simple selection.
		HitProxyMode = Nanite::FSceneProxyBase::EHitProxyMode::MaterialSection;
		for (int32 SectionIndex = 0; SectionIndex < MaterialSections.Num(); ++SectionIndex)
		{
			FMaterialSection& Section = MaterialSections[SectionIndex];
			HHitProxy* HitProxy = new HActor(Actor, Component, SectionIndex, SectionIndex);
			Section.HitProxy = HitProxy;
			HitProxies.Add(HitProxy);
		}
	}
#endif

	// Initialize to rest transforms.
	TArray<FMatrix44f> RestTransforms;
	Component->GetRestTransforms(RestTransforms);

	CollisionResponse = Component->GetCollisionResponseToChannels();

	UpdateInstanceSceneDataBuffers(Component->GetRenderMatrix());

#if RHI_RAYTRACING
	if (HasRayTracingRepresentation())
	{
		bUseStaticRayTracingInstances = !MeshDescription.SubSections.IsEmpty() && (!bAnyMaterialHasWorldPositionOffset || !GRayTracingGeometryCollectionWPO) && CVarRayTracingGeometryCollectionCombinedBLAS.GetValueOnGameThread() == 0;
	}
#endif
}

FNaniteGeometryCollectionSceneProxy::~FNaniteGeometryCollectionSceneProxy()
{

}

void FNaniteGeometryCollectionSceneProxy::CreateRenderThreadResources(FRHICommandListBase& RHICmdList)
{
	FGeometryCollectionSceneProxyBase::CreateRenderThreadResources(RHICmdList);

	// Should have valid Nanite data at this point.
	Nanite::FResources& NaniteResources = *GeometryCollection->RenderData->NaniteResourcesPtr.Get();
	check(NaniteResources.RuntimeResourceID != INDEX_NONE && NaniteResources.HierarchyOffset != INDEX_NONE);

	FGeometryCollectionSceneProxyBase::SetDynamicData_RenderThread(RHICmdList, DynamicData);
}

void FNaniteGeometryCollectionSceneProxy::DestroyRenderThreadResources()
{
	FGeometryCollectionSceneProxyBase::DestroyRenderThreadResources();
}

SIZE_T FNaniteGeometryCollectionSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

void FNaniteGeometryCollectionSceneProxy::GetPreSkinnedLocalBounds(FBoxSphereBounds& OutBounds) const
{
	OutBounds = PreSkinnedBounds;
}

FPrimitiveViewRelevance FNaniteGeometryCollectionSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	LLM_SCOPE_BYTAG(Nanite);

	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View) && View->Family->EngineShowFlags.NaniteMeshes;
	Result.bShadowRelevance = IsShadowCast(View);
	Result.bRenderCustomDepth = Nanite::GetSupportsCustomDepthRendering() && ShouldRenderCustomDepth();
	Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();

	// Always render the Nanite mesh data with static relevance.
	Result.bStaticRelevance = true;

	// dynamic still relevance still must be used when drawing collisions
	Result.bDynamicRelevance = ShowCollisionMeshes(View->Family->EngineShowFlags);

	// Should always be covered by constructor of Nanite scene proxy.
	Result.bRenderInMainPass = true;

#if WITH_EDITOR
	// Only check these in the editor
	Result.bEditorVisualizeLevelInstanceRelevance = IsEditingLevelInstanceChild();
	Result.bEditorStaticSelectionRelevance = (IsSelected() || IsHovered());
#endif

	Result.bOpaque = true;

	MaterialRelevance.SetPrimitiveViewRelevance(Result);
	Result.bVelocityRelevance = Result.bOpaque && Result.bRenderInMainPass && DrawsVelocity();

	return Result;
}

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
HHitProxy* FNaniteGeometryCollectionSceneProxy::CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy>>& OutHitProxies)
{
	LLM_SCOPE_BYTAG(Nanite);
	OutHitProxies.Append(HitProxies);
	return Super::CreateHitProxies(Component, OutHitProxies);
}
#endif

void FNaniteGeometryCollectionSceneProxy::DrawStaticElements(FStaticPrimitiveDrawInterface* PDI)
{
	const FLightCacheInterface* LCI = nullptr;
	DrawStaticElementsInternal(PDI, LCI);
}

uint32 FNaniteGeometryCollectionSceneProxy::GetAllocatedSize() const
{
	return FPrimitiveSceneProxy::GetAllocatedSize()
		+ FGeometryCollectionSceneProxyBase::GetAllocatedSize();
}

Nanite::FResourceMeshInfo FNaniteGeometryCollectionSceneProxy::GetResourceMeshInfo() const
{
	Nanite::FResources& NaniteResources = *GeometryCollection->RenderData->NaniteResourcesPtr.Get();

	Nanite::FResourceMeshInfo OutInfo;

	OutInfo.NumClusters = NaniteResources.NumClusters;
	OutInfo.NumNodes = NaniteResources.HierarchyNodes.Num();
	OutInfo.NumVertices = NaniteResources.NumInputVertices;
	OutInfo.NumTriangles = NaniteResources.NumInputTriangles;
	OutInfo.NumCurves = NaniteResources.NumInputCurves;
	OutInfo.NumMaterials = MaterialMaxIndex + 1;
	OutInfo.DebugName = GeometryCollection->GetFName();

	OutInfo.NumResidentClusters = NaniteResources.NumResidentClusters;

	// TODO: SegmentMapping
	OutInfo.NumSegments = 0;

	return MoveTemp(OutInfo);
}

Nanite::FResourcePrimitiveInfo FNaniteGeometryCollectionSceneProxy::GetResourcePrimitiveInfo() const
{
	Nanite::FResources& NaniteResources = *GeometryCollection->RenderData->NaniteResourcesPtr.Get();

	Nanite::FResourcePrimitiveInfo OutInfo;

	OutInfo.ResourceID = NaniteResources.RuntimeResourceID;
	OutInfo.HierarchyOffset = NaniteResources.HierarchyOffset;
	
	// TODO: Nanite-Imposters?
	// TODO: Nanite-Assemblies?

	return MoveTemp(OutInfo);
}

void FNaniteGeometryCollectionSceneProxy::UpdateInstanceSceneDataBuffers(const FMatrix& PrimitiveLocalToWorld)
{
	FInstanceSceneDataBuffers::FAccessTag AccessTag(PointerHash(this));
	FInstanceSceneDataBuffers::FWriteView ProxyData = InstanceSceneDataBuffersImpl.BeginWriteAccess(AccessTag);
	InstanceSceneDataBuffersImpl.SetPrimitiveLocalToWorld(PrimitiveLocalToWorld, AccessTag);

	const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> Collection = GeometryCollection->GetGeometryCollection();
	const TManagedArray<int32>& TransformToGeometryIndices = Collection->TransformToGeometryIndex;
	const TManagedArray<TSet<int32>>& TransformChildren = Collection->Children;
	const TManagedArray<int32>& SimulationType = Collection->SimulationType;

	const int32 TransformCount = DynamicData->Transforms.Num();
	check(TransformCount == TransformToGeometryIndices.Num());
	check(TransformCount == TransformChildren.Num());

	TransformToInstanceMapping.Reset(TransformCount);

	// set the prev by copying the last current
	ProxyData.PrevInstanceToPrimitiveRelative = ProxyData.InstanceToPrimitiveRelative;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bCanSkipRedundantTransformUpdates = false; // should we compare the transform to better decide about this ? 
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	ProxyData.InstanceToPrimitiveRelative.Reset(TransformCount);
	ProxyData.InstanceLocalBounds.Reset(TransformCount);
	ProxyData.InstanceHierarchyOffset.Reset(TransformCount);

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	ProxyData.InstanceEditorData.Reset(bEnableBoneSelection ? TransformCount : 0);
#endif

	ProxyData.Flags.bHasPerInstanceDynamicData = true;
	ProxyData.Flags.bHasPerInstanceLocalBounds = true;
	ProxyData.Flags.bHasPerInstanceHierarchyOffset = true;

	for (int32 TransformIndex = 0; TransformIndex < TransformCount; ++TransformIndex)
	{
		const int32 TransformToGeometryIndex = TransformToGeometryIndices[TransformIndex];
		if (SimulationType[TransformIndex] != FGeometryCollection::ESimulationTypes::FST_Rigid)
		{
			TransformToInstanceMapping.Add(-1);
			continue;
		}

		TransformToInstanceMapping.Add(ProxyData.InstanceToPrimitiveRelative.Num());

		const FGeometryNaniteData& NaniteData = GeometryNaniteData[TransformToGeometryIndex];
		const FRenderTransform& InstanceToPrimitiveRelative = ProxyData.InstanceToPrimitiveRelative.Emplace_GetRef(InstanceSceneDataBuffersImpl.ComputeInstanceToPrimitiveRelative(DynamicData->Transforms[TransformIndex], AccessTag));

		ProxyData.InstanceLocalBounds.Emplace(PadInstanceLocalBounds(NaniteData.LocalBounds));
		ProxyData.InstanceHierarchyOffset.Emplace(NaniteData.HierarchyOffset);

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
		if (bEnableBoneSelection)
		{
			ProxyData.InstanceEditorData.Emplace(FInstanceEditorData::Pack(HitProxies[TransformToGeometryIndex]->Id.GetColor(), false));
		}
#endif
	}

	// make sure the previous transform count do match the current one
	// if not simply use the current as previous
	if (ProxyData.PrevInstanceToPrimitiveRelative.Num() != ProxyData.InstanceToPrimitiveRelative.Num())
	{
		ProxyData.PrevInstanceToPrimitiveRelative = ProxyData.InstanceToPrimitiveRelative;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bCanSkipRedundantTransformUpdates = true;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	InstanceSceneDataBuffersImpl.EndWriteAccess(AccessTag);
}

void FNaniteGeometryCollectionSceneProxy::SetDynamicData_RenderThread(FRHICommandListBase& RHICmdList, FGeometryCollectionDynamicData* NewDynamicData, const FMatrix &PrimitiveLocalToWorld)
{
	FGeometryCollectionSceneProxyBase::SetDynamicData_RenderThread(RHICmdList, NewDynamicData);

	UpdateInstanceSceneDataBuffers(PrimitiveLocalToWorld);
}

void FNaniteGeometryCollectionSceneProxy::FlushGPUSceneUpdate_GameThread()
{
	ENQUEUE_RENDER_COMMAND(NaniteProxyUpdateGPUScene)(
		[this](FRHICommandListImmediate& RHICmdList)
		{
			FPrimitiveSceneInfo* NanitePrimitiveInfo = GetPrimitiveSceneInfo();
			if (NanitePrimitiveInfo && GetRequiresGPUSceneUpdate_RenderThread())
			{
				// Attempt to queue up a GPUScene update - maintain dirty flag if the request fails.
				const bool bRequiresUpdate = !NanitePrimitiveInfo->RequestGPUSceneUpdate();
				SetRequiresGPUSceneUpdate_RenderThread(bRequiresUpdate);
			}
		}
	);
}

bool FNaniteGeometryCollectionSceneProxy::ShowCollisionMeshes(const FEngineShowFlags& EngineShowFlags) const
{
	if (IsCollisionEnabled())
	{
		if (EngineShowFlags.CollisionPawn && CollisionResponse.GetResponse(ECC_Pawn) != ECR_Ignore)
		{
			return true;
		}
		if (EngineShowFlags.CollisionVisibility && CollisionResponse.GetResponse(ECC_Visibility) != ECR_Ignore)
		{
			return true;
		}
		if (EngineShowFlags.Collision)
		{
			return true;
		}
	}
	return false;
}

void FNaniteGeometryCollectionSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_NaniteGeometryCollectionSceneProxy_GetDynamicMeshElements);

	const FEngineShowFlags& EngineShowFlags = ViewFamily.EngineShowFlags;
	const bool bDrawWireframeCollision = EngineShowFlags.Collision && IsCollisionEnabled();

	// draw extra stuff ( collision , bounds ... )
	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			// collision modes
			if (ShowCollisionMeshes(EngineShowFlags) && GeometryCollection && GeometryCollection->GetGeometryCollection() && AllowDebugViewmodes())
			{
				FTransform GeomTransform(GetLocalToWorld());
				if (bDrawWireframeCollision)
				{
					GeometryCollectionDebugDraw::DrawWireframe(*GeometryCollection->GetGeometryCollection(), GeomTransform, Collector, ViewIndex, GetWireframeColor().ToFColor(true));
				}
				else
				{
					FMaterialRenderProxy* CollisionMaterialInstance = new FColoredMaterialRenderProxy(GEngine->ShadedLevelColorationUnlitMaterial->GetRenderProxy(), GetWireframeColor());
					Collector.RegisterOneFrameMaterialProxy(CollisionMaterialInstance);
					GeometryCollectionDebugDraw::DrawSolid(*GeometryCollection->GetGeometryCollection(), GeomTransform, Collector, ViewIndex, CollisionMaterialInstance);
				}
			}

			// render bounds
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			RenderBounds(Collector.GetPDI(ViewIndex), ViewFamily.EngineShowFlags, GetBounds(), IsSelected());
#endif
		}
	}
}

#if RHI_RAYTRACING
ERayTracingPrimitiveFlags FNaniteGeometryCollectionSceneProxy::GetRayTracingPrimitiveFlags()
{
	if (!bUseStaticRayTracingInstances)
	{
		return Super::GetRayTracingPrimitiveFlags();
	}

	if (!(IsVisibleInRayTracing() && ShouldRenderInMainPass() && (IsDrawnInGame() || AffectsIndirectLightingWhileHidden() || CastsHiddenShadow())) && !IsRayTracingFarField())
	{
		return ERayTracingPrimitiveFlags::Exclude;
	}

	if (!HasRayTracingRepresentation())
	{
		return ERayTracingPrimitiveFlags::Exclude;
	}

	if (IsFirstPerson())
	{
		// First person primitives are currently not supported in raytracing as this kind of geometry only makes sense from the camera's point of view.
		return ERayTracingPrimitiveFlags::Exclude;
	}

	return ERayTracingPrimitiveFlags::CacheInstances;
}
#endif // RHI_RAYTRACING

FGeometryCollectionDynamicDataPool::FGeometryCollectionDynamicDataPool()
{
	FreeList.SetNum(32);
	for (int32 ListIndex = 0; ListIndex < FreeList.Num(); ++ListIndex)
	{
		FreeList[ListIndex] = new FGeometryCollectionDynamicData;
	}
}

FGeometryCollectionDynamicDataPool::~FGeometryCollectionDynamicDataPool()
{
	FScopeLock ScopeLock(&ListLock);

	for (FGeometryCollectionDynamicData* Entry : FreeList)
	{
		delete Entry;
	}

	for (FGeometryCollectionDynamicData* Entry : UsedList)
	{
		delete Entry;
	}

	FreeList.Empty();
	UsedList.Empty();
}

FGeometryCollectionDynamicData* FGeometryCollectionDynamicDataPool::Allocate()
{
	FScopeLock ScopeLock(&ListLock);

	FGeometryCollectionDynamicData* NewEntry = nullptr;
	if (FreeList.Num() > 0)
	{
		NewEntry = FreeList.Pop(EAllowShrinking::No);
	}

	if (NewEntry == nullptr)
	{
		NewEntry = new FGeometryCollectionDynamicData;
	}

	NewEntry->Reset();
	UsedList.Push(NewEntry);

	return NewEntry;
}

void FGeometryCollectionDynamicDataPool::Release(FGeometryCollectionDynamicData* DynamicData)
{
	FScopeLock ScopeLock(&ListLock);

	int32 UsedIndex = UsedList.Find(DynamicData);
	if (ensure(UsedIndex != INDEX_NONE))
	{
		UsedList.RemoveAt(UsedIndex, EAllowShrinking::No);
		FreeList.Push(DynamicData);
	}
}

void FGeometryCollectionTransformBuffer::UpdateDynamicData(FRHICommandListBase& RHICmdList, const TArray<FMatrix44f>& Transforms, EResourceLockMode LockMode)
{
	check(NumTransforms == Transforms.Num());

	void* VertexBufferData = RHICmdList.LockBuffer(VertexBufferRHI, 0, Transforms.Num() * sizeof(FMatrix44f), LockMode);
	FMemory::Memcpy(VertexBufferData, Transforms.GetData(), Transforms.Num() * sizeof(FMatrix44f));
	RHICmdList.UnlockBuffer(VertexBufferRHI);
}

FNaniteGeometryCollectionSceneProxy::FEmptyLightCacheInfo FNaniteGeometryCollectionSceneProxy::EmptyLightCacheInfo;

FLightInteraction FNaniteGeometryCollectionSceneProxy::FEmptyLightCacheInfo::GetInteraction(const FLightSceneProxy* LightSceneProxy) const
{
	// Ask base class
	TArray<FGuid> Empty_IrrelevantLights;
	ELightInteractionType LightInteraction = GetStaticInteraction(LightSceneProxy, Empty_IrrelevantLights);

	if (LightInteraction != LIT_MAX)
	{
		return FLightInteraction(LightInteraction);
	}

	// Use dynamic lighting if the light doesn't have static lighting.
	return FLightInteraction::Dynamic();
}

