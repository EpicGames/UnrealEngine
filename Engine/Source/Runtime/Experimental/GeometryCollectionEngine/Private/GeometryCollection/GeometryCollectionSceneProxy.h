// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DynamicMeshBuilder.h"
#include "EngineGlobals.h"
#include "HAL/CriticalSection.h"
#include "PrimitiveViewRelevance.h"
#include "PrimitiveSceneProxy.h"
#include "StaticMeshResources.h"
#include "Rendering/SkinWeightVertexBuffer.h"
#include "GeometryCollectionRendering.h"
#include "GeometryCollection/GeometryCollectionEditorSelection.h"
#include "HitProxies.h"
#include "EngineUtils.h"
#include "NaniteSceneProxy.h"
#include "GeometryCollection/GeometryCollectionObject.h"
#include "GeometryCollection/GeometryCollectionRenderData.h"
#include "GeometryCollection/GeometryCollectionHitProxy.h"
#include "InstanceDataSceneProxy.h"

class UGeometryCollection;
class UGeometryCollectionComponent;
struct FGeometryCollectionSection;
struct FGeometryCollectionMeshElement;

namespace Nanite
{
	struct FResources;
}


/** Vertex Buffer for transform data */
class FGeometryCollectionTransformBuffer : public FVertexBuffer
{
public:
	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		// #note: This differs from instanced static mesh in that we are storing the entire transform in the buffer rather than
		// splitting out the translation.  This is to simplify transferring data at runtime as a memcopy
		const FRHIBufferCreateDesc CreateDesc =
			FRHIBufferCreateDesc::CreateVertex<FVector4f>(TEXT("FGeometryCollectionTransformBuffer"), NumTransforms * 4)
			.AddUsage(EBufferUsageFlags::Dynamic | EBufferUsageFlags::ShaderResource)
			.DetermineInitialState();

		VertexBufferRHI = RHICmdList.CreateBuffer(CreateDesc);
		VertexBufferSRV = RHICmdList.CreateShaderResourceView(
			VertexBufferRHI,
			FRHIViewDesc::CreateBufferSRV()
				.SetType(FRHIViewDesc::EBufferType::Typed)
				.SetFormat(PF_A32B32G32R32F));
	}

	void UpdateDynamicData(FRHICommandListBase& RHICmdList, const TArray<FMatrix44f>& Transforms, EResourceLockMode LockMode);

	int32 NumTransforms;

	FShaderResourceViewRHIRef VertexBufferSRV;
};

inline void CopyTransformsWithConversionWhenNeeded(TArray<FMatrix44f>& DstTransforms, const TArray<FMatrix>& SrcTransforms)
{
	// LWC_TODO : we have no choice but to convert each element at this point to avoid changing GeometryCollectionAlgo::GlobalMatrices that is used all over the place
	DstTransforms.SetNumUninitialized(SrcTransforms.Num());
	for (int TransformIndex = 0; TransformIndex < SrcTransforms.Num(); ++TransformIndex)
	{
		DstTransforms[TransformIndex] = FMatrix44f(SrcTransforms[TransformIndex]); // LWC_TODO: Perf pessimization
	}
}

inline void CopyTransformsWithConversionWhenNeeded(TArray<FMatrix44f>& DstTransforms, const TArray<FTransform>& SrcTransforms)
{
	// LWC_TODO : we have no choice but to convert each element at this point to avoid changing GeometryCollectionAlgo::GlobalMatrices that is used all over the place
	DstTransforms.SetNumUninitialized(SrcTransforms.Num());
	for (int TransformIndex = 0; TransformIndex < SrcTransforms.Num(); ++TransformIndex)
	{
		DstTransforms[TransformIndex] = FTransform3f(SrcTransforms[TransformIndex]).ToMatrixWithScale(); // LWC_TODO: Perf pessimization
	}
}

inline void CopyTransformsWithConversionWhenNeeded(TArray<FMatrix44f>& DstTransforms, const TArray<FTransform3f>& SrcTransforms)
{
	DstTransforms.SetNumUninitialized(SrcTransforms.Num());
	for (int TransformIndex = 0; TransformIndex < SrcTransforms.Num(); ++TransformIndex)
	{
		DstTransforms[TransformIndex] = SrcTransforms[TransformIndex].ToMatrixWithScale();
	}
}

/** Mutable rendering data */
struct FGeometryCollectionDynamicData
{
	TArray<FMatrix44f> Transforms;
	uint64 FrameIndex = 0;

	FGeometryCollectionDynamicData()
	{
		Reset();
	}

	void Reset()
	{
		Transforms.Reset();
		FrameIndex = GFrameCounter;
	}

	void SetTransforms(const TArray<FTransform>& InTransforms)
	{
		// use for LWC as FMatrix and FMatrix44f are different when LWC is on 
		CopyTransformsWithConversionWhenNeeded(Transforms, InTransforms);
	}

	void SetTransforms(const TArray<FTransform3f>& InTransforms)
	{
		CopyTransformsWithConversionWhenNeeded(Transforms, InTransforms);
	}
};


class FGeometryCollectionDynamicDataPool
{
public:
	FGeometryCollectionDynamicDataPool();
	~FGeometryCollectionDynamicDataPool();

	FGeometryCollectionDynamicData* Allocate();
	void Release(FGeometryCollectionDynamicData* DynamicData);

private:
	TArray<FGeometryCollectionDynamicData*> UsedList;
	TArray<FGeometryCollectionDynamicData*> FreeList;

	FCriticalSection ListLock;
};


class FGeometryCollectionSceneProxyBase
{
protected:
	FGeometryCollectionSceneProxyBase(UGeometryCollectionComponent* Component, bool bInIsNanite);
	virtual ~FGeometryCollectionSceneProxyBase();

	void CreateRenderThreadResources(FRHICommandListBase& RHICmdList);
	void DestroyRenderThreadResources();

	/** Setup a geometry collection vertex factory. */
	void SetupVertexFactory(FRHICommandListBase& RHICmdList, FGeometryCollectionVertexFactory& GeometryCollectionVertexFactory, FColorVertexBuffer* ColorOverride = nullptr) const;

	/** Called on render thread to setup dynamic geometry for rendering */
	void SetDynamicData_RenderThread(FRHICommandListBase& RHICmdList, FGeometryCollectionDynamicData* NewDynamicData);

	FGeometryCollectionTransformBuffer& GetCurrentTransformBuffer()
	{
		return TransformBuffers[CurrentTransformBufferIndex];
	}

	const FGeometryCollectionTransformBuffer& GetCurrentTransformBuffer() const
	{
		return TransformBuffers[CurrentTransformBufferIndex];
	}

	const FGeometryCollectionTransformBuffer& GetCurrentPrevTransformBuffer() const
	{
		const int32 NumBuffers = TransformBuffers.Num();
		const int32 PreviousIndex = (CurrentTransformBufferIndex + NumBuffers - 1) % NumBuffers;
		return TransformBuffers[PreviousIndex];
	}

	void CycleTransformBuffers(bool bCycle)
	{
		if (bCycle)
		{
			CurrentTransformBufferIndex = (CurrentTransformBufferIndex + 1) % TransformBuffers.Num();
		}
	}

	TConstArrayView<FGeometryCollectionMeshElement> GetSectionArray(bool bUsesSubSections, bool bRemoveInternalFaces) const;

	/** Sets up a FMeshBatch for a specific LOD and section. */
	void GetMeshElement(
		int32 LODIndex,
		int32 SectionIndex,
		const FGeometryCollectionMeshElement& Section,
		const FVertexFactory* InVertexFactory,
		FMaterialRenderProxy* MaterialProxy,
		uint8 InDepthPriorityGroup,
		FRHIUniformBuffer* UniformBuffer,
		bool bWireframe,
		bool bReverseCulling,
		FMeshBatch& OutMeshBatch) const;

	using FMaterialArrayType = TArray<FMaterialRenderProxy*, TInlineAllocator<32>>;

	void GetSectionMaterialProxies(TConstArrayView<FGeometryCollectionMeshElement> InSectionArray, FMaterialArrayType& OutMaterialProxies) const;

#if RHI_RAYTRACING
	void UpdatingRayTracingGeometry_RenderingThread(FRHICommandListBase& RHICmdList, TConstArrayView<FGeometryCollectionMeshElement> InSectionArray, bool bRayTracingGeometryPerSection);

	bool IsRayTracingRelevant() const { return true; }
	bool IsRayTracingStaticRelevant() const { return bUseStaticRayTracingInstances; }
	bool HasRayTracingRepresentation() const { return bHasRayTracingRepresentation; }
	void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector, const FMatrix& LocalToWorld, FRHIUniformBuffer* UniformBuffer, bool bAnyMaterialHasWorldPositionOffset);
	void GetStaticRayTracingInstances(FStaticRayTracingInstances& OutRayTracingInstances, FRHIUniformBuffer* UniformBuffer);
	RayTracing::FGeometryGroupHandle GetRayTracingGeometryGroupHandle() const;
#endif

	uint32 GetAllocatedSize() const;

protected:

	const bool bIsNanite;
	ERHIFeatureLevel::Type FeatureLevel;

	const FGeometryCollectionMeshResources& MeshResource;
	FGeometryCollectionMeshDescription MeshDescription;
	FMaterialRelevance MaterialRelevance;

	FBoxSphereBounds PreSkinnedBounds;

	FGeometryCollectionVertexFactory VertexFactory;
	const bool bUseShaderBoneTransform;
	const bool bSupportsTripleBufferVertexUpload;

	TArray<UMaterialInterface*> Materials;

	TArray<int32> TransformToInstanceMapping;

	int32 NumTransforms = 0;

	FGeometryCollectionDynamicData* DynamicData = nullptr;

#if RHI_RAYTRACING
	bool bUseStaticRayTracingInstances = false;
#endif

private:

	/** Update skinned position buffer used by mobile CPU skinning path. */
	void UpdateSkinnedPositions(FRHICommandListBase& RHICmdList, TArray<FMatrix44f> const& Transforms);

	FPositionVertexBuffer SkinnedPositionVertexBuffer;

	int32 CurrentTransformBufferIndex = 0;
	TArray<FGeometryCollectionTransformBuffer, TInlineAllocator<4>> TransformBuffers;

	const bool bHasRayTracingRepresentation = false;

#if RHI_RAYTRACING
	const FGeometryCollectionRayTracingResources& RayTracingResources;
	RayTracing::FGeometryGroupHandle RayTracingGeometryGroupHandle;

	FRayTracingGeometry RayTracingGeometry;
	FRWBuffer RayTracingDynamicVertexBuffer;

	TArray<FMeshBatch> CachedPartRayTracingMaterials;
	TArray<uint32> PrimitiveInstanceIndices;
#endif

	bool bRenderResourcesCreated = false;
};


/***
*   FGeometryCollectionSceneProxy
*    
*	The FGeometryCollectionSceneProxy manages the interaction between the GeometryCollectionComponent
*   on the game thread and the vertex buffers on the render thread.
*
*   NOTE : This class is still in flux, and has a few pending todos. Your comments and 
*   thoughts are appreciated though. The remaining items to address involve:
*   - @todo double buffer - The double buffering of the FGeometryCollectionDynamicData.
*   - @todo GPU skin : Make the skinning use the GpuVertexShader
*/
class FGeometryCollectionSceneProxy final : public FPrimitiveSceneProxy, private FGeometryCollectionSceneProxyBase
{
	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollection;

	FCollisionResponseContainer CollisionResponse;

	bool bRenderResourcesCreated = false;

#if WITH_EDITOR
	bool bShowBoneColors = false;
	bool bSuppressSelectionMaterial = false;
	TArray<FColor> BoneColors;
	FColorVertexBuffer ColorVertexBuffer;
	FGeometryCollectionVertexFactory VertexFactoryDebugColor;
	UMaterialInterface* BoneSelectedMaterial = nullptr;
	#endif

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	bool bUsesSubSections = false;
	bool bEnableBoneSelection = false;
	TArray<TRefCountPtr<HHitProxy>> HitProxies;
	FColorVertexBuffer HitProxyIdBuffer;
#endif

public:
	FGeometryCollectionSceneProxy(UGeometryCollectionComponent* Component);
	virtual ~FGeometryCollectionSceneProxy();

	/** Called on render thread to setup dynamic geometry for rendering */
	void SetDynamicData_RenderThread(FRHICommandListBase& RHICmdList, FGeometryCollectionDynamicData* NewDynamicData);

	virtual uint32 GetMemoryFootprint() const override { return sizeof(*this) + GetAllocatedSize(); }
	uint32 GetAllocatedSize() const;

	SIZE_T GetTypeHash() const override;
	void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;
	void DestroyRenderThreadResources() override;
	void GetPreSkinnedLocalBounds(FBoxSphereBounds& OutBounds) const override;
	FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

	virtual bool AllowInstanceCullingOcclusionQueries() const override { return true; }

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	virtual HHitProxy* CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
 	virtual const FColorVertexBuffer* GetCustomHitProxyIdBuffer() const override
 	{
		return (bEnableBoneSelection || bUsesSubSections) ? &HitProxyIdBuffer : nullptr;
 	}
#endif
	virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override;

#if RHI_RAYTRACING
	virtual bool IsRayTracingRelevant() const override { return FGeometryCollectionSceneProxyBase::IsRayTracingRelevant(); }
	virtual bool IsRayTracingStaticRelevant() const override { return FGeometryCollectionSceneProxyBase::IsRayTracingStaticRelevant(); }
	virtual bool HasRayTracingRepresentation() const override { return FGeometryCollectionSceneProxyBase::HasRayTracingRepresentation(); }
	virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override { return FGeometryCollectionSceneProxyBase::GetDynamicRayTracingInstances(Collector, GetLocalToWorld(), GetUniformBuffer(), bAnyMaterialHasWorldPositionOffset); }
#endif

protected:
	/** Get material proxy from material ID */
	FMaterialRenderProxy* GetMaterial(FMeshElementCollector& Collector, int32 MaterialIndex) const;
	/** Get the standard or debug vertex factory dependent on current state. */
	FVertexFactory const* GetVertexFactory() const;

private:
	bool ShowCollisionMeshes(const FEngineShowFlags& EngineShowFlags) const;
};


class FNaniteGeometryCollectionSceneProxy : public Nanite::FSceneProxyBase, private FGeometryCollectionSceneProxyBase
{
public:
	using Super = Nanite::FSceneProxyBase;
	
	FNaniteGeometryCollectionSceneProxy(UGeometryCollectionComponent* Component);
	virtual ~FNaniteGeometryCollectionSceneProxy();

public:
	// FPrimitiveSceneProxy interface.
	virtual void CreateRenderThreadResources(FRHICommandListBase& RHICmdList) override;
	virtual void DestroyRenderThreadResources() override;
	virtual void GetPreSkinnedLocalBounds(FBoxSphereBounds& OutBounds) const override;
	virtual SIZE_T GetTypeHash() const override;
	virtual FPrimitiveViewRelevance	GetViewRelevance(const FSceneView* View) const override;
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;
#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	virtual HHitProxy* CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
#endif
	virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override;

	virtual uint32 GetMemoryFootprint() const override { return sizeof(*this) + GetAllocatedSize(); }
	uint32 GetAllocatedSize() const;

	// FSceneProxyBase interface.
	virtual Nanite::FResourceMeshInfo GetResourceMeshInfo() const override;
	virtual Nanite::FResourcePrimitiveInfo GetResourcePrimitiveInfo() const override;

	/** Called on render thread to setup dynamic geometry for rendering */
	void SetDynamicData_RenderThread(FRHICommandListBase& RHICmdList, FGeometryCollectionDynamicData* NewDynamicData, const FMatrix &PrimitiveLocalToWorld);

	void FlushGPUSceneUpdate_GameThread();

	FORCEINLINE void SetRequiresGPUSceneUpdate_RenderThread(bool bRequireUpdate)
	{
		bRequiresGPUSceneUpdate = bRequireUpdate;
	}

	FORCEINLINE bool GetRequiresGPUSceneUpdate_RenderThread() const
	{
		return bRequiresGPUSceneUpdate;
	}

	inline virtual void GetLCIs(FLCIArray& LCIs) override
	{
		FLightCacheInterface* LCI = &EmptyLightCacheInfo;
		LCIs.Add(LCI);
	}

#if RHI_RAYTRACING
	virtual bool IsRayTracingRelevant() const override { return FGeometryCollectionSceneProxyBase::IsRayTracingRelevant(); }
	virtual bool IsRayTracingStaticRelevant() const override { return FGeometryCollectionSceneProxyBase::IsRayTracingStaticRelevant(); }
	virtual bool HasRayTracingRepresentation() const override { return FGeometryCollectionSceneProxyBase::HasRayTracingRepresentation(); }
	virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override { return FGeometryCollectionSceneProxyBase::GetDynamicRayTracingInstances(Collector, GetLocalToWorld(), GetUniformBuffer(), bAnyMaterialHasWorldPositionOffset); }
	virtual ERayTracingPrimitiveFlags GetRayTracingPrimitiveFlags() override;
	virtual void GetStaticRayTracingInstances(FStaticRayTracingInstances& OutRayTracingInstances) override { return FGeometryCollectionSceneProxyBase::GetStaticRayTracingInstances(OutRayTracingInstances, GetUniformBuffer()); }
	virtual RayTracing::FGeometryGroupHandle GetRayTracingGeometryGroupHandle() const override { return FGeometryCollectionSceneProxyBase::GetRayTracingGeometryGroupHandle(); }
#endif

protected:
	// TODO : Copy required data from UObject instead of using unsafe object pointer.
	const UGeometryCollection* GeometryCollection = nullptr;
	FCollisionResponseContainer CollisionResponse;

	struct FGeometryNaniteData
	{
		FBoxSphereBounds LocalBounds;
		uint32 HierarchyOffset;
	};
	TArray<FGeometryNaniteData> GeometryNaniteData;

	uint32 bCastShadow : 1;
	uint32 bReverseCulling : 1;
	uint32 bHasMaterialErrors : 1;
	uint32 bRequiresGPUSceneUpdate : 1;
	uint32 bEnableBoneSelection : 1;
	
#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	TArray<TRefCountPtr<HHitProxy>> HitProxies;
#endif

	FInstanceSceneDataBuffers InstanceSceneDataBuffersImpl;

	// Geometry collection doesn't currently support baked light maps, so we use this simple empty light cache info for all nanite geometry collection proxies
	class FEmptyLightCacheInfo : public FLightCacheInterface
	{
	public:

		// FLightCacheInterface.
		GEOMETRYCOLLECTIONENGINE_API virtual FLightInteraction GetInteraction(const FLightSceneProxy* LightSceneProxy) const override;
	};

private:
	void UpdateInstanceSceneDataBuffers(const FMatrix& PrimitiveLocalToWorld);

	bool ShowCollisionMeshes(const FEngineShowFlags& EngineShowFlags) const;

private:
	static FEmptyLightCacheInfo EmptyLightCacheInfo;
};
