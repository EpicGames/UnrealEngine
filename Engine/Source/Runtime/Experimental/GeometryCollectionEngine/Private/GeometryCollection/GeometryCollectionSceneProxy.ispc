// Copyright Epic Games, Inc. All Rights Reserved.

#include "Math/Vector.isph"
#include "Math/Matrix.isph"

export void SetDynamicData_RenderThread(uniform FVector3f PositionBuffer[], 
										const uniform unsigned int NumVertices, 
										const uniform unsigned int Stride, 
										const uniform uint16 BoneMap[], 
										const uniform FMatrix44f Transforms[], 
										const uniform FVector3f Vertices[])
{
	uniform unsigned int Chunk = 0;

	foreach(i = 0 ... NumVertices)
	{
		uniform float * uniform pVertices = (uniform float * uniform)&Vertices[Chunk];
		uniform float * uniform pPosition = (uniform float * uniform)&PositionBuffer[Chunk];

		FVector3f P;
		aos_to_soa3(pVertices, &P.V[0], &P.V[1], &P.V[2]);

		FVector3f Out;
		const int32 Bone = BoneMap[i];
		foreach_unique(Index in Bone)
		{
			Out = MatrixTransformPosition(P, Transforms[Index]);
		}

		soa_to_aos3(Out.V[0], Out.V[1], Out.V[2], pPosition);

		Chunk += programCount;
	}
}