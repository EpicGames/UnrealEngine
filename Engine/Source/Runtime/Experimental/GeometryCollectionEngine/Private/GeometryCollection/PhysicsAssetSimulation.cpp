// Copyright Epic Games, Inc. All Rights Reserved.
#include "GeometryCollection/PhysicsAssetSimulation.h"
#include "GeometryCollection/GeometryCollectionCollisionStructureManager.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"

#include "PhysicsSolver.h"
#include "ChaosSolversModule.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/ErrorReporter.h"
#include "Chaos/TriangleMesh.h"
#include "Chaos/Utilities.h"

#include "Animation/Skeleton.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/SkeletalMeshSocket.h"
#include "Chaos/ChaosPhysicalMaterial.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PhysicsEngine/SkeletalBodySetup.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "PhysicsProxy/SkeletalMeshPhysicsProxy.h"

#include "DrawDebugHelpers.h"
#include "Async/ParallelFor.h"
#include "Math/Box.h"
#include "Math/NumericLimits.h"
#include "Modules/ModuleManager.h"

void FPhysicsAssetSimulationUtil::BuildParams(const UObject* Caller, const AActor* OwningActor, const USkeletalMeshComponent* SkelMeshComponent, const UPhysicsAsset* PhysicsAsset, FSkeletalMeshPhysicsProxyParams& Params)
{
	Params.LocalToWorld = Params.InitialTransform = OwningActor->GetTransform();

	FVector ActorOriginVec, ActorBoxExtentVec;	// LWC_TODO: Perf pessimization
	OwningActor->GetActorBounds(true, ActorOriginVec, ActorBoxExtentVec);
	Chaos::FVec3 ActorOrigin(ActorOriginVec);
	Chaos::FVec3 ActorBoxExtent(ActorBoxExtentVec);

	const USkeletalMesh* SkeletalMesh = SkelMeshComponent->GetSkeletalMeshAsset();
	if (!SkeletalMesh)
	{
		return;
	}

	const USkeleton* Skeleton = SkeletalMesh->GetSkeleton();
	const FSkeletalMeshRenderData* SkelMeshRenderData = SkeletalMesh->GetResourceForRendering();
	if (!PhysicsAsset || !Skeleton || !SkelMeshRenderData || !SkelMeshRenderData->LODRenderData.Num())
	{
		return;
	}

	const FReferenceSkeleton& ReferenceSkeleton = Skeleton->GetReferenceSkeleton();
	const TArray<FBoneIndexType>& SkeletalBoneMap = SkelMeshRenderData->LODRenderData[0].RenderSections[0].BoneMap;

	const TArray<USkeletalBodySetup*> &SkeletalBodySetups = PhysicsAsset->SkeletalBodySetups;
	Params.BoneHierarchy.InitPreAdd(SkeletalBodySetups.Num());

	TArray<int32> ParentIds;

	for (USkeletalBodySetup* Setup : SkeletalBodySetups)
	{
		if (!Setup)
			continue;

		TUniquePtr<FAnalyticImplicitGroup> AnalyticShapeGroup(
			new FAnalyticImplicitGroup(
				Setup->BoneName,
				ReferenceSkeleton.FindBoneIndex(Setup->BoneName)));
		check(ReferenceSkeleton.IsValidIndex(AnalyticShapeGroup->GetBoneIndex()));

		//
		// Transform hierarchy info
		//

		FTransform InitialBoneWorldXf = FTransform::Identity;
		if (ReferenceSkeleton.IsValidIndex(AnalyticShapeGroup->GetBoneIndex()))
		{
			int32 BoneId = AnalyticShapeGroup->GetBoneIndex();
			int32 ParentId = ReferenceSkeleton.GetParentIndex(BoneId);
			AnalyticShapeGroup->SetParentBoneIndex(ParentId);
			AnalyticShapeGroup->RefBoneXf = ReferenceSkeleton.GetRefBonePose()[BoneId];
			ParentIds.AddUnique(ParentId);

			InitialBoneWorldXf = AnalyticShapeGroup->RefBoneXf;
			while (ParentId != INDEX_NONE)
			{
				InitialBoneWorldXf *= ReferenceSkeleton.GetRefBonePose()[ParentId];
				ParentId = ReferenceSkeleton.GetParentIndex(ParentId);
			}
			InitialBoneWorldXf *= Params.InitialTransform;
		}
		const Chaos::FVec3 Scale3D = InitialBoneWorldXf.GetScale3D();

		//
		// Body Settings
		//
		switch (Setup->PhysicsType)
		{
		case EPhysicsType::PhysType_Default:
			AnalyticShapeGroup->SetRigidBodyState(EObjectStateTypeEnum::Chaos_Object_UserDefined);
			break;
		case EPhysicsType::PhysType_Kinematic:
			AnalyticShapeGroup->SetRigidBodyState(EObjectStateTypeEnum::Chaos_Object_Kinematic);
			break;
		case EPhysicsType::PhysType_Simulated:
			AnalyticShapeGroup->SetRigidBodyState(EObjectStateTypeEnum::Chaos_Object_Dynamic);
			break;
		}

		//
		// Implicit geometry
		//

		FKAggregateGeom& AggGeom = Setup->AggGeom;
		const TArray<FKSphereElem> &FKSphereElems = AggGeom.SphereElems;
		const TArray<FKBoxElem> &FKBoxElems = AggGeom.BoxElems;
		const TArray<FKSphylElem> &FKSphylElems = AggGeom.SphylElems;
		const TArray<FKTaperedCapsuleElem> &FKTaperedCapsuleElems = AggGeom.TaperedCapsuleElems;
		const TArray<FKConvexElem> &FKConvexElems = AggGeom.ConvexElems;

		const int32 NumElems = FKSphereElems.Num() + FKBoxElems.Num() + FKSphylElems.Num() + FKTaperedCapsuleElems.Num() + FKConvexElems.Num();
		const int32 NumImplicits = NumElems + FKTaperedCapsuleElems.Num() * 2;

		const bool DoCollisionGeom = Params.CollisionType == ECollisionTypeEnum::Chaos_Surface_Volumetric;
		AnalyticShapeGroup->Init(NumImplicits, DoCollisionGeom);

		// Sphere
		if (const int32 NumSpheres = FKSphereElems.Num() + FKTaperedCapsuleElems.Num() * 2)
		{
			AnalyticShapeGroup->Spheres.Reserve(NumSpheres);
			for (const FKSphereElem &Elem : FKSphereElems)
			{
				// Ensure scaling is uniform
				check(FMath::Abs(Scale3D[0] - Scale3D[1]) < KINDA_SMALL_NUMBER && FMath::Abs(Scale3D[0] - Scale3D[2]) < KINDA_SMALL_NUMBER);

				const Chaos::FReal Radius = Elem.Radius * Scale3D[0];
				if (Radius < SMALL_NUMBER)
					continue;
				const FTransform Xf = Elem.GetTransform(); // Xf is Elem.Center

				// Translation is in the transform, scaling is baked into the implicit shape.
				AnalyticShapeGroup->Add(
					Xf,
					new Chaos::FSphere(Chaos::FVec3(0), Radius));

				UE_LOGF(USkeletalMeshSimulationComponentLogging, Log,
					"USkeletalMeshSimulationComponent::OnCreatePhysicsState() - Caller: %p Actor: '%ls' - "
						"ADDED SPHERE", Caller, &OwningActor->GetName()[0]);
			}
		}
		// Box
		if (const int32 NumBoxes = FKBoxElems.Num())
		{
			AnalyticShapeGroup->Boxes.Reserve(NumBoxes);
			for (const FKBoxElem &Elem : FKBoxElems)
			{
				const FKBoxElem ScaledElem = Elem.GetFinalScaled(Scale3D, FTransform::Identity);
				if (ScaledElem.X < SMALL_NUMBER || ScaledElem.Y < SMALL_NUMBER || ScaledElem.Z < SMALL_NUMBER)
					continue;
				const FTransform Xf = Elem.GetTransform(); // Xf is Elem.Center and Elem.Rotation

				// Translation is in the transform, scaling is baked into the implicit shape.
				const FVector HalfExtents(ScaledElem.X / 2, ScaledElem.Y / 2, ScaledElem.Z / 2);
				AnalyticShapeGroup->Add(
					Xf,
					new Chaos::TBox<Chaos::FReal, 3>(-HalfExtents, HalfExtents));

				UE_LOGF(USkeletalMeshSimulationComponentLogging, Log,
					"USkeletalMeshSimulationComponent::OnCreatePhysicsState() - Caller: %p Actor: '%ls' - "
						"ADDED BOX - Extents: %g, %g, %g", Caller, &OwningActor->GetName()[0], ScaledElem.X, ScaledElem.Y, ScaledElem.Z);
			}
		}
		// Capsule
		if (const int32 NumCapsules = FKSphylElems.Num())
		{
			// @todo(ccaulfield): put back when we have capsule collision. For now use spheres...
#if 0
			AnalyticShapeGroup->Capsules.Reserve(NumCapsules);
			for (const FKSphylElem &Elem : FKSphylElems)
			{
				const Chaos::FReal Radius = Elem.GetScaledRadius(Scale3D);
				if (Radius < SMALL_NUMBER)
					continue;

				const FTransform Xf = Elem.GetTransform(); // Xf is Elem.Center and Elem.Rotation

				// Translation is in the transform, scaling is baked into the implicit shape.
				const Chaos::FReal HalfHeight = Elem.GetScaledHalfLength(Scale3D) - Radius;
				AnalyticShapeGroup->Add(
					Xf,
					new Chaos::FCapsule(
						Chaos::FVec3(0.f, 0.f, -HalfHeight), // Min
						Chaos::FVec3(0.f, 0.f, HalfHeight), // Max
						Radius));

				UE_LOGF(USkeletalMeshSimulationComponentLogging, Log,
					"USkeletalMeshSimulationComponent::OnCreatePhysicsState() - Caller: %p Actor: '%ls' - "
						"ADDED CAPSULE - height: %g radius: %g", Caller, &OwningActor->GetName()[0], HalfHeight * 2, Radius);
			}
#endif
			AnalyticShapeGroup->Spheres.Reserve(NumCapsules);
			for (const FKSphylElem& Elem : FKSphylElems)
			{
				// Ensure scaling is uniform
				check(FMath::Abs(Scale3D[0] - Scale3D[1]) < KINDA_SMALL_NUMBER && FMath::Abs(Scale3D[0] - Scale3D[2]) < KINDA_SMALL_NUMBER);

				const Chaos::FReal Radius = Elem.Radius * Scale3D[0];
				if (Radius < SMALL_NUMBER)
					continue;
				const FTransform Xf = Elem.GetTransform(); // Xf is Elem.Center

				// Translation is in the transform, scaling is baked into the implicit shape.
				AnalyticShapeGroup->Add(
					Xf,
					new Chaos::FSphere(Chaos::FVec3(0), Radius));

				UE_LOGF(USkeletalMeshSimulationComponentLogging, Log,
					"USkeletalMeshSimulationComponent::OnCreatePhysicsState() - Caller: %p Actor: '%ls' - "
						"ADDED SPHERE", Caller, &OwningActor->GetName()[0]);
			}
		}
		// Tapered capsule
		if (const int32 NumTaperedCapsules = FKTaperedCapsuleElems.Num())
		{
			AnalyticShapeGroup->TaperedCylinders.Reserve(NumTaperedCapsules);
			for (const FKTaperedCapsuleElem &Elem : FKTaperedCapsuleElems)
			{
				float Radius0, Radius1;
				Elem.GetScaledRadii(Scale3D, Radius0, Radius1);
				if (Radius0 < SMALL_NUMBER && Radius1 < SMALL_NUMBER)
					continue;

				const FTransform Xf = Elem.GetTransform(); // Xf is Elem.Center and Elem.Rotation

				const Chaos::FReal HalfHeight = Elem.GetScaledCylinderLength(Scale3D) / 2;
				const Chaos::FVec3 Pt(0, 0, HalfHeight);
				AnalyticShapeGroup->Add(
					Xf,
					new Chaos::FTaperedCylinder(-Pt, Pt, Radius0, Radius1));

				if (Radius0 > KINDA_SMALL_NUMBER)
				{
					FTransform SphereXf = FTransform(Elem.Center - Pt) * Xf;
					AnalyticShapeGroup->Add(SphereXf, new Chaos::FSphere(Chaos::FVec3(0), Radius0));
				}
				if (Radius1 > KINDA_SMALL_NUMBER)
				{
					FTransform SphereXf = FTransform(Elem.Center + Pt) * Xf;
					AnalyticShapeGroup->Add(SphereXf, new Chaos::FSphere(Chaos::FVec3(0), Radius1));
				}

				UE_LOG(USkeletalMeshSimulationComponentLogging, Log,
					TEXT("USkeletalMeshSimulationComponent::OnCreatePhysicsState() - Caller: %p Actor: '%s' - "
						"ADDED TAPERED CYLINDER"), Caller, &OwningActor->GetName()[0]);
			}
		}
		// Convex hull
		if (const int32 NumConvexHulls = FKConvexElems.Num())
		{
			AnalyticShapeGroup->ConvexHulls.Reserve(NumConvexHulls);
			for (const FKConvexElem &Elem : FKConvexElems)
			{
				const int32 NumPoints = Elem.VertexData.Num();
				if (NumPoints < 4)
					continue;

				// Get the particles, apply Elem transform, plus bone level scaling.
				FTransform Xf = Elem.GetTransform();
				Xf.SetScale3D(
					FVector(Scale3D[0] * Xf.GetScale3D()[0],
						Scale3D[1] * Xf.GetScale3D()[1],
						Scale3D[2] * Xf.GetScale3D()[2]));

				TArray<Chaos::FVec3> Points;
				Points.SetNumUninitialized(NumPoints);
				FBox Bounds(ForceInitToZero);
				for (int32 i = 0; i < NumPoints; i++)
				{
					Points[i] = Elem.VertexData[i];
					Bounds += Points[i];
				}

				// TODO: Use Chaos convex.
				CHAOS_ENSURE(false);
			}
		}
		Params.BoneHierarchy.Add(MoveTemp(AnalyticShapeGroup));
	} // for all SkeletalBodySetups

	// Make sure all parent bones are accounted for
	int32 Index = 0;
	for (; Index < ParentIds.Num(); Index++)
	{
		const int32 ParentId = ParentIds[Index];
		if (ParentId != INDEX_NONE)
		{
			if (!Params.BoneHierarchy.HasBoneIndex(ParentId))
			{
				check(ReferenceSkeleton.IsValidIndex(ParentId));

				TUniquePtr<FAnalyticImplicitGroup> AnalyticShapeGroup(
					new FAnalyticImplicitGroup(
						ReferenceSkeleton.GetBoneName(ParentId), ParentId));

				const int32 NewParentId = ReferenceSkeleton.GetParentIndex(ParentId);
				AnalyticShapeGroup->SetParentBoneIndex(NewParentId);

				Params.BoneHierarchy.Add(MoveTemp(AnalyticShapeGroup));

				UE_LOGF(USkeletalMeshSimulationComponentLogging, Log,
					"USkeletalMeshSimulationComponent::OnCreatePhysicsState() - Caller: %p Actor: '%ls' - "
						"ADDED PARENT BONE", Caller, &OwningActor->GetName()[0]);

				if (NewParentId != INDEX_NONE)
				{
					ParentIds.AddUnique(NewParentId);
				}
			}
		}
	}
	check(Index == ParentIds.Num());

	// Initialize the hierarchy.
	Params.BoneHierarchy.InitPostAdd();

	// Set initial bone transforms
	Params.BoneHierarchy.PrepareForUpdate();
	Params.BoneHierarchy.SetActorWorldSpaceTransform(OwningActor->GetTransform());
	const TArray<int32>& BoneIndices = Params.BoneHierarchy.GetBoneIndices();
	for (const int32 BoneIndex : BoneIndices)
	{
		//Params.BoneHierarchy.SetAnimLocalSpaceTransform(
		//	BoneIndex, 
		//	ReferenceSkeleton.GetRefBonePose()[BoneIndex]);

		//Params.BoneHierarchy.SetAnimLocalSpaceTransform(
		//	BoneIndex,
		//	SkelMeshComponent->GetBoneTransform(BoneIndex, FTransform::Identity));

		//Params.BoneHierarchy.SetAnimLocalSpaceTransform(
		//	BoneIndex,
		//	SkelMeshComponent->BoneSpaceTransforms[BoneIndex]);

		FTransform BoneXf = FTransform::Identity;
		int32 TmpBoneIndex = BoneIndex;
		int32 SocketIndex = INDEX_NONE;
		if (USkeletalMeshSocket const* Socket =
			SkelMeshComponent->GetSkeletalMeshAsset()->FindSocketInfo(
				Params.BoneHierarchy.GetAnalyticShapeGroup(BoneIndex)->GetBoneName(),
				BoneXf,
				TmpBoneIndex,
				SocketIndex))
		{
			Params.BoneHierarchy.SetSocketIndexForBone(BoneIndex, SocketIndex);
		}
		else
		{
			Params.BoneHierarchy.SetAnimLocalSpaceTransform(
				BoneIndex,
				SkelMeshComponent->GetSocketTransform(
					Params.BoneHierarchy.GetAnalyticShapeGroup(BoneIndex)->GetBoneName(),
					ERelativeTransformSpace::RTS_ParentBoneSpace));
		}
	}
	Params.BoneHierarchy.PrepareAnimWorldSpaceTransforms();

	UE_LOGF(USkeletalMeshSimulationComponentLogging, Log,
		"USkeletalMeshSimulationComponent::OnCreatePhysicsState() - Caller: %p Actor: '%ls' - "
			"Bone hierarchy, num shape groups: %d",
		Caller, &OwningActor->GetName()[0],
		Params.BoneHierarchy.GetAnalyticShapeGroups().Num());
}

bool FPhysicsAssetSimulationUtil::UpdateAnimState(const UObject* Caller, const AActor* OwningActor, const USkeletalMeshComponent* SkelMeshComponent, const float Dt, FSkeletalMeshPhysicsProxyParams& Params)
{
	int32 UpdatedBones = 0;

	const USkeletalMesh* SkeletalMesh = SkelMeshComponent->GetSkeletalMeshAsset();
	
	if (!SkeletalMesh)
	{
		return false;
	}

	const FSkeletalMeshRenderData* SkelMeshRenderData = SkeletalMesh->GetResourceForRendering();
	if (!SkelMeshRenderData || !SkelMeshRenderData->LODRenderData.Num())
	{
		return false;
	}

	// Calling this doesn't seem to matter...
	//SkelMeshComponent->UpdateKinematicBonesToAnim(
	//	//SkelMeshComponent->GetComponentSpaceTransforms(),
	//	SkelMeshComponent->GetEditableComponentSpaceTransforms(),
	//	ETeleportType::None,
	//	false);

	// Tell the bone hierarchy we're about to update it.
	Params.BoneHierarchy.PrepareForUpdate();

	// Update the local-to-world.
	Params.BoneHierarchy.SetActorWorldSpaceTransform(OwningActor->GetTransform());

	// Update bone transforms from animation.
	// @todo(ccaulfield): BoneSpaceTransforms is now copied - can we avoid that?
	// The ocnst cast is unfortunate - maybe the task waiting in the component should use mutable...
	TArray<FTransform> BoneSpaceTransforms = const_cast<USkeletalMeshComponent*>(SkelMeshComponent)->GetBoneSpaceTransforms();
	const TArray<int32>& BoneIndices = Params.BoneHierarchy.GetBoneIndices();
	for (const int32 BoneIndex : BoneIndices)
	{
		if (BoneSpaceTransforms.IsValidIndex(BoneIndex))
		{
			const int32 SocketIndex = Params.BoneHierarchy.GetSocketIndexForBone(BoneIndex);
			if (SocketIndex != INDEX_NONE)
			{
				USkeletalMeshSocket *Socket = SkelMeshComponent->GetSkeletalMeshAsset()->GetSocketByIndex(SocketIndex);
				Params.BoneHierarchy.SetAnimLocalSpaceTransform(
					BoneIndex,
					Socket->GetSocketLocalTransform());
			}
			else
			{
				Params.BoneHierarchy.SetAnimLocalSpaceTransform(
					BoneIndex,
					SkelMeshComponent->GetSocketTransform(
						Params.BoneHierarchy.GetAnalyticShapeGroup(BoneIndex)->GetBoneName(),
						ERelativeTransformSpace::RTS_ParentBoneSpace));
			}
			UpdatedBones++;
		}
	}

	return (UpdatedBones > 0);
}
