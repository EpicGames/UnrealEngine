// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Chaos/Declares.h"
#include "Chaos/ChaosSolver.h"
#include "Chaos/ChaosSolverActor.h"
#include "EventManager.h"
#include "EventsData.h"
#include "HAL/ThreadSafeBool.h"
#include "ChaosCollisionEventFilter.h"
#include "ChaosBreakingEventFilter.h"
#include "ChaosTrailingEventFilter.h"
#include "ChaosRemovalEventFilter.h"
#include "ChaosBlueprint.generated.h"

class AGeometryCollectionActor;

/**
* Called when new destruction events are available for collisions. Collision listening must be enabled to get callbacks on this delegate.
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnChaosCollisionEvents, const TArray<FChaosCollisionEventData>&, CollisionEvents);

/**
* Called when new destruction events are available for breaks. Breaking listening must be enabled to get callbacks on this delegate.
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnChaosBreakingEvents, const TArray<FChaosBreakingEventData>&, BreakingEvents);

/**
* Called when new trailing events are available for breaks. Trailing listening must be enabled to get callbacks on this delegate.
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnChaosTrailingEvents, const TArray<FChaosTrailingEventData>&, TrailingEvents);

/**
* Called when new destruction events are available for removals. Removal listening must be enabled to get callbacks on this delegate.
*/
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnChaosRemovalEvents, const TArray<FChaosRemovalEventData>&, RemovalEvents);


/** Object allowing for retrieving Chaos Destruction data. */
UCLASS(ClassGroup = (Chaos), hidecategories = (Object, ActorComponent, Physics, Rendering, Mobility, LOD), ShowCategories = Trigger, meta = (BlueprintSpawnableComponent), MinimalAPI)
class UChaosDestructionListener : public USceneComponent
{
	GENERATED_UCLASS_BODY()
		
public:
	//~ Begin UObject Interface
#if WITH_EDITOR
	GEOMETRYCOLLECTIONENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject Interface
	
	//~ Begin UActorComponent interface
	GEOMETRYCOLLECTIONENGINE_API virtual void BeginPlay() override;
	GEOMETRYCOLLECTIONENGINE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	//~ End UActorComponent interface

	// Whether or not collision event listening is enabled
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Collision Events")
	uint8 bIsCollisionEventListeningEnabled : 1;

	// Whether or not collision event listening is enabled
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Breaking Events")
	uint8 bIsBreakingEventListeningEnabled : 1;

	// Whether or not trailing event listening is enabled
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Trailing Events")
	uint8 bIsTrailingEventListeningEnabled : 1;

	// Whether or not removal event listening is enabled
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Removal Events")
	uint8 bIsRemovalEventListeningEnabled : 1;

	// The settings to use for collision event listening
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Collision Events", meta = (EditCondition = "bIsCollisionEventListeningEnabled"))
	FChaosCollisionEventRequestSettings CollisionEventRequestSettings;

	// The settings to use for breaking event listening
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Breaking Events", meta = (EditCondition = "bIsBreakingEventListeningEnabled"))
	FChaosBreakingEventRequestSettings BreakingEventRequestSettings;

	// The settings to use for trailing event listening
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Trailing Events", meta = (EditCondition = "bIsTrailingEventListeningEnabled"))
	FChaosTrailingEventRequestSettings TrailingEventRequestSettings;

	// The settings to use for removal event listening
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Removal Events", meta = (EditCondition = "bIsRemovalEventListeningEnabled"))
	FChaosRemovalEventRequestSettings RemovalEventRequestSettings;

	// Which chaos solver actors we're using. If empty, this listener will fallback to the "world" solver.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Solvers")
	TSet<TObjectPtr<AChaosSolverActor>> ChaosSolverActors;

	// Which chaos solver actors we're using. If empty, this listener will fallback to the "world" solver.
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "GeometryCollections")
	TSet<TObjectPtr<AGeometryCollectionActor>> GeometryCollectionActors; // Using TSet automatically blocks if user tries to add same actor twice

	// Dynamically adds a chaos solver to the listener
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void AddChaosSolverActor(AChaosSolverActor* ChaosSolverActor);
	
	// Dynamically removes a chaos solver from the listener
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void RemoveChaosSolverActor(AChaosSolverActor* ChaosSolverActor);

	// Dynamically adds a chaos solver to the listener
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void AddGeometryCollectionActor(AGeometryCollectionActor* GeometryCollectionActor);

	// Dynamically removes a chaos solver from the listener
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void RemoveGeometryCollectionActor(AGeometryCollectionActor* GeometryCollectionActor);

	// Sets collision event request settings dynamically
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void SetCollisionEventRequestSettings(const FChaosCollisionEventRequestSettings& InSettings);

	// Sets breaking event request settings dynamically
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void SetBreakingEventRequestSettings(const FChaosBreakingEventRequestSettings& InSettings);

	// Sets trailing event request settings dynamically
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void SetTrailingEventRequestSettings(const FChaosTrailingEventRequestSettings& InSettings);

	// Sets removal event request settings dynamically
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void SetRemovalEventRequestSettings(const FChaosRemovalEventRequestSettings& InSettings);

	// Enables or disables collision event listening
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void SetCollisionEventEnabled(bool bIsEnabled);

	// Enables or disables breaking event listening
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void SetBreakingEventEnabled(bool bIsEnabled);

	// Enables or disables trailing event listening
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void SetTrailingEventEnabled(bool bIsEnabled);

	// Enables or disables removal event listening
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void SetRemovalEventEnabled(bool bIsEnabled);

	// Returns if the destruction listener is listening to any events
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API bool IsEventListening() const;

	/** Called when new collision events are available. */
	UPROPERTY(BlueprintAssignable)
	FOnChaosCollisionEvents OnCollisionEvents;

	/** Called when new breaking events are available. */
	UPROPERTY(BlueprintAssignable)
	FOnChaosBreakingEvents OnBreakingEvents;

	/** Called when new trailing events are available. */
	UPROPERTY(BlueprintAssignable)
	FOnChaosTrailingEvents OnTrailingEvents;

	/** Called when new trailing events are available. */
	UPROPERTY(BlueprintAssignable)
	FOnChaosRemovalEvents OnRemovalEvents;

	// Sorts collision events according to the given sort method	
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void SortCollisionEvents(UPARAM(ref) TArray<FChaosCollisionEventData>& CollisionEvents, EChaosCollisionSortMethod SortMethod);

	// Sorts breaking events according to the given sort method	
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void SortBreakingEvents(UPARAM(ref) TArray<FChaosBreakingEventData>& BreakingEvents, EChaosBreakingSortMethod SortMethod);

	// Sorts trailing events according to the given sort method	
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void SortTrailingEvents(UPARAM(ref) TArray<FChaosTrailingEventData>& TrailingEvents, EChaosTrailingSortMethod SortMethod);

	// Sorts removal events according to the given sort method	
	UFUNCTION(BlueprintCallable, Category = "Destruction Listener", meta = (WorldContext = "WorldContextObject"))
	GEOMETRYCOLLECTIONENGINE_API void SortRemovalEvents(UPARAM(ref) TArray<FChaosRemovalEventData>& RemovalEvents, EChaosRemovalSortMethod SortMethod);

private:
	// Updates the scene component transform settings
	GEOMETRYCOLLECTIONENGINE_API void UpdateTransformSettings();

	// Retrieves data from solvers
	GEOMETRYCOLLECTIONENGINE_API void GetDataFromSolvers();

	GEOMETRYCOLLECTIONENGINE_API void ClearEvents();
	GEOMETRYCOLLECTIONENGINE_API void UpdateEvents();

protected:

	// Task state enumeration used to track if there is an async processing task in-flight
	enum class ETaskState
	{
		// If there is no filtering and sorting task in-flight
		NoTask = 0,

		// If there is a task in-flight
		Processing,

		// If there was a task in flight but it is now finished with results ready for broadcast
		Finished
	};
	FThreadSafeCounter TaskState;

	// The raw data arrays derived from the solvers
	Chaos::FCollisionDataArray RawCollisionDataArray;
	Chaos::FBreakingDataArray RawBreakingDataArray;
	Chaos::FTrailingDataArray RawTrailingDataArray;
	Chaos::FRemovalDataArray RawRemovalDataArray;

	FTransform ChaosComponentTransform;

	FThreadSafeBool bChanged;

	float LastCollisionDataTimeStamp;
	float LastBreakingDataTimeStamp;
	float LastTrailingDataTimeStamp;
	float LastRemovalDataTimeStamp;

	// The list of rigid body solvers, used to retrieve destruction events
	TSet<Chaos::FPhysicsSolver*> Solvers;

	// The list of GeometryCollectionPhysicsProxy, used to retrieve destruction events
	TArray<const FGeometryCollectionPhysicsProxy*> GeometryCollectionPhysicsProxies;

	TSharedPtr<FChaosCollisionEventFilter> ChaosCollisionFilter;
	TSharedPtr<FChaosBreakingEventFilter> ChaosBreakingFilter;
	TSharedPtr<FChaosTrailingEventFilter> ChaosTrailingFilter;
	TSharedPtr<FChaosRemovalEventFilter> ChaosRemovalFilter;

public:
	void SetCollisionFilter(TSharedPtr<FChaosCollisionEventFilter> InCollisionFilter)
	{ 
		ChaosCollisionFilter = InCollisionFilter;
	}

	void SetBreakingFilter(TSharedPtr<FChaosBreakingEventFilter> InBreakingFilter)
	{
		ChaosBreakingFilter = InBreakingFilter;
	}

	void SetTrailingFilter(TSharedPtr<FChaosTrailingEventFilter> InTrailingFilter)
	{
		ChaosTrailingFilter = InTrailingFilter;
	}

	void SetRemovalFilter(TSharedPtr<FChaosRemovalEventFilter> InRemovalFilter)
	{
		ChaosRemovalFilter = InRemovalFilter;
	}

	GEOMETRYCOLLECTIONENGINE_API void RegisterChaosEvents(FPhysScene* Scene);
	GEOMETRYCOLLECTIONENGINE_API void UnregisterChaosEvents(FPhysScene* Scene);

	GEOMETRYCOLLECTIONENGINE_API void RegisterChaosEvents(TSharedPtr<FPhysScene_Chaos> Scene);
	GEOMETRYCOLLECTIONENGINE_API void UnregisterChaosEvents(TSharedPtr<FPhysScene_Chaos> Scene);


	// Chaos Event Handlers
	GEOMETRYCOLLECTIONENGINE_API void HandleCollisionEvents(const Chaos::FCollisionEventData& CollisionData);
	GEOMETRYCOLLECTIONENGINE_API void HandleBreakingEvents(const Chaos::FBreakingEventData& BreakingData);
	GEOMETRYCOLLECTIONENGINE_API void HandleTrailingEvents(const Chaos::FTrailingEventData& TrailingData);
	GEOMETRYCOLLECTIONENGINE_API void HandleRemovalEvents(const Chaos::FRemovalEventData& RemovalData);
};
