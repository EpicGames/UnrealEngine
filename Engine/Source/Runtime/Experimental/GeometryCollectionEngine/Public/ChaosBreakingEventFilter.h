// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ChaosFilter.h"
#include "EventsData.h"
#include "ChaosBreakingEventFilter.generated.h"

// A breaking event data structure. 
USTRUCT(BlueprintType)
struct FChaosBreakingEventData
{
	GENERATED_USTRUCT_BODY()

	// Location of the breaking event (centroid)
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Data")
	FVector Location;

	// The velocity of the breaking event
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Data")
	FVector Velocity;

	// The mass of the breaking event
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Data")
	float Mass;

	FChaosBreakingEventData()
		: Location(FVector::ZeroVector)
		, Velocity(FVector::ZeroVector)
		, Mass(0.0f)
	{
	}
};

// Enumeration defining how to sort breaking results
UENUM(BlueprintType)
enum class EChaosBreakingSortMethod : uint8
{
	SortNone,
	SortByHighestMass,
	SortByHighestSpeed,
	SortByNearestFirst,

	Count UMETA(Hidden)
};

// Settings used to refine breaking event requests
USTRUCT(BlueprintType)
struct FChaosBreakingEventRequestSettings
{
	GENERATED_USTRUCT_BODY()

	/** The maximum number of results to return. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	int32 MaxNumberOfResults;

	/** The minimum breaking radius threshold for the results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MinRadius;

	/** The minimum speed threshold for the results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MinSpeed;

	/** The minimum mass threshold for the results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MinMass;

	/** The maximum distance threshold for the results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MaxDistance;

	/** The method used to sort the breaking events. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sort")
	EChaosBreakingSortMethod SortMethod;

	FChaosBreakingEventRequestSettings()
		: MaxNumberOfResults(0)
		, MinRadius(0.0f)
		, MinSpeed(0.0f)
		, MinMass(0.0f)
		, MaxDistance(0.0f)
		, SortMethod(EChaosBreakingSortMethod::SortByHighestMass)
	{}
};

class FChaosBreakingEventFilter : public IChaosEventFilter<Chaos::FBreakingDataArray, TArray<FChaosBreakingEventData>, EChaosBreakingSortMethod>
{
public:
	FChaosBreakingEventFilter(FChaosBreakingEventRequestSettings* FilterSettingsIn) : BreakingEventRequestSettings(FilterSettingsIn) {}

	GEOMETRYCOLLECTIONENGINE_API virtual void FilterEvents(const FTransform& ChaosComponentTransform, const Chaos::FBreakingDataArray& RawBreakingDataArray) override;

	GEOMETRYCOLLECTIONENGINE_API virtual void SortEvents(TArray<FChaosBreakingEventData>& InOutBreakingEvents, EChaosBreakingSortMethod SortMethod, const FTransform& InTransform) override;

private:
	FChaosBreakingEventFilter() : BreakingEventRequestSettings(nullptr) {}
	const FChaosBreakingEventRequestSettings* BreakingEventRequestSettings;
};
