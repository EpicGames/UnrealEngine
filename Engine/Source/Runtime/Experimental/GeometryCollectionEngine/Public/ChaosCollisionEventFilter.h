// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ChaosFilter.h"
#include "EventsData.h"
#include "ChaosCollisionEventFilter.generated.h"

// A collision event data structure
USTRUCT(BlueprintType)
struct FChaosCollisionEventData
{
	GENERATED_USTRUCT_BODY()

	// Location of the collision event
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Data")
	FVector Location;

	// Normal of the collision event
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Data")
	FVector Normal;

	// The velocity of object 1 of the collision event
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Data")
	FVector Velocity1;

	// The velocity of object 2 of the collision event
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Data")
	FVector Velocity2;

	// The mass of object 1 of the collision event
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Data")
	float Mass1;

	// The mass of object 2 of the collision event
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Data")
	float Mass2;

	// The accumulated impulse vector of the collision event
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Data")
	FVector Impulse;

	FChaosCollisionEventData()
		: Location(FVector::ZeroVector)
		, Normal(FVector::ZeroVector)
		, Velocity1(FVector::ZeroVector)
		, Velocity2(FVector::ZeroVector)
		, Mass1(0.0f)
		, Mass2(0.0f)
		, Impulse(FVector::ZeroVector)
	{
	}
};

// Enumeration defining how to sort collision results
UENUM(BlueprintType)
enum class EChaosCollisionSortMethod : uint8
{
	SortNone,
	SortByHighestMass,
	SortByHighestSpeed,
	SortByHighestImpulse,
	SortByNearestFirst,

	Count UMETA(Hidden)
};

// Settings used to define collision event requests
USTRUCT(BlueprintType)
struct FChaosCollisionEventRequestSettings
{
	GENERATED_USTRUCT_BODY()

	/** The maximum number of results to return. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	int32 MaxNumberResults;

	/** The minimum mass threshold for the results (compared with min of particle 1 mass and particle 2 mass). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MinMass;

	/** The min speed threshold for the results (compared with min of particle 1 speed and particle 2 speed). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MinSpeed;

	/** The minimum impulse threshold for the results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MinImpulse;

	/** The maximum distance threshold for the results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MaxDistance;

	/** The method used to sort the collision events. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sort")
	EChaosCollisionSortMethod SortMethod;

	FChaosCollisionEventRequestSettings()
		: MaxNumberResults(0)
		, MinMass(0.0f)
		, MinSpeed(0.0f)
		, MinImpulse(0.0f)
		, MaxDistance(0.0f)
		, SortMethod(EChaosCollisionSortMethod::SortByHighestMass)
	{}
};

class FChaosCollisionEventFilter : public IChaosEventFilter<Chaos::FCollisionDataArray, TArray<FChaosCollisionEventData>, EChaosCollisionSortMethod>
 {
public:

	FChaosCollisionEventFilter(FChaosCollisionEventRequestSettings* FilterSettingsIn) : CollisionEventRequestSettings(FilterSettingsIn) {}

	GEOMETRYCOLLECTIONENGINE_API virtual void FilterEvents(const FTransform& ChaosComponentTransform, const Chaos::FCollisionDataArray& RawCollisionDataArray) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void SortEvents(TArray<FChaosCollisionEventData>& InOutCollisionEvents, EChaosCollisionSortMethod SortMethod, const FTransform& InTransform) override;

private:
	FChaosCollisionEventFilter() : CollisionEventRequestSettings(nullptr) { check(false);  }
	const FChaosCollisionEventRequestSettings* CollisionEventRequestSettings;
};
