// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

/**
* TChaosEventFilter (template for Chaos Event Filters)
*/
template<class SourceType, class DestinationType, class SortMethodType>
class IChaosEventFilter
{
public:

	virtual ~IChaosEventFilter() = default;

	/**
	* Filters the raw events from the physics system to reduce the number supplied to the game systems
	*/
	virtual void FilterEvents(const FTransform& ChaosComponentTransform, const SourceType& RawInputDataArray) = 0;

	/**
	* Filters the raw events from the physics system to reduce the number supplied to the game systems
	*/
	virtual void FilterEvents(const SourceType& RawInputDataArray) {};

	/**
	* Optionally Sort events based on increasing or decreasing values of FilteredDataArray fields
	*/
	virtual void SortEvents(DestinationType& InOutFilteredEvents, SortMethodType SortMethod, const FTransform& InTransform) = 0;


	/**
	* Gain access to the filtered results
	*/
	const DestinationType& GetFilteredResults() const { return FilteredDataArray; };

	/**
	* Get the number of filtered results
	*/
	int32 GetNumEvents() const { return FilteredDataArray.Num(); }

	void SetTransform(const FTransform& TransformIn)
	{
		Transform = TransformIn;
	}

protected:

	/**
	* The filtered results (a subset of the full raw events buffer)
	*/
	DestinationType FilteredDataArray;

	/**
	 * Filter results relative to a transform location
	 */
	FTransform Transform;
};
