// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ChaosFilter.h"
#include "EventsData.h"
#include "ChaosRemovalEventFilter.generated.h"

// A Removal event data structure. 
USTRUCT(BlueprintType)
struct FChaosRemovalEventData
{
	GENERATED_USTRUCT_BODY()

	/** Current removal location. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	FVector Location;

	/** The mass of the removal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float Mass;

	/** The particle index of the removal. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	int32 ParticleIndex;

	FChaosRemovalEventData()
		: Location(FVector::ZeroVector)
		, Mass(0.0f)
		, ParticleIndex(INDEX_NONE)
	{
	}
};

// Enumeration defining how to sort removal results
UENUM(BlueprintType)
enum class EChaosRemovalSortMethod : uint8
{
	SortNone,
	SortByHighestMass,
	SortByNearestFirst,

	Count UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FChaosRemovalEventRequestSettings
{
	GENERATED_USTRUCT_BODY()

	/** The maximum number of results to return. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	int32 MaxNumberOfResults;

	/** The minimum mass treshold for the results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MinMass;

	/** The maximum distance threshold for the results (if location is set on destruction event listener). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MaxDistance;

	/** The method used to sort the removal events. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sort")
	EChaosRemovalSortMethod SortMethod;

	FChaosRemovalEventRequestSettings()
		: MaxNumberOfResults(0)
		, MinMass(0.0f)
		, MaxDistance(0.0f)
		, SortMethod(EChaosRemovalSortMethod::SortByHighestMass)
	{
	}
};

class FChaosRemovalEventFilter
	: public IChaosEventFilter<Chaos::FRemovalDataArray, TArray<FChaosRemovalEventData>, EChaosRemovalSortMethod>
{
public:
	FChaosRemovalEventFilter(FChaosRemovalEventRequestSettings* FilterSettingsIn) : RemovalEventRequestSettings(FilterSettingsIn) {}

	GEOMETRYCOLLECTIONENGINE_API virtual void FilterEvents(const FTransform& ChaosComponentTransform, const Chaos::FRemovalDataArray& RawRemovalDataArray) override;

	GEOMETRYCOLLECTIONENGINE_API virtual void SortEvents(TArray<FChaosRemovalEventData>& InOutRemovalEvents, EChaosRemovalSortMethod SortMethod, const FTransform& InTransform) override;

private:
	FChaosRemovalEventFilter() : RemovalEventRequestSettings(nullptr) {}
	const FChaosRemovalEventRequestSettings* RemovalEventRequestSettings;
};

