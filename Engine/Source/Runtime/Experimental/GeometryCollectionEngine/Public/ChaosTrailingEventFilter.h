// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ChaosFilter.h"
#include "EventsData.h"
#include "ChaosTrailingEventFilter.generated.h"

// A trailing event data structure. 
USTRUCT(BlueprintType)
struct FChaosTrailingEventData
{
	GENERATED_USTRUCT_BODY()

	/** Current trail location. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	FVector Location;

	/** The velocity of the trail. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	FVector Velocity;

	/** The angular velocity of the trail. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	FVector AngularVelocity;

	/** The mass of the trail. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float Mass;

	/** The particle index of the trail. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	int32 ParticleIndex;

	FChaosTrailingEventData()
		: Location(FVector::ZeroVector)
		, Velocity(FVector::ZeroVector)
		, AngularVelocity(FVector::ZeroVector)
		, Mass(0.0f)
		, ParticleIndex(INDEX_NONE)
	{
	}
};

// Enumeration defining how to sort trailing results
UENUM(BlueprintType)
enum class EChaosTrailingSortMethod : uint8
{
	SortNone,
	SortByHighestMass,
	SortByHighestSpeed,
	SortByNearestFirst,

	Count UMETA(Hidden)
};

USTRUCT(BlueprintType)
struct FChaosTrailingEventRequestSettings
{
	GENERATED_USTRUCT_BODY()

	/** The maximum number of results to return. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	int32 MaxNumberOfResults;

	/** The minimum mass treshold for the results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MinMass;

	/** The minimum speed threshold for the results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MinSpeed;

	/** The minimum angular speed threshold for the results. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MinAngularSpeed;

	/** The maximum distance threshold for the results (if location is set on destruction event listener). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Filters")
	float MaxDistance;

	/** The method used to sort the breaking events. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sort")
	EChaosTrailingSortMethod SortMethod;

	FChaosTrailingEventRequestSettings()
		: MaxNumberOfResults(0)
		, MinMass(0.0f)
		, MinSpeed(0.0f)
		, MinAngularSpeed(0.0f)
		, MaxDistance(0.0f)
		, SortMethod(EChaosTrailingSortMethod::SortByHighestMass)
	{
	}
};

class FChaosTrailingEventFilter 
	: public IChaosEventFilter<Chaos::FTrailingDataArray, TArray<FChaosTrailingEventData>, EChaosTrailingSortMethod>
{
public:
	FChaosTrailingEventFilter(FChaosTrailingEventRequestSettings* FilterSettingsIn) : TrailingEventRequestSettings(FilterSettingsIn) {}

	GEOMETRYCOLLECTIONENGINE_API virtual void FilterEvents(const FTransform& ChaosComponentTransform, const Chaos::FTrailingDataArray& RawTrailingDataArray) override;

	GEOMETRYCOLLECTIONENGINE_API virtual void SortEvents(TArray<FChaosTrailingEventData>& InOutTrailingEvents, EChaosTrailingSortMethod SortMethod, const FTransform& InTransform) override;

private:
	FChaosTrailingEventFilter() : TrailingEventRequestSettings(nullptr) {}
	const FChaosTrailingEventRequestSettings* TrailingEventRequestSettings;
};
