// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if WITH_EDITOR
	#include "DerivedDataPluginInterface.h"
#endif

class UGeometryCollection;

#if WITH_EDITOR

class FDerivedDataGeometryCollectionCooker : public FDerivedDataPluginInterface
{
public:
	FDerivedDataGeometryCollectionCooker(UGeometryCollection& InGeometryCollection);

	virtual const TCHAR* GetPluginName() const override
	{
		return TEXT("GeometryCollection");
	}

	virtual const TCHAR* GetVersionString() const override;
	

	virtual FString GetPluginSpecificCacheKeySuffix() const override;
	

	virtual bool IsBuildThreadsafe() const override
	{
		return false;
	}

	virtual bool IsDeterministic() const override
	{
		return true;
	}

	virtual FString GetDebugContextString() const override;

	virtual bool Build(TArray<uint8>& OutData) override;

	/** Return true if we can build **/
	bool CanBuild() const
	{
		return true;
	}

private:
	UGeometryCollection& GeometryCollection;
};


class FDerivedDataGeometryCollectionRenderDataCooker : public FDerivedDataPluginInterface
{
public:
	FDerivedDataGeometryCollectionRenderDataCooker(UGeometryCollection& InGeometryCollection) : GeometryCollection(InGeometryCollection) {}

	virtual const TCHAR* GetPluginName() const override { return TEXT("GeometryCollectionRenderData"); }
	virtual const TCHAR* GetVersionString() const override;
	virtual FString GetPluginSpecificCacheKeySuffix() const override;
	virtual bool IsBuildThreadsafe() const override { return false;	}
	virtual bool IsDeterministic() const override {	return true; }
	virtual FString GetDebugContextString() const override;
	virtual bool Build(TArray<uint8>& OutData) override;

	bool CanBuild() const { return true; }

	static const TCHAR* VersionString;

private:
	UGeometryCollection& GeometryCollection;
};

#endif	// WITH_EDITOR
