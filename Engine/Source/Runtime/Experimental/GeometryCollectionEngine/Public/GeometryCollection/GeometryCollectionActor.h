// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

/** This class represents an GeometryCollection Actor. */

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"

#include "GeometryCollectionActor.generated.h"


class UGeometryCollectionComponent;
class UGeometryCollectionDebugDrawComponent;
struct FHitResult;

UCLASS(MinimalAPI)
class AGeometryCollectionActor: public AActor
{
	GENERATED_UCLASS_BODY()

public:
#if WITH_EDITOR
	GEOMETRYCOLLECTIONENGINE_API virtual bool GetReferencedContentObjects(TArray<UObject*>& Objects) const override;
#endif

	/* GeometryCollectionComponent */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Destruction, meta = (ExposeFunctionCategories = "Components|GeometryCollection", AllowPrivateAccess = "true"))
	TObjectPtr<UGeometryCollectionComponent> GeometryCollectionComponent;
	UGeometryCollectionComponent* GetGeometryCollectionComponent() const { return GeometryCollectionComponent; }

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY()
	TObjectPtr<UGeometryCollectionDebugDrawComponent> GeometryCollectionDebugDrawComponent_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	UE_DEPRECATED(5.8, "RaycastSingle is now deprecated, use regular Unreal trace system to query geometry collections")
	UFUNCTION(BlueprintCallable, Category = "Physics")
	GEOMETRYCOLLECTIONENGINE_API bool RaycastSingle(FVector Start, FVector End, FHitResult& OutHit) const;

};
