// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"

#include "GeometryCollectionBlueprintLibrary.generated.h"

#define UE_API GEOMETRYCOLLECTIONENGINE_API

/** Blueprint library for Geometry Collections. */
UCLASS(MinimalAPI, meta = (ScriptName = "GeometryCollectionLibrary"))
class UGeometryCollectionBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/** 
	 * Set a custom instance data value for all instances associated with a geometry collection. 
	 * This assumes that the geometry collection is using a custom renderer that supports IGeometryCollectionCustomDataInterface.
	 * @param GeometryCollectionComponent	The Geometry Collection Component that we want to set custom instance data on.
	 * @param CustomDataIndex	The index of the custom instance data slot that we want to set.
	 * @param CustomDataValue	The value to set to the custom instance data slot.
	 */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	static UE_API void SetCustomInstanceDataByIndex(UGeometryCollectionComponent* GeometryCollectionComponent, int32 CustomDataIndex, float CustomDataValue);

	/** 
	 * Set a custom instance data value for all instances associated with a geometry collection. 
	 * This assumes that the geometry collection is using a custom renderer that supports IGeometryCollectionCustomDataInterface.
	 * @param GeometryCollectionComponent	The Geometry Collection Component that we want to set custom instance data on.
	 * @param CustomDataName	The name of the custom instance data slot that we want to set.
	 * @param CustomDataValue	The value to set to the custom instance data slot.
	 */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	static UE_API void SetCustomInstanceDataByName(UGeometryCollectionComponent* GeometryCollectionComponent, FName CustomDataName, float CustomDataValue);

	/** 
	 * Specify custom depth and a value to write during the custom depth pass for all instances associated with a geometry collection. 
	 * This assumes that the geometry collection is using a custom renderer that supports IGeometryCollectionCustomDataInterface.
	 * @param GeometryCollectionComponent	The Geometry Collection Component that we want to set custom depth data on.
	 * @param bEnable	Enable or disable custom depth writing.
	 * @param CustomStencilValue	The value which will be written to the stencil buffer during the custom depth pass if bEnable is true.
	 */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	static UE_API void SetCustomDepthStencil(UGeometryCollectionComponent* GeometryCollectionComponent, bool bEnable, int32 CustomStencilValue);

	UE_DEPRECATED(5.5, "Use SetCustomInstanceDataByIndex() instead")
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics", meta = (DeprecatedFunction, DeprecationMessage = "Use SetCustomInstanceDataByIndex() instead"))
	static UE_API void SetISMPoolCustomInstanceData(UGeometryCollectionComponent* GeometryCollectionComponent, int32 CustomDataIndex, float CustomDataValue);

	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	static UE_API bool AddGeometryCollectionsToBlueprint(UBlueprint* Blueprint, const TArray<UObject*>& Collections);
};

#undef UE_API
 