// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/RecordedTransformTrack.h"

#include "GeometryCollectionCache.generated.h"

class UGeometryCollection;

GEOMETRYCOLLECTIONENGINE_API DECLARE_LOG_CATEGORY_EXTERN(LogGeometryCollectionCache, Log, All);

UCLASS(Experimental, MinimalAPI)
class UGeometryCollectionCache : public UObject
{
	GENERATED_BODY()

public:

	/** Tagnames for asset registry tags */
	static GEOMETRYCOLLECTIONENGINE_API FName TagName_Name;			// Name of the cache
	static GEOMETRYCOLLECTIONENGINE_API FName TagName_IdGuid;		// ID GUID for the cache - never changes for a given cache
	static GEOMETRYCOLLECTIONENGINE_API FName TagName_StateGuid;		// State GUID - changes when an edit is made to 

	/**
	 * Given a raw track with transforms per-particle on each frame record, set to this cache
	 * and strip out any data we don't need (transform repeats and disabled particles etc.)
	 */
	GEOMETRYCOLLECTIONENGINE_API void SetFromRawTrack(const FRecordedTransformTrack& InTrack);

	/** Set directly from a track, does not do any data stripping. */
	GEOMETRYCOLLECTIONENGINE_API void SetFromTrack(const FRecordedTransformTrack& InTrack);

	/** Sets the geometry collection that this cache supports, empties the recorded data in this cache */
	GEOMETRYCOLLECTIONENGINE_API void SetSupportedCollection(const UGeometryCollection* InCollection);

	/** UObject Interface */
	GEOMETRYCOLLECTIONENGINE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	GEOMETRYCOLLECTIONENGINE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	/** End UObject Interface */

	/** Access the recorded tracks */
	const FRecordedTransformTrack* GetData() const { return &RecordedData; }

	/** Given a collection, create an empty compatible cache for it */
	static GEOMETRYCOLLECTIONENGINE_API UGeometryCollectionCache* CreateCacheForCollection(const UGeometryCollection* InCollection);

	/** Get the GUID for the state of the supported collection when this cache was last recorded to. */
	FGuid GetCompatibleStateGuid() const { return CompatibleCollectionState; }

	/** Tests whether a cache is compatible with a provided collection */
	GEOMETRYCOLLECTIONENGINE_API bool CompatibleWithForRecord(const UGeometryCollection* InCollection);
	GEOMETRYCOLLECTIONENGINE_API bool CompatibleWithForPlayback(const UGeometryCollection* InCollection);

private:

	GEOMETRYCOLLECTIONENGINE_API void ProcessRawRecordedDataInternal(const FRecordedTransformTrack& InTrack);

	/** The recorded data from the simulation */
	UPROPERTY()
	FRecordedTransformTrack RecordedData;

	/** The collection that we recorded the data from */
	UPROPERTY()
	TObjectPtr<const UGeometryCollection> SupportedCollection;

	/** Guid pulled from the collection when the recording was last saved */
	UPROPERTY()
	FGuid CompatibleCollectionState;

};

/**
 * Provider for target caches when recording is requested but we don't have a target cache
 * Initial purpose is to allow an opaque way to call some editor system to generate new assets
 * but this could be expanded later for runtime recording and playback if the need arises
 */
class ITargetCacheProvider : public IModularFeature
{
public:
	static FName GetFeatureName() { return "GeometryCollectionTargetCacheProvider"; }
	virtual UGeometryCollectionCache* GetCacheForCollection(const UGeometryCollection* InCollection) = 0;
};
