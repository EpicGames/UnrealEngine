// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MeshComponent.h"
#include "Chaos/Defines.h"
#include "Field/FieldSystem.h"
#include "Field/FieldSystemActor.h"
#include "Field/FieldSystemNodes.h"
#include "GameFramework/Actor.h"
#include "GeometryCollectionEditorSelection.h"
#include "GeometryCollection/GeometryCollection.h"
#include "GeometryCollection/GeometryCollectionDamagePropagationData.h"
#include "GeometryCollectionObject.h"
#include "GeometryCollection/RecordedTransformTrack.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
#include "Templates/UniquePtr.h"
#include "Chaos/ChaosGameplayEventDispatcher.h"
#include "Chaos/ChaosNotifyHandlerInterface.h"
#include "Chaos/ChaosSolverComponentTypes.h"
#include "Chaos/PBDRigidsEvolutionFwd.h"
#include "EngineDefines.h"
#include "Dataflow/Interfaces/DataflowPhysicsObject.h"
#include "Math/MathFwd.h"

#include "GeometryCollectionComponent.generated.h"

struct FGeometryCollectionConstantData;
struct FGeometryCollectionDynamicData;
class FManagedArrayBase;
class UGeometryCollectionComponent;
class UBoxComponent;
class UGeometryCollectionCache;
class UChaosPhysicalMaterial;
class AChaosSolverActor;
struct FGeometryCollectionEmbeddedExemplar;
class UInstancedStaticMeshComponent;
class FGeometryCollectionDecayDynamicFacade;
class FGeometryDynamicCollection;
struct FGeometryCollectionDecayContext;
struct FGeometryCollectionSection;
struct FDamageCollector;
class FPhysScene_Chaos;
class AISMPoolActor;
class IGeometryCollectionExternalRenderInterface;
enum class ESimulationInitializationState : uint8;
enum class EClusterConnectionTypeEnum : uint8;
enum class EInitialVelocityTypeEnum : uint8;
enum class EObjectStateTypeEnum : uint8;
namespace Chaos { enum class EObjectStateType: int8; }
template<class InElementType> class TManagedArray;

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnChaosBreakEvent, const FChaosBreakEvent&, BreakEvent);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnChaosRemovalEvent, const FChaosRemovalEvent&, RemovalEvent);

DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnChaosCrumblingEvent, const FChaosCrumblingEvent&, CrumbleEvent);

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnGeometryCollectionFullyDecayedEvent);

DECLARE_DYNAMIC_MULTICAST_DELEGATE(FOnGeometryCollectionRootMovedEvent);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGeometryCollectionRootMovedNativeEvent, UGeometryCollectionComponent*);

namespace GeometryCollection
{
	enum class ESelectionMode : uint8
	{
		None = 0,
		AllGeometry,
		InverseGeometry,
		Siblings,
		Neighbors,
		Parent,
		Children,
		Level,
		Leaves,
		Clusters
	};
}

USTRUCT()
struct FGeomComponentCacheParameters
{
	GENERATED_BODY()

	FGeomComponentCacheParameters();

	// Cache mode, whether disabled, playing or recording
	UPROPERTY(EditAnywhere, Category = Cache)
	EGeometryCollectionCacheType CacheMode;

	// The cache to target when recording or playing
	UPROPERTY(EditAnywhere, Category = Cache)
	TObjectPtr<UGeometryCollectionCache> TargetCache;

	// Cache mode, whether disabled, playing or recording
	UPROPERTY(EditAnywhere, Category = Cache)
	float ReverseCacheBeginTime;

	// Whether to buffer collisions during recording
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Record Collision Data"))
	bool SaveCollisionData;

	// Whether to generate collisions during playback
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Generate Collision Data during Playback"))
	bool DoGenerateCollisionData;

	// Maximum size of the collision buffer
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Collision Data Size Maximum"))
	int32 CollisionDataSizeMax;

	// Spatial hash collision data
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Spatial Hash Collision Data"))
	bool DoCollisionDataSpatialHash;

	// Spatial hash radius for collision data
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Spatial Hash Radius"))
	float CollisionDataSpatialHashRadius;

	// Maximum number of collisions per cell
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Maximum Number of Collisions Per Cell"))
	int32 MaxCollisionPerCell;

	// Whether to buffer breakings during recording
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Record Breaking Data"))
	bool SaveBreakingData;

	// Whether to generate breakings during playback
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Generate Breaking Data during Playback"))
	bool DoGenerateBreakingData;

	// Maximum size of the breaking buffer
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Breaking Data Size Maximum"))
	int32 BreakingDataSizeMax;

	// Spatial hash breaking data
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Spatial Hash Breaking Data"))
	bool DoBreakingDataSpatialHash;

	// Spatial hash radius for breaking data
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Spatial Hash Radius"))
	float BreakingDataSpatialHashRadius;

	// Maximum number of breaking per cell
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Maximum Number of Breakings Per Cell"))
	int32 MaxBreakingPerCell;

	// Whether to buffer trailings during recording
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Record Trailing Data"))	
	bool SaveTrailingData;

	// Whether to generate trailings during playback
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Generate Trailing Data during Playback"))
	bool DoGenerateTrailingData;

	// Maximum size of the trailing buffer
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Trailing Data Size Maximum"))
	int32 TrailingDataSizeMax;

	// Minimum speed to record trailing
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Trailing Minimum Speed Threshold"))
	float TrailingMinSpeedThreshold;

	// Minimum volume to record trailing
	UPROPERTY(EditAnywhere, Category = Cache, meta = (DisplayName = "Trailing Minimum Volume Threshold"))
	float TrailingMinVolumeThreshold;
};

namespace GeometryCollection
{
	/** Type of updates used at the end of an edit operation. */
	enum class EEditUpdate : uint8
	{
		/** No update. */
		None = 0,
		/** Mark the rest collection as changed. */
		Rest = 1 << 0,
		/** Recreate the physics state (proxy). */
		Physics = 1 << 1,
		/** Reset the dynamic collection. */
		Dynamic = 1 << 2,
		/** Mark the rest collection as changed, and recreate the physics state (proxy). */
		RestPhysics = Rest | Physics,
		/** Reset dynamic collection, mark the rest collection as changed, and recreate the physics state (proxy). */
		RestPhysicsDynamic = Rest | Physics | Dynamic,
	};
	ENUM_CLASS_FLAGS(EEditUpdate);
}

/**
*	FGeometryCollectionEdit
*     Structured RestCollection access where the scope
*     of the object controls serialization back into the
*     dynamic collection
*
*	This will force any simulating geometry collection out of the
*	solver so it can be edited and afterwards will recreate the proxy
*	The update can also be specified to reset the dynamic collection
*/
class FGeometryCollectionEdit
{
public:
	/**
	 * @param InComponent					The component to edit
	 * @param EditUpdate					What parts of the geometry collection to update
	 * @param bShapeIsUnchanged				Override indicating the overall shape of the geometry and clusters is unchanged, even if the rest collection changed.  Useful to e.g., not re-compute convex hulls when we don't need to.
	 * @param bPropagateAcrossComponents	Propagate updates to all components with the same underlying Rest Collection
	 */
	GEOMETRYCOLLECTIONENGINE_API FGeometryCollectionEdit(UGeometryCollectionComponent* InComponent, GeometryCollection::EEditUpdate EditUpdate = GeometryCollection::EEditUpdate::RestPhysicsDynamic, bool bShapeIsUnchanged = false, bool bPropagateToAllMatchingComponents = true);
	GEOMETRYCOLLECTIONENGINE_API ~FGeometryCollectionEdit();

	GEOMETRYCOLLECTIONENGINE_API UGeometryCollection* GetRestCollection();

private:
	UGeometryCollectionComponent* Component;
	const GeometryCollection::EEditUpdate EditUpdate;
	TSet<UGeometryCollectionComponent*> HadPhysicsState;
	bool bShapeIsUnchanged;
	bool bPropagateToAllMatchingComponents;
};

#if WITH_EDITOR
class FScopedColorEdit
{
public:
	GEOMETRYCOLLECTIONENGINE_API FScopedColorEdit(UGeometryCollectionComponent* InComponent, bool bForceUpdate = false);
	GEOMETRYCOLLECTIONENGINE_API ~FScopedColorEdit();

	GEOMETRYCOLLECTIONENGINE_API void SetShowBoneColors(bool ShowBoneColorsIn);
	GEOMETRYCOLLECTIONENGINE_API bool GetShowBoneColors() const;

	GEOMETRYCOLLECTIONENGINE_API void SetEnableBoneSelection(bool ShowSelectedBonesIn);
	GEOMETRYCOLLECTIONENGINE_API bool GetEnableBoneSelection() const;

	GEOMETRYCOLLECTIONENGINE_API bool IsBoneSelected(int BoneIndex) const;
	GEOMETRYCOLLECTIONENGINE_API void Sanitize();
	GEOMETRYCOLLECTIONENGINE_API void SetSelectedBones(const TArray<int32>& SelectedBonesIn);
	GEOMETRYCOLLECTIONENGINE_API void AppendSelectedBones(const TArray<int32>& SelectedBonesIn);
	GEOMETRYCOLLECTIONENGINE_API void ToggleSelectedBones(const TArray<int32>& SelectedBonesIn, bool bAdd, bool bSnapToLevel = true);
	GEOMETRYCOLLECTIONENGINE_API void AddSelectedBone(int32 BoneIndex);
	GEOMETRYCOLLECTIONENGINE_API void ClearSelectedBone(int32 BoneIndex);
	GEOMETRYCOLLECTIONENGINE_API const TArray<int32>& GetSelectedBones() const;
	GEOMETRYCOLLECTIONENGINE_API void ResetBoneSelection();
	GEOMETRYCOLLECTIONENGINE_API void SelectBones(GeometryCollection::ESelectionMode SelectionMode);
	GEOMETRYCOLLECTIONENGINE_API void FilterSelectionToLevel(bool bPreferLowestOnly = false);
	GEOMETRYCOLLECTIONENGINE_API int32 GetMaxSelectedLevel(bool bOnlyRigid) const;
	GEOMETRYCOLLECTIONENGINE_API bool IsSelectionValidAtLevel(int32 TargetLevel) const;

	GEOMETRYCOLLECTIONENGINE_API bool IsBoneHighlighted(int BoneIndex) const;
	GEOMETRYCOLLECTIONENGINE_API void SetHighlightedBones(const TArray<int32>& HighlightedBonesIn, bool bHighlightChildren = false);
	GEOMETRYCOLLECTIONENGINE_API void AddHighlightedBone(int32 BoneIndex);
	GEOMETRYCOLLECTIONENGINE_API const TArray<int32>& GetHighlightedBones() const;
	GEOMETRYCOLLECTIONENGINE_API void ResetHighlightedBones();

	GEOMETRYCOLLECTIONENGINE_API void SetLevelViewMode(int ViewLevel);
	GEOMETRYCOLLECTIONENGINE_API int GetViewLevel();

private:
	void UpdateBoneColors();

	bool bUpdated;


	UGeometryCollectionComponent * Component;
	static TArray<FLinearColor> RandomColors;
};

#endif

//Provides copy on write functionality:
//GetArray (const access)
//GetArrayCopyOnWrite
//GetArrayRest (gives original rest value)
//This generates pointers to arrays marked private. Macro assumes getters are public
//todo(ocohen): may want to take in a static name
#define COPY_ON_WRITE_ATTRIBUTE(Type, Name, Group)											\
	UE_DEPRECATED(5.4, "Use GetGeometryCollection()->"#Name" instead.")						\
	GEOMETRYCOLLECTIONENGINE_API const TManagedArray<Type>& Get##Name##Array() const;		\
	UE_DEPRECATED(5.4, "Use GetGeometryCollection()->"#Name" instead.")						\
	GEOMETRYCOLLECTIONENGINE_API TManagedArray<Type>& Get##Name##ArrayCopyOnWrite();		\
	UE_DEPRECATED(5.4, "Use GetGeometryCollection()->"#Name" instead.")						\
	GEOMETRYCOLLECTIONENGINE_API void Reset##Name##ArrayDynamic();							\
	UE_DEPRECATED(5.4, "Use GetGeometryCollection()->"#Name" instead.")						\
	GEOMETRYCOLLECTIONENGINE_API const TManagedArray<Type>& Get##Name##ArrayRest() const;	\
private:																					\
	/* Deprecated */																		\
	/*TManagedArray<Type>* Indirect##Name##Array;*/											\
public:

/**
 * Raw struct to serialize for network. We need to custom netserialize to optimize
 * the vector serialize as much as possible and rather than have the property system
 * iterate an array of reflected structs we handle everything in the NetSerialize for
 * the container (FGeometryCollectionClusterRep)
 */
struct FGeometryCollectionClusterRep
{
	struct FClusterState
	{
		static constexpr uint8 StateMask = 0b111;
		static constexpr uint8 StateOffset = 0;
		static constexpr uint8 InternalClusterMask = 0b1;
		static constexpr uint8 InternalClusterOffset = 3;

		void SetMaskedValue(uint8 Val, uint8 Mask, uint8 Offset)
		{
			Value &= ~Mask;
			Value |= ((Val & Mask) << Offset);
		}
		
		uint8 GetMaskedValue(uint8 Mask, uint8 Offset) const
		{
			return (Value >> Offset) & Mask;
		}
		
		void SetObjectState(Chaos::EObjectStateType State)
		{
			SetMaskedValue((uint8)State, StateMask, StateOffset);
		}

		void SetInternalCluster(bool bInternalCluster)
		{
			SetMaskedValue(bInternalCluster?1:0, InternalClusterMask, InternalClusterOffset);
		}
		
		Chaos::EObjectStateType GetObjectState()
		{
			return (Chaos::EObjectStateType)GetMaskedValue(StateMask, StateOffset);
		}

		bool IsInternalCluster() const
		{
			return (bool)GetMaskedValue(InternalClusterMask, InternalClusterOffset);
		}
		
		uint8 Value = 0;
	};
	
	FVector_NetQuantize100 Position;
	FVector_NetQuantize100 LinearVelocity;
	FVector_NetQuantize100 AngularVelocity;
	FQuat Rotation;
	uint16 ClusterIdx;		  // index of the cluster or one of its child if the cluster is internal ( see ClusterState)
	FClusterState ClusterState;

	bool ClusterChanged(const FGeometryCollectionClusterRep& Other) const
	{
		return Other.ClusterState.Value != ClusterState.Value
			|| Other.ClusterIdx != ClusterIdx
			|| Other.Position != Position
			|| Other.LinearVelocity != LinearVelocity
			|| Other.AngularVelocity != AngularVelocity
			|| Other.Rotation != Rotation;
	}
};

struct FGeometryCollectionActivatedCluster
{
	FGeometryCollectionActivatedCluster() = default;
	FGeometryCollectionActivatedCluster(uint16 Index, const FVector& InitialLinearVel, const FVector& InitialAngularVel )
		: ActivatedIndex(Index)
		, InitialLinearVelocity(InitialLinearVel)
		, InitialAngularVelocity(InitialAngularVel)
	{}

	uint16 ActivatedIndex;
	FVector_NetQuantize100 InitialLinearVelocity;
	FVector_NetQuantize100 InitialAngularVelocity;

	bool operator==(const FGeometryCollectionActivatedCluster& Other) const
	{
		return ActivatedIndex == Other.ActivatedIndex;
	}
};

inline FArchive& operator<<(FArchive& Ar, FGeometryCollectionActivatedCluster& ActivatedCluster)
{
	Ar << ActivatedCluster.ActivatedIndex;
	Ar << ActivatedCluster.InitialLinearVelocity;
	Ar << ActivatedCluster.InitialAngularVelocity;

	return Ar;
}

/**
 * Replicated data for a geometry collection when bEnableReplication is true for
 * that component. See UGeomtryCollectionComponent::UpdateRepData
 */
USTRUCT()
struct FGeometryCollectionRepData
{
	GENERATED_BODY()

	FGeometryCollectionRepData()
		: Version(0), ServerFrame(0), bIsRootAnchored(false)
	{

	}

	//Array of one off pieces that became activated
	TArray<FGeometryCollectionActivatedCluster> OneOffActivated;

	// Array of cluster data requires to synchronize clients
	TArray<FGeometryCollectionClusterRep> Clusters;

	// Version counter, every write to the rep data is a new state so Identical only references this version
	// as there's no reason to compare the Poses array.
	int32 Version;

	// For Network Prediction Mode we require the frame number on the server when the data was gathered
	int32 ServerFrame;

	// The sim-time that this rep data was received
	TOptional<float> RepDataReceivedTime;

	// Is the root particle of the GC currently anchored
	bool bIsRootAnchored;

	// Just test version to skip having to traverse the whole pose array for replication
	bool Identical(const FGeometryCollectionRepData* Other, uint32 PortFlags) const;
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	// Check if the data has changed
	bool HasChanged(const FGeometryCollectionRepData& BaseData) const;

	void Reset()
	{
		OneOffActivated.Reset();
		Clusters.Reset();
	}

};

template<>
struct TStructOpsTypeTraits<FGeometryCollectionRepData> : public TStructOpsTypeTraitsBase2<FGeometryCollectionRepData>
{
	enum
	{
		WithNetSerializer = true,
		WithIdentical = true,
	};
};

/**
* Replicated state data for a geometry collection when bEnableReplication is true for that component.
* State data means what is broken and what is not 
* See UGeomtryCollectionComponent::UpdateRepData
*/
USTRUCT()
struct FGeometryCollectionRepStateData
{
	GENERATED_BODY()

	FGeometryCollectionRepStateData()
	: Version(0)
	, bIsRootAnchored(false)
	{
	}

	// mark a transform as broken nd return true if this was a state change
	bool SetBroken(int32 TransformIndex, int32 NumTransforms, bool bDisabled, const FVector& LinV, const FVector& AngVInRadiansPerSecond);

	// version for fast comparison
	int32 Version;

	// broken state of each piece of the GC
	TBitArray<> BrokenState;

	// Is the root particle of the GC currently anchored
	// could possibily change in the future to also be a bit array 
	uint8 bIsRootAnchored;

	// this represents the data for when a particle is released from its parent cluster 
	// this data is added when the particle is released but will be cleared after a while
	// so that late client will not replay the break as it is in their past 
	struct FReleasedData
	{
		int16 TransformIndex;
		FVector_NetQuantize10 LinearVelocity;
		FVector_NetQuantize10 AngularVelocityInDegreesPerSecond;
	};
	TArray<FReleasedData> ReleasedData;

	// Just test version to skip having to traverse the whole pose array for replication
	bool Identical(const FGeometryCollectionRepStateData* Other, uint32 PortFlags) const;
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	// Check if the data has changed
	bool HasChanged(const FGeometryCollectionRepStateData& BaseData) const;

	void Reset()
	{
		BrokenState.Reset();
		bIsRootAnchored = 0;
		ReleasedData.Reset();
	}
};

template<>
struct TStructOpsTypeTraits<FGeometryCollectionRepStateData> : public TStructOpsTypeTraitsBase2<FGeometryCollectionRepStateData>
{
	enum
	{
		WithNetSerializer = true,
		WithIdentical = true,
	};
};

// this structure holds entries for the tracked pieces to be replicated
USTRUCT()
struct FGeometryCollectionRepDynamicData
{
	GENERATED_BODY()

	struct FClusterData
	{
		FVector_NetQuantize10 Position;
		FVector_NetQuantize10 EulerRotation;
		FVector_NetQuantize10 LinearVelocity;
		FVector_NetQuantize10 AngularVelocityInDegreesPerSecond;

		// Index of the cluster or one of its child if the cluster is internal ( see bIsInternalCluster)
		uint16 TransformIndex = INDEX_NONE; 

		// Whether this refers to an internal cluster or directly to a cluster in the geometry collection
		uint8  bIsInternalCluster = false;

		// non serialized data, used to trimn the data back when no longer updated
		int32 LastUpdatedVersion = 0;

		// comp
		bool IsEqualPositionsAndVelocities(const FClusterData& Data) const;
	};

	FGeometryCollectionRepDynamicData()
		: Version(0)
	{}

	int32 Version;
	TArray<FClusterData> ClusterData;

	// return true if the data has changed from stored one
	bool SetData(const FClusterData& Data);

	// return true if any entries was removed
	bool RemoveOutOfDateClusterData();

	// Just test version to skip having to traverse the whole pose array for replication
	bool Identical(const FGeometryCollectionRepDynamicData* Other, uint32 PortFlags) const;
	bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	// Check if the data has changed
	bool HasChanged(const FGeometryCollectionRepDynamicData& BaseData) const;

	void Reset()
	{
		ClusterData.Reset();
	}
};

template<>
struct TStructOpsTypeTraits<FGeometryCollectionRepDynamicData> : public TStructOpsTypeTraitsBase2<FGeometryCollectionRepDynamicData>
{
	enum
	{
		WithNetSerializer = true,
		WithIdentical = true,
	};
};

struct FGCCollisionProfileScopedTransaction;

/**
*	GeometryCollectionComponent
*/
UCLASS(meta = (BlueprintSpawnableComponent), MinimalAPI)
class UGeometryCollectionComponent : public UMeshComponent, public IChaosNotifyHandlerInterface
{
	GENERATED_UCLASS_BODY()
	friend class FGeometryCollectionEdit;
#if WITH_EDITOR
	friend class FScopedColorEdit;
#endif
	friend class FGeometryCollectionCommands;
	friend class FGeometryCollectionSceneProxyBase;

public:

	GEOMETRYCOLLECTIONENGINE_API virtual ~UGeometryCollectionComponent();

	// Collision profile name that indicates we should use the geometry collection's default collision profile.
	GEOMETRYCOLLECTIONENGINE_API static FName DefaultCollisionProfileName;

	//~ Begin UObject Interface.
	GEOMETRYCOLLECTIONENGINE_API virtual void Serialize(FArchive& Ar) override;
	//~ End UObject Interface.

	//~ Begin UActorComponent Interface.
	GEOMETRYCOLLECTIONENGINE_API virtual bool ShouldCreateRenderState() const override;
	GEOMETRYCOLLECTIONENGINE_API virtual void CreateRenderState_Concurrent(FRegisterComponentContext* Context) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void SendRenderDynamicData_Concurrent() override;

	UE_DEPRECATED(5.6, "Render state dirty state is now based on the geometry collection change events")
	FORCEINLINE void SetRenderStateDirty() { /* Deprecated. */ }

	GEOMETRYCOLLECTIONENGINE_API virtual void SetCollisionObjectType(ECollisionChannel Channel) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void OnActorEnableCollisionChanged() override;
	GEOMETRYCOLLECTIONENGINE_API virtual void BeginPlay() override;
	GEOMETRYCOLLECTIONENGINE_API virtual void EndPlay(const EEndPlayReason::Type ReasonEnd) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void OnVisibilityChanged() override;
	GEOMETRYCOLLECTIONENGINE_API virtual void OnActorVisibilityChanged() override;
	GEOMETRYCOLLECTIONENGINE_API virtual void OnHiddenInGameChanged() override;
	GEOMETRYCOLLECTIONENGINE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	GEOMETRYCOLLECTIONENGINE_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;

#if WITH_EDITOR
	GEOMETRYCOLLECTIONENGINE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	DECLARE_MULTICAST_DELEGATE(FOnGeometryCollectionPropertyChangedMulticaster)
	FOnGeometryCollectionPropertyChangedMulticaster OnGeometryCollectionPropertyChanged;
	typedef FOnGeometryCollectionPropertyChangedMulticaster::FDelegate FOnGeometryCollectionPropertyChanged;

	/** Register / Unregister delegates called when the skeletal mesh property is changed */
	GEOMETRYCOLLECTIONENGINE_API FDelegateHandle RegisterOnGeometryCollectionPropertyChanged(const FOnGeometryCollectionPropertyChanged& Delegate);
	GEOMETRYCOLLECTIONENGINE_API void UnregisterOnGeometryCollectionPropertyChanged(FDelegateHandle Handle);
#endif
	//~ End UActorComponent Interface. 

	//~ Begin INavRelevantInterface Interface
	GEOMETRYCOLLECTIONENGINE_API virtual bool IsNavigationRelevant() const override;
	//~ End INavRelevantInterface Interface
	
	//~ Begin USceneComponent Interface.
	GEOMETRYCOLLECTIONENGINE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	virtual FBoxSphereBounds CalcLocalBounds() const { return ComponentSpaceBounds; }

	GEOMETRYCOLLECTIONENGINE_API virtual bool HasAnySockets() const override;
	GEOMETRYCOLLECTIONENGINE_API virtual bool DoesSocketExist(FName InSocketName) const override;
	GEOMETRYCOLLECTIONENGINE_API virtual FTransform GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace = RTS_World) const override;
	GEOMETRYCOLLECTIONENGINE_API virtual void QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const override;
	
	GEOMETRYCOLLECTIONENGINE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction *ThisTickFunction) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void AsyncPhysicsTickComponent(float DeltaTime, float SimTime) override;
	//~ End USceneComponent Interface.


	//~ Begin UPrimitiveComponent Interface.
public:
	GEOMETRYCOLLECTIONENGINE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	GEOMETRYCOLLECTIONENGINE_API virtual void OnRegister() override;
	GEOMETRYCOLLECTIONENGINE_API virtual void OnUnregister() override;
	GEOMETRYCOLLECTIONENGINE_API virtual FBodyInstance* GetBodyInstance(FName BoneName = NAME_None, bool bGetWelded = true, int32 Index = -1) const override;
	GEOMETRYCOLLECTIONENGINE_API virtual void SetNotifyRigidBodyCollision(bool bNewNotifyRigidBodyCollision) override;
	GEOMETRYCOLLECTIONENGINE_API virtual bool CanEditSimulatePhysics() override;
	GEOMETRYCOLLECTIONENGINE_API virtual void SetSimulatePhysics(bool bEnabled) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void SetEnableGravity(bool bGravityEnabled) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void AddForce(FVector Force, FName BoneName = NAME_None, bool bAccelChange = false) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void AddForceAtLocation(FVector Force, FVector WorldLocation, FName BoneName = NAME_None) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void AddImpulse(FVector Impulse, FName BoneName = NAME_None, bool bVelChange = false) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void AddImpulseAtLocation(FVector Impulse, FVector WorldLocation, FName BoneName = NAME_None) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void AddRadialForce(FVector Origin, float Radius, float Strength, ERadialImpulseFalloff Falloff, bool bAccelChange = false) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void AddRadialImpulse(FVector Origin, float Radius, float Strength, enum ERadialImpulseFalloff Falloff, bool bVelChange = false) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void AddTorqueInRadians(FVector Torque, FName BoneName = NAME_None, bool bAccelChange = false) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void PostLoad() override;
	GEOMETRYCOLLECTIONENGINE_API virtual void SetPhysMaterialOverride(UPhysicalMaterial* NewPhysMaterial) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void SetLinearDamping(float InDamping) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void SetAngularDamping(float InDamping) override;

protected:
	GEOMETRYCOLLECTIONENGINE_API virtual void OnComponentCollisionSettingsChanged(bool bUpdateOverlaps=true) override;
	GEOMETRYCOLLECTIONENGINE_API virtual bool CanBeUsedInPhysicsReplication(const FName BoneName = NAME_None) const override;
	//~ End UPrimitiveComponent Interface.

	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void SetLinearDampingPerTransform(const TArray<int32>& TransformIndices, float InDamping);

	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void SetAngularDampingPerTransform(const TArray<int32>& TransformIndices, float InDamping);

	//~ Begin UMeshComponent Interface.	
public:
	GEOMETRYCOLLECTIONENGINE_API virtual int32 GetNumMaterials() const override;
	GEOMETRYCOLLECTIONENGINE_API virtual UMaterialInterface* GetMaterial(int32 MaterialIndex) const override;
	GEOMETRYCOLLECTIONENGINE_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;
	UE_DEPRECATED(5.7, "Please use GetMaterialRelevance with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	GEOMETRYCOLLECTIONENGINE_API virtual FMaterialRelevance GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const override;
	GEOMETRYCOLLECTIONENGINE_API virtual FMaterialRelevance GetMaterialRelevance(EShaderPlatform InShaderPlatform) const override;
	//~ End UMeshComponent Interface.

	/** Chaos RBD Solver override. Will use the world's default solver actor if null. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintGetter=GetSolverActor, BlueprintSetter=SetSolverActor, Category = "ChaosPhysics", meta = (DisplayName = "Chaos Solver"))
	TObjectPtr<AChaosSolverActor> ChaosSolverActor;

	

	/**
	* Get local bounds of the geometry collection
	*/
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	FBox GetLocalBounds() const { return ComponentSpaceBounds; }

	/**
	 * Apply an external strain to specific piece of the geometry collection
	 * @param ItemIndex item index ( from HitResult) of the piece to apply strain on
	 * @param Location world location of where to apply the strain
	 * @param Radius radius from the location point to apply the strain to ( using the center of mass of the pieces )
	 * @param PropagationDepth How many level of connection to follow to propagate the strain through
	 * @param PropagationFactor when using propagation, the factor to multiply the strain from one level to the other, allowing falloff effect
	 * @param Strain strain / damage to apply 
	 */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void ApplyExternalStrain(int32 ItemIndex, const FVector& Location, float Radius = 0, int32 PropagationDepth = 0, float PropagationFactor = 1, float Strain = 0);

	/**
	 * Apply an internal strain to specific piece of the geometry collection
	 * @param ItemIndex item index ( from HitResult) of the piece to apply strain on
	 * @param Location world location of where to apply the strain
	 * @param Radius radius from the location point to apply the strain to ( using the center of mass of the pieces )
	 * @param PropagationDepth How many level of connection to follow to propagate the strain through
	 * @param PropagationFactor when using propagation, the factor to multiply the strain from one level to the other, allowing falloff effect
	 * @param Strain strain / damage to apply 
	 */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void ApplyInternalStrain(int32 ItemIndex, const FVector& Location, float Radius = 0, int32 PropagationDepth = 0, float PropagationFactor = 1, float Strain = 0);
	
	/**
	 * Crumbe a cluster into all its pieces
	 * @param ItemIndex item index ( from HitResult) of the cluster to crumble
	 */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void CrumbleCluster(int32 ItemIndex);

	/**
	* Crumbe active clusters for this entire geometry collection
	* this will apply to internal and regular clusters
	*/
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void CrumbleActiveClusters();

	/**
	* Set a piece or cluster to be anchored or not 
	*/
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void SetAnchoredByIndex(int32 Index, bool bAnchored);

	/**
	* Set all pieces within a world space bounding box to be anchored or not 
	*/
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void SetAnchoredByBox(FBox WorldSpaceBox, bool bAnchored, int32 MaxLevel = -1);

	/**
	* Set all pieces within a world transformed bounding box to be anchored or not
	*/
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void SetAnchoredByTransformedBox(FBox Box, FTransform Transform, bool bAnchored, int32 MaxLevel = -1);

	/**
	* this will remove anchors on all the pieces ( including the static and kinematic initial states ones ) of the geometry colection
	*/
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void RemoveAllAnchors();

	/**
	 * Apply linear velocity on breaking pieces for a specific cluster
	 * If ItemIndex does not represent a cluster this will do nothing  
	 * @param ItemIndex item index ( from HitResult) of the cluster owning the breaking pieces to apply velocity on
	 * @param LinearVelocity linear velocity to apply
	 */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void ApplyBreakingLinearVelocity(int32 ItemIndex, const FVector& LinearVelocity);

	/**
	 * Apply linear velocity on breaking pieces for a specific cluster
	 * If ItemIndex does not represent a cluster this will do nothing  
	 * @param ItemIndex item index ( from HitResult) of the cluster owning the breaking pieces to apply velocity on
	 * @param AngularVelocity linear velocity to apply
	 */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void ApplyBreakingAngularVelocity(int32 ItemIndex, const FVector& AngularVelocity);
	
	/**
	 * Apply linear velocity on specific piece 
	 * @param ItemIndex item index ( from HitResult) of the piece to apply velocity on
	* @param LinearVelocity linear velocity to apply
	 */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void ApplyLinearVelocity(int32 ItemIndex, const FVector& LinearVelocity);

	/**
	 * Apply angular velocity on specific piece 
	 * @param ItemIndex item index ( from HitResult) of the piece to apply velocity on
	* @param AngularVelocity linear velocity to apply
	 */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void ApplyAngularVelocity(int32 ItemIndex, const FVector& AngularVelocity);

	/**
	 * Get the initial level of a specific piece
	 * Initial level means the level as it is in the unbroken state 
	 * @param ItemIndex item index ( from HitResult) of the cluster to get level from
	 * @param Level of the piece ( 0 for root level and positive for the rest ) 
	 */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API int32 GetInitialLevel(int32 ItemIndex);
	
	/** Get the root item index of the hierarchy */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API int32 GetRootIndex() const;

	/** Get the root item initial transform in world space */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API FTransform GetRootInitialTransform() const;

	/** Get the root item current world transform */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API FTransform GetRootCurrentTransform() const;

	GEOMETRYCOLLECTIONENGINE_API FTransform GetRootCurrentComponentSpaceTransform() const;

	GEOMETRYCOLLECTIONENGINE_API FTransform GetRootParticleMassOffset() const;

	/** return true if the root cluster is not longer active at runtime */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	bool IsRootBroken() const { return BrokenAndDecayedStates.GetIsRootBroken(); }

	/** 
	* Get the initial rest transforms in component (local) space  space, 
	* they are the transforms as defined in the rest collection asset 
	*/
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API TArray<FTransform> GetInitialLocalRestTransforms() const;

	/**
	* Get the rest transforms in component (local) space  space,
	* if none have been set by SetLocalRestTransforms or if RestTransform property is empty , then the initial ones are returned
	* @param bInitialTransform force the initial transforms to be returned ( false by default )
	*/
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API TArray<FTransform> GetLocalRestTransforms(bool bInitialTransforms = false) const;

	/** 
	* Set the local rest transform, this may be different from the rest collection 
	* If the geometry collection is already simulating those matrices will be overriden by the physics state updates
	*/
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void SetLocalRestTransforms(const TArray<FTransform>& Transforms, bool bOnlyLeaves);

	/**
	* Get mass and extent of a specific piece
	* @param ItemIndex item index ( from HitResult) of the cluster to get level from
	* @param Level of the piece ( 0 for root level and positive for the rest )
	*/
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void GetMassAndExtents(int32 ItemIndex, float& OutMass, FBox& OutExtents);

	/** Returns the mass of this component in kg. */
	GEOMETRYCOLLECTIONENGINE_API virtual float GetMass() const override;

	/** Returns the calculated mass in kg. This is not 100% exactly the mass physx will calculate, but it is very close ( difference < 0.1kg ). */
	GEOMETRYCOLLECTIONENGINE_API virtual float CalculateMass(FName BoneName = NAME_None) override;

	/** RestCollection */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void SetRestCollection(const UGeometryCollection * RestCollectionIn, bool bApplyAssetDefaults = true);

	/** Reset the state of the component : object will be rest to be unbroken and physics will reset accordingly */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void ResetState();

	/** RestCollection */
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API FString GetDebugInfo();

	inline const UGeometryCollection* GetRestCollection() const { return RestCollection; }

	inline FGeometryCollectionEdit EditRestCollection(GeometryCollection::EEditUpdate EditUpdate = GeometryCollection::EEditUpdate::RestPhysicsDynamic, bool bShapeIsUnchanged = false) { return FGeometryCollectionEdit(this, EditUpdate, bShapeIsUnchanged); }
#if WITH_EDITOR
	inline FScopedColorEdit EditBoneSelection(bool bForceUpdate = false) { return FScopedColorEdit(this, bForceUpdate); }

	/** Propagate bone selection to embedded geometry components. */
	GEOMETRYCOLLECTIONENGINE_API void SelectEmbeddedGeometry();
#endif

	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void SetEnableDamageFromCollision(bool bValue);


	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void SetAbandonedParticleCollisionProfileName(FName CollisionProfile);

	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void SetPerLevelCollisionProfileNames(const TArray<FName>& ProfileNames);

	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API void SetPerParticleCollisionProfileName(const TArray<int32>& BoneIds, FName ProfileName);

	GEOMETRYCOLLECTIONENGINE_API void SetPerParticleCollisionProfileName(const TSet<int32>& BoneIds, FName ProfileName);

	GEOMETRYCOLLECTIONENGINE_API void SetParticleCollisionProfileName(int32 BoneId, FName ProfileName, FGCCollisionProfileScopedTransaction& InProfileNameUpdateTransaction);

private:

	GEOMETRYCOLLECTIONENGINE_API bool UpdatePerParticleCollisionProfilesNum();

public:

	/** API for getting at geometry collection data */
	GEOMETRYCOLLECTIONENGINE_API int32 GetNumElements(FName Group) const;

	// Update cached bounds; used e.g. when updating the exploded view of the geometry collection
	GEOMETRYCOLLECTIONENGINE_API void UpdateCachedBounds();

#define COPY_ON_WRITE_ATTRIBUTES \
	/* Vertices Group */ \
	COPY_ON_WRITE_ATTRIBUTE(FVector3f, Vertex, FGeometryCollection::VerticesGroup)		/* GetVertexArray, GetVertexArrayCopyOnWrite, GetVertexArrayRest */ \
	COPY_ON_WRITE_ATTRIBUTE(FLinearColor, Color, FGeometryCollection::VerticesGroup)	/* GetColorArray */		\
	COPY_ON_WRITE_ATTRIBUTE(FVector3f, TangentU, FGeometryCollection::VerticesGroup)	/* GetTangentUArray */	\
	COPY_ON_WRITE_ATTRIBUTE(FVector3f, TangentV, FGeometryCollection::VerticesGroup)	/* //... */		\
	COPY_ON_WRITE_ATTRIBUTE(FVector3f, Normal, FGeometryCollection::VerticesGroup)						\
	COPY_ON_WRITE_ATTRIBUTE(int32, BoneMap, FGeometryCollection::VerticesGroup)							\
																										\
	/* Faces Group */																					\
	COPY_ON_WRITE_ATTRIBUTE(FIntVector, Indices, FGeometryCollection::FacesGroup)						\
	COPY_ON_WRITE_ATTRIBUTE(bool, Visible, FGeometryCollection::FacesGroup)								\
	COPY_ON_WRITE_ATTRIBUTE(int32, MaterialIndex, FGeometryCollection::FacesGroup)						\
	COPY_ON_WRITE_ATTRIBUTE(int32, MaterialID, FGeometryCollection::FacesGroup)							\
	COPY_ON_WRITE_ATTRIBUTE(bool, Internal, FGeometryCollection::FacesGroup)							\
																										\
	/* Geometry Group */																				\
	COPY_ON_WRITE_ATTRIBUTE(int32, TransformIndex, FGeometryCollection::GeometryGroup)					\
	COPY_ON_WRITE_ATTRIBUTE(FBox, BoundingBox, FGeometryCollection::GeometryGroup)						\
	COPY_ON_WRITE_ATTRIBUTE(float, InnerRadius, FGeometryCollection::GeometryGroup)						\
	COPY_ON_WRITE_ATTRIBUTE(float, OuterRadius, FGeometryCollection::GeometryGroup)						\
	COPY_ON_WRITE_ATTRIBUTE(int32, VertexStart, FGeometryCollection::GeometryGroup)						\
	COPY_ON_WRITE_ATTRIBUTE(int32, VertexCount, FGeometryCollection::GeometryGroup)						\
	COPY_ON_WRITE_ATTRIBUTE(int32, FaceStart, FGeometryCollection::GeometryGroup)						\
	COPY_ON_WRITE_ATTRIBUTE(int32, FaceCount, FGeometryCollection::GeometryGroup)						\
																										\
	/* Material Group */																				\
	COPY_ON_WRITE_ATTRIBUTE(FGeometryCollectionSection, Sections, FGeometryCollection::MaterialGroup)	\
																										\
	/* Transform group */																				\
	COPY_ON_WRITE_ATTRIBUTE(FString, BoneName, FTransformCollection::TransformGroup)					\
	COPY_ON_WRITE_ATTRIBUTE(FLinearColor, BoneColor, FTransformCollection::TransformGroup)				\
	COPY_ON_WRITE_ATTRIBUTE(int32, TransformToGeometryIndex, FTransformCollection::TransformGroup)		\
	COPY_ON_WRITE_ATTRIBUTE(int32, ExemplarIndex, FTransformCollection::TransformGroup)					\

	// Declare all the methods
	COPY_ON_WRITE_ATTRIBUTES

	GEOMETRYCOLLECTIONENGINE_API int32 GetParent(int32 Index) const;
	GEOMETRYCOLLECTIONENGINE_API const TManagedArray<int32>& GetParentArrayRest() const;

	UPROPERTY(EditAnywhere, NoClear, BlueprintReadOnly, Category = "ChaosPhysics")
	TObjectPtr<const UGeometryCollection> RestCollection;

	/** Apply default values from asset ( damage related data and physics material ) */
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "ChaosPhysics", meta = (EditCondition = "RestCollection != nullptr"))
	GEOMETRYCOLLECTIONENGINE_API void ApplyAssetDefaults();

	UPROPERTY(EditAnywhere, NoClear, BlueprintReadOnly, Category = "ChaosPhysics")
	TArray<TObjectPtr<const AFieldSystemActor>> InitializationFields;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "GeometryCollection now abides the bSimulatePhysics flag from the base class."))
	bool Simulating_DEPRECATED;

	ESimulationInitializationState InitializationState;

	/** ObjectType defines how to initialize the rigid objects state, Kinematic, Sleeping, Dynamic. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|General")
	EObjectStateTypeEnum ObjectType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetGravityGroupIndex, Category = "ChaosPhysics|General")
	int32 GravityGroupIndex;

	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	GEOMETRYCOLLECTIONENGINE_API void SetGravityGroupIndex(int32 InGravityGroupIndex);

	// All bodies with a level greater than or equal to this will have One-Way Interaction enabled and act like debris (will not apply forces to non-debris bodies)
	// Set to -1 to disable (no bodies will have One-Way Interaction enabled)
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetOneWayInteractionLevel, Category = "ChaosPhysics|General")
	int32 OneWayInteractionLevel;

	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	GEOMETRYCOLLECTIONENGINE_API void SetOneWayInteractionLevel(int32 InOneWayInteractionLevel);

	/** when true, density will be used to compute mass using the assigned physics material */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetDensityFromPhysicsMaterial, Category = "ChaosPhysics|General")
	bool bDensityFromPhysicsMaterial;

	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	GEOMETRYCOLLECTIONENGINE_API void SetDensityFromPhysicsMaterial(bool bInDensityFromPhysicsMaterial);

	/** If ForceMotionBlur is on, motion blur will always be active, even if the GeometryCollection is at rest. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|General")
	bool bForceMotionBlur;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Clustering")
	bool EnableClustering;

	/** Cluster group index - Geometry Collections using the same index will simulate as one single cluster. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Clustering")
	int32 ClusterGroupIndex;

	/** Maximum level for cluster breaks. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Clustering")
	int32 MaxClusterLevel;

	/** The maximum level to create rigid bodies that could be simulated.
	Example: If we have a Geometry Collection with 2 levels, where:
	0 = Root
	1 = Clusters
	2 = Leaf Nodes
	A setting of '1' would only generate a physics representation of the Root transform and Level 1 clusters. 
	The leaf nodes on Level 2 would never be created on the solver, and would therefore never be considered as part of the simulation. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Clustering")
	int32 MaxSimulatedLevel;

	/** Damage model to use for evaluating destruction. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetDamageModel, Category = "ChaosPhysics|Damage")
	EDamageModelTypeEnum DamageModel;

	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	GEOMETRYCOLLECTIONENGINE_API void SetDamageModel(EDamageModelTypeEnum InDamageModel);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintGetter=GetDamageThreshold, BlueprintSetter=SetDamageThreshold, Category = "ChaosPhysics|Damage", meta = (EditCondition = "!bUseSizeSpecificDamageThreshold && DamageModel == EDamageModelTypeEnum::Chaos_Damage_Model_UserDefined_Damage_Threshold"))
	TArray<float> DamageThreshold;

	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly)
	TArray<float> GetDamageThreshold() const { return DamageThreshold; }

	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	GEOMETRYCOLLECTIONENGINE_API void SetDamageThreshold(const TArray<float>& InDamageThreshold);

	/** Damage threshold for clusters at different levels. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Damage", meta = (EditCondition = "DamageModel == EDamageModelTypeEnum::Chaos_Damage_Model_UserDefined_Damage_Threshold"))
	bool bUseSizeSpecificDamageThreshold;

	/** When on , use the modifiers on the material to adjust the user defined damage threshold values */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetUseMaterialDamageModifiers, Category = "ChaosPhysics|Damage", meta = (EditCondition = "DamageModel == EDamageModelTypeEnum::Chaos_Damage_Model_UserDefined_Damage_Threshold"))
	bool bUseMaterialDamageModifiers;

	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	GEOMETRYCOLLECTIONENGINE_API void SetUseMaterialDamageModifiers(bool bInUseMaterialDamageModifiers);

	/** Data about how damage propagation shoudl behave. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter = SetDamagePropagationData, Category = "ChaosPhysics|Damage")
	FGeometryCollectionDamagePropagationData DamagePropagationData;

	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	GEOMETRYCOLLECTIONENGINE_API void SetDamagePropagationData(const FGeometryCollectionDamagePropagationData& InDamagePropagationData);

	/** Whether or not collisions against this geometry collection will apply strain which could cause the geometry collection to fracture. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetEnableDamageFromCollision, Category = "ChaosPhysics|Damage")
	bool bEnableDamageFromCollision;

	/** Allow removal on sleep for the instance if the rest collection has it enabled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Removal")
	bool bAllowRemovalOnSleep;

	/** Allow removal on break for the instance if the rest collection has it enabled */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Removal")
	bool bAllowRemovalOnBreak;
	
	/** Update transforms of active particles even when they are not moving. Has performance implications. Use only when GC is a child of a moving actor, to prevent released particle 'following the actor around' */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Clustering")
	bool bForceUpdateActiveTransforms;

	/** */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Connection types are defined on the asset now."))
	EClusterConnectionTypeEnum ClusterConnectionType_DEPRECATED;

	/** */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Collisions")
	int32 CollisionGroup;

	/** Fraction of collision sample particles to keep */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Collisions")
	float CollisionSampleFraction;

	/** Uniform linear ether drag. */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use PhysicalMaterial instead."))
	float LinearEtherDrag_DEPRECATED;

	/** Physical Properties */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Physical material now derived from render materials, for instance overrides use PhysicalMaterialOverride."))
	TObjectPtr<const UChaosPhysicalMaterial> PhysicalMaterial_DEPRECATED;

	/** */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Initial Velocity")
	EInitialVelocityTypeEnum InitialVelocityType;

	/** */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Initial Velocity")
	FVector InitialLinearVelocity;

	/** */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Initial Velocity")
	FVector InitialAngularVelocity;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Physical material now derived from render materials, for instance overrides use Colliisons PhysicalMaterialOverride."))
	TObjectPtr<UPhysicalMaterial> PhysicalMaterialOverride_DEPRECATED;

	UPROPERTY()
	FGeomComponentCacheParameters CacheParameters;

	/** Optional transforms to initialize scene proxy if difference from the RestCollection. */
	UPROPERTY()
	TArray<FTransform> RestTransforms;

	/**
	*  SetDynamicState
	*    This function will dispatch a command to the physics thread to apply
	*    a kinematic to dynamic state change for the geo collection particles within the field.
	*
	*	 @param Radius Radial influence from the position
	*    @param Position The location of the command
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "Field", DisplayName = "Set Dynamic State")
	GEOMETRYCOLLECTIONENGINE_API void ApplyKinematicField(UPARAM(DisplayName = "Field Radius") float Radius, 
							 UPARAM(DisplayName = "Center Position") FVector Position);

	/**
	*  AddPhysicsField
	*    This function will dispatch a command to the physics thread to apply
	*    a generic evaluation of a user defined transient field network. See documentation,
	*    for examples of how to recreate variations of the above generic
	*    fields using field networks
	*
	*	 @param Enabled Is this force enabled for evaluation.
	*    @param Target Type of field supported by the solver.
	*    @param MetaData Meta data used to assist in evaluation
	*    @param Field Base evaluation node for the field network.
	*
	*/
	UFUNCTION(BlueprintCallable, Category = "Field", DisplayName = "Add Physics Field")
	GEOMETRYCOLLECTIONENGINE_API void ApplyPhysicsField(UPARAM(DisplayName = "Enable Field") bool Enabled, 
						   UPARAM(DisplayName = "Physics Type") EGeometryCollectionPhysicsTypeEnum Target, 
						   UPARAM(DisplayName = "Meta Data") UFieldSystemMetaData* MetaData, 
						   UPARAM(DisplayName = "Field Node") UFieldNodeBase* Field);

	/**
	* Blueprint event
	*/

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FNotifyGeometryCollectionPhysicsStateChange, UGeometryCollectionComponent*, FracturedComponent);

	UPROPERTY(BlueprintAssignable, Category = "Game|Damage")
	FNotifyGeometryCollectionPhysicsStateChange NotifyGeometryCollectionPhysicsStateChange;

	GEOMETRYCOLLECTIONENGINE_API bool GetIsObjectDynamic() const;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FNotifyGeometryCollectionPhysicsLoadingStateChange, UGeometryCollectionComponent*, FracturedComponent);
	UPROPERTY(BlueprintAssignable, Category = "Game|Loading")
	FNotifyGeometryCollectionPhysicsLoadingStateChange NotifyGeometryCollectionPhysicsLoadingStateChange;
	
	bool GetIsObjectLoading() { return IsObjectLoading; }

	/**
	*
	*/
	/* ---------------------------------------------------------------------------------------- */
	
	bool GetNotifyTrailing() const { return bNotifyTrailing; }

	void SetShowBoneColors(bool ShowBoneColorsIn) { bShowBoneColors = ShowBoneColorsIn; }
	bool GetShowBoneColors() const { return bShowBoneColors; }
	bool GetEnableBoneSelection() const { return bEnableBoneSelection; }
	
	GEOMETRYCOLLECTIONENGINE_API bool GetSuppressSelectionMaterial() const;
	
	UE_DEPRECATED(5.5, "Bone Selected Material does not have a material ID anymore.")
	const int GetBoneSelectedMaterialID() const
	{
		return INDEX_NONE;
	}
	
#if WITH_EDITORONLY_DATA
	inline const TArray<int32>& GetSelectedBones() const { return SelectedBones; }
	inline const TArray<int32>& GetHighlightedBones() const { return HighlightedBones; }
#endif

	GEOMETRYCOLLECTIONENGINE_API FPhysScene_Chaos* GetInnerChaosScene() const;
	GEOMETRYCOLLECTIONENGINE_API AChaosSolverActor* GetPhysicsSolverActor() const;

	const FGeometryCollectionPhysicsProxy* GetPhysicsProxy() const { return PhysicsProxy; }
	FGeometryCollectionPhysicsProxy* GetPhysicsProxy() { return PhysicsProxy; }

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	/** Enable/disable the scene proxy per transform selection mode. When disabled the per material id default selection is used instead. */
	GEOMETRYCOLLECTIONENGINE_API void EnableTransformSelectionMode(bool bEnable);
	bool GetIsTransformSelectionMode() const { return bIsTransformSelectionModeEnabled; }
#endif  // #if GEOMETRYCOLLECTION_EDITOR_SELECTION

#if UE_ENABLE_DEBUG_DRAWING
	/** Force render after constant data changes (such as visibility, or hitproxy subsections). Will also work while paused. */
	void ForceRenderUpdateConstantData() { MarkRenderStateDirty(); }

	/** Force render after dynamic data changes (such as transforms). Will also work while paused. */
	void ForceRenderUpdateDynamicData() { MarkRenderDynamicDataDirty(); }
#endif  // UE_ENABLE_DEBUG_DRAWING

	/**/
	UE_DEPRECATED(5.4, "Disabled flags are no longer used.")
	const TArray<bool>& GetDisabledFlags() const { return DisabledFlags; }

	GEOMETRYCOLLECTIONENGINE_API virtual void OnCreatePhysicsState() override;
	GEOMETRYCOLLECTIONENGINE_API void RegisterAndInitializePhysicsProxy();
	GEOMETRYCOLLECTIONENGINE_API virtual void OnDestroyPhysicsState() override;
	GEOMETRYCOLLECTIONENGINE_API virtual bool ShouldCreatePhysicsState() const override;
	GEOMETRYCOLLECTIONENGINE_API virtual bool HasValidPhysicsState() const override;

	GEOMETRYCOLLECTIONENGINE_API virtual bool MoveComponentImpl(const FVector& Delta, const FQuat& NewRotation, bool bSweep, FHitResult* Hit, EMoveComponentFlags MoveFlags, ETeleportType Teleport) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void OnUpdateTransform(EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport = ETeleportType::None) override;

	// Mirrored from the proxy on a sync
	//TManagedArray<int32> RigidBodyIds;
	TArray<bool> DisabledFlags;
	int32 BaseRigidBodyIndex;
	int32 NumParticlesAdded;

	/** Changes whether or not this component will get future break notifications. */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	GEOMETRYCOLLECTIONENGINE_API void SetNotifyBreaks(bool bNewNotifyBreaks);

	/** Changes whether or not this component will get future removal notifications. */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	GEOMETRYCOLLECTIONENGINE_API void SetNotifyRemovals(bool bNewNotifyRemovals);

	/** Changes whether or not this component will get future crumbling notifications. */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	GEOMETRYCOLLECTIONENGINE_API void SetNotifyCrumblings(bool bNewNotifyCrumblings, bool bNewCrumblingEventIncludesChildren = false);

	/** Changes whether or not this component will get future trailing notifications. */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	GEOMETRYCOLLECTIONENGINE_API void SetNotifyTrailings(bool bNewNotifyTrailings);

	/** Changes whether or not this component will get future global break notifications. */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	GEOMETRYCOLLECTIONENGINE_API void SetNotifyGlobalBreaks(bool bNewNotifyGlobalBreaks);

	/** Changes whether or not this component will get future global collision notifications. */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	GEOMETRYCOLLECTIONENGINE_API void SetNotifyGlobalCollision(bool bNewNotifyGlobalCollisions);

	/** Changes whether or not this component will get future global removal notifications. */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	GEOMETRYCOLLECTIONENGINE_API void SetNotifyGlobalRemovals(bool bNewNotifyGlobalRemovals);

	/** Changes whether or not this component will get future global crumbling notifications. */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	GEOMETRYCOLLECTIONENGINE_API void SetNotifyGlobalCrumblings(bool bNewNotifyGlobalCrumblings, bool bGlobalNewCrumblingEventIncludesChildren);
	
	/** Changes whether or not this component will get future global trailing notifications. */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	GEOMETRYCOLLECTIONENGINE_API void SetNotifyGlobalTrailings(bool bNewNotifyGlobalTrailings);

	/** Overrideable native notification */
	virtual void NotifyBreak(const FChaosBreakEvent& Event) {};

	/** Overrideable native notification */
	virtual void NotifyRemoval(const FChaosRemovalEvent& Event) {};

	UPROPERTY(BlueprintAssignable, Category = "Chaos")
	FOnChaosBreakEvent OnChaosBreakEvent;

	UPROPERTY(BlueprintAssignable, Category = "Chaos")
	FOnChaosRemovalEvent OnChaosRemovalEvent;

	UPROPERTY(BlueprintAssignable, Category = "Chaos")
	FOnChaosCrumblingEvent OnChaosCrumblingEvent;

	// todo(chaos) remove when no longer necessary
	FOnChaosBreakEvent OnRootBreakEvent;

	FOnGeometryCollectionFullyDecayedEvent OnFullyDecayedEvent;
	FOnGeometryCollectionRootMovedEvent OnRootMovedEvent;
	FOnGeometryCollectionRootMovedNativeEvent OnRootMovedNativeEvent;

	GEOMETRYCOLLECTIONENGINE_API bool IsFullyDecayed() const;

	GEOMETRYCOLLECTIONENGINE_API void DispatchBreakEvent(const FChaosBreakEvent& Event);

	GEOMETRYCOLLECTIONENGINE_API void DispatchRemovalEvent(const FChaosRemovalEvent& Event);

	GEOMETRYCOLLECTIONENGINE_API void DispatchCrumblingEvent(const FChaosCrumblingEvent& Event);

	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadWrite, Interp, Category = "Chaos")
	float DesiredCacheTime;

	UPROPERTY(Transient, VisibleAnywhere, BlueprintReadWrite, Category = "Chaos")
	bool CachePlayback;

	GEOMETRYCOLLECTIONENGINE_API bool DoCustomNavigableGeometryExport(FNavigableGeometryExport& GeomExport) const override;

	/** Gets the physical material to use for this geometry collection, taking into account instance overrides and render materials */
	GEOMETRYCOLLECTIONENGINE_API UPhysicalMaterial* GetPhysicalMaterial() const;

	/** Update component structure to reflect any changes to the embedded geometry */
	GEOMETRYCOLLECTIONENGINE_API void InitializeEmbeddedGeometry();
	
	/** Update instanced static mesh components to reflect internal embedded geometry state. */
	GEOMETRYCOLLECTIONENGINE_API void RefreshEmbeddedGeometry();

#if WITH_EDITOR
	GEOMETRYCOLLECTIONENGINE_API void SetEmbeddedGeometrySelectable(bool bSelectableIn);
	GEOMETRYCOLLECTIONENGINE_API void ForceNativeRendering(bool bForce);
	GEOMETRYCOLLECTIONENGINE_API int32 EmbeddedIndexToTransformIndex(const UInstancedStaticMeshComponent* ISMComponent, int32 InstanceIndex) const;
	GEOMETRYCOLLECTIONENGINE_API void GetBoneColors(TArray<FColor>& OutColors) const;
	GEOMETRYCOLLECTIONENGINE_API void GetHiddenTransforms(TArray<bool>& OutHiddenTransforms) const;
#endif

	GEOMETRYCOLLECTIONENGINE_API void GetRestTransforms(TArray<FMatrix44f>& OutRestTransforms) const;

#if WITH_EDITORONLY_DATA
	GEOMETRYCOLLECTIONENGINE_API const FDamageCollector* GetRunTimeDataCollector() const;
#endif
	
	// #todo should this only be available in editor?
	GEOMETRYCOLLECTIONENGINE_API void SetRestState(TArray<FTransform>&& InRestTransforms);

	// this reset the rest transform to use the rest collection asset ones
	GEOMETRYCOLLECTIONENGINE_API void ResetRestTransforms();

	/** Set the dynamic state for all bodies in the DynamicCollection. */
	GEOMETRYCOLLECTIONENGINE_API void SetDynamicState(const Chaos::EObjectStateType& NewDynamicState);

	/** Set transforms for all bodies in the DynamicCollection. */
	GEOMETRYCOLLECTIONENGINE_API void SetInitialTransforms(const TArray<FTransform>& InitialTransforms);

	/** Modify DynamicCollection transform hierarchy to effect cluster breaks releasing the specified indices. */
	GEOMETRYCOLLECTIONENGINE_API void SetInitialClusterBreaks(const TArray<int32>& ReleaseIndices);

	/** Used by Niagara DI to query global matrices rather than recalculating them again */
	UE_DEPRECATED(5.3, "Use GetComponentSpaceTransforms instead")
	TArray<FMatrix> GetGlobalMatrices() { return ComputeGlobalMatricesFromComponentSpaceTransforms(); }

	UE_DEPRECATED(5.4, "Use GetComponentSpaceTransforms3f instead")
	GEOMETRYCOLLECTIONENGINE_API TArray<FTransform> GetComponentSpaceTransforms();

	GEOMETRYCOLLECTIONENGINE_API const TArray<FTransform3f>& GetComponentSpaceTransforms3f();

	/**
	* Get current transforms
	* The transforms are representing the center of mass of each piece and are in component space 
	*/
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API TArray<FTransform> GetCurrentTransforms();

	/**
	* Get current transforms from a specific list of indices
	* The transforms are representing the center of mass of each piece and are in component space
	*/
	UFUNCTION(BlueprintCallable, Category = "ChaosPhysics")
	GEOMETRYCOLLECTIONENGINE_API TArray<FTransform> GetCurrentTransformsFromIndices(const TArray<int32>& TransformIndices);

	GEOMETRYCOLLECTIONENGINE_API const FGeometryDynamicCollection* GetDynamicCollection() const;
	GEOMETRYCOLLECTIONENGINE_API FGeometryDynamicCollection* GetDynamicCollection();  // TEMP HACK?

	void SetUpdateNavigationInTick(const bool bUpdateInTick) { bUpdateNavigationInTick = bUpdateInTick; }

	void SetUseRootProxyForNavigation(const bool bUseRootProxyForNav) { bUseRootProxyForNavigation = bUseRootProxyForNav; }

	// todo(chaos): Remove this and move to a cook time approach of the SM data based on the GC property
	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly, Category = "Physics")
	bool GetUseStaticMeshCollisionForTraces() const { return bUseStaticMeshCollisionForTraces; }

	// todo(chaos): Remove this and move to a cook time approach of the SM data based on the GC property
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly, Category = "Physics")
	GEOMETRYCOLLECTIONENGINE_API void SetUseStaticMeshCollisionForTraces(bool bInUseStaticMeshCollisionForTraces);

	/** Get any custom renderer. Returns nullptr if none is set. */
	IGeometryCollectionExternalRenderInterface* GetCustomRenderer() { return CustomRenderer.GetInterface(); }

	/** Force any custom renderer to render using the broken/decayed path. This can be set at runtime */
	UFUNCTION(BlueprintCallable, Category = "Physics")
	GEOMETRYCOLLECTIONENGINE_API void ForceBrokenForCustomRenderer(bool bForceBroken);

	bool IsForceBrokenForCustomRenderer() const { return bForceBrokenForCustomRenderer; }

	/**
	* Find a leaf transform from a line trace
	* @param Start Start position for the line trace
	* @param End position for the line trace
	* @param LeafTransformIndex the index of leaf transform found by the line trace
	* @param LeafTransformName the name of leaf transform found by the line trace
	* @return true if a leaf was found
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	bool FindLeafTransformByLineTrace(const FVector Start, const FVector End, int32& LeafTransformIndex, FName& LeafTransformName) const;

	/** 
	* blueprint function to set a specific root proxy local transform 
	* warning: when called from C++ and calling it on multiple root proxies it is recommended to use SetRootProxyLocalTransform instead and then call RefreshCustomRenderer
	* to avoid the cost of refreshing the renderer each time
	*/
	UFUNCTION(BlueprintCallable, Category = "Physics")
	GEOMETRYCOLLECTIONENGINE_API void SetRootProxyComponentSpaceTransform(int32 Index, const FTransform& RootProxyTransform);

	/** Set a specific root proxy local transform */
	GEOMETRYCOLLECTIONENGINE_API void SetRootProxyLocalTransform(int32 Index, const FTransform3f& RootProxyTransform);

	/** clear all the root proxies local transforms - all proxies will now have the same default transform */
	GEOMETRYCOLLECTIONENGINE_API void ClearRootProxyLocalTransforms();

	/** Force all GC components to reregister their custom renderer objects. */
	static GEOMETRYCOLLECTIONENGINE_API void ReregisterAllCustomRenderers();

	/** update of the custom renderer when post physics sync callback is executing ( valid if custom redner is being used ) - true by default */
	void SetUpdateCustomRendererOnPostPhysicsSync(bool bValue) { bUpdateCustomRendererOnPostPhysicsSync = bValue; }
	bool GetUpdateCustomRendererOnPostPhysicsSync() const { return bUpdateCustomRendererOnPostPhysicsSync; }

	UE_DEPRECATED(5.5, "SetUpdateCustomRenderer() shouldn't be called")
	void SetUpdateCustomRenderer(bool bValue) {}
	
	UE_DEPRECATED(5.5, "Please use ForceBrokenForCustomRenderer() instead")
	UFUNCTION(BlueprintCallable, Category = "Physics", meta = (DeprecatedFunction, DeprecationMessage = "Please use ForceBrokenForCustomRenderer() instead"))
	void EnableRootProxyForCustomRenderer(bool bEnable) { ForceBrokenForCustomRenderer(!bEnable); }

	bool ShouldUpdateComponentTransformToRootBone() const { return bUpdateComponentTransformToRootBone; }

	double GetRootBrokenElapsedTimeInMs() const { return BrokenAndDecayedStates.GetRootBrokenElapsedTimeInMs(); }

	/** Attn: these replication methods are helpers meant to be called before the component is fully registered, like a constructor! */
	void SetEnableReplication(bool bInEnableReplication) { bEnableReplication = bInEnableReplication; }
	void SetReplicationAbandonAfterLevel(int32 InReplicationAbandonAfterLevel) { ReplicationAbandonAfterLevel = InReplicationAbandonAfterLevel; }
	void SetReplicationMaxPositionAndVelocityCorrectionLevel(int32 InReplicationMaxPositionAndVelocityCorrectionLevel) { ReplicationMaxPositionAndVelocityCorrectionLevel = InReplicationMaxPositionAndVelocityCorrectionLevel; }
	
	GEOMETRYCOLLECTIONENGINE_API const FTransform& GetPreviousComponentToWorld() const;
public:
	UPROPERTY(BlueprintAssignable, Category = "Collision")
	FOnChaosPhysicsCollision OnChaosPhysicsCollision;
	
	UFUNCTION(BlueprintImplementableEvent, meta = (DisplayName = "Physics Collision"), Category = "Collision")
	GEOMETRYCOLLECTIONENGINE_API void ReceivePhysicsCollision(const FChaosPhysicsCollisionInfo& CollisionInfo);

	// IChaosNotifyHandlerInterface
	GEOMETRYCOLLECTIONENGINE_API virtual void DispatchChaosPhysicsCollisionBlueprintEvents(const FChaosPhysicsCollisionInfo& CollisionInfo) override;
	
	/** If true, this component will generate breaking events that other systems may subscribe to. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ChaosPhysics|Events")
	bool bNotifyBreaks;

	/** If true, this component will generate collision events that other systems may subscribe to. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ChaosPhysics|Events")
	bool bNotifyCollisions;

	/** If true, this component will generate trailing events that other systems may subscribe to. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ChaosPhysics|Events")
	bool bNotifyTrailing;

	/** If true, this component will generate removal events that other systems may subscribe to. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ChaosPhysics|Events")
	bool bNotifyRemovals;

	/** If true, this component will generate crumbling events that other systems may subscribe to. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ChaosPhysics|Events")
	bool bNotifyCrumblings;

	/** If this and bNotifyCrumblings are true, the crumbling events will contain released children indices. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ChaosPhysics|Events", meta = (EditCondition = "bNotifyCrumblings"))
	bool bCrumblingEventIncludesChildren;

	/** If true, this component will generate breaking events that will be listened by the global event relay. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ChaosPhysics|Events|Global")
	bool bNotifyGlobalBreaks;

	/** If true, this component will generate collision events that will be listened by the global event relay. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ChaosPhysics|Events|Global")
	bool bNotifyGlobalCollisions;

	/** If true, this component will generate removal events that will be listened by the global event relay. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ChaosPhysics|Events|Global")
	bool bNotifyGlobalRemovals;

	/** If true, this component will generate crumbling events that will be listened by the global event relay. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ChaosPhysics|Events|Global")
	bool bNotifyGlobalCrumblings;

	/** If this and bNotifyGlobalCrumblings are true, the crumbling events will contain released children indices. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ChaosPhysics|Events|Global", meta = (EditCondition = "bNotifyGlobalCrumblings"))
	bool bGlobalCrumblingEventIncludesChildren;

	/** If true, this component will generate trailing events that will be listened by the global event relay. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ChaosPhysics|Events|Global")
	bool bNotifyGlobalTrailings;

	/** If true, this component will save linear and angular velocities on its DynamicCollection. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "ChaosPhysics|General")
	bool bStoreVelocities;

	UPROPERTY(Transient)
	bool bIsCurrentlyNavigationRelevant = true;

protected:
	/** Display Bone Colors instead of assigned materials */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|General")
	bool bShowBoneColors;

	/**
	 * Move the component each tick to follow the dynamically-simulated root bone; stops once the root breaks.
	 * Not supported for kinematic roots or cluster union members.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|General")
	bool bUpdateComponentTransformToRootBone;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Collision, AdvancedDisplay, config)
	bool bUseRootProxyForNavigation;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Collision, AdvancedDisplay, config)
	bool bUpdateNavigationInTick;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "ChaosPhysics|General")
	bool bEnableRunTimeDataCollection;
	
	UPROPERTY(BlueprintReadOnly, Category = "ChaosPhysics|General")
	FGuid RunTimeDataCollectionGuid;
#endif

	/** Force the broken state for custom renderer rendering. */
	uint8 bForceBrokenForCustomRenderer : 1;
	/** Whether to refresh the custom renderer on physics updates. */
	uint8 bUpdateCustomRendererOnPostPhysicsSync : 1;
	/** Custom renderer flag to specify if custom renderer can ever require native fallback. */
	uint8 bCustomRendererCanUseNativeFallback : 1;
	/** Custom renderer flag to specify if custom renderer should use native fallback. */
	uint8 bCustomRendererShouldUseNativeFallback : 1;
	/** Force native geometry collection rendering. This is used in the editor fracture mode. */
	uint8 bForceNativeRenderer : 1;

	/* Per-instance override to enable/disable replication for the geometry collection */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Network)
	bool bEnableReplication;

	/** 
	 * Enables use of ReplicationAbandonAfterLevel to stop providing network updates to
	 * clients when the updated particle is of a level higher then specified.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Network)
	bool bEnableAbandonAfterLevel;

	/**
	 * Whether abandoned particles on the client should continue to have collision (i.e.
	 * still be in the external/internal acceleration structure).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter= SetAbandonedParticleCollisionProfileName, Category = Network)
	FName AbandonedCollisionProfileName;

	/** Deprecated for CustomRendererType. */
	UPROPERTY()
	TObjectPtr<AISMPoolActor> ISMPool_DEPRECATED;

	/** Custom class type that will be used to render the geometry collection instead of using the native rendering. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Rendering", meta = (editcondition = "bOverrideCustomRenderer", MustImplement = "/Script/GeometryCollectionEngine.GeometryCollectionExternalRenderInterface"))
	TObjectPtr<UClass> CustomRendererType;

	/** If true, CustomRendererType will be used. If false, CustomRendererType comes from the RestCollection. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ChaosPhysics|Rendering", meta = (InlineEditConditionToggle))
	bool bOverrideCustomRenderer = false;

	/** Deprecated for CustomRendererType. */
	UPROPERTY()
	bool bAutoAssignISMPool_DEPRECATED = false;

	void CheckFullyDecayed();
	bool bAlreadyFullyDecayed = false;

	// todo(chaos): Remove the ability to change this at runtime, as we'll want to use this at cook time instead
	UPROPERTY(EditAnywhere, BlueprintGetter="GetUseStaticMeshCollisionForTraces", BlueprintSetter="SetUseStaticMeshCollisionForTraces", Category = "Physics")
	bool bUseStaticMeshCollisionForTraces  = false;

	/**
	 * If replicating - the cluster level to stop sending corrections for geometry collection chunks.
	 * recommended for smaller leaf levels when the size of the objects means they are no longer
	 * gameplay relevant to cut down on required bandwidth to update a collection.
	 * @see bEnableAbandonAfterLevel
	 */ 
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "GeometryCollection now uses ReplicationAbandonAfterLevel instead of ReplicationAbandonClusterLevel."))
	int32 ReplicationAbandonClusterLevel_DEPRECATED;

	/** A custom renderer object created from CustomRenderType. */
	UPROPERTY(SkipSerialization)
	TScriptInterface<IGeometryCollectionExternalRenderInterface> CustomRenderer;

	/** Collect all the PSO precache data used by the geometry collection */
	GEOMETRYCOLLECTIONENGINE_API virtual void CollectPSOPrecacheData(const FPSOPrecacheParams& BasePrecachePSOParams, FMaterialInterfacePSOPrecacheParamsList& OutParams) override;

	/** Populate the dynamic particle data for the render thread. */
	GEOMETRYCOLLECTIONENGINE_API FGeometryCollectionDynamicData* InitDynamicData(bool bInitialization = false);

	/** Reset the dynamic collection from the current rest state. */
	GEOMETRYCOLLECTIONENGINE_API void ResetDynamicCollection();

	/** Combine the commands from the input field assets */
	GEOMETRYCOLLECTIONENGINE_API void GetInitializationCommands(TArray<FFieldSystemCommand>& CombinedCommmands);

	/** Issue a field command for the physics thread */
	GEOMETRYCOLLECTIONENGINE_API void DispatchFieldCommand(const FFieldSystemCommand& InCommand);

	GEOMETRYCOLLECTIONENGINE_API static Chaos::FPhysicsSolver* GetSolver(const UGeometryCollectionComponent& GeometryCollectionComponent);

	UE_DEPRECATED(5.4, "CalculateLocalBounds is now Deprecated as it does not need to be called anymore, see ComponentSpaceBounds which replace LocalBounds")
	void CalculateLocalBounds() {};

	UE_DEPRECATED(5.3, "Use ComputeBoundsFromComponentSpaceTransforms instead")
	GEOMETRYCOLLECTIONENGINE_API FBox ComputeBoundsFromGlobalMatrices(const FMatrix& LocalToWorldWithScale, const TArray<FMatrix>& GlobalMatricesArray) const;

	GEOMETRYCOLLECTIONENGINE_API FBox ComputeBoundsFromComponentSpaceTransforms(const FTransform& LocalToWorldWithScale, const TArray<FTransform>& ComponentSpaceTransformsArray) const;
	FBox ComputeBoundsFromComponentSpaceTransforms(const FTransform& LocalToWorldWithScale, const TArray<FTransform3f>& ComponentSpaceTransformsArray) const;

	UE_DEPRECATED(5.3, "Use FTransform version of ComputeBounds instead")
	GEOMETRYCOLLECTIONENGINE_API FBox ComputeBounds(const FMatrix& LocalToWorldWithScale) const;

	GEOMETRYCOLLECTIONENGINE_API FBox ComputeBounds(const FTransform& LocalToWorldWithScale) const;

	GEOMETRYCOLLECTIONENGINE_API void RegisterForEvents();
	GEOMETRYCOLLECTIONENGINE_API void UpdateRBCollisionEventRegistration();
	GEOMETRYCOLLECTIONENGINE_API void UpdateGlobalCollisionEventRegistration();
	GEOMETRYCOLLECTIONENGINE_API void UpdateGlobalRemovalEventRegistration();

	/**
	 * A per-level collision profile name. If the name is set to NONE or an invalid collision profile, nothing will be changed.
	 * If the there are more levels than elements in this array, then each level will use the index that best matches it.
	 * For example, if the particle is level 2, and there is only 1 element in the array, then the particle will use the 0th
	 * collision profile. AbandonedCollisionProfileName will override this on the client when relevant.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetPerLevelCollisionProfileNames, Category=Physics)
	TArray<FName> CollisionProfilePerLevel;

	/**
	 * A per-particle collision profile name. If the per-particle collision profile name exists, it will override the per-level profile name.
	 */
	TArray<FName> CollisionProfilePerParticle;

	/**
	* If replicating - the cluster level after which replication will not happen 
	* @see bEnableAbandonAfterLevel
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Network)
	int32 ReplicationAbandonAfterLevel;

	/**
	* If replicating - the maximum level where clusters will have their position and velocity send over to the client for tracking and correcting
	* When breaking, client will only receive the initial break velocity
	* This helps save bandwidth where only the destruction state of the GC is to be replicated but the falling pieces do not need to be tracked precisely
	* @note This will have an effect only if set to a value less than ReplicationAbandonAfterLevel
	* @see ReplicationAbandonAfterLevel
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Network)
	int32 ReplicationMaxPositionAndVelocityCorrectionLevel;

	UPROPERTY(ReplicatedUsing=OnRep_RepData)
	FGeometryCollectionRepData RepData;

	UPROPERTY(ReplicatedUsing = OnRep_RepStateData)
	FGeometryCollectionRepStateData RepStateData;

	UPROPERTY(ReplicatedUsing = OnRep_RepDynamicData)
	FGeometryCollectionRepDynamicData RepDynamicData;

	/** Called post solve to allow authoritative components to update their replication data */
	UFUNCTION()
	GEOMETRYCOLLECTIONENGINE_API void OnRep_RepData();

	UFUNCTION()
	GEOMETRYCOLLECTIONENGINE_API void OnRep_RepStateData();

	UFUNCTION()
	GEOMETRYCOLLECTIONENGINE_API void OnRep_RepDynamicData();
	
	GEOMETRYCOLLECTIONENGINE_API void RequestUpdateRepData();
	GEOMETRYCOLLECTIONENGINE_API virtual void UpdateRepData();
	GEOMETRYCOLLECTIONENGINE_API virtual void UpdateRepStateAndDynamicData();

	/** Clear all rep data, this is required if the physics proxy has been recreated */
	GEOMETRYCOLLECTIONENGINE_API virtual void ResetRepData();
 
	UE_DEPRECATED(5.3, "The argument-free version of ProcessRepData will be removed. Please use the version which takes DeltaTime and SimTime instead.")
	GEOMETRYCOLLECTIONENGINE_API virtual void ProcessRepData();

	GEOMETRYCOLLECTIONENGINE_API virtual bool ProcessRepData(float DeltaTime, float SimTime);

	int32 VersionProcessed = INDEX_NONE;
	int32 DynamicRepDataVersionProcessed = INDEX_NONE;

	// The last time (in milliseconds) the async physics component tick fired.
	// We track this on the client to be able to turn off the tick for perf reasons
	// if we spend a lot of ticks sequentially doing nothing.
	int64 LastAsyncPhysicsTickMs = 0;

private:
	void ProcessRepDataOnPT();
	void ProcessRepStateDataOnPT();
	void ProcessRepDynamicDataOnPT();
	void InitializeRemovalDynamicAttributesIfNeeded();
	void SendDynamicDataToSceneProxy();

	// called when the rest transform are updated from SetRestState / ResetRestTransforms
	// this updates only the renderer, the dynamic collection should be initialized when calling this function
	void RestTransformsChanged();

	// return the most actual transforms
	// this can be the rest collection ones, the overridden RestTransforms or the dynamic collection ones
	FTransform3f GetCurrentTransform(int32 Index) const;
	void ComputeCurrentGlobalsMatrices(TArray<FTransform3f>& OutTransforms) const;

	uint8 bInitializedRemovalDynamicAttribute : 1;
	uint8 bEnableBoneSelection : 1;
	uint8 IsObjectDynamic : 1;
	uint8 IsObjectLoading : 1;

	int ViewLevel;

	uint32 NavmeshInvalidationTimeSliceIndex;

	Chaos::Filter::FShapeFilterData InitialShapeFilterData;
	Chaos::Filter::FInstanceData InitialFilterInstanceData;
	float CurrentCacheTime;
	FChaosUserData PhysicsUserData;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TArray<int32> SelectedBones;

	UPROPERTY(Transient)
	TArray<int32> HighlightedBones;
#endif

	UE_DEPRECATED(5.3, "Use ComponentSpaceTransforms instead")
	TArray<FMatrix> GlobalMatrices;

	struct FComponentSpaceTransforms
	{
	public:
		FComponentSpaceTransforms(const UGeometryCollectionComponent* InComponent = nullptr)
			: RootIndex(INDEX_NONE)
			, Component(InComponent)
		{
			bIsRootDirty = 1;
			bIsDirty = 1;
		}
		
		void Reset(int32 NumTransforms, int32 InRootIndex)
		{
			Transforms.SetNumUninitialized(NumTransforms);
			RootIndex = InRootIndex;
			MarkDirty();
		}

		void MarkDirty()
		{
			bIsRootDirty = true;
			bIsDirty = true;
		}

		int32 Num() const { return Transforms.Num(); }

		SIZE_T GetAllocatedSize() const { return Transforms.GetAllocatedSize(); }

		// request all transform to be update
		// this will trigger an update if it is still marked dirty
		const TArray<FTransform3f>& RequestAllTransforms() const;

		// request the root transform, this may compute it if it is still marked dirty
		const FTransform3f& RequestRootTransform() const;

	private:
		int32 RootIndex;
		mutable uint8 bIsRootDirty : 1;
		mutable uint8 bIsDirty : 1;
		mutable TArray<FTransform3f> Transforms;
		const UGeometryCollectionComponent* Component;
	};

	FComponentSpaceTransforms ComponentSpaceTransforms;

	/** bounds for unbroken state bounds in root space */
	mutable FBox RootSpaceBounds;

	/** 
	* Bounds in component space 
	* if unbroken this will use computed from RootSpaceBounds
	*/
	mutable FBox ComponentSpaceBounds;

	TArray<bool> EventsPlayed;

	FGeometryCollectionPhysicsProxy* PhysicsProxy = nullptr;
	TSharedPtr<FGeometryDynamicCollection> DynamicCollection;

	// Temporary storage for body setup in order to initialise a dummy body instance
	UPROPERTY(Transient)
	TObjectPtr<UBodySetup> DummyBodySetup;

#if WITH_EDITORONLY_DATA
	// Tracked editor actor that owns the original component so we can write back recorded caches
	// from PIE.
	UPROPERTY(Transient)
	TObjectPtr<AActor> EditorActor;
#endif
	GEOMETRYCOLLECTIONENGINE_API void SwitchRenderModels(const AActor* Actor);

	// Event dispatcher for break, crumble, removal and collision events
	UPROPERTY(transient)
	TObjectPtr<UChaosGameplayEventDispatcher> EventDispatcher;

#if GEOMETRYCOLLECTION_EDITOR_SELECTION
	bool bIsTransformSelectionModeEnabled;
#endif  // #if GEOMETRYCOLLECTION_EDITOR_SELECTION

	/** The information of all the embedded instanced static meshes */
	UPROPERTY()
	TArray<TObjectPtr<UInstancedStaticMeshComponent>> EmbeddedGeometryComponents;

#if WITH_EDITORONLY_DATA
	TArray<TArray<int32>> EmbeddedBoneMaps;
	TArray<int32> EmbeddedInstanceIndex;
#endif

	GEOMETRYCOLLECTIONENGINE_API bool IsEmbeddedGeometryValid() const;
	GEOMETRYCOLLECTIONENGINE_API void ClearEmbeddedGeometry();

	/** return true if a custom renderer has been set and the feature is enabled */
	GEOMETRYCOLLECTIONENGINE_API bool IsCustomRendererAvailable() const;
	/** return true if a custom renderer is active */
	GEOMETRYCOLLECTIONENGINE_API bool IsUsingCustomRenderer() const;

	void InitializeCustomRenderer();
	GEOMETRYCOLLECTIONENGINE_API void RegisterCustomRenderer();
	GEOMETRYCOLLECTIONENGINE_API void UnregisterCustomRenderer();

public:
	/** Updates the custom renderer to the reflect the current state of the Geometry Collection */
	GEOMETRYCOLLECTIONENGINE_API void RefreshCustomRenderer();

	/** Refresh root proxies whether they are drawn through the custom renderer or normal static mesh components */
	GEOMETRYCOLLECTIONENGINE_API void RefreshRootProxies();

private:

	/** Return true if the simulation can be run in editor */
	GEOMETRYCOLLECTIONENGINE_API bool CanRunSimulationInEditor() const;

	/** BP internal function to get the solver actor */
	UFUNCTION(BlueprintPure, BlueprintInternalUseOnly)
	AChaosSolverActor* GetSolverActor() const { return ChaosSolverActor; }

	/** BP internal function to set the solver actor */
	UFUNCTION(BlueprintCallable, BlueprintInternalUseOnly)
	GEOMETRYCOLLECTIONENGINE_API void SetSolverActor(AChaosSolverActor* InSolverActor);

	GEOMETRYCOLLECTIONENGINE_API void IncrementSleepTimer(float DeltaTime);
	GEOMETRYCOLLECTIONENGINE_API void IncrementBreakTimer(float DeltaTime);
	GEOMETRYCOLLECTIONENGINE_API bool CalculateInnerSphere(int32 TransformIndex, UE::Math::TSphere<double>& SphereOut) const;
	GEOMETRYCOLLECTIONENGINE_API void UpdateDecay(int32 TransformIdx, float UpdatedDecay, bool UseClusterCrumbling, bool HasDynamicInternalClusterParent, FGeometryCollectionDecayContext& ContextInOut);

	GEOMETRYCOLLECTIONENGINE_API void UpdateAttachedChildrenTransform() const;
	
	GEOMETRYCOLLECTIONENGINE_API void UpdateRenderSystemsIfNeeded(bool bDynamicCollectionDirty);
	GEOMETRYCOLLECTIONENGINE_API void UpdateNavigationDataIfNeeded(bool bDynamicCollectionDirty);
	GEOMETRYCOLLECTIONENGINE_API void UpdateRemovalIfNeeded();

	GEOMETRYCOLLECTIONENGINE_API void BuildInitialFilterData();

	GEOMETRYCOLLECTIONENGINE_API void LoadCollisionProfiles();

	GEOMETRYCOLLECTIONENGINE_API void OnPostPhysicsSync();

	GEOMETRYCOLLECTIONENGINE_API void OnPostCreateParticles();

	GEOMETRYCOLLECTIONENGINE_API bool HasVisibleGeometry() const;

	/** backward compatibility method, until we can remove GlobalMatrices */
	GEOMETRYCOLLECTIONENGINE_API TArray<FMatrix> ComputeGlobalMatricesFromComponentSpaceTransforms() const;

	float ComputeMassScaleRelativeToAsset() const;

	void MoveComponentToRootTransform();

	/** called when the dynamic collection is found to be dirty */
	void OnTransformsDirty();

	/** The clusters we need to replicate */
	TUniquePtr<TSet<Chaos::FPBDRigidClusteredParticleHandle*>> ClustersToRep;

	double LastHardsnapTimeInMs = 0;

	/** One off activation is processed in the same order as server so remember the last one we processed */
	int32 OneOffActivatedProcessed = 0;

public:
	/** Uniform angular ether drag. */
	UPROPERTY(meta=(DeprecatedProperty, DeprecationMessage="Use PhysicalMaterial instead."))
	float AngularEtherDrag_DEPRECATED;

private:
	struct FBrokenAndDecayedStates
	{
	public:
		void Reset(int32 NumTransforms);

		bool GetIsRootBroken() const { return bIsRootBroken; }
		bool GetIsBroken(int32 TransformIndex) const;
		bool GetHasDecayed(int32 TransformIndex) const;
		double GetRootBrokenEventTimeInMs() const;
		GEOMETRYCOLLECTIONENGINE_API double GetRootBrokenElapsedTimeInMs() const;

		void SetRootIsBroken(bool bIsBroken);
		void SetIsBroken(int32 TransformIndex);
		void SetHasDecayed(int32 TransformIndex);
		void SetHasDecayedRecursive(int32 TransformIndex, const TArray<TSet<int32>>& Children);

		// return true if any broken piece is not yet decayed 
		bool HasAnyDecaying() const;

		bool HasFullyDecayed() const;

	private:
		int32 NumTransforms = 0;
		bool bIsRootBroken = false;
		double RootBrokenEventTimeInMs = 0;
		TBitArray<> IsBroken;
		TBitArray<> HasDecayed;
		int32 NumDecaying = 0;
	};

	FBrokenAndDecayedStates BrokenAndDecayedStates;

	void UpdateBrokenAndDecayedStates();

	bool ShouldCreateRootProxyComponents() const;

	TArray<FTransform3f> RootProxyLocalTransforms;

private:

	enum class ENetAwakeningMode
	{
		ForceDormancyAwake,
		FlushNetDormancy,
	};

	/** Flushes the net dormancy of our owner if this action is enabled and we are dormant */
	void FlushNetDormancyIfNeeded() const;

	ENetAwakeningMode GetDesiredNetAwakeningMode() const;

	//~ Begin IPhysicsComponent Interface.
public:
	GEOMETRYCOLLECTIONENGINE_API virtual Chaos::FPhysicsObject* GetPhysicsObjectById(Chaos::FPhysicsObjectId Id) const override;
	GEOMETRYCOLLECTIONENGINE_API virtual Chaos::FPhysicsObject* GetPhysicsObjectByName(const FName& Name) const override;
	GEOMETRYCOLLECTIONENGINE_API virtual TArray<Chaos::FPhysicsObject*> GetAllPhysicsObjects() const override;
	GEOMETRYCOLLECTIONENGINE_API virtual Chaos::FPhysicsObjectId GetIdFromGTParticle(Chaos::FGeometryParticle* Particle) const override;
	//~ End IPhysicsComponent Interface.

#if WITH_EDITOR
	//~ Begin UActorComponent interface.
	GEOMETRYCOLLECTIONENGINE_API virtual bool IsHLODRelevant() const override;
	GEOMETRYCOLLECTIONENGINE_API virtual TArray<UActorComponent*> GetHLODProxyComponents() const override;
	//~ End UActorComponent interface.
#endif

	/**
	 * Currently, component space transforms for every particle is compute relative to the component transform.
	 * By default, this means that when the component transform changes, every particle is shifted along with the component.
	 * However, there are times when you may change the component transform but instead want every particle to stay in the same world position.
	 * In those cases, RebaseDynamicCollectionTransformsOnNewWorldTransform can be called.
	 */
	GEOMETRYCOLLECTIONENGINE_API void RebaseDynamicCollectionTransformsOnNewWorldTransform();

	friend struct FGCCollisionProfileScopedTransaction;
};

/** Struct to be used as Transaction object used to make updates on particle per particle basis within a scope.
 * It makes sure the collision profile names containers is up to date and the Collision profiles are loaded if needed when it goes out of scope
 */
struct FGCCollisionProfileScopedTransaction
{
	explicit FGCCollisionProfileScopedTransaction(UGeometryCollectionComponent* InGCComponentInstance) : GCComponentInstance(InGCComponentInstance)
	{
		if (!ensure(GCComponentInstance))
		{
			return;
		}

		bHasChanged = InGCComponentInstance->UpdatePerParticleCollisionProfilesNum();	
	}

	~FGCCollisionProfileScopedTransaction()
	{
		if (!GCComponentInstance)
		{
			return;
		}

		if (bHasChanged)
		{
			GCComponentInstance->LoadCollisionProfiles();
		}
	}

	FGCCollisionProfileScopedTransaction& operator=(const FGCCollisionProfileScopedTransaction& Other) = delete;
	FGCCollisionProfileScopedTransaction& operator=(FGCCollisionProfileScopedTransaction&& Other) = delete;
	FGCCollisionProfileScopedTransaction(FGCCollisionProfileScopedTransaction&& Other) = delete;
	FGCCollisionProfileScopedTransaction(FGCCollisionProfileScopedTransaction& Other) = delete;

	/** Marks this transaction dirty. It will load the collision profiles if needed when this transaction goes out of scope */
	void MarkDirty()
	{
		bHasChanged = true;
	}

	bool IsValid() const { return GCComponentInstance != nullptr; }

private:
	UGeometryCollectionComponent* GCComponentInstance = nullptr;
	bool bHasChanged = false;
};
