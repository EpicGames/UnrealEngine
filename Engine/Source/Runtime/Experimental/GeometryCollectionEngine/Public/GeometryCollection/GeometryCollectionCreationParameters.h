// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"


class FGeometryCollection;

class FGeometryCollectionCreationParameters
{
public:
	GEOMETRYCOLLECTIONENGINE_API FGeometryCollectionCreationParameters(FGeometryCollection& GeometryCollection, bool ReCalculateNormalsIn = false, bool ReCalculateTangetsIn = false);
	GEOMETRYCOLLECTIONENGINE_API ~FGeometryCollectionCreationParameters();

private:
	bool ReCalculateNormals;
	bool ReCalculateTangents;
	FGeometryCollection& GeometryCollection;

};
