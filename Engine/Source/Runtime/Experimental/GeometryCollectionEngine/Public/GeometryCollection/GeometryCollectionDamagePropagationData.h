// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollectionDamagePropagationData.generated.h"

USTRUCT(BlueprintType)
struct FGeometryCollectionDamagePropagationData
{
public:
	GENERATED_BODY()

	/** Whether or not damage propagation is enabled. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Damage Propagation")
	bool bEnabled = true;

	/** factor of the remaining strain propagated through the connection graph after a piece breaks. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Damage Propagation")
	float BreakDamagePropagationFactor = 1.0f;

	/** factor of the received strain propagated throug the connection graph if the piece did not break. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Damage Propagation")
	float ShockDamagePropagationFactor = 0.0f;
};
