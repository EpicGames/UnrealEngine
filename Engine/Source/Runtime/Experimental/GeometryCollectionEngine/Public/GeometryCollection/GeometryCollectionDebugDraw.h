// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"

class FMeshElementCollector;
class FMaterialRenderProxy;
class FGeometryCollection;
struct FColor;

namespace GeometryCollectionDebugDraw
{
	void DrawSolid(const FGeometryCollection& Collection, const FTransform& CollectionWorldTransform, FMeshElementCollector& MeshCollector, int32 ViewIndex, const FMaterialRenderProxy* MaterialProxy);
	void DrawWireframe(const FGeometryCollection& Collection, const FTransform& CollectionWorldTransform, FMeshElementCollector& MeshCollector, int32 ViewIndex, const FColor& Color);
};
