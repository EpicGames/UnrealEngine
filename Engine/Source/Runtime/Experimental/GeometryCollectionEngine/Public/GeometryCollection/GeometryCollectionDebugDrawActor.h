// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "Components/BillboardComponent.h"

#include <type_traits>

#include "GeometryCollectionDebugDrawActor.generated.h"

class AChaosSolverActor;
class AGeometryCollectionActor;
class UGeometryCollectionComponent;
template<class InElementType> class TManagedArray;
class FGeometryCollectionParticlesData;
template<class T> class TAutoConsoleVariable;

/**
* EGeometryCollectionDebugDrawActorHideGeometry
*   Visibility enum.
*/
UENUM()
enum class EGeometryCollectionDebugDrawActorHideGeometry : uint8
{
	// Do not hide any geometry.
	HideNone,
	// Hide the geometry associated with rigid bodies that are selected for collision volume visualization.
	HideWithCollision,
	// Hide the geometry associated with the selected rigid bodies.
	HideSelected,
	// Hide the entire geometry collection associated with the selected rigid bodies.
	HideWholeCollection,
	// Hide all geometry collections.
	HideAll
};

/**
* FGeometryCollectionDebugDrawWarningMessage
*   Empty structure used to embed a warning message in the UI through a detail customization.
*/
USTRUCT()
struct UE_DEPRECATED(5.0, "Deprecated. Use normal debug draw Chaos Physics commands") FGeometryCollectionDebugDrawWarningMessage
{
	GENERATED_USTRUCT_BODY()
};

/**
* FGeometryCollectionDebugDrawActorSelectedRigidBody
*   Structure used to select a rigid body id with a picking tool through a detail customization.
*/
USTRUCT()
struct UE_DEPRECATED(5.0, "Deprecated. Use normal debug draw Chaos Physics commands") FGeometryCollectionDebugDrawActorSelectedRigidBody
{
	GENERATED_USTRUCT_BODY()

	explicit FGeometryCollectionDebugDrawActorSelectedRigidBody(int32 InId = -1) : Id(InId), Solver(nullptr), GeometryCollection(nullptr) {}
	//explicit FGeometryCollectionDebugDrawActorSelectedRigidBody(FGuid InId = FGuid()) : Id(InId), Solver(nullptr), GeometryCollection(nullptr) {}

	/** Id of the selected rigid body whose to visualize debug informations. Use -1 to visualize all Geometry Collections. */
	UPROPERTY(EditAnywhere, Category = "Selected Rigid Body", meta = (ClampMin="-1"))
	int32 Id;
	//FGuid Id;

	/** Chaos RBD Solver. Will use the world's default solver actor if null. */
	UPROPERTY(EditAnywhere, Category = "Selected Rigid Body")
	TObjectPtr<AChaosSolverActor> Solver;

	/** Currently selected geometry collection. */
	UPROPERTY(VisibleAnywhere, Category = "Selected Rigid Body")
	TObjectPtr<AGeometryCollectionActor> GeometryCollection;

	/** Return the name of selected solver, or "None" if none is selected. */
	FString GetSolverName() const;
};

/**
* AGeometryCollectionDebugDrawActor
*   An actor representing the collection of data necessary to visualize the 
*   geometry collections' debug informations.
*   Only one actor is to be used in the world, and should be automatically 
*   spawned by any GeometryDebugDrawComponent that needs it.
*/
class UE_DEPRECATED(5.0, "Deprecated. Use normal debug draw Chaos Physics commands") AGeometryCollectionDebugDrawActor;
UCLASS(HideCategories = ("Rendering", "Replication", "Input", "Actor", "Collision", "LOD", "Cooking"), MinimalAPI)
class AGeometryCollectionDebugDrawActor : public AActor
{
	GENERATED_UCLASS_BODY()
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Warning message to explain that the debug draw properties have no effect until starting playing/simulating. */
	UPROPERTY()
	FGeometryCollectionDebugDrawWarningMessage WarningMessage_DEPRECATED;

	/** Picking tool used to select a rigid body id. */
	UPROPERTY()
	FGeometryCollectionDebugDrawActorSelectedRigidBody SelectedRigidBody_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** Show debug visualization for the rest of the geometry collection related to the current rigid body id selection. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	bool bDebugDrawWholeCollection;

	/** Show debug visualization for the top level node rather than the bottom leaf nodes of a cluster's hierarchy. * Only affects Clustering and Geometry visualization.
	*/
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	bool bDebugDrawHierarchy;

	/** Show debug visualization for all clustered children associated to the current rigid body id selection. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	bool bDebugDrawClustering;

	/** Geometry visibility setting. Select the part of the geometry to hide in order to better visualize the debug information. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw")
	EGeometryCollectionDebugDrawActorHideGeometry HideGeometry;

	/** Display the selected rigid body's id. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Rigid Body")
	bool bShowRigidBodyId;

	/** Show the selected rigid body's collision volume. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Rigid Body")
	bool bShowRigidBodyCollision;

	/** Show the selected rigid body's collision volume at the origin, in local space. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Rigid Body")
	bool bCollisionAtOrigin;

	/** Show the selected rigid body's transform. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Rigid Body")
	bool bShowRigidBodyTransform;

	/** Show the selected rigid body's inertia tensor box. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Rigid Body")
	bool bShowRigidBodyInertia;

	/** Show the selected rigid body's linear and angular velocity. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Rigid Body")
	bool bShowRigidBodyVelocity;

	/** Show the selected rigid body's applied force and torque. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Rigid Body")
	bool bShowRigidBodyForce;

	/** Show the selected rigid body's on screen text information. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Rigid Body")
	bool bShowRigidBodyInfos;

	/** Show the transform index for the selected rigid body's associated cluster nodes. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Clustering")
	bool bShowTransformIndex;

	/** Show the transform for the selected rigid body's associated cluster nodes. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Clustering")
	bool bShowTransform;

	/** Show a link from the selected rigid body's associated cluster nodes to their parent's nodes. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Clustering")
	bool bShowParent;

	/** Show the hierarchical level for the selected rigid body's associated cluster nodes. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Clustering")
	bool bShowLevel;

	/** Show the connectivity edges for the selected rigid body's associated cluster nodes. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Clustering")
	bool bShowConnectivityEdges;

	/** Show the geometry index for the selected rigid body's associated geometries. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Geometry")
	bool bShowGeometryIndex;

	/** Show the geometry transform for the selected rigid body's associated geometries. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Geometry")
	bool bShowGeometryTransform;

	/** Show the bounding box for the selected rigid body's associated geometries. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Geometry")
	bool bShowBoundingBox;

	/** Show the faces for the selected rigid body's associated geometries. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Geometry")
	bool bShowFaces;

	/** Show the face indices for the selected rigid body's associated geometries. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Geometry")
	bool bShowFaceIndices;

	/** Show the face normals for the selected rigid body's associated geometries. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Geometry")
	bool bShowFaceNormals;

	/** Enable single face visualization for the selected rigid body's associated geometries. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Geometry")
	bool bShowSingleFace;

	/** The index of the single face to visualize. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Geometry", meta = (ClampMin="0"))
	int32 SingleFaceIndex;

	/** Show the vertices for the selected rigid body's associated geometries. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Geometry")
	bool bShowVertices;

	/** Show the vertex indices for the selected rigid body's associated geometries. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Geometry")
	bool bShowVertexIndices;

	/** Show the vertex normals for the selected rigid body's associated geometries. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Geometry")
	bool bShowVertexNormals;

	/** Adapt visualization depending of the cluster nodes' hierarchical level. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings")
	bool bUseActiveVisualization;

	/** Thickness of points when visualizing vertices. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (ClampMin="0"))
	float PointThickness;

	/** Thickness of lines when visualizing faces, normals, ...etc. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (ClampMin="0"))
	float LineThickness;

	/** Draw shadows under the displayed text. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings")
	bool bTextShadow;

	/** Scale of the font used to display text. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (ClampMin="0.0001"))
	float TextScale;

	/** Scale factor used for visualizing normals. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (ClampMin="0.0001"))
	float NormalScale;

	/** Scale of the axis used for visualizing all transforms. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (ClampMin="0.0001"))
	float AxisScale;

	/** Size of arrows used for visualizing normals, breaking information, ...etc. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (ClampMin="0.0001"))
	float ArrowScale;

	/** Color used for the visualization of the rigid body ids. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor RigidBodyIdColor;

	/** Scale for rigid body transform visualization. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (ClampMin="0.0001"))
	float RigidBodyTransformScale;

	/** Color used for collision primitives visualization. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor RigidBodyCollisionColor;

	/** Color used for the visualization of the rigid body inertia tensor box. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor RigidBodyInertiaColor;

	/** Color used for rigid body velocities visualization. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor RigidBodyVelocityColor;

	/** Color used for rigid body applied force and torque visualization. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor RigidBodyForceColor;

	/** Color used for the visualization of the rigid body infos. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor RigidBodyInfoColor;

	/** Color used for the visualization of the transform indices. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor TransformIndexColor;

	/** Scale for cluster transform visualization. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (ClampMin="0.0001"))
	float TransformScale;

	/** Color used for the visualization of the levels. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor LevelColor;

	/** Color used for the visualization of the link from the parents. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor ParentColor;

	/** Line thickness used for the visualization of the rigid clustering connectivity edges. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (ClampMin="0.0001"))
	float ConnectivityEdgeThickness;

	/** Color used for the visualization of the geometry indices. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor GeometryIndexColor;

	/** Scale for geometry transform visualization. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (ClampMin="0.0001"))
	float GeometryTransformScale;

	/** Color used for the visualization of the bounding boxes. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor BoundingBoxColor;

	/** Color used for the visualization of the faces. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor FaceColor;

	/** Color used for the visualization of the face indices. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor FaceIndexColor;

	/** Color used for the visualization of the face normals. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor FaceNormalColor;

	/** Color used for the visualization of the single face. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor SingleFaceColor;

	/** Color used for the visualization of the vertices. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor VertexColor;

	/** Color used for the visualization of the vertex indices. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor VertexIndexColor;

	/** Color used for the visualization of the vertex normals. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw|Settings", meta = (HideAlphaChannel))
	FColor VertexNormalColor;

	/** Display icon in the editor. */
	UPROPERTY()
	TObjectPtr<UBillboardComponent> SpriteComponent;
};
