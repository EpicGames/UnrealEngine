// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MeshComponent.h"
#include "GeometryCollection/GeometryCollectionParticlesData.h"
#include "GeometryCollectionDebugDrawComponent.generated.h"


class AGeometryCollectionRenderLevelSetActor;
class UGeometryCollectionComponent;
class AGeometryCollectionDebugDrawActor;
class AChaosSolverActor;

#if GEOMETRYCOLLECTION_DEBUG_DRAW
namespace Chaos { class FImplicitObject; }
namespace Chaos { template<class T, int d> class TPBDRigidParticles; }
#endif  // #if GEOMETRYCOLLECTION_DEBUG_DRAW

/**
* UGeometryCollectionDebugDrawComponent
*   Component adding debug drawing functionality to a GeometryCollectionActor.
*   This component is automatically added to every GeometryCollectionActor.
*/
class UE_DEPRECATED(5.0, "Deprecated. Use normal debug draw Chaos Physics commands") UGeometryCollectionDebugDrawComponent;
UCLASS(meta = (BlueprintSpawnableComponent), HideCategories = ("Tags", "Activation", "Cooking", "AssetUserData", "Collision"), MinimalAPI)
class UGeometryCollectionDebugDrawComponent : public UActorComponent
{
	GENERATED_UCLASS_BODY()

public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Singleton actor, containing the debug draw properties. Automatically populated at play time unless explicitly set. */
	UPROPERTY()
	TObjectPtr<AGeometryCollectionDebugDrawActor> GeometryCollectionDebugDrawActor_DEPRECATED;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** Level Set singleton actor, containing the Render properties. Automatically populated at play time unless explicitly set. */
	UPROPERTY(EditAnywhere, Category = "Debug Draw", AdvancedDisplay)
	TObjectPtr<AGeometryCollectionRenderLevelSetActor> GeometryCollectionRenderLevelSetActor;

	UGeometryCollectionComponent* GeometryCollectionComponent;  // the component we are debug rendering for, set by the GeometryCollectionActor after creation
};
