// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#ifndef GEOMETRYCOLLECTION_EDITOR_SELECTION
#define GEOMETRYCOLLECTION_EDITOR_SELECTION WITH_EDITOR
#endif
