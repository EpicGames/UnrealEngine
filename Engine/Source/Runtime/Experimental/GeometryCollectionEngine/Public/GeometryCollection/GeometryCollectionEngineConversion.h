// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "GeometryCollectionObject.h"
#include "Engine/World.h"

class UStaticMesh;
class USkeletalMesh;
class USkeleton;
class USkeletalMeshComponent;
class UGeometryCollection;
class UMaterialInterface;
class UGeometryCollectionComponent;
class FGeometryCollection;
struct FManagedArrayCollection;
class UBodySetup;
class FSkeletalMeshLODRenderData;
struct FMeshDescription;

typedef TTuple<const UStaticMesh *, const UStaticMeshComponent *, FTransform> GeometryCollectionStaticMeshConversionTuple;
typedef TTuple<const USkeletalMesh *, const USkeletalMeshComponent *, FTransform> GeometryCollectionSkeletalMeshConversionTuple;

/**
 * The public interface to this module
 */
class FGeometryCollectionEngineConversion
{
public:

	struct FSkeletalMeshToCollectionConversionParameters
	{
		bool bParentAllBonesUnderNewRoot = true;
	};

	/**
	 * Appends materials to a GeometryCollectionComponent.
	 * @param Materials : Materials fetched from the StaticMeshComponent used to configure this geometry
	 * @param GeometryCollection  : Collection to append the mesh into.
	 */
	static GEOMETRYCOLLECTIONENGINE_API int32 AppendMaterials(const TArray<UMaterialInterface*>& Materials, UGeometryCollection* GeometryCollectionObject, bool bAddInteriorCopy);

	/**
	 * Appends instanced mesh indices 
	 * @param GeometryCollectionObject geometry collection to add to 
	 * @param FromTransformIndex transform index to start from 
	 * @param StaticMesh  static mesh to add reference to 
	 * @param Materials materials corresponding to the static mesh instance to get the index from 
	 */
	static GEOMETRYCOLLECTIONENGINE_API void AppendAutoInstanceMeshIndices(UGeometryCollection* GeometryCollectionObject, int32 FromTransformIndex, const UStaticMesh* StaticMesh, const TArray<UMaterialInterface*>& Materials);

	/**
	 * Appends a MeshDescription to a GeometryCollection.
	 * @param MeshDescription : Const mesh description to read vertex/normals/index data from
	 * @param Name : A name to use for the Mesh, e.g. from the source UObject's GetName()
	 * @param StartMaterialIndex : Index of materials to be used, e.g. as returned by AppendMaterials() (see below)
	 * @param StaticMeshTransform : Mesh transform.
	 * @param GeometryCollection  : Collection to append the mesh into.
	 * @param BodySetup : Optional collision setup to transfer to the geometry collection
	 * @param bReindexMaterials	: Whether to reindex materials -- if appending multiple meshes, pass false and call ReindexMaterials afterwards
	 */
	static GEOMETRYCOLLECTIONENGINE_API void AppendMeshDescription(const FMeshDescription* MeshDescription, const FString& Name, int32 StartMaterialIndex, const FTransform& StaticMeshTransform,
		FGeometryCollection* GeometryCollection, UBodySetup* BodySetup = nullptr, bool bReindexMaterials = true, bool bAddInternalMaterials = true, bool bSetInternalFromMaterialIndex = false);

	/**
	 * Get a HiRes (or LOD 0 if no HiRes available) MeshDescription for the given static mesh, and make sure it includes normals and tangents.
	 * (i.e., to pass to AppendMeshDescription)
	 */
	static GEOMETRYCOLLECTIONENGINE_API FMeshDescription* GetMaxResMeshDescriptionWithNormalsAndTangents(const UStaticMesh* StaticMesh);

	/**
	 * Appends a static mesh to a GeometryCollectionComponent.
	 * @param StaticMesh : Const mesh to read vertex/normals/index data from
	 * @param Materials : Materials fetched from the StaticMeshComponent used to configure this geometry
	 * @param StaticMeshTransform : Mesh transform.
	 * @param GeometryCollection  : Collection to append the mesh into.
	 * @param bReindexMaterials	: Whether to reindex materials -- if appending multiple meshes, pass false and call ReindexMaterials afterwards
	 * @return true if succeeded in appending a mesh
	 */
	static GEOMETRYCOLLECTIONENGINE_API bool AppendStaticMesh(const UStaticMesh* StaticMesh, const TArray<UMaterialInterface*>& Materials, const FTransform& StaticMeshTransform, 
		UGeometryCollection* GeometryCollectionObject, bool bReindexMaterials = true, bool bAddInternalMaterials = true, bool bSplitComponents = false, bool bSetInternalFromMaterialIndex = false);

	// TODO: consider having the below fn return bool, and have a helper that returns the StartMaterialIndex from UGeometryCollection
	//  before any materials are actually added, so we can call the AppendMaterial function after, and not need to use its return value ...
	//  This would let the is-there-a-valid-mesh logic live inside this function
	// NOTE: and similar change will need to propagate to similar places ...

	/**
	 * Appends a static mesh to a GeometryCollectionComponent.
	 * @param StaticMesh : Const mesh to read vertex/normals/index data from
	 * @param StartMaterialIndex : Index of materials to be used, e.g. as returned by AppendMaterials() (see below)
	 * @param StaticMeshTransform : Mesh transform.
	 * @param GeometryCollection  : Collection to append the mesh into.
	 * @param bReindexMaterials	: Whether to reindex materials -- if appending multiple meshes, pass false and call ReindexMaterials afterwards
	 * @return true if succeeded in appending a mesh
	 */
	static GEOMETRYCOLLECTIONENGINE_API bool AppendStaticMesh(const UStaticMesh* StaticMesh, int32 StartMaterialIndex, const FTransform& StaticMeshTransform, 
		FGeometryCollection* GeometryCollection, bool bReindexMaterials = true, bool bAddInternalMaterials = true, bool bSplitComponents = false, bool bSetInternalFromMaterialIndex = false);

	/**
	*  Appends a static mesh to a GeometryCollectionComponent.
	*  @param StaticMeshComponent : Const mesh to read vertex/normals/index data from
	*  @param StaticMeshTransform : Mesh transform.
	*  @param GeometryCollection  : Collection to append the mesh into.
	*/
	static GEOMETRYCOLLECTIONENGINE_API void AppendStaticMesh(const UStaticMesh* StaticMesh, const UStaticMeshComponent *StaticMeshComponent, const FTransform& StaticMeshTransform, UGeometryCollection* GeometryCollection,
		bool bReindexMaterials = true, bool bAddInternalMaterials = true, bool bSplitComponents = false, bool bSetInternalFromMaterialIndex = false);

	/**
	*  Appends a skeletal mesh to a GeometryCollectionComponent.
	*  @param SkeletalMeshComponent : Const mesh to read vertex/normals/index data from
	*  @param SkeletalMeshTransform : Mesh transform.
	*  @param GeometryCollection    : Collection to append the mesh into.
	*/
	static GEOMETRYCOLLECTIONENGINE_API void AppendSkeletalMesh(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent *SkeletalMeshComponent, const FTransform& SkeletalMeshTransform, UGeometryCollection* GeometryCollection, bool bReindexMaterials = true);

	/**
	*  Appends a skeletal mesh to a GeometryCollection.
	*  @param SkeletalMesh : Const mesh to read vertex/normals/index data from
	*  @param MaterialStartIndex : First index of materials to use, as returned by AppendSkeletalMeshMaterials
	*  @param SkeletalMeshTransform : Mesh transform.
	*  @param GeometryCollection    : Collection to append the mesh into.
	*/
	static GEOMETRYCOLLECTIONENGINE_API bool AppendSkeletalMesh(const USkeletalMesh* SkeletalMesh, int32 MaterialStartIndex, const FTransform& SkeletalMeshTransform, FManagedArrayCollection* InCollection, bool bReindexMaterials = true, bool bImportTransformOnly = false);

	/**
	*  Appends a skeleton mesh to a GeometryCollection.
	*  @param USkeleton : Const mesh to read vertex/normals/index data from
	*  @param SkeletalMeshTransform : Mesh transform.
	*  @param GeometryCollection    : Collection to append the mesh into.
	*/
	static GEOMETRYCOLLECTIONENGINE_API void AppendSkeleton(const USkeleton* Skeleton, const FTransform& SkeletalMeshTransform, FManagedArrayCollection* InCollection);

	/**
	*  Appends a skeletal mesh to a GeometryCollectionComponent.
	*  @param SkeletalMeshComponent : Const mesh to read vertex/normals/index data from
	*  @param SkeletalMeshTransform : Mesh transform.
	*  @param GeometryCollection    : Collection to append the mesh into.
	*/
	static GEOMETRYCOLLECTIONENGINE_API int32 AppendSkeletalMeshMaterials(const USkeletalMesh* SkeletalMesh, const USkeletalMeshComponent* SkeletalMeshComponent, UGeometryCollection* GeometryCollectionObject);

	static GEOMETRYCOLLECTIONENGINE_API const FSkeletalMeshLODRenderData* GetSkeletalMeshLOD(const USkeletalMesh* SkeletalMesh, int32 LOD);

	static GEOMETRYCOLLECTIONENGINE_API int32 AppendGeometryCollectionMaterials(const UGeometryCollection* SourceGeometryCollection, const UGeometryCollectionComponent* GeometryCollectionComponent, UGeometryCollection* TargetGeometryCollectionObject);

	static GEOMETRYCOLLECTIONENGINE_API void AppendGeometryCollectionInstancedMeshes(const UGeometryCollection* SourceGeometryCollectionObject, UGeometryCollection* TargetGeometryCollectionObject, int32 TargetTransformStartIndex);

	/**
	 * Appends a GeometryCollection to another GeometryCollection.
	 * @param SourceGeometryCollection : Const GeometryCollection to read vertex/normals/index data from
	 * @param MaterialStartIndex : First index of materials to use
	 * @param GeometryCollectionTransform : GeometryCollection transform.
	 * @param TargetGeometryCollection  : Collection to append the GeometryCollection into.
	 * @param bReindexMaterials	: Whether to reindex materials -- if appending multiple meshes, pass false and call ReindexMaterials afterwards
	 * @return true if succeeded in appending a geometry collection
	 */
	static GEOMETRYCOLLECTIONENGINE_API bool AppendGeometryCollection(const FGeometryCollection* SourceGeometryCollection, int32 MaterialStartIndex, const FTransform& GeometryCollectionTransform, FGeometryCollection* TargetGeometryCollection, bool bReindexMaterials = true);

	/**
	*  Appends a GeometryCollection to a GeometryCollectionComponent.
	*  @param SourceGeometryCollection : Const GeometryCollection to read vertex/normals/index data from
	*  @param Materials : Materials fetched from the GeometryCollectionComponent used to configure this geometry
	*  @param GeometryCollectionTransform : GeometryCollection transform.
	*  @param TargetGeometryCollection  : Collection to append the GeometryCollection into.
	*/
	static GEOMETRYCOLLECTIONENGINE_API void AppendGeometryCollection(const UGeometryCollection* SourceGeometryCollection, const TArray<UMaterialInterface*>& Materials, const FTransform& GeometryCollectionTransform, UGeometryCollection* TargetGeometryCollectionObject, bool ReindexMaterials = true);

	/**
	*  Appends a GeometryCollection to a GeometryCollectionComponent.
	*  @param GeometryCollectionComponent : Const GeometryCollection to read vertex/normals/index data from
	*  @param GeometryCollectionTransform : GeometryCollection transform.
	*  @param TargetGeometryCollection  : Collection to append the GeometryCollection into.
	*/
	static GEOMETRYCOLLECTIONENGINE_API void AppendGeometryCollection(const UGeometryCollection* SourceGeometryCollection, const UGeometryCollectionComponent* GeometryCollectionComponent, const FTransform& GeometryCollectionTransform, UGeometryCollection* TargetGeometryCollectionObject, bool ReindexMaterials = true);

	/**
	*  Appends a GeometryCollectionSource to a GeometryCollection
	*  @param GeometryCollectionSource : geometry collection source object ( from UGeometryCollection collection asset ) 
	*  @param GeometryCollectionInOut : GeometryCollection to append to 
	*  @param MaterialsInOut  : array of materials to append to
	*/
	static GEOMETRYCOLLECTIONENGINE_API void AppendGeometryCollectionSource(const FGeometryCollectionSource& GeometryCollectionSource, FGeometryCollection& GeometryCollectionInOut, TArray<UMaterial*>& MaterialsInOut, bool ReindexMaterials = true);
	static GEOMETRYCOLLECTIONENGINE_API void AppendGeometryCollectionSource(const FGeometryCollectionSource& GeometryCollectionSource, FGeometryCollection& GeometryCollectionInOut, TArray<UMaterialInterface*>& MaterialInstancesInOut, bool ReindexMaterials = true);

	/**
	*  Converts a StaticMesh to a GeometryCollection
	*  @param StaticMesh : Const mesh to read vertex/normals/index data from
	*  @param OutCollection : FGeometryCollection output
	*  @param OutMaterials : materials from the StaticMesh
	*  @param OutInstancedMeshes : InstancedMeshes
	*  @param bSetInternalFromMaterialIndex : Set the internal faces using the materials
	*  @param bSplitComponents : Split the components
	*/
	static GEOMETRYCOLLECTIONENGINE_API void ConvertStaticMeshToGeometryCollection(const TObjectPtr<UStaticMesh> StaticMesh, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterial>>& OutMaterials, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes, bool bSetInternalFromMaterialIndex = true, bool bSplitComponents = false);
	static GEOMETRYCOLLECTIONENGINE_API void ConvertStaticMeshToGeometryCollection(const TObjectPtr<UStaticMesh> StaticMesh, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterialInterface>>& OutMaterialInstances, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes, bool bSetInternalFromMaterialIndex = true, bool bSplitComponents = false);
	static GEOMETRYCOLLECTIONENGINE_API void ConvertStaticMeshToGeometryCollection(const TObjectPtr<UStaticMesh> StaticMesh, const FTransform& MeshTransform, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterialInterface>>& OutMaterialInstances, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes, bool bSetInternalFromMaterialIndex = true, bool bSplitComponents = false);

	/**
	*  Converts a UGeometryCollection asset to an FManagedArrayCollection, and arrays of materials and instanced meshes
	*  @param GeometryCollection : UGeometryCollection input to convert
	*  @param OutCollection : FGeometryCollection output
	*  @param OutMaterials : Materials from the UGeometryCollection
	*  @param OutInstancedMeshes : InstancedMeshes
	*/	
	static GEOMETRYCOLLECTIONENGINE_API void ConvertGeometryCollectionToGeometryCollection(const TObjectPtr<UGeometryCollection> InGeometryCollectionAssetPtr, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterial>>& OutMaterials, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes);
	static GEOMETRYCOLLECTIONENGINE_API void ConvertGeometryCollectionToGeometryCollection(const TObjectPtr<UGeometryCollection> InGeometryCollectionAssetPtr, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterialInterface>>& OutMaterialInstances, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes);

	/**
	*  Converts an actor to an FManagedArrayCollection, and arrays of materials and instanced meshes
	*  @param Actor : actor input to convert
	*  @param OutCollection : FGeometryCollection output
	*  @param OutMaterials : Materials from the UGeometryCollection
	*  @param OutInstancedMeshes : InstancedMeshes
	*  @param bSplitComponents : Split the components
	*/
	static GEOMETRYCOLLECTIONENGINE_API void ConvertActorToGeometryCollection(const AActor* Actor, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterial>>& OutMaterials, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes, const FSkeletalMeshToCollectionConversionParameters& ConversionParameters, bool bSplitComponents = false);
	static GEOMETRYCOLLECTIONENGINE_API void ConvertActorToGeometryCollection(const AActor* Actor, FManagedArrayCollection& OutCollection, TArray<TObjectPtr<UMaterialInterface>>& OutMaterialInstances, TArray<FGeometryCollectionAutoInstanceMesh>& OutInstancedMeshes, const FSkeletalMeshToCollectionConversionParameters& ConversionParameters, bool bSplitComponents = false);

	/**
	*  Converts an FManagedArrayCollection to a USkeleton
	*  @param InManagedArrayCollection : Collection that supports transform hierarchy
	*  @param OutSkeleton : USkeleton to populate
	*  @param OutIndexRemap : Remapping from Collection transforms to skeletal transforms.
	*/
	static GEOMETRYCOLLECTIONENGINE_API void ConvertCollectionToSkeleton(const FManagedArrayCollection& InCollection, USkeleton* OutSkeleton, TArray<int32>& OutIndexRemap, bool bCenterBonesOnGeometry  = false);

	/**
	*  Converts a Geometry collection to a SkeletalMesh
	*  @param InCollection: Geometry collection to convert to a skeletal mesh
	*  @param OutSkeletalMesh : skeletal mesh asset to export to
	*  @param OutSkeleton : Skeleton asset to export the bone hierarchy to
	*/
	static GEOMETRYCOLLECTIONENGINE_API void ConvertCollectionToSkeletalMesh(const FManagedArrayCollection& InCollection, const TArray<TObjectPtr<UMaterialInterface>> InMaterials, USkeletalMesh& OutSkeletalMesh, USkeleton& OutSkeleton);

	/** utility function to convert an array of material instances to an array of material */
	static GEOMETRYCOLLECTIONENGINE_API void GetMaterialsFromInstances(const TArray<TObjectPtr<UMaterialInterface>>& MaterialInstances, TArray<TObjectPtr<UMaterial>>& OutMaterials);

private:
	/**
	*  Appends a GeometryCollectionSource to a GeometryCollection ignoring the materials
	*  this is meant to be used by the AppendGeometryCollectionSource variants
	*  @param GeometryCollectionSource : geometry collection source object ( from UGeometryCollection collection asset )
	*  @param GeometryCollectionInOut : GeometryCollection to append to
	*  @return true if the source was added successfully ( valid object )
	*/
	static bool AppendGeometryCollectionSourceNoMaterial(const FGeometryCollectionSource& GeometryCollectionSource, FGeometryCollection& GeometryCollectionInOut, int32 StartMaterialIndex, bool ReindexMaterials = true);
};
