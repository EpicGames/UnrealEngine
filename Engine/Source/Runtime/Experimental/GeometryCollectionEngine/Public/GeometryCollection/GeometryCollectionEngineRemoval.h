// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/ManagedArrayAccessor.h"

struct FManagedArrayCollection;
class FGeometryDynamicCollection;

namespace GeometryCollection::Facades
{
	class FCollectionRemoveOnBreakFacade;
}

/**
 * Provides an API for the run time aspect of the remove on break feature
 * this is to be used with the dynamic collection
 */
class FGeometryCollectionRemoveOnBreakDynamicFacade
{
public:
	static constexpr float DisabledBreakTimer = -1;
	static constexpr float BreakTimerStartValue = 0;
	static constexpr float DisabledPostBreakDuration = -1;
	static constexpr float CrumblingRemovalTimer = -1;
	
	GEOMETRYCOLLECTIONENGINE_API FGeometryCollectionRemoveOnBreakDynamicFacade(FGeometryDynamicCollection& InCollection);

	/** 
	 * returns true if all the necessary attributes are present
	 * if not then the API can be used to create  
	 */
	GEOMETRYCOLLECTIONENGINE_API bool IsValid() const;

	/** Is this facade const access */
	GEOMETRYCOLLECTIONENGINE_API bool IsConst() const;

	/** Add the relevant attributes */
	GEOMETRYCOLLECTIONENGINE_API void DefineSchema();

	/**
	 * Add the necessary attributes if they are missing and initialize them if necessary
	 * @param RemoveOnBreakFacade remove on break facade from the rest collection that contain the original user set attributes
	*/
	GEOMETRYCOLLECTIONENGINE_API void SetAttributeValues(const GeometryCollection::Facades::FCollectionRemoveOnBreakFacade& RemoveOnBreakFacade);

	/** true if the removal is active for a specific piece */
	GEOMETRYCOLLECTIONENGINE_API bool IsRemovalActive(int32 TransformIndex) const;

	/** true if a specific transform uses cluster crumbling */
	GEOMETRYCOLLECTIONENGINE_API bool UseClusterCrumbling(int32 TransformIndex) const;
	
	/**
	 * Update break timer and return the matching decay  
	 * @param TransformIndex index of the transform to update
	 * @param DeltaTime elapsed time since the last update in second
	 * @return decay value computed from the timer and duration ( [0,1] range )
	 */
	GEOMETRYCOLLECTIONENGINE_API float UpdateBreakTimerAndComputeDecay(int32 TransformIndex, float DeltaTime);
	
private:
	/** Time elapsed since the break in seconds */
	TManagedArrayAccessor<float> BreakTimerAttribute;

	/** duration after the break before the removal process starts */
	TManagedArrayAccessor<float> PostBreakDurationAttribute;
	
	/** removal duration */
	TManagedArrayAccessor<float> BreakRemovalDurationAttribute;

	const FGeometryDynamicCollection& DynamicCollection;
};

/**
 * Provides an API for the run time aspect of the remove on sleep feature
 * this is to be used with the dynamic collection
 */
class FGeometryCollectionRemoveOnSleepDynamicFacade
{
public:
	GEOMETRYCOLLECTIONENGINE_API FGeometryCollectionRemoveOnSleepDynamicFacade(FManagedArrayCollection& InCollection);

	/** 
	 * returns true if all the necessary attributes are present
	 * if not then the API can be used to create  
	 */
	GEOMETRYCOLLECTIONENGINE_API bool IsValid() const;

	/** Is this facade const access */
	GEOMETRYCOLLECTIONENGINE_API bool IsConst() const;

	/** Add the relevant attributes */
	GEOMETRYCOLLECTIONENGINE_API void DefineSchema();

	GEOMETRYCOLLECTIONENGINE_API float GetSleepRemovalDuration(int32 TransformIndex) const;
	GEOMETRYCOLLECTIONENGINE_API float GetMaxSleepTime(int32 TransformIndex) const;

	GEOMETRYCOLLECTIONENGINE_API void SetSleepRemovalDuration(int32 TransformIndex, float SleepRemovalDuration);
	GEOMETRYCOLLECTIONENGINE_API void SetMaxSleepTime(int32 TransformIndex, float MaxSleepTime);

	/**
	 * Add the necessary attributes if they are missing and initialize them if necessary
	 * @param MaximumSleepTime range of time to initialize the sleep duration
	 * @param RemovalDuration range of time to initialize the removal duration
	*/
	GEOMETRYCOLLECTIONENGINE_API void SetAttributeValues(const FVector2D& MaximumSleepTime, const FVector2D& RemovalDuration);

	/** true if the removal is active for a specific piece */
	GEOMETRYCOLLECTIONENGINE_API bool IsRemovalActive(int32 TransformIndex) const;

	/**
	 * Compute the slow moving state and update from the last position
	 * After calling this method, LastPosition will be updated with Position
	 * @param Position Current world position
	 * @param DeltaTime elapsed time since last update
	 * @return true if the piece if the piece is considered slow moving 
	 **/
	GEOMETRYCOLLECTIONENGINE_API bool ComputeSlowMovingState(int32 TransformIndex, const FVector& Position, float DeltaTime, FVector::FReal VelocityThreshold);
	
	/**
	 * Update the sleep timer
	 * @param TransformIndex index of the transform to update
	 * @param DeltaTime elapsed time since last update
	 */
	GEOMETRYCOLLECTIONENGINE_API void UpdateSleepTimer(int32 TransformIndex, float DeltaTime);

	/** Compute decay from elapsed timer and duration attributes */
	GEOMETRYCOLLECTIONENGINE_API float ComputeDecay(int32 TransformIndex) const;

	
private:
	/** Time elapsed since the sleep detection */
	TManagedArrayAccessor<float> SleepTimerAttribute;
	
	/** duration after the sleep detection before the removal process starts (read only from outside) */
	TManagedArrayAccessor<float> MaxSleepTimeAttribute;
	
	/** removal duration (read only from outside) */
	TManagedArrayAccessor<float> SleepRemovalDurationAttribute;

	/** Last position used to detect slow moving pieces */
	TManagedArrayAccessor<FVector> LastPositionAttribute;
};

/**
 * Provides an API for decay related attributes ( use for remove on break and remove on sleep )
 */
class FGeometryCollectionDecayDynamicFacade
{
public:
	GEOMETRYCOLLECTIONENGINE_API FGeometryCollectionDecayDynamicFacade(FManagedArrayCollection& InCollection);

	/** 
	 * returns true if all the necessary attributes are present
	 * if not then the API can be used to create  
	 */
	GEOMETRYCOLLECTIONENGINE_API bool IsValid() const;

	/** Add the necessary attributes if they are missing and initialize them if necessary */
	GEOMETRYCOLLECTIONENGINE_API void AddAttributes();

	/** Get decay value for a specific transform index */
	GEOMETRYCOLLECTIONENGINE_API float GetDecay(int32 TransformIndex) const;

	/** Set decay value for a specific transform index */
	GEOMETRYCOLLECTIONENGINE_API void SetDecay(int32 TransformIndex, float DecayValue);

	/** Get the size of the decay attribute - this should match the number of transforms of the collection */
	GEOMETRYCOLLECTIONENGINE_API int32 GetDecayAttributeSize() const;

private:
	/** state of decay ([0-1] range) */
	TManagedArrayAccessor<float> DecayAttribute;
};
