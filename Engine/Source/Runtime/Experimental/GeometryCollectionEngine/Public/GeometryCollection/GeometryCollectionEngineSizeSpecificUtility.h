// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/GeometryCollectionObject.h"

namespace GeometryCollection
{
	namespace SizeSpecific
	{
		//
		// Return if true/false if this size specific object uses the specified collision type. 
		//
		bool GEOMETRYCOLLECTIONENGINE_API UsesImplicitCollisionType(const TArray<FGeometryCollectionSizeSpecificData>& SizeSpecificData, EImplicitTypeEnum ImplicitType);

		//
		// Set the collision types for a size specific data, also will only set the types matching ImplicitTypeFrom if specified.
		//
		void GEOMETRYCOLLECTIONENGINE_API SetImplicitCollisionType(TArray<FGeometryCollectionSizeSpecificData>& SizeSpecificData, EImplicitTypeEnum ImplicitTypeTo, EImplicitTypeEnum ImplicitTypeFrom = EImplicitTypeEnum::Chaos_Implicit_None);

	}
}