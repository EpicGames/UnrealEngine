// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "UObject/ObjectMacros.h"
#include "GeometryCollectionProxyData.h"
#include "GeometryCollection/GeometryCollection.h"

#include "GeometryCollectionEngineTypes.generated.h"

UENUM()
enum class UE_DEPRECATED(5.4, "ECollectionAttributeEnum is no longer supported") ECollectionAttributeEnum : uint8
{
	Chaos_Active           UMETA(DisplayName = "Active"),
	Chaos_DynamicState     UMETA(DisplayName = "DynamicState"),
	Chaos_CollisionGroup   UMETA(DisplayName = "CollisionGroup"),
	//
	Chaos_Max              UMETA(Hidden)
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_DEPRECATED(5.4, "GetCollectionAttributeName is no longer supported")
FName GetCollectionAttributeName(ECollectionAttributeEnum TypeIn);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

UENUM()
enum class UE_DEPRECATED(5.4, "ECollectionGroupEnum is no longer supported") ECollectionGroupEnum : uint8
{
	Chaos_Traansform        UMETA(DisplayName = "Transform"),
	//
	Chaos_Max                UMETA(Hidden)
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_DEPRECATED(5.4, "GetCollectionGroupName is no longer supported")
FName GetCollectionGroupName(ECollectionGroupEnum TypeIn);
PRAGMA_ENABLE_DEPRECATION_WARNINGS