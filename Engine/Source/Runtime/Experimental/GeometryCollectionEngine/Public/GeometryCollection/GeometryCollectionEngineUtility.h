// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FGeometryCollection;
class UGeometryCollectionCache;
class USkeletalMesh;

namespace GeometryCollectionEngineUtility
{
	void GEOMETRYCOLLECTIONENGINE_API PrintDetailedStatistics(const FGeometryCollection* GeometryCollection, const UGeometryCollectionCache* InCache);

	void GEOMETRYCOLLECTIONENGINE_API PrintDetailedStatisticsSummary(const TArray<const FGeometryCollection*> GeometryCollectionArray);

	void GEOMETRYCOLLECTIONENGINE_API ComputeNormals(FGeometryCollection* GeometryCollection);

	void GEOMETRYCOLLECTIONENGINE_API ComputeTangents(FGeometryCollection* GeometryCollection);

	/***
	* Generate a vertex to component map that defines the disjoint geometries.
	* @param IndexBuffer			Input flat index of triangles
	* @param ComponentIndices		Output list of triangle indices by component
	* @param TriangleComponentMap	Output Component remapping for triangles
	* @param VertexComponentMap		Output Component remapping for vertices
	* @param TriangleCount		    Output Sum of the triangles in the Components
	* @param VertexCount		    Output Num of vertices in the Components
	* @return If the remapping was valid.
	*/
	void
	GEOMETRYCOLLECTIONENGINE_API
	GenerateConnectedComponents(const USkeletalMesh* InSkeletalMesh, 
		TArray<TArray<FIntVector>>& ComponentIndices,
		TArray<TArray<FIntVector2>>& TrangleComponentMap, 
		TArray<int32>& VertexComponentMap, 
		int32& TriangleCount, int32& VertexCount);
}