// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "GeometryCollectionExternalRenderInterface.generated.h"

class UGeometryCollection;
class UGeometryCollectionComponent;

UINTERFACE(MinimalAPI)
class UGeometryCollectionExternalRenderInterface : public UInterface
{
	GENERATED_BODY()
};

class IGeometryCollectionExternalRenderInterface
{
	GENERATED_BODY()

public:
	enum EStateFlags
	{
		EState_Visible		= 1 << 0,
		EState_Broken		= 1 << 1,
		EState_ForcedBroken	= 1 << 2,
	};

	/** Returns true if the custom renderer can't render for the given state, and requires native rendering instead. */
	virtual bool ShouldUseNativeFallback(uint32 InStateFlags) const { return false; }
	/** Returns true if this renderer can ever return true for ShouldUseNativeFallback(). */
	virtual bool CanEverUseNativeFallback() const { return false; }

	/** Called on geometry collection component registration. */
	virtual void OnRegisterGeometryCollection(UGeometryCollectionComponent& InComponent) = 0;
	/** Called on geometry collection component unregistration. */
	virtual void OnUnregisterGeometryCollection() = 0;

	/** 
	 * Set the state of the geometry collection.
	 * This is used by the renderer to managed resources with regards to the state (see EStateFlags).
	 */
	virtual void UpdateState(UGeometryCollection const& InGeometryCollection, FTransform const& InComponentTransform, uint32 InStateFlags) = 0;

	/**
	 * Update the root bone transform of the geometry collection.
	 * If the geometry collection is using of multiple proxy root meshes this transform applies to all of them.
	 */
	virtual void UpdateRootTransform(UGeometryCollection const& InGeometryCollection, FTransform const& InRootTransform) = 0;

	/**
	 * Update the root proxy transforms of the geometry collection.
	 * If the geometry collection is using of multiple proxy root meshes, InRootTransforms is expected to contain an entry for each of them .
	 * @param InRootTransform		Component space root transform
	 * @param InRootLocalTransforms	Root space local transforms
	 */
	virtual void UpdateRootTransforms(UGeometryCollection const& InGeometryCollection, FTransform const& InRootTransform, TArrayView<const FTransform3f> InRootLocalTransforms) { check(false); };

	/**
	 * Update all the bones transforms.
	 */
	virtual void UpdateTransforms(UGeometryCollection const& InGeometryCollection, TArrayView<const FTransform3f> InTransforms) = 0;

	UE_DEPRECATED(5.4, "Use flags version of UpdateState instead")
	virtual void UpdateState(UGeometryCollection const& InGeometryCollection, FTransform const& InComponentTransform, bool bInIsBroken, bool bInIsVisible) 
	{
		uint32 StateFlags = 0;
		StateFlags |= bInIsVisible ? IGeometryCollectionExternalRenderInterface::EState_Visible : 0;
		StateFlags |= bInIsBroken ? IGeometryCollectionExternalRenderInterface::EState_Broken : 0;
		return UpdateState(InGeometryCollection, InComponentTransform, StateFlags);
	};
	UE_DEPRECATED(5.3, "Use FTransform version of UpdateTransforms instead")
	virtual void UpdateTransforms(UGeometryCollection const& InGeometryCollection, TArrayView<const FMatrix> InMatrices) {};
};


UINTERFACE(MinimalAPI)
class UGeometryCollectionCustomDataInterface : public UInterface
{
	GENERATED_BODY()
};

class IGeometryCollectionCustomDataInterface
{
	GENERATED_BODY()

public:
	/** Set value to a custom instance data slot by index. The value will be set for all instances the renderer is spawning. */
	virtual void SetCustomInstanceData(int32 CustomDataIndex, float CustomDataValue) = 0;

	/** Set value a custom instance data slot by name. The value will be set for all instances the renderer is spawning. */
	virtual void SetCustomInstanceData(FName CustomDataName, float CustomDataValue) = 0;

	/** Enable or disable custom depth writing and, if enabled, the value written to the stencil buffer. The value will be set for all instances the renderer is spawning. */
	virtual void SetRenderCustomDepthStencil(bool bEnable, int32 CustomStencilValue) = 0;
};
