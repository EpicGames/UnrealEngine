// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_EDITOR

#include "EngineUtils.h"

class UGeometryCollectionComponent;

/** HitProxy with transform index information. */
struct HGeometryCollection : public HHitProxy
{
	DECLARE_HIT_PROXY(GEOMETRYCOLLECTIONENGINE_API)

	TObjectPtr<UGeometryCollectionComponent> Component;
	int32 BoneIndex;

	HGeometryCollection(UGeometryCollectionComponent* InGeometryCollectionComponent, int32 InBoneIndex)
		: Component(InGeometryCollectionComponent)
		, BoneIndex(InBoneIndex)
	{}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(Component);
	}
};

#endif // WITH_EDITOR
