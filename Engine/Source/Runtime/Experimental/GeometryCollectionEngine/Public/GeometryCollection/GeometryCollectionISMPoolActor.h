// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GameFramework/Actor.h"
#include "GeometryCollectionISMPoolActor.generated.h"

class UE_DEPRECATED(5.6, "UGeometryCollectionISMPoolComponent is deprecated, please use UISMPoolComponent instead.") UGeometryCollectionISMPoolComponent;
class UE_DEPRECATED(5.6, "UGeometryCollectionISMPoolDebugDrawComponent is deprecated, please use UISMPoolDebugDrawComponent instead.") UGeometryCollectionISMPoolDebugDrawComponent;

UCLASS(ConversionRoot, ComponentWrapperClass, MinimalAPI)
class UE_DEPRECATED(5.6, "AGeometryCollectionISMPoolActor is deprecated, please use AISMPoolActor instead.") AGeometryCollectionISMPoolActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(Category = ISMPoolActor, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UGeometryCollectionISMPoolComponent> ISMPoolComp;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UPROPERTY(Category = ISMPoolActor, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UGeometryCollectionISMPoolDebugDrawComponent> ISMPoolDebugDrawComp;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

public:
	/** Returns ISMPoolComp subobject **/
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UGeometryCollectionISMPoolComponent* GetISMPoolComp() const { return ISMPoolComp; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

