// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "Containers/Map.h"
#include "InstancedStaticMeshDelegates.h"
#include "Materials/MaterialInterface.h"
#include "InstanceDataTypes.h"

#include "GeometryCollectionISMPoolComponent.generated.h"

class AActor;
class UE_DEPRECATED(5.6, "UGeometryCollectionISMPoolComponent is deprecated, please use UISMPoolComponent instead.") UGeometryCollectionISMPoolComponent;
class UInstancedStaticMeshComponent;

/** 
 * Structure containing a set of allocated instance ranges in an FGeometryCollectionISM which is the manager for a single ISM component.
 * The instance ranges don't change once allocated, and aren't the same as the actual render indices in the ISM.
 * The reason that we don't store the the actual ISM render indices is that ISM component is free to reorder its instances whenever it likes.
 */
struct FInstanceGroups
{
	using FInstanceGroupId = int32;

	/** A single continuous range associated with an FInstanceGroupId. */
	struct FInstanceGroupRange
	{
		FInstanceGroupRange(int32 InStart, int32 InCount)
			: Start(InStart)
			, Count(InCount)
		{
		}

		int32 Start = 0;
		int32 Count = 0;
	};

	/** Reset all contents. */
	void Reset()
	{
		TotalInstanceCount = 0;
		TotalFreeInstanceCount = 0;
		GroupRanges.Empty();
		FreeList.Empty();
	}

	/** Returns true if no groups ranges are in use. */
	bool IsEmpty() const
	{
		return GroupRanges.Num() == FreeList.Num();
	}

	/** Returns the maximum instance index that we have allocated in a group. */
	int32 GetMaxInstanceIndex() const
	{
		return TotalInstanceCount;
	}

	/** Add a new group range. */
	FInstanceGroupId AddGroup(int32 Count)
	{
		// First check to see if we can recycle a group from the free list.
		for (int32 Index = 0; Index < FreeList.Num(); ++Index)
		{
			const FInstanceGroupId GroupId = FreeList[Index];
			// todo: Could also allow allocating a subrange if Count is less than the group range count.
			if (Count == GroupRanges[GroupId].Count)
			{
				TotalFreeInstanceCount -= Count;
				FreeList.RemoveAtSwap(Index, EAllowShrinking::No);
				return GroupId;
			}
		}
		
		// Create a new group.
		const int32 StartIndex = TotalInstanceCount;
		TotalInstanceCount += Count;
		const FInstanceGroupId GroupId = GroupRanges.Add(FInstanceGroupRange(StartIndex, Count));
		return GroupId;
	}

	/** Remove a group range. */
	void RemoveGroup(FInstanceGroupId GroupId)
	{
		// Remove the group by putting on free list for reuse.
		// Actually removing it would require too much shuffling of the render instance index remapping.
		TotalFreeInstanceCount += GroupRanges[GroupId].Count;
		FreeList.Add(GroupId);
	}

	int32 TotalInstanceCount = 0;
	int32 TotalFreeInstanceCount = 0;
	TArray<FInstanceGroupRange> GroupRanges;
	TArray<FInstanceGroupId> FreeList;
};

/** 
 * A description for an ISM component.
 */
struct FISMComponentDescription
{
	enum EFlags
	{
		UseHISM = 1 << 1,							// HISM is no longer supported. This flag is ignored.
		GpuLodSelection = 1 << 2,
		ReverseCulling = 1 << 3,
		StaticMobility = 1 << 4,
		WorldPositionOffsetWritesVelocity = 1 << 5,
		EvaluateWorldPositionOffset = 1 << 6,
		AffectShadow = 1 << 7,
		AffectDistanceFieldLighting = 1 << 8,
		AffectDynamicIndirectLighting = 1 << 9,
		AffectFarShadow = 1 << 10,
		DistanceCullPrimitive = 1 << 11,
	};

	uint32 Flags = WorldPositionOffsetWritesVelocity|EvaluateWorldPositionOffset|AffectShadow;
	int32 NumCustomDataFloats = 0;
	FVector Position = FVector::ZeroVector;
	int32 StartCullDistance = 0;
	int32 EndCullDistance = 0;
	int32 MinLod = 0;
	uint32 GroupHash = 0;	// Optional, allows identical SMs to be separated into different groups for finer grained culling
	float LodScale = 1.f;
	TArray<FName> Tags;
	FName StatsCategory;

	bool operator==(const FISMComponentDescription& Other) const
	{
		return Flags == Other.Flags &&
			NumCustomDataFloats == Other.NumCustomDataFloats &&
			Position == Other.Position &&
			StartCullDistance == Other.StartCullDistance &&
			EndCullDistance == Other.EndCullDistance &&
			MinLod == Other.MinLod &&
			LodScale == Other.LodScale &&
			Tags == Other.Tags &&
			GroupHash == Other.GroupHash &&
			StatsCategory == Other.StatsCategory;
	}
};

inline uint32 GetTypeHash(const FISMComponentDescription& Desc)
{
	uint32 Hash = HashCombineFast(GetTypeHash(Desc.Flags), GetTypeHash(Desc.NumCustomDataFloats));
	Hash = HashCombineFast(Hash, GetTypeHash(Desc.Position));
	Hash = HashCombineFast(Hash, GetTypeHash(Desc.StartCullDistance));
	Hash = HashCombineFast(Hash, GetTypeHash(Desc.EndCullDistance));
	Hash = HashCombineFast(Hash, GetTypeHash(Desc.MinLod));
	Hash = HashCombineFast(Hash, GetTypeHash(Desc.LodScale));
	Hash = HashCombineFast(Hash, GetTypeHash(Desc.GroupHash));
	Hash = HashCombineFast(Hash, GetArrayHash(Desc.Tags.GetData(), Desc.Tags.Num()));
	return HashCombineFast(Hash, GetTypeHash(Desc.StatsCategory));
}

/**
 * A mesh with potentially overriden materials and ISM property description.
 * We batch instances into ISMs that have equivalent values for this structure.
 */
struct FGeometryCollectionStaticMeshInstance
{
	TWeakObjectPtr<UStaticMesh> StaticMesh;
	TArray<TWeakObjectPtr<UMaterialInterface>> MaterialsOverrides;
	TArray<float> CustomPrimitiveData;
	FISMComponentDescription Desc;

	bool operator==(const FGeometryCollectionStaticMeshInstance& Other) const 
	{
		if (!(Desc == Other.Desc))
		{
			return false;
		}
		if (!StaticMesh.HasSameIndexAndSerialNumber(Other.StaticMesh))
		{
			return false;
		}
		if (MaterialsOverrides.Num() != Other.MaterialsOverrides.Num())
		{
			return false;
		}
		for (int32 MatIndex = 0; MatIndex < MaterialsOverrides.Num(); MatIndex++)
		{
			if (!MaterialsOverrides[MatIndex].HasSameIndexAndSerialNumber(Other.MaterialsOverrides[MatIndex]))
			{
				return false;
			}
		}
		if (CustomPrimitiveData.Num() != Other.CustomPrimitiveData.Num())
		{
			return false;
		}
		for (int32 DataIndex = 0; DataIndex < CustomPrimitiveData.Num(); DataIndex++)
		{
			if (CustomPrimitiveData[DataIndex] != Other.CustomPrimitiveData[DataIndex])
			{
				return false;
			}
		}

		return true;
	}
};

inline uint32 GetTypeHash(const FGeometryCollectionStaticMeshInstance& MeshInstance)
{
	uint32 CombinedHash = GetTypeHash(MeshInstance.StaticMesh);
	CombinedHash = HashCombineFast(CombinedHash, GetTypeHash(MeshInstance.MaterialsOverrides.Num()));
	for (const TWeakObjectPtr<UMaterialInterface> Material: MeshInstance.MaterialsOverrides)
	{
		CombinedHash = HashCombineFast(CombinedHash, GetTypeHash(Material));
	}
	for (const float CustomFloat : MeshInstance.CustomPrimitiveData)
	{
		CombinedHash = HashCombineFast(CombinedHash, GetTypeHash(CustomFloat));
	}
	CombinedHash = HashCombineFast(CombinedHash, GetTypeHash(MeshInstance.Desc));
	return CombinedHash;
}

/** Describes a group of instances within an ISM. */
struct FGeometryCollectionMeshInfo
{
	int32 ISMIndex;
	FInstanceGroups::FInstanceGroupId InstanceGroupIndex;

	TArray<float> CustomData;
	TArrayView<const float> CustomDataSlice(int32 InstanceIndex, int32 NumCustomDataFloatsPerInstance);
	void ShadowCopyCustomData(int32 InstanceCount, int32 NumCustomDataFloatsPerInstance, TArrayView<const float> CustomDataFloats);
};

struct FGeometryCollectionISMPool;

/**
 * A mesh group which is a collection of meshes and their related FGeometryCollectionMeshInfo.
 * We group these with a single handle with the expectation that a client will want to own multiple meshs and release them together.
 */
struct FGeometryCollectionMeshGroup
{
	using FMeshId = int32;

	/** Adds a new mesh with instance count. We expect to only add a unique mesh instance once to each group. Returns a ID that can be used to update the instances. */
	FMeshId AddMesh(const FGeometryCollectionStaticMeshInstance& MeshInstance, int32 InstanceCount, const FGeometryCollectionMeshInfo& ISMInstanceInfo, TArrayView<const float> CustomDataFloats);
	/** Update instance transforms for a group of instances. */
	bool BatchUpdateInstancesTransforms(FGeometryCollectionISMPool& ISMPool, FMeshId MeshId, int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport);
	void BatchUpdateInstanceCustomData(FGeometryCollectionISMPool& ISMPool, int32 CustomFloatIndex, float CustomFloatValue);

	/** Remove all of our managed meshes and associated instances. */
	void RemoveAllMeshes(FGeometryCollectionISMPool& ISMPool);

	/** Array of allocated mesh infos. */
	TArray<FGeometryCollectionMeshInfo> MeshInfos;

	/** Flag for whether we allow removal of instances when transform scale is set to zero. */
	bool bAllowPerInstanceRemoval = false;
};

/** Structure containting all info for a single ISM. */
struct FGeometryCollectionISM
{
	/** Create the ISMComponent according to settings on the mesh instance. */
	void CreateISM(USceneComponent* InOwningComponent);
	/** Initialize the ISMComponent according to settings on the mesh instance. */
	void InitISM(const FGeometryCollectionStaticMeshInstance& InMeshInstance, bool bKeepAlive, bool bOverrideTransformUpdates = false);
	/** Add a group to the ISM. Returns the group index. */
	FInstanceGroups::FInstanceGroupId AddInstanceGroup(int32 InstanceCount, TArrayView<const float> CustomDataFloats);

	/** Unique description of ISM component settings. */
	FGeometryCollectionStaticMeshInstance MeshInstance;
	/** Created ISM component. Will be nullptr when this slot has been recycled to FGeometryCollectionISMPool FreeList. */
	TObjectPtr<UInstancedStaticMeshComponent> ISMComponent;
	/** Groups of instances allocated in the ISM. */
	FInstanceGroups InstanceGroups;
	/** Id of Instance in ISMC */
	TArray<FPrimitiveInstanceId> InstanceIds;
};

/** A pool of ISMs. */
struct FGeometryCollectionISMPool
{
	using FISMIndex = int32;

	FGeometryCollectionISMPool();

	/** Find or add an ISM and return an ISM index handle. */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FISMIndex GetOrAddISM(UGeometryCollectionISMPoolComponent* OwningComponent, const FGeometryCollectionStaticMeshInstance& MeshInstance, bool& bOutISMCreated);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Remove an ISM. */
	void RemoveISM(FISMIndex ISMIndex, bool bKeepAlive, bool bRecycle);
	/** Add instances to ISM and return a mesh info handle. */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FGeometryCollectionMeshInfo AddInstancesToISM(UGeometryCollectionISMPoolComponent* OwningComponent, const FGeometryCollectionStaticMeshInstance& MeshInstance, int32 InstanceCount, TArrayView<const float> CustomDataFloats);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Remove instances from an ISM. */
	void RemoveInstancesFromISM(const FGeometryCollectionMeshInfo& MeshInfo);
	/** Update ISM contents. */
	bool BatchUpdateInstancesTransforms(FGeometryCollectionMeshInfo& MeshInfo, int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport, bool bAllowPerInstanceRemoval);
	void BatchUpdateInstanceCustomData(FGeometryCollectionMeshInfo const& MeshInfo, int32 CustomFloatIndex, float CustomFloatValue);

	/** Clear all ISM components and associated data. */
	void Clear();

	/** Tick maintenance of free list and preallocation. */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void Tick(UGeometryCollectionISMPoolComponent* OwningComponent);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Add an ISM description to the preallocation queue. */
	void RequestPreallocateMeshInstance(const FGeometryCollectionStaticMeshInstance& MeshInstances);
	/** Process the preallocation queue. Processing is timesliced so that only some of the queue will be processed in every call. */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void ProcessPreallocationRequests(UGeometryCollectionISMPoolComponent* OwningComponent, int32 MaxPreallocations);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void UpdateAbsoluteTransforms(const FTransform& BaseTransform, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport);

	/** Array of ISM objects. */
	TArray<FGeometryCollectionISM> ISMs;
	/** Mapping from mesh description to ISMs array slot. */
	TMap<FGeometryCollectionStaticMeshInstance, FISMIndex> MeshToISMIndex;
	
	/** Set of ISM descriptions that we would like to preallocate. */
	TSet<FGeometryCollectionStaticMeshInstance> PrellocationQueue;

	/** Free list of indices in ISMs that are empty. */
	TArray<int32> FreeList;
	/** Free list of indices in ISMs that have registered ISM components. */
	TArray<int32> FreeListISM;

	// Cached state of lifecycle cvars from the last Tick()
	bool bCachedKeepAlive = false;
	bool bCachedRecycle = false;

	// Whether we force ISMs to use parent bounds and disable transform updates
	bool bDisableBoundsAndTransformUpdate = false;
};


/**
* UGeometryCollectionISMPoolComponent.
* Component that manages a pool of ISM components in order to allow multiple client components that use the same meshes to the share ISMs.
*/
class UE_DEPRECATED(5.6, "UGeometryCollectionISMPoolDebugDrawComponent is deprecated, please use UISMPoolDebugDrawComponent instead.") UGeometryCollectionISMPoolDebugDrawComponent;

UCLASS(meta = (BlueprintSpawnableComponent), MinimalAPI)
class UE_DEPRECATED(5.6, "UGeometryCollectionISMPoolComponent is deprecated, please use UISMPoolComponent instead.") UGeometryCollectionISMPoolComponent: public USceneComponent
{
	GENERATED_UCLASS_BODY()

public:
	using FMeshGroupId = int32;
	using FMeshId = int32;

	//~ Begin UActorComponent Interface
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UActorComponent Interface

	/** 
	* Create an Mesh group which represent an arbitrary set of mesh with their instance 
	* no resources are created until the meshes are added for this group 
	* return a mesh group Id used to add and update instances
	*/
	GEOMETRYCOLLECTIONENGINE_API FMeshGroupId CreateMeshGroup(bool bAllowPerInstanceRemoval = false);

	/** Destroy  a mesh group and its associated resources */
	GEOMETRYCOLLECTIONENGINE_API void DestroyMeshGroup(FMeshGroupId MeshGroupId);

	/** Add a static mesh for a mesh group */
	GEOMETRYCOLLECTIONENGINE_API FMeshId AddMeshToGroup(FMeshGroupId MeshGroupId, const FGeometryCollectionStaticMeshInstance& MeshInstance, int32 InstanceCount, TArrayView<const float> CustomDataFloats);

	/** Update transforms for a mesh group */
	GEOMETRYCOLLECTIONENGINE_API bool BatchUpdateInstancesTransforms(FMeshGroupId MeshGroupId, FMeshId MeshId, int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace = false, bool bMarkRenderStateDirty = false, bool bTeleport = false);

	UE_DEPRECATED(5.3, "BatchUpdateInstancesTransforms Array parameter version is deprecated, use the TArrayView version instead")
	GEOMETRYCOLLECTIONENGINE_API bool BatchUpdateInstancesTransforms(FMeshGroupId MeshGroupId, FMeshId MeshId, int32 StartInstanceIndex, const TArray<FTransform>& NewInstancesTransforms, bool bWorldSpace = false, bool bMarkRenderStateDirty = false, bool bTeleport = false);

	/** Update a single slot of custom instance data for all instances in a mesh group */
	GEOMETRYCOLLECTIONENGINE_API bool BatchUpdateInstanceCustomData(FMeshGroupId MeshGroupId, int32 CustomFloatIndex, float CustomFloatValue);

	/** 
	 * Preallocate an ISM in the pool. 
	 * Doing this early for known mesh instance descriptions can reduce the component registration cost of AddMeshToGroup() for newly discovered mesh descriptions.
	 */
	GEOMETRYCOLLECTIONENGINE_API void PreallocateMeshInstance(const FGeometryCollectionStaticMeshInstance& MeshInstance);

	GEOMETRYCOLLECTIONENGINE_API void SetTickablePoolManagement(bool bEnablePoolManagement);

	GEOMETRYCOLLECTIONENGINE_API void SetOverrideTransformUpdates(bool bOverrideUpdates);

	GEOMETRYCOLLECTIONENGINE_API void UpdateAbsoluteTransforms(const FTransform& BaseTransform, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport);

private:
	uint32 NextMeshGroupId = 0;
	TMap<FMeshGroupId, FGeometryCollectionMeshGroup> MeshGroups;
	FGeometryCollectionISMPool Pool;

	// Expose internals for debug draw support.
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	friend UGeometryCollectionISMPoolDebugDrawComponent;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
