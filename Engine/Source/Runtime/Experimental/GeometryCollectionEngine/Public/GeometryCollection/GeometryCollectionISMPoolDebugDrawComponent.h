// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/DebugDrawComponent.h"
#include "DebugRenderSceneProxy.h"
#include "GeometryCollectionISMPoolDebugDrawComponent.generated.h"

class FGeometryCollectionISMPoolDebugDrawDelegateHelper;
class UInstancedStaticMeshComponent;

UCLASS(ClassGroup = Debug)
class UE_DEPRECATED(5.6, "UGeometryCollectionISMPoolDebugDrawComponent is deprecated, please use UISMPoolDebugDrawComponent instead.") UGeometryCollectionISMPoolDebugDrawComponent : public UDebugDrawComponent
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Debug)
	bool bShowGlobalStats = false;

	UPROPERTY(EditAnywhere, Category = Debug)
	bool bShowStats = false;

	UPROPERTY(EditAnywhere, Category = Debug)
	bool bShowBounds = false;

	UPROPERTY(Transient)
	TObjectPtr<const UInstancedStaticMeshComponent> SelectedComponent;

	float SelectTimer = 0.f;

protected:
	virtual void BeginPlay() override;
	virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	void UpdateTickEnabled();

#if UE_ENABLE_DEBUG_DRAWING
  	virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
	
	FDebugDrawDelegateHelper DebugDrawDelegateHelper;
	virtual FDebugDrawDelegateHelper& GetDebugDrawDelegateHelper() override { return DebugDrawDelegateHelper; }

	FDelegateHandle OnScreenMessagesHandle;
	void GetOnScreenMessages(TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText>& OutMessages);
#endif

public:
	static void UpdateAllTickEnabled();
};
