// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "GeometryCollectionISMPoolSubSystem.generated.h"

class AGeometryCollectionISMPoolActor;
class ULevel;
/**
 * A subsystem managing ISMPool actors.
 * Used by geometry collection now but repurposed for more general use.
 */
UCLASS(MinimalAPI)
class UE_DEPRECATED(5.6, "UGeometryCollectionISMPoolSubSystem is deprecated, please use UISMPoolSubSystem instead.") UGeometryCollectionISMPoolSubSystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	GEOMETRYCOLLECTIONENGINE_API UGeometryCollectionISMPoolSubSystem();

	// USubsystem BEGIN
	GEOMETRYCOLLECTIONENGINE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void Deinitialize() override;
	// USubsystem END

	/** Finds or creates an actor. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GEOMETRYCOLLECTIONENGINE_API AGeometryCollectionISMPoolActor* FindISMPoolActor(ULevel* Level);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/** Get all actors managed by the subsystem. */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	GEOMETRYCOLLECTIONENGINE_API void GetISMPoolActors(TArray<AGeometryCollectionISMPoolActor*>& OutActors) const;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

protected:
	UFUNCTION()
	void OnActorEndPlay(AActor* InSource, EEndPlayReason::Type Reason);

	/** ISMPool are per level **/
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TMap<TObjectPtr<ULevel>, TObjectPtr<AGeometryCollectionISMPoolActor> > PerLevelISMPoolActors;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
