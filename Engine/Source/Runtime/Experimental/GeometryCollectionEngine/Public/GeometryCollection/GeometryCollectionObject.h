// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Chaos/ChaosSolverActor.h"
#include "GeometryCollection/GeometryCollectionDamagePropagationData.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
#include "Dataflow/DataflowInstance.h"
#include "GeometryCollection/ManagedArray.h"
#include "InstanceUniformShaderParameters.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "Misc/Crc.h"
#include "Containers/Map.h"

#include "GeometryCollectionObject.generated.h"

class FGeometryCollection;
class FGeometryCollectionRenderData;
struct FGeometryCollectionSection;
struct FManagedArrayCollection;
struct FSharedSimulationParameters;
class UDataflow;
class UGeometryCollectionCache;
class UMaterial;
class UMaterialInterface;
class UPhysicalMaterial;

USTRUCT(BlueprintType)
struct FGeometryCollectionSource
{
	GENERATED_BODY()

	FGeometryCollectionSource() {}
	FGeometryCollectionSource(const FSoftObjectPath& SourceSoftObjectPath, const FTransform& ComponentTransform, const TArray<TObjectPtr<UMaterialInterface>>& SourceMaterials, bool bSplitComponents = false, bool bSetInternalFromMaterialIndex = false)
		: SourceGeometryObject(SourceSoftObjectPath), LocalTransform(ComponentTransform), SourceMaterial(SourceMaterials), bAddInternalMaterials(false), bSplitComponents(bSplitComponents), bSetInternalFromMaterialIndex(bSetInternalFromMaterialIndex)
	{

	}

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GeometrySource", meta=(AllowedClasses="/Script/Engine.StaticMesh, /Script/Engine.SkeletalMesh, /Script/GeometryCollectionEngine.GeometryCollection"))
	FSoftObjectPath SourceGeometryObject;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GeometrySource")
	FTransform LocalTransform;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GeometrySource")
	TArray<TObjectPtr<UMaterialInterface>> SourceMaterial;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GeometrySource|Instance")
	TArray<float> InstanceCustomData;

	//~ Note: bAddInternalMaterials defaults to true so a 'Reset' of a geometry collection that was created before this member was added will have consistent behavior. New geometry collections should always set bAddInternalMaterials to false.
	/** (Legacy) Whether source materials will be duplicated to create new slots for internal materials, or existing odd materials will be considered internal. (For non-Geometry Collection inputs only.) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GeometrySource", DisplayName = "(Legacy) Add Internal Materials")
	bool bAddInternalMaterials = true;

	/** Whether individual source mesh components should be split into separate pieces of geometry based on mesh connectivity. If checked, triangles that are not topologically connected will be assigned separate bones. (For non-Geometry Collection inputs only.) */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GeometrySource", DisplayName = "Split Meshes")
	bool bSplitComponents = false;

	/** Whether to set the 'internal' flag for faces with odd-numbered materials slots. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GeometrySource")
	bool bSetInternalFromMaterialIndex = false;

	// TODO: add primtive custom data
};

USTRUCT(BlueprintType)
struct FGeometryCollectionAutoInstanceMesh
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Mesh instead."))
	FSoftObjectPath StaticMesh_DEPRECATED;
#endif

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "AutoInstance", meta = (AllowedClasses = "/Script/Engine.StaticMesh"))
	TObjectPtr<const UStaticMesh> Mesh;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "AutoInstance")
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	UPROPERTY(VisibleAnywhere, Category = "AutoInstance")
	int32 NumInstances = 0;

	UPROPERTY(VisibleAnywhere, Category = "AutoInstance")
	TArray<float> CustomData;

	GEOMETRYCOLLECTIONENGINE_API int32 GetNumDataPerInstance() const;

	GEOMETRYCOLLECTIONENGINE_API bool operator ==(const FGeometryCollectionAutoInstanceMesh& Other) const;
};

USTRUCT(BlueprintType)
struct FGeometryCollectionEmbeddedExemplar
{
	GENERATED_BODY()

	FGeometryCollectionEmbeddedExemplar() 
		: StaticMeshExemplar(FString("None"))
		, StartCullDistance(0.0f)
		, EndCullDistance(0.0f)
		, InstanceCount(0)
	{ };
	
	FGeometryCollectionEmbeddedExemplar(FSoftObjectPath NewExemplar)
		: StaticMeshExemplar(NewExemplar)
		, StartCullDistance(0.0f)
		, EndCullDistance(0.0f)
		, InstanceCount(0)
	{ }

	UPROPERTY(EditAnywhere, Category = "EmbeddedExemplar", meta = (AllowedClasses = "/Script/Engine.StaticMesh"))
	FSoftObjectPath StaticMeshExemplar;

	UPROPERTY(EditAnywhere, Category = "EmbeddedExemplar")
	float StartCullDistance;

	UPROPERTY(EditAnywhere, Category = "EmbeddedExemplar")
	float EndCullDistance;

	UPROPERTY(VisibleAnywhere, Category = "EmbeddedExemplar")
	int32 InstanceCount;
};

USTRUCT()
struct FGeometryCollectionLevelSetData
{
	GENERATED_BODY()

	GEOMETRYCOLLECTIONENGINE_API FGeometryCollectionLevelSetData();

	/*
	*  Resolution on the smallest axes for the level set. (def: 5)
	*/
	UPROPERTY(EditAnywhere, Category = "LevelSet")
	int32 MinLevelSetResolution;

	/*
	*  Resolution on the smallest axes for the level set. (def: 10)
	*/
	UPROPERTY(EditAnywhere, Category = "LevelSet")
	int32 MaxLevelSetResolution;

	/*
	*  Resolution on the smallest axes for the level set. (def: 5)
	*/
	UPROPERTY(EditAnywhere, Category = "LevelSet")
	int32 MinClusterLevelSetResolution;

	/*
	*  Resolution on the smallest axes for the level set. (def: 10)
	*/
	UPROPERTY(EditAnywhere, Category = "LevelSet")
	int32 MaxClusterLevelSetResolution;
};


USTRUCT()
struct FGeometryCollectionCollisionParticleData
{
	GENERATED_BODY()

	GEOMETRYCOLLECTIONENGINE_API FGeometryCollectionCollisionParticleData();

	/**
	 * Number of particles on the triangulated surface to use for collisions.
	 */
	UPROPERTY(EditAnywhere, Category = "Particle")
	float CollisionParticlesFraction;

	/**
	 * Max number of particles.
	 */
	UPROPERTY(EditAnywhere, Category = "Particle")
	int32 MaximumCollisionParticles;
};



USTRUCT()
struct FGeometryCollectionCollisionTypeData
{
	GENERATED_BODY()

	GEOMETRYCOLLECTIONENGINE_API FGeometryCollectionCollisionTypeData();

	/*
	*  CollisionType defines how to initialize the rigid collision structures.
	*/
	UPROPERTY(EditAnywhere, Category = "Collisions")
	ECollisionTypeEnum CollisionType;

	/*
	*  CollisionType defines how to initialize the rigid collision structures.
	*/
	UPROPERTY(EditAnywhere, Category = "Collisions")
	EImplicitTypeEnum ImplicitType;

	/*
	*  LevelSet Resolution data for rasterization.
	*/
	UPROPERTY(EditAnywhere, Category = "Collisions", meta = (EditCondition = "ImplicitType == EImplicitTypeEnum::Chaos_Implicit_LevelSet", EditConditionHides))
	FGeometryCollectionLevelSetData LevelSet;

	/*
	*  Collision Particle data for surface samples during Particle-LevelSet collisions.
	*/
	UPROPERTY(EditAnywhere, Category = "Collisions", meta = (EditCondition = "CollisionType == ECollisionTypeEnum::Chaos_Surface_Volumetric", EditConditionHides))
	FGeometryCollectionCollisionParticleData CollisionParticles;

	/*
	*  Uniform scale on the collision body. (def: 0)
	*/
	UPROPERTY(EditAnywhere, Category = "Collisions")
	float CollisionObjectReductionPercentage;

	/**
	* A collision margin is a fraction of size used by some boxes and convex shapes to improve collision detection results.
	* The core geometry of shapes that support a margin are reduced in size by the margin, and the margin
	* is added back on during collision detection. The net result is a shape of the same size but with rounded corners.
	*/
	UPROPERTY(EditAnywhere, Category = "Collisions", meta = (EditCondition = "ImplicitType == EImplicitTypeEnum::Chaos_Implicit_Convex || ImplicitType == EImplicitTypeEnum::Chaos_Implicit_Box", EditConditionHides))
	float CollisionMarginFraction;

};


USTRUCT()
struct FGeometryCollectionSizeSpecificData
{
	GENERATED_BODY()

	GEOMETRYCOLLECTIONENGINE_API FGeometryCollectionSizeSpecificData();

	/** The max size these settings apply to*/
	UPROPERTY(EditAnywhere, Category = "Collisions")
	float MaxSize;

	/*
	* Collision Shapes allow kfor multiple collision types per rigid body. 
	*/
	UPROPERTY(EditAnywhere, Category = "Collisions")
	TArray<FGeometryCollectionCollisionTypeData> CollisionShapes;

#if WITH_EDITORONLY_DATA
	/*
	 *  CollisionType defines how to initialize the rigid collision structures.
	 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Collision.CollisionType instead."))
	ECollisionTypeEnum CollisionType_DEPRECATED;

	/*
	 *  CollisionType defines how to initialize the rigid collision structures.
	 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Collision.ImplicitType instead."))
	EImplicitTypeEnum ImplicitType_DEPRECATED;

	/*
	 *  Resolution on the smallest axes for the level set. (def: 5)
	 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Collision.LevelSet.MinLevelSetResolution instead."))
	int32 MinLevelSetResolution_DEPRECATED;

	/*
	 *  Resolution on the smallest axes for the level set. (def: 10)
	 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Collision.LevelSet.MaxLevelSetResolution instead."))
	int32 MaxLevelSetResolution_DEPRECATED;

	/*
	 *  Resolution on the smallest axes for the level set. (def: 5)
	 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Collision.LevelSet.MinClusterLevelSetResolution instead."))
	int32 MinClusterLevelSetResolution_DEPRECATED;

	/*
	 *  Resolution on the smallest axes for the level set. (def: 10)
	 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Collision.LevelSet.MaxClusterLevelSetResolution instead."))
	int32 MaxClusterLevelSetResolution_DEPRECATED;

	/*
	 *  Resolution on the smallest axes for the level set. (def: 10)
	 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Collision.CollisionObjectReductionPercentage instead."))
	int32 CollisionObjectReductionPercentage_DEPRECATED;

	/**
	 * Number of particles on the triangulated surface to use for collisions.
	 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Collision.CollisionParticlesFraction instead."))
	float CollisionParticlesFraction_DEPRECATED;

	/**
	 * Max number of particles.
	 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use Collision.MaximumCollisionParticles instead."))
	int32 MaximumCollisionParticles_DEPRECATED;
#endif

	/**
	 * Max number of particles.
	 */
	UPROPERTY(EditAnywhere, Category = "Collisions")
	int32 DamageThreshold;

	GEOMETRYCOLLECTIONENGINE_API bool Serialize(FArchive& Ar);
#if WITH_EDITORONLY_DATA
	GEOMETRYCOLLECTIONENGINE_API void PostSerialize(const FArchive& Ar);
#endif
};

template<>
struct TStructOpsTypeTraits<FGeometryCollectionSizeSpecificData> : public TStructOpsTypeTraitsBase2<FGeometryCollectionSizeSpecificData>
{
	enum
	{
		WithSerializer = true,
#if WITH_EDITORONLY_DATA
		WithPostSerialize = true
#endif
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

USTRUCT(BlueprintType)
struct FGeometryCollectionProxyMeshMaterials
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Materials")
	TArray<TObjectPtr<UMaterialInterface>> Materials;
};

USTRUCT(BlueprintType)
struct FGeometryCollectionProxyMeshData
{
	GENERATED_BODY()

	/** References to meshes to be used as proxies for rendering when the geometry collection is not broken yet */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (AllowedClasses = "/Script/Engine.StaticMesh"))
	TArray<TObjectPtr<UStaticMesh>> ProxyMeshes;

	/** 
	* Transform of the mesh relative to the geometry collection it belongs to 
	* if an entry is missing or if the arary is empty , those transforms are assume to be identity
	* making any proxy mesh done before 5.6 be fully backward compatible
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	TArray<FTransform3f> MeshTransforms;

	/**
	* Optional arrays of override material matching each proxy Mesh
	* if an entry is missing or if the arary is empty , then no material will be assigned
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	TArray<FGeometryCollectionProxyMeshMaterials> MeshOverrideMaterials;

	GEOMETRYCOLLECTIONENGINE_API const FTransform3f& GetMeshTransform(int32 MeshIndex) const;

	GEOMETRYCOLLECTIONENGINE_API void Reset();
};

USTRUCT()
struct FGeometryCollectionRenderResourceSizeInfo
{
	GENERATED_BODY();

	// Total size of the arrays for the MeshResources
	UPROPERTY()
	uint64 MeshResourcesSize = 0;

	// Total size of the arrays for the NaniteResources
	UPROPERTY()
	uint64 NaniteResourcesSize = 0;
};

/**
* UGeometryCollectionObject (UObject)
*
* UObject wrapper for the FGeometryCollection
*
*/
UCLASS(BlueprintType, customconstructor, MinimalAPI)
class UGeometryCollection : 
	public UObject, 
	public IInterface_AssetUserData, 
	public IDataflowContentOwner,
	public IDataflowInstanceInterface
{
	GENERATED_UCLASS_BODY()

public:
	GEOMETRYCOLLECTIONENGINE_API UGeometryCollection(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/** UObject Interface */
#if WITH_EDITOR
	GEOMETRYCOLLECTIONENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void PostEditUndo() override;
	GEOMETRYCOLLECTIONENGINE_API virtual bool Modify(bool bAlwaysMarkDirty = true) override;
#endif
	GEOMETRYCOLLECTIONENGINE_API virtual void PostInitProperties() override;
	GEOMETRYCOLLECTIONENGINE_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void PostLoad() override;
	GEOMETRYCOLLECTIONENGINE_API virtual void BeginDestroy() override;
	GEOMETRYCOLLECTIONENGINE_API virtual bool IsReadyForFinishDestroy() override;
	/** End UObject Interface */

	GEOMETRYCOLLECTIONENGINE_API void Serialize(FArchive& Ar);

#if WITH_EDITOR
	GEOMETRYCOLLECTIONENGINE_API static void AppendToClassSchema(FAppendToClassSchemaContext& Context);
	GEOMETRYCOLLECTIONENGINE_API void EnsureDataIsCooked(bool bInitResources, bool bIsTransacting, bool bIsPersistent, bool bAllowCopyFromDDC = true);
#endif

	/** Accessors for internal geometry collection */
	void SetGeometryCollection(TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollectionIn) { GeometryCollection = GeometryCollectionIn; }
	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe>       GetGeometryCollection() { return GeometryCollection; }
	const TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GetGeometryCollection() const { return GeometryCollection; }

	/** Return true if the collection is "Empty" ( has no transforms at all ) */
	GEOMETRYCOLLECTIONENGINE_API bool IsEmpty() const;

	/** Return collection to initial (ie. empty) state. */
	GEOMETRYCOLLECTIONENGINE_API void Reset();

	/** Reset the collection from another set of attributes and materials. */
	GEOMETRYCOLLECTIONENGINE_API void ResetFrom(const FManagedArrayCollection& InCollection, const TArray<UMaterial*>& InMaterials, bool bHasInternalMaterials);
	GEOMETRYCOLLECTIONENGINE_API void ResetFrom(const FManagedArrayCollection& InCollection, const TArray<UMaterialInterface*>& InMaterialInstances, bool bHasInternalMaterials);
	
	GEOMETRYCOLLECTIONENGINE_API int32 AppendGeometry(const UGeometryCollection & Element, bool ReindexAllMaterials = false, const FTransform& TransformRoot = FTransform::Identity);
	GEOMETRYCOLLECTIONENGINE_API int32 NumElements(const FName& Group) const;
	GEOMETRYCOLLECTIONENGINE_API void RemoveElements(const FName& Group, const TArray<int32>& SortedDeletionList);

	/** Has data for static mesh rendering. */
	GEOMETRYCOLLECTIONENGINE_API bool HasMeshData() const;
	/** Has data for nanite rendering. */
	GEOMETRYCOLLECTIONENGINE_API bool HasNaniteData() const;

	GEOMETRYCOLLECTIONENGINE_API uint32 GetNaniteResourceID() const;
	GEOMETRYCOLLECTIONENGINE_API uint32 GetNaniteHierarchyOffset() const;
	GEOMETRYCOLLECTIONENGINE_API uint32 GetNaniteHierarchyOffset(int32 GeometryIndex, bool bFlattened = false) const;

	/** ReindexMaterialSections */
	GEOMETRYCOLLECTIONENGINE_API void ReindexMaterialSections();

	/** appends the standard materials to this UObject */
	GEOMETRYCOLLECTIONENGINE_API void InitializeMaterials(bool bHasLegacyInternalMaterialsPairs = false);

	/** Add a material to the materials array and update the selected bone material to be at the end of the array */
	GEOMETRYCOLLECTIONENGINE_API int32 AddNewMaterialSlot(bool bCopyLastMaterial = true);

	/** Remove a material from the materials array, keeping the selected bone material at the end of the array. Returns false if materials could not be removed (e.g. because there were too few). */
	GEOMETRYCOLLECTIONENGINE_API bool RemoveLastMaterialSlot();


	/** Returns true if there is anything to render */
	GEOMETRYCOLLECTIONENGINE_API bool HasVisibleGeometry() const;

	/** Invalidates this collection signaling a structural change and renders any previously recorded caches unable to play with this collection */
	GEOMETRYCOLLECTIONENGINE_API void InvalidateCollection();

	/** Check to see if Simulation Data requires regeneration */
	GEOMETRYCOLLECTIONENGINE_API bool IsSimulationDataDirty() const;

	/** Attach a Static Mesh exemplar for embedded geometry, if that mesh has not already been attached. Return the exemplar index. */
	GEOMETRYCOLLECTIONENGINE_API int32 AttachEmbeddedGeometryExemplar(const UStaticMesh* Exemplar);

	/** Remove embedded geometry exemplars with indices matching the sorted removal list. */
	GEOMETRYCOLLECTIONENGINE_API void RemoveExemplars(const TArray<int32>& SortedRemovalIndices);

	/** find or add a auto instance mesh and return its index */
	GEOMETRYCOLLECTIONENGINE_API const FGeometryCollectionAutoInstanceMesh& GetAutoInstanceMesh(int32 AutoInstanceMeshIndex) const;

	/** 
	* Assign an auto instanced meshes array
	* if there's duplicate entry in the array , they will be collapsed as one and the index attribute will be adjusted acoordingly
	*/
	GEOMETRYCOLLECTIONENGINE_API void SetAutoInstanceMeshes(const TArray<FGeometryCollectionAutoInstanceMesh>& InAutoInstanceMeshes);
	
	/**  find or add a auto instance mesh from another one and return its index */
	GEOMETRYCOLLECTIONENGINE_API int32 FindOrAddAutoInstanceMesh(const FGeometryCollectionAutoInstanceMesh& AutoInstanceMesh);

	/** find or add a auto instance mesh from a mesh and alist of material and return its index */
	GEOMETRYCOLLECTIONENGINE_API int32 FindOrAddAutoInstanceMesh(const UStaticMesh* StaticMesh, const TArray<UMaterialInterface*>& Materials);

	/** Produce a deep copy of GeometryCollection member, stripped of data unecessary for gameplay. */
	GEOMETRYCOLLECTIONENGINE_API TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GenerateMinimalGeometryCollection() const;

	/** copy a collection and remove geometry from it */
	static GEOMETRYCOLLECTIONENGINE_API TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> CopyCollectionAndRemoveGeometry(const TSharedPtr<const FGeometryCollection, ESPMode::ThreadSafe>& CollectionToCopy);

	/** get the size of the render data resources associated with this collection */
	GEOMETRYCOLLECTIONENGINE_API FGeometryCollectionRenderResourceSizeInfo GetRenderResourceSizeInfo() const;

#if WITH_EDITOR
	/** If this flag is set, we only regenerate simulation data when requested via CreateSimulationData() */
	bool bManualDataCreate;
	
	/** 
	* Create the simulation data that can be shared among all instances (mass, volume, etc...) 
	* Note : this does not check if the simulation data is drty or not and will cause a load from the DDC
	* use CreateSimulationDataIfNeeded() for avoiding extra runtime cost 
	*/
	GEOMETRYCOLLECTIONENGINE_API void CreateSimulationData();

	/** Create the simulation data ( calls CreateSimulationData) only if the simulation data is dirty */
	GEOMETRYCOLLECTIONENGINE_API void CreateSimulationDataIfNeeded();

	/** Rebuild the render data. */
	GEOMETRYCOLLECTIONENGINE_API void RebuildRenderData();

	/** Propogate render state dirty to components */
	GEOMETRYCOLLECTIONENGINE_API void PropagateMarkDirtyToComponents() const;

	/** Propogate the fact that transform have been updated to all components */
	GEOMETRYCOLLECTIONENGINE_API void PropagateTransformUpdateToComponents() const;

	/** Create root proxies from geometryt sources */
	GEOMETRYCOLLECTIONENGINE_API void SetRootProxiesFromGeometrySources();

#endif

	GEOMETRYCOLLECTIONENGINE_API void InitResources();
	GEOMETRYCOLLECTIONENGINE_API void ReleaseResources();
	
	/** Fills params struct with parameters used for precomputing content. */
	GEOMETRYCOLLECTIONENGINE_API void GetSharedSimulationParams(FSharedSimulationParameters& OutParams) const;

	/**
	* Get Mass or density as set by the asset ( this is the value used by to compute the cached attributes )
	* Mass is return in Kg and Density is returned in Kg/Cm3
	* @param bOutIsDensity  is set to true by the function if the returned value is to be treated as a density
	*/
	GEOMETRYCOLLECTIONENGINE_API float GetMassOrDensity(bool& bOutIsDensity) const;

	/*
	* cache the material density used to compute attribute
	* Warning : this should only be called after recomputing the mass based on those values
	*/
	GEOMETRYCOLLECTIONENGINE_API void CacheMaterialDensity();

	/** Accessors for the two guids used to identify this collection */
	GEOMETRYCOLLECTIONENGINE_API FGuid GetIdGuid() const;
	GEOMETRYCOLLECTIONENGINE_API FGuid GetStateGuid() const;

	/** Get the cached root index */
	int32 GetRootIndex() const { return RootIndex; }

	/** Pointer to the data used to render this geometry collection. */
	TUniquePtr<FGeometryCollectionRenderData> RenderData;

	UPROPERTY(EditAnywhere, Category = "Clustering")
	bool EnableClustering;

	/** Cluster group index - Geometry Collections using the same index will simulate as one single cluster. */
	UPROPERTY(EditAnywhere, Category = "Clustering")
	int32 ClusterGroupIndex;

	/** Maximum level for cluster breaks. */
	UPROPERTY(EditAnywhere, Category = "Clustering")
	int32 MaxClusterLevel;

	/** Damage model to use for evaluating destruction. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Damage")
	EDamageModelTypeEnum DamageModel;

	/** Damage threshold for clusters at different levels. */
	UPROPERTY(EditAnywhere, Category = "Damage", meta = (EditCondition = "!bUseSizeSpecificDamageThreshold && DamageModel == EDamageModelTypeEnum::Chaos_Damage_Model_UserDefined_Damage_Threshold"))
	TArray<float> DamageThreshold;

	/** whether to use size specific damage threshold instead of level based ones ( see Size Specific Data array ). */
	UPROPERTY(EditAnywhere, Category = "Damage", meta = (EditCondition = "DamageModel == EDamageModelTypeEnum::Chaos_Damage_Model_UserDefined_Damage_Threshold"))
	bool bUseSizeSpecificDamageThreshold;

	/** When on , use the modifiers on the material to adjust the user defined damage threshold values */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Damage", meta = (EditCondition = "DamageModel == EDamageModelTypeEnum::Chaos_Damage_Model_UserDefined_Damage_Threshold"))
	bool bUseMaterialDamageModifiers;

	/** compatibility check, when true, only cluster compute damage from parameters and propagate to direct children
	 *  when false, each child will compute it's damage threshold allowing for more precise and intuitive destruction behavior
	 */
	UPROPERTY(EditAnywhere, Category = "Compatibility")
	bool PerClusterOnlyDamageThreshold;

	/** Data about how damage propagation shoudl behave. */
	UPROPERTY(EditAnywhere, Category = "Damage")
	FGeometryCollectionDamagePropagationData DamagePropagationData;

	/** */
	UPROPERTY(EditAnywhere, Category = "Clustering")
	EClusterConnectionTypeEnum ClusterConnectionType;

	UPROPERTY(EditAnywhere, Category = "Clustering")
	float ConnectionGraphBoundsFilteringMargin;

#if WITH_EDITORONLY_DATA
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "GeometrySource")
	TArray<FGeometryCollectionSource> GeometrySource;
#endif

	UPROPERTY(EditAnywhere, Category = "Materials")
	TArray<TObjectPtr<UMaterialInterface>> Materials;

	/** References for embedded geometry generation */
	UPROPERTY(EditAnywhere, Category = "EmbeddedGeometry")
	TArray<FGeometryCollectionEmbeddedExemplar> EmbeddedGeometryExemplar;

	/** Whether to use full precision UVs when rendering this geometry. (Does not apply to Nanite rendering) */
	UPROPERTY(EditAnywhere, Category = "Rendering")
	bool bUseFullPrecisionUVs = false;

	/**
	 * Strip unnecessary data from the Geometry Collection to keep the memory footprint as small as possible.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (DisplayName = "Strip Source Data On Cook"))
	bool bStripOnCook;

	/**
	 * Strip unnecessary render data from the Geometry Collection to keep the memory footprint as small as possible.
	 * This may be used if the cooked build uses a custom renderer such as the ISMPool renderer.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	bool bStripRenderDataOnCook;

	/**
	 * If true, a ray tracing acceleration structure will be built for this geometry collection and it may be used in ray tracing effects
	 */
	UPROPERTY(EditAnywhere, Category = "Rendering")
	uint8 bSupportRayTracing : 1;

	/** Custom class type that will be used to render the geometry collection instead of using the native rendering. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering", meta = (MustImplement = "/Script/GeometryCollectionEngine.GeometryCollectionExternalRenderInterface"))
	TObjectPtr<UClass> CustomRendererType;

	/** Static mesh to use as a proxy for rendering until the geometry collection is broken. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Rendering")
	FGeometryCollectionProxyMeshData RootProxyData;

	/** List of unique static mesh / materials pairs for auto instancing. */
	UPROPERTY(EditAnywhere, Category = "Rendering")
	TArray<FGeometryCollectionAutoInstanceMesh> AutoInstanceMeshes;

	UFUNCTION(BlueprintCallable, Category = "Nanite")
	GEOMETRYCOLLECTIONENGINE_API void SetEnableNanite(bool bValue);

	/**
	 * Enable support for Nanite.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetEnableNanite, Category = "Nanite")
	bool EnableNanite;

	/**
	 * Enable Non-Nanite fallback mesh when Nanite support is enabled.
	 */
	UPROPERTY(EditAnywhere, Category = "Nanite", Meta = (EditCondition = "EnableNanite"))
	bool bEnableNaniteFallback;

	/**
	 * How much of the nanite resource should always be resident (In KB). Approximate due to paging. 0: Minimum size (single page). MAX_uint32: Entire mesh.
	 */
	UPROPERTY(EditAnywhere, Category = "Nanite", Meta = (EditCondition = "EnableNanite", DisplayName = "Nanite Minimum Residency"))
	uint32 NaniteMinimumResidencyInKB;

	UFUNCTION(BlueprintCallable, Category = "Rendering")
	GEOMETRYCOLLECTIONENGINE_API void SetConvertVertexColorsToSRGB(bool bValue);

	/**
	 * Convert vertex colors to sRGB for rendering. Exposed to avoid changing vertex color rendering for legacy assets; should typically be true for new geometry collections.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetConvertVertexColorsToSRGB, Category = "Rendering")
	bool bConvertVertexColorsToSRGB = true;

#if WITH_EDITORONLY_DATA
	/*
	*  CollisionType defines how to initialize the rigid collision structures.
	*/
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated. Use the default SizeSpecificData instead."))
	ECollisionTypeEnum CollisionType_DEPRECATED;

	/*
	*  CollisionType defines how to initialize the rigid collision structures.
	*/
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated. Use the default SizeSpecificData instead."))
	EImplicitTypeEnum ImplicitType_DEPRECATED;

	/*
	*  Resolution on the smallest axes for the level set. (def: 5)
	*/
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated. Use the default SizeSpecificData instead."))
	int32 MinLevelSetResolution_DEPRECATED;

	/*
	*  Resolution on the smallest axes for the level set. (def: 10)
	*/
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated. Use the default SizeSpecificData instead."))
	int32 MaxLevelSetResolution_DEPRECATED;

	/*
	*  Resolution on the smallest axes for the level set. (def: 5)
	*/
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated. Use the default SizeSpecificData instead."))
	int32 MinClusterLevelSetResolution_DEPRECATED;

	/*
	*  Resolution on the smallest axes for the level set. (def: 10)
	*/
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated. Use the default SizeSpecificData instead."))
	int32 MaxClusterLevelSetResolution_DEPRECATED;

	/*
	*  Resolution on the smallest axes for the level set. (def: 10)
	*/
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated. Use the default SizeSpecificData instead."))
	float CollisionObjectReductionPercentage_DEPRECATED;

	/** static mesh to use as a proxy for rendering until the geometry collection is broken */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated. Use RootProxyData instead."))
	FSoftObjectPath RootProxy_DEPRECATED;
#endif
	
	/**	Physics material to use for the geometry collection */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Collisions")
	TObjectPtr<UPhysicalMaterial> PhysicsMaterial;

	/**
	* Whether to use density ( for mass computation ) from physics material ( if physics material is not set, use the component one or defaults )
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Collisions", meta = (EditCondition = "PhysicsMaterial != nullptr"))
	bool bDensityFromPhysicsMaterial;

	/**
	* Cached Material density value used to compute the Mass attribute  ( In gram per cm3 )
	* this is necessary because the material properties could be changed after without causing the mass attribute to be recomputed ( because the GC asset will not get notified )
	*/
	UPROPERTY()
	float CachedDensityFromPhysicsMaterialInGCm3;

	/**
	* Mass As Density, units are in kg/m^3 ( only enabled if physics material is not set )
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Collisions", meta = (EditCondition = "!bDensityFromPhysicsMaterial"))
	bool bMassAsDensity;

	/**
	* Total Mass of Collection. If density, units are in kg/m^3 ( only enabled if physics material is not set )
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Collisions", meta = (EditCondition = "!bDensityFromPhysicsMaterial"))
	float Mass;

	/**
	* Smallest allowable mass (def:0.1)
	*/
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Collisions")
	float MinimumMassClamp;

	/**
	* whether to import collision from the source asset
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Collisions")
	bool bImportCollisionFromSource;

	/**
	* whether to optimize convexes for collisions. If true the convex optimizer will generate at runtime one 
	* single convex shape for physics collisions ignoring all the user defined ones. 
	* Enable p.Chaos.Convex.SimplifyUnion cvar to be able to use it (experimental)
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Collisions")
	bool bOptimizeConvexes = true;
	
#if WITH_EDITORONLY_DATA
	/**
	 * Number of particles on the triangulated surface to use for collisions.
	 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated. Use the default SizeSpecificData instead."))
	float CollisionParticlesFraction_DEPRECATED;

	/**
	 * Max number of particles.
	 */
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "This property is deprecated. Use the default SizeSpecificData instead."))
	int32 MaximumCollisionParticles_DEPRECATED;
#endif

	/** When enabled, particle will scale down (shrink) when using being removed ( using both remove on sleep or remove on break ) - Enabled by default */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Removal)
	bool bScaleOnRemoval;

	/** Remove particle from simulation and dissolve rendered geometry once sleep threshold has been exceeded. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Removal, meta = (DisplayName = "Remove on Sleep"))
	bool bRemoveOnMaxSleep;

	/** When using remove on sleep , automatically crumble partial cluster into smaller pieces when possible */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Removal, meta = (EditCondition = "bRemoveOnMaxSleep"))
	bool bAutomaticCrumblePartialClusters;

	/** How long may the particle sleep before initiating removal (in seconds). */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Removal, meta = (DisplayName = "Sleep Min Max", EditCondition="bRemoveOnMaxSleep"))
	FVector2D MaximumSleepTime;

	/** How long does the removal process take (in seconds). */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Removal, meta = (DisplayName = "Removal Duration", EditCondition="bRemoveOnMaxSleep"))
	FVector2D RemovalDuration;

	/** when on non-sleeping, slow moving pieces will be considered as sleeping, this helps removal of jittery but not really moving objects. */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Removal, meta = (DisplayName = "Slow-Moving as sleeping", EditCondition="bRemoveOnMaxSleep"))
	bool bSlowMovingAsSleeping;

	/** When slow moving detection is on, this defines the linear velocity thresholds in cm/s to consider the object as sleeping . */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Removal, meta = (DisplayName = "Slow-Moving Velocity Threshold", EditCondition="bRemoveOnMaxSleep && bSlowMovingAsSleeping"))
	float SlowMovingVelocityThreshold;
	
	/*
	* Size Specfic Data reflects the default geometry to bind to rigid bodies smaller
	* than the max size volume. This can also be empty to reflect no collision geometry
	* for the collection. 
	*/
	UPROPERTY(EditAnywhere, Category = "Collisions")
	TArray<FGeometryCollectionSizeSpecificData> SizeSpecificData;

	GEOMETRYCOLLECTIONENGINE_API int GetDefaultSizeSpecificDataIndex() const;
	GEOMETRYCOLLECTIONENGINE_API FGeometryCollectionSizeSpecificData& GetDefaultSizeSpecificData();
	GEOMETRYCOLLECTIONENGINE_API const FGeometryCollectionSizeSpecificData& GetDefaultSizeSpecificData() const;
	static GEOMETRYCOLLECTIONENGINE_API FGeometryCollectionSizeSpecificData GeometryCollectionSizeSpecificDataDefaults();

	/**
	* Enable remove pieces on fracture
	*/
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use remove on break feature instead ( Fracture editor tools )."))
	bool EnableRemovePiecesOnFracture_DEPRECATED;

	/**
	* Materials relating to remove on fracture
	*/
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use remove on break feature instead ( Fracture editor tools )."))
	TArray<TObjectPtr<UMaterialInterface>> RemoveOnFractureMaterials_DEPRECATED;

	UE_DEPRECATED(5.5, "Bone Selected Material does not have a material index anymore.")
	FORCEINLINE const int32 GetBoneSelectedMaterialIndex() const { return BoneSelectedMaterialIndex; }

	// Get the material to use for rendering bone selections in the editor, or nullptr
	static UMaterialInterface* GetBoneSelectedMaterial();

	/** Returns the asset path for the automatically populated selected material. */
	static GEOMETRYCOLLECTIONENGINE_API const TCHAR* GetSelectedMaterialPath();

#if WITH_EDITORONLY_DATA
	/** Importing data and options used for this geometry collection */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Instanced, Category = ImportSettings)
	TObjectPtr<class UAssetImportData> AssetImportData;

	/** Information for thumbnail rendering */
	UPROPERTY(VisibleAnywhere, Instanced, AdvancedDisplay, Category = GeometryCollection)
	TObjectPtr<class UThumbnailInfo> ThumbnailInfo;
#endif // WITH_EDITORONLY_DATA

	/*
	* Update the convex geometry on the collection.
	*/
	GEOMETRYCOLLECTIONENGINE_API void UpdateConvexGeometry();
	GEOMETRYCOLLECTIONENGINE_API void UpdateConvexGeometryIfMissing();
	
	/*
	 * Update properties that depend on the geometry and clustering: Proximity, Convex Hulls, Volume and Size data.
	 */
	GEOMETRYCOLLECTIONENGINE_API void UpdateGeometryDependentProperties();


	//~ Begin IInterface_AssetUserData Interface
	GEOMETRYCOLLECTIONENGINE_API virtual void AddAssetUserData(UAssetUserData* InUserData) override;
	GEOMETRYCOLLECTIONENGINE_API virtual void RemoveUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	GEOMETRYCOLLECTIONENGINE_API virtual UAssetUserData* GetAssetUserDataOfClass(TSubclassOf<UAssetUserData> InUserDataClass) override;
	GEOMETRYCOLLECTIONENGINE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

	//
	// Dataflow
	//
#if WITH_EDITORONLY_DATA
	// this is deprecated data replaced with Setter and Getter but it is exposed to blueprint 
	// so we use the Setter/Getter to make sure the blueprint will use them 
	UPROPERTY(BlueprintReadWrite, BlueprintSetter=SetDataflowAsset, BlueprintGetter = GetDataflowAsset, Category = "Dataflow")
	TObjectPtr<UDataflow> DataflowAsset;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Dataflow terminal name is now stored in DataflowInstance"))
	FString DataflowTerminal_DEPRECATED = "GeometryCollectionTerminal";
#endif

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dataflow", DisplayName = "DataFlow Overrides", meta = (DisplayAfter = "DataflowInstance"))
	TMap<FString, FString> Overrides;

	UPROPERTY(EditAnywhere, Category = "Dataflow")
	FDataflowInstance DataflowInstance;

	UFUNCTION(BlueprintCallable, Category = "Dataflow")
	void SetDataflowAsset(UDataflow* InDataflowAsset);

	UFUNCTION(BlueprintCallable, Category = "Dataflow")
	UDataflow* GetDataflowAsset() const;

	//~ Begin IDataflowGeneratedAssetInterface
	virtual const FDataflowInstance& GetDataflowInstance() const override { return DataflowInstance; };
	virtual FDataflowInstance& GetDataflowInstance() override { return DataflowInstance; };
	//~ End IDataflowGeneratedAssetInterface

	//~ Begin IDataflowContentOwner interface
	virtual TObjectPtr<UDataflowBaseContent> CreateDataflowContent() override;
	virtual void WriteDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) const override;
	virtual void ReadDataflowContent(const TObjectPtr<UDataflowBaseContent>& DataflowContent) override;
	//~ End IDataflowContentOwner interface

	const TArray<int32>& GetBreadthFirstTransformIndices() const { return BreadthFirstTransformIndices; }

	const TArray<int32>& GetAutoInstanceTransformRemapIndices() const { return AutoInstanceTransformRemapIndices; }

#if WITH_EDITOR
	virtual bool CanEditChange(const FProperty* InProperty) const override;
#endif // WITH_EDITOR

private:
#if WITH_EDITOR
	GEOMETRYCOLLECTIONENGINE_API void CreateSimulationDataImp(bool bCopyFromDDC);
	GEOMETRYCOLLECTIONENGINE_API void CreateRenderDataImp(bool bCopyFromDDC);

	GEOMETRYCOLLECTIONENGINE_API void EnsureSimulationDataIsCooked(bool bIsTransacting, bool bAllowCopyFromDDC);
	GEOMETRYCOLLECTIONENGINE_API void EnsureRenderDataIsCooked(bool bInitResources);

#endif

	/*
	* Used to transfer deprecated properties to the size specific structures during serialization
	* and to add back the default size specific data when deleted.
	*/
	GEOMETRYCOLLECTIONENGINE_API void ValidateSizeSpecificDataDefaults();

	// update cached root index using the current hierarchy setup
	GEOMETRYCOLLECTIONENGINE_API void UpdateRootIndex();

	// fill instanced mesh instance count from geometry collection data if not done yet 
	GEOMETRYCOLLECTIONENGINE_API void FillAutoInstanceMeshesInstancesIfNeeded();

	void CacheBreadthFirstTransformIndices();
	void CacheAutoInstanceTransformRemapIndices();

private:
	/** Guid created on construction of this collection. It should be used to uniquely identify this collection */
	UPROPERTY()
	FGuid PersistentGuid;

	/** 
	 * Guid that can be invalidated on demand - essentially a 'version' that should be changed when a structural change is made to
	 * the geometry collection. This signals to any caches that attempt to link to a geometry collection whether the collection
	 * is still valid (hasn't structurally changed post-recording)
	 */
	UPROPERTY()
	FGuid StateGuid;

#if WITH_EDITOR
	//Used to determine whether we need to cook simulation data
	FGuid LastBuiltSimulationDataGuid;
	//Used to determine whether we need to regenerate simulation data
	FGuid SimulationDataGuid;

	//Used to determine whether we need to cook render data
	FGuid LastBuiltRenderDataGuid;
	//Used to determine whether we need to regenerate render data
	FGuid RenderDataGuid;
#endif

#if WITH_EDITORONLY_DATA
	/** 
	 * Serialized version of RenderDataGuid. 
	 * This is used to determine when the stored RenderData mismatches the current GC object state.
	 * This can happen temporarily in editor during a undo/redo transaction when a GC component processing the PostUndo() before the GC object.
	 * todo: Remove this fix and move all relevent state to the RenderData so that it is the only source of truth for render scene proxies.
	 */
	UPROPERTY()
	FGuid ValidRenderDataGuid;
#endif

	// cached root index for faster queries
	UPROPERTY(VisibleAnywhere, Category = "Clustering")
	int32 RootIndex = INDEX_NONE;

	// cache transform indices in breadth-first order
	UPROPERTY(VisibleAnywhere, Transient, Category = "Clustering")
	TArray<int32> BreadthFirstTransformIndices;

	// cache transform remapping for instanced meshes indices
	UPROPERTY(VisibleAnywhere, Transient, Category = "Clustering")
	TArray<int32> AutoInstanceTransformRemapIndices;

	// Legacy index of the bone selected material in the object's Materials array, or INDEX_NONE if it is not stored there.
	// Note for new objects the bone selected material should not be stored in the Materials array, so this should be INDEX_NONE
	// The property is kept solely to support deletion of the bone selected material from the materials list of legacy assets
	UPROPERTY()
	int32 BoneSelectedMaterialIndex = INDEX_NONE;

	TSharedPtr<FGeometryCollection, ESPMode::ThreadSafe> GeometryCollection;

	/** Array of user data stored with the asset */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Instanced, Category = AssetUserData)
	TArray<TObjectPtr<UAssetUserData>> AssetUserData;

	/** A fence to track render resource release before destroy. */
	FRenderCommandFence RenderResourceReleaseFence;

	float GetMassOrDensityInternal(bool& bOutIsDensity, bool bCached) const;

	void MigrateDeprecatedRootProxyData();
	void MigrateDeprecatedDataflowData();

	virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
};
