// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"
#include "EngineDefines.h"

#ifndef GEOMETRYCOLLECTION_DEBUG_DRAW
#define GEOMETRYCOLLECTION_DEBUG_DRAW UE_ENABLE_DEBUG_DRAWING
#endif

#if GEOMETRYCOLLECTION_DEBUG_DRAW
#include "Containers/StaticBitArray.h"
#include "Chaos/ArrayCollectionArray.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/Matrix.h"
#include "Chaos/PBDRigidClusteredParticles.h"
#include "Chaos/UniformGrid.h"
#include "Chaos/Framework/BufferedData.h"
#include "Chaos/Declares.h"

class FChaosSolversModule;
template<class InElementType> class TManagedArray;

/** Enumeration of the synchronizable data. */
enum class EGeometryCollectionParticlesData : uint8
{
	X,
	R,
	Geometry,
	GeometryType,
	GeometryIsConvex,
	GeometryHasBoundingBox,
	GeometryBoxMin,
	GeometryBoxMax,
	GeometrySphereCenter,
	GeometrySphereRadius,
	GeometryLevelSetGrid,
	V,
	W,
	F,
	Torque,
	I,
	InvI,
	M,
	InvM,
	CollisionParticlesSize,
	Disabled,
	Sleeping,
	Island,
	P,
	Q,
	PreV,
	PreW,
	ConnectivityEdges,
	ChildToParentMap,
	Count
};

/**
	Object holding selected particles data for a specific range of rigid body ids.
	Use SetDataSyncFlag, SetAllDataSyncFlag, or RequestSyncedData to have the required data made available at the next tick.
	Each time Sync is called, the particles' data is copied and the sync flags cleared.
	Use HasSyncedData or RequestSyncedData to check which data is available at any one time.
*/
class FGeometryCollectionParticlesData
{
public:
	/** Constructor. */
	FGeometryCollectionParticlesData();

	/** Set this data type to copy at the next sync. */
	void SetDataSyncFlag(EGeometryCollectionParticlesData Data) const { BufferedData.GetGameDataForRead().SetDataSyncFlag(Data); }

	/** Set all data type to copy at the next sync. */
	void SetAllDataSyncFlag() const { BufferedData.GetGameDataForRead().SetAllDataSyncFlag(); }

	/** Return whether the specified type of data has been copied during the last sync. */
	bool HasSyncedData(EGeometryCollectionParticlesData Data) const { return BufferedData.GetGameDataForRead().HasSyncedData(Data); }

	/** Shorthand for both setting the flag and checking the data has already been synced. */
	bool RequestSyncedData(EGeometryCollectionParticlesData Data) const { SetDataSyncFlag(Data); return HasSyncedData(Data); }

	/** Copy the data of the specified set of particles/rigid body ids to this object. */
	//void Sync(const Chaos::FPhysicsSolver* Solver, const TManagedArray<int32>& RigidBodyIds);
	void Sync(Chaos::FPhysicsSolver* Solver, const TManagedArray<FGuid>& RigidBodyIds);

	const Chaos::FVec3                         & GetX                     (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::X                     )); return BufferedData.GetGameDataForRead().X                     [Index]; }
	const Chaos::FRotation3                    & GetR                     (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::R                     )); return BufferedData.GetGameDataForRead().R                     [Index]; }
	const Chaos::FImplicitObject* const        & GetGeometry              (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::Geometry              )); return BufferedData.GetGameDataForRead().Geometry              [Index]; }
	const Chaos::EImplicitObjectType           & GetGeometryType          (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::GeometryType          )); return BufferedData.GetGameDataForRead().GeometryType          [Index]; }
	const bool                                 & IsGeometryConvex         (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::GeometryIsConvex      )); return BufferedData.GetGameDataForRead().GeometryIsConvex      [Index]; }
	const bool                                 & HasGeometryBoundingBoxm  (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::GeometryHasBoundingBox)); return BufferedData.GetGameDataForRead().GeometryHasBoundingBox[Index]; }
	const Chaos::FVec3                         & GetGeometryBoxMin        (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::GeometryBoxMin        )); return BufferedData.GetGameDataForRead().GeometryBoxMin        [Index]; }
	const Chaos::FVec3                         & GetGeometryBoxMax        (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::GeometryBoxMax        )); return BufferedData.GetGameDataForRead().GeometryBoxMax        [Index]; }
	const Chaos::FVec3                         & GetGeometrySphereCenter  (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::GeometrySphereCenter  )); return BufferedData.GetGameDataForRead().GeometrySphereCenter  [Index]; }
	const Chaos::FReal                         & GetGeometrySphereRadius  (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::GeometrySphereRadius  )); return BufferedData.GetGameDataForRead().GeometrySphereRadius  [Index]; }
	const Chaos::TUniformGrid<Chaos::FReal, 3> & GetGeometryLevelSetGrid  (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::GeometryLevelSetGrid  )); return BufferedData.GetGameDataForRead().GeometryLevelSetGrid  [Index]; }
	const Chaos::FVec3                         & GetV                     (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::V                     )); return BufferedData.GetGameDataForRead().V                     [Index]; }
	const Chaos::FVec3                         & GetW                     (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::W                     )); return BufferedData.GetGameDataForRead().W                     [Index]; }
	const Chaos::FVec3                         & GetF                     (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::F                     )); return BufferedData.GetGameDataForRead().F                     [Index]; }
	const Chaos::FVec3                         & GetTorque                (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::Torque                )); return BufferedData.GetGameDataForRead().Torque                [Index]; }
	const Chaos::FMatrix33                     & GetI                     (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::I                     )); return BufferedData.GetGameDataForRead().I                     [Index]; }
	const Chaos::FMatrix33                     & GetInvI                  (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::InvI                  )); return BufferedData.GetGameDataForRead().InvI                  [Index]; }
	const Chaos::FReal                         & GetM                     (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::M                     )); return BufferedData.GetGameDataForRead().M                     [Index]; }
	const Chaos::FReal                         & GetInvM                  (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::InvM                  )); return BufferedData.GetGameDataForRead().InvM                  [Index]; }
	const int32                                & GetCollisionParticlesSize(int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::CollisionParticlesSize)); return BufferedData.GetGameDataForRead().CollisionParticlesSize[Index]; }
	const bool                                 & IsDisabled               (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::Disabled              )); return BufferedData.GetGameDataForRead().Disabled              [Index]; }
	const bool                                 & IsSleeping               (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::Sleeping              )); return BufferedData.GetGameDataForRead().Sleeping              [Index]; }
	const int32                                & GetIsland                (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::Island                )); return BufferedData.GetGameDataForRead().Island                [Index]; }
	const Chaos::FVec3                         & GetP                     (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::P                     )); return BufferedData.GetGameDataForRead().P                     [Index]; }
	const Chaos::FRotation3                    & GetQ                     (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::Q                     )); return BufferedData.GetGameDataForRead().Q                     [Index]; }
	const Chaos::FVec3                         & GetPreV                  (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::PreV                  )); return BufferedData.GetGameDataForRead().PreV                  [Index]; }
	const Chaos::FVec3                         & GetPreW                  (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::PreW                  )); return BufferedData.GetGameDataForRead().PreW                  [Index]; }
	const TArray<Chaos::TConnectivityEdge<Chaos::FReal>> & GetConnectivityEdges     (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::ConnectivityEdges )); return BufferedData.GetGameDataForRead().ConnectivityEdges     [Index]; }
	const Chaos::FRigidTransform3              & GetChildToParentMap      (int32 Index) const { check(HasSyncedData(EGeometryCollectionParticlesData::ChildToParentMap      )); return BufferedData.GetGameDataForRead().ChildToParentMap      [Index]; }

	/** Return a string with the entire set of value for the synced data of the specified particle. */
	FString ToString(int32 Index, const TCHAR* Separator = TEXT(", ")) const { return BufferedData.GetGameDataForRead().ToString(Index, Separator); }

private:
	/** Data bit selection type. */
	typedef TStaticBitArray<uint32(EGeometryCollectionParticlesData::Count)> FDataFlags;

	/** Structure used to exchange data between game and physics thread. */
	struct FData
	{
		mutable FDataFlags RequiredDataFlags;  // Mutable in order to allow const versions of all of this object's methods outside of the Sync process.
		FDataFlags SyncedDataFlags;
		Chaos::TArrayCollectionArray<Chaos::FVec3                           > X;
		Chaos::TArrayCollectionArray<Chaos::FRotation3                      > R;
		Chaos::TArrayCollectionArray<const Chaos::FImplicitObject*          > Geometry;
		Chaos::TArrayCollectionArray<Chaos::EImplicitObjectType             > GeometryType;
		Chaos::TArrayCollectionArray<bool                                   > GeometryIsConvex;
		Chaos::TArrayCollectionArray<bool                                   > GeometryHasBoundingBox;
		Chaos::TArrayCollectionArray<Chaos::FVec3                           > GeometryBoxMin;
		Chaos::TArrayCollectionArray<Chaos::FVec3                           > GeometryBoxMax;
		Chaos::TArrayCollectionArray<Chaos::FVec3                           > GeometrySphereCenter;
		Chaos::TArrayCollectionArray<Chaos::FReal                           > GeometrySphereRadius;
		Chaos::TArrayCollectionArray<Chaos::TUniformGrid<Chaos::FReal, 3>   > GeometryLevelSetGrid;
		Chaos::TArrayCollectionArray<Chaos::FVec3                           > V;
		Chaos::TArrayCollectionArray<Chaos::FVec3                           > W;
		Chaos::TArrayCollectionArray<Chaos::FVec3                           > F;
		Chaos::TArrayCollectionArray<Chaos::FVec3                           > Torque;
		Chaos::TArrayCollectionArray<Chaos::FMatrix33                       > I;
		Chaos::TArrayCollectionArray<Chaos::FMatrix33                       > InvI;
		Chaos::TArrayCollectionArray<Chaos::FReal                           > M;
		Chaos::TArrayCollectionArray<Chaos::FReal                           > InvM;
		Chaos::TArrayCollectionArray<int32                                  > CollisionParticlesSize;
		Chaos::TArrayCollectionArray<bool                                   > Disabled;
		Chaos::TArrayCollectionArray<bool                                   > Sleeping;
		Chaos::TArrayCollectionArray<int32                                  > Island;
		Chaos::TArrayCollectionArray<Chaos::FVec3                           > P;
		Chaos::TArrayCollectionArray<Chaos::FRotation3                      > Q;
		Chaos::TArrayCollectionArray<Chaos::FVec3                           > PreV;
		Chaos::TArrayCollectionArray<Chaos::FVec3                           > PreW;
		Chaos::TArrayCollectionArray<TArray<Chaos::TConnectivityEdge<Chaos::FReal>>> ConnectivityEdges;
		Chaos::TArrayCollectionArray<Chaos::FRigidTransform3                > ChildToParentMap;

		/** Set this data type to copy at the next sync. */
		void SetDataSyncFlag(EGeometryCollectionParticlesData Data) const { RequiredDataFlags[uint32(Data)] = true; }

		/** Set all data type to copy at the next sync. */
		void SetAllDataSyncFlag() const;

		/** Return whether the specified type of data has been copied during the last sync. */
		bool HasSyncedData(EGeometryCollectionParticlesData Data) const { return SyncedDataFlags[uint32(Data)]; }

		/** Deallocate the array containing the specified particle information. */
		void Reset(EGeometryCollectionParticlesData Data);

		/** Copy the specified particle information for the specified range of rigid body id. */
		void Copy(EGeometryCollectionParticlesData Data, const Chaos::FPhysicsSolver* Solver, const TManagedArray<FGuid>& RigidBodyIds);

		/** Return a string with the entire set of value for the synced data of the specified particle. */
		FString ToString(int32 Index, const TCHAR* Separator) const;

	};

	const FChaosSolversModule* ChaosModule;
	Chaos::TBufferedData<FData> BufferedData;
	FThreadSafeCounter PhysicsSyncCount;
	int32 GameSyncCount;
	uint64 SyncFrame;
};	

template<class T, int d>
using TGeometryCollectionParticlesData UE_DEPRECATED(4.27, "Deprecated. this class is to be deleted, use FGeometryCollectionParticlesData instead") = FGeometryCollectionParticlesData;

#endif  // #if GEOMETRYCOLLECTION_DEBUG_DRAW
