// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "GeometryCollection/GeometryCollection.h"
#include "Engine/VolumeTexture.h"
#include "Chaos/Levelset.h"
#include "Engine/StaticMesh.h"
#include "Components/PostProcessComponent.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "GeometryCollectionRenderLevelSetActor.generated.h"


/**
*	AGeometryCollectionRenderLevelSetActor
*    An actor representing the collection of data necessary to 
*    render volumes.  This references a ray marching material, which
*    is used internally by a post process component blendable.  This
*    is a workflow that can be improved with a deeper implementation
*    in the future if we decide to.  Note that behavior with multiple
*    render level set actors isn't currently supported very well,
*    but could be improved in the future
*/
UCLASS(MinimalAPI)
class AGeometryCollectionRenderLevelSetActor : public AActor
{
	GENERATED_UCLASS_BODY()
public:

	static GEOMETRYCOLLECTIONENGINE_API int InstanceCount;

	// Volume texture to fill	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Volume")
	TObjectPtr<UVolumeTexture> TargetVolumeTexture;

	// Material that performs ray marching.  Note this must have certain parameters in order
	// to work correctly
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	TObjectPtr<UMaterial> RayMarchMaterial;

	// Surface tolerance used for rendering.  When surface reconstruction is noisy,
	// try tweaking this value
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	float SurfaceTolerance;

	// Isovalue of the level set to use for surface reconstruction.  Generally you want
	// this to be zero, but it can be useful for exploring the distance values to make
	// this negative to see the interior structure of the levelset 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	float Isovalue;

	// Enable or disable rendering
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	bool Enabled;

	// Enable or disable rendering
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	bool RenderVolumeBoundingBox;

	// Find/create the level set renderer singleton actor as required. Return whether the found or created actor.
	static GEOMETRYCOLLECTIONENGINE_API AGeometryCollectionRenderLevelSetActor* FindOrCreate(UWorld* World);

	// Load a new level set to render
	GEOMETRYCOLLECTIONENGINE_API bool SetLevelSetToRender(const Chaos::FLevelSet &LevelSet, const FTransform &LocalToWorld);

	// Sync level set transform to the render material
	GEOMETRYCOLLECTIONENGINE_API void SyncLevelSetTransform(const FTransform &LocalToWorld);

	// Some initialization happens in here 
	GEOMETRYCOLLECTIONENGINE_API virtual void BeginPlay() override;

#if WITH_EDITOR
	// Allowed for live updates to parameters from inside the editor when ejected
	GEOMETRYCOLLECTIONENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& e) override;
#endif

	// set and sync enabled
	void SetEnabled(bool enabled)
	{
		Enabled = enabled;
		PostProcessComponent->bEnabled = Enabled;
	}

private:
	FVector MinBBoxCorner;
	FVector MaxBBoxCorner;

	FMatrix WorldToLocal;

	float VoxelSize;

	UPostProcessComponent *PostProcessComponent;

	// Dynamic material instance so we can update parameters based on volume changes
	UMaterialInstanceDynamic* DynRayMarchMaterial;

	// Synchronizes the state of this actor class with the post process render material
	GEOMETRYCOLLECTIONENGINE_API void SyncMaterialParameters();

	// Private for now since step size mult might not be super useful due to the current
	// rendering algorithms employed in the shaders
	// @todo: expose this in a meaningful way in the future if necessary
	// UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Rendering")
	float StepSizeMult;
};
