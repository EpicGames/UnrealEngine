// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCollection/GeometryCollectionExternalRenderInterface.h"

#include "GeometryCollectionRootProxyRenderer.generated.h"

class UGeometryCollectionComponent;

/** Implementation of a geometry collection custom renderer that creates static mesh components representing the root proxies. */
UCLASS(MinimalAPI)
class UGeometryCollectionRootProxyRenderer : public UObject, public IGeometryCollectionExternalRenderInterface
{
	GENERATED_BODY()

	/** Static mesh components for the root proxy meshes. */
	UPROPERTY()
	TArray<TObjectPtr<UStaticMeshComponent>> StaticMeshComponents;

public:
	//~ Begin IGeometryCollectionExternalRenderInterface Interface.
	virtual void OnRegisterGeometryCollection(UGeometryCollectionComponent& InComponent) override;
	virtual void OnUnregisterGeometryCollection() override;
	virtual void UpdateState(UGeometryCollection const& InGeometryCollection, FTransform const& InComponentTransform, uint32 InStateFlags) override;
	virtual void UpdateRootTransform(UGeometryCollection const& InGeometryCollection, FTransform const& InRootTransform) override;
	virtual void UpdateRootTransforms(UGeometryCollection const& InGeometryCollection, FTransform const& InRootTransform, TArrayView<const FTransform3f> InRootLocalTransforms) override;
	virtual void UpdateTransforms(UGeometryCollection const& InGeometryCollection, TArrayView<const FTransform3f> InTransforms) override;
	virtual bool ShouldUseNativeFallback(uint32 InStateFlags) const override { return (InStateFlags & EState_Broken) != 0; }
	virtual bool CanEverUseNativeFallback() const override { return true; }
	//~ End IGeometryCollectionExternalRenderInterface Interface.

protected:
	/** Current visibility state. */
	bool bIsVisible = true;

private:
	void CreateRootProxyComponents(UGeometryCollectionComponent& InComponent);
	void UpdateRootProxyTransforms(UGeometryCollection const& InGeometryCollection, FTransform const& InRootTransform, TArrayView<const FTransform3f> InLocalRootTransforms);
	void ClearRootProxyComponents();
};
