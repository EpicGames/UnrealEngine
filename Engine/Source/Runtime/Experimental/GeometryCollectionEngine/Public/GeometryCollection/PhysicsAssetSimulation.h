// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Chaos/ChaosNotifyHandlerInterface.h"
#include "GeometryCollection/GeometryCollectionSimulationTypes.h"
#include "PhysicsInterfaceDeclaresCore.h"

class AActor;
class FSkeletalMeshPhysicsProxy;
struct FSkeletalMeshPhysicsProxyParams;
class UChaosPhysicalMaterial;
class UPhysicsAsset;
class USkeletalMeshComponent;


struct FPhysicsAssetSimulationUtil
{
	static GEOMETRYCOLLECTIONENGINE_API void BuildParams(const UObject* Caller, const AActor* OwningActor, const USkeletalMeshComponent* SkelMeshComponent, const UPhysicsAsset* PhysicsAsset, FSkeletalMeshPhysicsProxyParams& OutParams);
	static GEOMETRYCOLLECTIONENGINE_API bool UpdateAnimState(const UObject* Caller, const AActor* OwningActor, const USkeletalMeshComponent* SkelMeshComponent, const float Dt, FSkeletalMeshPhysicsProxyParams& OutParams);
};
