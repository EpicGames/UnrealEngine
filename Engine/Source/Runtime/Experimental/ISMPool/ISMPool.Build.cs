// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ISMPool : ModuleRules
	{
		public ISMPool(ReadOnlyTargetRules Target) : base(Target)
		{
			SetupModulePhysicsSupport(Target);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
				}
				);

			if (Target.bBuildEditor)
			{
				PublicDependencyModuleNames.Add("UnrealEd");
			}

			bDisableAutoRTFMInstrumentation = true;
		}
	}
}
