// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISMPool/ISMPoolActor.h"
#include "ISMPool/ISMPoolComponent.h"
#include "ISMPool/ISMPoolDebugDrawComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ISMPoolActor)

AISMPoolActor::AISMPoolActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ISMPoolComp = CreateDefaultSubobject<UISMPoolComponent>(TEXT("ISMPoolComp"));
	RootComponent = ISMPoolComp;

#if UE_ENABLE_DEBUG_DRAWING
	ISMPoolDebugDrawComp = CreateDefaultSubobject<UISMPoolDebugDrawComponent>(TEXT("ISMPoolDebug"));
	ISMPoolDebugDrawComp->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	ISMPoolDebugDrawComp->SetCanEverAffectNavigation(false);
	ISMPoolDebugDrawComp->CanCharacterStepUpOn = ECanBeCharacterBase::ECB_No;
	ISMPoolDebugDrawComp->SetGenerateOverlapEvents(false);
	ISMPoolDebugDrawComp->SetupAttachment(ISMPoolComp);
#endif
}
