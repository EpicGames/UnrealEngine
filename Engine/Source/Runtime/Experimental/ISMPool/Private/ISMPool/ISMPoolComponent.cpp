// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISMPool/ISMPoolComponent.h"

#include "ChaosLog.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/CollisionProfile.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ISMPoolComponent)

static void ISMPoolComponent_EnableTicksForPoolManagedComponents(IConsoleVariable*)
{
	//exclude CDOs & garbage
	constexpr EObjectFlags ExclusionFlags = RF_ClassDefaultObject;
	constexpr EInternalObjectFlags InternalExclusionFlags = EInternalObjectFlags::Garbage;
	
	constexpr bool bIncludeDerivedClasses = true;

	for (TObjectIterator<UISMPoolComponent> It(ExclusionFlags, bIncludeDerivedClasses, InternalExclusionFlags); It; ++It)
	{
		//non-pool managed components don't need their tick re-enabling
		It->EnableComponentTickIfPoolManagementEnabled();
	}
}

// Don't release ISM components when they empty, but keep them (and their scene proxy) alive.
// This can remove the high cost associated with repeated registration, scene proxy creation and mesh draw command creation.
// But it can also have a high memory overhead since the ISMs retain hard references to their static meshes.
static bool GComponentKeepAlive = false; 
FAutoConsoleVariableRef CVarISMPoolComponentKeepAlive(
	TEXT("r.ISMPool.ComponentKeepAlive"),
	GComponentKeepAlive,
	TEXT("Keep ISM components alive when all their instances are removed."),
	FConsoleVariableDelegate::CreateStatic(ISMPoolComponent_EnableTicksForPoolManagedComponents));

// Use a FreeList to enable recycling of ISM components.
// ISM components aren't unregistered, but their scene proxy is destroyed.
// When recycling a component, a new mesh description can be used.
// This removes the high CPU cost of unregister/register.
// But there is more CPU cost to recycling a component then to simply keeping it alive because scene proxy creation and mesh draw command caching isn't cheap.
// The component memory cost is kept bounded when compared to keeping components alive.
static bool GComponentRecycle = true;
FAutoConsoleVariableRef CVarISMPoolComponentRecycle(
	TEXT("r.ISMPool.ComponentRecycle"),
	GComponentRecycle,
	TEXT("Recycle ISM components to a free list for reuse when all their instances are removed."),
	FConsoleVariableDelegate::CreateStatic(ISMPoolComponent_EnableTicksForPoolManagedComponents));

static bool GISMPoolClearComponentMeshOnRecycle = true;
FAutoConsoleVariableRef CVarISMPoolClearComponentMeshOnRecycle(
	TEXT("r.ISMPool.ClearComponentMeshOnRecycle"),
	GISMPoolClearComponentMeshOnRecycle,
	TEXT("If true, ISM components on the free list will have their StaticMesh property cleared - to prevent holding a reference to an unused mesh"));

// Target free list size when recycling ISM components.
// We try to maintain a pool of free components for fast allocation, but want to clean up when numbers get too high.
static int32 GComponentFreeListTargetSize = 50;
FAutoConsoleVariableRef CVarISMPoolComponentFreeListTargetSize(
	TEXT("r.ISMPool.ComponentFreeListTargetSize"),
	GComponentFreeListTargetSize,
	TEXT("Target size for number of ISM components in the recycling free list."),
	FConsoleVariableDelegate::CreateStatic(ISMPoolComponent_EnableTicksForPoolManagedComponents));

// Keep copies of all custom instance data for restoration on readding an instance.
static bool GShadowCopyCustomData = false;
FAutoConsoleVariableRef CVarISMPoolShadowCopyCustomData(
	TEXT("r.ISMPool.ShadowCopyCustomData"),
	GShadowCopyCustomData,
	TEXT("Keeps a copy of custom instance data so it can be restored if the instance is removed and readded."));


void FISMPoolMeshInfo::ShadowCopyCustomData(int32 InstanceCount, int32 NumCustomDataFloatsPerInstance, TArrayView<const float> CustomDataFloats)
{
	CustomData.SetNum(InstanceCount * NumCustomDataFloatsPerInstance + NumCustomDataFloatsPerInstance);

	for (int32 InstanceIndex = 0; InstanceIndex < InstanceCount; ++InstanceIndex)
	{
		int32 Offset = InstanceIndex * NumCustomDataFloatsPerInstance;
		FMemory::Memcpy(&CustomData[Offset], CustomDataFloats.GetData() + Offset, NumCustomDataFloatsPerInstance * CustomDataFloats.GetTypeSize());
	}
}

TArrayView<const float> FISMPoolMeshInfo::CustomDataSlice(int32 InstanceIndex, int32 NumCustomDataFloatsPerInstance)
{
	TArrayView<const float> DataView = CustomData;
	return DataView.Slice(InstanceIndex * NumCustomDataFloatsPerInstance, NumCustomDataFloatsPerInstance);
}

FISMPoolMeshGroup::FMeshId FISMPoolMeshGroup::AddMesh(const FISMPoolStaticMeshInstance& MeshInstance, int32 InstanceCount, const FISMPoolMeshInfo& ISMInstanceInfo, TArrayView<const float> CustomDataFloats)
{
	const FMeshId MeshInfoIndex = MeshInfos.Emplace(ISMInstanceInfo);

	if (bAllowPerInstanceRemoval && GShadowCopyCustomData)
	{
		FISMPoolMeshInfo& MeshInfo = MeshInfos[MeshInfoIndex];
		MeshInfo.ShadowCopyCustomData(InstanceCount, MeshInstance.Desc.NumCustomDataFloats, CustomDataFloats);
	}

	return MeshInfoIndex;
}

bool FISMPoolMeshGroup::BatchUpdateInstancesTransforms(FISMPool& ISMPool, FMeshId MeshId, int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	if (MeshInfos.IsValidIndex(MeshId))
	{
		return ISMPool.BatchUpdateInstancesTransforms(MeshInfos[MeshId], StartInstanceIndex, NewInstancesTransforms, bWorldSpace, bMarkRenderStateDirty, bTeleport, bAllowPerInstanceRemoval);
	}
	UE_LOGF(LogChaos, Warning, "UISMPoolComponent : Invalid mesh Id (%d) for this mesh group", MeshId);
	return false;
}

void FISMPoolMeshGroup::BatchUpdateInstanceCustomData(FISMPool& ISMPool, int32 CustomFloatIndex, float CustomFloatValue)
{
	for (const FISMPoolMeshInfo& MeshInfo : MeshInfos)
	{
		ISMPool.BatchUpdateInstanceCustomData(MeshInfo, CustomFloatIndex, CustomFloatValue);
	}
}

void FISMPoolMeshGroup::RemoveAllMeshes(FISMPool& ISMPool)
{
	for (const FISMPoolMeshInfo& MeshInfo: MeshInfos)
	{
		ISMPool.RemoveInstancesFromISM(MeshInfo);
	}
	MeshInfos.Empty();
}

void FISMPoolISM::CreateISM(USceneComponent* InOwningComponent)
{
	check(InOwningComponent);

	AActor* OwningActor = InOwningComponent->GetOwner();
	USceneComponent* RootComponent = OwningActor->GetRootComponent();

	ISMComponent = NewObject<UInstancedStaticMeshComponent>(InOwningComponent, NAME_None, RF_Transient | RF_DuplicateTransient);

	ISMComponent->SetRemoveSwap();
	ISMComponent->SetCanEverAffectNavigation(false);
	ISMComponent->SetCollisionEnabled(ECollisionEnabled::NoCollision);
	ISMComponent->SetupAttachment(RootComponent);
	ISMComponent->RegisterComponent();

#if WITH_EDITOR
	UWorld const* World = InOwningComponent->GetWorld();
	const bool bShowInWorldOutliner = World && World->IsGameWorld();
	if (bShowInWorldOutliner)
	{
		OwningActor->AddInstanceComponent(ISMComponent);
	}
#endif
}

void FISMPoolISM::InitISM(const FISMPoolStaticMeshInstance& InMeshInstance, bool bKeepAlive, bool bOverrideTransformUpdates)
{
	MeshInstance = InMeshInstance;
	check(ISMComponent != nullptr);

	UStaticMesh* StaticMesh = MeshInstance.StaticMesh.Get();
	// We should only get here for valid static mesh objects.
	ensureMsgf(StaticMesh != nullptr, TEXT("StaticMesh is not valid."));

#if WITH_EDITOR
	const FName MeshName = StaticMesh != nullptr ? StaticMesh->GetFName() : NAME_None;
	const FName ISMName = MakeUniqueObjectName(ISMComponent->GetOwner(), UInstancedStaticMeshComponent::StaticClass(), MeshName);
	const FString ISMNameString = ISMName.ToString();
	ISMComponent->Rename(*ISMNameString);
#endif

	ISMComponent->bUseAttachParentBound = bOverrideTransformUpdates;
	ISMComponent->SetAbsolute(bOverrideTransformUpdates, bOverrideTransformUpdates, bOverrideTransformUpdates);

	bool bDisallowNanite = false;

	ISMComponent->EmptyOverrideMaterials();
	for (int32 MaterialIndex = 0; MaterialIndex < MeshInstance.MaterialsOverrides.Num(); MaterialIndex++)
	{
		UMaterialInterface* Material = MeshInstance.MaterialsOverrides[MaterialIndex].Get();
		// We should only get here for valid material objects.
		check(Material != nullptr);
		ISMComponent->SetMaterial(MaterialIndex, Material);

		// Nanite doesn't support translucent materials.
		bDisallowNanite |= Material->GetBlendMode() == BLEND_Translucent;
	}

	ISMComponent->SetMobility((MeshInstance.Desc.Flags & FISMPoolComponentDescription::StaticMobility) != 0 ? EComponentMobility::Static : EComponentMobility::Movable);

	ISMComponent->NumCustomDataFloats = MeshInstance.Desc.NumCustomDataFloats;
	for (int32 DataIndex = 0; DataIndex < MeshInstance.CustomPrimitiveData.Num(); DataIndex++)
	{
		ISMComponent->SetDefaultCustomPrimitiveDataFloat(DataIndex, MeshInstance.CustomPrimitiveData[DataIndex]);
	}

	const bool bReverseCulling = (MeshInstance.Desc.Flags & FISMPoolComponentDescription::ReverseCulling) != 0;
	// Instead of reverse culling we put the mirror in the component transform so that PRIMITIVE_SCENE_DATA_FLAG_DETERMINANT_SIGN will be set for use by materials.
	//ISMComponent->SetReverseCulling(bReverseCulling);
	const FVector Scale = bReverseCulling ? FVector(-1, 1, 1) : FVector(1, 1, 1);

	if(bOverrideTransformUpdates)
	{
		FTransform TempTm = ISMComponent->GetAttachParent() ?
			ISMComponent->GetAttachParent()->GetComponentToWorld() :
			FTransform::Identity;

		// Apply above identified scale to the transform directly
		TempTm.SetScale3D(TempTm.GetScale3D() * Scale);

		ISMComponent->SetComponentToWorld(TempTm);
		ISMComponent->UpdateComponentTransform(EUpdateTransformFlags::None, ETeleportType::None);
		ISMComponent->MarkRenderTransformDirty();
	}
	else
	{
		const FTransform NewRelativeTransform(FQuat::Identity, MeshInstance.Desc.Position, Scale);

		if(!ISMComponent->GetRelativeTransform().Equals(NewRelativeTransform))
		{
			// If we're not overriding the transform and need a relative offset, apply that here
			ISMComponent->SetRelativeTransform(FTransform(FQuat::Identity, MeshInstance.Desc.Position, Scale));
		}
	}

	if ((MeshInstance.Desc.Flags & FISMPoolComponentDescription::DistanceCullPrimitive) != 0)
	{
		ISMComponent->SetCachedMaxDrawDistance(MeshInstance.Desc.EndCullDistance);
	}

	ISMComponent->SetCullDistances(MeshInstance.Desc.StartCullDistance, MeshInstance.Desc.EndCullDistance);
	ISMComponent->SetCastShadow((MeshInstance.Desc.Flags & FISMPoolComponentDescription::AffectShadow) != 0);
	ISMComponent->bAffectDynamicIndirectLighting = (MeshInstance.Desc.Flags & FISMPoolComponentDescription::AffectDynamicIndirectLighting) != 0;
	ISMComponent->bAffectDistanceFieldLighting = (MeshInstance.Desc.Flags & FISMPoolComponentDescription::AffectDistanceFieldLighting) != 0;
	ISMComponent->bCastFarShadow = (MeshInstance.Desc.Flags & FISMPoolComponentDescription::AffectFarShadow) != 0;
	ISMComponent->bWorldPositionOffsetWritesVelocity = (MeshInstance.Desc.Flags & FISMPoolComponentDescription::WorldPositionOffsetWritesVelocity) != 0;
	ISMComponent->bEvaluateWorldPositionOffset = (MeshInstance.Desc.Flags & FISMPoolComponentDescription::EvaluateWorldPositionOffset) != 0;
	ISMComponent->bUseGpuLodSelection = (MeshInstance.Desc.Flags & FISMPoolComponentDescription::GpuLodSelection) != 0;
	ISMComponent->bOverrideMinLOD = MeshInstance.Desc.MinLod > 0;
	ISMComponent->MinLOD = MeshInstance.Desc.MinLod;
	ISMComponent->SetLODDistanceScale(MeshInstance.Desc.LodScale);
	ISMComponent->SetUseConservativeBounds(true);
	ISMComponent->bComputeFastLocalBounds = true;
	ISMComponent->bDisallowNanite = bDisallowNanite;
	ISMComponent->SetMeshDrawCommandStatsCategory(MeshInstance.Desc.StatsCategory);
	ISMComponent->ComponentTags = MeshInstance.Desc.Tags;
	ISMComponent->SetRenderCustomDepth(MeshInstance.Desc.Flags & FISMPoolComponentDescription::UsesCustomDepthStencil);
	ISMComponent->SetCustomDepthStencilValue(MeshInstance.Desc.CustomDepthStencilValue);

	// Use a fixed seed to avoid getting a different seed at every run (see UInstancedStaticMeshComponent::OnRegister())
	// A possible improvement would be to compute a hash from the owner component and use that as the seed.
	ISMComponent->InstancingRandomSeed = 1;	

	// Make sure this is called at the end so the full state is set, since SetStaticMesh implicitly calls PrecachePSO()
	ISMComponent->SetStaticMesh(StaticMesh);
}

FISMPoolInstanceGroups::FInstanceGroupId FISMPoolISM::AddInstanceGroup(int32 InstanceCount, TArrayView<const float> CustomDataFloats)
{
	// When adding new group it will always have a single range
	const FISMPoolInstanceGroups::FInstanceGroupId InstanceGroupIndex = InstanceGroups.AddGroup(InstanceCount);
	const FISMPoolInstanceGroups::FInstanceGroupRange& NewInstanceGroup = InstanceGroups.GroupRanges[InstanceGroupIndex];

	// Ensure that remapping arrays are big enough to hold any new items.
	InstanceIds.SetNum(InstanceGroups.GetMaxInstanceIndex(), EAllowShrinking::No);

	FTransform ZeroScaleTransform;
	ZeroScaleTransform.SetIdentityZeroScale();
	TArray<FTransform> ZeroScaleTransforms;
	ZeroScaleTransforms.Init(ZeroScaleTransform, InstanceCount);

	ISMComponent->PreAllocateInstancesMemory(InstanceCount);
	TArray<FPrimitiveInstanceId> AddedInstanceIds = ISMComponent->AddInstancesById(ZeroScaleTransforms, true, true);
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceCount; ++InstanceIndex)
	{
		InstanceIds[NewInstanceGroup.Start + InstanceIndex] = AddedInstanceIds[InstanceIndex];
	}

	// Set any custom data.
	if (CustomDataFloats.Num())
	{
		const int32 NumCustomDataFloats = ISMComponent->NumCustomDataFloats;
		if (ensure(NumCustomDataFloats * InstanceCount == CustomDataFloats.Num()))
		{
			for (int32 InstanceIndex = 0; InstanceIndex < InstanceCount; ++InstanceIndex)
			{
				ISMComponent->SetCustomDataById(AddedInstanceIds[InstanceIndex], CustomDataFloats.Slice(InstanceIndex * NumCustomDataFloats, NumCustomDataFloats));
			}
		}
	}

	return InstanceGroupIndex;
}

FISMPool::FISMPool()
	: bCachedKeepAlive(GComponentKeepAlive)
	, bCachedRecycle(GComponentRecycle)
{
}

FISMPool::FISMIndex FISMPool::GetOrAddISM(UISMPoolComponent* OwningComponent, const FISMPoolStaticMeshInstance& MeshInstance, bool& bOutISMCreated)
{
	FISMIndex* ISMIndexPtr = MeshToISMIndex.Find(MeshInstance);
	if (ISMIndexPtr != nullptr)
	{
		bOutISMCreated = false;
		return *ISMIndexPtr;
	}

	// Take an ISM from the current FreeLists if available instead of allocating a new slot.
	FISMIndex ISMIndex = INDEX_NONE;
	if (FreeListISM.Num())
	{
		ISMIndex = FreeListISM.Last();
		FreeListISM.RemoveAt(FreeListISM.Num() - 1);
	}
	else if (FreeList.Num())
	{
		ISMIndex = FreeList.Last();
		FreeList.RemoveAt(FreeList.Num() - 1);
		ISMs[ISMIndex].CreateISM(OwningComponent);
	}
	else
	{
		ISMIndex = ISMs.AddDefaulted();
		ISMs[ISMIndex].CreateISM(OwningComponent);
	}
	
	ISMs[ISMIndex].InitISM(MeshInstance, bCachedKeepAlive, bDisableBoundsAndTransformUpdate);
	
	bOutISMCreated = true;
	MeshToISMIndex.Add(MeshInstance, ISMIndex);
	return ISMIndex;
}

FISMPoolMeshInfo FISMPool::AddInstancesToISM(UISMPoolComponent* OwningComponent, const FISMPoolStaticMeshInstance& MeshInstance, int32 InstanceCount, TArrayView<const float> CustomDataFloats)
{
	bool bISMCreated = false;
	FISMPoolMeshInfo Info;
	Info.ISMIndex = GetOrAddISM(OwningComponent, MeshInstance, bISMCreated);
	Info.InstanceGroupIndex = ISMs[Info.ISMIndex].AddInstanceGroup(InstanceCount, CustomDataFloats);
	return Info;
}

bool FISMPool::BatchUpdateInstancesTransforms(FISMPoolMeshInfo& MeshInfo, int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport, bool bAllowPerInstanceRemoval)
{
	if (!ISMs.IsValidIndex(MeshInfo.ISMIndex))
	{
		UE_LOGF(LogChaos, Warning, "UISMPoolComponent : Invalid ISM Id (%d) when updating the transform ", MeshInfo.ISMIndex);
		return false;
	}
		
	FISMPoolISM& ISM = ISMs[MeshInfo.ISMIndex];
	const FISMPoolInstanceGroups::FInstanceGroupRange& InstanceGroup = ISM.InstanceGroups.GroupRanges[MeshInfo.InstanceGroupIndex];
	// If ISM component has identity transform (the common case) then we can skip world space to component space maths inside BatchUpdateInstancesTransforms()
	bWorldSpace &= !ISM.ISMComponent->GetComponentTransform().Equals(FTransform::Identity, 0.f);

	// The transform count should fit within the instance group.
	// Clamp it if it doesn't, but if we hit this ensure we need to investigate why.
	ensure(StartInstanceIndex + NewInstancesTransforms.Num() <= InstanceGroup.Count);
	const int32 NumTransforms = FMath::Min(NewInstancesTransforms.Num(), InstanceGroup.Count - StartInstanceIndex);

	// Loop over transforms.
	// todo: There may be some value in batching InstanceIds and caling one function for each of Add/Remove/Update.
	// However the ISM batched calls themselves seem to be just simple loops over the single instance calls, so probably no benefit.
	for (int InstanceIndex = StartInstanceIndex; InstanceIndex < StartInstanceIndex + NumTransforms; ++InstanceIndex)
	{
		FPrimitiveInstanceId InstanceId = ISM.InstanceIds[InstanceGroup.Start + InstanceIndex];
		FTransform const& Transform = NewInstancesTransforms[InstanceIndex];

		if (bAllowPerInstanceRemoval)
		{
			if (Transform.GetScale3D().IsZero() && InstanceId.IsValid())
			{
				// Zero scale is used to indicate that we should remove the instance from the ISM.
				ISM.ISMComponent->RemoveInstanceById(InstanceId);
				ISM.InstanceIds[InstanceGroup.Start + InstanceIndex] = FPrimitiveInstanceId();
				continue;
			}
			else if (!Transform.GetScale3D().IsZero() && !InstanceId.IsValid())
			{
				// Re-add the instance to the ISM if the scale becomes non-zero.
				FPrimitiveInstanceId Id = ISM.ISMComponent->AddInstanceById(Transform, bWorldSpace);
				ISM.InstanceIds[InstanceGroup.Start + InstanceIndex] = Id;

				if (MeshInfo.CustomData.Num())
				{
					ISM.ISMComponent->SetCustomDataById(Id, MeshInfo.CustomDataSlice(InstanceIndex, ISM.ISMComponent->NumCustomDataFloats));
				}
				continue;
			}
		}

		if (InstanceId.IsValid())
		{
			ISM.ISMComponent->UpdateInstanceTransformById(InstanceId, Transform, bWorldSpace, bTeleport);
		}
	}

	return true;
}

void FISMPool::BatchUpdateInstanceCustomData(FISMPoolMeshInfo const& MeshInfo, int32 CustomFloatIndex, float CustomFloatValue)
{
	if (!ISMs.IsValidIndex(MeshInfo.ISMIndex))
	{
		return;
	}
	
	FISMPoolISM& ISM = ISMs[MeshInfo.ISMIndex];
	if (!ensure(CustomFloatIndex < ISM.MeshInstance.Desc.NumCustomDataFloats))
	{
		return;
	}
		
	const FISMPoolInstanceGroups::FInstanceGroupRange& InstanceGroup = ISM.InstanceGroups.GroupRanges[MeshInfo.InstanceGroupIndex];
	for (int32 InstanceIndex = 0; InstanceIndex < InstanceGroup.Count; ++InstanceIndex)
	{
		const FPrimitiveInstanceId InstanceId = ISM.InstanceIds[InstanceGroup.Start + InstanceIndex];
		if (InstanceId.IsValid())
		{
			ISM.ISMComponent->SetCustomDataValueById(InstanceId, CustomFloatIndex, CustomFloatValue);
		}
	}
}

void FISMPool::RemoveInstancesFromISM(const FISMPoolMeshInfo& MeshInfo)
{
	if (ISMs.IsValidIndex(MeshInfo.ISMIndex))
	{
		FISMPoolISM& ISM = ISMs[MeshInfo.ISMIndex];
		const FISMPoolInstanceGroups::FInstanceGroupRange& InstanceGroup = ISM.InstanceGroups.GroupRanges[MeshInfo.InstanceGroupIndex];
		
		for (int32 Index = 0; Index < InstanceGroup.Count; ++Index)
		{
			FPrimitiveInstanceId InstanceId = ISM.InstanceIds[InstanceGroup.Start + Index];
			if (InstanceId.IsValid())
			{
				// todo: Could RemoveInstanceByIds() instead as long as that function can handle skipping invalid InstanceIds.
				ISM.ISMComponent->RemoveInstanceById(InstanceId);
			}
		}
		
#if DO_CHECK
		// clear the IDs
		for (int32 Index = 0; Index < InstanceGroup.Count; ++Index)
		{
			ISM.InstanceIds[InstanceGroup.Start + Index] = FPrimitiveInstanceId();
		}
#endif
		ISM.InstanceGroups.RemoveGroup(MeshInfo.InstanceGroupIndex);
	
		if (ISM.InstanceGroups.IsEmpty())
		{
			ensure(ISM.ISMComponent->PerInstanceSMData.Num() == 0);

			// No live instances, so take opportunity to reset indexing.
			ISM.InstanceGroups.Reset();
			ISM.InstanceIds.Reset();

			RemoveISM(MeshInfo.ISMIndex, bCachedKeepAlive, bCachedRecycle);

			if (!bCachedKeepAlive)
			{
				MeshToISMIndex.Remove(ISM.MeshInstance);
			}
		}
	}
}

void FISMPool::RemoveISM(FISMIndex ISMIndex, bool bKeepAlive, bool bRecycle)
{
	FISMPoolISM& ISM = ISMs[ISMIndex];
	ensure(ISM.InstanceGroups.IsEmpty());
	ensure(ISM.InstanceIds.IsEmpty());

	if (bKeepAlive)
	{
		// Nothing to do.
	}
	else if (bRecycle)
	{
		// Recycle to the free list.
#if WITH_EDITOR
		ISM.ISMComponent->Rename(nullptr);
#endif

		if (GISMPoolClearComponentMeshOnRecycle)
		{
			check(ISM.ISMComponent);
			ISM.ISMComponent->SetStaticMesh(nullptr);
		}
		FreeListISM.Add(ISMIndex);
	}
	else
	{
		// Completely unregister and destroy the component and mark the ISM slot as free.
		ISM.ISMComponent->DestroyComponent();
		ISM.ISMComponent = nullptr;
		
		FreeList.Add(ISMIndex);
	}
}

void FISMPool::Clear()
{
	MeshToISMIndex.Reset();
	PrellocationQueue.Reset();
	FreeList.Reset();
	FreeListISM.Reset();
	if (ISMs.Num() > 0)
	{
		if (AActor* OwningActor = ISMs[0].ISMComponent->GetOwner())
		{
			for(FISMPoolISM& ISM : ISMs)
			{
				ISM.ISMComponent->DestroyComponent();
			}
		}
		ISMs.Reset();
	}
}

void FISMPool::RequestPreallocateMeshInstance(const FISMPoolStaticMeshInstance& MeshInstance)
{
	// Preallocation only makes sense when we are keeping empty components alive.
	if (bCachedKeepAlive)
	{
		uint32 KeyHash = GetTypeHash(MeshInstance);
		if (MeshToISMIndex.FindByHash(KeyHash, MeshInstance) == nullptr)
		{
			PrellocationQueue.AddByHash(KeyHash, MeshInstance);
		}
	}
}

static bool AreWeakPointersValid(FISMPoolStaticMeshInstance& InMeshInstance)
{
	if (!InMeshInstance.StaticMesh.IsValid())
	{
		return false;
	}

	for (TWeakObjectPtr<UMaterialInterface> Material : InMeshInstance.MaterialsOverrides)
	{
		if (!Material.IsValid())
		{
			return false;
		}
	}

	return true;
}

void FISMPool::ProcessPreallocationRequests(UISMPoolComponent* OwningComponent, int32 MaxPreallocations)
{
	int32 NumAdded = 0;
	for (TSet<FISMPoolStaticMeshInstance>::TIterator It(PrellocationQueue); It; ++It)
	{
		bool bISMCreated = false;

		// Objects in the entries of the preallocation queue may no longer be loaded.
		if (AreWeakPointersValid(*It))
		{
			GetOrAddISM(OwningComponent, *It, bISMCreated);
		}

		It.RemoveCurrent();

		if (bISMCreated)
		{
			if (++NumAdded >= MaxPreallocations)
			{
				break;
			}
		}
	}
}

void FISMPool::UpdateAbsoluteTransforms(const FTransform& BaseTransform, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	for(const FISMPoolISM& GcIsm : ISMs)
	{
		const bool bReverseCulling = (GcIsm.MeshInstance.Desc.Flags & FISMPoolComponentDescription::ReverseCulling) != 0;
		check(GcIsm.MeshInstance.Desc.Position == FVector::ZeroVector);
		
		if(UInstancedStaticMeshComponent* Ism = GcIsm.ISMComponent)
		{
			if(bReverseCulling)
			{
				// As in InitISM we need to apply the inverted X scale for reverse culling.
				// Just copy the transform and set an inverted scale to apply to the ISM
				FVector BaseScale = BaseTransform.GetScale3D();
				BaseScale.X = -BaseScale.X;
				FTransform Flipped = BaseTransform;
				Flipped.SetScale3D(BaseScale);

				Ism->SetComponentToWorld(Flipped);
			}
			else
			{
				Ism->SetComponentToWorld(BaseTransform);
			}

			Ism->UpdateComponentTransform(UpdateTransformFlags | EUpdateTransformFlags::SkipPhysicsUpdate, Teleport);
			Ism->MarkRenderTransformDirty();
		}
	}
}

bool FISMPool::RequiresTick() const
{
	return !PrellocationQueue.IsEmpty() || (FreeListISM.Num() > GComponentFreeListTargetSize);
}

void FISMPool::Tick(UISMPoolComponent* OwningComponent)
{
	// Recache component lifecycle state from cvar.
	const bool bRemovedKeepAlive = bCachedKeepAlive && !GComponentKeepAlive;
	const bool bRemovedReycle = bCachedRecycle && !GComponentRecycle;
	bCachedKeepAlive = GComponentKeepAlive;
	bCachedRecycle = GComponentRecycle;

	// If we disabled keep alive behavior since last update then deal with the zombie components.
	if (bRemovedKeepAlive)
	{
		for (int32 ISMIndex = 0; ISMIndex < ISMs.Num(); ++ISMIndex)
		{
			FISMPoolISM& ISM = ISMs[ISMIndex];
			if (ISM.ISMComponent && ISM.InstanceGroups.IsEmpty())
			{
				// Actually release the ISM.
				RemoveISM(ISMIndex, false, bCachedRecycle);
				MeshToISMIndex.Remove(ISM.MeshInstance);
			}
		}
	}

	// Process preallocation queue.
	if (!bCachedKeepAlive)
	{
		PrellocationQueue.Reset();
	}
	else if (!PrellocationQueue.IsEmpty())
	{
		// Preallocate components per tick until the queue is empty.
		const int32 PreallocateCountPerTick = 2;
		ProcessPreallocationRequests(OwningComponent, PreallocateCountPerTick);
	}

	if (FreeListISM.Num() > 0)
	{
		// Release components per tick until we reach minimum pool size.
		const int32 RemoveCountPerTick = 1;
		const int32 FreeListTargetSize = bRemovedReycle ? 0 : FMath::Max(FMath::Max(FreeListISM.Num() - RemoveCountPerTick, GComponentFreeListTargetSize), 0);
		while (FreeListISM.Num() > FreeListTargetSize)
		{
			const int32 ISMIndex = FreeListISM.Pop(EAllowShrinking::No);
			RemoveISM(ISMIndex, false, false);
		}
	}
}


UISMPoolComponent::UISMPoolComponent(const FObjectInitializer& ObjectInitializer)
	: NextMeshGroupId(0)
{
	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = true;
	PrimaryComponentTick.bAllowTickOnDedicatedServer = false;
	PrimaryComponentTick.TickInterval = 0.25f;
}

void UISMPoolComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);
	Pool.Tick(this);
	SetComponentTickEnabled(Pool.RequiresTick());
}

UISMPoolComponent::FMeshGroupId  UISMPoolComponent::CreateMeshGroup(bool bAllowPerInstanceRemoval)
{
	FISMPoolMeshGroup Group;
	Group.bAllowPerInstanceRemoval = bAllowPerInstanceRemoval;
	MeshGroups.Add(NextMeshGroupId, Group);
	return NextMeshGroupId++;
}

void UISMPoolComponent::DestroyMeshGroup(FMeshGroupId MeshGroupId)
{
	if (FISMPoolMeshGroup* MeshGroup = MeshGroups.Find(MeshGroupId))
	{
		MeshGroup->RemoveAllMeshes(Pool);
		MeshGroups.Remove(MeshGroupId);
	}
}

UISMPoolComponent::FMeshId UISMPoolComponent::AddMeshToGroup(FMeshGroupId MeshGroupId, const FISMPoolStaticMeshInstance& MeshInstance, int32 InstanceCount, TArrayView<const float> CustomDataFloats)
{
	if (FISMPoolMeshGroup* MeshGroup = MeshGroups.Find(MeshGroupId))
	{
		const FISMPoolMeshInfo ISMInstanceInfo = Pool.AddInstancesToISM(this, MeshInstance, InstanceCount, CustomDataFloats);
		return MeshGroup->AddMesh(MeshInstance, InstanceCount, ISMInstanceInfo, CustomDataFloats);
	}
	UE_LOGF(LogChaos, Warning, "UISMPoolComponent : Trying to add a mesh to a mesh group (%d) that does not exists", MeshGroupId);
	return INDEX_NONE;
}

bool UISMPoolComponent::BatchUpdateInstancesTransforms(FMeshGroupId MeshGroupId, FMeshId MeshId, int32 StartInstanceIndex, const TArray<FTransform>& NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	return BatchUpdateInstancesTransforms(MeshGroupId, MeshId, StartInstanceIndex, MakeArrayView(NewInstancesTransforms), bWorldSpace, bMarkRenderStateDirty, bTeleport);
}

bool UISMPoolComponent::BatchUpdateInstancesTransforms(FMeshGroupId MeshGroupId, FMeshId MeshId, int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
	if (FISMPoolMeshGroup* MeshGroup = MeshGroups.Find(MeshGroupId))
	{
		return MeshGroup->BatchUpdateInstancesTransforms(Pool, MeshId, StartInstanceIndex, NewInstancesTransforms, bWorldSpace, bMarkRenderStateDirty, bTeleport);
	}
	UE_LOGF(LogChaos, Warning, "%ls's ISMPoolComponent is trying to update instance with mesh group (%d) that not exists", GetOwner() ? *GetOwner()->GetName() : TEXT(""), MeshGroupId) ;
	return false;
}

bool UISMPoolComponent::BatchUpdateInstanceCustomData(FMeshGroupId MeshGroupId, int32 CustomFloatIndex, float CustomFloatValue)
{
	if (FISMPoolMeshGroup* MeshGroup = MeshGroups.Find(MeshGroupId))
	{
		MeshGroup->BatchUpdateInstanceCustomData(Pool, CustomFloatIndex, CustomFloatValue);
		return true;
	}
	UE_LOGF(LogChaos, Warning, "%ls's ISMPoolComponent is trying to update instance with mesh group (%d) that not exists", GetOwner() ? *GetOwner()->GetName() : TEXT(""), MeshGroupId) ;
	return false;
}

void UISMPoolComponent::PreallocateMeshInstance(const FISMPoolStaticMeshInstance& MeshInstance)
{
	Pool.RequestPreallocateMeshInstance(MeshInstance);
	SetComponentTickEnabled(Pool.RequiresTick());
}

void UISMPoolComponent::SetTickablePoolManagement(bool bEnablePoolManagement)
{
	bIsPoolManagementEnabled = bEnablePoolManagement;
	if (!bEnablePoolManagement)
	{
		// Disable the keep alive and recycle pool management systems.
		// This also disables preallocation for this pool.
		Pool.bCachedKeepAlive = false;
		Pool.bCachedRecycle = false;
	}
	// Disable the Tick that is used to manage the pool.
	PrimaryComponentTick.SetTickFunctionEnable(bEnablePoolManagement);
}

void UISMPoolComponent::EnableComponentTickIfPoolManagementEnabled()
{
	if (bIsPoolManagementEnabled)
	{
		SetComponentTickEnabled(true);
	}
}

void UISMPoolComponent::SetOverrideTransformUpdates(bool bOverrideUpdates)
{
	Pool.bDisableBoundsAndTransformUpdate = bOverrideUpdates;
}

void UISMPoolComponent::UpdateAbsoluteTransforms(const FTransform& BaseTransform, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport)
{
	Pool.UpdateAbsoluteTransforms(BaseTransform, UpdateTransformFlags, Teleport);
}

void UISMPoolComponent::GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize)
{
	Super::GetResourceSizeEx(CumulativeResourceSize);

	int32 SizeBytes =
		MeshGroups.GetAllocatedSize()
		+ Pool.MeshToISMIndex.GetAllocatedSize()
		+ Pool.ISMs.GetAllocatedSize()
		+ Pool.FreeList.GetAllocatedSize()
		+ Pool.FreeListISM.GetAllocatedSize();
	
	for (FISMPoolISM ISM : Pool.ISMs)
	{
		SizeBytes += ISM.InstanceIds.GetAllocatedSize()
			+ ISM.InstanceGroups.GroupRanges.GetAllocatedSize()
			+ ISM.InstanceGroups.FreeList.GetAllocatedSize();
	}

	CumulativeResourceSize.AddDedicatedSystemMemoryBytes(SizeBytes);
}
