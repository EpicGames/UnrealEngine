// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISMPool/ISMPoolSubSystem.h"
#include "ISMPool/ISMPoolActor.h"
#include "Engine/World.h"

//----------------------------------------------------------------------//
// UISMPoolSubSystem
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(ISMPoolSubSystem)
UISMPoolSubSystem::UISMPoolSubSystem()
{
}

void UISMPoolSubSystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	Collection.InitializeDependency<UISMPoolSubSystem>();
}

void UISMPoolSubSystem::Deinitialize()
{
	PerLevelISMPoolActors.Reset();
	Super::Deinitialize();
}

AISMPoolActor* UISMPoolSubSystem::FindISMPoolActor(ULevel* Level)
{
	AISMPoolActor* ISMPoolActor = nullptr;

	// on demand creation of the actor based on level
	TObjectPtr<AISMPoolActor>* ISMPoolActorInLevel = PerLevelISMPoolActors.Find(Level);
	if (ISMPoolActorInLevel)
	{
		ISMPoolActor = ISMPoolActorInLevel->Get();
	}
	else
	{
		// we keep it as transient to avoid accumulation of those actors in saved levels
		FActorSpawnParameters Params;
		Params.ObjectFlags = EObjectFlags::RF_DuplicateTransient | EObjectFlags::RF_Transient;
		Params.OverrideLevel = Level;
		ISMPoolActor = GetWorld()->SpawnActor<AISMPoolActor>(Params);
		// spawn can still fail if we are in the middle or tearing down the world
		if (ISMPoolActor)
		{
			PerLevelISMPoolActors.Add(Level, ISMPoolActor);
			// make sure we capture when the actor get removed so we can update our internal structure accordingly 
			ISMPoolActor->OnEndPlay.AddDynamic(this, &UISMPoolSubSystem::OnActorEndPlay);
		}
	}
	return ISMPoolActor;
}

void UISMPoolSubSystem::OnActorEndPlay(AActor* InSource, EEndPlayReason::Type Reason)
{
	if (ULevel* ActorLevel = InSource->GetLevel())
	{
		PerLevelISMPoolActors.Remove(ActorLevel);
	}
}

void UISMPoolSubSystem::GetISMPoolActors(TArray<AISMPoolActor*>& OutActors) const
{
	for (const auto& MapEntry : PerLevelISMPoolActors)
	{
		if (MapEntry.Value != nullptr)
		{
			OutActors.Add(MapEntry.Value);
		}
	}
}
