// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GameFramework/Actor.h"
#include "ISMPoolActor.generated.h"

class UISMPoolComponent;
class UISMPoolDebugDrawComponent;

UCLASS(ConversionRoot, ComponentWrapperClass, MinimalAPI)
class AISMPoolActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(Category = ISMPoolActor, VisibleAnywhere, BlueprintReadOnly, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<class UISMPoolComponent> ISMPoolComp;

#if UE_ENABLE_DEBUG_DRAWING
	TObjectPtr<class UISMPoolDebugDrawComponent> ISMPoolDebugDrawComp;
#endif

public:
	/** Returns ISMPoolComp subobject **/
	UISMPoolComponent* GetISMPoolComp() const { return ISMPoolComp; }
};



