// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SceneComponent.h"
#include "Containers/Map.h"
#include "InstancedStaticMeshDelegates.h"
#include "Materials/MaterialInterface.h"
#include "InstanceDataTypes.h"

#include "ISMPoolComponent.generated.h"

class AActor;
class UISMPoolComponent;
class UInstancedStaticMeshComponent;

/** 
 * Structure containing a set of allocated instance ranges in an FISMPoolMeshISM which is the manager for a single ISM component.
 * The instance ranges don't change once allocated, and aren't the same as the actual render indices in the ISM.
 * The reason that we don't store the the actual ISM render indices is that ISM component is free to reorder its instances whenever it likes.
 */
struct FISMPoolInstanceGroups
{
	using FInstanceGroupId = int32;

	/** A single continuous range associated with an FInstanceGroupId. */
	struct FInstanceGroupRange
	{
		FInstanceGroupRange(int32 InStart, int32 InCount)
			: Start(InStart)
			, Count(InCount)
		{
		}

		int32 Start = 0;
		int32 Count = 0;
	};

	/** Reset all contents. */
	void Reset()
	{
		TotalInstanceCount = 0;
		TotalFreeInstanceCount = 0;
		GroupRanges.Empty();
		FreeList.Empty();
	}

	/** Returns true if no groups ranges are in use. */
	bool IsEmpty() const
	{
		return GroupRanges.Num() == FreeList.Num();
	}

	/** Returns the maximum instance index that we have allocated in a group. */
	int32 GetMaxInstanceIndex() const
	{
		return TotalInstanceCount;
	}

	/** Add a new group range. */
	FInstanceGroupId AddGroup(int32 Count)
	{
		// First check to see if we can recycle a group from the free list.
		for (int32 Index = 0; Index < FreeList.Num(); ++Index)
		{
			const FInstanceGroupId GroupId = FreeList[Index];
			// todo: Could also allow allocating a subrange if Count is less than the group range count.
			if (Count == GroupRanges[GroupId].Count)
			{
				TotalFreeInstanceCount -= Count;
				FreeList.RemoveAtSwap(Index, EAllowShrinking::No);
				return GroupId;
			}
		}
		
		// Create a new group.
		const int32 StartIndex = TotalInstanceCount;
		TotalInstanceCount += Count;
		const FInstanceGroupId GroupId = GroupRanges.Add(FInstanceGroupRange(StartIndex, Count));
		return GroupId;
	}

	/** Remove a group range. */
	void RemoveGroup(FInstanceGroupId GroupId)
	{
		// Remove the group by putting on free list for reuse.
		// Actually removing it would require too much shuffling of the render instance index remapping.
		TotalFreeInstanceCount += GroupRanges[GroupId].Count;
		FreeList.Add(GroupId);
	}

	int32 TotalInstanceCount = 0;
	int32 TotalFreeInstanceCount = 0;
	TArray<FInstanceGroupRange> GroupRanges;
	TArray<FInstanceGroupId> FreeList;
};

/** 
 * A description for an ISM component.
 */
struct FISMPoolComponentDescription
{
	enum EFlags
	{
		UseHISM = 1 << 1,							// HISM is no longer supported. This flag is ignored.
		GpuLodSelection = 1 << 2,
		ReverseCulling = 1 << 3,
		StaticMobility = 1 << 4,
		WorldPositionOffsetWritesVelocity = 1 << 5,
		EvaluateWorldPositionOffset = 1 << 6,
		AffectShadow = 1 << 7,
		AffectDistanceFieldLighting = 1 << 8,
		AffectDynamicIndirectLighting = 1 << 9,
		AffectFarShadow = 1 << 10,
		DistanceCullPrimitive = 1 << 11,
		UsesCustomDepthStencil = 1 << 12,
	};

	uint32 Flags = WorldPositionOffsetWritesVelocity|EvaluateWorldPositionOffset|AffectShadow;
	int32 NumCustomDataFloats = 0;
	FVector Position = FVector::ZeroVector;
	int32 StartCullDistance = 0;
	int32 EndCullDistance = 0;
	int32 MinLod = 0;
	int32 CustomDepthStencilValue = 0;
	uint32 GroupHash = 0;	// Optional, allows identical SMs to be separated into different groups for finer grained culling
	float LodScale = 1.f;
	TArray<FName> Tags;
	FName StatsCategory;

	bool operator==(const FISMPoolComponentDescription& Other) const
	{
		return Flags == Other.Flags &&
			NumCustomDataFloats == Other.NumCustomDataFloats &&
			Position == Other.Position &&
			StartCullDistance == Other.StartCullDistance &&
			EndCullDistance == Other.EndCullDistance &&
			MinLod == Other.MinLod &&
			CustomDepthStencilValue == Other.CustomDepthStencilValue &&
			LodScale == Other.LodScale &&
			Tags == Other.Tags &&
			GroupHash == Other.GroupHash &&
			StatsCategory == Other.StatsCategory;
	}
};

inline uint32 GetTypeHash(const FISMPoolComponentDescription& Desc)
{
	uint32 Hash = HashCombineFast(GetTypeHash(Desc.Flags), GetTypeHash(Desc.NumCustomDataFloats));
	Hash = HashCombineFast(Hash, GetTypeHash(Desc.Position));
	Hash = HashCombineFast(Hash, GetTypeHash(Desc.StartCullDistance));
	Hash = HashCombineFast(Hash, GetTypeHash(Desc.EndCullDistance));
	Hash = HashCombineFast(Hash, GetTypeHash(Desc.MinLod));
	Hash = HashCombineFast(Hash, GetTypeHash(Desc.CustomDepthStencilValue));
	Hash = HashCombineFast(Hash, GetTypeHash(Desc.LodScale));
	Hash = HashCombineFast(Hash, GetTypeHash(Desc.GroupHash));
	Hash = HashCombineFast(Hash, GetArrayHash(Desc.Tags.GetData(), Desc.Tags.Num()));
	return HashCombineFast(Hash, GetTypeHash(Desc.StatsCategory));
}

/**
 * A mesh with potentially overriden materials and ISM property description.
 * We batch instances into ISMs that have equivalent values for this structure.
 */
struct FISMPoolStaticMeshInstance
{
	TWeakObjectPtr<UStaticMesh> StaticMesh;
	TArray<TWeakObjectPtr<UMaterialInterface>> MaterialsOverrides;
	TArray<float> CustomPrimitiveData;
	FISMPoolComponentDescription Desc;

	bool operator==(const FISMPoolStaticMeshInstance& Other) const 
	{
		if (!(Desc == Other.Desc))
		{
			return false;
		}
		if (!StaticMesh.HasSameIndexAndSerialNumber(Other.StaticMesh))
		{
			return false;
		}
		if (MaterialsOverrides.Num() != Other.MaterialsOverrides.Num())
		{
			return false;
		}
		for (int32 MatIndex = 0; MatIndex < MaterialsOverrides.Num(); MatIndex++)
		{
			if (!MaterialsOverrides[MatIndex].HasSameIndexAndSerialNumber(Other.MaterialsOverrides[MatIndex]))
			{
				return false;
			}
		}
		if (CustomPrimitiveData.Num() != Other.CustomPrimitiveData.Num())
		{
			return false;
		}
		for (int32 DataIndex = 0; DataIndex < CustomPrimitiveData.Num(); DataIndex++)
		{
			if (CustomPrimitiveData[DataIndex] != Other.CustomPrimitiveData[DataIndex])
			{
				return false;
			}
		}

		return true;
	}
};

inline uint32 GetTypeHash(const FISMPoolStaticMeshInstance& MeshInstance)
{
	uint32 CombinedHash = GetTypeHash(MeshInstance.StaticMesh);
	CombinedHash = HashCombineFast(CombinedHash, GetTypeHash(MeshInstance.MaterialsOverrides.Num()));
	for (const TWeakObjectPtr<UMaterialInterface> Material: MeshInstance.MaterialsOverrides)
	{
		CombinedHash = HashCombineFast(CombinedHash, GetTypeHash(Material));
	}
	for (const float CustomFloat : MeshInstance.CustomPrimitiveData)
	{
		CombinedHash = HashCombineFast(CombinedHash, GetTypeHash(CustomFloat));
	}
	CombinedHash = HashCombineFast(CombinedHash, GetTypeHash(MeshInstance.Desc));
	return CombinedHash;
}

/** Describes a group of instances within an ISM. */
struct FISMPoolMeshInfo
{
	int32 ISMIndex;
	FISMPoolInstanceGroups::FInstanceGroupId InstanceGroupIndex;

	TArray<float> CustomData;
	TArrayView<const float> CustomDataSlice(int32 InstanceIndex, int32 NumCustomDataFloatsPerInstance);
	void ShadowCopyCustomData(int32 InstanceCount, int32 NumCustomDataFloatsPerInstance, TArrayView<const float> CustomDataFloats);
};

struct FISMPool;

/**
 * A mesh group which is a collection of meshes and their related FISMPoolMeshInfo.
 * We group these with a single handle with the expectation that a client will want to own multiple meshs and release them together.
 */
struct FISMPoolMeshGroup
{
	using FMeshId = int32;

	/** Adds a new mesh with instance count. We expect to only add a unique mesh instance once to each group. Returns a ID that can be used to update the instances. */
	FMeshId AddMesh(const FISMPoolStaticMeshInstance& MeshInstance, int32 InstanceCount, const FISMPoolMeshInfo& ISMInstanceInfo, TArrayView<const float> CustomDataFloats);
	/** Update instance transforms for a group of instances. */
	bool BatchUpdateInstancesTransforms(FISMPool& ISMPool, FMeshId MeshId, int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport);
	void BatchUpdateInstanceCustomData(FISMPool& ISMPool, int32 CustomFloatIndex, float CustomFloatValue);

	/** Remove all of our managed meshes and associated instances. */
	void RemoveAllMeshes(FISMPool& ISMPool);

	/** Array of allocated mesh infos. */
	TArray<FISMPoolMeshInfo> MeshInfos;

	/** Flag for whether we allow removal of instances when transform scale is set to zero. */
	bool bAllowPerInstanceRemoval = false;
};

/** Structure containting all info for a single ISM. */
struct FISMPoolISM
{
	/** Create the ISMComponent according to settings on the mesh instance. */
	void CreateISM(USceneComponent* InOwningComponent);
	/** Initialize the ISMComponent according to settings on the mesh instance. */
	void InitISM(const FISMPoolStaticMeshInstance& InMeshInstance, bool bKeepAlive, bool bOverrideTransformUpdates = false);
	/** Add a group to the ISM. Returns the group index. */
	FISMPoolInstanceGroups::FInstanceGroupId AddInstanceGroup(int32 InstanceCount, TArrayView<const float> CustomDataFloats);

	/** Unique description of ISM component settings. */
	FISMPoolStaticMeshInstance MeshInstance;
	/** Created ISM component. Will be nullptr when this slot has been recycled to FISMPool FreeList. */
	TObjectPtr<UInstancedStaticMeshComponent> ISMComponent;
	/** Groups of instances allocated in the ISM. */
	FISMPoolInstanceGroups InstanceGroups;
	/** Id of Instance in ISMC */
	TArray<FPrimitiveInstanceId> InstanceIds;
};


/** A pool of ISMs. */
struct FISMPool
{
	using FISMIndex = int32;

	FISMPool();

	/** Find or add an ISM and return an ISM index handle. */
	FISMIndex GetOrAddISM(UISMPoolComponent* OwningComponent, const FISMPoolStaticMeshInstance& MeshInstance, bool& bOutISMCreated);
	/** Remove an ISM. */
	void RemoveISM(FISMIndex ISMIndex, bool bKeepAlive, bool bRecycle);
	/** Add instances to ISM and return a mesh info handle. */
	FISMPoolMeshInfo AddInstancesToISM(UISMPoolComponent* OwningComponent, const FISMPoolStaticMeshInstance& MeshInstance, int32 InstanceCount, TArrayView<const float> CustomDataFloats);
	/** Remove instances from an ISM. */
	void RemoveInstancesFromISM(const FISMPoolMeshInfo& MeshInfo);
	/** Update ISM contents. */
	bool BatchUpdateInstancesTransforms(FISMPoolMeshInfo& MeshInfo, int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport, bool bAllowPerInstanceRemoval);
	void BatchUpdateInstanceCustomData(FISMPoolMeshInfo const& MeshInfo, int32 CustomFloatIndex, float CustomFloatValue);

	/** Clear all ISM components and associated data. */
	void Clear();

	/** Tick maintenance of free list and preallocation. */
	void Tick(UISMPoolComponent* OwningComponent);

	/** Add an ISM description to the preallocation queue. */
	void RequestPreallocateMeshInstance(const FISMPoolStaticMeshInstance& MeshInstances);
	/** Process the preallocation queue. Processing is timesliced so that only some of the queue will be processed in every call. */
	void ProcessPreallocationRequests(UISMPoolComponent* OwningComponent, int32 MaxPreallocations);

	void UpdateAbsoluteTransforms(const FTransform& BaseTransform, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport);

	bool RequiresTick() const;

	/** Array of ISM objects. */
	TArray<FISMPoolISM> ISMs;
	/** Mapping from mesh description to ISMs array slot. */
	TMap<FISMPoolStaticMeshInstance, FISMIndex> MeshToISMIndex;
	
	/** Set of ISM descriptions that we would like to preallocate. */
	TSet<FISMPoolStaticMeshInstance> PrellocationQueue;

	/** Free list of indices in ISMs that are empty. */
	TArray<int32> FreeList;
	/** Free list of indices in ISMs that have registered ISM components. */
	TArray<int32> FreeListISM;

	// Cached state of lifecycle cvars from the last Tick()
	bool bCachedKeepAlive = false;
	bool bCachedRecycle = false;

	// Whether we force ISMs to use parent bounds and disable transform updates
	bool bDisableBoundsAndTransformUpdate = false;
};


/**
* UISMPoolComponent.
* Component that manages a pool of ISM components in order to allow multiple client components that use the same meshes to the share ISMs.
*/
UCLASS(meta = (BlueprintSpawnableComponent), MinimalAPI)
class UISMPoolComponent: public USceneComponent
{
	GENERATED_UCLASS_BODY()

public:
	using FMeshGroupId = int32;
	using FMeshId = int32;

	//~ Begin UActorComponent Interface
	ISMPOOL_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	ISMPOOL_API virtual void GetResourceSizeEx(FResourceSizeEx& CumulativeResourceSize) override;
	//~ End UActorComponent Interface

	/** 
	* Create an Mesh group which represent an arbitrary set of mesh with their instance 
	* no resources are created until the meshes are added for this group 
	* return a mesh group Id used to add and update instances
	*/
	ISMPOOL_API FMeshGroupId CreateMeshGroup(bool bAllowPerInstanceRemoval = false);

	/** Destroy  a mesh group and its associated resources */
	ISMPOOL_API void DestroyMeshGroup(FMeshGroupId MeshGroupId);

	/** Add a static mesh for a mesh group */
	ISMPOOL_API FMeshId AddMeshToGroup(FMeshGroupId MeshGroupId, const FISMPoolStaticMeshInstance& MeshInstance, int32 InstanceCount, TArrayView<const float> CustomDataFloats);

	/** Update transforms for a mesh group */
	ISMPOOL_API bool BatchUpdateInstancesTransforms(FMeshGroupId MeshGroupId, FMeshId MeshId, int32 StartInstanceIndex, TArrayView<const FTransform> NewInstancesTransforms, bool bWorldSpace = false, bool bMarkRenderStateDirty = false, bool bTeleport = false);

	UE_DEPRECATED(5.3, "BatchUpdateInstancesTransforms Array parameter version is deprecated, use the TArrayView version instead")
	ISMPOOL_API bool BatchUpdateInstancesTransforms(FMeshGroupId MeshGroupId, FMeshId MeshId, int32 StartInstanceIndex, const TArray<FTransform>& NewInstancesTransforms, bool bWorldSpace = false, bool bMarkRenderStateDirty = false, bool bTeleport = false);

	/** Update a single slot of custom instance data for all instances in a mesh group */
	ISMPOOL_API bool BatchUpdateInstanceCustomData(FMeshGroupId MeshGroupId, int32 CustomFloatIndex, float CustomFloatValue);

	/** 
	 * Preallocate an ISM in the pool. 
	 * Doing this early for known mesh instance descriptions can reduce the component registration cost of AddMeshToGroup() for newly discovered mesh descriptions.
	 */
	ISMPOOL_API void PreallocateMeshInstance(const FISMPoolStaticMeshInstance& MeshInstance);

	ISMPOOL_API void SetTickablePoolManagement(bool bEnablePoolManagement);

	ISMPOOL_API void SetOverrideTransformUpdates(bool bOverrideUpdates);

	ISMPOOL_API void UpdateAbsoluteTransforms(const FTransform& BaseTransform, EUpdateTransformFlags UpdateTransformFlags, ETeleportType Teleport);

	void EnableComponentTickIfPoolManagementEnabled();

private:
	uint32 NextMeshGroupId = 0;
	TMap<FMeshGroupId, FISMPoolMeshGroup> MeshGroups;
	FISMPool Pool;
	bool bIsPoolManagementEnabled = true;

	// Expose internals for debug draw support.
	friend class UISMPoolDebugDrawComponent;
};
