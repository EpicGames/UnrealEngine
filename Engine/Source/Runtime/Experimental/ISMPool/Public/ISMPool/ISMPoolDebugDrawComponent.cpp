// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISMPool/ISMPoolDebugDrawComponent.h"

#include "CollisionQueryParams.h"
#include "ComponentRecreateRenderStateContext.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "GameFramework/Pawn.h"
#include "GameFramework/PlayerController.h"
#include "ISMPool/ISMPoolActor.h"
#include "ISMPool/ISMPoolComponent.h"
#include "Misc/CoreDelegates.h"

#if WITH_EDITOR
#include "Editor.h"
#include "EditorViewportClient.h"
#include "LevelEditorViewport.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ISMPoolDebugDrawComponent)

static TAutoConsoleVariable<int32> CVarISMPoolStats(
	TEXT("r.ISMPool.DebugStats"),
	0,
	TEXT("Show stats for the ISM pools"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable) { UISMPoolDebugDrawComponent::UpdateAllTickEnabled(); }),
	ECVF_Default);

static TAutoConsoleVariable<int32> CVarISMPoolDebugDraw(
	TEXT("r.ISMPool.DebugDraw"),
	0,
	TEXT("Show debug drawing for the ISM pools"),
	FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* InVariable) { UISMPoolDebugDrawComponent::UpdateAllTickEnabled(); FGlobalComponentRecreateRenderStateContext Context; }),
	ECVF_Default);


class FISMPoolDebugDrawSceneProxy final : public FDebugRenderSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FISMPoolDebugDrawSceneProxy(UISMPoolDebugDrawComponent const* InComponent)
		: FDebugRenderSceneProxy(InComponent)
	{
		DrawType = FDebugRenderSceneProxy::SolidAndWireMeshes;
	}

	FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = true;
		Result.bDynamicRelevance = true;
		Result.bSeparateTranslucency = Result.bNormalTranslucency = true;
		return Result;
	}
};


UISMPoolDebugDrawComponent::UISMPoolDebugDrawComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bIsEditorOnly = false;
	bSelectable = false;

	PrimaryComponentTick.bCanEverTick = true;
	PrimaryComponentTick.bStartWithTickEnabled = false;
	PrimaryComponentTick.bAllowTickOnDedicatedServer = false;
}

void UISMPoolDebugDrawComponent::UpdateTickEnabled()
{
	const bool bForceShowStats = CVarISMPoolStats.GetValueOnAnyThread() != 0;
	const bool bForceShowBounds = CVarISMPoolDebugDraw.GetValueOnAnyThread() != 0;
	PrimaryComponentTick.SetTickFunctionEnable(bShowStats || bForceShowStats || bShowBounds || bForceShowBounds);
}

void UISMPoolDebugDrawComponent::UpdateAllTickEnabled()
{
	for (TObjectIterator<UISMPoolDebugDrawComponent> It; It; ++It)
	{
		It->UpdateTickEnabled();
	}
}

void UISMPoolDebugDrawComponent::BeginPlay()
{
	Super::BeginPlay();
#if UE_ENABLE_DEBUG_DRAWING
	OnScreenMessagesHandle = FCoreDelegates::OnGetOnScreenMessages.AddUObject(this, &UISMPoolDebugDrawComponent::GetOnScreenMessages);
#endif
	UpdateTickEnabled();
}

void UISMPoolDebugDrawComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	Super::EndPlay(EndPlayReason);
#if UE_ENABLE_DEBUG_DRAWING
	FCoreDelegates::OnGetOnScreenMessages.Remove(OnScreenMessagesHandle);
#endif
}

FBoxSphereBounds UISMPoolDebugDrawComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return SelectedComponent ? SelectedComponent->CalcBounds(LocalToWorld) : FBox(ForceInitToZero);
}

#if WITH_EDITOR
void UISMPoolDebugDrawComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UISMPoolDebugDrawComponent, bShowStats) ||
			PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UISMPoolDebugDrawComponent, bShowBounds))
		{
			UpdateTickEnabled();
		}
	}
}
#endif

void UISMPoolDebugDrawComponent::TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction)
{
	Super::TickComponent(DeltaTime, TickType, ThisTickFunction);

#if UE_ENABLE_DEBUG_DRAWING
	UInstancedStaticMeshComponent const* FoundISM = nullptr;

	const bool bForceShowStats = CVarISMPoolStats.GetValueOnAnyThread() != 0;
	const bool bForceShowBounds = CVarISMPoolDebugDraw.GetValueOnAnyThread() != 0;
	if (bShowStats || bForceShowStats || bShowBounds || bForceShowBounds)
	{
#if WITH_EDITOR
		if (GCurrentLevelEditingViewportClient)
		{
			FViewport* Viewport = GCurrentLevelEditingViewportClient->Viewport;
			HHitProxy* HitResult = Viewport->GetHitProxy(Viewport->GetMouseX(), Viewport->GetMouseY());
			if (HActor* HitActor = HitProxyCast<HActor>(HitResult))
			{
				FoundISM = Cast<UInstancedStaticMeshComponent>(HitActor->PrimComponent);
			}
		} 
#endif
		if (FoundISM == nullptr)
		{
			if (UWorld* World = GetWorld())
			{
				APlayerController const* Controller = nullptr;
				for (FConstPlayerControllerIterator Iterator = World->GetPlayerControllerIterator(); Iterator; ++Iterator)
				{
					APlayerController* PC = Iterator->Get();
					if (PC && PC->IsLocalController())
					{
						Controller = PC;
					}
				}

				if (Controller != nullptr)
				{
					FVector CamLoc;
					FRotator CamRot;
					Controller->GetPlayerViewPoint(CamLoc, CamRot);
					const FVector CamForward = CamRot.Vector();
					const FVector TraceStart = CamLoc;
					const FVector TraceEnd = TraceStart + CamForward * 20000;

					FHitResult HitResult(ForceInit);
					FCollisionQueryParams TraceParams(SCENE_QUERY_STAT(ChaosDebugVisibilityTrace), true, Controller->GetPawn());
					const bool bHit = World->LineTraceSingleByChannel(HitResult, TraceStart, TraceEnd, ECC_Visibility, TraceParams);
					if (bHit && HitResult.GetComponent() != nullptr)
					{
						if (UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(HitResult.GetComponent()))
						{
							FoundISM = ISMComponent;
						}
					}
				}
			}
		}
	}

	if (FoundISM != nullptr && FoundISM->GetOuter() != GetOuter())
	{
		FoundISM = nullptr;
	}

	if (SelectedComponent != FoundISM)
	{
		bool bChangeSelection = (SelectedComponent == nullptr) || SelectTimer < 0.f;
		if (bChangeSelection)
		{
#if WITH_EDITOR
			if (SelectedComponent)
			{
				((UInstancedStaticMeshComponent*)SelectedComponent)->PushHoveredToProxy(false);
			}
#endif
			SelectedComponent = FoundISM;
#if WITH_EDITOR
			if (SelectedComponent)
			{
				((UInstancedStaticMeshComponent*)SelectedComponent)->PushHoveredToProxy(true);
			}
#endif
			SelectTimer = 0.3f;
			MarkRenderStateDirty();
		}
		else
		{
			SelectTimer -= DeltaTime;
		}
	}
#endif // UE_ENABLE_DEBUG_DRAWING
}

#if UE_ENABLE_DEBUG_DRAWING

static void DrawBoxAndChildren(FDebugRenderSceneProxy* DebugProxy, FTransform const& LocalToWorldTransform, TArray<FClusterNode> const& Nodes, int32 NodeIndex, int32 LevelIndex)
{
	FClusterNode const& Node = Nodes[NodeIndex];

	const FBox LocalBounds(FBox(Node.BoundMin, Node.BoundMax));
	
	static TArray<FColor> Colors = { FColorList::VeryLightGrey, FColorList::CoolCopper, FColorList::GreenYellow, FColorList::CornFlowerBlue, FColorList::DustyRose, FColorList::Red, FColorList::Magenta };
	const FColor LevelColor = Colors[FMath::Min(LevelIndex, Colors.Num() - 1)];
	
	DebugProxy->Boxes.Add(FDebugRenderSceneProxy::FDebugBox(LocalBounds, LevelColor, LocalToWorldTransform));

	if (Node.FirstChild > 0)
	{
		for (int32 ChildIndex = Node.FirstChild; ChildIndex <= Node.LastChild; ChildIndex++)
		{
			DrawBoxAndChildren(DebugProxy, LocalToWorldTransform, Nodes, ChildIndex, LevelIndex + 1);
		}
	}
}

FDebugRenderSceneProxy* UISMPoolDebugDrawComponent::CreateDebugSceneProxy()
{
	const bool bForceShowBounds = CVarISMPoolDebugDraw.GetValueOnAnyThread() != 0;
	if (!(SelectedComponent && (bShowBounds || bForceShowBounds)))
	{
		return nullptr;
	}

	FDebugRenderSceneProxy* DebugProxy = new FISMPoolDebugDrawSceneProxy(this);

	if (const UHierarchicalInstancedStaticMeshComponent* HISMComponent = Cast<UHierarchicalInstancedStaticMeshComponent>(SelectedComponent))
	{
		const FTransform LocalToWorldTransform = HISMComponent->GetComponentTransform();
		TArray<FClusterNode> TreeNodes;
		HISMComponent->GetTree(TreeNodes);
		DrawBoxAndChildren(DebugProxy, LocalToWorldTransform, TreeNodes, 0, 0);
	}
	else if(const UInstancedStaticMeshComponent* ISMComponent = Cast<UInstancedStaticMeshComponent>(SelectedComponent))
	{
		const FBox LocalBounds = ISMComponent->CalcBounds(FTransform::Identity).GetBox();
		const FTransform LocalToWorldTransform = ISMComponent->GetComponentTransform();
		DebugProxy->Boxes.Add(FDebugRenderSceneProxy::FDebugBox(LocalBounds, FColorList::CornFlowerBlue, LocalToWorldTransform));
	}

	return DebugProxy;
}

void UISMPoolDebugDrawComponent::GetOnScreenMessages(TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText>& OutMessages)
{
	const bool bForceShowStats = CVarISMPoolStats.GetValueOnAnyThread() != 0;
	if (!bShowStats && !bShowGlobalStats && !bForceShowStats)
	{
		return;
	}

	if (bShowGlobalStats || bForceShowStats)
	{
		AISMPoolActor const* ISMPoolActor = Cast<AISMPoolActor>(GetOuter());
		UISMPoolComponent const* ISMPoolComponent = ISMPoolActor ? ISMPoolActor->GetISMPoolComp() : nullptr;

		if (ISMPoolComponent)
		{
			int32 NumISMs = 0;
			int32 NumHISMs = 0;
			int32 NumInstancesTotal = 0;
			TArray<int32> NumInstancesPerISM;
			NumInstancesPerISM.Reserve(ISMPoolComponent->Pool.ISMs.Num());

			for (FISMPoolISM const& ISM : ISMPoolComponent->Pool.ISMs)
			{
				if (ISM.ISMComponent != nullptr)
				{
					const bool bIsHISM = ISM.ISMComponent->IsA(UHierarchicalInstancedStaticMeshComponent::StaticClass());
					NumISMs += bIsHISM ? 0 : 1;
					NumHISMs += bIsHISM ? 1 : 0;

					const int32 NumInstances = ISM.ISMComponent->GetNumRenderInstances();
					NumInstancesTotal += NumInstances;
					NumInstancesPerISM.Add(NumInstances);
				}
			}

			if (NumInstancesTotal > 0)
			{
				OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(*GetOwner()->GetPathName()));
				OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(FString::Printf(TEXT("Num ISMS %d"), NumISMs)));
				OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(FString::Printf(TEXT("Num HISMS %d"), NumHISMs)));
				OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(FString::Printf(TEXT("Num Instances %d"), NumInstancesTotal)));
				OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(FString::Printf(TEXT("Average Instances Per ISM %f"), (float)NumInstancesTotal / (float)(NumISMs + NumHISMs))));

				NumInstancesPerISM.Sort();
				for (int32 Index = 0; Index < 10; ++Index)
				{
					const int32 Percentile = (Index + 1) * 10;
					const int32 PercentileIndex = FMath::Min(NumInstancesPerISM.Num() * (Index + 1) / 10, NumInstancesPerISM.Num() - 1);
					OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(FString::Printf(TEXT("Num Instances at %d percentile: %d"), Percentile, NumInstancesPerISM[PercentileIndex])));
				}
			}
		}
	}

	if ((bShowStats || bForceShowStats) && SelectedComponent)
	{
		TCHAR const* ISMType = SelectedComponent->IsA(UHierarchicalInstancedStaticMeshComponent::StaticClass()) ? TEXT("HISM") : TEXT("ISM");

		FVector BoundsMin, BoundsMax;
		SelectedComponent->GetLocalBounds(BoundsMin, BoundsMax);
		const FVector BoundsSize = BoundsMax - BoundsMin;

		const int32 NumInstances = SelectedComponent->GetNumRenderInstances();
		const int32 NumCustomDataFloats = SelectedComponent->NumCustomDataFloats;
		const int32 StartCullDistance = SelectedComponent->InstanceStartCullDistance;
		const int32 EndCullDistance = SelectedComponent->InstanceEndCullDistance;
		const int32 NumMaterials = SelectedComponent->GetNumMaterials();

		UStaticMesh const* StaticMesh = SelectedComponent->GetStaticMesh();
		const int32 NumLods = StaticMesh ? StaticMesh->GetNumLODs() : 0;

		FString ISMDescription = FString::Printf(TEXT("Type=%s   Count=%d   Bounds=(%f., %f., %f.)   CullDistance=(%d - %d)   NumLods= %d    NumMaterials=%d   CustomDataSize=%d"), 
			ISMType, NumInstances, BoundsSize.X, BoundsSize.Z, BoundsSize.Z, StartCullDistance, EndCullDistance, NumLods, NumMaterials, NumCustomDataFloats);

		for (int32 LodIndex = 0; LodIndex < NumLods; ++LodIndex)
		{
			const int32 NumVertices = StaticMesh->GetNumVertices(LodIndex);
			const int32 NumTriangles = StaticMesh->GetNumTriangles(LodIndex);
			ISMDescription += FString::Printf(TEXT("\nLod %d: Vertices=%d, Triangles=%d."), LodIndex, NumVertices, NumTriangles);
		}

		OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(*GetOwner()->GetPathName()));
		OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Warning, FText::FromString(*SelectedComponent->GetName()));
		OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromString(ISMDescription));
	}
}

#endif
