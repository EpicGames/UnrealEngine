// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debug/DebugDrawComponent.h"
#include "DebugRenderSceneProxy.h"
#include "ISMPoolDebugDrawComponent.generated.h"

#define UE_API ISMPOOL_API

class FISMPoolDebugDrawDelegateHelper;
class UInstancedStaticMeshComponent;

UCLASS(MinimalAPI, ClassGroup = Debug)
class UISMPoolDebugDrawComponent : public UDebugDrawComponent
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category = Debug)
	bool bShowGlobalStats = false;

	UPROPERTY(EditAnywhere, Category = Debug)
	bool bShowStats = false;

	UPROPERTY(EditAnywhere, Category = Debug)
	bool bShowBounds = false;

	UPROPERTY(Transient)
	TObjectPtr<const UInstancedStaticMeshComponent> SelectedComponent;

	float SelectTimer = 0.f;

protected:
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual void TickComponent(float DeltaTime, enum ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UE_API void UpdateTickEnabled();

#if UE_ENABLE_DEBUG_DRAWING
  	UE_API virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
	
	FDebugDrawDelegateHelper DebugDrawDelegateHelper;
	virtual FDebugDrawDelegateHelper& GetDebugDrawDelegateHelper() override { return DebugDrawDelegateHelper; }

	FDelegateHandle OnScreenMessagesHandle;
	UE_API void GetOnScreenMessages(TMultiMap<FCoreDelegates::EOnScreenMessageSeverity, FText>& OutMessages);
#endif

public:
	static UE_API void UpdateAllTickEnabled();
};

#undef UE_API
