// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"
#include "ISMPoolSubSystem.generated.h"

class AISMPoolActor;
class ULevel;
/**
 * A subsystem managing ISMPool actors.
 */
UCLASS(MinimalAPI)
class UISMPoolSubSystem : public UWorldSubsystem
{
	GENERATED_BODY()

public:
	ISMPOOL_API UISMPoolSubSystem();

	// USubsystem BEGIN
	ISMPOOL_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	ISMPOOL_API virtual void Deinitialize() override;
	// USubsystem END

	/** Finds or creates an actor. */
	ISMPOOL_API AISMPoolActor* FindISMPoolActor(ULevel* Level);
	
	/** Get all actors managed by the subsystem. */
	ISMPOOL_API void GetISMPoolActors(TArray<AISMPoolActor*>& OutActors) const;

protected:
	UFUNCTION()
	void OnActorEndPlay(AActor* InSource, EEndPlayReason::Type Reason);

	/** ISMPool are per level **/
	TMap<TObjectPtr<ULevel>, TObjectPtr<AISMPoolActor> > PerLevelISMPoolActors;
};
