// Copyright Epic Games, Inc. All Rights Reserved.
#include "IndexedCacheStorage.h"

namespace Experimental
{

/**
 * Generic implementation of IIndexedCacheStorage
 */
class FGenericIndexedCacheStorage : public Experimental::IIndexedCacheStorage
{
public:
	virtual bool SupportsIndexedCacheStorage() override
	{
		return false;
	}

	virtual uint32 GetNumIndexedCacheStorages() override
	{
		return 0;
	}

	virtual bool CreateCacheStorage(uint64 InRequestNumberOfBytes, int32 CacheStorageIndex) override
	{
		return false;
	}

	virtual bool DestroyCacheStorage(int32 InCacheStorageIndex) override
	{
		return false;
	}

	virtual void EnumerateCacheStorages(TArray<int32>& OutCacheStorageIndices) override
	{
	}
	
	virtual void GetCacheStorageInfo(int32 InCacheStorageIndex, uint64& OutCacheStorageDataSize, uint64& OutCacheStorageJournalSize) override
	{
	}
	
	virtual void GetCacheStorageMountPath(FString& OutPlatformMountPath, const FString& InMountName) override
	{
	}

	virtual bool MountCacheStorage(FString& OutPlatformMountPath, int32 InCacheStorageIndex, const FString& InMountName) override
	{
		return false;
	}

	virtual void UnmountCacheStorage(const FString& InMountName) override
	{
	}

	virtual uint64 GetPersistentDownloadDirEarlyStartupSize() override
	{
		return 0;
	}

	virtual uint64 GetCacheStorageJournalSize(uint64 CacheDataSize, uint32 CacheIndex) override
	{
		return 0;
	}

};

#if !defined(HAS_CUSTOM_INDEXED_CACHE_STORAGE)
#	define HAS_CUSTOM_INDEXED_CACHE_STORAGE 0
#endif

#if (HAS_CUSTOM_INDEXED_CACHE_STORAGE == 0)
IIndexedCacheStorage& IIndexedCacheStorage::GetOrCreateIndexedCacheStorage()
{
	static FGenericIndexedCacheStorage Instance;
	return Instance;
}
#endif

IIndexedCacheStorage& IIndexedCacheStorage::Get()
{
	return IIndexedCacheStorage::GetOrCreateIndexedCacheStorage();
}

}


