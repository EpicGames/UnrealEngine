// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"

namespace Experimental
{

class IIndexedCacheStorage;

/**
  * Interface for managing cache storages: some platforms may store user data in different cache storages that may be created / destroyed
  * This allows the title to manage its disk space with a finer granularity. 
  * The data put in these caches need to be re-generated/re-obtained if needed.
  */
class IIndexedCacheStorage
{
public:
	static IIndexedCacheStorage& Get();

	static IIndexedCacheStorage& GetOrCreateIndexedCacheStorage();

	/**
	  * Check if indexed cache storage is available on the current platform.
	  * 
	  * @return True is available, false otherwise.
	  */
	virtual bool SupportsIndexedCacheStorage() = 0;

	/**
	  * Retrieve the number of indexed cache entries available on the platform.
	  * 
	  * @return number of indexed cache entries available on the platform.
	  */
	virtual uint32 GetNumIndexedCacheStorages() = 0;

	/**
	  * Requests a new indexed cache entry, capable of holding up to RequestNumberOfBytes.
	  * 
 	  * @param InRequestNumberOfBytes number of bytes this entry needs to hold
 	  * @param CacheStorageIndex - the index of the entry to create
	  * @return True on success, otherwise false
	  */
	virtual bool CreateCacheStorage(uint64 InRequestNumberOfBytes, int32 CacheStorageIndex) = 0;

	/**
	  * Destroy an indexed cache entry.
	  * 
 	  * @param InCacheStorageIndex - the index of the entry
	  * @return True on success, otherwise false
	  */
	virtual bool DestroyCacheStorage(int32 InCacheStorageIndex) = 0;

	/**
	  * Enumerate existing cache entries.
	  * 
 	  * @param OutCacheStorageIndices - indices of existing cache storages
	  */
	virtual void EnumerateCacheStorages(TArray<int32>& OutCacheStorageIndices) = 0;
	
	/**
	  * Queries cache storage.
	  * 
 	  * @param InCacheStorageIndex - the index of the cache
 	  * @param OutCacheStorageSize - How large the cache storage is
	  */
	virtual void GetCacheStorageInfo(int32 InCacheStorageIndex, uint64& OutCacheStorageDataSize, uint64& OutCacheStorageJournalSize) = 0;
	
	/**
	  * Gets the platform mount path for a cached storage based on its generic Name
	  * 
 	  * @param OutPlatformMountPath - the platform specific mount path
 	  * @param InMountName - the generic mount name
	  */
	virtual void GetCacheStorageMountPath(FString& OutPlatformMountPath, const FString& InMountName) = 0;

	/**
	  * Mounts a cache storage so file operations can be done
	  * 
 	  * @param OutPlatformMountPath - the platform specific mount path. Same name as the one returned by GetCacheStorageMountPath
 	  * @param InCacheStorageIndex - the index of the cache
 	  * @param InMountName - the generic mount name
	  */
	virtual bool MountCacheStorage(FString& OutPlatformMountPath, int32 InCacheStorageIndex, const FString& InMountName) = 0;

	/**
	  * Unmounts a cache storage
 	  * @param InMountName - the generic mount name
	  */
	virtual void UnmountCacheStorage(const FString& InMountName) = 0;

	virtual uint64 GetPersistentDownloadDirEarlyStartupSize() = 0;

	virtual uint64 GetCacheStorageJournalSize(uint64 CacheDataSize, uint32 CacheIndex) = 0;

};

}
