// Copyright Epic Games, Inc. All Rights Reserved.

#include "IndexedCacheStorageManager.h"
#include "IndexedCacheStorage.h"

#include "Async/Async.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Experimental/Async/ConditionVariable.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/PlatformStackWalk.h"
#include "Misc/ScopeExit.h"
#include "Misc/ScopeRWLock.h"
#include "String/LexFromString.h"
#include "String/ParseTokens.h"

DECLARE_LOG_CATEGORY_EXTERN(LogIndexedCacheStorageManager, Display, All);
DEFINE_LOG_CATEGORY(LogIndexedCacheStorageManager);
#define LOG_ICS(Verbosity, STRING, ...) UE_LOG(LogIndexedCacheStorageManager, Verbosity, TEXT("%s: ") STRING, ANSI_TO_TCHAR(__FUNCTION__), ##__VA_ARGS__ )

namespace Experimental
{

class FCacheStorageMetaData
{
public:
	enum class ECacheStorageState : uint8
	{
		Deleted,
		Created,
		Mounted,
		Count
	};

	const TCHAR* LexToString(ECacheStorageState Val)
	{
		static const TCHAR* Strings[] =
		{
			TEXT("ECacheStorageState::Deleted"),
			TEXT("ECacheStorageState::Created"),
			TEXT("ECacheStorageState::Mounted"),
		};
		static_assert(int32(ECacheStorageState::Count) == UE_ARRAY_COUNT(Strings), "");

		return Strings[int32(Val)];
	}
	
	FCacheStorageMetaData(int32 InCacheIndex, const FString& InMountName, bool bCacheExists, uint64 InEarlyStartupSize)
	: MountName(InMountName)
	, CacheIndex(InCacheIndex)
	, EarlyStartupSize(InEarlyStartupSize)
	{
		CacheStorageState = bCacheExists ? ECacheStorageState::Created : ECacheStorageState::Deleted;
	}

	bool IsValid() const
	{
		return !MountName.IsEmpty();
	}

	const FString& GetMountName() const
	{
		return MountName;
	}

	uint64 GetEarlyStartupSize() const
	{
		return EarlyStartupSize;
	}

	FString GetMountPath() const
	{
		FString MountedPath;
		IIndexedCacheStorage::Get().GetCacheStorageMountPath(MountedPath, MountName);
		return MountedPath;
	}

	void WaitForPredicateWhileLocked(const TFunction<bool()>& Predicate)
	{
		while (!Predicate())
		{
			ConditionVariable.Wait(Mutex);
		}
	}

	void NotifyWaitersWhileLocked()
	{
		ConditionVariable.NotifyAll();
	}

	bool CreateCacheStorage(uint64 RequestNumberOfBytes)
	{
		UE::TUniqueLock Lock(Mutex);
		 
		check(EarlyStartupSize == 0 || RequestNumberOfBytes == EarlyStartupSize);

		UE_LOGF(LogIndexedCacheStorageManager, Display, "CreateCacheStorage request for %llu bytes at cache %d. Current state %ls", RequestNumberOfBytes, CacheIndex, LexToString(CacheStorageState));

		WaitForPredicateWhileLocked([this]()->bool 
		{ 
			return CacheStorageState != ECacheStorageState::Mounted;
		});;

		check(MountRefCount == 0);

		// Do not uncomment the code below: even if the cache is in 'Created' state, we might want to handle the case of a reallocation and 
		// the lower level takes care of either reuse the one we asked for (same size), or destroy and create a new one
		//if (CacheStorageState == ECacheStorageState::Created)
		//{
		//	return true;
		//}

		UE_LOGF(LogIndexedCacheStorageManager, Display, "CreateCacheStorage creating cache %d with %llu bytes", CacheIndex, RequestNumberOfBytes);
		bool bCacheExists = IIndexedCacheStorage::Get().CreateCacheStorage(RequestNumberOfBytes, CacheIndex);
		
		if (bCacheExists)
		{
			CacheStorageState = ECacheStorageState::Created;
			UE_LOGF(LogIndexedCacheStorageManager, Display, "CreateCacheStorage created cache %d at %ls. Current state %ls", CacheIndex, *MountName, LexToString(CacheStorageState));
		}
		else
		{
			CacheStorageState = ECacheStorageState::Deleted;
			UE_LOGF(LogIndexedCacheStorageManager, Error, "CreateCacheStorage failed to create cache %d at %ls. Current state %ls", CacheIndex, *MountName, LexToString(CacheStorageState));
		}

		NotifyWaitersWhileLocked();

		return bCacheExists;
	}

	void DestroyCacheStorage()
	{
		UE::TUniqueLock Lock(Mutex);
		 
		UE_LOGF(LogIndexedCacheStorageManager, Display, "DestroyCacheStorage request at cache %d. Current state %ls", CacheIndex, LexToString(CacheStorageState));

		WaitForPredicateWhileLocked([this]()->bool 
		{ 
			return CacheStorageState != ECacheStorageState::Mounted;
		});;

		check(MountRefCount == 0);

		if (CacheStorageState == ECacheStorageState::Deleted)
		{
			UE_LOGF(LogIndexedCacheStorageManager, Display, "DestroyCacheStorage skipped deletion of cache %d at %ls", CacheIndex, *MountName);
			return;
		}

		UE_LOGF(LogIndexedCacheStorageManager, Display, "DestroyCacheStorage deleting cache %d", CacheIndex);
		bool bCachedDestroyed = IIndexedCacheStorage::Get().DestroyCacheStorage(CacheIndex);
		
		if (bCachedDestroyed)
		{
			CacheStorageState = ECacheStorageState::Deleted;
			UE_LOGF(LogIndexedCacheStorageManager, Display, "DestroyCacheStorage deleted cache %d at %ls. Current state %ls", CacheIndex, *MountName, LexToString(CacheStorageState));
		}
		else
		{
			CacheStorageState = ECacheStorageState::Created;
			UE_LOGF(LogIndexedCacheStorageManager, Error, "DestroyCacheStorage failed to delete cache %d at %ls. Current state %ls", CacheIndex, *MountName, LexToString(CacheStorageState));
		}

		NotifyWaitersWhileLocked();
	}

	uint64 GetCacheStorageInfo()
	{
		UE::TUniqueLock Lock(Mutex);

		UE_LOGF(LogIndexedCacheStorageManager, Display, "GetCacheStorageInfo request at cache %d. Current state %ls", CacheIndex, LexToString(CacheStorageState));

		uint64 CurrentDataSize = 0;
		uint64 CurrentJournalSize = 0;

		if (CacheStorageState == ECacheStorageState::Deleted)
		{
			UE_LOGF(LogIndexedCacheStorageManager, Display, "GetCacheStorageInfo cache %d is empty", CacheIndex);
			return CurrentDataSize;
		}
		
		UE_LOGF(LogIndexedCacheStorageManager, Display, "GetCacheStorageInfo cache %d", CacheIndex);
		IIndexedCacheStorage::Get().GetCacheStorageInfo(CacheIndex, CurrentDataSize, CurrentJournalSize);
		UE_LOGF(LogIndexedCacheStorageManager, Display, "GetCacheStorageInfo cache %d = data %llu bytes / journal %llu bytes", CacheIndex, CurrentDataSize, CurrentJournalSize);
		
		NotifyWaitersWhileLocked();

		return CurrentDataSize;
	}

	FString MountCacheStorage()
	{
		UE::TUniqueLock Lock(Mutex);

		UE_LOGF(LogIndexedCacheStorageManager, Display, "MountCacheStorage request at cache %d. Current state %ls", CacheIndex, LexToString(CacheStorageState));

		if (CacheStorageState == ECacheStorageState::Deleted)
		{
			check(MountRefCount == 0);
			UE_LOGF(LogIndexedCacheStorageManager, Display, "MountCacheStorage skip mount for cache %d deleted", CacheIndex);
			return FString();
		}

		FString MountedPath = GetMountPath();
		if (CacheStorageState == ECacheStorageState::Mounted)
		{
			check(MountRefCount > 0);
			UE_LOGF(LogIndexedCacheStorageManager, Display, "MountCacheStorage skip mount for cache %d already mounted (%d)", CacheIndex, MountRefCount);
			++MountRefCount;
			return MountedPath;
		}

		check(MountRefCount == 0);
		check(CacheStorageState == ECacheStorageState::Created);

		UE_LOGF(LogIndexedCacheStorageManager, Display, "MountCacheStorage mounting cache %d at %ls", CacheIndex, *MountedPath);
		bool bMounted = IIndexedCacheStorage::Get().MountCacheStorage(MountedPath, CacheIndex, MountName);
		
		check(MountRefCount == 0);

		if (bMounted)
		{
			CacheStorageState = ECacheStorageState::Mounted;
			MountRefCount = 1;
			UE_LOGF(LogIndexedCacheStorageManager, Display, "MountCacheStorage mounted cache %d at %ls. Current state %ls", CacheIndex, *MountedPath, LexToString(CacheStorageState));
		}
		else
		{
			CacheStorageState = ECacheStorageState::Created;
			UE_LOGF(LogIndexedCacheStorageManager, Error, "MountCacheStorage failed to mount cache %d at %ls. Current state %ls", CacheIndex, *MountedPath, LexToString(CacheStorageState));
		}

		NotifyWaitersWhileLocked();
		return (bMounted) ? MountedPath : FString();
	}


	void UnmountCacheStorage()
	{
		UE::TUniqueLock Lock(Mutex);

		UE_LOGF(LogIndexedCacheStorageManager, Display, "UnmountCacheStorage request at cache %d. Current state %ls", CacheIndex, LexToString(CacheStorageState));

		if (CacheStorageState == ECacheStorageState::Deleted || CacheStorageState == ECacheStorageState::Created)
		{
			check(MountRefCount == 0);
			UE_LOGF(LogIndexedCacheStorageManager, Display, "UnmountCacheStorage skip unmount for cache %d already unmounted", CacheIndex);
			return;
		}

		check(CacheStorageState == ECacheStorageState::Mounted);
		check(MountRefCount > 0);

		if (MountRefCount > 1)
		{
			--MountRefCount;
			UE_LOGF(LogIndexedCacheStorageManager, Display, "UnmountCacheStorage cache %d still mounted (%d)", CacheIndex, MountRefCount);
			return;
		}

		FString MountedPath = GetMountPath();
		UE_LOGF(LogIndexedCacheStorageManager, Display, "UnmountCacheStorage unmounting cache %d at %ls", CacheIndex, *MountedPath);
		IIndexedCacheStorage::Get().UnmountCacheStorage(*MountName);
		CacheStorageState = ECacheStorageState::Created;
		UE_LOGF(LogIndexedCacheStorageManager, Display, "UnmountCacheStorage unmounted cache %d at %ls. Current state %ls", CacheIndex, *MountedPath, LexToString(CacheStorageState));

		check(MountRefCount == 1);
		MountRefCount = 0;
		NotifyWaitersWhileLocked();
	}

private:
	const FString MountName;
	const int32 CacheIndex;
	const uint64 EarlyStartupSize;
	int32 MountRefCount = 0;
	ECacheStorageState CacheStorageState = ECacheStorageState::Deleted;
	UE::FMutex Mutex;
	UE::FConditionVariable ConditionVariable;

};

using FCacheStorageMetaDataArray = TArray<TUniquePtr<FCacheStorageMetaData> >;

struct FIndexedCacheStorageMetaData
{
	FIndexedCacheStorageMetaData(FCacheStorageMetaDataArray&& InCacheStorageMetaDataArray)
	: IndexedCacheEntriesMetaData(MoveTemp(InCacheStorageMetaDataArray))
	{
	}

	const FCacheStorageMetaDataArray IndexedCacheEntriesMetaData;

};

static FIndexedCacheStorageManager* GIndexedCacheStorageManager = nullptr;

FIndexedCacheStorageManager& FIndexedCacheStorageManager::Get()
{
	static FIndexedCacheStorageManager Instance;
	UE_CALL_ONCE([&]
	{
		GIndexedCacheStorageManager = &Instance;
	}
	);
	
	return Instance;
}

void FIndexedCacheStorageManager::EnsureEarlyStartupCache(const TArray<bool>& InCacheStorageAlreadyExists)
{
	int32 EarlyStartupCacheIndex = GetStorageIndex(TEXT("EarlyStartupCache"));
	if (EarlyStartupCacheIndex == 0)
	{
		LOG_ICS(Display, TEXT("No Storage named 'EarlyStartupCache' has been set. Skipping"));
		return;
	}

	TArray<bool> CacheStorageAlreadyExists(InCacheStorageAlreadyExists);

	if (CacheStorageAlreadyExists[EarlyStartupCacheIndex])
	{
		LOG_ICS(Error, TEXT("'EarlyStartupCache' entry already exists. Destroying it"));
		DestroyCacheStorage(EarlyStartupCacheIndex);
		CacheStorageAlreadyExists[EarlyStartupCacheIndex] = false;
	}

	TMap<int32, uint64> EarlyStartupCaches;
	{
		uint64 PersistentDownloadDirEarlyStartupSize = IIndexedCacheStorage::Get().GetPersistentDownloadDirEarlyStartupSize();
		if (PersistentDownloadDirEarlyStartupSize > 0)
		{
			EarlyStartupCaches.Add(0, PersistentDownloadDirEarlyStartupSize);
		}
		
		for (int32 CacheIndex = 0; CacheIndex < IndexedCacheStorageMetaData->IndexedCacheEntriesMetaData.Num(); ++CacheIndex)
		{
			FCacheStorageMetaData* CacheStorageMetaData = IndexedCacheStorageMetaData->IndexedCacheEntriesMetaData[CacheIndex].Get();
			if (CacheStorageMetaData == nullptr)
			{
				continue;
			}

			uint64 EarlyStartupSize = CacheStorageMetaData->GetEarlyStartupSize();

			if (EarlyStartupSize > 0)
			{
				EarlyStartupCaches.Add(CacheIndex, EarlyStartupSize);
			}
		}
	}

	uint64 EarlyStartupCacheSize = 0;
	uint64 NumCacheStorageInGroup = 0;

	for (const TPair<int32, uint64>& EarlyStartupCache : EarlyStartupCaches)
	{
		uint64 CacheStorageSizeForCommonAlloc = 0;
		const int32 CacheStorageIndex = EarlyStartupCache.Key;
		const uint64 CacheStorageNewDataSize = EarlyStartupCache.Value;
		const uint64 CacheStorageNewJournalSize = IIndexedCacheStorage::Get().GetCacheStorageJournalSize(CacheStorageNewDataSize, CacheStorageIndex);
		if (CacheStorageAlreadyExists[CacheStorageIndex])
		{
			uint64 CacheStorageCurrentDataSize = 0;
			uint64 CacheStorageCurrentJournalSize = 0;
			IIndexedCacheStorage::Get().GetCacheStorageInfo(CacheStorageIndex, CacheStorageCurrentDataSize, CacheStorageCurrentJournalSize);
			if (CacheStorageCurrentDataSize != CacheStorageNewDataSize || CacheStorageCurrentJournalSize != CacheStorageNewJournalSize)
			{
				LOG_ICS(Display, TEXT("Cache %d size mismatch: data: %llu vs %llu. journal: %llu vs %llu. Destroying cache"),  
					   CacheStorageIndex, CacheStorageCurrentDataSize, CacheStorageNewDataSize, CacheStorageCurrentJournalSize, CacheStorageNewJournalSize);

				DestroyCacheStorage(CacheStorageIndex);
				CacheStorageSizeForCommonAlloc = CacheStorageNewDataSize + CacheStorageNewJournalSize;
			}
			else
			{
				LOG_ICS(Display, TEXT("Cache %d up to date with %llu bytes. Skipping from grouped alloc"), CacheStorageIndex, CacheStorageCurrentDataSize);
			}
		}
		else if (CacheStorageNewDataSize > 0)
		{
			LOG_ICS(Display, TEXT("Cache %d does not exist, adding %llu bytes"), CacheStorageIndex, CacheStorageNewDataSize);
			CacheStorageSizeForCommonAlloc = CacheStorageNewDataSize + CacheStorageNewJournalSize;
		}

		if (CacheStorageSizeForCommonAlloc > 0)
		{
			EarlyStartupCacheSize += CacheStorageSizeForCommonAlloc;
			++NumCacheStorageInGroup;
		}
	}

	if (EarlyStartupCacheSize == 0)
	{
		LOG_ICS(Display, TEXT("All caches are up-to-date, skipping reservation"));
		return;
	}

	if (NumCacheStorageInGroup <= 1)
	{
		LOG_ICS(Display, TEXT("Only one cache requested, skipping reservation"));
		return;
	}

	{
		// Make sure we alloc at least 2MB
		const uint64 MinGroupAllocSize = 2 << 20;
		EarlyStartupCacheSize = FMath::Max(MinGroupAllocSize, EarlyStartupCacheSize);

		// Remove alloc meta data because we want to allocate exactly EarlyStartupCacheSize
		uint64 EarlyStartupCacheSizeMetaData = IIndexedCacheStorage::Get().GetCacheStorageJournalSize(EarlyStartupCacheSize, EarlyStartupCacheIndex);
		// make sure we don't underflow
		EarlyStartupCacheSize -= FMath::Min(EarlyStartupCacheSizeMetaData, EarlyStartupCacheSize);

		// Make sure we alloc at least the metadata size
		EarlyStartupCacheSize = FMath::Max(EarlyStartupCacheSizeMetaData, EarlyStartupCacheSize);
	}

	LOG_ICS(Display, TEXT("Ensure %llu bytes through cache %d"), EarlyStartupCacheSize, EarlyStartupCacheIndex);
	bool bCacheCreated = true;
	while (true)
	{
		if (CreateCacheStorage(EarlyStartupCacheSize, EarlyStartupCacheIndex))
		{
			DestroyCacheStorage(EarlyStartupCacheIndex);
			break;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
uint64 ParseSizeParam(FStringView Value)
{
	Value = Value.TrimStartAndEnd();

	uint64 Size = 0;
	LexFromString(Size, Value);
	if (Size > 0)
	{
		if (Value.EndsWith(TEXT("GB"))) return Size << 30;
		if (Value.EndsWith(TEXT("MB"))) return Size << 20;
		if (Value.EndsWith(TEXT("KB"))) return Size << 10;
	}
	return Size;
}

// Parse IniString expected to be in the form of OutIniName:[OutIniSection]:OutIniKey
bool GetIniParameters(FString& OutIniName, FString& OutIniSection, FString& OutIniKey, const FString& IniString)
{
	TArray<FStringView> ConfigStringArrayNameSize;
	UE::String::ParseTokens(IniString, TEXTVIEW(":"), [&](FStringView Element)
	{
		ConfigStringArrayNameSize.Add(Element);
	});

	if (ConfigStringArrayNameSize.Num() != 3 || 
		ConfigStringArrayNameSize[0].Len() == 0 ||
		ConfigStringArrayNameSize[1].Len() == 0 ||
		ConfigStringArrayNameSize[2].Len() == 0 )
	{
		UE_LOGF(LogIndexedCacheStorageManager, Error, "Error while parsing %ls: expecting OutIniName:[OutIniSection]:OutIniKey", *IniString);
		return false;
	}

	FStringView IniSectionView = ConfigStringArrayNameSize[1];
	if (IniSectionView[0] != TEXT('[') || IniSectionView[IniSectionView.Len()-1] != TEXT(']'))
	{
		UE_LOGF(LogIndexedCacheStorageManager, Error, "Error while parsing ini section %.*ls", IniSectionView.Len(), IniSectionView.GetData());
		return false;
	}

	IniSectionView.RemovePrefix(1);
	IniSectionView.RemoveSuffix(1);

	OutIniName = ConfigStringArrayNameSize[0];
	OutIniSection = IniSectionView;
	OutIniKey = ConfigStringArrayNameSize[2];

	return true;
}

void FIndexedCacheStorageManager::Initialize()
{
	uint32 NumIndexedCacheEntries = IIndexedCacheStorage::Get().GetNumIndexedCacheStorages();
	if (NumIndexedCacheEntries == 0)
	{
		return;
	}

	FCacheStorageMetaDataArray IndexedCacheEntriesMetaData;
	IndexedCacheEntriesMetaData.SetNum(NumIndexedCacheEntries);

	TArray<int32> CacheStorageIndices;
	IIndexedCacheStorage::Get().EnumerateCacheStorages(CacheStorageIndices);

	TArray<bool> CacheStorageAlreadyExists;
	CacheStorageAlreadyExists.SetNumZeroed(NumIndexedCacheEntries);

	for (int32 CacheStorageIndex : CacheStorageIndices)
	{
		CacheStorageAlreadyExists[CacheStorageIndex] = true;
	}


	// Takes on the pattern
	// (Name="CacheStorageName",Index=CacheStorageIndex,Mount="CacheStorageMount")
	TArray<FString> StorageConfigs;
	GConfig->GetArray(TEXT("IndexedCacheStorage"), TEXT("Storage"), StorageConfigs, GEngineIni);

	uint32 RegisteredStorages = 0;
	for (const FString& Category : StorageConfigs)
	{
		FString TrimmedCategory = Category;
		TrimmedCategory.TrimStartAndEndInline();
		if (TrimmedCategory.Left(1) == TEXT("("))
		{
			TrimmedCategory.RightChopInline(1, EAllowShrinking::No);
		}
		if (TrimmedCategory.Right(1) == TEXT(")"))
		{
			TrimmedCategory.LeftChopInline(1, EAllowShrinking::No);
		}

		// Find all custom chunks and parse
		const TCHAR* PropertyName = TEXT("Name=");
		const TCHAR* PropertyIndex = TEXT("Index=");
		const TCHAR* PropertyMount = TEXT("Mount=");
		const TCHAR* PropertyEarlyStartup = TEXT("EarlyStartup=");
		FString StorageName;
		int32 StorageIndex;
		FString StorageMount;

		if (FParse::Value(*TrimmedCategory, PropertyName, StorageName) &&
			FParse::Value(*TrimmedCategory, PropertyIndex, StorageIndex) &&
			FParse::Value(*TrimmedCategory, PropertyMount, StorageMount))
		{
			if (StorageName.Len() == 0)
			{
				UE_LOGF(LogIndexedCacheStorageManager, Error, "Found empty Name in [IndexedCacheStorage]:Storage");
				continue;
			}

			if (StorageIndex <= 0 || StorageIndex >= (int32)NumIndexedCacheEntries)
			{
				UE_LOGF(LogIndexedCacheStorageManager, Error, "Found invalid Index in [IndexedCacheStorage]:Storage: %d, max allowed = %d", StorageIndex, NumIndexedCacheEntries - 1);
				continue;
			}

			if (StorageMount.Len() == 0)
			{
				UE_LOGF(LogIndexedCacheStorageManager, Error, "Found empty Mount in [IndexedCacheStorage]:Storage");
				continue;
			}

			StorageName.ReplaceInline(TEXT("\""), TEXT(""));
			StorageMount.ReplaceInline(TEXT("\""), TEXT(""));

			int32& ExistingStorageIndex = StorageNameToStorageIndex.FindOrAdd(StorageName, 0);
			if (ExistingStorageIndex > 0)
			{
				UE_LOGF(LogIndexedCacheStorageManager, Error, "Found an existing entry with name %ls in [IndexedCacheStorage]:Storage", *StorageName);
				continue;
			}
			ExistingStorageIndex = StorageIndex;

			uint64 EarlyStartupSize = 0;
			{
				FString EarlyStartupIniPath;
				if (FParse::Value(*TrimmedCategory, PropertyEarlyStartup, EarlyStartupIniPath))
				{
					FString IniName;
					FString IniSection;
					FString IniKey;
					if (GetIniParameters(IniName, IniSection, IniKey, EarlyStartupIniPath))
					{
						// Fetch the value directly from EarlyStartupIniPath
						FString EarlyStartupSizeStr = GConfig->GetStr(*IniSection, *IniKey, *IniName);
						EarlyStartupSize = ParseSizeParam(EarlyStartupSizeStr);
						if (EarlyStartupSize == 0)
						{
							UE_LOGF(LogIndexedCacheStorageManager, Error, 
								   "%ls%ls specified, but no %ls:%ls exists in the %ls ini file", PropertyEarlyStartup, *EarlyStartupIniPath, *IniSection, *IniKey, *IniName);
						}
					}
					else if(TArray<FString> EarlyStartupIniPathArray; 
							GConfig->GetArray(TEXT("IndexedCacheStorage"), *EarlyStartupIniPath, EarlyStartupIniPathArray, GEngineIni))
					{
						// EarlyStartupIniPath is the key for an array, sum all the sections in the array
						for (const FString& IniPath : EarlyStartupIniPathArray)
						{
							if (GetIniParameters(IniName, IniSection, IniKey, IniPath))
							{
								FString EarlyStartupSizeStr = GConfig->GetStr(*IniSection, *IniKey, *IniName);
								const uint64 ParsedSize = ParseSizeParam(EarlyStartupSizeStr);
								if (ParsedSize == 0)
								{
									UE_LOGF(LogIndexedCacheStorageManager, Error,
										"%ls%ls specified, but no %ls:%ls exists in the %ls ini file", PropertyEarlyStartup, *IniPath, *IniSection, *IniKey, *IniName);
								}
								EarlyStartupSize += ParsedSize;
							}
							else
							{
								const uint64 ParsedSize = ParseSizeParam(IniPath);
								if (ParsedSize == 0)
								{
									UE_LOGF(LogIndexedCacheStorageManager, Error,
										"%ls%ls specified, but no does not specify a usable value", PropertyEarlyStartup, *IniPath);
								}
								EarlyStartupSize += ParsedSize;
							}
						}
					}
				}
			}

			if (IndexedCacheEntriesMetaData[StorageIndex] == nullptr)
			{
				IndexedCacheEntriesMetaData[StorageIndex] = MakeUnique<FCacheStorageMetaData>(StorageIndex, StorageMount, CacheStorageAlreadyExists[StorageIndex], EarlyStartupSize);
				++RegisteredStorages;
			}
			else
			{
				const FString& ExistingName = IndexedCacheEntriesMetaData[StorageIndex]->GetMountName();
				UE_LOGF(LogIndexedCacheStorageManager, Error, "Found an existing entry named %ls with at cache %d for name %ls in [IndexedCacheStorage]:Storage", *ExistingName, StorageIndex, *StorageName);
			}
		}
	}
	
	bHasRegisteredEntries = (RegisteredStorages > 0);
	if (bHasRegisteredEntries)
	{
		IndexedCacheStorageMetaData = MakeUnique<FIndexedCacheStorageMetaData>(MoveTemp(IndexedCacheEntriesMetaData));
		EnsureEarlyStartupCache(CacheStorageAlreadyExists);
	}
}

bool FIndexedCacheStorageManager::SupportsIndexedCacheStorage()
{
	return bHasRegisteredEntries && IIndexedCacheStorage::Get().SupportsIndexedCacheStorage();
}

int32 FIndexedCacheStorageManager::GetStorageIndex(const FString& StorageName)
{
	int32* StorageIndex = StorageNameToStorageIndex.Find(StorageName);
	return StorageIndex ? *StorageIndex : 0;
}

uint64 FIndexedCacheStorageManager::GetCacheEarlyStartupSize(int32 CacheIndex)
{
	FCacheStorageMetaData* CacheStorageMetaData = GetCacheStorageMetaData(CacheIndex);
	if (CacheStorageMetaData == nullptr)
	{
		UE_LOGF(LogIndexedCacheStorageManager, Error, "GetCacheEarlyStartupSize skip invalid cache %d", CacheIndex);
		return 0;
	}

	return CacheStorageMetaData->GetEarlyStartupSize();
}

void FIndexedCacheStorageManager::EnumerateCacheStorages(TArray<FString>& OutCacheStorageNames)
{
	OutCacheStorageNames.Reserve(StorageNameToStorageIndex.Num());
	for (const TPair<FString, int32>& StorageNameToStorageIndexIter : StorageNameToStorageIndex)
	{
		OutCacheStorageNames.Add(StorageNameToStorageIndexIter.Key);
	}
}

bool FIndexedCacheStorageManager::CreateCacheStorage(uint64 RequestNumberOfBytes, int32 CacheIndex)
{
	FCacheStorageMetaData* CacheStorageMetaData = GetCacheStorageMetaData(CacheIndex);
	if (CacheStorageMetaData == nullptr)
	{
		UE_LOGF(LogIndexedCacheStorageManager, Error, "CreateCacheStorage skip invalid cache %d", CacheIndex);
		return false;
	}

	return CacheStorageMetaData->CreateCacheStorage(RequestNumberOfBytes);
}

void FIndexedCacheStorageManager::DestroyCacheStorage(int32 CacheIndex)
{
	FCacheStorageMetaData* CacheStorageMetaData = GetCacheStorageMetaData(CacheIndex);
	if (CacheStorageMetaData == nullptr)
	{
		UE_LOGF(LogIndexedCacheStorageManager, Error, "DestroyCacheStorage skip invalid cache %d", CacheIndex);
		return;
	}

	CacheStorageMetaData->DestroyCacheStorage();
}

uint64 FIndexedCacheStorageManager::GetCacheStorageCapacity(int32 CacheIndex)
{
	FCacheStorageMetaData* CacheStorageMetaData = GetCacheStorageMetaData(CacheIndex);
	if (CacheStorageMetaData == nullptr)
	{
		UE_LOGF(LogIndexedCacheStorageManager, Warning, "GetCacheStorageCapacity skip invalid cache %d", CacheIndex);
		return 0;
	}

	return CacheStorageMetaData->GetCacheStorageInfo();
}

FString FIndexedCacheStorageManager::MountCacheStorage(int32 CacheIndex)
{
	FCacheStorageMetaData* CacheStorageMetaData = GetCacheStorageMetaData(CacheIndex);
	if (CacheStorageMetaData == nullptr)
	{
		UE_LOGF(LogIndexedCacheStorageManager, Warning, "MountCacheStorage skip invalid cache %d", CacheIndex);
		return FString();
	}

	return CacheStorageMetaData->MountCacheStorage();
}

void FIndexedCacheStorageManager::UnmountCacheStorage(int32 CacheIndex)
{
	FCacheStorageMetaData* CacheStorageMetaData = GetCacheStorageMetaData(CacheIndex);
	if (CacheStorageMetaData == nullptr)
	{
		UE_LOGF(LogIndexedCacheStorageManager, Warning, "UnmountCacheStorage skip invalid cache %d", CacheIndex);
		return;
	}

	return CacheStorageMetaData->UnmountCacheStorage();
}

FString FIndexedCacheStorageManager::GetMountName(int32 CacheIndex)
{
	const FCacheStorageMetaData* CacheStorageMetaData = GetCacheStorageMetaData(CacheIndex);
	if (CacheStorageMetaData == nullptr)
	{
		UE_LOGF(LogIndexedCacheStorageManager, Warning, "GetMountName skip invalid cache %d", CacheIndex);
		return FString();
	}

	return CacheStorageMetaData->GetMountName();
}

FString FIndexedCacheStorageManager::GetMountPath(int32 CacheIndex)
{
	const FCacheStorageMetaData* CacheStorageMetaData = GetCacheStorageMetaData(CacheIndex);
	if (CacheStorageMetaData == nullptr)
	{
		UE_LOGF(LogIndexedCacheStorageManager, Warning, "GetMountPath skip invalid cache %d", CacheIndex);
		return FString();
	}

	return CacheStorageMetaData->GetMountPath();
}

FCacheStorageMetaData* FIndexedCacheStorageManager::GetCacheStorageMetaData(int32 CacheIndex)
{
	// We must skip index 0 since it's mapped to GamePersistentDownloadDir()
	bool bValid = CacheIndex > 0 && IndexedCacheStorageMetaData && CacheIndex < IndexedCacheStorageMetaData->IndexedCacheEntriesMetaData.Num();
	if (!bValid)
	{
		return nullptr;
	}

	FCacheStorageMetaData* OutCacheStorageMetaData = IndexedCacheStorageMetaData->IndexedCacheEntriesMetaData[CacheIndex].Get();
	if (OutCacheStorageMetaData && OutCacheStorageMetaData->IsValid())
	{
		return OutCacheStorageMetaData;
	}

	return nullptr;
}

#if !UE_BUILD_SHIPPING
namespace IndexedCacheStorageManager_Debug
{
static void CopyFiles(const FString& TargetDirectory);
static void ListFiles();

static FAutoConsoleCommand GCopyFiles(
	TEXT("ics.CopyFiles"),
	TEXT("Copies all cache files to the supplied folder"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args)
	{
		if (Args.IsEmpty())
		{
			UE_LOGF(LogIndexedCacheStorageManager, Error, "usage: ics.CopyFiles <DestDir>");
			return;
		}

		CopyFiles(Args[0]);
	}));

static FAutoConsoleCommand GListFiles(
	TEXT("ics.ListFiles"),
	TEXT("Copies all cache files to the supplied folder"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>&)
	{
		ListFiles();
	}));

static void CollectCacheIndices(TArray<int32>& CacheIndices)
{
	uint32 NumIndexedCacheEntries = IIndexedCacheStorage::Get().GetNumIndexedCacheStorages();
	CacheIndices.Reserve(NumIndexedCacheEntries+1);

	if (FPaths::HasProjectPersistentDownloadDir())
	{
		CacheIndices.Add(0);
	}

	for (uint32 CacheIndex = 1; CacheIndex < NumIndexedCacheEntries; ++CacheIndex)
	{
		uint64 CacheCapacity = FIndexedCacheStorageManager::Get().GetCacheStorageCapacity(CacheIndex);
		if (CacheCapacity == 0)
		{
			continue;
		}

		CacheIndices.Add(CacheIndex);
	}

}

static void ListFiles()
{
	TArray<int32> CacheIndices;
	CollectCacheIndices(CacheIndices);

	UE_LOGF(LogConsoleResponse, Display, "Listing cache files ...");

	AsyncTask(
		ENamedThreads::AnyBackgroundThreadNormalTask, [CacheIndices]() mutable
		{
			for (int32 CacheIndex : CacheIndices)
			{
				FIndexedCacheStorageScopeMount IndexedCacheStorageScopeMount(CacheIndex);

				FString SourcePath = (CacheIndex == 0) ? FPaths::ProjectPersistentDownloadDir() : IndexedCacheStorageScopeMount.MountPath;

				if (SourcePath.IsEmpty())
				{
					return;
				}

				IFileManager::Get().IterateDirectoryStatRecursively(
					*SourcePath,
					[](const TCHAR* FileOrDir, const FFileStatData& StatData)
					{
						if (StatData.bIsValid)
						{
							int64 FileSize = !StatData.bIsDirectory ? StatData.FileSize : 0;
							UE_LOGF(LogConsoleResponse, Display, "%10llu %ls%c", FileSize, FileOrDir, (StatData.bIsDirectory ? TEXT('/') : TEXT('\0')));
						}
						return true;
					}
				);
			}

			UE_LOGF(LogConsoleResponse, Display, "Listing cache files done");

		}
	);
}

static void CopyFiles(const FString& InTargetDirectory)
{
	if (!IFileManager::Get().DirectoryExists(*InTargetDirectory))
	{
		IFileManager::Get().MakeDirectory(*InTargetDirectory);
	}

	TArray<int32> CacheIndices;
	CollectCacheIndices(CacheIndices);

	FString TargetDirectory(InTargetDirectory);

	UE_LOGF(LogConsoleResponse, Display, "Copy files to %ls ...", *TargetDirectory);

	AsyncTask(
		ENamedThreads::AnyBackgroundThreadNormalTask, [CacheIndices, TargetDirectory]() mutable
		{
			for (int32 CacheIndex : CacheIndices)
			{
				FIndexedCacheStorageScopeMount IndexedCacheStorageScopeMount(CacheIndex);

				FString SourcePath = (CacheIndex == 0) ? FPaths::ProjectPersistentDownloadDir() : IndexedCacheStorageScopeMount.MountPath;

				if (SourcePath.IsEmpty())
				{
					return;
				}

				if (!SourcePath.EndsWith(TEXT("/")))
				{
					SourcePath += TEXT("/");
				}

				TArray<FString> SourceFiles;
				TArray<FString> DestFiles;

				FString TargetDirectoryCache = FString::Printf(TEXT("%s/cache_%02d"), *TargetDirectory, CacheIndex);

				IFileManager::Get().IterateDirectoryStatRecursively(
					*SourcePath,
					[SourcePath, TargetDirectoryCache, &SourceFiles, &DestFiles](const TCHAR* FileOrDir, const FFileStatData& StatData)
					{
						if (StatData.bIsValid)
						{
							FString RelativeSourceFile(FileOrDir);

							if (StatData.bIsDirectory && !RelativeSourceFile.EndsWith(TEXT("/")))
							{
								RelativeSourceFile += TEXT("/");
							}

							FPaths::MakePathRelativeTo(RelativeSourceFile, *SourcePath);
							FString DestFile = TargetDirectoryCache / RelativeSourceFile;
							FString DestDir = StatData.bIsDirectory ? DestFile : FPaths::GetPath(DestFile);

							if (StatData.bIsDirectory)
							{
								IFileManager::Get().MakeDirectory(*DestDir, true);
							}
							else
							{
								SourceFiles.Add(FileOrDir);
								DestFiles.Add(DestFile);
							}
						}
						return true;
					}
				);

				for	(int32 SourceFileIndex = 0; SourceFileIndex < SourceFiles.Num(); ++SourceFileIndex)
				{
					UE_LOGF(LogConsoleResponse, Display, "[%d/%d] %ls -> %ls", SourceFileIndex+1, SourceFiles.Num(), *SourceFiles[SourceFileIndex], *DestFiles[SourceFileIndex]);
					IFileManager::Get().Copy(*DestFiles[SourceFileIndex], *SourceFiles[SourceFileIndex]);
				}
			}

			UE_LOGF(LogConsoleResponse, Display, "Copy files to %ls done", *TargetDirectory);
		}
	);
}

} // namespace IndexedCacheStorageManager_Debug
#endif // !UE_BUILD_SHIPPING

}
