// Copyright Epic Games, Inc. All Rights Reserved.

#include "IndexedCacheStorageManager.h"

#include "Modules/ModuleManager.h"

namespace Experimental
{

void FIndexedCacheStorageModule::StartupModule()
{
	FIndexedCacheStorageManager::Get().Initialize();
}

void FIndexedCacheStorageModule::ShutdownModule()
{
}

}

IMPLEMENT_MODULE(Experimental::FIndexedCacheStorageModule, IndexedCacheStorage);

