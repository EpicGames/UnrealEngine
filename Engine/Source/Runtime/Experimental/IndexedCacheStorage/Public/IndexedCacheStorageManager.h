// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HAL/CriticalSection.h"
#include "Modules/ModuleInterface.h"

#define UE_API INDEXEDCACHESTORAGE_API

namespace Experimental
{

class FIndexedCacheStorageModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

class FCacheStorageMetaData;

// Manager to handle ref-counting of ICS entries
class FIndexedCacheStorageManager
{
public:

	UE_API static FIndexedCacheStorageManager& Get();

	// Returns whether ICS is supported
	UE_API bool SupportsIndexedCacheStorage();

	// Returns the cache index registered to that storage name. Registration happens in the config section:
	// [IndexedCacheStorage]
	// +Storage=(Name="StorageName",Index=StorageIndex,Mount="StorageMount",EarlyStartup=IniName:[IniSection]:IniKey)
	// The EarlyStartup attribute is optional. It will try to fetch the value from the specified init path
	// The value in the ini file can be written as 512, 512KB, 512MB or 512GB
	// Requires that ones of the entries in the Storage array is called EarlyStartupCache
	UE_API int32 GetStorageIndex(const FString& CacheStorageName);

	// Returns 0 if no early startup size configured
	UE_API uint64 GetCacheEarlyStartupSize(int32 CacheIndex);

	// returns all registered storage names. The assoaciated cache storages might not have been created
	UE_API void EnumerateCacheStorages(TArray<FString>& OutCacheStorageNames);

	// Create a new storage at the specified index. Will destroy the previous one if it had a different capacity
	UE_API bool CreateCacheStorage(uint64 RequestNumberOfBytes, int32 CacheIndex);
	
	// Destroy the specified cache storage
	UE_API void DestroyCacheStorage(int32 CacheIndex);
	
	// Returns 0 if no cache currently exists
	UE_API uint64 GetCacheStorageCapacity(int32 CacheIndex);

	// Returns the path of the mounted cache
	UE_API FString MountCacheStorage(int32 CacheIndex);

	// Requests unmounting for CacheIndex. Actually unmounts if # of UnmountCacheEntry == # of MountCacheStorage for the same index
	UE_API void UnmountCacheStorage(int32 CacheIndex);

	// Returns mount name in cache index is mounted, empty if not mounted
	UE_API FString GetMountName(int32 CacheIndex);

	// Helper to retrieve the mount path. Equivalent to GetCacheStorageMountPath(GetMountName)
	UE_API FString GetMountPath(int32 CacheIndex);

private:
	friend class FIndexedCacheStorageModule;

	void Initialize();
	void EnsureEarlyStartupCache(const TArray<bool>& CacheStorageAlreadyExists);
	FCacheStorageMetaData* GetCacheStorageMetaData(int32 CacheIndex);

	TUniquePtr<struct FIndexedCacheStorageMetaData> IndexedCacheStorageMetaData;
	TMap<FString, int32> StorageNameToStorageIndex;
	bool bHasRegisteredEntries = false;
};

struct FIndexedCacheStorageScopeMount
{
	FIndexedCacheStorageScopeMount(int32 InMountPoint)
	: MountPoint(InMountPoint)
	{
		MountPath = Experimental::FIndexedCacheStorageManager::Get().MountCacheStorage(MountPoint);
	}

	~FIndexedCacheStorageScopeMount()
	{
		Experimental::FIndexedCacheStorageManager::Get().UnmountCacheStorage(MountPoint);
	}

	FString MountPath;
	const int32 MountPoint = 0;
};


}

#undef UE_API
