// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !defined(NO_UE_INCLUDES)
#include <Containers/StringView.h>
#include <Memory/MemoryView.h>
#endif

#if !defined(IAS_HTTP_WITH_PERF)
#	define IAS_HTTP_WITH_PERF 1
#endif

#define UE_API IOSTOREHTTPCLIENT_API

////////////////////////////////////////////////////////////////////////////////
class FIoBuffer;

namespace UE::IoStore::HTTP
{

////////////////////////////////////////////////////////////////////////////////
enum class EHttpVersion : uint8
{
	Default,
	One,		// 1.1 in actuality
	Two,
};

////////////////////////////////////////////////////////////////////////////////
enum class EMimeType
{
	Unknown = 0,
	Text,
	Binary,
	Json,
	Xml,
	CbObject,
	CbPackage,
	CompressedBuffer,
	Count
};

////////////////////////////////////////////////////////////////////////////////
enum class EStatusCodeClass
{
	Informational,
	Successful,
	Redirection,
	ClientError,
	ServerError,
	Unknown,
};

////////////////////////////////////////////////////////////////////////////////
enum class EContentStatus
{
	NoContent,		// response has no content; head, 100-class, 204, 304, etc.
	KnownSize,		// "content-length" present
	UnknownSize,	// no "content-length" field, streamed transfer (e.g. h1's chunked)
};

////////////////////////////////////////////////////////////////////////////////
using	FCertRootsRef	= UPTRINT;
using	FTicket			= uint64;
struct	FActivityNode;
class	FRecvStats;

////////////////////////////////////////////////////////////////////////////////
class FCertRoots
{
public:
								FCertRoots()					= default;
	UE_API 						~FCertRoots();
	UE_API 						FCertRoots(FMemoryView PemData);
								FCertRoots(FCertRoots&& Rhs)	{ *this = MoveTemp(Rhs); }
	FCertRoots&					operator = (FCertRoots&& Rhs)	{ Swap(Handle, Rhs.Handle); return *this; }
	bool						IsValid() const					{ return Handle != 0; }
	UE_API int32				Num() const;
	static UE_API void			SetDefault(FCertRoots&& CertRoots);
	static UE_API FCertRootsRef	NoTls();
	static UE_API FCertRootsRef	Default();
	static UE_API FCertRootsRef	Explicit(const FCertRoots& CertRoots);

private:
	UPTRINT					Handle = 0;

private:
							FCertRoots(const FCertRoots&)	= delete;
	FCertRoots&				operator = (const FCertRoots&)	= delete;
};

////////////////////////////////////////////////////////////////////////////////
class FConnectionPool
{
public:
	struct FParams
	{
		UE_API int32		SetHostFromUrl(FAnsiStringView Url);
		FAnsiStringView		HostName;
		FCertRootsRef		VerifyCert = FCertRoots::NoTls();
		int32				SendBufSize = -1;
		int32				RecvBufSize = -1;
		uint32				Port = 0;
		uint16				ConnectionCount = 1;
		EHttpVersion		HttpVersion = EHttpVersion::One;
		uint8				MaxInflight = 1;
		/*
		enum class ProxyType { Http, Socks4 };
		Proxy = { ip, port, type }
		 */
	};

							FConnectionPool() = default;
	UE_API 					FConnectionPool(const FParams& Params);
	UE_API 					~FConnectionPool();
							FConnectionPool(FConnectionPool&& Rhs)	{ *this = MoveTemp(Rhs); }
	FConnectionPool&		operator = (FConnectionPool&& Rhs)		{ Swap(Ptr, Rhs.Ptr); return *this; }
	UE_API bool				Resolve();
	UE_API uint32			GetIps(uint32* Out, uint32 Count) const;
	UE_API void				Describe(FAnsiStringBuilderBase&) const;

	static UE_API bool		IsValidHostUrl(FAnsiStringView Url);

private:
	friend					class FEventLoop;
	class FHost*			Ptr = nullptr;

private:
							FConnectionPool(const FConnectionPool&) = delete;
	FConnectionPool&		operator = (const FConnectionPool&) = delete;
};

////////////////////////////////////////////////////////////////////////////////
class FRequest
{
public:
	UE_API 				~FRequest();
	UE_API 				FRequest(FRequest&& Rhs);
						FRequest(const FRequest&) = delete;
	UE_API FRequest&	operator = (FRequest&& Rhs);
	FRequest&			operator = (const FRequest&) = delete;
	bool				IsValid() const { return Ptr != nullptr; }
	UE_API void			Accept(EMimeType MimeType);
	UE_API void			Accept(FAnsiStringView MimeType);
	UE_API void			Header(FAnsiStringView Key, FAnsiStringView Value);
	UE_API void			Content(const void* Data, SIZE_T Size, EMimeType MimeType);
	UE_API void			Content(const void* Data, SIZE_T Size, FAnsiStringView MimeType);

private:
	friend				class FEventLoop;
						FRequest() = default;
	FActivityNode*		Ptr = nullptr;
};

////////////////////////////////////////////////////////////////////////////////
class FResponse
{
public:
	using FHeaderSink = TFunction<bool (FAnsiStringView, FAnsiStringView)>;

	UE_API EStatusCodeClass	GetStatus() const;
	UE_API uint32			GetStatusCode() const;
	UE_API FAnsiStringView 	GetStatusMessage() const;
	UE_API EContentStatus	GetContentStatus() const;
	UE_API int64			GetContentLength() const;
	UE_API EMimeType		GetContentType() const;
	UE_API void				GetContentType(FAnsiStringView& Out) const;
	UE_API FAnsiStringView 	GetHeader(FAnsiStringView Name) const;
	UE_API void				ReadHeaders(FHeaderSink Sink) const;
	UE_API void				SetDestination(FIoBuffer* Buffer);

private:
							FResponse() = delete;
							FResponse(const FResponse&) = delete;
							FResponse(FResponse&&) = delete;
	FResponse&				operator = (const FResponse&) = delete;
	FResponse&				operator = (FResponse&&) = delete;
};

////////////////////////////////////////////////////////////////////////////////
class FTicketPerf
{
public:
	struct FSample
	{
		uint16			SendMs;
		uint16			WaitMs;
		uint16			RecvMs;
		uint16			RecvKiBps;
		uint32			GetTotalMs() const		{ return SendMs + WaitMs + RecvMs; }
		uint32			GetSizeEstKiB() const	{ return (RecvKiBps * RecvMs) / 1000; }
	};

	UE_API FSample		GetSample() const;
};

#if !IAS_HTTP_WITH_PERF
inline FTicketPerf::FSample FTicketPerf::GetSample() const { return {}; }
#endif

////////////////////////////////////////////////////////////////////////////////
class FTicketStatus
{
public:
	enum class EId : uint8 { Response, Content, Cancelled, Error };

	struct FError
	{
		const char*				Reason;
		uint32					Code;
	};

	UE_API EId					GetId() const;
	UE_API UPTRINT				GetParam() const;
	UE_API FTicket				GetTicket() const;
	UE_API uint32				GetIndex() const;
	UE_API bool					IsComplete() const;
	UE_API FResponse&			GetResponse() const;		// if GetId() == EId::Response
	UE_API const FIoBuffer&		GetContent() const;			// if GetId() == EId::Content
	UE_API uint32				GetContentLength() const;	//  |
	UE_API const FTicketPerf&	GetPerf() const;			// _|_
	UE_API FError				GetError() const;			// if GetId() == EId::Error

private:
								FTicketStatus() = delete;
								FTicketStatus(const FTicketStatus&) = delete;
								FTicketStatus(FTicketStatus&&) = delete;
	FTicketStatus&				operator = (const FTicketStatus&) = delete;
	FTicketStatus&				operator = (FTicketStatus&&) = delete;
};

using FTicketSink = TFunction<void (const FTicketStatus&)>;

////////////////////////////////////////////////////////////////////////////////
class FLoopPerf
{
public:
	enum { RecvBucketCount = 8 };

	UE_API	uint32	GetTotalRecvKiB() const;
	UE_API	uint32	GetRecvKiBpsCount() const;
	UE_API	uint32	GetRecvKiBps(uint32 Index) const;
	UE_API	void	GetRecvKiBps(uint32 (&Out)[RecvBucketCount]) const;
	UE_API	uint32	GetRecvKiBps() const;
	UE_API	uint32	GetTimeToFirstByteMs() const;

private:
	friend			class FRecvStats;
	uint64			TtfbSamples;
	uint32			RecvKiB;
	uint16			RecvKiBps[RecvBucketCount];
};

////////////////////////////////////////////////////////////////////////////////
class FEventLoop
{
	class FImpl;

public:
	static UE_API const uint32		MaxActiveTickets = 64;

	struct FRequestParams
	{
		FCertRootsRef				VerifyCert		= {};
		uint32						ContentSizeEst	= 0;
		uint16						BufferSize		= 256;
		bool						bAutoRedirect	= false;
		bool						bAllowChunked	= true;
		EHttpVersion				HttpVersion		= EHttpVersion::Default;
	};

	template <typename... T> [[nodiscard]] FRequest Get(T&&... t)  { return Request("GET",  Forward<T&&>(t)...); }
	template <typename... T> [[nodiscard]] FRequest Post(T&&... t) { return Request("POST", Forward<T&&>(t)...); }

	UE_API 							FEventLoop();
	UE_API 							~FEventLoop();
	UE_API void						SamplePerf(FLoopPerf& Perf) const;
	UE_API uint32					Tick(int32 PollTimeoutMs=0);
	UE_API void						Throttle(uint32 KiBPerSec);
	UE_API void						SetFailTimeout(int32 TimeoutMs);
	UE_API bool						IsIdle() const;
	UE_API void						Cancel(FTicket Ticket);
	[[nodiscard]] UE_API FRequest	Request(FAnsiStringView Method, FAnsiStringView Url, const FRequestParams* Params=nullptr);
	[[nodiscard]] UE_API FRequest	Request(FAnsiStringView Method, FAnsiStringView Path, FConnectionPool& Pool, const FRequestParams* Params=nullptr);
	UE_API FTicket					Send(FRequest&& Request, FTicketSink Sink, UPTRINT SinkParam=0);

private:
	bool					Redirect(const FTicketStatus& Status, FTicketSink& OuterSink);
	FImpl*					Impl;

private:
							FEventLoop(const FEventLoop&)	= delete;
							FEventLoop(FEventLoop&&)		= delete;
	FEventLoop&				operator = (const FEventLoop&)	= delete;
	FEventLoop&				operator = (FEventLoop&&)		= delete;
};

} // namespace UE::IoStore::HTTP

#undef UE_API
