// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if __has_include(<Containers/StringView.h>)
#	include <Trace/Trace.h>
#	include <Containers/StringView.h>
#	include <ProfilingDebugging/CpuProfilerTrace.h>
#	include <Templates/UnrealTemplate.h>
#else
#	define UE_LANETRACE_ENABLED		2
#	define IOSTOREHTTPCLIENT_API
#endif

////////////////////////////////////////////////////////////////////////////////
#if !defined(UE_LANETRACE_ENABLED)
#	define UE_LANETRACE_ENABLED CPUPROFILERTRACE_ENABLED && !UE_BUILD_SHIPPING
#endif

#if UE_LANETRACE_ENABLED
#	define LANETRACE_OFF_IMPL(...)
#	define UE_API					IOSTOREHTTPCLIENT_API
#else
#	define LANETRACE_OFF_IMPL(...)	{ return __VA_ARGS__ ; }
#	define UE_API					inline
#endif


////////////////////////////////////////////////////////////////////////////////
#if UE_LANETRACE_ENABLED == 2
namespace trace {
#endif

////////////////////////////////////////////////////////////////////////////////
struct FLaneTraceSpec
{
	FAnsiStringView	Name;
	FAnsiStringView	Group = "Lanes";
	const void*		Channel;
	int32			Weight = 100;
};

class				FLaneTrace;
UE_API FLaneTrace*	LaneTrace_New(const FLaneTraceSpec& Spec)			LANETRACE_OFF_IMPL(nullptr);
UE_API void			LaneTrace_Delete(FLaneTrace* Lane)					LANETRACE_OFF_IMPL();
UE_API uint32		LaneTrace_NewScope(const FAnsiStringView& Name)		LANETRACE_OFF_IMPL(1);
UE_API void			LaneTrace_Enter(FLaneTrace* Lane, uint32 ScopeId)	LANETRACE_OFF_IMPL();
UE_API void			LaneTrace_Change(FLaneTrace* Lane, uint32 ScopeId)	LANETRACE_OFF_IMPL();
UE_API void			LaneTrace_Leave(FLaneTrace* Lane)					LANETRACE_OFF_IMPL();
UE_API void			LaneTrace_LeaveAll(FLaneTrace* Lane)				LANETRACE_OFF_IMPL();



////////////////////////////////////////////////////////////////////////////////
struct FLanePostcode
{
			FLanePostcode(const void* In)	: Value(UPTRINT(In)) {}
			FLanePostcode(UPTRINT In)		: Value(In) {}
	UPTRINT Value;
};

class				FLaneEstate;
UE_API FLaneEstate*	LaneEstate_New(const FLaneTraceSpec& Spec)						LANETRACE_OFF_IMPL(nullptr);
UE_API void			LaneEstate_Delete(FLaneEstate* Estate)							LANETRACE_OFF_IMPL();
UE_API FLaneTrace*	LaneEstate_Build(FLaneEstate* Estate, FLanePostcode Postcode)	LANETRACE_OFF_IMPL(nullptr);
UE_API FLaneTrace*	LaneEstate_Lookup(FLaneEstate* Estate, FLanePostcode Postcode)	LANETRACE_OFF_IMPL(nullptr);
UE_API void			LaneEstate_Demolish(FLaneEstate* Estate, FLanePostcode Postcode)LANETRACE_OFF_IMPL();

#undef LANETRACE_OFF_IMPL
#undef UE_API



#if UE_LANETRACE_ENABLED

////////////////////////////////////////////////////////////////////////////////
class FLaneTraceScope
{
public:
						FLaneTraceScope(FLaneTrace* InLane, uint32 Scope);
						FLaneTraceScope() = default;
						~FLaneTraceScope();
						FLaneTraceScope(FLaneTraceScope&& Rhs);
	FLaneTraceScope&	operator = (FLaneTraceScope&& Rhs);
	void				Change(uint32 Scope) const;

private:
						FLaneTraceScope(const FLaneTraceScope&) = delete;
	FLaneTraceScope&	operator = (const FLaneTraceScope&) = delete;
	FLaneTrace*			Lane = nullptr;
};

////////////////////////////////////////////////////////////////////////////////
inline FLaneTraceScope::FLaneTraceScope(FLaneTrace* InLane, uint32 Scope)
: Lane(InLane)
{
	LaneTrace_Enter(Lane, Scope);
}

////////////////////////////////////////////////////////////////////////////////
inline FLaneTraceScope::~FLaneTraceScope()
{
	if (Lane)
	{
		LaneTrace_Leave(Lane);
	}
}

////////////////////////////////////////////////////////////////////////////////
inline FLaneTraceScope::FLaneTraceScope(FLaneTraceScope&& Rhs)
{
	Swap(Rhs.Lane, Lane);
}

////////////////////////////////////////////////////////////////////////////////
inline FLaneTraceScope& FLaneTraceScope::operator = (FLaneTraceScope&& Rhs)
{
	Swap(Rhs.Lane, Lane);
	return *this;
}

////////////////////////////////////////////////////////////////////////////////
inline void FLaneTraceScope::Change(uint32 Scope) const
{
	LaneTrace_Change(Lane, Scope);
}

#else // UE_LANETRACE_ENABLED

class FLaneTraceScope
{
public:
			FLaneTraceScope(...)	{}
	void	Change(uint32) const	{}
};

#endif // UE_LANETRACE_ENABLED

////////////////////////////////////////////////////////////////////////////////
#if UE_LANETRACE_ENABLED == 2
} // namespace trace
#endif

#if defined(TRACE_IMPLEMENT) && TRACE_IMPLEMENT
#	include "lane_trace.inl"
#endif
