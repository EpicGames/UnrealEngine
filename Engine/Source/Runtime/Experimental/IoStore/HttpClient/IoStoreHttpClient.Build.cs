// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IoStoreHttpClient : ModuleRules
{
	public IoStoreHttpClient(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.Add("Core");
		PublicDependencyModuleNames.Add("TraceLog");

		PrivateDependencyModuleNames.Add("OpenSSL");
		PrivateDefinitions.Add("IAS_HTTP_HAS_OPENSSL=1");

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error; 

		PrivateDependencyModuleNames.Add("nghttp2");
	}
}
