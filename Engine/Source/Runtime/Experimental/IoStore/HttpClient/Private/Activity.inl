// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::IoStore::HTTP
{

#if IAS_HTTP_WITH_PERF

////////////////////////////////////////////////////////////////////////////////
class FStopwatch
{
public:
	uint64	GetInterval(uint32 i) const;
	void	SendStart()		{ Impl(0); }
	void	SendEnd()		{ Impl(1); }
	void	RecvStart()		{ Impl(2); }
	void	RecvEnd()		{ Impl(3); }

private:
	void	Impl(uint32 Index);
	uint64	Samples[4] = {};
	uint32	Counts[2] = {};
};

////////////////////////////////////////////////////////////////////////////////
uint64 FStopwatch::GetInterval(uint32 i) const
{
	if (i >= UE_ARRAY_COUNT(Samples) - 1)
	{
		return 0;
	}
	return Samples[i + 1] - Samples[i];
}

////////////////////////////////////////////////////////////////////////////////
void FStopwatch::Impl(uint32 Index)
{
	if (uint64& Out = Samples[Index]; Out == 0)
	{
		Out = FPlatformTime::Cycles64();
	}
	Counts[Index >> 1] += !(Index & 1);
}

#endif // IAS_HTTP_WITH_PERF



////////////////////////////////////////////////////////////////////////////////
static void Trace(const struct FActivity*, ETrace, uint32);

static FLaneEstate* GActivityTraceEstate = LaneEstate_New({
	.Name = "Iax/Activity",
	.Group = "Iax",
	.Channel = GetIaxTraceChannel(),
	.Weight = 11,
});



////////////////////////////////////////////////////////////////////////////////
struct FActivity
{
	struct FParams
	{
		FAnsiStringView	Method;
		FAnsiStringView	Path;
		FHost*			Host = nullptr;
		char*			Buffer = nullptr;
		uint32			ContentSizeEst = 0;
		uint16			BufferSize = 0;
		bool			bIsKeepAlive = false;
		bool			bAllowChunked = true;
	};

	enum class EStage : int32
	{
		Build			= -1,
		Request			= -2,
		Response		= -3,
		Content			= -4,
		Done			= -5,
		Cancelled		= -6,
		Failed			= -7,
	};

						FActivity(const FParams& Params);
	void				AddHeader(FAnsiStringView Key, FAnsiStringView Value);
	FOutcome			Tick(FHttpPeer& Peer, int32* MaxRecvSize=nullptr);
	void				SetSink(FTicketSink&& InSink, UPTRINT Param);
	void				SetDestination(FIoBuffer* InDest);
	bool				CancelTransaction(FHttpPeer& Peer);
	void				Cancel();
	void				Done();
	void				Fail(const FOutcome& Outcome);
	EStage				GetStage() const;
	FTransactId			GetTransactId() const;
	const FTransactRef&	GetTransaction() const;
	FAnsiStringView		GetMethod() const;
	FAnsiStringView		GetPath() const;
	FHost*				GetHost() const;
	void				EnumerateHeaders(FResponse::FHeaderSink HeaderSink) const;
	UPTRINT				GetSinkParam() const;
	FIoBuffer&			GetContent() const;
	uint32				GetRemainingKiB() const;
	FOutcome			GetError() const;

#if IAS_HTTP_WITH_PERF
	const FStopwatch&	GetStopwatch() const;
#endif

private:
	enum class EState : uint8
	{
		Build,
		Send,
		RecvMessage,
		RecvStream,
		RecvContent,
		RecvDone,
		Completed,
		Cancelled,
		Failed,
		_Num,
	};

	void				CallSink();
	void				ChangeState(EState InState);
	FOutcome			Transact(FHttpPeer& Peer);
	FOutcome			Send(FHttpPeer& Peer);
	FOutcome			RecvMessage(FHttpPeer& Peer);
	FOutcome			RecvContent(FHttpPeer& Peer, int32& MaxRecvSize);
	FOutcome			RecvStream(FHttpPeer& Peer, int32& MaxRecvSize);
	FOutcome			Recv(FHttpPeer& Peer, int32& MaxRecvSize);

	EState				State = EState::Build;
	uint8				bAllowChunked : 1;
	uint8				_Unused : 3;
	uint8				LengthScore : 4;
	uint8				MethodOffset;
	uint8				PathOffset;
	uint16				PathLength;
	uint16				HeaderCount = 0;
	FTransactId			TransactId = 0;
#if IAS_HTTP_WITH_PERF
	FStopwatch			Stopwatch;
#endif
	union {
		FHost*			Host;
		FIoBuffer*		Dest;
		UPTRINT			Error;
	};
	UPTRINT				SinkParam;
	FTicketSink			Sink;
	FTransactRef		Transaction;
	FBuffer				Buffer;

	struct FHeaderRecord
	{
		int16			Key;
		uint16			ValueLength;
		char			Data[];
	};

	friend void Trace(const FActivity*, ETrace, uint32);

	UE_NONCOPYABLE(FActivity);
};

////////////////////////////////////////////////////////////////////////////////
FActivity::FActivity(const FParams& Params)
: bAllowChunked(Params.bAllowChunked)
, Host(Params.Host)
, Buffer(Params.Buffer, Params.BufferSize)
{
	check(Host != nullptr);

	// Make a copy of data.
	FAnsiStringView Path = Params.Path.IsEmpty() ? "/" : Params.Path;
	check(Path[0] == '/');

	auto Copy = [this] (FAnsiStringView Value)
	{
		uint32 Length = uint32(Value.Len());
		char* Ptr = Buffer.Alloc<char>(Length);
		std::memcpy(Ptr, Value.GetData(), Length);
		uint32 Ret = uint32(ptrdiff_t(Ptr - Buffer.GetData()));
		check(Ret <= 255);
		return uint8(Ret);
	};

	MethodOffset = Copy(Params.Method);
	PathOffset = Copy(Path);
	PathLength = uint16(Path.Len());

	// Calculate a length score.
	uint32 ContentEstKiB = (Params.ContentSizeEst + 1023) >> 10;
	uint32 Pow2 = FMath::FloorLog2(uint32(ContentEstKiB));
	Pow2 = FMath::Min(Pow2, 15u);
	LengthScore = uint8(Pow2);
}

////////////////////////////////////////////////////////////////////////////////
void FActivity::AddHeader(FAnsiStringView Key, FAnsiStringView Value)
{
	check(State == EState::Build);

	check(HeaderCount < 0xffff);
	HeaderCount++;

	static_assert(alignof(FHeaderRecord) == alignof(uint16));
	uint32 Count = sizeof(FHeaderRecord) + Key.Len() + Value.Len();
	Count = (Count + sizeof(uint16) - 1) / sizeof(uint16);
	auto* Record = (FHeaderRecord*)(Buffer.Alloc<uint16>(Count));

	Record->Key = int16(Key.Len());
	Record->ValueLength = int16(Value.Len());

	char* Cursor = Record->Data;
	std::memcpy(Cursor, Key.GetData(), Key.Len());
	std::memcpy(Cursor + Key.Len(), Value.GetData(), Value.Len());
}

////////////////////////////////////////////////////////////////////////////////
void FActivity::SetSink(FTicketSink&& InSink, UPTRINT Param)
{
	Sink = MoveTemp(InSink);
	SinkParam = Param;
}

////////////////////////////////////////////////////////////////////////////////
void FActivity::SetDestination(FIoBuffer* InDest)
{
	Dest = InDest;
}

////////////////////////////////////////////////////////////////////////////////
bool FActivity::CancelTransaction(FHttpPeer& Peer)
{
	if (TransactId == 0)
	{
		return true;
	}

	bool Result = Peer.Cancel(TransactId);
	return Result;
}

////////////////////////////////////////////////////////////////////////////////
void FActivity::Cancel()
{
	if (State >= EState::Completed)
	{
		return;
	}

	ChangeState(EState::Cancelled);
	CallSink();
}

////////////////////////////////////////////////////////////////////////////////
void FActivity::Done()
{
	ChangeState(EState::RecvDone);
	CallSink();
	ChangeState(EState::Completed);
}

////////////////////////////////////////////////////////////////////////////////
void FActivity::Fail(const FOutcome& Outcome)
{
	if (State == EState::Failed)
	{
		return;
	}

	static_assert(sizeof(Outcome) == sizeof(Error));
	std::memcpy(&Error, &Outcome, sizeof(Error));

	ChangeState(EState::Failed);
	CallSink();
}

////////////////////////////////////////////////////////////////////////////////
void FActivity::ChangeState(EState InState)
{
	Trace(this, ETrace::StateChange, uint32(InState));

	check(State != InState);
	State = InState;
}

////////////////////////////////////////////////////////////////////////////////
void FActivity::CallSink()
{
	static uint32 Scope = LaneTrace_NewScope("Iax/Sink");
	FLaneTrace* Lane = LaneEstate_Lookup(GActivityTraceEstate, this);
	FLaneTraceScope _(Lane, Scope);

	FTicketStatus& SinkArg = *(FTicketStatus*)this;
	Sink(SinkArg);
}

////////////////////////////////////////////////////////////////////////////////
FActivity::EStage FActivity::GetStage() const
{
	switch (State)
	{
	case EState::Build:			return EStage::Build;
	case EState::Send:
	case EState::RecvMessage:	return EStage::Response;
	case EState::RecvContent:
	case EState::RecvStream:
	case EState::RecvDone:		return EStage::Content;
	case EState::Completed:		return EStage::Done;
	case EState::Cancelled:		return EStage::Cancelled;
	case EState::Failed:		return EStage::Failed;
	default:					break;
	}
	return EStage::Failed;
}

////////////////////////////////////////////////////////////////////////////////
FTransactId FActivity::GetTransactId() const
{
	check(TransactId != 0);
	return TransactId;
}

////////////////////////////////////////////////////////////////////////////////
const FTransactRef& FActivity::GetTransaction() const
{
	check(State >= EState::Send && State <= EState::Completed);
	check(Transaction.IsValid());
	return Transaction;
}

////////////////////////////////////////////////////////////////////////////////
FAnsiStringView FActivity::GetMethod() const
{
	return { Buffer.GetData() + MethodOffset, PathOffset - MethodOffset };
}

////////////////////////////////////////////////////////////////////////////////
FAnsiStringView FActivity::GetPath() const
{
	return { Buffer.GetData() + PathOffset, PathLength };
}

////////////////////////////////////////////////////////////////////////////////
FHost* FActivity::GetHost() const
{
	check(State <= EState::RecvMessage);
	return Host;
}

////////////////////////////////////////////////////////////////////////////////
void FActivity::EnumerateHeaders(FResponse::FHeaderSink HeaderSink) const
{
	UPTRINT RecordAddr = uint32(PathOffset) + uint32(PathLength);
	for (uint32 i = 0, n = HeaderCount; i < n; ++i)
	{
		RecordAddr = (RecordAddr + alignof(FHeaderRecord) - 1) & ~(alignof(FHeaderRecord) - 1);
		const auto* Record = (FHeaderRecord*)(Buffer.GetData() + RecordAddr);

		FAnsiStringView Key(Record->Data, Record->Key);
		FAnsiStringView Value(Record->Data + Record->Key, Record->ValueLength);
		HeaderSink(Key, Value);

		RecordAddr += sizeof(FHeaderRecord) + Record->Key + Record->ValueLength;
	}
}

////////////////////////////////////////////////////////////////////////////////
UPTRINT FActivity::GetSinkParam() const
{
	return SinkParam;
}

////////////////////////////////////////////////////////////////////////////////
FIoBuffer& FActivity::GetContent() const
{
	check(State == EState::RecvContent || State == EState::RecvStream || State == EState::RecvDone);
	return *Dest;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FActivity::GetRemainingKiB() const
{
	if (State <= EState::RecvStream)  return MAX_uint32;
	if (State >  EState::RecvContent) return 0;
	return uint32(GetTransaction()->GetRemaining() >> 10);
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FActivity::GetError() const
{
	FOutcome Outcome = FOutcome::None();
	void* Ptr = &Outcome;
	std::memcpy(Ptr, &Error, sizeof(Outcome));
	return Outcome;
}

////////////////////////////////////////////////////////////////////////////////
#if IAS_HTTP_WITH_PERF
const FStopwatch& FActivity::GetStopwatch() const
{
	return Stopwatch;
}
#endif // IAS_HTTP_WITH_PERF

////////////////////////////////////////////////////////////////////////////////
FOutcome FActivity::Transact(FHttpPeer& Peer)
{
	Transaction = Peer.Transact();
	if (!Transaction.IsValid())
	{
		return FOutcome::Error("Unable to create a suitable transaction");
	}

	const FHost* TheHost = GetHost();
	bool bKeepAlive = TheHost->IsPooled();

	Transaction->Begin(TheHost->GetHostName(), GetMethod(), GetPath());
	EnumerateHeaders([this] (FAnsiStringView Key, FAnsiStringView Value)
	{
		Transaction->AddHeader(Key, Value);
		return true;
	});
	TransactId = Transaction->End(bKeepAlive);

	ChangeState(EState::Send);
	return FOutcome::Ok();
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FActivity::Send(FHttpPeer& Peer)
{
#if IAS_HTTP_WITH_PERF
	Stopwatch.SendStart();
#endif

	TRACE_CPUPROFILER_EVENT_SCOPE(IasHttp::DoSend);

	FOutcome Outcome = Transaction->TrySendRequest(Peer);
	if (Outcome.IsError())
	{
		Fail(Outcome);
		return Outcome;
	}

	if (Outcome.IsWaiting())
	{
		return Outcome;
	}

	check(Outcome.IsOk());

#if IAS_HTTP_WITH_PERF
	Stopwatch.SendEnd();
#endif

	ChangeState(EState::RecvMessage);
	return FOutcome::Ok(int32(EStage::Request));
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FActivity::RecvMessage(FHttpPeer& Peer)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IasHttp::DoRecvMessage);
	Trace(this, ETrace::StateChange, uint32(State));

#if IAS_HTTP_WITH_PERF
	Stopwatch.RecvStart();
#endif

	FOutcome Outcome = Transaction->TryRecvResponse(Peer);
	if (Outcome.IsError())
	{
		Fail(Outcome);
		return Outcome;
	}

	if (Outcome.IsWaiting())
	{
		return Outcome;
	}

	check(Outcome.IsOk());

	bool bChunked = Transaction->IsChunked();
	int64 ContentLength = Transaction->GetContentLength();

	// Validate that the server's told us how and how much it will transmit
	if (bChunked)
	{
		if (bAllowChunked == 0)
		{
			Outcome = FOutcome::Error("Chunked transfer encoding disabled (ERRNOCHUNK)");
			Fail(Outcome);
			return Outcome;
		}
	}
	else if (ContentLength < 0)
	{
		Outcome = FOutcome::Error("Invalid content length");
		Fail(Outcome);
		return Outcome;
	}

	// Call out to the sink to get a content destination
	FIoBuffer* PriorDest = Dest; // to retain unioned Host ptr (redirect uses it in sink)
	CallSink();

	// HEAD methods
	bool bHasBody = !GetMethod().Equals("HEAD", ESearchCase::IgnoreCase);
	bHasBody &= ((ContentLength > 0) | int32(Transaction->IsChunked())) == true;
	if (!bHasBody)
	{
		ChangeState(EState::RecvDone);
		return FOutcome::Ok(int32(EStage::Content));
	}

	// Check the user gave us a destination for content
	if (Dest == PriorDest)
	{
		Outcome = FOutcome::Error("User did not provide a destination buffer");
		Fail(Outcome);
		return Outcome;
	}

	// The user seems to have forgotten something. Let's help them along
	if (int32 DestSize = int32(Dest->GetSize()); DestSize == 0)
	{
		static const uint32 DefaultChunkSize = 4 << 10;
		uint32 Size = bChunked ? DefaultChunkSize : uint32(ContentLength);
		*Dest = FIoBuffer(Size);
	}
	else if (!bChunked && DestSize < ContentLength)
	{
		// Destination is too small for all the content. The user did however
		// provide a size so we'll use that to deliver content piecewise.
		bChunked = true;
	}
	else if (enum { MinStreamBuf = 256 }; bChunked && DestSize < MinStreamBuf)
	{
		*Dest = FIoBuffer(MinStreamBuf);
	}

	// We're all set to go and get content
	check(Dest != nullptr);
	auto NextState = bChunked ? EState::RecvStream : EState::RecvContent;
	ChangeState(NextState);
	return FOutcome::Ok(int32(EStage::Response));
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FActivity::RecvContent(FHttpPeer& Peer, int32& MaxRecvSize)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IasHttp::DoRecvContent);

	int64 Remaining = Transaction->GetRemaining();

	FMutableMemoryView View = Dest->GetMutableView();
	View = View.Right(Remaining).Left(MaxRecvSize);

	FOutcome Outcome = Transaction->TryRecv(View, Peer);
	if (Outcome.IsError())
	{
		Fail(Outcome);
		return Outcome;
	}

	MaxRecvSize -= Outcome.GetResult();

	if (Outcome.IsWaiting())
	{
		return Outcome;
	}

#if IAS_HTTP_WITH_PERF
	Stopwatch.RecvEnd();
#endif

	Done();
	return FOutcome::Ok(int32(EStage::Content));
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FActivity::RecvStream(FHttpPeer& Peer, int32& MaxRecvSize)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IasHttp::DoRecvStream);

	FMutableMemoryView View = Dest->GetMutableView();
	View = View.Left(MaxRecvSize);
	MaxRecvSize -= int32(View.GetSize());

	FOutcome Outcome = Transaction->TryRecv(View, Peer);
	if (Outcome.IsError())
	{
		Fail(Outcome);
		return Outcome;
	}

	int32 Result = Outcome.GetResult();

	if (Result != 0)
	{
		// Temporarily clamp IoBuffer so if the sink does GetView/GetSize() it
		// represents actual content and not the underlying working buffer.
		FIoBuffer& Outer = *Dest;

		FMemoryView SliceView = Outer.GetView();
		SliceView = SliceView.Left(Result);
		FIoBuffer Slice(SliceView, Outer);

		Swap(Outer, Slice);
		CallSink();
		Swap(Outer, Slice);
	}

	if (!Outcome.IsOk())
	{
		check(Outcome.IsWaiting());
		return Outcome;
	}

#if IAS_HTTP_WITH_PERF
	Stopwatch.RecvEnd();
#endif

	*Dest = FIoBuffer();

	Done();
	return FOutcome::Ok(int32(EStage::Content));
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FActivity::Recv(FHttpPeer& Peer, int32& MaxRecvSize)
{
	check(State >= EState::RecvMessage && State < EState::RecvDone);

	if (State == EState::RecvMessage)	return RecvMessage(Peer);
	if (State == EState::RecvContent)	return RecvContent(Peer, MaxRecvSize);
	if (State == EState::RecvStream)	return RecvStream(Peer, MaxRecvSize); //-V547
	
	check(false); // it is not expected that we'll get here
	return FOutcome::Error("unreachable");
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FActivity::Tick(FHttpPeer& Peer, int32* MaxRecvSize)
{
	if (State == EState::Build)
	{
		FOutcome Outcome = Transact(Peer);
		if (!Outcome.IsOk())
		{
			return Outcome;
		}
	}

	if (State == EState::Send)
	{
		return Send(Peer);
	}

	check(State > EState::Send && State < EState::RecvDone);
	check(MaxRecvSize != nullptr);
	return Recv(Peer, *MaxRecvSize);
}



////////////////////////////////////////////////////////////////////////////////
static void Trace(const struct FActivity* Activity, ETrace Action, uint32 Param)
{
	if (Action == ETrace::ActivityCreate)
	{
		static uint32 ActScopes[16] = {};
		if (ActScopes[0] == 0)
		{
			ActScopes[0] = LaneTrace_NewScope("Iax/Activity");

			TAnsiStringBuilder<32> Builder;
			for (int32 i = 1, n = UE_ARRAY_COUNT(ActScopes); i < n; ++i)
			{
				Builder.Reset();
				Builder << "Iax/Activity_";
				Builder << (1ull << (i - 1));
				ActScopes[i] = LaneTrace_NewScope(Builder);
			}
		}

		FLaneTrace* Lane = LaneEstate_Build(GActivityTraceEstate, Activity);
		LaneTrace_Enter(Lane, ActScopes[Activity->LengthScore]);
		return;
	}

	if (Action == ETrace::ActivityDestroy)
	{
		LaneEstate_Demolish(GActivityTraceEstate, Activity);
		return;
	}

	FLaneTrace* Lane = LaneEstate_Lookup(GActivityTraceEstate, Activity);

	if (Action == ETrace::StateChange)
	{
		static constexpr FAnsiStringView StateNames[] = {
			"Iax/Build",
			"Iax/WaitForSocket",
			"Iax/WaitResponse",
			"Iax/RecvStream",
			"Iax/RecvContent",
			"Iax/RecvDone",
			"Iax/Completed",
			"Iax/Cancelled",
			"Iax/Failed",
		};
		static_assert(UE_ARRAY_COUNT(StateNames) == uint32(FActivity::EState::_Num));
		static uint32 StateScopes[UE_ARRAY_COUNT(StateNames)] = {};
		if (StateScopes[0] == 0)
		{
			for (int32 i = 0; FAnsiStringView Name : StateNames)
			{
				StateScopes[i++] = LaneTrace_NewScope(Name);
			}
		}

		uint32 Scope = StateScopes[Param];
		if (Param == uint32(FActivity::EState::Build))
		{
			LaneTrace_Enter(Lane, Scope);
		}
		else
		{
			LaneTrace_Change(Lane, Scope);
		}

		return;
	}
}

} // namespace UE::IoStore::HTTP
