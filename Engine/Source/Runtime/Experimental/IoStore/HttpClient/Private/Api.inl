// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::IoStore::HTTP
{

// {{{1 request ................................................................

////////////////////////////////////////////////////////////////////////////////
static FAnsiStringView MimeTypeToString(EMimeType MimeType)
{
	switch (MimeType)
	{
	case EMimeType::Text:				return "text/html";
	case EMimeType::Binary:				return "application/octet-stream";
	case EMimeType::Json:				return "application/json";
	case EMimeType::Xml:				return "application/xml";
	case EMimeType::CbObject:			return "application/x-ue-cb";
	case EMimeType::CbPackage:			return "application/x-ue-pkg";
	case EMimeType::CompressedBuffer:	return "application/x-ue-comp";
	}
	return {};
}

////////////////////////////////////////////////////////////////////////////////
FRequest::FRequest(FRequest&& Rhs)
{
	Swap(Ptr, Rhs.Ptr);
}

////////////////////////////////////////////////////////////////////////////////
FRequest& FRequest::operator = (FRequest&& Rhs)
{
	Swap(Ptr, Rhs.Ptr);
	return *this;
}

////////////////////////////////////////////////////////////////////////////////
FRequest::~FRequest()
{
	if (Ptr == nullptr)
	{
		return;
	}

	FActivityNode::Destroy(Ptr);
}

////////////////////////////////////////////////////////////////////////////////
void FRequest::Accept(EMimeType MimeType)
{
	FAnsiStringView AsString = MimeTypeToString(MimeType);
	Accept(AsString);
}

////////////////////////////////////////////////////////////////////////////////
void FRequest::Accept(FAnsiStringView MimeType)
{
	if (!MimeType.IsEmpty())
	{
		Header("Accept", MimeType);
	}
}

////////////////////////////////////////////////////////////////////////////////
void FRequest::Header(FAnsiStringView Key, FAnsiStringView Value)
{
	Ptr->AddHeader(Key, Value);
}



// {{{1 response ...............................................................

////////////////////////////////////////////////////////////////////////////////
EStatusCodeClass FResponse::GetStatus() const
{
	uint32 Code = GetStatusCode();
	if (Code <= 199) return EStatusCodeClass::Informational;
	if (Code <= 299) return EStatusCodeClass::Successful;
	if (Code <= 399) return EStatusCodeClass::Redirection;
	if (Code <= 499) return EStatusCodeClass::ClientError;
	if (Code <= 599) return EStatusCodeClass::ServerError;
	return EStatusCodeClass::Unknown;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FResponse::GetStatusCode() const
{
	const auto* Activity = (const FActivity*)this;
	return Activity->GetTransaction()->GetStatusCode();
}

////////////////////////////////////////////////////////////////////////////////
FAnsiStringView FResponse::GetStatusMessage() const
{
	const auto* Activity = (const FActivity*)this;
	return Activity->GetTransaction()->GetStatusMessage();
}

////////////////////////////////////////////////////////////////////////////////
EContentStatus FResponse::GetContentStatus() const
{
	switch (GetContentLength())
	{
	case -1:	return EContentStatus::UnknownSize;
	case 0:		return EContentStatus::NoContent;
	default:	return EContentStatus::KnownSize;
	}
}

////////////////////////////////////////////////////////////////////////////////
int64 FResponse::GetContentLength() const
{
	const auto* Activity = (const FActivity*)this;
	return Activity->GetTransaction()->GetContentLength();
}

////////////////////////////////////////////////////////////////////////////////
EMimeType FResponse::GetContentType() const
{
	FAnsiStringView Value;
	GetContentType(Value);

	if (Value == "text/html")					return EMimeType::Text;
	if (Value == "application/octet-stream")	return EMimeType::Binary;
	if (Value == "application/json")			return EMimeType::Json;
	if (Value == "application/xml")				return EMimeType::Xml;
	if (Value == "application/x-ue-cb")			return EMimeType::CbObject;
	if (Value == "application/x-ue-pkg")		return EMimeType::CbPackage;
	if (Value == "application/x-ue-comp")		return EMimeType::CompressedBuffer;

	return EMimeType::Unknown;
}

////////////////////////////////////////////////////////////////////////////////
void FResponse::GetContentType(FAnsiStringView& Out) const
{
	Out = GetHeader("Content-Type");

	if (int32 SemiColon; Out.FindChar(';', SemiColon))
	{
		Out = Out.Left(SemiColon).TrimEnd();
	}
}

////////////////////////////////////////////////////////////////////////////////
FAnsiStringView FResponse::GetHeader(FAnsiStringView Name) const
{
	FAnsiStringView Result;
	ReadHeaders([&Result, Name] (FAnsiStringView Candidate, FAnsiStringView Value)
	{
		if (Candidate != Name)
		{
			return true;
		}

		Result = Value;
		return false;
	});
	return Result;
}

////////////////////////////////////////////////////////////////////////////////
void FResponse::ReadHeaders(FHeaderSink Sink) const
{
	const auto* Activity = (const FActivity*)this;
	Activity->GetTransaction()->ReadHeaders(Sink);
}

////////////////////////////////////////////////////////////////////////////////
void FResponse::SetDestination(FIoBuffer* Buffer)
{
	auto* Activity = (FActivity*)this;
	Activity->SetDestination(Buffer);
}



// {{{1 ticket-status ..........................................................

////////////////////////////////////////////////////////////////////////////////
FTicketStatus::EId FTicketStatus::GetId() const
{
	const auto* Activity = (FActivity*)this;
	switch (Activity->GetStage())
	{
	case FActivity::EStage::Response:	return EId::Response;
	case FActivity::EStage::Content:	return EId::Content;
	case FActivity::EStage::Cancelled:	return EId::Cancelled;
	case FActivity::EStage::Failed:		return EId::Error;
	default:							break;
	}
	return EId::Error;
}

////////////////////////////////////////////////////////////////////////////////
UPTRINT FTicketStatus::GetParam() const
{
	const auto* Activity = (FActivity*)this;
	return Activity->GetSinkParam();
}

////////////////////////////////////////////////////////////////////////////////
FTicket FTicketStatus::GetTicket() const
{
	const auto* Activity = (FActivityNode*)this;
	return 1ull << Activity->Slot;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FTicketStatus::GetIndex() const
{
	const auto* Activity = (FActivityNode*)this;
	return Activity->Slot;
}

////////////////////////////////////////////////////////////////////////////////
bool FTicketStatus::IsComplete() const
{
	EId Id = GetId();
	if (Id == EId::Response)
	{
		// If the response has no body there will be no further status updates
		return GetResponse().GetContentStatus() == EContentStatus::NoContent;
	}

	if (Id == EId::Content)
	{
		const auto* Activity = (FActivity*)this;

		// When the size of content is unknown then the end is signalled with
		// an empty destination buffer. Otherwise a status update is only
		// called when all the destination has been received.
		int64 ContentLength = Activity->GetTransaction()->GetContentLength();
		ContentLength = FMath::Max(ContentLength, 0ll);
		return GetContent().GetSize() == ContentLength;
	}

	return true;
}

////////////////////////////////////////////////////////////////////////////////
FResponse& FTicketStatus::GetResponse() const
{
	check(GetId() < EId::Content);
	const auto* Activity = (FActivity*)this;
	return *(FResponse*)Activity;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FTicketStatus::GetContentLength() const
{
	check(GetId() <= EId::Content);
	const auto* Activity = (FActivity*)this;
	return uint32(Activity->GetTransaction()->GetContentLength());
}

////////////////////////////////////////////////////////////////////////////////
const FTicketPerf& FTicketStatus::GetPerf() const
{
	check(GetId() == EId::Content);
	const auto* Activity = (FActivity*)this;
	return *(FTicketPerf*)Activity;
}

////////////////////////////////////////////////////////////////////////////////
const FIoBuffer& FTicketStatus::GetContent() const
{
	check(GetId() == EId::Content);
	const auto* Activity = (FActivity*)this;
	return Activity->GetContent();
}

////////////////////////////////////////////////////////////////////////////////
FTicketStatus::FError FTicketStatus::GetError() const
{
	check(GetId() == EId::Error);
	const auto* Activity = (FActivity*)this;
	FOutcome Error = Activity->GetError();
	return { Error.GetMessage().GetData(), uint32(Error.GetErrorCode()) };
}



// {{{1 perf ...................................................................

#if IAS_HTTP_WITH_PERF

////////////////////////////////////////////////////////////////////////////////
FTicketPerf::FSample FTicketPerf::GetSample() const
{
	const auto& Activity = *(FActivity*)this;

	static uint64 Freq;
	if (Freq == 0)
	{
		Freq = uint64(1.0 / FPlatformTime::GetSecondsPerCycle64());
	}

	auto Clamp = [] (auto Value) { return uint16(FMath::Min(uint32(Value), 0xffffu)); };
	auto ToMs = [&] (uint64 Value) { return Clamp((Value * 1000ull) / Freq); };

	const FStopwatch& Stopwatch = Activity.GetStopwatch();
	FSample Sample = {
		ToMs(Stopwatch.GetInterval(0)),
		ToMs(Stopwatch.GetInterval(1)),
		ToMs(Stopwatch.GetInterval(2)),
	};

	uint32 Bps = ~0u;
	if (Sample.RecvMs)
	{
		Bps = uint32((Activity.GetTransaction()->GetContentLength() * 1000ull) / Sample.RecvMs);
	}
	Sample.RecvKiBps = Clamp(Bps >> 10);

	return Sample;
}

#endif // IAS_HTTP_WITH_PERF



// }}}

} // namespace UE::IoStore::HTTP
