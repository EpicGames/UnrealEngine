// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::IoStore::HTTP
{

////////////////////////////////////////////////////////////////////////////////
class FHost
{
public:
	enum class EDirection : uint8 { Send, Recv };
	static const uint32 InvalidIp = 0x00ff'ffff;

	struct FParams
	{
		const ANSICHAR* HostName;
		uint32			Port = 0;
		uint16			MaxConnections = 1;
		bool			bPooled = false;
		uint8			MaxInflight = 1;
		EHttpVersion	HttpVersion = EHttpVersion::One;
		FCertRootsRef	VerifyCert = {};
	};

					FHost(const FParams& Params);
	void			SetBufferSize(EDirection Dir, int32 Size);
	int32			GetBufferSize(EDirection Dir) const;
	FOutcome		Connect(FSocket& Socket);
	int32			IsResolved() const;
	FOutcome		ResolveHostName();
	FCertRootsRef	GetVerifyCert() const		{ return VerifyCert; }
	uint32			GetMaxConnections() const	{ return MaxConnections; }
	uint32			GetMaxInflight() const		{ return MaxInflight; }
	EHttpVersion	GetHttpVersion() const		{ return HttpVersion; }
	bool			IsPooled() const			{ return bPooled; }
	uint32			GetIpAddress() const		{ return IpAddresses[0]; }
	FAnsiStringView	GetHostName() const			{ return HostName; }
	uint32			GetPort() const				{ return Port; }

private:
	FCertRootsRef	VerifyCert;
	const ANSICHAR*	HostName;
	uint32			IpAddresses[4] = {};
	int16			SendBufKb = -1;
	int16			RecvBufKb = -1;
	uint16			Port;
	uint8			MaxConnections;
	uint8			MaxInflight;
	EHttpVersion	HttpVersion;
	bool			bPooled;
};

////////////////////////////////////////////////////////////////////////////////
FHost::FHost(const FParams& Params)
: VerifyCert(Params.VerifyCert)
, HostName(Params.HostName)
, Port(uint16(Params.Port))
, MaxConnections(uint8(Params.MaxConnections))
, MaxInflight(Params.MaxInflight)
, HttpVersion(Params.HttpVersion)
, bPooled(Params.bPooled)
{
	check(MaxConnections && MaxConnections == Params.MaxConnections);

	if (Port == 0)
	{
		Port = (VerifyCert == ECertRootsRefType::None) ? 80 : 443;
	}
}

////////////////////////////////////////////////////////////////////////////////
void FHost::SetBufferSize(EDirection Dir, int32 Size)
{
	(Dir == EDirection::Send ? SendBufKb : RecvBufKb) = uint16(Size >> 10);
}

////////////////////////////////////////////////////////////////////////////////
int32 FHost::GetBufferSize(EDirection Dir) const
{
	return int32((Dir == EDirection::Send) ? SendBufKb : RecvBufKb) << 10;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FHost::ResolveHostName()
{
	// todo: GetAddrInfoW() for async resolve on Windows

	TRACE_CPUPROFILER_EVENT_SCOPE(IasHttp::PoolResolve);

	IpAddresses[0] = 1;

	addrinfo* Info = nullptr;
	ON_SCOPE_EXIT { if (Info != nullptr) Socket_FreeAddrInfo(Info); };

	addrinfo Hints = {};
	Hints.ai_family = AF_INET;
	Hints.ai_socktype = SOCK_STREAM;
	Hints.ai_protocol = IPPROTO_TCP;
	auto Result = Socket_GetAddrInfo(HostName, nullptr, &Hints, &Info);
	if (uint32(Result) || Info == nullptr)
	{
		return FOutcome::Error("Error encountered resolving");
	}

	if (Info->ai_family != AF_INET)
	{
		return FOutcome::Error("Unexpected address family during resolve");
	}

	uint32 AddressCount = 0;
	for (const addrinfo* Cursor = Info; Cursor != nullptr; Cursor = Cursor->ai_next)
	{
		const auto* AddrInet = (sockaddr_in*)(Cursor->ai_addr);
		if (AddrInet->sin_family != AF_INET)
		{
			continue;
		}

		uint32 IpAddress = 0;
		memcpy(&IpAddress, &(AddrInet->sin_addr), sizeof(uint32));

		if (IpAddress == 0)
		{
			break;
		}

		IpAddresses[AddressCount] = Socket_HtoNl(IpAddress);
		if (++AddressCount >= UE_ARRAY_COUNT(IpAddresses))
		{
			break;
		}
	}

	if (AddressCount > 0)
	{
		return FOutcome::Ok(AddressCount);
	}

	return FOutcome::Error("Unable to resolve host");
}

////////////////////////////////////////////////////////////////////////////////
int32 FHost::IsResolved() const
{
	switch (IpAddresses[0])
	{
	case 0:  return 0;
	case 1:  return -1;
	default: return 1;
	}
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FHost::Connect(FSocket& Socket)
{
	check(Socket.IsValid());

	FOutcome Outcome = FOutcome::None();

	if (IsResolved() <= 0)
	{
		if (Outcome = ResolveHostName(); Outcome.IsError())
		{
			return Outcome;
		}
	}

	check(IsResolved() > 0);

	uint32 IpAddress = GetIpAddress();

	// Attempt a SOCKS connect
	Outcome = MaybeConnectSocks(Socket, IpAddress, Port);
	if (Outcome.IsError())
	{
		return Outcome;
	}
	check(Outcome.IsOk());
	bool bSocksConnected = (Outcome.GetResult() == 1);

	// Condition the socket
	if (!Socket.SetBlocking(false))
	{
		return FOutcome::Error("Unable to set socket non-blocking");
	}

	if (int32 OptValue = GetBufferSize(FHost::EDirection::Send); OptValue >= 0)
	{
		Socket.SetSendBufSize(OptValue);
	}

	if (int32 OptValue = GetBufferSize(FHost::EDirection::Recv); OptValue >= 0)
	{
		Socket.SetRecvBufSize(OptValue);
	}

	// Socks connect in a blocking fashion so we're all set (ret=1)
	if (bSocksConnected)
	{
		return FOutcome::Ok();
	}

	// Issue the connect - this is done non-blocking so we need to wait (ret=0)
	if (Outcome = Socket.Connect(IpAddress, Port); Outcome.IsError())
	{
		return Outcome;
	}

	return Outcome;
}



////////////////////////////////////////////////////////////////////////////////
int32 FConnectionPool::FParams::SetHostFromUrl(FAnsiStringView Url)
{
	FUrlOffsets Offsets;
	if (ParseUrl(Url, Offsets) < 0)
	{
		return -1;
	}

	HostName = Offsets.HostName.Get(Url);
	
	VerifyCert = FCertRoots::NoTls();	
	if (Offsets.SchemeLength == 5)
	{
		VerifyCert = FCertRoots::Default();
	}

	if (Offsets.Port)
	{
		FAnsiStringView PortView = Offsets.Port.Get(Url);
		Port = uint16(CrudeToInt(PortView));
	}

	return Offsets.Path;
}

////////////////////////////////////////////////////////////////////////////////
FConnectionPool::FConnectionPool(const FParams& Params)
{
	check(Params.ConnectionCount - 1u <= 63u);
	check(Params.Port <= 0xffffu);
	check(Params.HttpVersion != EHttpVersion::Two || Params.ConnectionCount == 1); // only one as per rfc9113

	// Alloc a new internal object
	uint32 HostNameLen = Params.HostName.Len();
	uint32 AllocSize = sizeof(FHost) + (HostNameLen + 1);
	auto* Internal = (FHost*)FMemory::Malloc(AllocSize, alignof(FHost));

	// Copy host
	char* HostDest = (char*)(Internal + 1);
	memcpy(HostDest, Params.HostName.GetData(), HostNameLen);
	HostDest[HostNameLen] = '\0';

	// Init internal object
	new (Internal) FHost({
		.HostName		= HostDest,
		.Port			= Params.Port,
		.MaxConnections	= Params.ConnectionCount,
		.bPooled		= true,
		.MaxInflight	= Params.MaxInflight,
		.HttpVersion	= Params.HttpVersion,
		.VerifyCert		= Params.VerifyCert,
	});
	Internal->SetBufferSize(FHost::EDirection::Send, Params.SendBufSize);
	Internal->SetBufferSize(FHost::EDirection::Recv, Params.RecvBufSize);

	Ptr = Internal;
}

////////////////////////////////////////////////////////////////////////////////
FConnectionPool::~FConnectionPool()
{
	if (Ptr != nullptr)
	{
		FMemory::Free(Ptr);
	}
}

////////////////////////////////////////////////////////////////////////////////
bool FConnectionPool::Resolve()
{
	return Ptr->ResolveHostName().IsOk();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FConnectionPool::GetIps(uint32* Out, uint32 Count) const
{
	if (Ptr == nullptr)
	{
		return 0;
	}

	uint32 IpAddress = Ptr->GetIpAddress();
	if (IpAddress == 0)
	{
		return 0;
	}

	*Out = IpAddress;
	return 1;
}

////////////////////////////////////////////////////////////////////////////////
void FConnectionPool::Describe(FAnsiStringBuilderBase& OutString) const
{
	if (Ptr == nullptr)
	{
		return;
	}

	const FAnsiStringView HostName = Ptr->GetHostName();
	OutString.Appendf("%.*s", HostName.Len(), HostName.GetData());
	if (!!Ptr->IsResolved())
	{
		const auto IpAddress = Ptr->GetIpAddress();
		OutString.Appendf(" (%u.%u.%u.%u)",
						(IpAddress >> 24) & 0xff,
						(IpAddress >> 16) & 0xff,
						(IpAddress >> 8) & 0xff,
						IpAddress & 0xff
		);
	}
	else
	{
		OutString.Append(" (unresolved)");
	}
}

////////////////////////////////////////////////////////////////////////////////
bool FConnectionPool::IsValidHostUrl(FAnsiStringView Url)
{
	FUrlOffsets Tmp;
	return ParseUrl(Url, Tmp) >= 0;
}

} // namespace UE::IoStore::HTTP

