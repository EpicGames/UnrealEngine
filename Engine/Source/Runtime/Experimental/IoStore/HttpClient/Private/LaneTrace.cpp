// Copyright Epic Games, Inc. All Rights Reserved.

#if __has_include(<IO/Http/LaneTrace.h>)
#	include <IO/Http/LaneTrace.h>
#	if UE_LANETRACE_ENABLED
#		include <HAL/PlatformTime.h>
#		include <HAL/UnrealMemory.h>
#		include <Trace/Trace.inl>
#		include <Misc/ScopeLock.h>
#	endif
#endif

#if UE_LANETRACE_ENABLED

////////////////////////////////////////////////////////////////////////////////
#if UE_LANETRACE_ENABLED == 2

#include <mutex>

#define check(e) do { if ((e) == false) *(int volatile*)0 = 0; } while (false)

struct FCriticalSection
{
	void		Lock()		{ Inner.lock(); }
	void		Unlock()	{ Inner.unlock(); }
	std::mutex	Inner;
};

struct FScopeLock
{
	FScopeLock(FCriticalSection* Lock) : Inner(Lock->Inner) {}
	std::unique_lock<std::mutex> Inner;
};

namespace trace {
#endif

// For this to work Insights would need to know it there may be a ThreadId field
#define LANETRACE_USE_V2_EVENTS 0

namespace LaneTraceDetail
{

////////////////////////////////////////////////////////////////////////////////
UE_TRACE_EVENT_BEGIN($Trace, ThreadInfo, NoSync|Important)
	UE_TRACE_EVENT_FIELD(uint32, ThreadId)
	UE_TRACE_EVENT_FIELD(int32, SortHint)
	UE_TRACE_EVENT_FIELD(UE::Trace::AnsiString, Name)
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(CpuProfiler, EventBatch)
	UE_TRACE_EVENT_FIELD(uint8[], Data)
	UE_TRACE_EVENT_FIELD(uint16, ThreadId)
UE_TRACE_EVENT_END()

#if LANETRACE_USE_V2_EVENTS
UE_TRACE_EVENT_BEGIN(CpuProfiler, EventBatchV2)
	UE_TRACE_EVENT_FIELD(uint8[], Data)
	UE_TRACE_EVENT_FIELD(uint16, ThreadId)
UE_TRACE_EVENT_END()
#endif

////////////////////////////////////////////////////////////////////////////////
static int32 Encode32_7bit(int32 Value, void* __restrict Out)
{
	// Calculate the number of bytes
	int32 Length = 1;

	Length += (Value >= (1 <<  7));
	Length += (Value >= (1 << 14));
	Length += (Value >= (1 << 21));

	// Add a gap every eigth bit for the continuations
	int32 Ret = Value;
	Ret = (Ret & 0x0000'3fff) | ((Ret & 0x0fff'c000) << 2);
	Ret = (Ret & 0x007f'007f) | ((Ret & 0x3f80'3f80) << 1);

	// Set the bits indicating another byte follows
	int32 Continuations = 0x0080'8080;
	Continuations >>= (sizeof(Value) - Length) * 8;
	Ret |= Continuations;

	::memcpy(Out, &Ret, sizeof(Value));

	return Length;
}

////////////////////////////////////////////////////////////////////////////////
static int32 Encode64_7bit(int64 Value, void* __restrict Out)
{
	// Calculate the output length
	uint32 Length = 1;
	Length += (Value >= (1ll <<  7));
	Length += (Value >= (1ll << 14));
	Length += (Value >= (1ll << 21));
	Length += (Value >= (1ll << 28));
	Length += (Value >= (1ll << 35));
	Length += (Value >= (1ll << 42));
	Length += (Value >= (1ll << 49));

	// Add a gap every eigth bit for the continuations
	int64 Ret = Value;
	Ret = (Ret & 0x0000'0000'0fff'ffffull) | ((Ret & 0x00ff'ffff'f000'0000ull) << 4);
	Ret = (Ret & 0x0000'3fff'0000'3fffull) | ((Ret & 0x0fff'c000'0fff'c000ull) << 2);
	Ret = (Ret & 0x007f'007f'007f'007full) | ((Ret & 0x3f80'3f80'3f80'3f80ull) << 1);

	// Set the bits indicating another byte follows
	int64 Continuations = 0x0080'8080'8080'8080ull;
	Continuations >>= (sizeof(Value) - Length) * 8;
	Ret |= Continuations;

	::memcpy(Out, &Ret, sizeof(Value));

	return Length;
}

////////////////////////////////////////////////////////////////////////////////
static uint64 TimeGetTimestamp()
{
#if UE_LANETRACE_ENABLED != 2
	return FPlatformTime::Cycles64();
#else
	return trace::detail::TimeGetTimestamp();
#endif
}



////////////////////////////////////////////////////////////////////////////////
class FScopeBuffer
{
public:
							FScopeBuffer(UE::Trace::FChannel& InChannel);
	void					SetThreadId(uint32 Value);
	bool					IsInScope() const;
	uint32					GetDepth() const;
	void					Flush(bool Force=false);
	void					Enter(uint64 Timestamp, uint32 ScopeId);
	void					Leave(uint64 Timestamp);

private:
	enum
	{
		BufferSize			= 128,
		Overflow			= 24,
		EnterLsb			= 1,
		LeaveLsb			= 0,
		TraceEventBatchVer	= 1 + LANETRACE_USE_V2_EVENTS,
	};
	uint64					LastTimestamp = 0;
	uint64					PrevTimestamp = 0;
	uint8*					Cursor = Buffer;
	UE::Trace::FChannel&	Channel;
	uint32					ThreadIdOverride = 0;
	uint16					Depth = 0;
	uint8					Buffer[BufferSize];
};

////////////////////////////////////////////////////////////////////////////////
FScopeBuffer::FScopeBuffer(UE::Trace::FChannel& InChannel)
: Channel(InChannel)
{
}

////////////////////////////////////////////////////////////////////////////////
void FScopeBuffer::SetThreadId(uint32 Value)
{
	ThreadIdOverride = Value;
}

////////////////////////////////////////////////////////////////////////////////
bool FScopeBuffer::IsInScope() const
{
	return GetDepth() > 0;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FScopeBuffer::GetDepth() const
{
	return Depth;
}

////////////////////////////////////////////////////////////////////////////////
void FScopeBuffer::Flush(bool Force)
{
	if (Cursor == Buffer)
	{
		return;
	}

	if (Depth > 0 && !Force && (Cursor <= (Buffer + BufferSize - Overflow)))
	{
		return;
	}

	if (TraceEventBatchVer == 1)
	{
		UE_TRACE_LOG(CpuProfiler, EventBatch, Channel)
			<< EventBatch.ThreadId(uint16(ThreadIdOverride))
			<< EventBatch.Data(Buffer, uint32(ptrdiff_t(Cursor - Buffer)));
	}
#if LANETRACE_USE_V2_EVENTS
	else
	{
		UE_TRACE_LOG(CpuProfiler, EventBatchV2, Channel)
			<< EventBatchV2.ThreadId(uint16(ThreadIdOverride))
			<< EventBatchV2.Data(Buffer, uint32(ptrdiff_t(Cursor - Buffer)));

		// Both protocols should really do this rebase but it make analysis go
		// bonkers and I'm not looking into that right now
		PrevTimestamp = 0;
	}
#endif

	LastTimestamp = 0;
	PrevTimestamp = 0;

	Cursor = Buffer;
}

////////////////////////////////////////////////////////////////////////////////
void FScopeBuffer::Enter(uint64 Timestamp, uint32 ScopeId)
{
	check(Timestamp >= LastTimestamp);
	LastTimestamp = Timestamp;

	PrevTimestamp += (Timestamp -= PrevTimestamp);
	enum { Shift = (TraceEventBatchVer == 1) ? 1 : 2 };
	Cursor += Encode64_7bit((Timestamp << Shift) | EnterLsb, Cursor);
	Cursor += Encode32_7bit(ScopeId, Cursor);
	Depth++;
}

////////////////////////////////////////////////////////////////////////////////
void FScopeBuffer::Leave(uint64 Timestamp)
{
	check(Timestamp >= LastTimestamp);
	LastTimestamp = Timestamp;

	if (Depth == 0)
	{
		return;
	}

	PrevTimestamp += (Timestamp -= PrevTimestamp);
	enum { Shift = (TraceEventBatchVer == 1) ? 1 : 2 };
	Cursor += Encode64_7bit((Timestamp << Shift) | LeaveLsb, Cursor);
	Depth--;
}



////////////////////////////////////////////////////////////////////////////////
struct FLoctight
{
	struct FScope
	{
							FScope()					= default;
							~FScope();
							FScope(FScope&& Rhs);
							FScope(const FScope&)		= delete;
		FScope&				operator = (const FScope&)	= delete;
		FScope&				operator = (FScope&&)		= delete;
		FCriticalSection*	Outer = nullptr;
	};
	FScope					Scope() const;
	mutable FCriticalSection Loch;
};

////////////////////////////////////////////////////////////////////////////////
FLoctight::FScope::~FScope()
{
	if (Outer)
	{
		Outer->Unlock();
	}
}

////////////////////////////////////////////////////////////////////////////////
FLoctight::FScope::FScope(FScope&& Rhs)
{
	Swap(Outer, Rhs.Outer);
}

////////////////////////////////////////////////////////////////////////////////
FLoctight::FScope FLoctight::Scope() const
{
	Loch.Lock();
	FScope Ret;
	Ret.Outer = &Loch;
	return Ret;
}



////////////////////////////////////////////////////////////////////////////////
class FScopeBufferTs
	: protected FLoctight
	, protected FScopeBuffer
{
public:
	void	SetThreadId(uint32 Value);
	bool	IsInScope() const;
	void	Flush(bool Force=false);
	void	Enter(uint32 ScopeId);
	void	Leave();
};

////////////////////////////////////////////////////////////////////////////////
void FScopeBufferTs::SetThreadId(uint32 Value)
{
	FScope _ = Scope();
	FScopeBuffer::SetThreadId(Value);
}

////////////////////////////////////////////////////////////////////////////////
bool FScopeBufferTs::IsInScope() const
{
	FScope _ = Scope();
	return FScopeBuffer::IsInScope();
}

////////////////////////////////////////////////////////////////////////////////
void FScopeBufferTs::Flush(bool Force)
{
	FScope _ = Scope();
	FScopeBuffer::Flush(Force);
}

////////////////////////////////////////////////////////////////////////////////
void FScopeBufferTs::Enter(uint32 ScopeId)
{
	uint64 Timestamp = TimeGetTimestamp();
	FScope _ = Scope();
	FScopeBuffer::Enter(Timestamp, ScopeId);
}

////////////////////////////////////////////////////////////////////////////////
void FScopeBufferTs::Leave()
{
	uint64 Timestamp = TimeGetTimestamp();
	FScope _ = Scope();
	FScopeBuffer::Leave(Timestamp);
}



////////////////////////////////////////////////////////////////////////////////
class FLane
{
public:
							FLane(const FLaneTraceSpec& Spec);
							~FLane();
	static uint32			NewScope(const FAnsiStringView& Name);
	void					Enter(uint32 ScopeId);
	void					Change(uint32 ScopeId);
	void					Leave();
	void					LeaveAll();

private:
	FScopeBuffer			Buffer;
};

////////////////////////////////////////////////////////////////////////////////
FLane::FLane(const FLaneTraceSpec& Spec)
: Buffer(*(UE::Trace::FChannel*)(Spec.Channel))
{
	static uint32 volatile NextId = 0;
	uint32 Id = UE::Trace::Private::AtomicAddRelaxed(&NextId, 1u) + 1;
	Id += 2 << 10;

	uint32 NameSize = uint32(Spec.Name.Len());
	UE_TRACE_LOG($Trace, ThreadInfo, true, NameSize)
		<< ThreadInfo.ThreadId(Id)
		<< ThreadInfo.SortHint(Spec.Weight)
		<< ThreadInfo.Name(Spec.Name.GetData(), NameSize);

	Buffer.SetThreadId(Id);
}

////////////////////////////////////////////////////////////////////////////////
FLane::~FLane()
{
	Buffer.Flush(true);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FLane::NewScope(const FAnsiStringView& Name)
{
#if UE_LANETRACE_ENABLED != 2
	return FCpuProfilerTrace::OutputEventType(Name.GetData(), "", 0u);
#else
	return uint32(ScopeNew(Name));
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FLane::Enter(uint32 ScopeId)
{
	uint64 Timestamp = TimeGetTimestamp();
	Buffer.Enter(Timestamp, ScopeId);
	Buffer.Flush(false);
}

////////////////////////////////////////////////////////////////////////////////
void FLane::Change(uint32 ScopeId)
{
	uint64 Timestamp = TimeGetTimestamp();
	Buffer.Leave(Timestamp);
	Buffer.Enter(Timestamp, ScopeId);
	Buffer.Flush(false);
}

////////////////////////////////////////////////////////////////////////////////
void FLane::Leave()
{
	uint64 Timestamp = TimeGetTimestamp();
	Buffer.Leave(Timestamp);
	Buffer.Flush(false);
}

////////////////////////////////////////////////////////////////////////////////
void FLane::LeaveAll()
{
	uint64 Timestamp = TimeGetTimestamp();
	while (Buffer.IsInScope())
	{
		Buffer.Leave(Timestamp);
	}
	Buffer.Flush(true);
}

} // namespace LaneTraceDetail



////////////////////////////////////////////////////////////////////////////////
class FLaneTrace
	: public LaneTraceDetail::FLane
{
	using LaneTraceDetail::FLane::FLane;
};

////////////////////////////////////////////////////////////////////////////////
FLaneTrace*	LaneTrace_New(const FLaneTraceSpec& Spec)
{
	return new FLaneTrace(Spec);
}

////////////////////////////////////////////////////////////////////////////////
void LaneTrace_Delete(FLaneTrace* Lane)
{
	delete Lane;
}

////////////////////////////////////////////////////////////////////////////////
uint32 LaneTrace_NewScope(const FAnsiStringView& Name)
{
	return FLaneTrace::NewScope(Name);
}

////////////////////////////////////////////////////////////////////////////////
void LaneTrace_Enter(FLaneTrace* Lane, uint32 ScopeId)
{
	Lane->Enter(ScopeId);
}

////////////////////////////////////////////////////////////////////////////////
void LaneTrace_Change(FLaneTrace* Lane, uint32 ScopeId)
{
	Lane->Change(ScopeId);
}

////////////////////////////////////////////////////////////////////////////////
void LaneTrace_Leave(FLaneTrace* Lane)
{
	Lane->Leave();
}

////////////////////////////////////////////////////////////////////////////////
void LaneTrace_LeaveAll(FLaneTrace* Lane)
{
	Lane->LeaveAll();
}



namespace LaneTraceDetail
{

////////////////////////////////////////////////////////////////////////////////
class FEstate
{
public:
						FEstate(const FLaneTraceSpec& Spec);
						~FEstate();
	FLaneTrace*			Build(UPTRINT Postcode);
	FLaneTrace*			Lookup(UPTRINT Postcode);
	void				Demolish(UPTRINT Postcode);

private:
	struct FEntry
	{
		UPTRINT			Postcode = 0;
		FLaneTrace*		Lane = nullptr;
	};

	enum { GROWTH_SIZE = 4 };

	FLaneTraceSpec		LaneSpec;
	FCriticalSection	Lock;
	TArray<FEntry>		Directory;
};

////////////////////////////////////////////////////////////////////////////////
FEstate::FEstate(const FLaneTraceSpec& InSpec)
: LaneSpec(InSpec)
{
	Directory.SetNumZeroed(GROWTH_SIZE);
}

////////////////////////////////////////////////////////////////////////////////
FEstate::~FEstate()
{
	for (FEntry& Entry : Directory)
	{
		if (Entry.Lane != nullptr)
		{
			LaneTrace_Delete(Entry.Lane);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
FLaneTrace* FEstate::Build(UPTRINT Postcode)
{
	auto UseEstate = [this] (UPTRINT Postcode, FEntry& Entry)
	{
		Entry.Postcode = Postcode;
		if (Entry.Lane == nullptr)
		{
			Entry.Lane = LaneTrace_New(LaneSpec);
		}

		return Entry.Lane;
	};

	FScopeLock _(&Lock);

	for (FEntry& Entry : Directory)
	{
		if (Entry.Postcode == 0)
		{
			return UseEstate(Postcode, Entry);
		}
	}

	int32 NextSize = Directory.Num() + GROWTH_SIZE;
	Directory.SetNum(NextSize);
	return UseEstate(Postcode, Directory[NextSize - GROWTH_SIZE]);
}

////////////////////////////////////////////////////////////////////////////////
FLaneTrace* FEstate::Lookup(UPTRINT Postcode)
{
	FScopeLock _(&Lock);

	for (const FEntry& Entry : Directory)
	{
		if (Entry.Postcode == Postcode)
		{
			return Entry.Lane;
		}
	}

	checkf(false, TEXT("Invalid/unknown postcode given, unable to find estate: %llx"), Postcode);
	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////
void FEstate::Demolish(UPTRINT Postcode)
{
	FScopeLock _(&Lock);

	for (FEntry& Entry : Directory)
	{
		if (Entry.Postcode == Postcode)
		{
			LaneTrace_LeaveAll(Entry.Lane);
			Entry.Postcode = 0;
			return;
		}
	}

	checkf(false, TEXT("Invalid/unknown postcode given, unable to demolish estate: %llx"), Postcode);
}

} // namespace LaneTraceDetail



////////////////////////////////////////////////////////////////////////////////
class FLaneEstate
	: public LaneTraceDetail::FEstate
{
public:
	using LaneTraceDetail::FEstate::FEstate;
};

////////////////////////////////////////////////////////////////////////////////
FLaneEstate*LaneEstate_New(const FLaneTraceSpec& Spec)
{
	return new FLaneEstate(Spec);
}

////////////////////////////////////////////////////////////////////////////////
void LaneEstate_Delete(FLaneEstate* Estate)
{
	delete Estate;
}

////////////////////////////////////////////////////////////////////////////////
FLaneTrace*	LaneEstate_Build(FLaneEstate* Estate, FLanePostcode Postcode)
{
	return Estate->Build(Postcode.Value);
}

////////////////////////////////////////////////////////////////////////////////
FLaneTrace*	LaneEstate_Lookup(FLaneEstate* Estate, FLanePostcode Postcode)
{
	return Estate->Lookup(Postcode.Value);
}

////////////////////////////////////////////////////////////////////////////////
void LaneEstate_Demolish(FLaneEstate* Estate, FLanePostcode Postcode)
{
	return Estate->Demolish(Postcode.Value);
}

#undef LANETRACE_UNTESTED

#if UE_LANETRACE_ENABLED == 2
} // namespace trace
#endif

#endif // UE_LANETRACE_ENABLED
