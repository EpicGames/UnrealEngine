// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::IoStore::HTTP
{

////////////////////////////////////////////////////////////////////////////////
static bool GInflightCancels = true;
static FAutoConsoleVariableRef CVar_IaxInflightCancels(
	TEXT("iax.HttpInflightCancels"),
	GInflightCancels,
	TEXT("Allows cancelling of in-flight requests if supported by peer")
);

// {{{1 recv-stats .............................................................

////////////////////////////////////////////////////////////////////////////////
class FRecvStats
{
public:
							FRecvStats() = default;
	void					Update(uint32 NowMs, uint32 RecvSizeKiB, uint32 TtfbMs);
	void					Flush();
	void					Sample(uint32 NowMs, FLoopPerf& Out) const;

private:
	enum
	{
		BucketMs			= 1000,
	};

	union FBucket
	{
		struct FData
		{
			uint32			RecvKiB;
			uint32			TimeMs;
		};
		std::atomic<uint64> Value;
	};

	void					UpdateRecv(uint32 NowMs, uint32 RecvSizeKiB);
	void					UpdateTtfb(uint32 TtfbMs);
	FBucket					Buckets[8] = {};
	FBucket::FData			Accumulator = {};
	std::atomic<uint64>		TtfbSamples = 0;
	std::atomic<uint32>		TotalRecvKiB = 0;
	uint32					LastMs = 0;
	uint8					Cursor = 0;
};

////////////////////////////////////////////////////////////////////////////////
void FRecvStats::Update(uint32 NowMs, uint32 RecvSizeKiB, uint32 TtfbMs)
{
	UpdateRecv(NowMs, RecvSizeKiB);
	UpdateTtfb(TtfbMs);
}

////////////////////////////////////////////////////////////////////////////////
void FRecvStats::UpdateRecv(uint32 NowMs, uint32 RecvSizeKiB)
{
	if (RecvSizeKiB == 0)
	{
		return;
	}

	TotalRecvKiB.fetch_add(RecvSizeKiB, std::memory_order_relaxed);

	// Quantize time for a modicum of stability.
	uint32 NowQuantMs = NowMs - (NowMs % BucketMs);

	uint32 DeltaMs = NowQuantMs - LastMs;
	LastMs = NowQuantMs;

	Accumulator.TimeMs += DeltaMs;
	if (Accumulator.TimeMs < BucketMs)
	{
		Accumulator.RecvKiB += RecvSizeKiB;
		return;
	}

	uint32 RecvKiB = Accumulator.RecvKiB;
	uint32 TimeMs = NowQuantMs - DeltaMs;
	if (Accumulator.TimeMs > BucketMs)
	{
		Accumulator = {
			.RecvKiB = RecvSizeKiB,
			.TimeMs = 0,
		};
	}
	else
	{
		uint32 Partial = ((NowMs - NowQuantMs) * RecvSizeKiB) / BucketMs;

		RecvKiB += RecvSizeKiB - Partial,
		Accumulator = {
			.RecvKiB = Partial,
			.TimeMs = BucketMs - Accumulator.TimeMs,
		};
		TimeMs = NowQuantMs - Accumulator.TimeMs;
	}

	uint64 Value = (uint64(TimeMs) << 32u) + RecvKiB;

	Cursor += 1;
	uint32 Index = (Cursor - 1) % UE_ARRAY_COUNT(Buckets);
	Buckets[Index].Value.store(Value, std::memory_order_release);
}

////////////////////////////////////////////////////////////////////////////////
void FRecvStats::UpdateTtfb(uint32 TtfbMs)
{
	if (TtfbMs == 0)
	{
		return;
	}

	uint64 Current = TtfbSamples.load(std::memory_order_relaxed);
	Current = (Current << 8ull) | uint64(TtfbMs & 0xff);
	TtfbSamples.store(Current, std::memory_order_relaxed);
}

////////////////////////////////////////////////////////////////////////////////
void FRecvStats::Flush()
{
	if (Accumulator.RecvKiB == 0)
	{
		return;
	}

	uint64 Value = (uint64(LastMs) << 32u) + Accumulator.RecvKiB;
	uint32 Index = Cursor % UE_ARRAY_COUNT(Buckets);
	Buckets[Index].Value.store(Value, std::memory_order_release);
}

////////////////////////////////////////////////////////////////////////////////
void FRecvStats::Sample(uint32 NowMs, FLoopPerf& Out) const
{
	static_assert((BucketMs * UE_ARRAY_COUNT(Buckets)) / 1000 == UE_ARRAY_COUNT(Out.RecvKiBps));

	uint32 NowQuantMs = NowMs - (NowMs % BucketMs);

	union FData
	{
		FBucket::FData	Data;
		uint64			Value;
	};

	// Sample all the buckets
	FData Datas[UE_ARRAY_COUNT(Buckets)];
	for (uint32 i = UE_ARRAY_COUNT(Buckets); i--;)
	{
		Datas[i].Value = Buckets[i].Value.load(std::memory_order_acquire);
	}

	// Find the newest bucket
	int32 Index = -1;
	for (uint32 i = 0, PrevMs = 0u; const FData& Data : Datas)
	{
		if (uint32 TimeMs = Data.Data.TimeMs; TimeMs >= PrevMs)
		{
			Index = int32(i);
			PrevMs = TimeMs;
		}
		++i;
	}

	// Gather samples for the last N seconds.
	for (uint32 i = 0, n = UE_ARRAY_COUNT(Out.RecvKiBps), TimeMs = NowQuantMs; i < n; ++i)
	{
		const FBucket::FData& Data = Datas[Index].Data;

		TimeMs -= BucketMs;
		if (TimeMs > Data.TimeMs)
		{
			Out.RecvKiBps[i] = 0;
			continue;
		}

		uint32 Value = Data.RecvKiB;
		Value = (Value > 0xffff) ? 0xffff : Value;
		Out.RecvKiBps[i] = uint16(Value);

		Index = (Index - 1) % UE_ARRAY_COUNT(Buckets);
	}

	Out.RecvKiB = TotalRecvKiB.load(std::memory_order_relaxed);

	// Time to first byte
	Out.TtfbSamples = TtfbSamples.load(std::memory_order_relaxed);
}



////////////////////////////////////////////////////////////////////////////////
uint32 FLoopPerf::GetTotalRecvKiB() const
{
	return RecvKiB;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FLoopPerf::GetRecvKiBpsCount() const
{
	return UE_ARRAY_COUNT(RecvKiBps);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FLoopPerf::GetRecvKiBps(uint32 Index) const
{
	check(Index < GetRecvKiBpsCount());
	return RecvKiBps[Index];
}

////////////////////////////////////////////////////////////////////////////////
void FLoopPerf::GetRecvKiBps(uint32 (&Out)[RecvBucketCount]) const
{
	static_assert(UE_ARRAY_COUNT(Out) == UE_ARRAY_COUNT(RecvKiBps));

	for (uint32 i = 0; uint32& Item : Out)
	{
		Item = GetRecvKiBps(i);
		++i;
	}
}

////////////////////////////////////////////////////////////////////////////////
uint32 FLoopPerf::GetRecvKiBps() const
{
	// EMA to bias towards most recent data

	uint32 Denom = 0;
	uint32 Numer = 0;
	for (uint32 i = RecvBucketCount; uint32 Sample : RecvKiBps)
	{
		Numer += Sample * i;
		Denom += i;
		if (i >>= 1; i == 0)
		{
			break;
		}
	}

	return Numer / Denom;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FLoopPerf::GetTimeToFirstByteMs() const
{
	uint64 Samples = TtfbSamples;
	uint32 Accumulator = 0;
	uint32 Prev = 0;
	for (uint32 i = 0, n = 8; i < n; ++i, Samples >>= 8)
	{
		uint32 Value = Samples & 0xff;
		if (Value == 0)
		{
			Value = Prev;
		}
		Accumulator += Value;
		Prev = Value;
	}
	return Accumulator >> 3;
}



// {{{1 work-queue .............................................................

/*
 * - Activities (requests send with a loop) are managed in singly-linked lists
 * - Each activity has an associated host it is talking to.
 * - Hosts are ephemeral, or represented externally via a FConnectionPool object
 * - Loop has a group for each host, and each host-group has a bunch of socket-groups
 * - Host-group has a list of work; pending activities waiting to start
 * - Socket-groups own up to two activities; one sending, one receiving
 * - As it recvs, a socket-group will, if possible, fetch more work from the host
 *
 *  Loop:
 *    FHostGroup[HostPtr]:
 *	    Work: Act0 -> Act1 -> Act2 -> Act3 -> ...
 *      FPeerGroup[0...HostMaxConnections]:
 *			Act.Send
 *			Act.Recv
 */

////////////////////////////////////////////////////////////////////////////////
struct FActivityNode
	: public FActivity
{
	struct FParams
		: public FActivity::FParams
	{
		bool				bFollow30x = false;
		EHttpVersion		HttpVersion = EHttpVersion::Default;
	};

	static FActivityNode*	Create(FParams& Params, FAnsiStringView Url={}, FCertRootsRef VerifyCert={});
	static void				Destroy(FActivityNode* Activity);
	FActivityNode*			Next = nullptr;
	int8					Slot = -1;
	bool					bFollow30x;

private:
							FActivityNode(const FParams& Params);
							~FActivityNode() = default;
};

////////////////////////////////////////////////////////////////////////////////
FActivityNode::FActivityNode(const FParams& Params)
: FActivity(Params)
, bFollow30x(Params.bFollow30x)
{
}

////////////////////////////////////////////////////////////////////////////////
FActivityNode* FActivityNode::Create(FParams& Params, FAnsiStringView Url, FCertRootsRef VerifyCert)
{
	static_assert(alignof(FActivityNode) <= 16);
	static_assert(alignof(FHost) <= alignof(FActivityNode));

	check(!Params.Method.IsEmpty());

	if (Url.IsEmpty())
	{
		uint32 BufferSize = Params.BufferSize;
		BufferSize = (BufferSize >= 128) ? BufferSize : 128;
		BufferSize = (BufferSize + 15) & ~15;

		uint32 AllocSize = sizeof(FActivityNode) + BufferSize;
		auto* Ptr = (FActivityNode*)FMemory::Malloc(AllocSize, alignof(FActivityNode));

		check(Params.Host != nullptr);
		check(Params.Host->IsPooled());

		Params.Buffer = (char*)(Ptr + 1);
		Params.BufferSize = uint16(BufferSize);
		Params.bIsKeepAlive = true;
		return new (Ptr) FActivityNode(Params);
	}

	// Parse the URL into its components
	FUrlOffsets UrlOffsets;
	if (ParseUrl(Url, UrlOffsets) < 0)
	{
		return nullptr;
	}

	FAnsiStringView HostName = UrlOffsets.HostName.Get(Url);

	uint32 Port = 0;
	if (UrlOffsets.Port)
	{
		FAnsiStringView PortView = UrlOffsets.Port.Get(Url);
		Port = uint32(CrudeToInt(PortView));
	}

	FAnsiStringView Path;
	if (UrlOffsets.Path > 0)
	{
		Path = Url.Mid(UrlOffsets.Path);
	}

	uint32 BufferSize = Params.BufferSize;
	BufferSize = (BufferSize >= 128) ? BufferSize : 128;
	BufferSize += sizeof(FHost);
	BufferSize += HostName.Len() + 1;
	BufferSize = (BufferSize + 15) & ~15;

	uint32 AllocSize = sizeof(FActivityNode) + BufferSize;
	auto* Ptr = (FActivityNode*)FMemory::Malloc(AllocSize, alignof(FActivityNode));

	// Create an emphemeral host
	if (UrlOffsets.SchemeLength == 5)
	{
		if (VerifyCert == ECertRootsRefType::None)
		{
			VerifyCert = FCertRoots::Default();
		}
		check(VerifyCert != ECertRootsRefType::None);
	}
	else
	{
		VerifyCert = FCertRoots::NoTls();
	}

	FHost* Host = (FHost*)(Ptr + 1);

	char* HostNamePtr = (char*)(Host + 1);
	uint32 HostNameLength = HostName.Len();
	memcpy(HostNamePtr, HostName.GetData(), HostNameLength);
	HostNamePtr[HostNameLength] = '\0';

	EHttpVersion Version = Params.HttpVersion;
	Version = (Version != EHttpVersion::Two) ? EHttpVersion::One : Version;

	new (Host) FHost({
		.HostName	= HostNamePtr,
		.Port		= Port,
		.HttpVersion= Version,
		.VerifyCert = VerifyCert,
	});

	HostNameLength = (HostNameLength + 8) & ~7;

	check(Params.Host == nullptr);
	Params.Path = Path;
	Params.Host = Host;
	Params.Buffer = HostNamePtr + HostNameLength;
	Params.BufferSize = uint16(AllocSize - ptrdiff_t(Params.Buffer - (char*)Ptr));
	Params.bIsKeepAlive = false;
	return new (Ptr) (FActivityNode)(Params);
}

////////////////////////////////////////////////////////////////////////////////
void FActivityNode::Destroy(FActivityNode* Ptr)
{
	Trace(Ptr, ETrace::ActivityDestroy, 0);
	Ptr->~FActivityNode();
	FMemory::Free(Ptr);
}



////////////////////////////////////////////////////////////////////////////////
class FActivityList
{
public:
					FActivityList()							= default;
					~FActivityList();
					FActivityList(FActivityList&& Rhs)		{ Swap(Head, Rhs.Head); }
	void			operator = (FActivityList&& Rhs)		{ Swap(Head, Rhs.Head); }
					FActivityList(const FActivityList&)		= delete;
	void			operator = (const FActivityList&)		= delete;
	int32			IsEmpty() const							{ return Head == nullptr; }
	FActivityNode*	GetHead() const							{ return Head; }
	FActivityNode*	Detach()								{ FActivityNode* Ret = Head; Head = nullptr; return Ret; }
	void			Reverse();
	void			Join(FActivityList& Rhs);
	void			Prepend(FActivityNode* Node);
	void			Append(FActivityNode* Node);
	FActivityNode*	Pop();
	void			PopPrepend(FActivityList& ToList);
	void			PopAppend(FActivityList& ToList);

	template <typename LAMBDA>
	FActivityNode*	MoveToHead(LAMBDA&& Predicate);

private:
	FActivityNode*	PopImpl();
	FActivityNode*	Head = nullptr;
};

////////////////////////////////////////////////////////////////////////////////
FActivityList::~FActivityList()
{
	check(Head == nullptr);
}

////////////////////////////////////////////////////////////////////////////////
template <typename LAMBDA>
FActivityNode* FActivityList::MoveToHead(LAMBDA&& Predicate)
{
	FActivityNode* Node = Head;
	FActivityNode* Prev = nullptr;
	for (; Node != nullptr; Prev = Node, Node = Node->Next)
	{
		if (!Predicate(Node))
		{
			continue;
		}

		if (Prev == nullptr)
		{
			check(Head == Node);
			return Node;
		}

		Prev->Next = Node->Next;
		Node->Next = Head;
		Head = Node;
		return Node;
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////
void FActivityList::Reverse()
{
	FActivityNode* Reverse = nullptr;
	for (FActivityNode* Next; Head != nullptr; Head = Next)
	{
		Next = Head->Next;
		Head->Next = Reverse;
		Reverse = Head;
	}
	Head = Reverse;
}

////////////////////////////////////////////////////////////////////////////////
void FActivityList::Join(FActivityList& Rhs)
{
	if (Head == nullptr)
	{
		Swap(Head, Rhs.Head);
		return;
	}

	FActivityNode* Tail = Head;
	for (; Tail->Next != nullptr; Tail = Tail->Next);
	Swap(Tail->Next, Rhs.Head);
}

////////////////////////////////////////////////////////////////////////////////
void FActivityList::Prepend(FActivityNode* Node)
{
	check(Node->Next == nullptr);
	Node->Next = Head;
	Head = Node;
}

////////////////////////////////////////////////////////////////////////////////
void FActivityList::Append(FActivityNode* Node)
{
	check(Node->Next == nullptr);
	if (Head == nullptr)
	{
		Head = Node;
		return;
	}

	FActivityNode* Tail = Head;
	for (; Tail->Next != nullptr; Tail = Tail->Next);
	Tail->Next = Node;
}

////////////////////////////////////////////////////////////////////////////////
FActivityNode* FActivityList::Pop()
{
	if (Head == nullptr)
	{
		return Head;
	}
	return PopImpl();
}

////////////////////////////////////////////////////////////////////////////////
FActivityNode* FActivityList::PopImpl()
{
	FActivityNode* Node = Head;
	Head = Node->Next;
	Node->Next = nullptr;
	return Node;
}

////////////////////////////////////////////////////////////////////////////////
void FActivityList::PopPrepend(FActivityList& ToList)
{
	check(Head != nullptr);
	FActivityNode* Node = PopImpl();
	ToList.Prepend(Node);
}

////////////////////////////////////////////////////////////////////////////////
void FActivityList::PopAppend(FActivityList& ToList)
{
	check(Head != nullptr);
	FActivityNode* Node = PopImpl();
	ToList.Append(Node);
}



////////////////////////////////////////////////////////////////////////////////
struct FTickState
{
	FActivityList				DoneList;
	uint64						Cancels;
	int32&						RecvAllowance;
	class FWorkQueue*			Work;
	int32						PollTimeoutMs;
	int32						FailTimeoutMs;
	uint32						NowMs;
	uint32						RecvSizeKiB = 0;
	uint32						TtfbMs = 0;
};



////////////////////////////////////////////////////////////////////////////////
class FWorkQueue
{
public:
						FWorkQueue() = default;
	bool				HasWork() const { return !List.IsEmpty(); }
	void				AddActivity(FActivityNode* Activity);
	FActivityNode*		PopActivity();
	void				TickCancels(FTickState& State);

private:
	FActivityList		List;
	uint64				ActiveSlots = 0;

	UE_NONCOPYABLE(FWorkQueue);
};

////////////////////////////////////////////////////////////////////////////////
void FWorkQueue::AddActivity(FActivityNode* Activity)
{
	check(Activity->Next == nullptr);
	List.Append(Activity);

	ActiveSlots |= (1ull << Activity->Slot);
}

////////////////////////////////////////////////////////////////////////////////
FActivityNode* FWorkQueue::PopActivity()
{
	FActivityNode* Activity = List.Pop();
	if (Activity == nullptr)
	{
		return nullptr;
	}

	check(ActiveSlots & (1ull << Activity->Slot));
	ActiveSlots ^= (1ull << Activity->Slot);

	check(Activity->Next == nullptr);
	return Activity;
}

////////////////////////////////////////////////////////////////////////////////
void FWorkQueue::TickCancels(FTickState& State)
{
	if (State.Cancels == 0 || (State.Cancels & ActiveSlots) == 0)
	{
		return;
	}

	// We are going to rebuild the list of activities to maintain order as the
	// activity list is singular.

	check(!List.IsEmpty());
	ActiveSlots = 0;

	FActivityNode* Activity = List.Detach();
	for (FActivityNode* Next; Activity != nullptr; Activity = Next)
	{
		Next = Activity->Next;
		Activity->Next = nullptr;

		if (uint64 Slot = (1ull << Activity->Slot); (State.Cancels & Slot) == 0)
		{
			AddActivity(Activity);
			continue;
		}

		Activity->Cancel();

		State.DoneList.Prepend(Activity);
	}
}



// {{{1 peer-group .............................................................

////////////////////////////////////////////////////////////////////////////////
class FPeerGroup
{
public:
						FPeerGroup() = default;
	uint32				CollectRecvSizeKiB();
	void				SetMaxInflight(uint32 Maximum);
	void				Unwait()			{ check(bWaiting); bWaiting = false; }
	FWaiter				GetWaiter() const;
	bool				Tick(FTickState& State);
	void				TickSend(FTickState& State, FHost& Host, FPoller& Poller);
	void				Fail(FTickState& State, FOutcome Reason);

private:
	bool				MaybeStartNewWork(FTickState& State);
	void				TickCancels(FTickState& State);
	void				Negotiate(FTickState& State);
	void				RecvInternal(FTickState& State);
	void				SendInternal(FTickState& State);
	FActivityList		Send;
	FActivityList		Recv;
	FHttpPeer			Peer;
	uint64				LastRecvSize = 0;
	uint32				LastUseMs = 0;
	uint8				IsKeepAlive = 0;
	int8				InflightNum = 0;
	int8				InflightMax = 1;
	bool				bNegotiating = false;
	bool				bWaiting = false;

	UE_NONCOPYABLE(FPeerGroup);
};

////////////////////////////////////////////////////////////////////////////////
uint32 FPeerGroup::CollectRecvSizeKiB()
{
	if (!Peer.IsValid())
	{
		return 0;
	}

	uint64 Size = Peer.GetRecvSize();
	uint32 Ret = uint32(Size - LastRecvSize) >> 10;

	LastRecvSize = Size & ~1023;
	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
void FPeerGroup::SetMaxInflight(uint32 Maximum)
{
	InflightMax = uint8(FMath::Min(uint32(Maximum), 127u));
}

////////////////////////////////////////////////////////////////////////////////
FWaiter FPeerGroup::GetWaiter() const
{
	if (!bWaiting)
	{
		return FWaiter();
	}

	FWaitable Waitable = Peer.GetWaitable();

	FWaiter Waiter(MoveTemp(Waitable));
	Waiter.WaitFor((!Recv.IsEmpty()) ? FWaiter::EWhat::Recv : FWaiter::EWhat::Send);
	return Waiter;
}

////////////////////////////////////////////////////////////////////////////////
void FPeerGroup::Fail(FTickState& State, FOutcome Reason)
{
	// Any send left at this point is unrecoverable. Send is likely shorter.
	Send.Join(Recv);
	Recv = MoveTemp(Send);

	// Failure is quite terminal and we need to abort everything
	while (FActivityNode* Activity = Recv.Pop())
	{
		Activity->Fail(Reason);
		State.DoneList.Prepend(Activity);
	}

	Peer = FHttpPeer();
	Send = FActivityList();
	Recv = FActivityList();
	bWaiting = false;
	IsKeepAlive = 0;
	InflightNum = 0;
	bNegotiating = false;
}

////////////////////////////////////////////////////////////////////////////////
bool FPeerGroup::MaybeStartNewWork(FTickState& State)
{
	if (InflightNum >= InflightMax)		return false;
	if (IsKeepAlive == 0)				return false;

	if (!State.Work->HasWork())
	{
		if (InflightNum == 0)
		{
			LastUseMs = State.NowMs;
		}
		return false;
	}

	FActivityNode* Activity = State.Work->PopActivity();
	do
	{
		Trace(Activity, ETrace::StartWork);

		check(Activity->GetHost()->IsPooled());

		Send.Append(Activity);
		InflightNum++;
		if (InflightNum == InflightMax)
		{
			break;
		}

		Activity = State.Work->PopActivity();
	}
	while (Activity != nullptr);

	return true;
}

////////////////////////////////////////////////////////////////////////////////
void FPeerGroup::Negotiate(FTickState& State)
{
	check(bNegotiating);
	check(!Send.IsEmpty());
	check(Peer.IsValid());

	FOutcome Outcome = Peer.Handshake();
	if (Outcome.IsError())
	{
		Fail(State, Outcome);
		return;
	}

	if (Outcome.IsWaiting())
	{
		bWaiting = true;
		return;
	}

	LastUseMs = GetNowMs(); // exclude connect/neg. from ttfb timing.
	bNegotiating = false;
	return SendInternal(State);
}

////////////////////////////////////////////////////////////////////////////////
void FPeerGroup::RecvInternal(FTickState& State)
{
	check(bNegotiating == false);
	check(!Recv.IsEmpty());

	FActivityNode* Activity = nullptr;

	FOutcome Outcome = Peer.GetPendingTransactId();
	if (Outcome.IsWaiting())
	{
		bWaiting |= true;
		return;
	}

	while (Outcome.IsOk())
	{
		uint32 TransactId = Outcome.GetResult();
		Activity = Recv.MoveToHead(
			[TransactId] (FActivityNode* Activity)
			{
				return Activity->GetTransactId() == TransactId;
			}
		);

		if (Activity == nullptr)
		{
			// When an activity is cancelled, its sink is called and the activity retired. If
			// supported then the peer may also be informed about the cancel and it will not
			// send any further related frames. Some frames however may already be on the wire
			// and when received, have no corresponding activity. So here we have to skip over
			// transactions we are unable to match to an activity.
			if (GInflightCancels)
			{
				Peer.SkipPendingTransactId();
				return RecvInternal(State);
			}

			Outcome = FOutcome::Error("could not find work for transact id", TransactId);
			break;
		}

		Outcome = Activity->Tick(Peer, &State.RecvAllowance);
		break;
	}

	// Any sort of error here is unrecoverable
	if (Outcome.IsError())
	{
		Fail(State, Outcome);
		return;
	}

	check(Activity != nullptr);

	IsKeepAlive &= uint8(Activity->GetTransaction()->IsKeepAlive() == true);
	bWaiting |= (Outcome.IsWaiting() && Outcome.IsWaitData());

	// If there was no data available this is far as receiving can go
	if (Outcome.IsWaiting())
	{
		return;
	}

	// If we have a time here we are timing time-to-first-byte
	if (LastUseMs != 0)
	{
		uint32 TtfbMs = GetNowMs() - LastUseMs;
		TtfbMs = (TtfbMs <= 0xff) ? TtfbMs : 0xff;
		State.TtfbMs = (State.TtfbMs << 8) | TtfbMs;
		LastUseMs = 0;
	}

	// If an okay result is -1, then message recv is done and content is next
	// and it is likely that we have it.
	if (Outcome.GetResult() == uint32(FActivity::EStage::Response))
	{
		return RecvInternal(State);
	}

	check(Outcome.GetResult() == uint32(FActivity::EStage::Content));

	check(Recv.GetHead() == Activity);
	Recv.PopPrepend(State.DoneList);

	check(InflightNum > 0);
	InflightNum--;

	if (MaybeStartNewWork(State))
	{
		SendInternal(State);
	}

	// If the server wants the connection closed then we can drop our peer
	if (IsKeepAlive == 0)
	{
		Peer = FHttpPeer();
	}
}

////////////////////////////////////////////////////////////////////////////////
void FPeerGroup::SendInternal(FTickState& State)
{
	check(bNegotiating == false);
	check(IsKeepAlive == 1);
	check(!Send.IsEmpty());

	FActivityNode* Activity = Send.GetHead();

	FOutcome Outcome = Activity->Tick(Peer);

	if (Outcome.IsWaiting())
	{
		bWaiting = true;
		return;
	}

	if (Outcome.IsError())
	{
		Fail(State, Outcome);
		return;
	}

	check(Outcome.GetResult() == uint32(FActivity::EStage::Request));

	Send.PopAppend(Recv);

	if (!Send.IsEmpty())
	{
		return SendInternal(State);
	}
}

////////////////////////////////////////////////////////////////////////////////
bool FPeerGroup::Tick(FTickState& State)
{
	if (bNegotiating)
	{
		Negotiate(State);
	}

	else if (!Send.IsEmpty())
	{
		SendInternal(State);
	}

	if (!Recv.IsEmpty() && State.RecvAllowance)
	{
		RecvInternal(State);
	}

	if (!bNegotiating)
	{
		TickCancels(State);
	}

	return !!IsKeepAlive | !(Send.IsEmpty() & Recv.IsEmpty());
}

////////////////////////////////////////////////////////////////////////////////
void FPeerGroup::TickSend(FTickState& State, FHost& Host, FPoller& Poller)
{
	// This path is only for those that have spare capacity
	if (InflightNum >= InflightMax)
	{
		return;
	}

	// Failing will try and recover work which we don't want to happen yet
	FActivityNode* Pending = State.Work->PopActivity();
	check(Pending != nullptr);

	// Close idle sockets
	if (uint32 TimeoutMs = LastUseMs + GIdleMs; LastUseMs && TimeoutMs < State.NowMs)
	{
		if (InflightNum == 0)
		{
			LastUseMs = 0;
			Peer = FHttpPeer();
		}
	}

	// We don't have a connected socket on first use, or if a keep-alive:close
	// was received from the server. So we connect here.
	bool bWillBlock = false;
	if (!Peer.IsValid())
	{
		FOutcome Outcome = FOutcome::None();

		FSocket Socket;
		if (Socket.Create())
		{
			FWaitable Waitable = Socket.GetWaitable();
			Poller.Register(Waitable);

			Outcome = Host.Connect(Socket);
		}
		else
		{
			Outcome = FOutcome::Error("Failed to create socket");
		}

		if (Outcome.IsError())
		{
			// We failed to connect, let's bail.
			Recv.Prepend(Pending);
			Fail(State, Outcome);
			return;
		}

		IsKeepAlive = 1;
		bNegotiating = true;
		bWillBlock = Outcome.IsWaiting();
		InflightNum = 0;
		LastRecvSize = 0;

		FHttpPeer::FParams Params = {
			.Socket = MoveTemp(Socket),
			.Certs = Host.GetVerifyCert(),
			.HostName = Host.GetHostName().GetData(),
			.HttpVersion = Host.GetHttpVersion(),
		};
		Peer = FHttpPeer(MoveTemp(Params));
	}

	// If we are idle we can time the time-to-first-byte
	if (!bNegotiating && (InflightNum == 0) && Recv.IsEmpty())
	{
		LastUseMs = GetNowMs();
	}

	Send.Append(Pending);
	InflightNum++;

	MaybeStartNewWork(State);

	if (!bWillBlock)
	{
		if (bNegotiating)
		{
			return Negotiate(State);
		}
		return SendInternal(State);
	}

	// Non-blocking connect
	bWaiting = true;
}

////////////////////////////////////////////////////////////////////////////////
void FPeerGroup::TickCancels(FTickState& State)
{
	if (State.Cancels == 0 || !GInflightCancels)
	{
		return;
	}

	auto Impl = [this, &State] (FActivityList& List)
	{
		if (List.IsEmpty())
		{
			return;
		}

		for (FActivityNode* Next, *Activity = List.Detach(); Activity != nullptr; Activity = Next)
		{
			Next = Activity->Next;
			Activity->Next = nullptr;

			bool bCancelled = false;
			if (uint64 Slot = (1ull << Activity->Slot); (State.Cancels & Slot) != 0)
			{
				bCancelled = Activity->CancelTransaction(Peer);
			}

			if (!bCancelled)
			{
				List.Prepend(Activity);
				continue;
			}

			Activity->Cancel();
			State.DoneList.Prepend(Activity);
			--InflightNum;
		}
		
		List.Reverse();
	};

	Impl(Send);
	Impl(Recv);
}



// {{{1 host-group .............................................................

////////////////////////////////////////////////////////////////////////////////
class FHostGroup
{
public:
								FHostGroup(FHost& InHost);
	bool						IsBusy() const	{ return BusyCount != 0; }
	const FHost&				GetHost() const	{ return Host; }
	void						Tick(FTickState& State);
	void						AddActivity(FActivityNode* Activity);

private:
	int32						Wait(const FTickState& State);
	TArray<FPeerGroup>			PeerGroups;
	FWorkQueue					Work;
	FHost&						Host;
	FPoller						Poller;
	uint32						BusyCount = 0;
	int32						WaitTimeAccum = 0;

	UE_NONCOPYABLE(FHostGroup);
};

////////////////////////////////////////////////////////////////////////////////
FHostGroup::FHostGroup(FHost& InHost)
: Host(InHost)
{
	uint32 Num = InHost.GetMaxConnections();
	PeerGroups.SetNum(Num);

	for (uint32 Inflight = Host.GetMaxInflight(); FPeerGroup& Group : PeerGroups)
	{
		Group.SetMaxInflight(Inflight);
	}
}

////////////////////////////////////////////////////////////////////////////////
int32 FHostGroup::Wait(const FTickState& State)
{
	// Collect groups that are waiting on something
	TArray<FWaiter, TFixedAllocator<64>> Waiters;
	for (uint32 i = 0, n = PeerGroups.Num(); i < n; ++i)
	{
		FWaiter Waiter = PeerGroups[i].GetWaiter();
		if (!Waiter.IsValid())
		{
			continue;
		}

		Waiter.SetIndex(i);
		Waiters.Add(MoveTemp(Waiter));
	}

	if (Waiters.IsEmpty())
	{
		return 0;
	}

	Trace(ETrace::Wait);
	ON_SCOPE_EXIT { Trace(ETrace::Unwait); };

	// If the poll timeout is negative then treat that as a fatal timeout
	check(State.FailTimeoutMs);
	int32 PollTimeoutMs = State.PollTimeoutMs;
	if (PollTimeoutMs < 0)
	{
		PollTimeoutMs = State.FailTimeoutMs;
	}

	// Actually do the wait
	int32 Result = FWaiter::Wait(Waiters, Poller, PollTimeoutMs);
	if (Result <= 0)
	{
		// If the user opts to not block then we don't accumulate wait time and
		// leave it to them to manage time a fail timoue
		WaitTimeAccum += PollTimeoutMs;

		if (State.PollTimeoutMs < 0 || WaitTimeAccum >= State.FailTimeoutMs)
		{
			return MIN_int32;
		}

		return Result;
	}

	WaitTimeAccum = 0;

	// For each waiter that's ready, find the associated group "unwait" them.
	int32 Count = 0;
	for (int32 i = 0, n = Waiters.Num(); i < n; ++i)
	{
		if (!Waiters[i].IsReady())
		{
			continue;
		}

		uint32 Index = Waiters[i].GetIndex();
		check(Index < uint32(PeerGroups.Num()));
		PeerGroups[Index].Unwait();

		Waiters.RemoveAtSwap(i, EAllowShrinking::No);
		--n, --i, ++Count;
	}
	check(Count == Result);

	return Result;
}

////////////////////////////////////////////////////////////////////////////////
void FHostGroup::Tick(FTickState& State)
{
	State.Work = &Work;

	if (BusyCount = Work.HasWork(); BusyCount)
	{
		Work.TickCancels(State);

		// Get available work out on idle sockets as soon as possible
		for (FPeerGroup& Group : PeerGroups)
		{
			if (!Work.HasWork())
			{
				break;
			}

			Group.TickSend(State, Host, Poller);
		}
	}

	// Wait on the groups that are
	if (int32 Result = Wait(State); Result < 0)
	{
		FOutcome Reason = (Result == MIN_int32)
			? FOutcome::Error("FailTimeout hit")
			: FOutcome::Error("poll() returned an unexpected error");

		for (FPeerGroup& Group : PeerGroups)
		{
			Group.Fail(State, Reason);
		}

		return;
	}

	// Tick everything, starting with groups that are maybe closest to finishing
	for (FPeerGroup& Group : PeerGroups)
	{
		BusyCount += (Group.Tick(State) == true);
	}

	// Gather how much each peer has recv'd since the last time
	for (FPeerGroup& Group : PeerGroups)
	{
		State.RecvSizeKiB += Group.CollectRecvSizeKiB();
	}
}

////////////////////////////////////////////////////////////////////////////////
void FHostGroup::AddActivity(FActivityNode* Activity)
{
	Work.AddActivity(Activity);
}



// {{{1 event-loop .............................................................

////////////////////////////////////////////////////////////////////////////////
static const FEventLoop::FRequestParams GDefaultParams;

////////////////////////////////////////////////////////////////////////////////
class FEventLoop::FImpl
{
public:
							~FImpl();
	void					SamplePerf(FLoopPerf& Perf) const;
	uint32					Tick(int32 PollTimeoutMs=0);
	bool					IsIdle() const;
	void					Throttle(uint32 KiBPerSec);
	void					SetFailTimeout(int32 TimeoutMs);
	void					Cancel(FTicket Ticket);
	FRequest				Request(FActivityNode* Activity);
	FTicket					Send(FActivityNode* Activity);

private:
	void					ReceiveWork();
	FRecvStats				RecvStats;
	FCriticalSection		Lock;
	std::atomic<uint64>		FreeSlots		= ~0ull;
	std::atomic<uint64>		Cancels			= 0;
	uint64					PrevFreeSlots	= ~0ull;
	FActivityList			Pending;
	FThrottler				Throttler;
	TArray<FHostGroup>		Groups;
	int32					FailTimeoutMs	= GIdleMs;
	uint32					BusyCount		= 0;
};

////////////////////////////////////////////////////////////////////////////////
FEventLoop::FImpl::~FImpl()
{
	check(BusyCount == 0);
}

////////////////////////////////////////////////////////////////////////////////
void FEventLoop::FImpl::SamplePerf(FLoopPerf& Out) const
{
	uint32 NowMs = GetNowMs();
	RecvStats.Sample(NowMs, Out);
}

////////////////////////////////////////////////////////////////////////////////
FRequest FEventLoop::FImpl::Request(FActivityNode* Activity)
{
	Trace(Activity, ETrace::ActivityCreate, 0);

	FRequest Ret;
	Ret.Ptr = Activity;
	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
FTicket FEventLoop::FImpl::Send(FActivityNode* Activity)
{
	Trace(Activity, ETrace::RequestBegin);

	uint64 Slot;
	{
		FScopeLock _(&Lock);

		for (;; FPlatformProcess::SleepNoStats(0.0f))
		{
			uint64 FreeSlotsLoad = FreeSlots.load(std::memory_order_relaxed);
			if (!FreeSlotsLoad)
			{
				// we don't handle oversubscription at the moment. Could return
				// activity to Reqeust and return a 0 ticket.
				check(false);
			}
			Slot = -int64(FreeSlotsLoad) & FreeSlotsLoad;
			if (FreeSlots.compare_exchange_weak(FreeSlotsLoad, FreeSlotsLoad - Slot, std::memory_order_relaxed))
			{
				break;
			}
		}
		Activity->Slot = int8(63 - FMath::CountLeadingZeros64(Slot));

		// This puts pending requests in reverse order of when they were made
		// but this will be undone when ReceiveWork() adds work to the queue
		Pending.Prepend(Activity);
	}

	return Slot;
}

////////////////////////////////////////////////////////////////////////////////
bool FEventLoop::FImpl::IsIdle() const
{
	return FreeSlots.load(std::memory_order_relaxed) == ~0ull;
}

////////////////////////////////////////////////////////////////////////////////
void FEventLoop::FImpl::Throttle(uint32 KiBPerSec)
{
	Throttler.SetLimit(KiBPerSec);
}

////////////////////////////////////////////////////////////////////////////////
void FEventLoop::FImpl::SetFailTimeout(int32 TimeoutMs)
{
	if (TimeoutMs > 0)
	{
		FailTimeoutMs = TimeoutMs;
	}
	else
	{
		FailTimeoutMs = GIdleMs; // Reset to default
	}
}

////////////////////////////////////////////////////////////////////////////////
void FEventLoop::FImpl::Cancel(FTicket Ticket)
{
	Cancels.fetch_or(Ticket, std::memory_order_relaxed);
}

////////////////////////////////////////////////////////////////////////////////
void FEventLoop::FImpl::ReceiveWork()
{
	uint64 FreeSlotsLoad = FreeSlots.load(std::memory_order_relaxed);
	if (FreeSlotsLoad == PrevFreeSlots)
	{
		return;
	}
	PrevFreeSlots = FreeSlotsLoad;

	// Fetch the pending activities from out in the wild
	FActivityList Collected;
	{
		FScopeLock _(&Lock);
		Swap(Collected, Pending);
	}
	Collected.Reverse();

	// Group activities by their host.
	FActivityNode* Activity = Collected.Detach();
	for (FActivityNode* Next; Activity != nullptr; Activity = Next)
	{
		Next = Activity->Next;
		Activity->Next = nullptr;

		FHost& Host = *(Activity->GetHost());
		auto Pred = [&Host] (const FHostGroup& Lhs) { return &Lhs.GetHost() == &Host; };
		FHostGroup* Group = Groups.FindByPredicate(Pred);
		if (Group == nullptr)
		{
			Group = &(Groups.Emplace_GetRef(Host));
		}

		Group->AddActivity(Activity);
		++BusyCount;
	}
}

////////////////////////////////////////////////////////////////////////////////
uint32 FEventLoop::FImpl::Tick(int32 PollTimeoutMs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IasHttp::Tick);

	ReceiveWork();

	// We limit recv sizes as a way to control bandwidth use.
	int32 RecvAllowance = Throttler.GetAllowance();
	if (RecvAllowance <= 0)
	{
		if (PollTimeoutMs == 0)
		{
			return BusyCount;
		}

		int32 ThrottleWaitMs = -RecvAllowance;
		if (PollTimeoutMs > 0)
		{
			ThrottleWaitMs = FMath::Min(ThrottleWaitMs, PollTimeoutMs);
		}
		FPlatformProcess::SleepNoStats(float(ThrottleWaitMs) / 1000.0f);

		RecvAllowance = Throttler.GetAllowance();
		if (RecvAllowance <= 0)
		{
			return BusyCount;
		}
	}

	uint64 CancelsLoad = Cancels.load(std::memory_order_relaxed);

	// Tick groups
	FTickState TickState = {
		.Cancels = CancelsLoad,
		.RecvAllowance = RecvAllowance,
		.PollTimeoutMs = PollTimeoutMs,
		.FailTimeoutMs = FailTimeoutMs,
		.NowMs = GetNowMs(),
	};
	for (FHostGroup& Group : Groups)
	{
		Group.Tick(TickState);
	}

	// Track some stats
	RecvStats.Update(TickState.NowMs, TickState.RecvSizeKiB, TickState.TtfbMs);

	// Remove groups that are idle
	for (uint32 i = 0, n = Groups.Num(); i < n; ++i)
	{
		FHostGroup& Group = Groups[i];
		if (Group.IsBusy())
		{
			continue;
		}

		Groups.RemoveAtSwap(i, EAllowShrinking::No);
		--n, --i;
	}

	Throttler.ReturnUnused(RecvAllowance);

	uint64 ReturnedSlots = 0;
	for (FActivityNode* Activity = TickState.DoneList.Detach(); Activity != nullptr;)
	{
		FActivityNode* Next = Activity->Next;
		ReturnedSlots |= (1ull << Activity->Slot);

		FActivityNode::Destroy(Activity);

		--BusyCount;
		Activity = Next;
	}

	uint32 BusyBias = 0;
	if (ReturnedSlots)
	{
		uint64 LatestFree = FreeSlots.fetch_add(ReturnedSlots, std::memory_order_relaxed);
		BusyBias += (LatestFree != PrevFreeSlots);
		PrevFreeSlots += ReturnedSlots;
	}

	// To cover the scenario where one of the completed slots gets cancelled
	// after the earlier load but before now,  we will unset returned slots
	// from the cancel mask too.
	if (uint64 Mask = CancelsLoad | ReturnedSlots; Mask)
	{
		Cancels.fetch_and(~Mask, std::memory_order_relaxed);
	}

	// There is still some accounting in the stats object but there is no reason
	// to get any further ticks. Make sure the accounting is externally visible.
	uint32 Result = BusyCount + BusyBias;
	if (Result == 0)
	{
		RecvStats.Flush();
	}

	return Result;
}



////////////////////////////////////////////////////////////////////////////////
FEventLoop::FEventLoop()						{ Impl = new FEventLoop::FImpl(); Trace(Impl, ETrace::LoopCreate); }
FEventLoop::~FEventLoop()						{ Trace(Impl, ETrace::LoopDestroy); delete Impl; }
void FEventLoop::SamplePerf(FLoopPerf& Out) const { return Impl->SamplePerf(Out); }
uint32 FEventLoop::Tick(int32 PollTimeoutMs)	{ return Impl->Tick(PollTimeoutMs); }
bool FEventLoop::IsIdle() const					{ return Impl->IsIdle(); }
void FEventLoop::Cancel(FTicket Ticket)			{ return Impl->Cancel(Ticket); }
void FEventLoop::Throttle(uint32 KiBPerSec)		{ return Impl->Throttle(KiBPerSec); }
void FEventLoop::SetFailTimeout(int32 Ms)		{ return Impl->SetFailTimeout(Ms); }

////////////////////////////////////////////////////////////////////////////////
FRequest FEventLoop::Request(
	FAnsiStringView Method,
	FAnsiStringView Url,
	const FRequestParams* Params)
{
	Params = (Params != nullptr) ? Params : &GDefaultParams;

	FActivityNode::FParams ActivityParams;
	ActivityParams.Method = Method;
	ActivityParams.BufferSize = Params->BufferSize;
	ActivityParams.bFollow30x = (Params->bAutoRedirect == true);
	ActivityParams.bAllowChunked = (Params->bAllowChunked == true);
	ActivityParams.ContentSizeEst = Params->ContentSizeEst;
	ActivityParams.HttpVersion = Params->HttpVersion;

	FActivityNode* Activity = FActivityNode::Create(ActivityParams, Url, Params->VerifyCert);
	if (Activity == nullptr)
	{
		return FRequest();
	}

	return Impl->Request(Activity);
}

////////////////////////////////////////////////////////////////////////////////
FRequest FEventLoop::Request(
	FAnsiStringView Method,
	FAnsiStringView Path,
	FConnectionPool& Pool,
	const FRequestParams* Params)
{
	check(Pool.Ptr != nullptr);
	check(Params == nullptr || Params->VerifyCert == ECertRootsRefType::None); // add cert to FConPool instead
	check(Params == nullptr || Params->HttpVersion == EHttpVersion::Default); // set protocol version via FConPool

	Params = (Params != nullptr) ? Params : &GDefaultParams;

	FActivityNode::FParams ActivityParams;
	ActivityParams.Method = Method;
	ActivityParams.Path = Path;
	ActivityParams.Host = Pool.Ptr;
	ActivityParams.bFollow30x = (Params->bAutoRedirect == true);
	ActivityParams.bAllowChunked = (Params->bAllowChunked == true);
	ActivityParams.ContentSizeEst = Params->ContentSizeEst;
	FActivityNode* Activity = FActivityNode::Create(ActivityParams);
	check(Activity != nullptr);

	return Impl->Request(Activity);
}

////////////////////////////////////////////////////////////////////////////////
bool FEventLoop::Redirect(const FTicketStatus& Status, FTicketSink& OuterSink)
{
	const FResponse& Response = Status.GetResponse();

	switch (Response.GetStatusCode())
	{
		case 301:				// RedirectMoved
		case 302:				// RedirectFound
		case 307:				// RedirectTemp
		case 308: break;		// RedirectPerm
		default: return false;
	}

	FAnsiStringView Location = Response.GetHeader("Location");
	if (Location.IsEmpty())
	{
		// todo: turn source activity into an error?
		return false;
	}

	const auto& Activity = (FActivity&)Response; // todo: yuk

	// should we ever hit this, we'll fix it
	check(Activity.GetMethod().Equals("HEAD", ESearchCase::IgnoreCase) || Response.GetContentLength() == 0);

	// Original method should remain unchanged
	FAnsiStringView Method = Activity.GetMethod();
	check(!Method.IsEmpty());

	FRequestParams RequestParams = {
		.bAutoRedirect = true,
	};

	FRequest ForwardRequest;
	if (!Location.StartsWith("http://") && !Location.StartsWith("https://"))
	{
		if (Location[0] != '/')
		{
			return false;
		}

		const FHost& Host = *(Activity.GetHost());

		TAnsiStringBuilder<256> Url;
		Url << ((Host.GetVerifyCert() != ECertRootsRefType::None) ? "https" : "http");
		Url << "://";
		Url << Host.GetHostName();
		Url << ":" << Host.GetPort();
		Url << Location;

		RequestParams.VerifyCert = Host.GetVerifyCert();
		new (&ForwardRequest) FRequest(Request(Method, Url, &RequestParams));
	}
	else
	{
		new (&ForwardRequest) FRequest(Request(Method, Location, &RequestParams));
	}

	// Transfer original request headers
	Activity.EnumerateHeaders([&ForwardRequest] (FAnsiStringView Name, FAnsiStringView Value)
	{
		ForwardRequest.Header(Name, Value);
		return true;
	});

	// Send the request
	Send(MoveTemp(ForwardRequest), MoveTemp(OuterSink), Status.GetParam());

	// todo: activity slots should be swapped so original slot matches ticket

	return true;
}

////////////////////////////////////////////////////////////////////////////////
FTicket FEventLoop::Send(FRequest&& Request, FTicketSink Sink, UPTRINT SinkParam)
{
	FActivityNode* Activity = nullptr;
	Swap(Activity, Request.Ptr);

	// Intercept sink calls to catch 30x status codes and follow them
	if (Activity->bFollow30x) //-V522
	{
		auto RedirectSink = [
				this,
				OuterSink=MoveTemp(Sink)
			] (const FTicketStatus& Status) mutable
		{
			if (Status.GetId() == FTicketStatus::EId::Response)
			{
				if (Redirect(Status, OuterSink))
				{
					return;
				}
			}

			if (OuterSink)
			{
				return OuterSink(Status);
			}
		};
		Sink = MoveTemp(RedirectSink);
	}

	Activity->SetSink(MoveTemp(Sink), SinkParam);
	return Impl->Send(Activity);
}

// }}}

} // namespace UE::IoStore::HTTP

