// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::IoStore::HTTP
{

// {{{1 trace ..................................................................

////////////////////////////////////////////////////////////////////////////////
enum class ETrace
{
	LoopCreate,
	LoopTick,
	LoopDestroy,
	ActivityCreate,
	ActivityDestroy,
	SocketCreate,
	SocketDestroy,
	RequestBegin,
	StateChange,
	Wait,
	Unwait,
	Connect,
	Send,
	Recv,
	StartWork,
};

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_ENABLED

static void Trace(const struct FActivity*, ETrace, uint32);
static void Trace(UPTRINT, ETrace Action, const class FOutcome* =nullptr);

static void Trace(ETrace Action)					{}
static void Trace(const void*, ETrace Action, ...)	{}

////////////////////////////////////////////////////////////////////////////////
IOSTOREHTTPCLIENT_API const void* GetIaxTraceChannel()
{
	UE_TRACE_CHANNEL(Iax);
	return &Iax;
}

#else

static void							Trace(...)				{}
IOSTOREHTTPCLIENT_API const void*	GetIaxTraceChannel()	{ return nullptr; }

#endif // UE_TRACE_ENABLED



// {{{1 misc ...................................................................

////////////////////////////////////////////////////////////////////////////////
#define IAS_CVAR(Type, Name, Default, Desc, ...) \
	Type G##Name = Default; \
	static FAutoConsoleVariableRef CVar_Ias##Name( \
		TEXT("ias.Http" #Name), \
		G##Name, \
		TEXT(Desc) \
		__VA_ARGS__ \
	)

////////////////////////////////////////////////////////////////////////////////
static IAS_CVAR(int32, IdleMs, 50'000, "Time in milliseconds to close idle connections or fail waits");



////////////////////////////////////////////////////////////////////////////////
uint32 GetNowMs()
{
	// 4.2MM seconds will give us 50 days of uptime.
	static uint64 Freq = 0;
	static uint64 Base = 0;
	if (Freq == 0)
	{
		Freq = uint64(1.0 / FPlatformTime::GetSecondsPerCycle64());
		Base = FPlatformTime::Cycles64();
	}
	uint64 NowBig = ((FPlatformTime::Cycles64() - Base) * 1000) / Freq;
	uint32 NowMs = uint32(NowBig);
	check(NowMs == NowBig);
	return NowMs + 1;
	// +1 bias ----^ ...so 0 can be considered "not set"
}



////////////////////////////////////////////////////////////////////////////////
class FOutcome
{
public:
	static FOutcome Ok(int32 Result=0);
	static FOutcome Waiting(int32 Result=0);
	static FOutcome WaitBuffer(int32 Result=0);
	static FOutcome WaitStream(int32 Result=0);
	static FOutcome Error(const char* Message, int32 Code=-1);
	static FOutcome None()				{ return Error(""); }
	bool			IsError() const		{ return Message < 0x8000'0000'0000; }
	bool			IsWaiting() const	{ return (Tag & WaitTag) == WaitTag; }
	bool			IsWaitData() const	{ check(IsWaiting()); return (Tag & ~WaitTag) == 0; }
	bool			IsWaitBuffer() const{ check(IsWaiting()); return (Tag & BufferTag) != 0; }
	bool			IsWaitStream() const{ check(IsWaiting()); return (Tag & StreamTag) != 0; }
	bool			IsOk() const		{ return Tag == OkTag; }
	FAnsiStringView GetMessage() const	{ check(IsError()); return (const char*)UPTRINT(Message); }
	int32			GetErrorCode() const{ check(IsError()); return int32(Code); }
	uint32			GetResult() const	{ check(!IsError()); return Result; }

private:
					FOutcome() = default;

	static uint32 const OkTag		= 0x0000'8000;
	static uint32 const WaitTag		= 0x0001'8000;
	static uint32 const BufferTag	= 0x0002'0000;
	static uint32 const StreamTag	= 0x0004'0000;

	union {
		struct {
			UPTRINT	Message : 48;
			PTRINT	Code	: 16;
		};
		struct {
			int32	Result;
			uint32	Tag;
		};
	};
};
static_assert(sizeof(FOutcome) == sizeof(void*));

////////////////////////////////////////////////////////////////////////////////
FOutcome FOutcome::Ok(int32 Result)
{
	FOutcome Outcome;
	Outcome.Tag = OkTag;
	Outcome.Result = Result;
	return Outcome;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FOutcome::Waiting(int32 Result)
{
	FOutcome Outcome;
	Outcome.Tag = WaitTag;
	Outcome.Result = Result;
	return Outcome;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FOutcome::WaitBuffer(int32 Result)
{
	// Given buffer space exhausted
	FOutcome Outcome;
	Outcome.Tag = WaitTag|BufferTag;
	Outcome.Result = Result;
	return Outcome;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FOutcome::WaitStream(int32 Result)
{
	// Need to switch to another stream
	FOutcome Outcome;
	Outcome.Tag = WaitTag|StreamTag;
	Outcome.Result = Result;
	return Outcome;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FOutcome::Error(const char* Message, int32 Code)
{
	check(Message != nullptr);
	check(Code <= 0xffff && Code >= -0xffff);

	FOutcome Outcome;
	Outcome.Message = UPTRINT(Message);
	Outcome.Code = int16(Code);
	return Outcome;
}



////////////////////////////////////////////////////////////////////////////////
template <uint32 Base=10>
static int64 CrudeToInt(FAnsiStringView View)
{
	static_assert(Base == 10 || Base == 16);

	// FCStringAnsi::* is not used to mitigate any locale hiccups. By
	// initialising 'Value' with MSB set we can detect cases where View did not
	// start with digits. This works as we won't be using this on huge numbers.
	int64 Value = 0x8000'0000'0000'0000ll;
	for (int32 c : View)
	{
		uint32 Digit = c - '0';
		if (Digit > 9u)
		{
			if constexpr (Base != 16)
			{
				break;
			}
			else
			{
				Digit = (c | 0x20) - 'a';
				if (Digit > uint32('f' - 'a'))
				{
					break;
				}
				Digit += 10;
			}
		}
		Value *= Base;
		Value += Digit;
	}
	return Value;
};

template <uint32 Base=10> static int64 CrudeToInt(const char*) = delete;

////////////////////////////////////////////////////////////////////////////////
struct FUrlOffsets
{
	struct Slice
	{
						Slice() = default;
						Slice(int32 l, int32 r) : Left(uint8(l)), Right(uint8(r)) {}
		FAnsiStringView	Get(FAnsiStringView Url) const { return Url.Mid(Left, Right - Left); }
						operator bool () const { return Left > 0; }
		int32			Len() const { return Right - Left; }
		uint8			Left;
		uint8			Right;
	};
	Slice				UserInfo;
	Slice				HostName;
	Slice				Port;
	uint8				Path;
	uint8				SchemeLength;
};

static int32 ParseUrl(FAnsiStringView Url, FUrlOffsets& Out)
{
	if (Url.Len() < 5)
	{
		return -1;
	}

	Out = {};

	const char* Start = Url.GetData();
	const char* Cursor = Start;

	// Scheme
	int32 i = 0;
	for (; i < 5; ++i)
	{
		if (uint32(Cursor[i] - 'a') > uint32('z' - 'a'))
		{
			break;
		}
	}

	Out.SchemeLength = uint8(i);
	FAnsiStringView Scheme = Url.Left(i);
	if (Scheme != "http" && Scheme != "https")
	{
		return -1;
	}

	// Separator and authority
	if (Cursor[i] != ':' || Cursor[i + 1] != '/' || Cursor[i + 2] != '/')
	{
		return -1;
	}
	i += 3;

	struct { int32 c; int32 i; } Seps[2];
	int32 SepCount = 0;
	for (; i < Url.Len(); ++i)
	{
		int32 c = Cursor[i];
		if (c < '-')							break;
		if (c != ':' && c != '@' && c != '/')	continue;
		if (c == '/' || SepCount >= 2)			break;

		if (c == '@' && SepCount)
		{
			SepCount -= (Seps[SepCount - 1].c == ':');
		}
		Seps[SepCount++] = { c, i };
	}

	if (i > 0xff || i <= Scheme.Len() + 3)
	{
		return -1;
	}

	if (i < Url.Len())
	{
		Out.Path = uint8(i);
	}

	Out.HostName = { uint8(Scheme.Len() + 3), uint8(i) };

	switch (SepCount)
	{
	case 0:
		break;

	case 1:
		if (Seps[0].c == ':')
		{
			Out.Port = { Seps[0].i + 1, i };
			Out.HostName.Right = uint8(Seps[0].i);
		}
		else
		{
			Out.UserInfo = { Out.HostName.Left, Seps[0].i };
			Out.HostName.Left += uint8(Seps[0].i + 1);
			Out.HostName.Right += uint8(Seps[0].i + 1);
		}
		break;

	case 2:
		if ((Seps[0].c != '@') | (Seps[1].c != ':'))
		{
			return -1;
		}
		Out.UserInfo = { Out.HostName.Left, Seps[0].i };
		Out.Port.Left = uint8(Seps[1].i + 1);
		Out.Port.Right = Out.HostName.Right;
		Out.HostName.Left = Out.UserInfo.Right + 1;
		Out.HostName.Right = Out.Port.Left - 1;
		break;

	default:
		return -1;
	}

	bool Bad = false;
	Bad |= (Out.HostName.Len() == 0);
	Bad |= bool(int32(Out.UserInfo) & (Out.UserInfo.Len() == 0));

	if (Out.Port.Left)
	{
		Bad |= (Out.Port.Len() == 0);
		for (int32 j = 0, n = Out.Port.Len(); j < n; ++j)
		{
			Bad |= (uint32(Start[Out.Port.Left + j] - '0') > 9);
		}
	}

	return Bad ? -1 : 1;
}



// {{{1 buffer .................................................................

////////////////////////////////////////////////////////////////////////////////
class FBuffer
{
public:
	struct FMutableSection
	{
		char*		Data;
		uint32		Size;
	};

								FBuffer() = default;
								FBuffer(char* InData, uint32 InMax);
								~FBuffer();
	void						Resize(uint32 Size);
	const char*					GetData() const;
	uint32						GetSize() const;
	template <typename T> T*	Alloc(uint32 Count=1);
	FMutableSection				GetMutableFree(uint32 MinSize, uint32 PageSize=256);
	void						AdvanceUsed(uint32 Delta);

private:
	uint32						GetCapacity() const;
	char*						GetDataPtr();
	void						Extend(uint32 AtLeast, uint32 PageSize);
	UPTRINT						Data;
	uint32						Max = 0;
	union
	{
		struct
		{
			uint32				Used : 31;
			uint32				Inline : 1;
		};
		uint32					UsedInline = 0;
	};

private:
								FBuffer(FBuffer&&) = delete;
								FBuffer(const FBuffer&) = delete;
	FBuffer&					operator = (const FBuffer&) = delete;
	FBuffer&					operator = (FBuffer&& Rhs) = delete;
};

////////////////////////////////////////////////////////////////////////////////
FBuffer::FBuffer(char* InData, uint32 InMax)
: Data(UPTRINT(InData))
, Max(InMax)
{
	Inline = 1;
}

////////////////////////////////////////////////////////////////////////////////
FBuffer::~FBuffer()
{
	if (Data && !Inline)
	{
		FMemory::Free(GetDataPtr());
	}
}

////////////////////////////////////////////////////////////////////////////////
void FBuffer::Resize(uint32 Size)
{
	check(Size <= Max);
	Used = Size;
}

////////////////////////////////////////////////////////////////////////////////
char* FBuffer::GetDataPtr()
{
	return (char*)Data;
}

////////////////////////////////////////////////////////////////////////////////
const char* FBuffer::GetData() const
{
	return (char*)Data;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FBuffer::GetSize() const
{
	return Used;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FBuffer::GetCapacity() const
{
	return Max;
}

////////////////////////////////////////////////////////////////////////////////
template <typename T>
T* FBuffer::Alloc(uint32 Count)
{
	uint32 AlignBias = uint32(Data + Used) & (alignof(T) - 1);
	if (AlignBias)
	{
		AlignBias = alignof(T) - AlignBias;
	}

	uint32 PotentialUsed = Used + AlignBias + (sizeof(T) * Count);
	if (PotentialUsed > Max)
	{
		Extend(PotentialUsed, 256);
	}

	void* Ret = GetDataPtr() + Used + AlignBias;
	Used = PotentialUsed;
	return (T*)Ret;
}

////////////////////////////////////////////////////////////////////////////////
FBuffer::FMutableSection FBuffer::GetMutableFree(uint32 MinSize, uint32 PageSize)
{
	MinSize = (MinSize == 0 && Used == Max) ? PageSize : MinSize;

	uint32 PotentialUsed = Used + MinSize;
	if (PotentialUsed > Max)
	{
		Extend(PotentialUsed, PageSize);
	}

	return FMutableSection{ GetDataPtr() + Used, Max - Used };
}

////////////////////////////////////////////////////////////////////////////////
void FBuffer::AdvanceUsed(uint32 Delta)
{
	Used += Delta;
	check(Used <= Max);
}

////////////////////////////////////////////////////////////////////////////////
void FBuffer::Extend(uint32 AtLeast, uint32 PageSize)
{
	checkSlow(((PageSize - 1) & PageSize) == 0);

	--PageSize;
	Max = (AtLeast + PageSize) & ~PageSize;

	if (!Inline)
	{
		Data = UPTRINT(FMemory::Realloc(GetDataPtr(), Max, alignof(FBuffer)));
		return;
	}

	const char* PrevData = GetDataPtr();
	Data = UPTRINT(FMemory::Malloc(Max, alignof(FBuffer)));
	::memcpy(GetDataPtr(), PrevData, Used);
	Inline = 0;
}



// {{{1 throttler ..............................................................

////////////////////////////////////////////////////////////////////////////////
static void ThrottleTest(FAnsiStringView);

////////////////////////////////////////////////////////////////////////////////
class FThrottler
{
public:
			FThrottler();
	void	SetLimit(uint32 KiBPerSec);
	int32	GetAllowance();
	void	ReturnUnused(int32 Unused);

private:
	friend	void ThrottleTest(FAnsiStringView);
	int32	GetAllowance(int64 CycleDelta);
	int64	CycleFreq;
	int64	CycleLast = 0;
	int64	CyclePeriod = 0;
	uint32	Limit = 0;

	enum : uint32 {
		LIMITLESS	= MAX_int32,
		SLICES_POW2	= 5,
	};
};

////////////////////////////////////////////////////////////////////////////////
FThrottler::FThrottler()
{
	CycleFreq = int64(1.0 / FPlatformTime::GetSecondsPerCycle64());
	check(CycleFreq >> SLICES_POW2);
}

////////////////////////////////////////////////////////////////////////////////
void FThrottler::SetLimit(uint32 KiBPerSec)
{
	// 512MiB/s might as well be limitless.
	KiBPerSec = (KiBPerSec < (512 << 10)) ? KiBPerSec : 0;
	if (KiBPerSec)
	{
		KiBPerSec = FMath::Max(KiBPerSec, 1u << SLICES_POW2);
	}
	Limit = KiBPerSec << 10;
}

////////////////////////////////////////////////////////////////////////////////
int32 FThrottler::GetAllowance()
{
	int64 Cycle = FPlatformTime::Cycles64();
	int64 CycleDelta = Cycle - CycleLast;
	CycleLast = Cycle;
	return GetAllowance(CycleDelta);
}

////////////////////////////////////////////////////////////////////////////////
int32 FThrottler::GetAllowance(int64 CycleDelta)
{
	if (Limit == 0)
	{
		return LIMITLESS;
	}

	int64 CycleSlice = CycleFreq >> SLICES_POW2;
	CycleDelta = FMath::Min(CycleDelta, CycleSlice);
	CyclePeriod -= CycleDelta;
	if (CyclePeriod > 0)
	{
		return 0 - int32((CyclePeriod * 1000ll) / CycleFreq);
	}
	CyclePeriod += CycleSlice;

	int32 Released = Limit >> SLICES_POW2;
	return Released;
}

////////////////////////////////////////////////////////////////////////////////
void FThrottler::ReturnUnused(int32 Unused)
{
	if (Limit == 0 || Unused == 0)
	{
		return;
	}

	int64 CycleReturn = (CycleFreq * Unused) / Limit;
	CycleLast -= CycleReturn;
}

// }}}

} // namespace UE::IoStore::HTTP
