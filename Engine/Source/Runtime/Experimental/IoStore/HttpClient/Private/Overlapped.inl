// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::IoStore::HTTP
{

#if 1

using SocketType = SOCKET;

#else

////////////////////////////////////////////////////////////////////////////////
using SocketType	= struct FWsaSocket*;
using PollType		= HANDLE;

static int32 Poll_CollectCompletions(PollType, int32);

////////////////////////////////////////////////////////////////////////////////
struct FWsaSocket
{
public:
	static constexpr uint16 Flag_Overlapped		= 0x1000;
	static constexpr uint16 Flag_Connecting		= 0x0001;
	static constexpr uint16 Flag_Sending		= 0x0010;
	static constexpr uint16 Flag_Recving		= 0x0100;
	static constexpr uint16 Flag_Busy			= Flag_Connecting|Flag_Sending|Flag_Recving;

					FWsaSocket(SOCKET InSocket);
					~FWsaSocket();
	bool			IsAsync() const;
	void			Cancel(bool bAndComplete=false);
	int32			Connect(const sockaddr* Addr, socklen_t AddrLen);
	int32			Send(const char* Data, uint32 DataSize, uint32 SendFlags);
	int32			Recv(char* Data, uint32 DataSize, uint32 RecvFlags);
	int32			Shutdown(int32 How);
	int32			GetSockOpt(int32 Level, int32 Name, char* Dest, int32* Size);
	int32			SetSockOpt(int32 Level, int32 Name, const char* Dest, int32 Size);
	void			OnCompletion(uint32 Transfered, WSAOVERLAPPED& Overlap, uint32 Result);
	WSAOVERLAPPED	SendOverlap = {};
	WSAOVERLAPPED	RecvOverlap = {};
	SOCKET			Socket;
	PollType		Iocp;
	uint16			Flags = 0;
	int16			PollEvents = 0;
};

////////////////////////////////////////////////////////////////////////////////
FWsaSocket::FWsaSocket(SOCKET InSocket)
: Socket(InSocket)
{
}

////////////////////////////////////////////////////////////////////////////////
FWsaSocket::~FWsaSocket()
{
	Cancel(true);
	closesocket(Socket);
}

////////////////////////////////////////////////////////////////////////////////
bool FWsaSocket::IsAsync() const
{
	return (Flags & Flag_Overlapped) != 0;
}

////////////////////////////////////////////////////////////////////////////////
void FWsaSocket::Cancel(bool bAndComplete)
{
	if ((Flags & Flag_Busy) == 0)		return;
	if ((Flags & Flag_Overlapped) == 0)	return;

	CancelIo(HANDLE(Socket));

	if (!bAndComplete)
	{
		return;
	}

	for (int32 i = 1; i; ::SwitchToThread(), i = (Flags & Flag_Busy))
	{
		if (Poll_CollectCompletions(Iocp, 0) < 0)
		{
			return;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
void FWsaSocket::OnCompletion(uint32 Transfered, WSAOVERLAPPED& Overlap, uint32 Result)
{
	check((Flags & ~(Flags & (Flag_Busy|Flag_Overlapped))) == 0);

	if (Result != ERROR_SUCCESS)
	{
		if (Result >= 0xffff)
		{
			Result = (Result & 0x3fff) | ((Result >> 16) & 0xc000);
		}

		Flags &= ~(Flag_Connecting|Flag_Sending|Flag_Recving);
		Overlap.Offset = ~0u;
		Overlap.OffsetHigh = Result;
		PollEvents |= POLLERR;
		return;
	}

	bool IsSend = (&Overlap == &SendOverlap);

	if (Transfered != 0)
	{
		Flags &= ~(IsSend ? Flag_Sending : Flag_Recving);
		PollEvents |= IsSend ? POLLOUT : POLLIN;
		Overlap.Offset = Transfered;
		return;
	}

	if (Flags & Flag_Connecting)
	{
		check(IsSend);
		Flags &= ~Flag_Connecting;
		PollEvents |= POLLOUT;
		return;
	}

	OnCompletion(0, Overlap, WSAECONNRESET);
}

////////////////////////////////////////////////////////////////////////////////
int32 FWsaSocket::Connect(const sockaddr* Addr, socklen_t AddrLen)
{
	check((Flags & FWsaSocket::Flag_Connecting) == 0);

	ON_SCOPE_EXIT {
		if (WSAGetLastError() == WSAEWOULDBLOCK)
		{
			Flags |= FWsaSocket::Flag_Connecting;
		}
	};

	/*
	if (true)
	{
		int32 Value = 1;
		DWORD Unused = 0;
		WSAIoctl(Socket, SIO_LOOPBACK_FAST_PATH, &Value, sizeof(Value), nullptr,
			0, &Unused, nullptr, nullptr);
	}
	*/

	if ((Flags & Flag_Overlapped) == 0)
	{
		return connect(Socket, Addr, AddrLen);
	}

	static LPFN_CONNECTEX ConnectFunc = nullptr;
	if (ConnectFunc == nullptr)
	{
		DWORD BytesWritten = 0;
		GUID FuncGuid = WSAID_CONNECTEX;
		int32 Result = WSAIoctl(
			Socket,
			SIO_GET_EXTENSION_FUNCTION_POINTER,
			&FuncGuid, sizeof(FuncGuid),
			&ConnectFunc, sizeof(ConnectFunc),
			&BytesWritten,
			nullptr, nullptr
		);

		if (Result == SOCKET_ERROR || ConnectFunc == nullptr)
		{
			return -1;
		}
	}

	struct sockaddr_in LocalAddr = {};
	LocalAddr.sin_family = AF_INET;
	LocalAddr.sin_addr.s_addr = INADDR_ANY;
	LocalAddr.sin_port = 0;
	if (bind(Socket, (sockaddr*)(&LocalAddr), sizeof(LocalAddr)) < 0)
	{
		return -1;
	}

	DWORD Unused;
	ConnectFunc(Socket, Addr, AddrLen, nullptr, 0, &Unused, &SendOverlap);

	if (WSAGetLastError() == WSA_IO_PENDING)
	{
		WSASetLastError(WSAEWOULDBLOCK);
	}

	return -1;
}

////////////////////////////////////////////////////////////////////////////////
int32 FWsaSocket::Send(const char* Data, uint32 DataSize, uint32 SendFlags)
{
	if (uint32 OverlapResult = SendOverlap.Offset; OverlapResult != 0)
	{
		WSASetLastError(SendOverlap.OffsetHigh);
		SendOverlap.Offset = 0;
		SendOverlap.OffsetHigh = 0;
		return int32(OverlapResult);
	}

	if (Flags & (Flag_Sending|Flag_Connecting))
	{
		WSASetLastError(WSAEWOULDBLOCK);
		return -1;
	}

	WSAOVERLAPPED* Overlap = nullptr;
	if (Flags & FWsaSocket::Flag_Overlapped)
	{
		check((PollEvents & (POLLOUT|POLLERR)) == 0);
		Overlap = &SendOverlap;
	}

	DWORD BytesOut;
	WSABUF Buffer = { DataSize, (char*)Data };
	uint32 Result = WSASend(Socket, &Buffer, 1, &BytesOut, 0, Overlap, nullptr);
	if (Overlap == nullptr)
	{
		return (Result == 0) ? BytesOut : -1;
	}

	if (Result == 0 || WSAGetLastError() == WSA_IO_PENDING)
	{
		WSASetLastError(WSAEWOULDBLOCK);
		Flags |= Flag_Sending;
	}

	return -1;
}

////////////////////////////////////////////////////////////////////////////////
int32 FWsaSocket::Recv(char* Data, uint32 DataSize, uint32 RecvFlags)
{
	if (uint32 OverlapResult = RecvOverlap.Offset; OverlapResult != 0)
	{
		WSASetLastError(RecvOverlap.OffsetHigh);
		RecvOverlap.Offset = 0;
		RecvOverlap.OffsetHigh = 0;
		return OverlapResult;
	}

	if (Flags & (Flag_Recving|Flag_Connecting))
	{
		WSASetLastError(WSAEWOULDBLOCK);
		return -1;
	}

	WSAOVERLAPPED* Overlap = nullptr;
	if (Flags & FWsaSocket::Flag_Overlapped)
	{
		check((PollEvents & (POLLIN|POLLERR)) == 0);
		Overlap = &RecvOverlap;
	}

	DWORD DwFlags = RecvFlags;
	DWORD BytesIn = 0;
	WSABUF Buffer = { DataSize, Data };
	uint32 Result = WSARecv(Socket, &Buffer, 1, &BytesIn, &DwFlags, Overlap, nullptr);
	if (Overlap == nullptr)
	{
		return (Result == 0) ? BytesIn : -1;
	}

	if (Result == 0 || WSAGetLastError() == WSA_IO_PENDING)
	{
		WSASetLastError(WSAEWOULDBLOCK);
		Flags |= Flag_Recving;
	}

	return -1;
}

////////////////////////////////////////////////////////////////////////////////
int32 FWsaSocket::Shutdown(int32 How)
{
	return shutdown(Socket, How);
}

////////////////////////////////////////////////////////////////////////////////
int32 FWsaSocket::GetSockOpt(int32 Level, int32 Name, char* Dest, int32* Size)
{
	return getsockopt(Socket, Level, Name, Dest, Size);
}

////////////////////////////////////////////////////////////////////////////////
int32 FWsaSocket::SetSockOpt(int32 Level, int32 Name, const char* Dest, int32 Size)
{
	return setsockopt(Socket, Level, Name, Dest, Size);
}



////////////////////////////////////////////////////////////////////////////////
static IAS_CVAR(
	int32,
	IasEnableOverlappedIo,
	true,
	"Toggles the use of overlapped IO on Microsoft platforms"
);

////////////////////////////////////////////////////////////////////////////////
static SocketType Socket_Socket(int Family, int Type, int Protocol)
{
	DWORD Flags = WSA_FLAG_OVERLAPPED;
	SOCKET Socket = WSASocketW(Family, Type, Protocol, nullptr, 0, Flags);
	if (Socket == INVALID_SOCKET)
	{
		return SocketType(-1ll); // InvalidSocket
	}

	FWsaSocket* Inner = new FWsaSocket(Socket);
	if (!GIasEnableOverlappedIo)
	{
		return SocketType(Inner);
	}

	return SocketType(Inner);
}

////////////////////////////////////////////////////////////////////////////////
static void Socket_Close(SocketType Socket)
{
	delete Socket;
}

////////////////////////////////////////////////////////////////////////////////
static bool Socket_IsAsync(SocketType Socket)
{
	if (!GIasEnableOverlappedIo || PTRINT(Socket) <= 0)
	{
		return false;
	}

	auto* Inner = (FWsaSocket*)(Socket);
	return Inner->IsAsync();
}

////////////////////////////////////////////////////////////////////////////////
#define WSASOCK_FORWARD(Method, Fallback) \
	template <typename... ArgTypes> \
	static auto Socket_##Method(SocketType Socket, ArgTypes&&... Args) { \
		if (PTRINT(Socket) <= 0) return -1; \
		auto* Inner = (FWsaSocket*)(Socket); \
		if (!GIasEnableOverlappedIo) \
			return Fallback(Inner->Socket, Forward<ArgTypes>(Args)...); \
		return Inner->Method(Forward<ArgTypes>(Args)...); \
	}
	WSASOCK_FORWARD(Connect,	connect)
	WSASOCK_FORWARD(Send,		send)
	WSASOCK_FORWARD(Recv,		recv)
	WSASOCK_FORWARD(Shutdown,	shutdown)
	WSASOCK_FORWARD(GetSockOpt,	getsockopt)
	WSASOCK_FORWARD(SetSockOpt,	setsockopt)
#undef WSASOCK_FORWARD



////////////////////////////////////////////////////////////////////////////////
static bool SetNonBlockingSocket(SocketType Socket, bool bEnabled)
{
#	define IAS_HTTP_HAS_NONBLOCK_IMPL

	auto* Inner = (FWsaSocket*)(Socket);
	if (bEnabled && GIasEnableOverlappedIo)
	{
		Inner->Flags |= FWsaSocket::Flag_Overlapped;

		int32 BufSize = 0;
		setsockopt(Inner->Socket, SOL_SOCKET, SO_SNDBUF, &(char&)BufSize, sizeof(BufSize));
		setsockopt(Inner->Socket, SOL_SOCKET, SO_RCVBUF, &(char&)BufSize, sizeof(BufSize));
		bEnabled = false;
	}
	else
	{
		Inner->Flags &= ~FWsaSocket::Flag_Overlapped;
	}

	unsigned long NonBlockingMode = (bEnabled == true);
	return (ioctlsocket(Inner->Socket, FIONBIO, &NonBlockingMode) != SOCKET_ERROR);
}



////////////////////////////////////////////////////////////////////////////////
static PollType Poll_CreateContext(uint32 ConcurrencyHint=1)
{
	if (!GIasEnableOverlappedIo)
	{
		return 0;
	}

	return CreateIoCompletionPort(INVALID_HANDLE_VALUE, nullptr, 0, ConcurrencyHint);
}

////////////////////////////////////////////////////////////////////////////////
static void Poll_DestroyContext(PollType Iocp)
{
	if (Iocp == 0)
	{
		return;
	}

	CloseHandle(Iocp);
}

////////////////////////////////////////////////////////////////////////////////
static bool Poll_Register(PollType Iocp, SocketType Socket)
{
	if (Iocp == 0)
	{
		return !GIasEnableOverlappedIo;
	}

	Socket->Iocp = Iocp;

	auto Key = ULONG_PTR(Socket);
	auto Fd = HANDLE(Socket->Socket);
	HANDLE Result = CreateIoCompletionPort(Fd, Iocp, Key, 0);
	return (Result != nullptr);
}

////////////////////////////////////////////////////////////////////////////////
static int32 Poll_CollectCompletions(PollType Iocp, int32 TimeOutMs)
{
	DWORD EntryCount = 0;
	OVERLAPPED_ENTRY Entries[16];
	if (!GetQueuedCompletionStatusEx(Iocp, Entries, UE_ARRAY_COUNT(Entries), &EntryCount, TimeOutMs, false))
	{
		return -1;
	}

	for (const OVERLAPPED_ENTRY& Entry : MakeArrayView(Entries, EntryCount))
	{
		uint32 Result = uint32(Entry.Internal);
		auto& Socket = *(FWsaSocket*)(Entry.lpCompletionKey);
		auto& Overlap = *(WSAOVERLAPPED*)(Entry.lpOverlapped);
		Socket.OnCompletion(Entry.dwNumberOfBytesTransferred, Overlap, Result);
	}

	return EntryCount;
}

////////////////////////////////////////////////////////////////////////////////
static int32 Poll_Wait(PollType Iocp, WSAPOLLFD* const Polls, uint32 Num, int32 TimeOutMs)
{
	static_assert(sizeof(WSAPOLLFD::fd) == sizeof(FWsaSocket*));

	if (!GIasEnableOverlappedIo)
	{
		for (WSAPOLLFD& Poll : MakeArrayView(Polls, Num))
		{
			auto* Socket = (FWsaSocket*)(Poll.fd);
			Poll.fd = Socket->Socket;
		}
		return poll(Polls, Num, TimeOutMs);
	}

	if (Poll_CollectCompletions(Iocp, TimeOutMs) < 0)
	{
		return (GetLastError() == WAIT_TIMEOUT) - 1;
	}

	int32 Ret = 0;
	for (WSAPOLLFD& Poll : MakeArrayView(Polls, Num))
	{
		auto* Socket = (FWsaSocket*)(Poll.fd);
		Poll.revents = Socket->PollEvents;
		Socket->PollEvents = 0;
		Ret += (Poll.revents != 0);
	}

	/*
	if (EntryCount == UE_ARRAY_COUNT(Entries))
	{
		return Poll_Wait(Iocp, Polls, Num, TimeOutMs);
	}
	*/

	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
IAS_HTTP_SOCKET_FORWARD(FreeAddrInfo,	freeaddrinfo)
IAS_HTTP_SOCKET_FORWARD(GetAddrInfo,	getaddrinfo)
#undef IAS_HTTP_SOCKET_FORWARD

#endif // 0

} // namespace UE::IoStore::HTTP
