// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !defined(IAS_HTTP_HAS_OPENSSL)
#	define IAS_HTTP_HAS_OPENSSL 0
#endif

#if IAS_HTTP_HAS_OPENSSL
#	if defined(IAS_HTTP_EXPLICIT_VERIFY_TIME)
#		if !IAS_HTTP_EXPLICIT_VERIFY_TIME 
#			error Either define this to >=1 or not at all
#		endif
#		include <HAL/PlatformTime.h>
#		include <ctime>
#	endif
#	include <openssl/engine.h>
#	include <openssl/err.h>
#	include <openssl/ssl.h>
#	include <openssl/store.h>

#	if PLATFORM_WINDOWS
#		include "Windows/AllowWindowsPlatformTypes.h"
#			include <wincrypt.h>
#		include "Windows/HideWindowsPlatformTypes.h"
#	endif
#endif

namespace UE::IoStore::HTTP
{

////////////////////////////////////////////////////////////////////////////////
static FCertRoots GDefaultCertRoots;

struct ECertRootsRefType
{
	static const FCertRootsRef	None	= 0;
	static const FCertRootsRef	Default = ~0ull;
};



////////////////////////////////////////////////////////////////////////////////
class FPeer
{
public:
				FPeer() = default;
				FPeer(FSocket InSocket);
	uint64		GetRecvSize() const					{ return Socket.GetRecvSize(); }
	FWaitable	GetWaitable() const					{ return Socket.GetWaitable(); }
	FOutcome	Send(const char* Data, int32 Size)	{ return Socket.Send(Data, Size); }
	FOutcome	Recv(char* Out, int32 MaxSize)		{ return Socket.Recv(Out, MaxSize); }
	bool		IsValid() const						{ return Socket.IsValid(); }

private:
	FSocket		Socket;
};

////////////////////////////////////////////////////////////////////////////////
FPeer::FPeer(FSocket InSocket)
: Socket(MoveTemp(InSocket))
{
}



#if IAS_HTTP_HAS_OPENSSL

////////////////////////////////////////////////////////////////////////////////
static void Ssl_ContextDestroy(UPTRINT Handle)
{
	auto* Context = (SSL_CTX*)Handle;
	SSL_CTX_free(Context);
}

////////////////////////////////////////////////////////////////////////////////
static uint32 Ssl_ContextProvision(SSL_CTX* Context)
{
	uint32 NumAdded = 0;

#if PLATFORM_WINDOWS
	HANDLE StoreHandle = CertOpenSystemStoreW(0, L"ROOT");
	for (PCCERT_CONTEXT CertContext = nullptr;;)
	{
		CertContext = CertEnumCertificatesInStore(StoreHandle, CertContext);
		if (CertContext == nullptr)
		{
			break;
		}

		if ((CertContext->dwCertEncodingType & X509_ASN_ENCODING) == 0)
		{
			continue;
		}

		const uint8* CertData = (uint8*)(CertContext->pbCertEncoded);
		uint32 CertSize = CertContext->cbCertEncoded;
		X509* FiveOhNine = d2i_X509(nullptr, &CertData, CertSize);
		if (FiveOhNine == nullptr)
		{
			continue;
		}

		X509_STORE* Store = SSL_CTX_get_cert_store(Context);
		int32 Result = X509_STORE_add_cert(Store, FiveOhNine);
		NumAdded += (Result == 1);

		X509_free(FiveOhNine);
	}

	CertCloseStore(StoreHandle, 0);
#endif // PLATFORM_WINDOWS

	return NumAdded;
}

////////////////////////////////////////////////////////////////////////////////
static uint32 Ssl_ContextProvision(SSL_CTX* Context, FMemoryView PemData)
{
	const void* Data = PemData.GetData();
	uint32 Size = uint32(PemData.GetSize());
	BIO* Bio = BIO_new_mem_buf(Data, Size);

	uint32 NumAdded = 0;
	while (true)
	{
		X509* FiveOhNine = PEM_read_bio_X509(Bio, nullptr, 0, nullptr);
		if (FiveOhNine == nullptr)
		{
			break;
		}

		X509_STORE* Store = SSL_CTX_get_cert_store(Context);
		int32 Result = X509_STORE_add_cert(Store, FiveOhNine);
		NumAdded += (Result == 1);

		X509_free(FiveOhNine);
	}

	BIO_free(Bio);
	return NumAdded;
}

////////////////////////////////////////////////////////////////////////////////
static UPTRINT Ssl_ContextCreate(FMemoryView PemData)
{
	if (static bool InitOnce = false; !InitOnce)
	{
		// While OpenSSL will lazily initialise itself, the defaults used will fail
		// initialisation on some platforms. So we have a go here. We do not register
		// anything for clean-up as we do not know if anyone else has done so.
		uint64 InitOpts = OPENSSL_INIT_NO_ATEXIT;
		OPENSSL_init_ssl(InitOpts, nullptr);
		InitOnce = true;
	}

	auto* Method = TLS_client_method();
	SSL_CTX* Context = SSL_CTX_new(Method);
	checkf(Context != nullptr, TEXT("ERR_get_error() == %ld"), ERR_get_error());

	SSL_CTX_set_options(Context, SSL_OP_NO_SSLv2|SSL_OP_NO_SSLv3);

	uint32 NumAdded;
	if (!PemData.IsEmpty())
	{
		NumAdded = Ssl_ContextProvision(Context, PemData);
	}
	else
	{
		NumAdded = Ssl_ContextProvision(Context);
	}

	if (NumAdded == 0)
	{
		SSL_CTX_free(Context);
		return 0;
	}

#if defined(IAS_HTTP_EXPLICIT_VERIFY_TIME)
	if (X509_VERIFY_PARAM* VerifyParam = SSL_CTX_get0_param(Context); VerifyParam != nullptr)
	{
		int32 AliasTown;
		std::tm Utc = {};
		FPlatformTime::UtcTime(
			Utc.tm_year, Utc.tm_mon,
			AliasTown,
			Utc.tm_mday, Utc.tm_hour, Utc.tm_min,
			AliasTown, AliasTown
		);

		Utc.tm_year -= 1900;
		Utc.tm_mon -= 1;

		time_t Now = std::mktime(&Utc);

		X509_VERIFY_PARAM_set_time(VerifyParam, Now);
	}
#endif

	return UPTRINT(Context);
}

////////////////////////////////////////////////////////////////////////////////
static int32 Ssl_ContextCertNum(UPTRINT Handle)
{
	auto* Context = (SSL_CTX*)Handle;
	X509_STORE* Store = SSL_CTX_get_cert_store(Context);
	STACK_OF(X509_OBJECT)* Objects = X509_STORE_get0_objects(Store);
	return sk_X509_OBJECT_num(Objects);
}

////////////////////////////////////////////////////////////////////////////////
static int32 Ssl_BioWrite(BIO* Bio, const char* Data, size_t Size, size_t* BytesWritten)
{
	*BytesWritten = 0;
	BIO_clear_retry_flags(Bio);

	auto* Peer = (FPeer*)BIO_get_data(Bio);
	FOutcome Outcome = Peer->Send(Data, int32(Size));
	if (Outcome.IsWaiting())
	{
		BIO_set_retry_write(Bio);
		return 0;
	}

	if (Outcome.IsError())
	{
		return -1;
	}

	*BytesWritten = Outcome.GetResult();
	return 1;
}

////////////////////////////////////////////////////////////////////////////////
static int32 Ssl_BioRead(BIO* Bio, char* Data, size_t Size, size_t* BytesRead)
{
	*BytesRead = 0;
	BIO_clear_retry_flags(Bio);

	auto* Peer = (FPeer*)BIO_get_data(Bio);
	FOutcome Outcome = Peer->Recv(Data, int32(Size));
	if (Outcome.IsWaiting())
	{
		BIO_set_retry_read(Bio);
		return 0;
	}

	if (Outcome.IsError())
	{
		return -1;
	}

	*BytesRead = Outcome.GetResult();
	return 1;
}

////////////////////////////////////////////////////////////////////////////////
static long Ssl_BioControl(BIO*, int Cmd, long, void*)
{
	return (Cmd == BIO_CTRL_FLUSH) ? 1 : 0;
}

////////////////////////////////////////////////////////////////////////////////
static SSL* Ssl_Create(FCertRootsRef Certs, const char* HostName=nullptr)
{
	static_assert(OPENSSL_VERSION_NUMBER >= 0x10100000L, "version supporting autoinit required");

	if (Certs == ECertRootsRefType::Default)
	{
		Certs = FCertRoots::Explicit(GDefaultCertRoots);
		check(Certs != 0);
	}
	auto* Context = (SSL_CTX*)Certs;

	static BIO_METHOD* BioMethod = nullptr;
	if (BioMethod == nullptr)
	{
		int32 BioId = BIO_get_new_index() | BIO_TYPE_SOURCE_SINK;
		BioMethod = BIO_meth_new(BioId, "IasBIO");
		BIO_meth_set_write_ex(BioMethod, Ssl_BioWrite);
		BIO_meth_set_read_ex(BioMethod,	Ssl_BioRead);
		BIO_meth_set_ctrl(BioMethod, Ssl_BioControl);
	}

	BIO* Bio = BIO_new(BioMethod);

	// SSL_MODE_ENABLE_PARTIAL_WRITE ??!!!

	SSL* Ssl = SSL_new(Context);
	SSL_set_connect_state(Ssl);
	SSL_set0_rbio(Ssl, Bio);
	SSL_set0_wbio(Ssl, Bio);
	BIO_up_ref(Bio);

	if (HostName != nullptr)
	{
		SSL_set_tlsext_host_name(Ssl, HostName);
	}

	return Ssl;
}

////////////////////////////////////////////////////////////////////////////////
static void Ssl_Destroy(SSL* Ssl)
{
	SSL_free(Ssl);
}

////////////////////////////////////////////////////////////////////////////////
static void Ssl_AssociatePeer(SSL* Ssl, FPeer* Peer)
{
	BIO* Bio = SSL_get_rbio(Ssl);
	check(Bio == SSL_get_wbio(Ssl));
	BIO_set_data(Bio, Peer);
}

////////////////////////////////////////////////////////////////////////////////
static void Ssl_SetupAlpn(SSL* Ssl, int32 HttpVersion)
{
	FAnsiStringView AlpnProtos;
	switch (HttpVersion)
	{
	case 1:		AlpnProtos = "\x08" "http/1.1";	break;
	case 2:		AlpnProtos = "\x02" "h2";		break;
	default:	break;
	}

	check(!AlpnProtos.IsEmpty());
	SSL_set_alpn_protos(Ssl, (uint8*)(AlpnProtos.GetData()), int32(AlpnProtos.Len()));
}

////////////////////////////////////////////////////////////////////////////////
static int32 Ssl_GetProtocolVersion(SSL* Ssl)
{
	int32 Proto = 1;

	const char* AlpnProto = nullptr;
	uint32 AlpnProtoLen;
	SSL_get0_alpn_selected(Ssl, &(const uint8*&)AlpnProto, &AlpnProtoLen);
	if (AlpnProto == nullptr)
	{
		return Proto;
	}

	FAnsiStringView Needle(AlpnProto, AlpnProtoLen);
	FAnsiStringView Candidates[] = {
		"http/1.1",
		"h2",
	};
	for (uint32 i = 0; i < UE_ARRAY_COUNT(Candidates); ++i)
	{
		const FAnsiStringView& Candidate = Candidates[i];
		if (AlpnProtoLen != uint32(Candidate.Len()))
		{
			continue;
		}

		if (Candidate != Needle)
		{
			continue;
		}

		Proto = i + 1;
		break;
	}

	return Proto;
}

////////////////////////////////////////////////////////////////////////////////
static FOutcome Ssl_GetOutcome(SSL* Ssl, int32 SslResult, const char* Message="tls error")
{
	int32 Error = SSL_get_error(Ssl, SslResult);
	if (Error != SSL_ERROR_WANT_READ && Error != SSL_ERROR_WANT_WRITE)
	{
		return FOutcome::Error(Message, Error);
	}
	return FOutcome::Waiting();
}

////////////////////////////////////////////////////////////////////////////////
static FOutcome Ssl_Handshake(SSL* Ssl)
{
	int32 Result = SSL_do_handshake(Ssl);
	if (Result == 0) return FOutcome::Error("unsuccessful tls handshake");
	if (Result != 1) return Ssl_GetOutcome(Ssl, Result, "tls handshake error");

	Result = SSL_get_verify_result(Ssl);
	if (Result == X509_V_OK)
	{
		return FOutcome::Ok();
	}

	const char* Msg;
	switch (Result)
	{
	case X509_V_ERR_CERT_HAS_EXPIRED:			Msg = "certificate has expired (check local date)";		break;
	case X509_V_ERR_CERT_NOT_YET_VALID:			Msg = "certificate not yet valid (check local date)";	break;
	case X509_V_ERR_DEPTH_ZERO_SELF_SIGNED_CERT:Msg = "self-signed certificate";						break;
	case X509_V_ERR_SELF_SIGNED_CERT_IN_CHAIN:	Msg = "self-signed certificate in chain";				break;
	default:									Msg = "x509 verification error";						break;
	}
	return FOutcome::Error(Msg, Result);
}

////////////////////////////////////////////////////////////////////////////////
static FOutcome Ssl_Write(SSL* Ssl, const char* Data, int32 Size)
{
	int32 Result = SSL_write(Ssl, Data, Size);
	return (Result > 0) ? FOutcome::Ok(Result) : Ssl_GetOutcome(Ssl, Result);
}

////////////////////////////////////////////////////////////////////////////////
static FOutcome Ssl_Read(SSL* Ssl, char* Out, int32 MaxSize)
{
	int32 Result = SSL_read(Ssl, Out, MaxSize);
	return (Result > 0) ? FOutcome::Ok(Result) : Ssl_GetOutcome(Ssl, Result);
}

#else

struct SSL;
static void		Ssl_ContextDestroy(...)		{}
static UPTRINT	Ssl_ContextCreate(...)		{ return 0; }
static int32	Ssl_ContextCertNum(...)		{ return 0; }
static SSL*		Ssl_Create(...)				{ return nullptr; }
static void		Ssl_Destroy(...)			{}
static void		Ssl_AssociatePeer(...)		{}
static void		Ssl_SetupAlpn(...)			{}
static int32	Ssl_GetProtocolVersion(...)	{ return 1; }
static FOutcome	Ssl_Handshake(...)			{ return FOutcome::Error("!impl"); }
static FOutcome	Ssl_Write(...)				{ return FOutcome::Error("!impl"); }
static FOutcome	Ssl_Read(...)				{ return FOutcome::Error("!impl"); }

#endif // IAS_HTTP_HAS_OPENSSL



////////////////////////////////////////////////////////////////////////////////
FCertRoots::~FCertRoots()
{
	if (Handle != 0)
	{
		Ssl_ContextDestroy(Handle);
	}
}

////////////////////////////////////////////////////////////////////////////////
FCertRoots::FCertRoots(FMemoryView PemData)
{
	Handle = Ssl_ContextCreate(PemData);
}

////////////////////////////////////////////////////////////////////////////////
int32 FCertRoots::Num() const
{
	if (Handle == 0)
	{
		return -1;
	}

	return Ssl_ContextCertNum(Handle);
}

////////////////////////////////////////////////////////////////////////////////
void FCertRoots::SetDefault(FCertRoots&& CertRoots)
{
	check(GDefaultCertRoots.IsValid() != CertRoots.IsValid());
	GDefaultCertRoots = MoveTemp(CertRoots);
}

////////////////////////////////////////////////////////////////////////////////
FCertRootsRef FCertRoots::NoTls()
{
	return ECertRootsRefType::None;
}

////////////////////////////////////////////////////////////////////////////////
FCertRootsRef FCertRoots::Default()
{
	return ECertRootsRefType::Default;
}

////////////////////////////////////////////////////////////////////////////////
FCertRootsRef FCertRoots::Explicit(const FCertRoots& CertRoots)
{
	check(CertRoots.IsValid());
	return CertRoots.Handle;
}



////////////////////////////////////////////////////////////////////////////////
class FTlsPeer
	: public FPeer
{
public:
				FTlsPeer()								= default;
				~FTlsPeer();
				FTlsPeer(FTlsPeer&& Rhs)				{ Move(MoveTemp(Rhs)); }
				FTlsPeer& operator = (FTlsPeer&& Rhs)	{ return Move(MoveTemp(Rhs)); }
				FTlsPeer(FSocket InSocket, FCertRootsRef Certs=ECertRootsRefType::None, const char* HostName=nullptr);
	bool		IsUsingTls() const;
	FOutcome	Handshake();
	FOutcome	Send(const char* Data, int32 Size);
	FOutcome	Recv(char* Out, int32 MaxSize);

protected:
	FTlsPeer&	Move(FTlsPeer&& Rhs);
	SSL*		Ssl = nullptr;
};

////////////////////////////////////////////////////////////////////////////////
FTlsPeer::FTlsPeer(FSocket InSocket, FCertRootsRef Certs, const char* HostName)
: FPeer(MoveTemp(InSocket))
{
	if (Certs == ECertRootsRefType::None)
	{
		return;
	}

	Ssl = Ssl_Create(Certs, HostName);
	Ssl_AssociatePeer(Ssl, this);
}

////////////////////////////////////////////////////////////////////////////////
FTlsPeer::~FTlsPeer()
{
	if (Ssl != nullptr)
	{
		Ssl_Destroy(Ssl);
	}
}

////////////////////////////////////////////////////////////////////////////////
bool FTlsPeer::IsUsingTls() const
{
	return (Ssl != nullptr);
}

////////////////////////////////////////////////////////////////////////////////
FTlsPeer& FTlsPeer::Move(FTlsPeer&& Rhs)
{
	Swap(Ssl, Rhs.Ssl);
	if (Ssl != nullptr)		Ssl_AssociatePeer(Ssl, this);
	if (Rhs.Ssl != nullptr) Ssl_AssociatePeer(Rhs.Ssl, &Rhs);

	FPeer::operator = (MoveTemp(Rhs));

	return *this;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FTlsPeer::Handshake()
{
	if (Ssl == nullptr)
	{
		return FOutcome::Ok();
	}

	return Ssl_Handshake(Ssl);
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FTlsPeer::Send(const char* Data, int32 Size)
{
	if (Ssl == nullptr)
	{
		return FPeer::Send(Data, Size);
	}

	return Ssl_Write(Ssl, Data, Size);
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FTlsPeer::Recv(char* Out, int32 MaxSize)
{
	if (Ssl == nullptr)
	{
		return FPeer::Recv(Out, MaxSize);
	}

	return Ssl_Read(Ssl, Out, MaxSize);
}

} // namespace UE::IoStore::HTTP



////////////////////////////////////////////////////////////////////////////////
#include "TransactionTwo.inl"
#include "TransactionOne.inl"

namespace UE::IoStore::HTTP
{

////////////////////////////////////////////////////////////////////////////////
class FHttpPeer
	: public FTlsPeer
{
public:
	struct FParams
	{
		FSocket			Socket;
		FCertRootsRef 	Certs = ECertRootsRefType::None;
		const char* 	HostName = nullptr;
		EHttpVersion	HttpVersion = EHttpVersion::One;
	};

					FHttpPeer() = default;
					~FHttpPeer();
					FHttpPeer(FHttpPeer&& Rhs) = delete;
					FHttpPeer(const FHttpPeer& Rhs) = delete;
	FHttpPeer&		operator = (FHttpPeer&& Rhs);
	FHttpPeer&		operator = (const FHttpPeer& Rhs) = delete;
					FHttpPeer(FParams&& Params);
	uint32			GetVersion() const;
	FOutcome		Handshake();
	FTransactRef	Transact();
	bool			Cancel(FTransactId Id);
	FOutcome		GetPendingTransactId();
	void			SkipPendingTransactId();

private:
	void*			PeerData = nullptr;
	uint8			Proto = 0;
};

////////////////////////////////////////////////////////////////////////////////
FHttpPeer::FHttpPeer(FParams&& Params)
: FTlsPeer(MoveTemp(Params.Socket), Params.Certs, Params.HostName)
, Proto(uint8(Params.HttpVersion))
{
	if (Ssl != nullptr)
	{
		Ssl_SetupAlpn(Ssl, Proto);
	}
}

////////////////////////////////////////////////////////////////////////////////
FHttpPeer::~FHttpPeer()
{
	switch (Proto)
	{
	case 1:		GoAwayOne(*this, PeerData); break;
	case 2:		GoAwayTwo(*this, PeerData); break;
	default:	break;
	}
}

////////////////////////////////////////////////////////////////////////////////
FHttpPeer& FHttpPeer::operator = (FHttpPeer&& Rhs)
{
	Swap(PeerData, Rhs.PeerData);
	Swap(Proto, Rhs.Proto);

	FTlsPeer::operator = (MoveTemp(Rhs));

	return *this;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FHttpPeer::GetVersion() const
{
	return Proto;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FHttpPeer::Handshake()
{
	if (Ssl != nullptr)
	{
		if (FOutcome Outcome = FTlsPeer::Handshake(); !Outcome.IsOk())
		{
			return Outcome;
		}

		int32 Negotiated = Ssl_GetProtocolVersion(Ssl);
		if (Negotiated != 1 && Negotiated != 2)
		{
			return FOutcome::Error("Unexpected negotiated protocol version", Negotiated);
		}
		Proto = uint8(Negotiated);

		/* intentional fallthrough */
	}

	FOutcome Outcome = FOutcome::None();
	switch (Proto)
	{
	case 1:		Outcome = HandshakeOne(*this, PeerData); break;
	case 2:		Outcome = HandshakeTwo(*this, PeerData); break;
	default:	return FOutcome::Error("Unsupported");
	}

	if (!Outcome.IsOk())
	{
		return Outcome;
	}

	return FOutcome::Ok(Proto);
}

////////////////////////////////////////////////////////////////////////////////
FTransactRef FHttpPeer::Transact()
{
	switch (Proto)
	{
	case 1:		return CreateTransactOne(PeerData);
	case 2:		return CreateTransactTwo(PeerData);
	default:	return FTransactRef();
	}
}

////////////////////////////////////////////////////////////////////////////////
bool FHttpPeer::Cancel(FTransactId Id)
{
	switch (Proto)
	{
	case 1:		return false;
	case 2:		return CancelTwo(*this, PeerData, Id);
	default:	return false;
	}
}

////////////////////////////////////////////////////////////////////////////////
void FHttpPeer::SkipPendingTransactId()
{
	switch (Proto)
	{
	case 1:		return;
	case 2:		return SkipPendingTransactTwo(*this, PeerData);
	default:	return;
	}
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FHttpPeer::GetPendingTransactId()
{
	switch (Proto)
	{
	case 1:		return TickOne(*this, PeerData);
	case 2:		return TickTwo(*this, PeerData);
	default:	return FOutcome::Error("Unsupported");
	}
}

} // namespace UE::IoStore::HTTP
