// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// {{{1 platforms ..............................................................

#define IAS_HTTP_SOCKET_FORWARD(Src, Dest)			\
	template <typename... ArgTypes>					\
	static auto Socket_##Src(ArgTypes&&... Args) {	\
		return Dest(Forward<ArgTypes>(Args)...);	\
	}

#if PLATFORM_MICROSOFT
#	if !defined(NO_UE_INCLUDES)
#	include "Windows/AllowWindowsPlatformTypes.h"
#		include <winsock2.h>
#		include <ws2tcpip.h>
#		include <mswsock.h>
#	include "Windows/HideWindowsPlatformTypes.h"
#	else
#		include <winsock2.h>
#		include <ws2tcpip.h>
#		pragma comment(lib, "Ws2_32.lib")
#	endif // NO_UE_INCLUDES

	// Winsock defines "PF_MAX" indicating the protocol family count. This
	// however competes with UE definitions related to pixel formats.
	#if defined(PF_MAX)
	#	undef PF_MAX
	#endif

	namespace UE::IoStore::HTTP
	{
		using MsgFlagType	= int;

		enum {
			SHUT_RDWR		= SD_BOTH,
			MSG_NOSIGNAL	= 0,
		};

#	define IAS_HTTP_USE_POLL
		template <typename... ArgTypes>
		static auto poll(ArgTypes&&... Args)
		{
			return WSAPoll(Forward<ArgTypes>(Args)...);
		}
	} // namespace UE::IoStore::HTTP
#	include "Overlapped.inl"
#elif PLATFORM_APPLE | PLATFORM_UNIX | PLATFORM_ANDROID
#	include <arpa/inet.h>
#	include <fcntl.h>
#	include <netdb.h>
#	include <netinet/tcp.h>
#	include <poll.h>
#	include <sys/select.h>
#	include <sys/socket.h>
#	include <unistd.h>
	namespace UE::IoStore::HTTP
	{
		using SocketType	= int;
		using MsgFlagType	= int;
		static int32 closesocket(int32 Socket) { return close(Socket); }
	}
#	define IAS_HTTP_USE_POLL
#else
#	include "CoreHttp/Http.inl"
#endif

#if defined(IAS_HTTP_SOCKET_FORWARD)
	namespace UE::IoStore::HTTP {
		IAS_HTTP_SOCKET_FORWARD(Close,			closesocket)
		IAS_HTTP_SOCKET_FORWARD(Connect,		connect)
		IAS_HTTP_SOCKET_FORWARD(FreeAddrInfo,	freeaddrinfo)
		IAS_HTTP_SOCKET_FORWARD(GetAddrInfo,	getaddrinfo)
		IAS_HTTP_SOCKET_FORWARD(GetSockOpt,		getsockopt)
		IAS_HTTP_SOCKET_FORWARD(Recv,			recv)
		IAS_HTTP_SOCKET_FORWARD(Select,			select)
		IAS_HTTP_SOCKET_FORWARD(Send,			send)
		IAS_HTTP_SOCKET_FORWARD(SetSockOpt,		setsockopt)
		IAS_HTTP_SOCKET_FORWARD(Shutdown,		shutdown)
		IAS_HTTP_SOCKET_FORWARD(Socket,			socket)

		using PollType = UPTRINT;
		static PollType	Poll_CreateContext()			{ return 0; }
		static void		Poll_DestroyContext(PollType)	{}
		static bool		Poll_Register(...)				{ return true; }

#if defined(IAS_HTTP_USE_POLL)
		template <typename... ArgTypes>
		static auto Poll_Wait(PollType, ArgTypes&&... Args)
		{
			return poll(Forward<ArgTypes>(Args)...);
		}
#endif
	} // namespace UE::IoStore::HTTP
#endif // IAS_HTTP_SOCKET_FORWARD

namespace UE::IoStore::HTTP {

////////////////////////////////////////////////////////////////////////////////
static const SocketType InvalidSocket = SocketType(~0ull);

static_assert(sizeof(sockaddr_in::sin_addr) == sizeof(uint32));

#if PLATFORM_MICROSOFT
	static int32 LastSocketResult() { return WSAGetLastError(); }
#	define IsSocketResult(err)		(LastSocketResult() == WSA##err)
#else
	static int32 LastSocketResult() { return errno; }
#	define IsSocketResult(err)		int32(LastSocketResult() == err)
	static_assert(EWOULDBLOCK == EAGAIN);
#endif

// hton* can sometimes be implemented as a macro which does not work with the
// forward approach above. Hence we have our own here. Thanks for reading!
uint32 Socket_HtoNl(uint32 v) { return ByteSwap(v); }
uint16 Socket_HtoNs(uint16 v) { return ByteSwap(v); }

} // namespace UE::IoStore::HTTP

// }}}

namespace UE::IoStore::HTTP
{

// {{{1 wait ...................................................................

////////////////////////////////////////////////////////////////////////////////
class FWaitable
{
public:
				FWaitable() = default;
				FWaitable(FWaitable&&) = default;
				FWaitable& operator = (FWaitable&&) = default;

private:
	friend		class FSocket;
	friend		class FWaiter;
	friend		class FPoller;
				FWaitable(SocketType InSocket);
	SocketType	Socket = InvalidSocket;

private:
				FWaitable(const FWaitable&) = delete;
				FWaitable& operator = (const FWaitable&) = delete;
};

////////////////////////////////////////////////////////////////////////////////
FWaitable::FWaitable(SocketType InSocket)
: Socket(InSocket)
{
}



// {{{1 poll ...................................................................

////////////////////////////////////////////////////////////////////////////////
#if defined(IAS_HTTP_USE_POLL)
	using FPollFdBase = pollfd;
#else
	struct FPollFdBase
	{
		SocketType			fd;
		int16				events;
		int16				revents;
		static const int16	POLLIN  = 1 << 0;
		static const int16	POLLOUT = 1 << 1;
		static const int16	POLLERR = 1 << 2;
		static const int16	POLLHUP = POLLERR;
		static const int16	POLLNVAL= POLLERR;
	};
#endif

////////////////////////////////////////////////////////////////////////////////
struct FPollFd
	: public FPollFdBase
{
	static const int16	PollIn	= int16(POLLIN);
	static const int16	PollOut	= int16(POLLOUT);
	static const int16	PollErr	= int16(POLLERR);
	static const int16	PollHup	= int16(POLLHUP);
	static const int16	PollNVal= int16(POLLNVAL);
	static const int16	PollAll = PollIn|PollOut|PollErr|PollHup|PollNVal;

	FPollFd(SocketType Socket, int16 Events)
	{
		fd = FdType(Socket);
		events = EventType(Events);
		revents = EventType(0);
	}

	bool HasTrigger() const		{ return !!revents; }
	bool HasSendTrigger() const	{ return !!(revents & EventType(PollOut)); }
	bool HasRecvTrigger() const	{ return !!(revents & EventType(PollIn)); }

private:
	// The following is odd because POLLFD varies subtly from one platform to
	// the next. With typedefs we can cleanly set members and not get narrowing
	// warnings from the compiler.
	using FdType = decltype(fd);
	using EventType = decltype(events);
};

////////////////////////////////////////////////////////////////////////////////
class FPoller
{
public:
				FPoller()							{ Handle = Poll_CreateContext(); }
				FPoller(FPoller&& Rhs)				{ Swap(Handle, Rhs.Handle); }
				~FPoller()							{ Poll_DestroyContext(Handle); }
	FPoller&	operator = (FPoller&& Rhs)			{ Swap(Handle, Rhs.Handle); return *this; }
	bool		Register(const FWaitable& Waitable)	{ return Poll_Register(Handle, Waitable.Socket); }
	int32		Wait(FPollFd* Polls, int32 Num, int32 TimeOutMs);

private:
				FPoller(const FPoller&)		= delete;
	FPoller&	operator = (const FPoller&)	= delete;
	PollType	Handle = 0;
};

////////////////////////////////////////////////////////////////////////////////
int32 FPoller::Wait(FPollFd* Polls, int32 Num, int32 TimeOutMs)
{
#if defined(IAS_HTTP_USE_POLL)
	static_assert(sizeof(FPollFd) == sizeof(pollfd));
	return Poll_Wait(Handle, Polls, Num, TimeOutMs);
#else
	timeval TimeVal = {};
	timeval* TimeValPtr = (TimeOutMs >= 0 ) ? &TimeVal : nullptr;
	if (TimeOutMs > 0)
	{
		int32 TimeoutUs = TimeOutMs << 10;
		TimeVal = { TimeOutMs >> 10, TimeoutUs & ((1 << 20) - 1) };
	}

	fd_set FdSetRead;	FD_ZERO(&FdSetRead);
	fd_set FdSetWrite;	FD_ZERO(&FdSetWrite);
	fd_set FdSetExcept; FD_ZERO(&FdSetExcept);

	SocketType MaxFd = 0;
	for (FPollFd& Poll : MakeArrayView(Polls, Num))
	{
		fd_set* RwSet = (Poll.events & FPollFd::PollIn) ? &FdSetRead : &FdSetWrite;
		FD_SET(Poll.fd, RwSet);
		FD_SET(Poll.fd, &FdSetExcept);
		MaxFd = FMath::Max(Poll.fd, MaxFd);
	}

	int32 Result = Socket_Select(int32(MaxFd + 1), &FdSetRead, &FdSetWrite, &FdSetExcept, TimeValPtr);
	if (Result <= 0)
	{
		return Result;
	}

	for (FPollFd& Poll : MakeArrayView(Polls, Num))
	{
		if (FD_ISSET(Poll.fd, &FdSetExcept))
		{
			Poll.revents = FPollFd::PollErr;
			continue;
		}

		fd_set* RwSet = (Poll.events & FPollFd::PollIn) ? &FdSetRead : &FdSetWrite;
		if (FD_ISSET(Poll.fd, RwSet))
		{
			Poll.revents = Poll.events;
		}
	}

	return Result;
#endif // IAS_HTTP_USE_POLL
}



// {{{1 waiter .................................................................

////////////////////////////////////////////////////////////////////////////////
class FWaiter
{
public:
	enum class EWhat : uint8
	{
		None = 0b00,
		Send = 0b01,
		Recv = 0b10,
		Both = Send|Recv
	};

	static int32	Wait(TArrayView<FWaiter> Waiters, FPoller& Poller, int32 TimeoutMs);

					FWaiter() = default;
					FWaiter(FWaitable&& Waitable);
					FWaiter(FWaiter&&) = default;
	FWaiter&		operator = (FWaiter&&) = default;
	bool			IsValid() const				{ return Socket != InvalidSocket; }
	bool			IsReady() const				{ return Ready != 0; }
	void			WaitFor(EWhat What)			{ WaitOn = What; }
	void			SetIndex(uint32 InIndex)	{ Index = uint16(InIndex); }
	uint32			GetIndex() const			{ return Index; }

private:
	SocketType		Socket = InvalidSocket;
	uint16			Index = 0;
	EWhat			WaitOn = EWhat::None;
	uint8			Ready = 0;

private:
					FWaiter(const FWaiter&) = delete;
	FWaiter&		operator = (const FWaiter&) = delete;
};

////////////////////////////////////////////////////////////////////////////////
FWaiter::FWaiter(FWaitable&& Waitable)
: Socket(Waitable.Socket)
{
}

////////////////////////////////////////////////////////////////////////////////
int32 FWaiter::Wait(TArrayView<FWaiter> Waiters, FPoller& Poller, int32 TimeoutMs)
{
	uint16 Events[] = {
		FPollFd::PollErr,
		FPollFd::PollOut,
		FPollFd::PollIn,
		FPollFd::PollIn | FPollFd::PollOut,
	};
	TArray<FPollFd, TFixedAllocator<64>> Polls;
	for (FWaiter& Waiter : Waiters)
	{
		Polls.Emplace(Waiter.Socket, Events[uint32(Waiter.WaitOn)]);
	}

	// Poll the sockets
	int32 Result = Poller.Wait(Polls.GetData(), Polls.Num(), TimeoutMs);
	if (Result <= 0)
	{
		return Result;
	}

	// Transfer poll results to the input sockets. We don't transfer across error
	// states. Subsequent sockets ops can take care of that instead.
	for (uint32 i = 0, n = Waiters.Num(); i < n; ++i)
	{
		const FPollFd& Poll = Polls[i];
		if (!Poll.HasTrigger())
		{
			continue;
		}

		uint8 Value = 0;
		if (Poll.HasSendTrigger()) Value |= uint8(EWhat::Send);
		if (Poll.HasRecvTrigger()) Value |= uint8(EWhat::Recv);
		Waiters[i].Ready = Value ? Value : uint8(EWhat::Both);
	}

	return Result;
}



// {{{1 socket .................................................................

////////////////////////////////////////////////////////////////////////////////
class FSocket
{
public:
				FSocket() = default;
				~FSocket()					{ Destroy(); }
				FSocket(FSocket&& Rhs)		{ Move(MoveTemp(Rhs)); }
	FSocket&	operator = (FSocket&& Rhs)	{ Move(MoveTemp(Rhs)); return *this; }
	bool		IsValid() const				{ return Socket != InvalidSocket; }
	FWaitable	GetWaitable() const			{ return { Socket }; }
	uint64		GetRecvSize() const;
	bool		Create();
	void		Destroy();
	FOutcome	Connect(uint32 Ip, uint32 Port);
	void		Disconnect();
	FOutcome	Send(const char* Data, uint32 Size);
	FOutcome	Recv(char* Dest, uint32 Size);
	bool		SetBlocking(bool bBlocking);
	bool		SetSendBufSize(int32 Size);
	bool		SetRecvBufSize(int32 Size);

private:
	void		Move(FSocket&& Rhs);
	uint64		RecvSize = 0;
	SocketType	Socket = InvalidSocket;

				FSocket(const FSocket&) = delete;
	FSocket&	operator = (const FSocket&) = delete;
};

////////////////////////////////////////////////////////////////////////////////
void FSocket::Move(FSocket&& Rhs)
{
	check(!IsValid() || !Rhs.IsValid()); // currently we only want to pass one around
	Swap(Socket, Rhs.Socket);
	RecvSize = 0;
}

////////////////////////////////////////////////////////////////////////////////
uint64 FSocket::GetRecvSize() const
{
	return RecvSize;
}

////////////////////////////////////////////////////////////////////////////////
bool FSocket::Create()
{
	check(!IsValid());
	Socket = Socket_Socket(AF_INET, SOCK_STREAM, IPPROTO_TCP);

	if (!IsValid())
	{
		return false;
	}

	int32 Yes = 1;
	Socket_SetSockOpt(Socket, IPPROTO_TCP, TCP_NODELAY, &(char&)(Yes), int32(sizeof(Yes)));

	Trace(Socket, ETrace::SocketCreate);
	return true;
}

////////////////////////////////////////////////////////////////////////////////
void FSocket::Destroy()
{
	if (Socket == InvalidSocket)
	{
		return;
	}

	Trace(Socket, ETrace::SocketDestroy);

	Socket_Close(Socket);
	Socket = InvalidSocket;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FSocket::Connect(uint32 IpAddress, uint32 Port)
{
	check(IsValid());

	Trace(Socket, ETrace::Connect);
	auto Return = [this] (FOutcome Outcome) {
		Trace(Socket, ETrace::Connect, &Outcome);
		return Outcome;
	};

	IpAddress = Socket_HtoNl(IpAddress);

	sockaddr_in AddrInet = { sizeof(sockaddr_in) };
	AddrInet.sin_family = AF_INET;
	AddrInet.sin_port = Socket_HtoNs(uint16(Port));
	memcpy(&(AddrInet.sin_addr), &IpAddress, sizeof(IpAddress));

	int32 Result = Socket_Connect(Socket, &(sockaddr&)AddrInet, uint32(sizeof(AddrInet)));

	if (IsSocketResult(EWOULDBLOCK) | IsSocketResult(EINPROGRESS))
	{
		return Return(FOutcome::Waiting());
	}

	if (Result < 0)
	{
		return Return(FOutcome::Error("Socket connect failed", Result));
	}

	return Return(FOutcome::Ok());
}

////////////////////////////////////////////////////////////////////////////////
void FSocket::Disconnect()
{
	check(IsValid());
	Socket_Shutdown(Socket, SHUT_RDWR);
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FSocket::Send(const char* Data, uint32 Size)
{
	Trace(Socket, ETrace::Send);
	int32 Result = Socket_Send(Socket, Data, Size, MsgFlagType(MSG_NOSIGNAL));

	auto Return = [this] (FOutcome Outcome) {
		Trace(Socket, ETrace::Send, &Outcome);
		return Outcome;
	};

	if (Result > 0)						return Return(FOutcome::Ok(Result));
	if (Result == 0)					return Return(FOutcome::Error("Send ATH0"));
	if (IsSocketResult(EWOULDBLOCK))	return Return(FOutcome::Waiting());

	if (IsSocketResult(ENOTCONN))
	{
		int32 Error = 0;
		socklen_t ErrorSize = sizeof(Error);
		Result = Socket_GetSockOpt(Socket, SOL_SOCKET, SO_ERROR, (char*)&Error, &ErrorSize);
		if (Result < 0 || Error != 0)
		{
			Result = (Result < 0) ? LastSocketResult() : Error;
			return Return(FOutcome::Error("Error while connecting", Result));
		}

		return Return(FOutcome::Waiting());
	}

	Result = LastSocketResult();
	return Return(FOutcome::Error("Send", Result));
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FSocket::Recv(char* Dest, uint32 Size)
{
	Trace(Socket, ETrace::Recv);
	int32 Result = Socket_Recv(Socket, Dest, Size, MsgFlagType(0));

	auto Return = [this] (FOutcome Outcome) {
		Trace(Socket, ETrace::Recv, &Outcome);
		return Outcome;
	};

	if (Result > 0)
	{
		RecvSize += Result;
	}

	if (Result > 0)						return Return(FOutcome::Ok(Result));
	if (Result == 0)					return Return(FOutcome::Error("Recv ATH0"));
	if (IsSocketResult(EWOULDBLOCK))	return Return(FOutcome::Waiting());

	Result = LastSocketResult();
	return Return(FOutcome::Error("Recv", Result));
}

////////////////////////////////////////////////////////////////////////////////
bool FSocket::SetBlocking(bool bBlocking)
{
#if defined(IAS_HTTP_HAS_NONBLOCK_IMPL)
	return SetNonBlockingSocket(Socket, !bBlocking);
#elif PLATFORM_MICROSOFT
	unsigned long NonBlockingMode = (bBlocking != true);
	return (ioctlsocket(Socket, FIONBIO, &NonBlockingMode) != SOCKET_ERROR);
#else
	int32 Flags = fcntl(Socket, F_GETFL, 0);
	if (Flags == -1)
	{
		return false;
	}

	int32 NewFlags = bBlocking
		? (Flags & ~int32(O_NONBLOCK))
		: (Flags |	int32(O_NONBLOCK));
	return (Flags == NewFlags) || (fcntl(Socket, F_SETFL, NewFlags) >= 0);
#endif
}

////////////////////////////////////////////////////////////////////////////////
bool FSocket::SetSendBufSize(int32 Size)
{
	return 0 == Socket_SetSockOpt(Socket, SOL_SOCKET, SO_SNDBUF, &(char&)Size, int32(sizeof(Size)));
}

////////////////////////////////////////////////////////////////////////////////
bool FSocket::SetRecvBufSize(int32 Size)
{
	return 0 == Socket_SetSockOpt(Socket, SOL_SOCKET, SO_RCVBUF, &(char&)Size, int32(sizeof(Size)));
}



////////////////////////////////////////////////////////////////////////////////
static FLaneEstate* GSocketLaneEstate = LaneEstate_New({
	.Name = "Iax/Socket",
	.Group = "Iax",
	.Channel = GetIaxTraceChannel(),
	.Weight = 12,
});

////////////////////////////////////////////////////////////////////////////////
static void Trace(UPTRINT Socket, ETrace Action, const FOutcome* Outcome)
{
	static uint32 SockScopeId = LaneTrace_NewScope("Iax/Socket");
	static uint32 ConnScopeId = LaneTrace_NewScope("Iax/Connect");
	static uint32 SendScopeId = LaneTrace_NewScope("Iax/Send");
	static uint32 RecvScopeId = LaneTrace_NewScope("Iax/Recv");
	static uint32 WaitScopeId = LaneTrace_NewScope("Iax/Wait");
	static uint32 IdleScopeId = LaneTrace_NewScope("Iax/Idle");

	if (Action == ETrace::SocketCreate)
	{
		FLaneTrace* Lane = LaneEstate_Build(GSocketLaneEstate, Socket);
		LaneTrace_Enter(Lane, SockScopeId);
		LaneTrace_Enter(Lane, IdleScopeId);
		return;
	}

	if (Action == ETrace::SocketDestroy)
	{
		LaneEstate_Demolish(GSocketLaneEstate, Socket);
		return;
	}

	FLaneTrace* Lane = LaneEstate_Lookup(GSocketLaneEstate, Socket);

	if (Action == ETrace::Send)
	{
		(Outcome == nullptr)
			? LaneTrace_Enter(Lane, SendScopeId)
			: LaneTrace_Leave(Lane);
		return;
	}

	if (Action == ETrace::Connect)
	{
		if (Outcome == nullptr)
		{
			LaneTrace_Change(Lane, ConnScopeId);
		}
		else if (!Outcome->IsWaiting())
		{
			LaneTrace_Change(Lane, IdleScopeId);
		}
		return;
	}

	if (Outcome == nullptr)
	{
		LaneTrace_Change(Lane, RecvScopeId);
		return;
	}

	if (Outcome->IsWaiting())
	{
		LaneTrace_Change(Lane, WaitScopeId);
		return;
	}

	LaneTrace_Change(Lane, IdleScopeId);
}

// }}}

} // namespace UE::IoStore::HTTP
