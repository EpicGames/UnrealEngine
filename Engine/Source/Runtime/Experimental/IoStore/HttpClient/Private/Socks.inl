// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::IoStore::HTTP
{

#if !UE_BUILD_SHIPPING

///////////////////////////////////////////////////////////////////////////////
static IAS_CVAR(int32,		SocksVersion,	5,		"SOCKS proxy protocol version to use");
static IAS_CVAR(FString,	SocksIp,		"",		"Routes all IAS HTTP traffic through the given SOCKS proxy");
static IAS_CVAR(int32,		SocksPort,		1080,	"Port of the SOCKS proxy to use");

////////////////////////////////////////////////////////////////////////////////
static uint32 GetSocksIpAddress()
{
	const TCHAR* Value = *GSocksIp;
	uint32 IpAddress = 0;
	uint32 Accumulator = 0;
	while (true)
	{
		uint32 c = *Value++;

		if (c - '0' <= '9' - '0')
		{
			Accumulator *= 10;
			Accumulator += (c - '0');
			continue;
		}

		if (c == '.' || c == '\0')
		{
			IpAddress <<= 8;
			IpAddress |= Accumulator;
			Accumulator = 0;
			if (c == '\0')
			{
				break;
			}
			continue;
		}

		return 0;
	}
	return IpAddress;
}

////////////////////////////////////////////////////////////////////////////////
static FOutcome ConnectSocks4(FSocket& Socket, uint32 IpAddress, uint32 Port)
{
	struct FSocks4Request
	{
		uint8	Version = 4;
		uint8	Command = 1;
		uint16	Port;
		uint32	IpAddress;
	};

	struct FSocks4Reply
	{
		uint8	Version;
		uint8	Code;
		uint16	Port;
		uint32	IpAddress;
	};

	uint32 SocksIpAddress = GetSocksIpAddress();
	if (!SocksIpAddress)
	{
		return FOutcome::Error("Invalid socks IP address");
	}

	FOutcome Outcome = FOutcome::None();

	if (Outcome = Socket.Connect(SocksIpAddress, GSocksPort); Outcome.IsError())
	{
		return Outcome;
	}

	FSocks4Request Request = {
		.Port		= Socket_HtoNs(uint16(Port)),
		.IpAddress	= Socket_HtoNl(IpAddress),
	};
	if (Outcome = Socket.Send((const char*)&Request, sizeof(Request)); Outcome.IsError())
	{
		return Outcome;
	}

	FSocks4Reply Reply;
	if (Outcome = Socket.Recv((char*)&Reply, sizeof(Reply)); Outcome.IsError())
	{
		return Outcome;
	}

	return FOutcome::Ok(1);
}

////////////////////////////////////////////////////////////////////////////////
static FOutcome ConnectSocks5(FSocket& Socket, uint32 IpAddress, uint32 Port)
{
#ifdef _MSC_VER
	// MSVC's static analysis doesn't see that 'Result' from recv() is checked
	// to be the exact size of the destination buffer.
#pragma warning(push)
#pragma warning(disable : 6385)
#endif

	uint32 SocksIpAddress = GetSocksIpAddress();
	if (!SocksIpAddress)
	{
		return FOutcome::Error("Invalid socks5 IP address");
	}

	FOutcome Outcome = FOutcome::None();

	if (Outcome = Socket.Connect(SocksIpAddress, GSocksPort); Outcome.IsError())
	{
		return Outcome;
	}

	// Greeting
	const char Greeting[] = { 5, 1, 0 };
	Outcome = Socket.Send(Greeting, sizeof(Greeting));
	if (!Outcome.IsOk())						return Outcome;
	if (Outcome.GetResult() != sizeof(Greeting))return FOutcome::Error("Could not send socks5 greeting");

	// Server auth-choice
	char Choice[1 + 1];
	Outcome = Socket.Recv(Choice, sizeof(Choice));
	if (!Outcome.IsOk())						return Outcome;
	if (Outcome.GetResult() != sizeof(Choice))	return FOutcome::Error("Recv too short from socks5 server");
	if (Choice[0] != 0x05 || Choice[1] != 0x00) return FOutcome::Error("Got unexpected socks5 version from server");

	// Connection request
	IpAddress = htonl(IpAddress);
	uint16 NsPort = htons(uint16(Port));
	char Request[] = { 5, 1, 0, 1, 0x11,0x11,0x11,0x11, 0x22,0x22 };
	std::memcpy(Request + 4, &IpAddress, sizeof(IpAddress));
	std::memcpy(Request + 8, &NsPort, sizeof(NsPort));
	Outcome = Socket.Send(Request, sizeof(Request));
	if (!Outcome.IsOk())						return Outcome;
	if (Outcome.GetResult() != sizeof(Request)) return FOutcome::Error("Sent too little to socks5 server");

	// Connect reply
	char Reply[3 + (1 + 4) + 2];
	Outcome = Socket.Recv(Reply, sizeof(Reply));
	if (!Outcome.IsOk())						return Outcome;
	if (Outcome.GetResult() != sizeof(Reply))	return FOutcome::Error("Socks5 reply too short");
	if (Reply[0] != 0x05 || Reply[1] != 0x00)	return FOutcome::Error("Reply has unexpected socks5 version");

	return FOutcome::Ok(1);

#ifdef _MSC_VER
#pragma warning(pop)
#endif
}

#endif // UE_BUILD_SHIPPING

////////////////////////////////////////////////////////////////////////////////
static FOutcome MaybeConnectSocks(FSocket& Socket, uint32 IpAddress, uint32 Port)
{
#if UE_BUILD_SHIPPING
	return FOutcome::Ok();
#else
	if (GSocksIp.IsEmpty())
	{
		return FOutcome::Ok();
	}

	Socket.SetBlocking(true);

	switch (GSocksVersion)
	{
	case 4: return ConnectSocks4(Socket, IpAddress, Port);
	case 5: return ConnectSocks5(Socket, IpAddress, Port);
	}

	return FOutcome::Error("Unsupported socks version");
#endif // UE_BUILD_SHIPPING
}

} // namespace UE::IoStore::HTTP
