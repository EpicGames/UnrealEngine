// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !defined(NO_UE_INCLUDES)
#include <HAL/FileManager.h>
#include <Misc/Paths.h>
#endif

namespace UE::IoStore::HTTP
{

#if !(UE_BUILD_SHIPPING|UE_BUILD_TEST)

////////////////////////////////////////////////////////////////////////////////
static void MiscTest()
{
#define CRLF "\r\n"
#if 0
	struct {
		FAnsiStringView Input;
		int32 Output;
	} FmtTestCases[] = {
		{ "", -1 },
		{ "abcd", -1 },
		{ "abcd\r", -1 },
		{ CRLF "\r\r", -1 },
		{ CRLF CRLF, 4 },
		{ "abc" CRLF CRLF, 7 },
	};
	for (const auto [Input, Output] : FmtTestCases)
	{
		check(FindMessageTerminal(Input.GetData(), Input.Len()) == Output);
	}
#endif // 0

	FMessageOffsets MsgOut;
	check(ParseMessage("", MsgOut) == -1);
	check(ParseMessage("MR", MsgOut) == -1);
	check(ParseMessage("HTTP/1.1", MsgOut) == -1);
	check(ParseMessage("HTTP/1.1 ", MsgOut) == -1);
	check(ParseMessage("HTTP/1.1 1" CRLF, MsgOut) > 0);
	check(ParseMessage("HTTP/1.1    1" CRLF, MsgOut) > 0);
	check(ParseMessage("HTTP/1.1 100 " CRLF, MsgOut) > 0);
	check(ParseMessage("HTTP/1.1 100  Message of some sort    " CRLF, MsgOut) > 0);
	check(ParseMessage("HTTP/1.1 100 _Message with a \r in it" CRLF, MsgOut) == -1);

	bool AllIsWell = true;
	auto NotExpectedToBeCalled = [&AllIsWell] (auto, auto)
	{
		AllIsWell = false;
		return false;
	};

	EnumerateHeaders("",		NotExpectedToBeCalled); check(AllIsWell);
	EnumerateHeaders(CRLF,		NotExpectedToBeCalled); check(AllIsWell);
	EnumerateHeaders("foo",		NotExpectedToBeCalled); check(AllIsWell);
	EnumerateHeaders(" foo",	NotExpectedToBeCalled); check(AllIsWell);
	EnumerateHeaders(" foo ",	NotExpectedToBeCalled); check(AllIsWell);
	EnumerateHeaders("foo:bar",	NotExpectedToBeCalled); check(AllIsWell);

	auto IsBar = [&] (auto, auto Value) { return AllIsWell = (Value == "bar"); };
	EnumerateHeaders("foo: bar" CRLF,		IsBar); check(AllIsWell);
	EnumerateHeaders("foo: bar \t" CRLF,	IsBar); check(AllIsWell);
	EnumerateHeaders("foo:\tbar " CRLF,		IsBar); check(AllIsWell);
	EnumerateHeaders("foo:bar " CRLF,		IsBar); check(AllIsWell);
	EnumerateHeaders("foo:bar" CRLF "!",	IsBar); check(AllIsWell);
	EnumerateHeaders("foo:bar" CRLF " ",	IsBar); check(AllIsWell);
	EnumerateHeaders("foo:bar" CRLF "n:ej",	IsBar); check(AllIsWell);

	check(CrudeToInt(FAnsiStringView("")) < 0);
	check(CrudeToInt(FAnsiStringView("X")) < 0);
	check(CrudeToInt(FAnsiStringView("/")) < 0);
	check(CrudeToInt(FAnsiStringView(":")) < 0);
	check(CrudeToInt(FAnsiStringView("-1")) < -1);
	check(CrudeToInt(FAnsiStringView("0")) == 0);
	check(CrudeToInt(FAnsiStringView("9")) == 9);
	check(CrudeToInt(FAnsiStringView("493")) == 493);

	check(CrudeToInt<16>(FAnsiStringView("56")) == 0x56);
	check(CrudeToInt<16>(FAnsiStringView("1")) == 0x01);
	check(CrudeToInt<16>(FAnsiStringView("9")) == 0x09);
	check(CrudeToInt<16>(FAnsiStringView("a")) == 0x0a);
	check(CrudeToInt<16>(FAnsiStringView("A")) == 0x0a);
	check(CrudeToInt<16>(FAnsiStringView("f")) == 0x0f);
	check(CrudeToInt<16>(FAnsiStringView("F")) == 0x0f);
	check(CrudeToInt<16>(FAnsiStringView("g")) < 0);
	check(CrudeToInt<16>(FAnsiStringView("49e")) == 0x49e);
	check(CrudeToInt<16>(FAnsiStringView("aBcD")) == 0xabcd);
	check(CrudeToInt<16>(FAnsiStringView("eEeE")) == 0xeeee);

	FUrlOffsets UrlOut;

	check(ParseUrl("", UrlOut) == -1);
	check(ParseUrl("abc://asd/", UrlOut) == -1);
	check(ParseUrl("http://", UrlOut) == -1);
	check(ParseUrl("http://:/", UrlOut) == -1);
	check(ParseUrl("http://@:/", UrlOut) == -1);
	check(ParseUrl("http://foo:ba:r/", UrlOut) == -1);
	check(ParseUrl("http://foo@ba:r/", UrlOut) == -1);
	check(ParseUrl("http://foo@ba:r", UrlOut) == -1);
	check(ParseUrl("http://foo@ba:/", UrlOut) == -1);
	check(ParseUrl("http://foo@ba@9/", UrlOut) == -1);
	check(ParseUrl("http://@ba:9/", UrlOut) == -1);
	check(ParseUrl(
		"http://zzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzz"
		"zzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzz"
		"zzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzz"
		"zzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzz"
		"zzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzzz.com",
		UrlOut) == -1);

	check(ParseUrl("http://ab-c.com/", UrlOut) > 0);
	check(ParseUrl("http://a@bc.com/", UrlOut) > 0);
	check(ParseUrl("https://abc.com", UrlOut) > 0);
	check(ParseUrl("https://abc.com:999", UrlOut) > 0);
	check(ParseUrl("https://abc.com:999/", UrlOut) > 0);
	check(ParseUrl("https://foo:bar@abc.com:999", UrlOut) > 0);
	check(ParseUrl("https://foo:bar@abc.com:999/", UrlOut) > 0);
	check(ParseUrl("https://foo_bar@abc.com:999", UrlOut) > 0);
	check(ParseUrl("https://foo_bar@abc.com:999/", UrlOut) > 0);

	for (int32 i : { 0x10, 0x20, 0x40, 0x7f, 0xff })
	{
		char Url[] = "http://sweden.denmark.finland.norway.iceland:123";
		char Buffer[512];
		std::memset(Buffer, i, sizeof(Buffer));
		std::memcpy(Buffer, Url, sizeof(Url) - 1);
		check(ParseUrl(FAnsiStringView(Buffer, sizeof(Url) - 1), UrlOut) > 0);
		check(UrlOut.Port.Get(Url) == "123");
	}

	FAnsiStringView Url = "http://abc:123@bc.com:999/";
	check(ParseUrl(Url, UrlOut) > 0);
	check(UrlOut.SchemeLength == 4);
	check(UrlOut.UserInfo.Get(Url) == "abc:123");
	check(UrlOut.HostName.Get(Url) == "bc.com");
	check(UrlOut.Port.Get(Url) == "999");
	check(UrlOut.Path == 25);
#undef CRLF

	static const char* OutcomeMsg = "\x4d\x52";
	check(FOutcome::Error(OutcomeMsg, -5).IsOk() == false);
	check(FOutcome::Error(OutcomeMsg, -5).IsWaiting() == false);
	check(FOutcome::Error(OutcomeMsg, -5).IsError());
	check(FOutcome::Error(OutcomeMsg, -5).GetErrorCode() == -5);
	check(FOutcome::Error(OutcomeMsg,  5).GetErrorCode() ==  5);
	check(FOutcome::Error(OutcomeMsg, -5).GetMessage() == OutcomeMsg);

	check(FOutcome::Ok(  0).IsOk());
	check(FOutcome::Ok(-13).IsWaiting() == false);
	check(FOutcome::Ok( 13).IsError() == false);

	check(FOutcome::Waiting().IsOk() == false);
	check(FOutcome::Waiting().IsWaiting());
	check(FOutcome::Waiting().IsError() == false);
	check(FOutcome::Waiting().IsWaitData() == true);
	check(FOutcome::Waiting().IsWaitBuffer() == false);
	check(FOutcome::Waiting().IsWaitStream() == false);

	check(FOutcome::WaitBuffer().IsWaitData() == false);
	check(FOutcome::WaitBuffer().IsWaitBuffer() == true);
	check(FOutcome::WaitBuffer().IsWaitStream() == false);

	check(FOutcome::WaitStream().IsWaitData() == false);
	check(FOutcome::WaitStream().IsWaitBuffer() == false);
	check(FOutcome::WaitStream().IsWaitStream() == true);
}

////////////////////////////////////////////////////////////////////////////////
static void ThrottleTest(FAnsiStringView TestUrl)
{
	enum { TheMax = 0x7fff'fffful };
	check(FThrottler().GetAllowance() >= TheMax);

	FThrottler Throttler;
	uint64 OneSecond = Throttler.CycleFreq;

	// timing test
	FIoBuffer RecvData;
	for (uint32 SizeKiB : { 64, 128, 192 })
	{
		const uint32 ThrottleKiB = 64;

		TAnsiStringBuilder<128> Url;
		Url << TestUrl;
		Url << (SizeKiB << 10);

		FEventLoop Loop;
		Loop.Throttle(ThrottleKiB);

		FRequest Request = Loop.Request("GET", Url);
		Request.Accept("*/*");
		Loop.Send(MoveTemp(Request), [&] (const FTicketStatus& Status) {
			check(Status.GetId() != FTicketStatus::EId::Error);
			if (Status.GetId() == FTicketStatus::EId::Response)
			{
				Status.GetResponse().SetDestination(&RecvData);
			}
		});

		int32 Timeout = -1;
		if (SizeKiB < 128) Timeout = 123;
		if (SizeKiB > 128) Timeout = 4567;

		uint64 Time = FPlatformTime::Cycles64();
		while (Loop.Tick(Timeout));
		Time = FPlatformTime::Cycles64() - Time;
		Time /= OneSecond;

		// It's dangerous stuff testing elapsed time you know. The +1 is because
		// throttling assumes one second has already passed when initialised.
#if PLATFORM_WINDOWS
		check(Time + 1 == (SizeKiB / ThrottleKiB));
#endif

		RecvData = FIoBuffer();
	}
}

#if IAS_HTTP_HAS_OPENSSL

////////////////////////////////////////////////////////////////////////////////
static void TlsLoadRootCerts()
{
#if !defined(NO_UE_INCLUDES)
	IFileManager& Ifm = IFileManager::Get();
	FString PemPath = FPaths::EngineDir() / TEXT("Content/Certificates/ThirdParty/cacert.pem");
	FArchive* Reader = Ifm.CreateFileReader(*PemPath);

	uint32 Size = uint32(Reader->TotalSize());
	FIoBuffer PemData(Size);
	FMutableMemoryView PemView = PemData.GetMutableView();
	Reader->Serialize(PemView.GetData(), Size);

	FCertRoots CaRoots(PemData.GetView());
	FCertRoots::SetDefault(MoveTemp(CaRoots));

	delete Reader;
#endif // NO_UE_INCLUDES
}

////////////////////////////////////////////////////////////////////////////////
static void TlsTest()
{
	FEventLoop Loop;

	auto WaitForLoopIdle = [&] {
		for (; Loop.Tick(-1); FPlatformProcess::SleepNoStats(0.02f));
	};

	auto OkSink = [Dest=FIoBuffer()] (const FTicketStatus& Status) mutable {
		check(Status.GetId() != FTicketStatus::EId::Error);
		if (Status.GetId() == FTicketStatus::EId::Response)
		{
			FResponse& Response = Status.GetResponse();
			check(Response.GetStatus() == EStatusCodeClass::Successful);
			Response.SetDestination(&Dest);
			return;
		}
		check(Status.GetId() == FTicketStatus::EId::Content);
	};

	static const ANSICHAR* Url = "http://epicgames.com";

	{
		FEventLoop::FRequestParams RequestParams = { .bAutoRedirect = true };
		FRequest Request = Loop.Request("HEAD", Url, &RequestParams);
		Loop.Send(MoveTemp(Request), OkSink);
		WaitForLoopIdle();
	}

	{
		FCertRoots NotACert(FMemoryView("493", 3));
		check(NotACert.IsValid() == false);
	}
}

#endif // IAS_HTTP_HAS_OPENSSL

////////////////////////////////////////////////////////////////////////////////
static void RedirectTest(const ANSICHAR* TestHost, FCertRootsRef VerifyCert)
{
	FEventLoop Loop;

	auto WaitForLoopIdle = [&] {
		for (; Loop.Tick(-1); FPlatformProcess::SleepNoStats(0.02f));
	};

	FEventLoop::FRequestParams RequestParams = {
		.bAutoRedirect = true,
	};

	enum ReTyp { ReAbs, ReAbsTls, ReRel, ReRelTls };
	enum { RecvDataSize = 48 };

	TAnsiStringBuilder<64> Builder;
	auto BuildUrl = [&] (ReTyp Typ, uint32 Code) -> const auto&
	{
		bool bTls = (Typ & 1);
		Builder.Reset();
		Builder << ((bTls) ? "https://" : "http://");
		Builder << TestHost;
		Builder << ":" << (bTls ? 4939 : 9493);
		Builder << "/redirect";
		Builder << ((Typ <= ReAbsTls) ? "/abs/" : "/rel/");
		Builder << Code;
		Builder << "/data/" << uint32(RecvDataSize);
		return Builder;
	};

	UPTRINT SinkParam = 0xaa'493'493'493'493'bbull;

	uint32 RecvCount;
	auto OkSink = [Dest=FIoBuffer(), SinkParam, &RecvCount] (const FTicketStatus& Status) mutable {
		check(Status.GetParam() == SinkParam);
		check(Status.GetId() != FTicketStatus::EId::Error);
		if (Status.GetId() == FTicketStatus::EId::Response)
		{
			FResponse& Response = Status.GetResponse();
			check(Response.GetStatusCode() == 200);
			Response.SetDestination(&Dest);
			return;
		}
		check(Status.GetId() == FTicketStatus::EId::Content);
		RecvCount += uint32(Dest.GetSize());
	};

	uint32 TestCodes[] = { 301, 302, 307, 308 };

	for (auto ReTest : { ReAbs, ReAbsTls, ReRel, ReRelTls })
	{
#if !IAS_HTTP_HAS_OPENSSL
		if (ReTest & 1)
		{
			continue;
		}
#endif

		RequestParams.VerifyCert = (ReTest & 1) ? VerifyCert : 0;
		RecvCount = 0;
		for (uint32 Code : TestCodes)
		{
			FRequest Request = Loop.Get(BuildUrl(ReTest, Code), &RequestParams);
			if (Code > TestCodes[1])
			{
				Request.Header("TestCodeHeader", "Header-Of-Test-Codes");
			}
			Loop.Send(MoveTemp(Request), OkSink, SinkParam);
		}
		WaitForLoopIdle();
		check(RecvCount == RecvDataSize * UE_ARRAY_COUNT(TestCodes));
	}

	RequestParams = FEventLoop::FRequestParams();
	RequestParams.bAutoRedirect = true;

	for (auto ReTest : { ReAbs, ReAbsTls, ReRel, ReRelTls })
	{
#if !IAS_HTTP_HAS_OPENSSL
		if (ReTest & 1)
		{
			continue;
		}
#endif

		FConnectionPool::FParams Params;
		Params.SetHostFromUrl(BuildUrl(ReTest, 0));
		Params.VerifyCert = (ReTest & 1) ? VerifyCert : 0;
		Params.ConnectionCount = 4;
		FConnectionPool Pool(Params);

		RecvCount = 0;
		uint32 ExpectCount = 0;
		for (uint32 TestCount : { 4, 267, 55, 17, 1024, 13, 26, 39, 52, 493 })
		{
			ExpectCount += TestCount;
			TAnsiStringBuilder<64> Path;
			Path << "/redirect/abs/307/data/";
			Path << TestCount;
			Loop.Send(Loop.Get(Path.ToString(), Pool, &RequestParams), OkSink, SinkParam);
		}
		WaitForLoopIdle();
		check(RecvCount == ExpectCount);
	}
}

////////////////////////////////////////////////////////////////////////////////
static void ChunkedTest(const ANSICHAR* TestHost)
{
	FEventLoop Loop;

	auto WaitForLoopIdle = [&]
	{
		while (Loop.Tick(0))
			;
	};

	TAnsiStringBuilder<64> Url;

	// TestServer proxy doesn't support chunked transfer so find the actual httpd
	int32 HttpdPort = -1;
	Url << "http://" << TestHost << ":9493/port";
	Loop.Send(Loop.Get(Url), [&HttpdPort, Dest=FIoBuffer()] (const FTicketStatus& Status) mutable
	{
		if (Status.GetId() == FTicketStatus::EId::Response)
		{
			Status.GetResponse().SetDestination(&Dest);
			return;
		}

		check(Status.GetId() == FTicketStatus::EId::Content);
		HttpdPort = int32(CrudeToInt({ (char*)Dest.GetView().GetData(), int32(Dest.GetSize()) }));
	});
	WaitForLoopIdle();
	check(HttpdPort > -1);

	auto BuildUrl = [&Url, TestHost, HttpdPort] (int32 PayloadSize, FAnsiStringView UrlSuffix) -> FAnsiStringView
	{
		Url.Reset();
		Url << "http://" << TestHost << ":" << HttpdPort << "/chunked/" << PayloadSize << UrlSuffix;
		return Url;
	};

	struct FTestState
	{
		int32	Size = 0;
		uint32	Hash = 0x493;
		uint32	ExpectedHash = 0;
		int32	ExpectedSize = -1;
	};
	auto ChunkedSink = [State=FTestState(), Dest=FIoBuffer()] (const FTicketStatus& Status) mutable
	{
		if (Status.GetId() == FTicketStatus::EId::Response)
		{
			FResponse& Response = Status.GetResponse();
			check(Response.GetStatus() == EStatusCodeClass::Successful);
			check(Response.GetStatusCode() == 200);
			check(Response.GetContentStatus() == EContentStatus::UnknownSize);

			State.ExpectedHash = uint32(CrudeToInt(Response.GetHeader("X-TestServer-Hash")));
			State.ExpectedSize = uint32(CrudeToInt(Response.GetHeader("X-TestServer-Size")));

			uint32 DestSize = ((State.ExpectedHash & 0x3f) / 7) * 67;
			Dest = FIoBuffer(DestSize);

			Response.SetDestination(&Dest);
			return;
		}

		check(Status.GetId() == FTicketStatus::EId::Content);

		check(Status.GetContent().GetSize() == Dest.GetSize());

		bool IsTheEnd = (Status.GetContent().GetSize() == 0);
		check(Status.IsComplete() == IsTheEnd);

		FMemoryView View = Dest.GetView(); 
		State.Size += uint32(View.GetSize());
		for (uint32 i = 0, n = uint32(View.GetSize()); i < n; ++i)
		{
			uint8 c = ((const uint8*)(View.GetData()))[i];
			State.Hash = (State.Hash + c) * 0x493;
		}

		if (View.GetSize() == 0)
		{
			check(State.Hash == State.ExpectedHash);
			check(State.Size == State.ExpectedSize);
		}
	};

	// General soak test
	FAnsiStringView UrlSuffices[] = {
		"",
	//	"/ext", /* chunk-ext support was removed */
	};
	for (FAnsiStringView UrlSuffix : UrlSuffices)
	{
		for (uint32 Mixer : { 1, 2, 3, 17, 71, 4931, 0xa9e })
		{
			for (uint32 SizeToGet : { 4,8,32,64,1,2,3,5,7,11,13,17,19,41,43,47,59,67,71,83,89,103,109 })
			{
				BuildUrl(SizeToGet * Mixer, UrlSuffix);
				FRequest Request = Loop.Get(Url);
				Loop.Send(MoveTemp(Request), ChunkedSink);
			}
			WaitForLoopIdle();
		}
	}

	// Rudimentary coverage for transfers with trailing headers.
	uint32 ErrorMarks;
	auto ExpectError = [&ErrorMarks, Dest=FIoBuffer()] (const FTicketStatus& Status) mutable
	{
		if (Status.GetId() == FTicketStatus::EId::Response)
		{
			FResponse& Response = Status.GetResponse();
			Response.SetDestination(&Dest);
			return;
		}

		if (Status.GetId() != FTicketStatus::EId::Error)
		{
			return;
		}

		FAnsiStringView Reason = Status.GetError().Reason;
		ErrorMarks |= Reason.Contains("ERRTRAIL",   ESearchCase::CaseSensitive) ? 1 : 0;
		ErrorMarks |= Reason.Contains("ERRNOCHUNK", ESearchCase::CaseSensitive) ? 2 : 0;
		ErrorMarks |= Reason.Contains("ERREXT", 	ESearchCase::CaseSensitive) ? 4 : 0;
	};
	ErrorMarks = 0;
	BuildUrl(16 << 10, "/trailer");
	Loop.Send(Loop.Get(Url), ExpectError);
	WaitForLoopIdle();
	check(ErrorMarks == 1);

	// Chunk extensions are not supported
	{
		ErrorMarks = 0;
		BuildUrl(493, "/ext");
		Loop.Send(Loop.Get(Url), ExpectError);
		WaitForLoopIdle();
		check(ErrorMarks == 4);
	}

	// Disabling of chunked transfers 
	{
		ErrorMarks = 0;
		FEventLoop::FRequestParams RequestParams = { .bAllowChunked = false };
		BuildUrl(16 << 10, "");
		Loop.Send(Loop.Get(Url, &RequestParams), ExpectError);
		WaitForLoopIdle();
		check(ErrorMarks == 2);
	}
}

////////////////////////////////////////////////////////////////////////////////
static void SeedHttp(const ANSICHAR* TestHost, uint32 Seed)
{
	TAnsiStringBuilder<64> Url;
	Url << "http://" << TestHost << ":9493/seed/" << Seed;
	FEventLoop Loop;
	FRequest Request = Loop.Request("GET", Url, nullptr);
	Loop.Send(MoveTemp(Request), [] (const FTicketStatus&) {});
	for (; Loop.Tick(-1); FPlatformProcess::SleepNoStats(0.02f));
}

////////////////////////////////////////////////////////////////////////////////
static void HttpTest(const ANSICHAR* TestHost, FCertRootsRef VerifyCert)
{
	const uint32 DefaultPort = (VerifyCert != 0) ? 4939 : 9493;

	TAnsiStringBuilder<64> Ret;
	auto BuildUrl = [&] (const ANSICHAR* Suffix=nullptr, uint32 Port=0) -> const auto&
	{
		Port = Port ? Port : DefaultPort;
		Ret.Reset();
		Ret << ((Port == 4939) ? "https://" : "http://");
		Ret << TestHost;
		Ret << ":" << Port;
		return (Suffix != nullptr) ? (Ret << Suffix) : Ret;
	};

	struct
	{
		FIoBuffer Dest;
		uint64 Hash = 0;
	} Content[64];

	auto HashSink = [&] (const FTicketStatus& Status) -> FIoBuffer*
	{
		check(Status.GetId() != FTicketStatus::EId::Error);

		uint32 Index = Status.GetIndex();

		if (Status.GetId() == FTicketStatus::EId::Response)
		{
			check(Status.IsComplete() == false);

			FResponse& Response = Status.GetResponse();
			check(Response.GetStatus() == EStatusCodeClass::Successful);
			check(Response.GetStatusCode() == 200);
			check(Response.GetContentLength() == Status.GetContentLength());
			check(Response.GetContentStatus() == EContentStatus::KnownSize);

			FAnsiStringView HashView = Response.GetHeader("X-TestServer-Hash");
			Content[Index].Hash = CrudeToInt(HashView);
			check(int64(Content[Index].Hash) > 0);

			Content[Index].Dest = FIoBuffer();
			Response.SetDestination(&(Content[Index].Dest));
			return nullptr;
		}

		check(Status.IsComplete());

		uint32 ReceivedHash = 0x493;
		FMemoryView ContentView = Content[Index].Dest.GetView();
		check(ContentView.GetSize() == Status.GetContentLength());
		for (uint32 i = 0; i < Status.GetContentLength(); ++i)
		{
			uint8 c = ((const uint8*)(ContentView.GetData()))[i];
			ReceivedHash = (ReceivedHash + c) * 0x493;
		}
		check(Content[Index].Hash == ReceivedHash);
		Content[Index].Hash = 0;

		return nullptr;
	};

	auto NullSink = [] (const FTicketStatus&) {};

	auto NoErrorSink = [&] (const FTicketStatus& Status)
	{
		check(Status.GetId() != FTicketStatus::EId::Error);
		if (Status.GetId() != FTicketStatus::EId::Response)
		{
			return;
		}

		uint32 Index = Status.GetIndex();

		FResponse& Response = Status.GetResponse();
		Content[Index].Dest = FIoBuffer();
		Response.SetDestination(&(Content[Index].Dest));
	};

	FEventLoop Loop;
	volatile bool LoopStop = false;
	volatile bool LoopTickDelay = false;
	auto LoopTask = UE::Tasks::Launch(TEXT("IasHttpTest.Loop"), [&] () {
		uint32 DelaySeed = 493;
		while (!LoopStop)
		{
			while (Loop.Tick())
			{
				if (!LoopTickDelay)
				{
					continue;
				}

				float DelayFloat = float(DelaySeed % 75) / 1000.0f;
				FPlatformProcess::SleepNoStats(DelayFloat);
				DelaySeed *= 0xa93;
			}

			FPlatformProcess::SleepNoStats(0.005f);
		}
	});

	auto WaitForLoopIdle = [&Loop] ()
	{
		FPlatformProcess::SleepNoStats(0.25f);
		while (!Loop.IsIdle())
		{
			FPlatformProcess::SleepNoStats(0.03f);
		}
	};

	FEventLoop::FRequestParams ReqParamObj = { .VerifyCert = VerifyCert };
	const FEventLoop::FRequestParams* ReqParams = nullptr;
	if (VerifyCert != 0)
	{
		ReqParams = &ReqParamObj;
	}

	// unused request
	{
		FRequest Request = Loop.Request("GET", BuildUrl("/data"));
	}

	// foundational
	{
		FRequest Request = Loop.Request("GET", BuildUrl("/data/67"), ReqParams);
		Request.Accept(EMimeType::Json);
		Loop.Send(MoveTemp(Request), HashSink);
		WaitForLoopIdle();
	}

	// pool
	for (uint16 i = 1; i < 64; ++i)
	{
		FConnectionPool::FParams Params;
		Params.SetHostFromUrl(BuildUrl());
		Params.VerifyCert = VerifyCert;
		Params.ConnectionCount = (i % 2) + 1;
		FConnectionPool Pool(Params);
		for (int32 j = 0; j < i; ++j)
		{
			TAnsiStringBuilder<16> Path;
			Path << "/data?pool=" << i << "x" << j;
			FRequest Request = Loop.Get(Path, Pool);
			Loop.Send(MoveTemp(Request), HashSink);
		}
		WaitForLoopIdle();
	}

	// head barage
	{
		auto HeadSink = [] (const FTicketStatus& Status)
		{
			if (Status.GetId() != FTicketStatus::EId::Response)
			{
				return;
			}

			check(Status.GetResponse().GetContentStatus() == EContentStatus::NoContent);
			check(Status.IsComplete());
		};

		FConnectionPool::FParams Params;
		Params.SetHostFromUrl(BuildUrl());
		Params.VerifyCert = VerifyCert;
		Params.ConnectionCount = 1;
		FConnectionPool Pool(Params);
		for (int32 i = 0; i < 61; ++i)
		{
			TAnsiStringBuilder<16> Path;
			Path << "/data?head";
			FRequest Request = Loop.Request("HEAD", Path, Pool);
			Loop.Send(MoveTemp(Request), HeadSink);
		}
		WaitForLoopIdle();
	}

	// fatal timeout
	for (int32 i = 0; i < 14; ++i)
	{
		bool bExpectFailTimeout = !!(i & 1);
		auto Sink = [bExpectFailTimeout, Dest=FIoBuffer()] (const FTicketStatus& Status) mutable
		{
			if (Status.GetId() == FTicketStatus::EId::Response)
			{
				FResponse& Response = Status.GetResponse();
				Response.SetDestination(&Dest);
				return;
			}

			check(Status.GetId() == FTicketStatus::EId::Error);

			const char* Reason = Status.GetError().Reason;
			bool IsFailTimeout = (FCStringAnsi::Strstr(Reason, "FailTimeout") != nullptr);
			check(IsFailTimeout == bExpectFailTimeout);
		};

		auto ErrorSink = [] (const FTicketStatus& Status)
		{
			check(Status.GetId() == FTicketStatus::EId::Error);
		};

		FConnectionPool::FParams Params;
		Params.SetHostFromUrl(BuildUrl());
		Params.VerifyCert = VerifyCert;
		Params.MaxInflight = 2;
		FConnectionPool Pool(Params);

		FEventLoop Loop2;
		Loop2.Send(Loop2.Get("/data?stall=1", Pool), Sink);

		// Requests are pipelined. The second one will get sent during the stall so
		// we expect it to fail. The subsequent ones are expected to succeed.
		Loop2.Send(Loop2.Get("/data", Pool), ErrorSink);
		Loop2.Send(Loop2.Get("/data", Pool), HashSink);
		Loop2.Send(Loop2.Get("/data", Pool), HashSink);

		int32 PollTimeoutMs = -1;
		if (bExpectFailTimeout)
		{
			Loop2.SetFailTimeout(1000);

			if ((i & 3) == 1)
			{
				PollTimeoutMs = 1000;
			}
		}
		while (Loop2.Tick(PollTimeoutMs));

		Loop2.Send(Loop2.Get("/data/23", Pool), NoErrorSink);
		while (Loop2.Tick(PollTimeoutMs));
	}

	// no connect
	{
		FRequest Requests[] = {
			Loop.Request("GET", BuildUrl(nullptr, 10930)),
			Loop.Request("GET", "http://thisdoesnotexistihope/"),
		};
		Loop.Send(MoveTemp(Requests[0]), NullSink);
		Loop.Send(MoveTemp(Requests[1]), NullSink);
		WaitForLoopIdle();
	}

	// head and large requests
	{
		auto MixTh = [Th=uint32(0)] () mutable { return (Th = (Th * 75) + 74) & 255; };

		char AsciiData[257];
		for (char& c : AsciiData)
		{
			c = 0x41 + (MixTh() % 26);
			c += (MixTh() & 2) ? 0x20 : 0;
		}

		for (int32 i = 0; (i += 69493) < 2 << 20;)
		{
			FRequest Request = Loop.Request("HEAD", BuildUrl("/data"), ReqParams);
			for (int32 j = i; j > 0;)
			{
				FAnsiStringView Name(AsciiData, MixTh() + 1);
				FAnsiStringView Value(AsciiData, MixTh() + 1);
				Request.Header(Name, Value);
				j -= Name.Len() + Value.Len();

			}

			Loop.Send(MoveTemp(Request), [] (const FTicketStatus& Status) {
				if (Status.GetId() == FTicketStatus::EId::Response)
				{
					FResponse& Response = Status.GetResponse();
					check(Response.GetStatusCode() == 431); // "too many headers"
				}
			});

			WaitForLoopIdle();
		}
	}

	// stress 1
	{
		const uint32 StressLoad = 32;

		struct {
			const ANSICHAR* Uri;
			bool Disconnect;
		} StressUrls[] = {
			{ "/data?slowly=1",		false },
			{ "/data?disconnect=1",	true },
		};

		uint64 Errors = 0;
		auto ErrorSink = [&] (const FTicketStatus& Status)
		{
			FTicket Ticket = Status.GetTicket();
			uint32 Index = uint32(63 - FMath::CountLeadingZeros64(uint64(Ticket)));

			if (Status.GetId() == FTicketStatus::EId::Error)
			{
				Errors |= 1ull << Index;
				return;
			}

			else if (Status.GetId() == FTicketStatus::EId::Response)
			{
				FResponse& Response = Status.GetResponse();
				Content[Index].Dest = FIoBuffer();
				Response.SetDestination(&(Content[Index].Dest));
				return;
			}

			check(false);
		};

		for (const auto& [StressUri, ExpectDisconnect] : StressUrls)
		{
			FTicketSink Sink = ExpectDisconnect ? FTicketSink(ErrorSink) : FTicketSink(HashSink);

			const auto& StressUrl = BuildUrl(StressUri);
			for (bool AddDelay : {false, true})
			{
				FTicket Tickets[StressLoad];
				for (FTicket& Ticket : Tickets)
				{
					FRequest Request = Loop.Get(StressUrl, ReqParams);
					Request.Header("Accept", "*/*");
					Ticket = Loop.Send(MoveTemp(Request), Sink);
				}

				LoopTickDelay = AddDelay;

				WaitForLoopIdle();
			}

			LoopTickDelay = false;
		}
	}

	// stress 2
	{
		const uint32 StressLoad = 3;
		const uint32 StressTaskCount = 7;
		static_assert(StressLoad * StressTaskCount <= 32);

		FAnsiStringView Url = BuildUrl("/data");

		auto StressTaskEntry = [&] {
			for (uint32 i = 0; i < StressLoad; ++i)
			{
				FTicket Ticket = Loop.Send(Loop.Get(Url, ReqParams), HashSink);
				if (!Ticket)
				{
					FPlatformProcess::SleepNoStats(0.01f);
					--i;
				}
			}
		};

		UE::Tasks::FTask StressTasks[StressTaskCount];
		for (auto& StressTask : StressTasks)
		{
			StressTask = UE::Tasks::Launch(TEXT("StressTask"), [&] { StressTaskEntry(); });
		}
		for (auto& StressTask : StressTasks)
		{
			StressTask.Wait();
		}

		WaitForLoopIdle();
	}

	// tamper
	for (int32 i = 1; i <= 100; ++i)
	{
		TAnsiStringBuilder<32> TamperUrl;
		TamperUrl << "/data?tamper=" << i;
		FAnsiStringView Url = BuildUrl(TamperUrl.ToString());

		for (int j = 0; j < 13; ++j)
		{
			FRequest Request = Loop.Request("GET", Url, ReqParams);
			Loop.Send(MoveTemp(Request), NullSink);
		}

		WaitForLoopIdle();
	}

	LoopStop = true;
	LoopTask.Wait();

	check(Loop.IsIdle());

#if IS_PROGRAM
	if (VerifyCert == 0)
	{
		ThrottleTest(BuildUrl("/data/"));
	}
#endif

	// pre-generated headers
	// request-with-body
	// proxy
	// gzip / deflate
	// loop multi-req.
	// url auth credentials
	// transfer-file / splice / sendfile
	// (header field parser)
	// (form-data)
	// (cookies)
	// (cache)
	// (websocket)
	// (ipv6)
	// (utf-8 host names)
}

////////////////////////////////////////////////////////////////////////////////
IOSTOREHTTPCLIENT_API void IasHttpTest(const ANSICHAR* TestHost="localhost", uint32 Seed=493)
{
#if PLATFORM_WINDOWS
	WSADATA WsaData;
	if (WSAStartup(MAKEWORD(2, 2), &WsaData) == 0x0a9e0493)
		return;
	ON_SCOPE_EXIT { WSACleanup(); };
#endif

	MiscTest();

#if IAS_HTTP_HAS_OPENSSL
	FCertRoots TestServerCaChain;
	{
		TAnsiStringBuilder<64> CaUrl;
		CaUrl << "http://" << TestHost << ":9493/ca";

		FIoBuffer CertBuffer;

		FEventLoop Loop;
		FRequest Request = Loop.Get(CaUrl);
		Request.Header("X-TestServer-M", "Girders");
		Request.Header("X-TestServer-R", "RedRigs");
		Loop.Send(MoveTemp(Request), [&] (const FTicketStatus& Status) {
			check(Status.GetId() != FTicketStatus::EId::Error);

			if (Status.GetId() == FTicketStatus::EId::Response)
			{
				FResponse& Response = Status.GetResponse();
				Response.SetDestination(&CertBuffer);
			}
		});
		for (; Loop.Tick(-1); FPlatformProcess::SleepNoStats(0.02f));

		TestServerCaChain = FCertRoots(CertBuffer.GetView());
	}
	FCertRootsRef TestServerCertRef = FCertRoots::Explicit(TestServerCaChain);
#else
	FCertRootsRef TestServerCertRef = FCertRoots::NoTls();
#endif // IAS_HTTP_HAS_OPENSSL

	SeedHttp(TestHost, Seed);
	HttpTest(TestHost, FCertRoots::NoTls());
	ChunkedTest(TestHost);
	RedirectTest(TestHost, TestServerCertRef);

#if IAS_HTTP_HAS_OPENSSL
	HttpTest(TestHost, TestServerCertRef);
	TlsLoadRootCerts();
	TlsTest();
#endif
}

#endif // !SHIP|TEST

} // namespace UE::IoStore::HTTP
