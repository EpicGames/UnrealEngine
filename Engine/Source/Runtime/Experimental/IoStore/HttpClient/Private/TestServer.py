# Copyright Epic Games, Inc. All Rights Reserved.

import io
import os
import time
import base64
import socket
import random
import tempfile
import threading
import http.server
import http.client
import subprocess as sp
from pathlib import Path
from urllib.parse import parse_qsl

# {{{1 proxied .................................................................

def intercept(fd):
    line = fd.readline()
    if b"HTTP/1.1" not in line:
        return False
    try:
        headers = http.client.parse_headers(fd)
    except http.client.HTTPException:
        return 413
    return line, headers

def make_preamble(line, headers):
    msg = line
    for k, v in headers.items():
        msg += (k + ": " + v + "\r\n").encode()
    msg += b"\r\n"
    return msg

def proxy_impl(client, httpd):
    # get request
    req = intercept(client)
    if not req:
        return False
    if isinstance(req, int):
        client.write(f"HTTP/1.1 {req} IasTestServerProxyError\r\n".encode())
        client.write(b"Content-Length: 0\r\n\r\n")
        return False
    line, headers = req
    if not (line or headers):
        return False
    close = (headers.get("Connection", "").lower() == "close")
    msg = make_preamble(line, headers)
    httpd.write(msg)
    httpd.flush()

    # extract request
    method, path, proto = line.split(b" ")

    if b"?" in path:
        _, query = path.split(b"?", 1)
        query = {k:v for k,v in parse_qsl(query)}
    else:
        query = {}

    # establish behaviour
    tamper = float(query.get(b"tamper", 0)) / 100.0
    disconnect = b"disconnect" in query
    stall = b"stall" in query
    slowly = b"slowly" in query

    # get response
    line, headers = intercept(httpd)
    close = close or (headers.get("Connection", "").lower() == "close")

    # get data to retransmit
    data = make_preamble(line, headers)
    if method != b"HEAD":
        content_len = int(headers.get("Content-Length", "0"))
        data += httpd.read(content_len)
    data_size = len(data)

    if stall:
        data = data[:-1]
    elif disconnect:
        trunc = int(data_size * random.random())
        data = data[:trunc]

    if tamper > 0:
        data = bytearray(data)
        for i in range(data_size):
            c = data[i] if random.random() > tamper else (int(random.random() * 0x4567) & 0xff)
            data[i] = c

    # retransmit
    send_time = (0.75 + (random.random() * 0.75)) if slowly else 0
    while data:
        percent = 0.02 + (random.random() * 0.08)
        send_size = max(int(data_size * percent), 1)
        piece = data[:send_size]
        data = data[send_size:]

        client.write(piece)
        client.flush()

        if send_time:
            time.sleep(send_time * percent)

    # we're done
    if stall:
        time.sleep(2)

    return not (close or disconnect or stall)

def proxy_client(client, httpd_port):
    client_fd = client.makefile("rwb")
    with socket.socket(socket.AF_INET, socket.SOCK_STREAM) as httpd:
        httpd.connect(("127.0.0.1", httpd_port))
        httpd_fd = httpd.makefile("rwb")
        try:
            while proxy_impl(client_fd, httpd_fd):
                pass
        except ConnectionError:
            pass
    client.close()

def proxy_loop(httpd_port):
    port = 9493
    print(f"clear-text proxy: {port}")
    with socket.socket(socket.AF_INET, socket.SOCK_STREAM) as sock:
        sock.bind(("", port))
        sock.listen(16)
        sock.setblocking(True)
        while True:
            client, address = sock.accept()
            threading.Thread(target=proxy_client, args=(client, httpd_port), daemon=True).start()



# {{{1 tls .....................................................................

def tls_proxy_loop(httpd_port, root_cert, *server_pems):
    port = 4939
    print(f"tls proxy: {port}")

    temp_dir_obj = tempfile.TemporaryDirectory(prefix="iastestserver_")
    temp_dir = Path(temp_dir_obj.name)

    cert_path = temp_dir / "server_kc"
    with cert_path.open("wb") as out:
        for pem in server_pems:
            out.write(pem)
        # out.write(root_cert)

    ca_path = temp_dir / "server_ca"
    with ca_path.open("wb") as out:
        out.write(root_cert)

    import ssl
    ssl_ctx = ssl.SSLContext(ssl.PROTOCOL_TLS_SERVER)
    # ssl_ctx.load_verify_locations(ca_path)
    ssl_ctx.load_cert_chain(cert_path)

    with socket.socket(socket.AF_INET, socket.SOCK_STREAM) as sock:
        sock.bind(("", port))
        sock.listen(16)
        sock.setblocking(True)
        while True:
            try:
                client, address = sock.accept()
                client = ssl_ctx.wrap_socket(client, server_side=True)
            except (ssl.SSLError, Exception) as e:
                print("ERR:", str(e))
                continue
            threading.Thread(target=proxy_client, args=(client, httpd_port), daemon=True).start()



# {{{1 httpd ...................................................................

payload_data = bytearray((x % (128 - 34)) + 33 for x in random.randbytes(2 << 20))

def _make_payload(size):
    size = int(size)
    if size <= 0:
        size = int(random.random() * (8 << 10)) + 16
    size = min(size, len(payload_data))

    ret = payload_data[-size:]

    ret_hash = 0x493
    for c in ret:
        ret_hash = ((ret_hash + c) * 0x493) & 0xFFffFFff

    return ret, ret_hash

def http_seed(handler, value=0):
    handler.send_response(200)
    handler.send_header("Content-Length", 0)
    handler.end_headers()
    random.seed(value)

def http_data(handler, payload_size=0):
    payload, payload_hash = _make_payload(payload_size)

    handler.send_response(200)

    mega_size = int(random.random() * (4 << 10))
    for i in range(1024):
        key = "X-MegaHeader-%04d" % i
        value = payload_data[:int(random.random() * 64)]
        value = base64.b64encode(value)
        handler.send_header(key, value.decode())
        mega_size -= len(key) + len(value) + 4
        if mega_size <= 0:
            break

    handler.send_header("X-TestServer-Hash", payload_hash)
    handler.send_header("Content-Length", len(payload))
    handler.send_header("Content-Type", "application/octet-stream")
    handler.end_headers()
    handler.wfile.write(payload)
    handler.wfile.flush()

def http_chunked(handler, payload_size=0, *options):
    ext_payload = b""
    if "ext" in options:
        n = int(random.random() * 32)
        ext_payload = "".join(random.choices("Trigrams; Diner", k=n))
        ext_payload = b";" + ext_payload.encode()

    trailer_payload = b"X-TestServer-Trailer" if "trailer" in options else b""

    get_crlf = lambda: b"\r\n"
    if "tamper" in options:
        get_crlf = lambda: "".join(random.choices("XX\r\r\n", k=2)).encode()

    payload, payload_hash = _make_payload(payload_size)

    handler.send_response(200)
    handler.send_header("Transfer-Encoding", "chunked")
    handler.send_header("X-TestServer-Hash", payload_hash)
    handler.send_header("X-TestServer-Size", len(payload))
    handler.send_header("Content-Type", "application/octet-stream")
    if trailer_payload:
        handler.send_header("Trailer:", trailer_payload)
    handler.end_headers()

    max_chunk_size = int(random.random() * 1024) + 1
    while True:
        chunk_size = int(random.random() * max_chunk_size) + 1
        piece = payload[:chunk_size]

        header = b"%x" % len(piece)
        if piece and piece[0] & 0b0100:
            header = header.upper()
        header += ext_payload + get_crlf()

        handler.wfile.write(header)
        handler.wfile.write(piece)
        if trailer_payload and not piece:
            handler.wfile.write(trailer_payload + b": true\r\n")
        handler.wfile.write(get_crlf())
        # handler.wfile.flush()
        if not piece:
            break
        payload = payload[chunk_size:]

def http_redirect(handler, style="abs", code=302, *dest):
    loc = "/" + "/".join(dest)
    if style.startswith("abs"):
        proto = "https://" if style == "abs_s" else "http://"
        port = 4939 if style == "abs_s" else 9493
        host = handler.headers.get("Host")
        if host.startswith("localhost"):
            host = "127.0.49.3"
        loc = f"{proto}{host}:{port}{loc}"
    handler.send_response(int(code))
    handler.send_header("Content-Length", "0")
    handler.send_header("Location", loc)
    handler.end_headers()

class Handler(http.server.BaseHTTPRequestHandler):
    corpus_thunk = None

    def _preroll(self):
        self.protocol_version = "HTTP/1.1"

        conn_value = self.headers.get("Connection", "").lower()
        self.close_connection = (conn_value == "close")

        parts = [x for x in self.path.split("/") if x]
        if len(parts) >= 1:
            return parts

        self.send_error(404)

    def parse_request(self):
        ret = super().parse_request()

        if self.corpus_thunk:
            self.corpus_thunk(self.path, self.wfile)

        return ret

    def do_HEAD(self):
        parts = self._preroll()
        if not parts:
            return

        self.send_response(200)
        if random.random() < 0.75:
            self.send_header("Content-Length", 0)
        self.end_headers()

    def end_headers(self):
        if self.close_connection:
            self.send_header("Connection", "close")
        super().end_headers()

    def do_GET(self):
        parts = self._preroll()
        if not parts:
            return self.send_error(404)

        if query := parts[-1].split("?"):
            parts[-1] = query[0]

        if parts[0] == "port":
            payload = str(self.server.server_address[1]).encode()
            self.send_response(200)
            self.send_header("Content-Length", len(payload))
            self.end_headers()
            self.wfile.write(payload)
            return

        if parts[0] == "ca":
            ca_pem = self.server.ca_pem
            self.send_response(200)
            self.send_header("Content-Length", len(ca_pem))
            self.end_headers()
            self.wfile.write(ca_pem)
            return

        if parts[0] == "hello":
            self.send_response(200)
            self.send_header("Content-Length", 5)
            self.end_headers()
            self.wfile.write(b"hello")
            return

        if parts[0] == "redirect":
            return http_redirect(self, *parts[1:])

        if parts[0] == "data":
            return http_data(self, *parts[1:])

        if parts[0] == "chunked":
            return http_chunked(self, *parts[1:])

        if parts[0] == "seed":
            return http_seed(self, *parts[1:])

        return self.send_error(404, f"not found '{self.path}'")

def plain_httpd_loop(port, root_pem, corpus_thunk):
    Handler.corpus_thunk = corpus_thunk

    print(f"httpd: {port}")
    print("corpus: ", corpus_thunk.__name__ if corpus_thunk else "")
    server = http.server.ThreadingHTTPServer(("", port), Handler)
    server.ca_pem = root_pem
    server.serve_forever()



# {{{1 certs ...................................................................

def gen_test_certs(openssl_bin):
    temp_dir_obj = tempfile.TemporaryDirectory(prefix="iastestserver_")
    temp_dir = Path(temp_dir_obj.name)

    empty_cnf = temp_dir / "empty.cnf"
    with empty_cnf.open("wb") as out:
        out.write(b"[req]\n")
        out.write(b"distinguished_name=ridgers\n")
        out.write(b"[ridgers]\n")

    root_key_path = temp_dir / "root_k"
    with root_key_path.open("wb") as out:
        sp.run((openssl_bin, "genrsa", "2048"), stdout=out)

    root_path = temp_dir / "root_c"
    with root_path.open("wb") as out:
        sp.run((openssl_bin,
            "req", "-new", "-x509",
            "-nodes",
            "-sha256",
            "-key", str(root_key_path),
            "-subj", "/C=SE/ST=SE/L=Stockholm/O=IasRoot/CN=localhost",
            "-days", "10",
            "-config", str(empty_cnf)),
            stdout=out
        )

    key_path = temp_dir / "server_k"
    with key_path.open("wb") as out:
        sp.run((openssl_bin, "genrsa", "2048"), stdout=out)

    req_path = temp_dir / "server_r"
    with req_path.open("wb") as out:
        sp.run((openssl_bin,
            "req", "-new",
            "-nodes",
            "-sha256",
            "-key", str(key_path),
            "-subj", "/C=SE/ST=SE/L=Stockholm/O=Ias/CN=localhost",
            "-config", str(empty_cnf)),
            stdout=out
        )

    cert_path = temp_dir / "server_c"
    with cert_path.open("wb") as out:
        sp.run((openssl_bin,
            "x509", "-req",
            "-sha256",
            "-in", str(req_path),
            "-days", "10",
            "-set_serial", "493",
            "-CA", str(root_path),
            "-CAkey", str(root_key_path)),
            stdout=out
        )

    with root_path.open("rb") as inp: r = inp.read()
    with cert_path.open("rb") as inp: s = inp.read()
    with key_path.open("rb") as inp:  k = inp.read()
    return r, s, k



# {{{1 main ....................................................................

def setup_corpus(args):
    if not (corpus_dir := args.corpusdir[0]):
        return

    corpus_dir.mkdir(parents=True, exist_ok=True)
    for item in corpus_dir.glob("*.iax"):
        item.unlink()

    def corpus_collector(self, path, writer):
        if random.random() > args.corpuschance[0]:
            return

        path = path[1:]
        path = path.replace("/", "-")
        path = path.replace("?", "-")
        path = path.replace(":", "-")

        name = "test_%08x_%s.iax" % (random.randrange(0, 0x7fffffff), path)
        out = (corpus_dir / name).open("wb")

        if not (orig_writer := getattr(writer, "orig_writer", None)):
            writer.orig_writer = orig_writer = writer.write

        def write(data):
            out.write(data)
            return orig_writer(data)

        def close():
            out.close()

        writer.write = write
        writer.close = close

    return corpus_collector

def main(args):
    for item in Path(__file__).parents:
        if (item / "GenerateProjectFiles.bat").is_file():
            os.chdir(item)
            with_tls = True
            break
    else:
        with_tls = False

    if with_tls:
        print("\n## generating certificates")
        openssl_bin = args.opensslbin[0] or os.getenv("OPENSSL_BIN", "openssl")
        if (x := Path(".") / "Engine/Binaries/DotNET/IOS/openssl.exe").is_file():
            openssl_bin = str(x)
        root_cert, server_cert, server_key = gen_test_certs(openssl_bin)
    else:
        root_cert = None

    httpd_port = args.port[0] or int(os.getenv("IasTestServerPort", 0))
    httpd_port = httpd_port or (int(random.random() * 0x8000) + 0x4000)

    def start_svc(target, *args):
        threading.Thread(target=target, args=args, daemon=True).start()

    corpus_thunk = setup_corpus(args)

    print("\n## starting servers")
    start_svc(proxy_loop, httpd_port)
    start_svc(plain_httpd_loop, httpd_port, root_cert, corpus_thunk)
    if with_tls:
        start_svc(tls_proxy_loop, httpd_port, root_cert, server_key, server_cert)

    time.sleep(0.25)
    print("\n## ready")
    while True:
        time.sleep(3600)

if __name__ == "__main__":
    import argparse
    parser = argparse.ArgumentParser(description="Test server to 'IasTool tests'")
    parser.add_argument("--port",         nargs=1, default=(0,),    type=int,   help="Port to listen on", action="store")
    parser.add_argument("--opensslbin",   nargs=1, default=(None,), type=Path,  help="Path to OpenSSL binary used to generate certs")
    parser.add_argument("--corpusdir",    nargs=1, default=(None,), type=Path,  help="Directory to output response fuzzing corpus")
    parser.add_argument("--corpuschance", nargs=1, default=(0.2,),  type=float, help="Chance of writing corpus item (0.0-1.0)")
    args = parser.parse_args()
    raise SystemExit(main(args))

# vim: et
