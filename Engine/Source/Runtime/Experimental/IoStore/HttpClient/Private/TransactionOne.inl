// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::IoStore::HTTP
{

////////////////////////////////////////////////////////////////////////////////
struct FMessageOffsets
{
	uint8	StatusCode;
	uint8	Message;
	uint16	Headers;
};

static int32 ParseMessage(FAnsiStringView Message, FMessageOffsets& Out)
{
	const FAnsiStringView Protocol("HTTP/1.1 ");

	// Check there's enough data
	if (Message.Len() < Protocol.Len() + 1) // "+1" accounts for at least one digit
	{
		return -1;
	}

	const char* Cursor = Message.GetData();

	// Check for the expected protocol
	if (FAnsiStringView(Cursor, 9) != Protocol)
	{
		return -1;
	}
	int32 i = Protocol.Len();

	// Trim left and tightly reject anything adventurous
	for (int n = 32; i < n && Cursor[i] == ' '; ++i);
	Out.StatusCode = uint8(i);

	// At least one status line digit. (Note to self; expect exactly three)
	for (int n = 32; i < n && uint32(Cursor[i] - 0x30) <= 9; ++i);
	if (uint32(i - Out.StatusCode - 1) > 32)
	{
		return -1;
	}

	// Trim left
	for (int n = 32; i < n && Cursor[i] == ' '; ++i);
	Out.Message = uint8(i);

	// Extra conservative length allowance
	if (i > 32)
	{
		return -1;
	}

	// Find \r\n
	for (; Cursor[i] != '\r'; ++i)
	{
		if (i >= 2048)
		{
			return -1;
		}
	}
	if (Cursor[i + 1] != '\n')
	{
		return -1;
	}
	Out.Headers = uint16(i + 2);

	return 1;
}



////////////////////////////////////////////////////////////////////////////////
template <typename LambdaType>
static void EnumerateHeaders(FAnsiStringView Headers, LambdaType&& Lambda)
{
	// NB. here we are assuming that we will be dealing with servers that will
	// not be returning headers with "obsolete line folding".

	auto IsOws = [] (int32 c) { return (c == ' ') | (c == '\t'); };

	const char* Cursor = Headers.GetData();
	const char* End = Cursor + Headers.Len();
	do
	{
		int32 ColonIndex = 0;
		for (; Cursor + ColonIndex < End; ++ColonIndex)
		{
			if (Cursor[ColonIndex] == ':')
			{
				break;
			}
		}

		Cursor += ColonIndex;

		const char* Right = Cursor + 1;
		while (Right + 1 < End)
		{
			if (Right[0] != '\r' || Right[1] != '\n')
			{
				Right += 1 + (Right[1] != '\r');
				continue;
			}

			FAnsiStringView Name(Cursor - ColonIndex, ColonIndex);

			const char* Left = Cursor + 1;
			for (; IsOws(Left[0]); ++Left);

			Cursor = Right;
			for (; Cursor > Left + 1 && IsOws(Cursor[-1]); --Cursor);

			FAnsiStringView Value (Left, int32(ptrdiff_t(Cursor - Left)));

			if (!Lambda(Name, Value))
			{
				Right = End;
			}

			break;
		}

		Cursor = Right + 2;
	} while (Cursor < End);
}



namespace DetailOne
{

////////////////////////////////////////////////////////////////////////////////
class FBase
{
public:
					FBase();

protected:
	using FMutableSection = FBuffer::FMutableSection;

	const char*		GetData() const;
	uint32			GetSize() const;
	FMutableSection	GetMutableFree(uint32 MinSize, uint32 PageSize=256);
	void			AdvanceUsed(uint32 Delta);

private:
	FBuffer			Buffer;
	char			Data[256];
};

////////////////////////////////////////////////////////////////////////////////
FBase::FBase()
: Buffer(Data, sizeof(Data)) // -V670
{
}

////////////////////////////////////////////////////////////////////////////////
const char* FBase::GetData() const
{
	return Buffer.GetData();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FBase::GetSize() const
{
	return Buffer.GetSize();
}

////////////////////////////////////////////////////////////////////////////////
FBase::FMutableSection FBase::GetMutableFree(uint32 MinSize, uint32 PageSize)
{
	return Buffer.GetMutableFree(MinSize, PageSize);
}

////////////////////////////////////////////////////////////////////////////////
void FBase::AdvanceUsed(uint32 Delta)
{
	return Buffer.AdvanceUsed(Delta);
}



////////////////////////////////////////////////////////////////////////////////
class FRequest
	: public FBase
{
public:
	using FBase::FBase;

	void			Begin(FAnsiStringView Host, FAnsiStringView Method, FAnsiStringView Path);
	void			AddHeader(FAnsiStringView Key, FAnsiStringView Value);
	FTransactId		End(bool bKeepAlive);
	FOutcome		TrySendRequest(FTlsPeer& Peer);

private:
	FRequest&		operator << (FAnsiStringView Value);
	uint16			HeaderLeft;
	uint16			AlreadySent = 0;
	int16			HeaderRight = -1;
	int8			MethodLength = -1;
	uint8			_Unnused;
};

////////////////////////////////////////////////////////////////////////////////
void FRequest::Begin(FAnsiStringView Host, FAnsiStringView Method, FAnsiStringView Path)
{
	check(MethodLength < 0);
	AlreadySent = uint16(GetSize());
	*this << Method << " " << Path << " HTTP/1.1" "\r\n";
	MethodLength = int8(Method.Len());
	HeaderLeft = uint16(GetSize());

	AddHeader("Host", Host);
}

////////////////////////////////////////////////////////////////////////////////
void FRequest::AddHeader(FAnsiStringView Key, FAnsiStringView Value)
{
	check(HeaderRight < 0);
	*this << Key << ":" << Value << "\r\n";
}

////////////////////////////////////////////////////////////////////////////////
FTransactId FRequest::End(bool bKeepAlive)
{
	// HTTP/1.1 is persistent by default thus "Connection" header isn't required
	// unless we want to opt in to a single transaction.
	if (!bKeepAlive)
	{
		AddHeader("Connection", "close");
	}

	*this << "\r\n";
	HeaderRight = int16(GetSize());
	return 1;
}

////////////////////////////////////////////////////////////////////////////////
FRequest& FRequest::operator << (FAnsiStringView Value)
{
	uint32 Length = uint32(Value.Len());
	FBuffer::FMutableSection Section = GetMutableFree(Length);
	::memcpy(Section.Data, Value.GetData(), Length);
	AdvanceUsed(Length);
	return *this;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FRequest::TrySendRequest(FTlsPeer& Peer)
{
	const char* SendData = GetData();
	int32 SendSize = GetSize();

	SendData += AlreadySent;
	SendSize -= AlreadySent;
	check(SendSize > 0);

	FOutcome Outcome = Peer.Send(SendData, SendSize);
	if (!Outcome.IsOk())
	{
		return Outcome;
	}

	int32 Result = Outcome.GetResult();
	AlreadySent += uint16(Result);
	if (AlreadySent == GetSize())
	{
		return FOutcome::Ok();
	}

	check(AlreadySent < GetSize());
	return FOutcome::Waiting();
}



////////////////////////////////////////////////////////////////////////////////
class FStatusHeaders
	: public FRequest
{
public:
	using FRequest::FRequest;

	FOutcome		TryRecvResponse(FTlsPeer& Peer);
	bool			IsKeepAlive() const;
	uint32			GetStatusCode() const;
	FAnsiStringView	GetStatusMessage() const;
	int64			GetContentLength() const;
	bool			IsChunked() const;
	void			ReadHeaders(FResponse::FHeaderSink Sink) const;

protected:
	const char*		GetMessageRight() const;

private:
	FOutcome		Parse();
	int64			ContentLength = -1;
	FMessageOffsets	Offsets = {};
	int16			MessageLeft = -1;
	int16			MessageRight = -1;
	uint16			StatusCode = 0;
	uint8			bKeepAlive : 1;
	uint8			bChunked : 1;
	uint8			_Unused : 7;
};

////////////////////////////////////////////////////////////////////////////////
bool FStatusHeaders::IsKeepAlive() const
{
	return (MessageRight > 0) ? bKeepAlive : true;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStatusHeaders::GetStatusCode() const
{
	check(StatusCode >= 100);
	return StatusCode;
}

////////////////////////////////////////////////////////////////////////////////
FAnsiStringView	FStatusHeaders::GetStatusMessage() const
{
	check(Offsets.Message > 0);
	const char* Cursor = GetData() + MessageLeft;
	return FAnsiStringView(Cursor + Offsets.Message, Offsets.Headers - Offsets.Message);
}

////////////////////////////////////////////////////////////////////////////////
int64 FStatusHeaders::GetContentLength() const
{
	check(Offsets.Headers > 0);
	return ContentLength;
}

////////////////////////////////////////////////////////////////////////////////
bool FStatusHeaders::IsChunked() const
{
	check(MessageRight > 0);
	return bChunked;
}

////////////////////////////////////////////////////////////////////////////////
void FStatusHeaders::ReadHeaders(FResponse::FHeaderSink Sink) const
{
	uint32 Offset = MessageLeft + Offsets.Headers;
	uint32 Length = MessageRight - Offset - 2; // "-2" trims off '\r\n' that signals end of headers
	const char* Cursor = GetData() + Offset;
	FAnsiStringView Headers(Cursor, Length);
	UE::IoStore::HTTP::EnumerateHeaders(Headers, Sink);
}

////////////////////////////////////////////////////////////////////////////////
const char* FStatusHeaders::GetMessageRight() const
{
	return GetData() + MessageRight;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FStatusHeaders::TryRecvResponse(FTlsPeer& Peer)
{
	auto FindMessageTerminal = [] (const char* Cursor, int32 Length)
	{
		for (int32 i = 4; i <= Length; ++i)
		{
			uint32 Candidate;
			::memcpy(&Candidate, Cursor + i - 4, sizeof(Candidate));
			if (Candidate == 0x0a0d0a0d)
			{
				return i;
			}

			i += (Cursor[i - 1] > 0x0d) ? 3 : 0;
		}

		return -1;
	};

	if (MessageLeft < 0)
	{
		MessageLeft = int16(GetSize());
		bKeepAlive = true;
		bChunked = false;
	}

	while (true)
	{
		static const uint32 PageSize = 256;
		static const uint32 MaxHeaderSize = 8 << 10;

		auto [Dest, DestSize] = GetMutableFree(0, PageSize);
		FOutcome Outcome = Peer.Recv(Dest, DestSize);
		if (!Outcome.IsOk())
		{
			return Outcome;
		}

		int32 Result = Outcome.GetResult();
		AdvanceUsed(Result);

		// Rewind a little to cover cases where the terminal is fragmented across
		// recv() calls
		uint32 DestBias = 0;
		if (Dest - 3 >= GetData() + MessageLeft)
		{
			Dest -= (DestBias = 3);
		}

		int32 MessageEnd = FindMessageTerminal(Dest, Result + DestBias);
		if (MessageEnd < 0)
		{
			if (GetSize() - MessageLeft > MaxHeaderSize)
			{
				return FOutcome::Error("Headers have grown larger than expected");
			}

			continue;
		}

		MessageRight = int16(ptrdiff_t(Dest + MessageEnd - GetData()));
		return Parse();
	}
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FStatusHeaders::Parse()
{
	const char* Cursor = GetData() + MessageLeft;
	FAnsiStringView MessageView(Cursor, MessageRight - MessageLeft);

	if (ParseMessage(MessageView, Offsets) < 0)
	{
		return FOutcome::Error("Failed to parse message status");
	}

	FAnsiStringView StatusCodeView(
		Cursor + Offsets.StatusCode,
		Offsets.Message - Offsets.StatusCode
	);
	StatusCode = uint16(CrudeToInt(StatusCodeView));
	if (!IsStatusCodeOk(StatusCode))
	{
		return FOutcome::Error("Invalid status code", StatusCode);
	}

	static const uint32 Flag_Length		= 1 << 0;
	static const uint32 Flag_XferEnc	= 1 << 1;
	static const uint32 Flag_Connection = 1 << 2;
	static const uint32 Flag_All		= 0x07;
	uint32 Flags = 0;

	ReadHeaders(
		[this, &Flags] (FAnsiStringView Name, FAnsiStringView Value) mutable
		{
			// todo; may need smarter value handling; ;/, separated options & key-value pairs (ex. in rfc2068)

			if (Name.Equals("Content-Length", ESearchCase::IgnoreCase))
			{
				ContentLength = int32(CrudeToInt(Value));
				Flags |= Flag_Length;
			}

			else if (Name.Equals("Transfer-Encoding", ESearchCase::IgnoreCase))
			{
				bChunked = Value.Equals("chunked", ESearchCase::IgnoreCase);
				Flags |= Flag_XferEnc;
			}

			else if (Name.Equals("Connection", ESearchCase::IgnoreCase))
			{
				bKeepAlive = !Value.Equals("close", ESearchCase::IgnoreCase);
				Flags |= Flag_Connection;
			}

			return Flags != Flag_All;
		}
	);

	if (Flags & Flag_XferEnc)
	{
		if (!bChunked)				return FOutcome::Error("Unsupported Transfer-Encoding");
		if (Flags & Flag_Length)	return FOutcome::Error("Chunked yet with length");
		ContentLength = -1;
	}
	else if (uint32 Overshoot = GetSize() - MessageRight; Flags & Flag_Length)
	{
		if (ContentLength < 0)			return FOutcome::Error("Invalid Content-Length field", uint32(ContentLength));
		if (Overshoot > ContentLength)	return FOutcome::Error("More data received that expected");
	}
	else if (Overshoot)
	{
		return FOutcome::Error("Received content when none was expected");
	}
	else
	{
		ContentLength = 0;
	}

	if (IsContentless(StatusCode))
	{
		ContentLength = 0;
		bChunked = false;
	}

	return FOutcome::Ok();
}



////////////////////////////////////////////////////////////////////////////////
class FBody
	: public FStatusHeaders
{
public:
	using FStatusHeaders::FStatusHeaders;

	int64				GetRemaining() const;
	FOutcome			TryRecv(FMutableMemoryView Dest, FTlsPeer& Peer);

private:
	enum class EState : uint8 { Init, Recv, Prologue, Chunk, Epilogue, Done };

	FOutcome			Gather(FMutableMemoryView Dest, FTlsPeer& Peer);
	FOutcome			Init(FMutableMemoryView Dest, FTlsPeer& Peer);
	FOutcome			Recv(FMutableMemoryView Dest, FTlsPeer& Peer);
	FOutcome			Prologue(FMutableMemoryView Dest, FTlsPeer& Peer);
	FOutcome			Chunk(FMutableMemoryView Dest, FTlsPeer& Peer);
	FOutcome			Epilogue(FMutableMemoryView Dest, FTlsPeer& Peer);
	FMemoryView			Overspill;
	FMutableMemoryView	Scratch;
	int64				Remaining = 0;
	EState				State = EState::Init;

	enum
	{
		HeaderBufSize	= 32,
		CrLfLength		= 2,
		EndOfXfer		= -1,
	};
};

////////////////////////////////////////////////////////////////////////////////
int64 FBody::GetRemaining() const
{
	return (State == EState::Recv) ? Remaining : -1;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FBody::TryRecv(FMutableMemoryView Dest, FTlsPeer& Peer)
{
	if (Dest.GetSize() == 0)
	{
		return FOutcome::Error("Empty destination");
	}

	FOutcome Outcome = FOutcome::None();

	switch (State)
	{
	case EState::Init:	Outcome = Init(Dest, Peer);		break;
	case EState::Recv:	Outcome = Recv(Dest, Peer);		break;
	default:			Outcome = Gather(Dest, Peer);	break;
	case EState::Done:	Outcome = FOutcome::Error("Transaction is complete"); break;
	}

	if (Outcome.IsOk())
	{
		State = EState::Done;
	}

	return Outcome;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FBody::Gather(FMutableMemoryView Dest, FTlsPeer& Peer)
{
	FOutcome Outcome = FOutcome::None();

	bool bDone = false;
	int32 Size = 0;
	do
	{
		switch (State)
		{
		case EState::Prologue:	Outcome = Prologue(Dest, Peer); break;
		case EState::Chunk:		Outcome = Chunk(Dest, Peer);	break;
		case EState::Epilogue:	Outcome = Epilogue(Dest, Peer);	break;
		default:				Outcome = FOutcome::Error("Unexpected try-chunked state"); break;
		}

		if (Outcome.IsError())
		{
			return Outcome;
		}

		bDone = Outcome.IsOk();
		int32 Result = Outcome.GetResult();
		if (Result == 0)
		{
			break;
		}

		Size += Result;
		Dest = Dest.Mid(Result);
	}
	while (!Dest.IsEmpty());

	return bDone ? FOutcome::Ok(Size) : FOutcome::Waiting(Size);
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FBody::Init(FMutableMemoryView Dest, FTlsPeer& Peer)
{
	const char* DataEnd = GetData() + GetSize();
	const char* MessageEnd = GetMessageRight();
	uint32 AlreadyReceived = uint32(ptrdiff_t(DataEnd - MessageEnd));
	if (AlreadyReceived < HeaderBufSize)
	{
		GetMutableFree(HeaderBufSize, HeaderBufSize);
		MessageEnd = GetMessageRight();
	}

	if (AlreadyReceived)
	{
		Overspill = { (const uint8*)MessageEnd, AlreadyReceived };
	}

	Scratch = { (uint8*)MessageEnd, HeaderBufSize };
	
	if (IsChunked())
	{
		check(Remaining == 0);
		State = EState::Prologue;
	}
	else
	{
		Remaining = GetContentLength();
		check(Remaining > 0);
		State = EState::Recv;
	}

	return TryRecv(Dest, Peer);
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FBody::Recv(FMutableMemoryView Dest, FTlsPeer& Peer)
{
	int64 OverspillSize = Overspill.GetSize();
	if (OverspillSize != 0)
	{
		int64 Size = FMath::Min<int64>(Dest.GetSize(), OverspillSize);
		::memcpy(Dest.GetData(), Overspill.GetData(), Size);
		Dest = Dest.Mid(Size);
		Overspill = Overspill.Mid(Size);
		Remaining -= Size;
		if (Remaining == 0 || Dest.IsEmpty())
		{
			int32 Result = int32(Size);
			return Remaining ? FOutcome::WaitBuffer(Result) : FOutcome::Ok(Result);
		}
	}

	auto* Cursor = (char*)(Dest.GetData());
	uint32 Size = uint32(FMath::Min<int64>(Dest.GetSize(), Remaining));
	FOutcome Outcome = Peer.Recv(Cursor, Size);
	if (Outcome.IsError())
	{
		return Outcome;
	}

	if (Outcome.IsWaiting())
	{
		check(Outcome.GetResult() == 0);
		return FOutcome::Waiting(int32(OverspillSize));
	}

	int32 Result = Outcome.GetResult();
	Remaining -= Result;
	if (Remaining < 0)
	{
		return FOutcome::Error("Unexpectedly received too much", int32(Remaining));
	}

	Result += int32(OverspillSize);

	if (Remaining == 0)
	{
		return FOutcome::Ok(Result);
	}

	if (Result == int32(Dest.GetSize()))
	{
		return FOutcome::WaitBuffer(Result);
	}

	return FOutcome::Waiting(Result);
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FBody::Prologue(FMutableMemoryView Dest, FTlsPeer& Peer)
{
	auto FindHeader = [] (const char* Cursor, int32 Size)
	{
		// Base-16 size
		int32 HexDigLen = 0;
		for (; HexDigLen < Size; ++HexDigLen)
		{
			uint32 c = Cursor[HexDigLen];
			if ((c - '0' >= 10) & (c - 'a' >= 6) & (c - 'A' >= 6))
			{
				break;
			}
			continue;
		}

		int32 Delta = Size - HexDigLen;

		if (HexDigLen >= HeaderBufSize - 2)	return FOutcome::Error("Chunk size too large");
		if (HexDigLen == 0)					return FOutcome::Error("Invalid chunk header");
		if (Delta < 1)						return FOutcome::Waiting();
		if (Cursor[HexDigLen] != '\r')		return FOutcome::Error("Chunk extensions are not supported (ERREXT)");
		if (Delta < 2)						return FOutcome::Waiting();
		if (Cursor[HexDigLen + 1] != '\n')	return FOutcome::Error("Invalid chunk CRLF");

		return FOutcome::Ok(HexDigLen + 2);
	};

	const char* Header = nullptr;
	FOutcome Outcome = FOutcome::None();
	int32 HeaderLength = 0;

	uint32 OverspillSize = uint32(Overspill.GetSize());
	if (OverspillSize != 0)
	{
		// We try all the overspill first - we don't know how much we're expecting
		// and overspill could contain both header and all data (thus further recvs
		// could falsely hang us up).
		Header = (char*)Overspill.GetData();

		Outcome = FindHeader(Header, OverspillSize);
		if (Outcome.IsError())
		{
			return Outcome;
		}

		HeaderLength = Outcome.GetResult();
		Overspill = Overspill.Mid(HeaderLength);
	}

	if (HeaderLength == 0)
	{
		FMutableMemoryView View = Scratch;

		if (OverspillSize != 0)
		{
			if (OverspillSize > uint32(View.GetSize()))
			{
				return FOutcome::Error("Unexpectedly large overspill");
			}
			std::memmove(View.GetData(), Overspill.GetData(), OverspillSize);
			View = View.Mid(OverspillSize);
		}

		if (!View.IsEmpty())
		{
			char* Cursor = (char*)View.GetData();
			Outcome = Peer.Recv(Cursor, uint32(View.GetSize()));
			if (!Outcome.IsOk())
			{
				return Outcome;
			}
			View = Scratch.Left(OverspillSize + Outcome.GetResult());
		}

		char* Cursor = (char*)View.GetData();
		Outcome = FindHeader(Cursor, uint32(View.GetSize()));
		if (!Outcome.IsOk())
		{
			return Outcome;
		}

		check(View.GetData() == Scratch.GetData());
		Header = (char*)Scratch.GetData();
		HeaderLength = Outcome.GetResult();
		Overspill = View.Mid(HeaderLength);
	}

	// Read chunk size
	Header += Remaining;
	int64 ChunkSize = CrudeToInt<16>(FAnsiStringView(Header, HeaderLength));
	if (ChunkSize < 0)			return FOutcome::Error("Unparsable chunk size");
	if (ChunkSize > 16 << 20)	return FOutcome::Error("Unacceptable chunk size");

	if (ChunkSize == 0)
	{
		Remaining = EndOfXfer;
		::memset(Scratch.GetData(), 0, CrLfLength);
		State = EState::Epilogue;
		return Epilogue(Dest, Peer);
	}

	Remaining = ChunkSize;
	State = EState::Chunk;
	return Chunk(Dest, Peer);
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FBody::Chunk(FMutableMemoryView Dest, FTlsPeer& Peer)
{
	check(Remaining != 0);

	Dest = Dest.Left(Remaining);
	FOutcome Outcome = Recv(Dest, Peer);
	if (!Outcome.IsOk())
	{
		return Outcome;
	}

	check(Remaining == 0);
	::memset(Scratch.GetData(), 0, CrLfLength);
	State = EState::Epilogue;

	int32 Result = Outcome.GetResult();
	return FOutcome::WaitBuffer(Result);
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FBody::Epilogue(FMutableMemoryView Dest, FTlsPeer& Peer)
{
	auto* Cursor = (uint8*)(Scratch.GetData());

	FMutableMemoryView CrLfBuffer(Cursor, CrLfLength);
	if (Cursor[0] != 0)
	{
		CrLfBuffer = { Cursor + 1, 1 };
	}

	check(Remaining <= 0);

	int64 RemainingCache = Remaining;
	Remaining = CrLfBuffer.GetSize();
	FOutcome Outcome = Recv(CrLfBuffer, Peer);
	if (!Outcome.IsOk())
	{
		Remaining = RemainingCache;
		return Outcome;
	}

	if (Cursor[0] != '\r' || Cursor[1] != '\n')
	{
		return FOutcome::Error("Trailing headers are not supported (ERRTRAIL)");
	}

	if (RemainingCache > EndOfXfer)
	{
		State = EState::Prologue;
		return Prologue(Dest, Peer);
	}

	return FOutcome::Ok();
}

} // namespace DetailOne



////////////////////////////////////////////////////////////////////////////////
class FTransactionOne
	: public DetailOne::FBody
{
public:
	using DetailOne::FBody::FBody;
};



////////////////////////////////////////////////////////////////////////////////
FOutcome HandshakeOne(FTlsPeer&, void*)
{
	return FOutcome::Ok();
}

////////////////////////////////////////////////////////////////////////////////
FOutcome TickOne(FTlsPeer&, void*)
{
	return FOutcome::Ok(1);
}

////////////////////////////////////////////////////////////////////////////////
void GoAwayOne(FTlsPeer&, void*)
{
}



////////////////////////////////////////////////////////////////////////////////
class FTransaction
{
public:
#define TR_METHOD(x, ...) \
	template <typename... Arg> auto x(Arg&&... Args) __VA_ARGS__ { \
		if (auto Self = UPTRINT(this); Self & 1) \
			return ((FTransactionTwo*)(Self ^ 1))->x(Forward<Arg>(Args)...); \
		return ((FTransactionOne*)this)->x(Forward<Arg>(Args)...); \
	}
	TR_METHOD(Begin)
	TR_METHOD(AddHeader)
	TR_METHOD(End)
	TR_METHOD(TrySendRequest)
	TR_METHOD(TryRecvResponse)
	TR_METHOD(IsKeepAlive, const)
	TR_METHOD(GetStatusCode, const)
	TR_METHOD(GetStatusMessage, const)
	TR_METHOD(GetContentLength, const)
	TR_METHOD(IsChunked, const)
	TR_METHOD(ReadHeaders, const)
	TR_METHOD(GetRemaining, const)
	TR_METHOD(TryRecv)
#undef TR_METHOD

private:
	friend class FTransactRef;

	void Destruct()
	{
		if (auto Self = UPTRINT(this); Self & 1)
		{
			void* Addr = (void*)(Self ^ 1);
			((FTransactionTwo*)Addr)->~FTransactionTwo();
			FMemory::Free(Addr);
			return;
		}

		((FTransactionOne*)this)->~FTransactionOne();
		FMemory::Free(this);
	}
};



////////////////////////////////////////////////////////////////////////////////
class FTransactRef
{
public:
						FTransactRef() = default;
						~FTransactRef();
						FTransactRef(UPTRINT Ptr, int32 Ver);
						FTransactRef(FTransactRef&&) = delete;
						FTransactRef(const FTransactRef&) = delete;
	void				operator = (const FTransactRef& Rhs) = delete;
	void				operator = (FTransactRef&& Rhs);
	const FTransaction*	operator -> () const;
	FTransaction*		operator -> ();
	bool				IsValid() const;

private:
	const FTransaction*	Get() const;
	FTransaction*		Get();
	UPTRINT				Self = 0;
};

////////////////////////////////////////////////////////////////////////////////
FTransactRef::FTransactRef(UPTRINT Ptr, int32 Ver)
: Self(Ptr | (Ver == 2))
{
	check((Ptr & 1) == 0);
}

////////////////////////////////////////////////////////////////////////////////
void FTransactRef::operator = (FTransactRef&& Rhs)
{
	Swap(Self, Rhs.Self);
}

////////////////////////////////////////////////////////////////////////////////
const FTransaction*	FTransactRef::operator -> () const
{
	return Get();
}

////////////////////////////////////////////////////////////////////////////////
FTransaction* FTransactRef::operator -> ()		
{
	return Get();
}

////////////////////////////////////////////////////////////////////////////////
bool FTransactRef::IsValid() const		
{
	return Get() != nullptr;
}

////////////////////////////////////////////////////////////////////////////////
const FTransaction* FTransactRef::Get() const			
{
	return (FTransaction*)Self;
}

////////////////////////////////////////////////////////////////////////////////
FTransaction* FTransactRef::Get()				
{
	return (FTransaction*)Self;
}

////////////////////////////////////////////////////////////////////////////////
FTransactRef::~FTransactRef()
{
	if (Self == 0)
	{
		return;
	}

	Get()->Destruct();
}



////////////////////////////////////////////////////////////////////////////////
FTransactRef CreateTransactOne(void*&)
{
	void* Ptr = FMemory::Malloc(sizeof(FTransactionOne), alignof(FTransactionOne));
	new (Ptr) FTransactionOne();
	return FTransactRef(UPTRINT(Ptr), 1);
}

////////////////////////////////////////////////////////////////////////////////
FTransactRef CreateTransactTwo(void* PeerData)
{
#if IAS_HTTP_WITH_TWO
	using namespace DetailTwo;

	auto* Driver = (FDriverNg*)PeerData;

	void* Ptr = FMemory::Malloc(sizeof(FTransactionTwo), alignof(FTransactionTwo));
	new (Ptr) FTransactionTwo(*Driver);

	return FTransactRef(UPTRINT(Ptr), 2);
#else
	return FTransactRef();
#endif
}

} // namespace UE::IoStore::HTTP
