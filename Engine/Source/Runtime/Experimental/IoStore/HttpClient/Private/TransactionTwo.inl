// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::IoStore::HTTP
{

////////////////////////////////////////////////////////////////////////////////
using FTransactId = uint32;

////////////////////////////////////////////////////////////////////////////////
bool IsContentless(uint32 StatusCode)
{
	return (StatusCode < 200) | (StatusCode == 204) | (StatusCode == 304);
}

////////////////////////////////////////////////////////////////////////////////
bool IsStatusCodeOk(uint32 StatusCode)
{
	enum { MinCode = 100, MaxCode = 999 };
	return uint32(StatusCode - MinCode) <= uint32(MaxCode - MinCode);
}

} // namespace UE::IoStore::HTTP



////////////////////////////////////////////////////////////////////////////////
#if !defined(WITH_NGHTTP2)
#	define WITH_NGHTTP2 0
#	if defined(__has_include)
#		if __has_include(<nghttp2/nghttp2.h>)
#			undef WITH_NGHTTP2
#			define WITH_NGHTTP2 1
#		endif
#	endif
#endif

#if !defined(IAS_HTTP_WITH_TWO)
#	define IAS_HTTP_WITH_TWO WITH_NGHTTP2
#endif

#if IAS_HTTP_WITH_TWO

////////////////////////////////////////////////////////////////////////////////
#if defined(_MSC_VER)
#	define	NGHTTP2_NO_SSIZE_T
#endif
#if PLATFORM_MICROSOFT & !defined(NGHTTP2_STATICLIB)
#	define NGHTTP2_STATICLIB
#endif

#include <nghttp2/nghttp2.h>

#if PLATFORM_MICROSOFT & !defined(NGHTTP2_STATICLIB)
#	undef NGHTTP2_STATICLIB
#endif
#if defined(_MSC_VER)
#	undef NGHTTP2_NO_SSIZE_T
#endif

namespace UE::IoStore::HTTP
{

namespace DetailTwo
{

////////////////////////////////////////////////////////////////////////////////
enum { RECV_WINDOW_SIZE = 1024 << 10 };

////////////////////////////////////////////////////////////////////////////////
struct FFrame
{
	enum class EType : uint8
	{
		Data,			// = 0x00,
		Headers,		// = 0x01,
		Priority,		// = 0x02,
		RstStream,		// = 0x03,
		Settings,		// = 0x04,
		PushPromise,	// = 0x05,
		Ping,			// = 0x06,
		Goaway,			// = 0x07,
		WindowUpdate,	// = 0x08,
		Continuation,	// = 0x09,
		_Num,
	};

	enum : uint8
	{
		Flag_Padding	= 0x08,
		Flag_EndHeaders	= 0x04,
		Flag_EndStream	= 0x01,
	};

	uint8	Size[3];
	EType	Type;
	uint8	Flags;
	uint8	Id[4];

	int32	IsPertinent() const	{ return (Id[3] & 1) != 0; }
	uint32	GetSize() const		{ return (Size[0] << 16u) | (Size[1] << 8u) | Size[2]; }
	uint32	GetId() const		{ return (Id[0] << 24u) | (Id[1] << 16u) | (Id[2] << 8u) | Id[3]; }
	uint32	GetIdFast() const	{ return Id[3]; }
};
static_assert(sizeof(FFrame) == 9);



////////////////////////////////////////////////////////////////////////////////
class FBufferNg
{
public:
					FBufferNg() = default;
					~FBufferNg()							{ if (Inner != nullptr) nghttp2_rcbuf_decref(Inner); }
					FBufferNg(nghttp2_rcbuf* InRcBuf)		{ nghttp2_rcbuf_incref(Inner = InRcBuf); }
					FBufferNg(FBufferNg&& Rhs)				{ Swap(Inner, Rhs.Inner); }
					FBufferNg(const FBufferNg&) = delete;
	FBufferNg&		operator = (FBufferNg&& Rhs)			{ Swap(Inner, Rhs.Inner); return *this; }
	FBufferNg&		operator = (const FBufferNg&) = delete;

	FAnsiStringView	Get() const
	{
		check(Inner != nullptr);
		nghttp2_vec Ret = nghttp2_rcbuf_get_buf(Inner);
		return { (char*)(Ret.base), int32(Ret.len) };
	}

private:
	nghttp2_rcbuf*	Inner = nullptr;
};

////////////////////////////////////////////////////////////////////////////////
struct FHeaderNg
{
	FBufferNg	Key;
	FBufferNg	Value;
};



////////////////////////////////////////////////////////////////////////////////
class FDriverHeaders
{
public:
						FDriverHeaders() = default;
	void				Add(FAnsiStringView Key, FAnsiStringView Value);

private:
	friend class FDriverNg;

	const nghttp2_nv*	GetData() const	{ return Headers.GetData(); }
	int32 				Num() const		{ return Headers.Num(); }
	TArray<nghttp2_nv>	Headers;

	UE_NONCOPYABLE(FDriverHeaders);
};

////////////////////////////////////////////////////////////////////////////////
void FDriverHeaders::Add(FAnsiStringView Key, FAnsiStringView Value)
{
	nghttp2_nv Header{
		(uint8*)Key.GetData(),
		(uint8*)Value.GetData(),
		size_t(Key.Len()),
		size_t(Value.Len()),
		NGHTTP2_NV_FLAG_NO_COPY_NAME | NGHTTP2_NV_FLAG_NO_COPY_VALUE,
	};
	Headers.Add(MoveTemp(Header));
}



////////////////////////////////////////////////////////////////////////////////
class FDriverNg
{
public:
						FDriverNg(FTlsPeer& Peer);
	bool				IsUsingTls() const;
	void				GoAway(FTlsPeer& Peer);
	FOutcome			Handshake(FTlsPeer& Peer);
	FOutcome			GetTransactId(FTlsPeer& Peer);
	FTransactId			Submit(const FDriverHeaders& Headers);
	FOutcome			Send(FTlsPeer& Peer);
	FOutcome			RecvResponse(FTlsPeer& Peer, TArray<FHeaderNg>* Sink);
	FOutcome			RecvBody(FTlsPeer& Peer, FMutableMemoryView Dest);
	bool				CancelTransaction(FTlsPeer& Peer, FTransactId Id);
	void				SkipTransaction();

private:
	FOutcome			Tick(FTlsPeer& Peer);
	FOutcome			TickInner(FTlsPeer& Peer);
	FOutcome			PumpSend(FTlsPeer& Peer);
	FOutcome			PumpRecv(FTlsPeer& Peer, FMemoryView Data);
	int32 				OnHeader(nghttp2_rcbuf* Name, nghttp2_rcbuf* Value);
	nghttp2_session*	Session;
	TArray<FHeaderNg>*	ResponseHeaders = nullptr;
	FMemoryView			SendPending;
	int32				RecvRemaining;
	bool				bWithTls;
	uint8				RecvBuffer[512 - sizeof(FFrame)];
	union
	{
		FFrame			RecvFrame;
		uint8			RecvFrameData[sizeof(RecvFrame)];
	};

	UE_NONCOPYABLE(FDriverNg);
};

////////////////////////////////////////////////////////////////////////////////
FDriverNg::FDriverNg(FTlsPeer& Peer)
: bWithTls(Peer.IsUsingTls())
{
	struct FStaticsNg
	{
		~FStaticsNg()
		{
			nghttp2_session_callbacks_del(Instance);
		}
		nghttp2_session_callbacks* Instance = nullptr;
	};

	static FStaticsNg Statics;
	if (Statics.Instance == nullptr)
	{
		nghttp2_session_callbacks* Callbacks;
		nghttp2_session_callbacks_new(&Callbacks);

		nghttp2_session_callbacks_set_on_header_callback2(
			Callbacks,
			[] (nghttp2_session*, const nghttp2_frame*, nghttp2_rcbuf* Name, nghttp2_rcbuf* Value, uint8_t, void* UserData) -> int32
			{
				auto* This = (FDriverNg*)UserData;
				return This->OnHeader(Name, Value);
			}
		);

		Statics.Instance = Callbacks;
	}

	static nghttp2_mem Mem = {
		.mem_user_data	= nullptr,
		.malloc			= [] (size_t Size, void*)				{ return FMemory::Malloc(Size); },
		.free			= [] (void* Ptr, void*)					{ return FMemory::Free(Ptr); },
		.calloc			= [] (size_t Count, size_t Size, void*) { return FMemory::MallocZeroed(Count * Size); },
		.realloc		= [] (void* Ptr, size_t Size, void*)	{ return FMemory::Realloc(Ptr, Size); },
	};

	RecvRemaining = 0 - int32(sizeof(RecvFrame));

	nghttp2_session_client_new3(&Session, Statics.Instance, this, nullptr, &Mem);
	nghttp2_session_set_local_window_size(Session, 0, 0, RECV_WINDOW_SIZE);
}

////////////////////////////////////////////////////////////////////////////////
void FDriverNg::GoAway(FTlsPeer& Peer)
{
	if (Session == nullptr)
	{
		return;
	}

	nghttp2_session_terminate_session(Session, NGHTTP2_NO_ERROR);
	PumpSend(Peer);
	nghttp2_session_del(Session);
	Session = nullptr;
}

////////////////////////////////////////////////////////////////////////////////
bool FDriverNg::IsUsingTls() const
{
	return bWithTls;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FDriverNg::PumpSend(FTlsPeer& Peer)
{
	for (uint32 Sent = 0;;)
	{
		if (SendPending.IsEmpty())
		{
			const uint8* Data = nullptr;
			int32 Result = int32(nghttp2_session_mem_send2(Session, &Data));
			if (Result < 0)
			{
				const char* Message = nghttp2_strerror(Result);
				return FOutcome::Error(Message);
			}

			if (Result == 0)
			{
				return FOutcome::Ok(Sent);
			}

			SendPending = { Data, uint32(Result) };
		}

		const char* Data = (char*)(SendPending.GetData());
		uint32 Size = uint32(SendPending.GetSize());
		FOutcome Outcome = Peer.Send((char*)Data, Size);
		if (Outcome.IsError())
		{
			return Outcome;
		}

		if (Outcome.IsOk())
		{
			int32 Result = Outcome.GetResult();
			Sent += Result;

			SendPending = SendPending.Mid(Result);
			if (SendPending.IsEmpty())
			{
				continue;
			}
		}

		check(!SendPending.IsEmpty());
		return FOutcome::Waiting();
	}
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FDriverNg::PumpRecv(FTlsPeer& Peer, FMemoryView Data)
{
	int32 Result = int32(nghttp2_session_mem_recv2(Session, (uint8*)Data.GetData(), Data.GetSize()));

	if (Result < 0)
	{
		const char* Message = nghttp2_strerror(Result);
		return FOutcome::Error(Message);
	}

	if (Result != int32(Data.GetSize()))
	{
		return FOutcome::Error("Not enough data consumed");
	}

	if (nghttp2_session_want_write(Session))
	{
		// We'll not care about the outcome here. If an error occurred it will
		// surface later and data-waits will solved themselves later.
		PumpSend(Peer);
	}

	return FOutcome::Ok();
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FDriverNg::Handshake(FTlsPeer& Peer)
{
	nghttp2_settings_entry Settings[] = {
		{ NGHTTP2_SETTINGS_INITIAL_WINDOW_SIZE,		RECV_WINDOW_SIZE },
		{ NGHTTP2_SETTINGS_MAX_FRAME_SIZE,			RECV_WINDOW_SIZE >> 2 },
		{ NGHTTP2_SETTINGS_ENABLE_PUSH,				0 },
	};
	nghttp2_submit_settings(Session, NGHTTP2_FLAG_NONE, Settings, UE_ARRAY_COUNT(Settings));

	FOutcome Outcome = PumpSend(Peer);
	return Outcome.IsError() ? Outcome : FOutcome::Ok();
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FDriverNg::GetTransactId(FTlsPeer& Peer)
{
	if (RecvRemaining >= 0 && RecvFrame.IsPertinent())
	{
		uint32 Id = RecvFrame.GetId();
		return FOutcome::Ok(Id);
	}

	return Tick(Peer);
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FDriverNg::Tick(FTlsPeer& Peer)
{
	while (true)
	{
		FOutcome Outcome = TickInner(Peer);
		if (!Outcome.IsWaiting() || (Outcome.GetResult() == 0))
		{
			return Outcome;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FDriverNg::TickInner(FTlsPeer& Peer)
{
	FMutableMemoryView Dest(RecvBuffer, sizeof(RecvBuffer) + sizeof(RecvFrame));
	if (RecvRemaining < 0)
	{
		Dest = Dest.Right(0 - RecvRemaining);
	}
	else if (RecvRemaining <= int32(sizeof(RecvBuffer)))
	{
		Dest = Dest.Right(RecvRemaining + sizeof(RecvFrame));
	}
	else
	{
		Dest = Dest.Left(sizeof(RecvBuffer));
	}

	FOutcome Outcome = Peer.Recv((char*)(Dest.GetData()), int32(Dest.GetSize()));
	if (!Outcome.IsOk())
	{
		return Outcome;
	}

	int32 Result = Outcome.GetResult();
	Dest = Dest.Left(Result);

	FMutableMemoryView FrameView;
	if (const uint8* End = (uint8*)(Dest.GetDataEnd()); End > RecvFrameData)
	{
		FrameView = FMutableMemoryView(RecvFrameData, End - RecvFrameData);
		Dest = Dest.LeftChop(FrameView.GetSize());
	}

	if (!Dest.IsEmpty() && (Outcome = PumpRecv(Peer, Dest)).IsError())
	{
		return Outcome;
	}

	if (FrameView.IsEmpty())
	{
		if (RecvRemaining < int32(Dest.GetSize()))
		{
			return FOutcome::Error("RecvRemaining value too small");
		}

		if ((RecvRemaining -= int32(Dest.GetSize())) == 0)
		{
			RecvRemaining = 0 - int32(sizeof(RecvFrame));
		}
		return FOutcome::Waiting(Dest.GetDataEnd() == RecvFrameData);
	}

	if (Result = int32(FrameView.GetSize() - sizeof(RecvFrame)); Result < 0)
	{
		RecvRemaining = Result;
		return FOutcome::Waiting();
	}

	if ((Outcome = PumpRecv(Peer, FrameView)).IsError())
	{
		return Outcome;
	}

	RecvRemaining = RecvFrame.GetSize();
	if (RecvFrame.IsPertinent())
	{
		uint32 Id = RecvFrame.GetId();
		return FOutcome::Ok(Id);
	}

	if (RecvRemaining == 0)
	{
		RecvRemaining = 0 - int32(sizeof(RecvFrame));
	}

	return FOutcome::Waiting(1);
}

////////////////////////////////////////////////////////////////////////////////
FTransactId FDriverNg::Submit(const FDriverHeaders& Headers)
{
	return nghttp2_submit_request2(Session, nullptr, Headers.GetData(),
		Headers.Num(), nullptr, this);
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FDriverNg::Send(FTlsPeer& Peer)
{
	return PumpSend(Peer);
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FDriverNg::RecvResponse(FTlsPeer& Peer, TArray<FHeaderNg>* Sink)
{
	if (RecvFrame.Type == FFrame::EType::Data)
	{
		// Sometimes streams of requests with no content can be closed with an
		// empty data frame instead of via header frame's flags. By deferring
		// RecvResponse() completion we avoid orphaning zero-sized data frames
		if ((RecvFrame.Flags & FFrame::Flag_EndStream) && RecvFrame.GetSize() == 0)
		{
			RecvFrame = {};
		}
		return FOutcome::Ok();
	}

	ResponseHeaders = Sink;
	ON_SCOPE_EXIT { ResponseHeaders = nullptr; };

	while (true)
	{
		if (RecvFrame.Type == FFrame::EType::RstStream)
		{
			return FOutcome::Error("Received stream reset");
		}

		// Assume if we're filling we are expecting something
		check(RecvRemaining > 0);
		if (RecvFrame.Type != FFrame::EType::Headers)
		{
			return FOutcome::Error("Not a headers frame", int32(RecvFrame.Type));
		}

		int32 RecvSize = FMath::Min<int32>(sizeof(RecvBuffer), RecvRemaining);

		FMutableMemoryView Dest(RecvBuffer, RecvSize);
		FOutcome Outcome = Peer.Recv((char*)(Dest.GetData()), RecvSize);
		if (!Outcome.IsOk())
		{
			return Outcome;
		}
		int32 Result = Outcome.GetResult();

		Dest = Dest.Left(Outcome.GetResult());
		if (!Dest.IsEmpty() && (Outcome = PumpRecv(Peer, Dest)).IsError())
		{
			return Outcome;
		}

		RecvRemaining -= Result;
		if (RecvRemaining > 0)
		{
			continue;
		}

		check(RecvRemaining == 0);
		RecvRemaining -= sizeof(RecvFrame);

		int32 EndOfHeaders = (RecvFrame.Flags & FFrame::Flag_EndHeaders);

		if (RecvFrame.Flags & FFrame::Flag_EndStream)
		{
			check(EndOfHeaders);
			return FOutcome::Ok();
		}

		uint32 ThisId = RecvFrame.GetId();
		if (Outcome = Tick(Peer); Outcome.IsError())
		{
			return Outcome;
		}

		if (Outcome.GetResult() != ThisId)
		{
			return FOutcome::WaitStream();
		}

		if (EndOfHeaders)
		{
			// We shall end headers once the first data frame arrives, which may
			// have already happened in the Tick() call.
			return RecvResponse(Peer, ResponseHeaders);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FDriverNg::RecvBody(FTlsPeer& Peer, FMutableMemoryView Dest)
{
	for (uint32 RecvSize = 0;;)
	{
		if (RecvFrame.Type == FFrame::EType::RstStream)
		{
			return FOutcome::Error("Received stream reset");
		}

		check(RecvRemaining >= 0);
		if (RecvFrame.Type != FFrame::EType::Data)
		{
			return FOutcome::Error("Not a data frame", int32(RecvFrame.Type));
		}

		int32 EndOfStream = (RecvFrame.Flags & FFrame::Flag_EndStream);

		FMutableMemoryView View = Dest.Left(RecvRemaining);
		if (View.IsEmpty())
		{
			if (RecvRemaining <= 0 && EndOfStream)
			{
				RecvFrame = {};
				return FOutcome::Ok(RecvSize);
			}

			return FOutcome::WaitBuffer(RecvSize);
		}

		FOutcome Outcome = Peer.Recv((char*)View.GetData(), int32(View.GetSize()));
		if (!Outcome.IsOk())
		{
			if (Outcome.IsError())
			{
				return Outcome;
			}
			return FOutcome::Waiting(RecvSize);
		}

		int32 Result = Outcome.GetResult();
		RecvRemaining -= Result;
		RecvSize += Result;
		Dest = Dest.Mid(Result);

		if ((Outcome = PumpRecv(Peer, View.Left(Result))).IsError())
		{
			return Outcome;
		}

		// more of this frame to go?
		if (RecvRemaining > 0)
		{
			continue;
		}

		RecvRemaining = 0 - int32(sizeof(RecvFrame)) - RecvRemaining;

		// End of the road?
		if (EndOfStream)
		{
			RecvFrame = {};
			return FOutcome::Ok(RecvSize);
		}

		uint32 ThisId = RecvFrame.GetId();
		Outcome = Tick(Peer);
		if (Outcome.IsError())				return Outcome;
		if (ThisId != Outcome.GetResult())	return FOutcome::WaitBuffer(RecvSize);
	}
}

////////////////////////////////////////////////////////////////////////////////
bool FDriverNg::CancelTransaction(FTlsPeer& Peer, FTransactId Id)
{
	int32 Result = nghttp2_submit_rst_stream(Session, NGHTTP2_FLAG_NONE, Id, NGHTTP2_CANCEL);
	if (Result)
	{
		return false;
	}

	PumpSend(Peer);
	return true;
}

////////////////////////////////////////////////////////////////////////////////
void FDriverNg::SkipTransaction()
{
	if (!RecvFrame.IsPertinent())
	{
		return;
	}

	RecvFrame = {};
}

////////////////////////////////////////////////////////////////////////////////
int32 FDriverNg::OnHeader(nghttp2_rcbuf* Name, nghttp2_rcbuf* Value)
{
	if (ResponseHeaders == nullptr)
	{
		return NGHTTP2_ERR_CALLBACK_FAILURE;
	}

	FHeaderNg Header = { Name, Value };
	ResponseHeaders->Add(MoveTemp(Header));
	return 0;
}



////////////////////////////////////////////////////////////////////////////////
class FBase
{
public:
					FBase(FDriverNg& InDriver);

protected:
	FDriverNg&		GetDriver();

private:
	FDriverNg&		Driver;
};

////////////////////////////////////////////////////////////////////////////////
FBase::FBase(FDriverNg& InDriver)
: Driver(InDriver)
{
}

////////////////////////////////////////////////////////////////////////////////
FDriverNg& FBase::GetDriver()
{
	return Driver;
}



////////////////////////////////////////////////////////////////////////////////
class FRequest
	: public FBase
{
public:
	using FBase::FBase;

	void			Begin(FAnsiStringView Host, FAnsiStringView Method, FAnsiStringView Path);
	void			AddHeader(FAnsiStringView Key, FAnsiStringView Value);
	FTransactId		End(bool bKeepAlive);
	FOutcome		TrySendRequest(FTlsPeer& Peer);

protected:
	FTransactId		GetTransactId() const;

private:
	FDriverHeaders	RequestHeaders;
	FTransactId		Id;
};

////////////////////////////////////////////////////////////////////////////////
void FRequest::Begin(FAnsiStringView Host, FAnsiStringView Method, FAnsiStringView Path)
{
	FAnsiStringView Scheme = GetDriver().IsUsingTls() ? "https" : "http";
	RequestHeaders.Add(":method", Method);
	RequestHeaders.Add(":path", Path);
	RequestHeaders.Add(":authority", Host);
	RequestHeaders.Add(":scheme", Scheme);
}

////////////////////////////////////////////////////////////////////////////////
void FRequest::AddHeader(FAnsiStringView Key, FAnsiStringView Value)
{
	check(!Key.Equals("connection", ESearchCase::IgnoreCase));

#if UE_BUILD_DEVELOPMENT
	for (uint32 Char : Key)
	{
		if (uint32(Char - 'A') > uint32('Z' - 'A'))
		{
			check("header keys must be lowercase");
		}
	}
#endif
	RequestHeaders.Add(Key, Value);
}

////////////////////////////////////////////////////////////////////////////////
FTransactId FRequest::GetTransactId() const
{
	return Id;
}

////////////////////////////////////////////////////////////////////////////////
FTransactId FRequest::End(bool /*bKeepAlive*/)
{
	Id = GetDriver().Submit(RequestHeaders);
	return Id;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FRequest::TrySendRequest(FTlsPeer& Peer)
{
	return GetDriver().Send(Peer);
}



////////////////////////////////////////////////////////////////////////////////
class FStatusHeaders
	: public FRequest
{
public:
	using FRequest::FRequest;

	FOutcome			TryRecvResponse(FTlsPeer& Peer);
	bool				IsKeepAlive() const;
	uint32				GetStatusCode() const;
	FAnsiStringView		GetStatusMessage() const;
	int64				GetContentLength() const;
	bool				IsChunked() const;
	void				ReadHeaders(FResponse::FHeaderSink Sink) const;

private:
	TArray<FHeaderNg>	ResponseHeaders;
	int64				ContentLength = -1;
	uint32				StatusCode = 0;
};

////////////////////////////////////////////////////////////////////////////////
bool FStatusHeaders::IsKeepAlive() const
{
	return true;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FStatusHeaders::GetStatusCode() const
{
	return StatusCode;
}

////////////////////////////////////////////////////////////////////////////////
FAnsiStringView	FStatusHeaders::GetStatusMessage() const
{
	return "";
}

////////////////////////////////////////////////////////////////////////////////
int64 FStatusHeaders::GetContentLength() const
{
	return ContentLength;
}

////////////////////////////////////////////////////////////////////////////////
bool FStatusHeaders::IsChunked() const
{
	return ContentLength < 0;
}

////////////////////////////////////////////////////////////////////////////////
void FStatusHeaders::ReadHeaders(FResponse::FHeaderSink Sink) const
{
	for (const FHeaderNg& Header : ResponseHeaders)
	{
		FAnsiStringView Key = Header.Key.Get();
		FAnsiStringView Value = Header.Value.Get();
		if (!Sink(Key, Value))
		{
			break;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FStatusHeaders::TryRecvResponse(FTlsPeer& Peer)
{
	FOutcome Outcome = GetDriver().RecvResponse(Peer, &ResponseHeaders);
	if (!Outcome.IsOk())
	{
		return Outcome;
	}

	enum : uint32 {
		Flag_Code	= 1 << 0,
		Flag_Length = 1 << 1,
		Flag_All	= Flag_Code | Flag_Length,
	};
	uint32 Flags = 0;
	for (const FHeaderNg& Header : ResponseHeaders)
	{
		if (Flags == Flag_All)
		{
			break;
		}

		FAnsiStringView Key = Header.Key.Get();
		FAnsiStringView Value = Header.Value.Get();

		if (!(Flags & Flag_Code) && (Key == ":status"))
		{
			StatusCode = uint32(CrudeToInt(Value));
			Flags |= Flag_Code;
		}
		else if (!(Flags & Flag_Length) && (Key == "content-length"))
		{
			ContentLength = CrudeToInt(Value);
			Flags |= Flag_Length;
		}
	}

	if ((Flags & Flag_Code) == 0 || !IsStatusCodeOk(StatusCode))
	{
		return FOutcome::Error("Invalid status code", StatusCode);
	}

	if (IsContentless(StatusCode))
	{
		ContentLength = 0;
	}

	return FOutcome::Ok();
}



////////////////////////////////////////////////////////////////////////////////
class FBody
	: public FStatusHeaders
{
public:
	using FStatusHeaders::FStatusHeaders;

	int64			GetRemaining() const;
	FOutcome		TryRecv(FMutableMemoryView Dest, FTlsPeer& Peer);

private:
	int64			Remaining = -1;
};

////////////////////////////////////////////////////////////////////////////////
int64 FBody::GetRemaining() const
{
	return Remaining;
}

////////////////////////////////////////////////////////////////////////////////
FOutcome FBody::TryRecv(FMutableMemoryView Dest, FTlsPeer& Peer)
{
	if (!IsChunked())
	{
		if (Remaining < 0)
		{
			Remaining = GetContentLength();
		}
		Dest = Dest.Left(Remaining);
	}

	FOutcome Outcome = GetDriver().RecvBody(Peer, Dest);
	if (Outcome.IsError())
	{
		return Outcome;
	}

	Remaining -= Outcome.GetResult();
	return Outcome;
}

} // namespace DetailTwo



////////////////////////////////////////////////////////////////////////////////
class FTransactionTwo
	: public DetailTwo::FBody
{
public:
	using DetailTwo::FBody::FBody;
};



////////////////////////////////////////////////////////////////////////////////
FOutcome HandshakeTwo(FTlsPeer& Peer, void*& PeerData)
{
	using namespace DetailTwo;

	if (PeerData == nullptr)
	{
		PeerData = new FDriverNg(Peer);
	}

	auto* Driver = (FDriverNg*)PeerData;
	return Driver->Handshake(Peer);
}

////////////////////////////////////////////////////////////////////////////////
FOutcome TickTwo(FTlsPeer& Peer, void* PeerData)
{
	using namespace DetailTwo;

	auto* Driver = (FDriverNg*)PeerData;
	if (Driver == nullptr)
	{
		return FOutcome::Error("ng-driver is unexpectedly null");
	}

	return Driver->GetTransactId(Peer);
}

////////////////////////////////////////////////////////////////////////////////
bool CancelTwo(FTlsPeer& Peer, void* PeerData, FTransactId Id)
{
	using namespace DetailTwo;

	auto* Driver = (FDriverNg*)PeerData;
	if (Driver != nullptr)
	{
		return Driver->CancelTransaction(Peer, Id);
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
void SkipPendingTransactTwo(FTlsPeer&, void* PeerData)
{
	using namespace DetailTwo;

	auto* Driver = (FDriverNg*)PeerData;
	if (Driver != nullptr)
	{
		return Driver->SkipTransaction();
	}
}

////////////////////////////////////////////////////////////////////////////////
void GoAwayTwo(FTlsPeer& Peer, void* PeerData)
{
	using namespace DetailTwo;

	auto* Driver = (FDriverNg*)PeerData;
	if (Driver == nullptr)
	{
		return;
	}

	Driver->GoAway(Peer);
	delete Driver;
}

} // namespace UE::IoStore::HTTP

#else // IAS_HTTP_WITH_TWO

////////////////////////////////////////////////////////////////////////////////
namespace UE::IoStore::HTTP {

class FTransactionTwo
{
public:
	void			Begin(...)									{}
	void			AddHeader(...)								{}
	FTransactId		End(...)									{ return 0; }
	FOutcome		TrySendRequest(FTlsPeer&)					{ return FOutcome::None(); }
	FOutcome		TryRecvResponse(FTlsPeer&)					{ return FOutcome::None(); }
	bool			IsKeepAlive() const							{ return false; }
	uint32			GetStatusCode() const						{ return 0; }
	FAnsiStringView	GetStatusMessage() const					{ return {}; }
	int64			GetContentLength() const					{ return -1; }
	bool			IsChunked() const							{ return false; }
	int64			GetRemaining() const						{ return -1; }
	void			ReadHeaders(FResponse::FHeaderSink) const	{}
	FOutcome		TryRecv(FMutableMemoryView, FTlsPeer&)		{ return FOutcome::None(); }
};

FOutcome	HandshakeTwo(FTlsPeer&, void*&)			{ return FOutcome::None(); }
FOutcome	TickTwo(FTlsPeer&, void*)				{ return FOutcome::None(); }
bool		CancelTwo(FTlsPeer&, void*, FTransactId){ return false; }
void		SkipPendingTransactTwo(FTlsPeer&, void*){}
void		GoAwayTwo(FTlsPeer&, void*)				{}

} // namespace UE::IoStore::HTTP

#endif // IAS_HTTP_WITH_TWO
