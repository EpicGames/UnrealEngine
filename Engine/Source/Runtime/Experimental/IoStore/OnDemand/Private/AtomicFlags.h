// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>

namespace UE
{

template <typename FlagType>
class TAtomicFlags
{
	using UnderlyingType = __underlying_type(FlagType);

public:
	FlagType Add(FlagType FlagsToAdd, std::memory_order Order = std::memory_order_seq_cst)
	{
		return (FlagType)Flags.fetch_or((UnderlyingType)FlagsToAdd, Order);
	}

	FlagType Remove(FlagType FlagsToAdd, std::memory_order Order = std::memory_order_seq_cst)
	{
		return (FlagType)Flags.fetch_and(~((UnderlyingType)FlagsToAdd), Order);
	}

	bool HasAny(FlagType Contains, std::memory_order Order = std::memory_order_seq_cst)
	{
		return (Flags.load(Order) & (UnderlyingType)Contains) != 0;
	}

	FlagType Get(std::memory_order Order = std::memory_order_seq_cst) const
	{
		return (FlagType)Flags.load(Order);
	}

	bool TrySet(FlagType FlagsToSet, std::memory_order Order = std::memory_order_seq_cst)
	{
		UnderlyingType Expected = Flags.load(Order);
		return Flags.compare_exchange_strong(Expected, (UnderlyingType)FlagsToSet);
	}

private:
	std::atomic<UnderlyingType> Flags{0};
};

} // namespace UE
