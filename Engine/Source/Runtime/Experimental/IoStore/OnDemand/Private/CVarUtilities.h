// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/UniquePtr.h"

// The following works in the header, make sure that you include 'HAL/IConsoleManager.h' in the cpp!
struct IConsoleCommand;

struct FConsoleCommandUnregister
{
	void operator()(IConsoleCommand* ConsoleCommandPtr) const;
};

/** Experimental wrapper around a registered IConsoleCommand */
using FConsoleCommandPtr = TUniquePtr<IConsoleCommand, FConsoleCommandUnregister>;
