// Copyright Epic Games, Inc. All Rights Reserved.

#include "DebugCommands.h"

#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "IO/IoChunkId.h"
#include "IO/IoDispatcher.h"
#include "IO/IoStoreOnDemand.h"
#include "IasHostGroup.h"
#include "LatencyTesting.h"
#include "Misc/AssertionMacros.h"
#include "OnDemandIoStore.h"
#include "OnDemandContentInstaller.h"
#include "OnDemandContentInstallReplay.h"
#include "String/Numeric.h"

#if !UE_BUILD_SHIPPING

namespace UE::IoStore
{

FOnDemandDebugCommands::FOnDemandDebugCommands(FOnDemandIoStore* InOnDemandIoStore)
	: OnDemandIoStore(InOnDemandIoStore)
{
	check(InOnDemandIoStore != nullptr);

	BindConsoleCommands();
}

FOnDemandDebugCommands::~FOnDemandDebugCommands()
{
	UnbindConsoleCommands();
}

void FOnDemandDebugCommands::BindConsoleCommands()
{
#if !NO_CVARS
	DynamicConsoleCommands.Emplace(
		IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("ias.InvokeHttpFailure"),
			TEXT("Marks the current ias http connections as failed and forcing them  to try to reconnect"),
			FConsoleCommandDelegate::CreateLambda([this]()
				{
					UE_LOGF(LogIas, Display, "User invoked http error via 'ias.InvokeHttpFailure'");
					FHostGroupManager::Get().DisconnectAll();
				}),
			ECVF_Default)
	);

	DynamicConsoleCommands.Emplace(
		IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("ias.RunRequestTest"),
			TEXT("[optional Number of requests to make] Creates a number of requests for chunks that can be found in that IAS system"),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FOnDemandDebugCommands::RunRequestTest),
			ECVF_Default));

	DynamicConsoleCommands.Emplace(
		IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("iax.LatencyTests"),
			TEXT(""),
			FConsoleCommandDelegate::CreateRaw(this, &FOnDemandDebugCommands::RunLatencyTests),
			ECVF_Default));

	DynamicConsoleCommands.Emplace(
		IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("iad.RecordReplayStop"),
			TEXT(""),
			FConsoleCommandDelegate::CreateRaw(this, &FOnDemandDebugCommands::RecordReplayStop),
			ECVF_Default));
#endif // !NO_CVARS
}

void FOnDemandDebugCommands::UnbindConsoleCommands()
{
	for (IConsoleCommand* Cmd : DynamicConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(Cmd);
	}
}

void FOnDemandDebugCommands::RunRequestTest(const TArray<FString>& Args) const
{
	int32 NumToRequest = -1;
	const float CancelWaitTime = 5.0f;
	bool bCancelRequests = false;
	bool bQuitAfterTest = false;


	for (const FString& Arg : Args)
	{
		if (UE::String::IsNumeric(Arg))
		{
			if (NumToRequest != -1)
			{
				UE_LOGF(LogIas, Error, "Too many numeric args for 'ias.RunRequestTest'");
				return;
			}

			int32 ArgValue = TCString<TCHAR>::Atoi(*Arg);
			NumToRequest = ArgValue > 0 ? ArgValue : MAX_int32;
		}
		else if (Arg == "Cancel")
		{
			bCancelRequests = true;
		}
		else if (Arg == "Quit")
		{
			bQuitAfterTest = true;
		}
		else
		{
			UE_LOGF(LogIas, Error, "Unknown arg '%ls' for 'ias.RunRequestTest'", *Arg);
			return;
		}
	}

	// Set number of requests if it was not provided as an arg
	NumToRequest = NumToRequest == -1 ? 1000 : NumToRequest;

	TArray<FIoChunkId> RequestIds = OnDemandIoStore->DebugFindStreamingChunkIds(NumToRequest);

	UE_LOGF(LogIas, Display, "Running IAS Request Test with %d requests...", RequestIds.Num());

	TArray<FIoRequest> Requests;

	FIoBatch IoBatch;
	uint64 TotalDownloadSize = 0;
	for (const FIoChunkId& Id : RequestIds)
	{
		FIoRequest Request = IoBatch.Read(Id, FIoReadOptions(), IoDispatcherPriority_Medium);
		Requests.Emplace(MoveTemp(Request));

		TotalDownloadSize += OnDemandIoStore->GetStreamingChunkInfo(Id).EncodedSize();
	}

	const double StartTime = FPlatformTime::Seconds();

	IoBatch.IssueWithCallback([StartTime, TotalDownloadSize, bQuitAfterTest]()
		{
			const double TotalTime = FPlatformTime::Seconds() - StartTime;
			const double DataSizeMiB = static_cast<double>(TotalDownloadSize) / (1024.0 * 1024.0);
			const double DataRate = TotalTime > 0.0 ? DataSizeMiB / TotalTime : 0.0;

			UE_LOGF(LogIas, Display, "IAS Request Test downloaded %.2f(MiB) in %.3f(s) %.3f(MiB/s)", DataSizeMiB, TotalTime, DataRate);

			if (bQuitAfterTest)
			{
				UE_LOGF(LogIas, Display, "Quitting now that the request batch has completed");
				FPlatformMisc::RequestExitWithStatus(false, 0);
			}
		});

	if (bCancelRequests)
	{
		UE_LOGF(LogIas, Display, "Requests will be canceled in %f seconds", CancelWaitTime);

		FTSTicker::GetCoreTicker().AddTicker(TEXT("IASDebugCancel"), CancelWaitTime, [Requests = MoveTemp(Requests)](float DeltaTime) mutable
			{
				for (FIoRequest& Request : Requests)
				{
					Request.Cancel();
				}

				UE_LOGF(LogIas, Display, "Canceled all requests!");
				return false;
			});
	}

	UE_CLOGF(bQuitAfterTest, LogIas, Display, "Process will quit once the requests have been completed");
}

void FOnDemandDebugCommands::RunLatencyTests() const
{
	bool bLoggedHostGroup = false;

	FHostGroupManager::Get().ForEachHostGroup([&bLoggedHostGroup](const FIASHostGroup& HostGroup)
	{
		if (!HostGroup.IsResolved() || HostGroup.GetHostUrls().IsEmpty())
		{
			return;
		}

		bLoggedHostGroup = true;

		UE_LOGF(LogIas, Log, "--------------------------------------------------------------------------------");
		UE_LOGF(LogIas, Log, "Testing Hostgroup '%ls':", *HostGroup.GetName().ToString());

		for (const FAnsiString& Url : HostGroup.GetHostUrls())
		{
			ConnectionTest(Url, HostGroup.GetTestPath(), 0);
		}
	});

	UE_CLOGF(bLoggedHostGroup, LogIas, Log, "--------------------------------------------------------------------------------");
}

void FOnDemandDebugCommands::RecordReplayStop()
{
	if (FOnDemandContentInstallReplayRecorder* Recorder = FOnDemandContentInstallReplayRecorder::Get())
	{
		TValueOrError<FOnDemandContentInstallReplay, void> Result = Recorder->StopRecording();
		if (Result.HasError())
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "Internal Error occured while recording replay!");
			return;
		}

		FOnDemandContentInstallReplay Replay = Result.StealValue();

		// TODO: save replay to disk

		int a= 0;
		++a;
	}
}

} // namespace UE::IoStore

#endif // !UE_BUILD_SHIPPING
