// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Build.h"

#if !UE_BUILD_SHIPPING

#include "Containers/Array.h"
#include "Containers/UnrealString.h"

struct IConsoleCommand;

namespace UE::IoStore
{

class FOnDemandIoStore;

class FOnDemandDebugCommands
{
public:
	FOnDemandDebugCommands() = delete;
	FOnDemandDebugCommands(FOnDemandIoStore* OnDemandIoStore);
	~FOnDemandDebugCommands();

private:

	void BindConsoleCommands();
	void UnbindConsoleCommands();

	void RunRequestTest(const TArray<FString>& Args) const;
	void RunLatencyTests() const;

	void RecordReplayStop();

	FOnDemandIoStore* OnDemandIoStore = nullptr;

	TArray<IConsoleCommand*> DynamicConsoleCommands;
};

} // namespace UE::IoStore

#endif // !UE_BUILD_SHIPPING
