// Copyright Epic Games, Inc. All Rights Reserved.

#include "DiskCacheGovernor.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "IO/IoStoreOnDemand.h" 
#include "Misc/ScopeExit.h"
#include "Statistics.h"

float GIasCacheSuspendTimeoutFactor = 0.0f;
static FAutoConsoleVariableRef CVar_IasCacheSuspendTimeoutFactor(
	TEXT("ias.CacheSuspendTimeoutFactor"),
	GIasCacheSuspendTimeoutFactor,
	TEXT("Factor between 0 - 1 for adjusting the streaming cache suspend timeout"));

namespace UE::IoStore
{

////////////////////////////////////////////////////////////////////////////////
FDiskCacheGovernor::FDiskCacheGovernor()
{
	FConfig Cfg;
	ConfigureStreamingCache(Cfg);
}

////////////////////////////////////////////////////////////////////////////////
void FDiskCacheGovernor::ConfigureStreamingCache(const FDiskCacheGovernor::FConfig& Config)
{
	Configure(StreamingCacheState, Config);
}

////////////////////////////////////////////////////////////////////////////////
int32 FDiskCacheGovernor::TryBeginWriteToStreamingCache(uint32 DemandPercent)
{
	return TryBeginWrite(StreamingCacheState, DemandPercent);
}

////////////////////////////////////////////////////////////////////////////////
int32 FDiskCacheGovernor::EndWriteToStreamingCache(uint32 UnusedAllowance)
{
	return EndWrite(StreamingCacheState, UnusedAllowance);
}

////////////////////////////////////////////////////////////////////////////////
void FDiskCacheGovernor::Configure(FDiskCacheGovernor::FCacheState& CacheState, const FDiskCacheGovernor::FConfig& Config)
{
	const int64 CycleFreq		= int64(1.0 / FPlatformTime::GetSecondsPerCycle64());
	const FAllowance& Allowance	= Config.Allowance;
	CacheState.Demand			= Config.Demand;

	ON_SCOPE_EXIT
	{
		UE_LOGF(LogIoStoreOnDemand, Log, "Streaming cache write limits, BytesPerPeriod=%u, OpsPerPeriod=%u, Period=%us, Factor=%.2f, BytesPerOp=%u, OpInterval=%ums, MaxOps=%u",
			Allowance.Bytes, Allowance.Ops, Allowance.Seconds, GIasCacheSuspendTimeoutFactor,
			CacheState.OpAllowance, uint32(FPlatformTime::ToMilliseconds64(CacheState.OpInterval)), CacheState.MaxOpCount);
	};

	if (Config.CommitBufferSize == 0)
	{
		CacheState.OpAllowance	= Allowance.Bytes / Allowance.Ops;
		CacheState.OpInterval	= (CycleFreq * Allowance.Seconds) / Allowance.Ops;
		CacheState.MaxOpCount	= 4;
		return;
	}

	// A small commit buffer will make us op bound
	if (uint32 CommitWidth = Config.CommitBufferSize * Allowance.Ops; CommitWidth < Allowance.Bytes)
	{
		CacheState.OpAllowance	= Config.CommitBufferSize;
		CacheState.OpInterval	= (Allowance.Seconds * CycleFreq * 3) / Allowance.Ops;
		CacheState.MaxOpCount	= 1;
		return;
	}

	// Allowance bound.
	const int32 BlockCount		= FMath::Max<int32>(1, Allowance.Bytes / Config.CommitBufferSize);
	const int32 CommitOpCost	= BlockCount * 3;
	CacheState.MaxOpCount		= (Allowance.Ops - CommitOpCost) / BlockCount;
	CacheState.OpAllowance		= Config.CommitBufferSize / CacheState.MaxOpCount;
	CacheState.OpInterval		= (Allowance.Seconds * CycleFreq) / (BlockCount * (CacheState.MaxOpCount - 1));
	CacheState.State			= FCacheState::EState::Waiting;
}

////////////////////////////////////////////////////////////////////////////////
int32 FDiskCacheGovernor::TryBeginWrite(FDiskCacheGovernor::FCacheState& CacheState, uint32 DemandPercent, int64 Cycles)
{
	int64 Interval = CacheState.OpInterval;
	Interval >>= int32(DemandPercent >= CacheState.Demand.Boost);
	Interval >>= int32(DemandPercent >= CacheState.Demand.SuperBoost);
	Interval <<= int32(DemandPercent <= CacheState.Demand.Threshold);

	const int64 Delta = Cycles - CacheState.PrevCycles;
	bool bNotYet = (Delta < Interval);

	// Calculate how much time we are into the shortest poll interval
	int64 Remainder = Delta;
	Interval = GetMaxWaitCycles(CacheState);
	for (; Remainder > Interval; Remainder -= Interval);

	if (bNotYet)
	{
		// We haven't hit the current interval length but might be drawn in if
		// demand increases. So we return a wait that takes us to that.
		return int32(Remainder - Interval);
	}

	// PrevCycles is adjusted so we do not lose any left over time
	CacheState.PrevCycles = Cycles - Remainder;
	CacheState.OpCount++;

	return CacheState.OpAllowance + CacheState.RunOff;
}

////////////////////////////////////////////////////////////////////////////////
int32 FDiskCacheGovernor::TryBeginWrite(FDiskCacheGovernor::FCacheState& CacheState, uint32 DemandPercent)
{
	// A return of >=0 is the allowance of bytes that can be read. Otherwise the
	// value is number of cycles to wait until allowance may be ready.

	if (CacheState.State == FCacheState::EState::Rolling)
	{
		const int64 Cycles = FPlatformTime::Cycles64();
		return TryBeginWrite(CacheState, DemandPercent, Cycles);
	}

	// Check if the cache has been suspended
	if (int64 CyclesSuspended = CacheState.CyclesSuspended.load(std::memory_order_relaxed); CyclesSuspended > 0)
	{
		const int64 Cycles		= FPlatformTime::Cycles64();
		const int64 Delta		= Cycles - CacheState.PrevCycles;
		const int64 Remainder	= CyclesSuspended - Delta;
		const int64 Consumed	= Remainder >= 0 ? Delta : CyclesSuspended;
		CacheState.PrevCycles	= Cycles;

		check(Consumed >= 0);
		CacheState.CyclesSuspended.fetch_sub(Consumed);
		FOnDemandIoBackendStats::Get()->OnCacheSuspended(
			FPlatformTime::ToSeconds64(CacheState.CyclesSuspended.load(std::memory_order_relaxed)));

		if (Remainder > 0)
		{
			return 0 - FMath::Min(int32(Remainder), GetMaxWaitCycles(CacheState));
		}
	}

	// Wait until the demand gets above this threshold
	if (DemandPercent < CacheState.Demand.Threshold)
	{
		return 0 - GetMaxWaitCycles(CacheState);
	}

	CacheState.State		= FCacheState::EState::Rolling;
	CacheState.PrevCycles	= FPlatformTime::Cycles64();
	CacheState.OpCount		= 1;
	CacheState.RunOff		= 0;

	return CacheState.OpAllowance;
}

////////////////////////////////////////////////////////////////////////////////
int32 FDiskCacheGovernor::EndWrite(FDiskCacheGovernor::FCacheState& CacheState, uint32 UnusedAllowance)
{
	CacheState.RunOff = UnusedAllowance;

	// Suspend writing if we have exceeded the max op count or if the install cache has been flushed to disk
	if (CacheState.OpCount >= CacheState.MaxOpCount || CacheState.CyclesSuspended.load(std::memory_order_relaxed) > 0)
	{
		CacheState.State = FCacheState::EState::Waiting;
		return 0 - GetMaxWaitCycles(CacheState);
	}

	return GetMaxWaitCycles(CacheState);
}

////////////////////////////////////////////////////////////////////////////////
int32 FDiskCacheGovernor::GetMaxWaitCycles(FDiskCacheGovernor::FCacheState& CacheState)
{
	// ">> 2" so we check at four times the speed in case of a S.U.P.E.R BOOST
	return int32(CacheState.OpInterval >> 2);
}

////////////////////////////////////////////////////////////////////////////////
void FDiskCacheGovernor::OnInstallCacheFlushed(uint64 ByteCount, uint32 OpCount_Unused)
{
	const double Factor = FMath::Max(0.0, FMath::Min(1.0, double(GIasCacheSuspendTimeoutFactor)));
	if (ByteCount == 0 || Factor <= 0.0 || StreamingCacheState.OpAllowance == 0)
	{
		return;
	}

	// Suspend the streaming cache by computing the number of cycles to wait before the next write
	const double Ops			= double(ByteCount) / double(StreamingCacheState.OpAllowance);
	const double CyclesPerOp	= double(StreamingCacheState.OpInterval);

	if (int64 Cycles = int64(Ops * CyclesPerOp * Factor); Cycles > 0)
	{
		StreamingCacheState.CyclesSuspended.fetch_add(Cycles);
	}
}

} // namespace UE::IoStore
