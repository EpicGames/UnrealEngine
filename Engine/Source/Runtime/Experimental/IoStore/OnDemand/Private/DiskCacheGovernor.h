// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"
#include <atomic>

namespace UE::IoStore
{

////////////////////////////////////////////////////////////////////////////////
class FDiskCacheGovernor
{
public:
	struct FAllowance
	{
		uint32	Bytes = 16 << 20;
		uint32	Ops = 32;
		uint32	Seconds = 60;
	};

	struct FDemand
	{
		uint8	Threshold = 30;
		uint8	Boost = 60;
		uint8	SuperBoost = 87;
	};

	struct FConfig
	{
		FAllowance	Allowance;
		FDemand		Demand;
		uint32		CommitBufferSize = 0;
	};

			FDiskCacheGovernor();
			UE_NONCOPYABLE(FDiskCacheGovernor);

	void	ConfigureStreamingCache(const FConfig& Config);
	int32	TryBeginWriteToStreamingCache(uint32 DemandPercent);
	int32	EndWriteToStreamingCache(uint32 UnusedAllowance);
	void	OnInstallCacheFlushed(uint64 ByteCount, uint32 OpCount);

private:
	struct FCacheState;

	static void		Configure(FCacheState& CacheState, const FConfig& Config);
	static int32	TryBeginWrite(FCacheState& CacheState, uint32 DemandPercent, int64 Cycles);
	static int32	TryBeginWrite(FCacheState& CacheState, uint32 DemandPercent);
	static int32	EndWrite(struct FCacheState& CacheState, uint32 UnusedAllowance);
	static int32	GetMaxWaitCycles(FCacheState& CacheState);

	struct FCacheState
	{
		enum class EState : uint8
		{
			Waiting,
			Rolling,
		};

		int64				OpInterval = 0;
		int64				PrevCycles = 0;
		uint32				RunOff = 0;
		uint32				OpCount = 0;
		uint32				MaxOpCount = 0;
		uint32				OpAllowance = 0;
		std::atomic_int64_t	CyclesSuspended{0};
		FDemand				Demand;
		EState				State = EState::Waiting;
	};

	FCacheState StreamingCacheState;
};

} // namespace UE::IoStore
