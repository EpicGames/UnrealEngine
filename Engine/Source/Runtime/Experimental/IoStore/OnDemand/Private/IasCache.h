// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringView.h"
#include "IO/IoStatus.h"
#include "Memory/MemoryFwd.h"
#include "Tasks/Task.h"
#include "Templates/UniquePtr.h"

class FIoBuffer;
class FIoReadOptions;
struct FIoHash;

#define UE_ALLOW_DROP_CACHE !UE_BUILD_SHIPPING

namespace UE::IoStore
{

/** Cache for binary blobs with a 20 byte cache key. */
class IIasCache
{
public:
	virtual ~IIasCache() = default;

	/** Deletes the IAS object, dropping all data persisted to disk and releasing
	OS resources. As this also deletes the object thus any unique pointers should
	be released prior to abandonment; TUniquePtr->Release()->Abandon().*/
	virtual void Abandon() = 0;

	/** Clears out the cache and resets it back to the default state. */
	virtual void Drop() = 0;

	/** Returns whether the specified cache key is present in the cache. */
	virtual bool ContainsChunk(const FIoHash& Key) const = 0;

	/** Get the chunk associated with the specified cache key. If the data is
	 already in memory it is return in OutData. Otherwise the returned status
	 indicates if the key can be materialized or if it does not exist */
	virtual EIoErrorCode Get(const FIoHash& Key, FIoBuffer& OutData) = 0;

	/** Materializes a cached items data from disk. The data is read into Dest
	 so Dest must remain valid throughout. The result of the disk read is returned
	 in Status (same lifetime needs as Dest). DoneEvent is triggered when the read
	 succeeds, is not found, or if an IO error occurred. */
	virtual void Materialize(const FIoHash& Key, FIoBuffer& Dest, EIoErrorCode& Status, UE::Tasks::FTaskEvent DoneEvent) = 0;

	/** Cancels a previously request to Materialize(). Note that the materialize
	 already can still complete as the read operation may already be in flight. If
	 the cancel succeeds, the materialize's DoneEvent is not triggered. */
	virtual void Cancel(FIoBuffer& GivenDest) = 0;

	/** Insert a new chunk into the cache. */
	virtual FIoStatus Put(const FIoHash& Key, FIoBuffer& Data) = 0;

	/** Evict an item from the cache. For example it might be corrupt */
	virtual FIoStatus Evict(const FIoHash& Key) = 0;

	/** Update the value initially provided by FIasCacheConfig::JournalMagic */
	virtual void UpdateMagicValue(uint32 MagicValue) = 0;
};

struct FIasCacheConfig
{
	struct FRate
	{
		uint32	Allowance = 16 << 20;
		uint32	Ops = 32;
		uint32	Seconds = 60;
	};

	struct FDemand
	{
		uint8	Threshold = 30;
		uint8	Boost = 60;
		uint8	SuperBoost = 87;
	};

	FStringView Name = TEXT("ias");
	uint64		DiskQuota = 512ull << 20;
	uint32		MemoryQuota = 2 << 20;
	uint32		JournalQuota = 4 << 20; // description in JournalCache.cpp
	uint32		JournalMagic = 0; // can be used to invalidate cache
	FRate		WriteRate;
	FDemand		Demand;
	bool		DropCache = false;
};

TUniquePtr<IIasCache> MakeIasCache(const TCHAR* RootDir, const FIasCacheConfig& Config, class FDiskCacheGovernor& Governor);

} // namespace UE::IoStore
