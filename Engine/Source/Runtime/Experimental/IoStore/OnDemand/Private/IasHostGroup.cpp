// Copyright Epic Games, Inc. All Rights Reserved.

#include "IasHostGroup.h"

#include "Algo/Find.h"
#include "HAL/IConsoleManager.h"
#include "IO/IoStoreOnDemand.h"
#include "IO/OnDemandHostGroup.h"
#include "LatencyTesting.h"
#include "Logging/StructuredLog.h"
#include "Misc/ScopeRWLock.h"
#include "Statistics.h"

namespace UE::IoStore
{

int32 GIasHttpErrorSampleCount = 8;
static FAutoConsoleVariableRef CVar_IasHttpErrorSampleCount(
	TEXT("ias.HttpErrorSampleCount"),
	GIasHttpErrorSampleCount,
	TEXT("Number of samples for computing the moving average of failed HTTP requests")
);

float GIasHttpErrorHighWater = 0.5f;
static FAutoConsoleVariableRef CVar_IasHttpErrorHighWater(
	TEXT("ias.HttpErrorHighWater"),
	GIasHttpErrorHighWater,
	TEXT("High water mark when HTTP streaming will be disabled")
);

struct FBitWindow
{
	FBitWindow()
	{
		Init(GIasHttpErrorSampleCount);
	}

	~FBitWindow() = default;

	void Reset()
	{
		Init(Bits.Num());
	}

	void Add(bool bValue)
	{
		if (!Bits.IsEmpty())
		{
			const uint32 Idx = Counter++ & Mask;
			Bits[Idx] = bValue;
		}
	}

	float AvgSetBits() const
	{
		if (Bits.IsEmpty())
		{
			return 0.0f;
		}

		return float(Bits.CountSetBits()) / float(Bits.Num());
	}

private:

	void Init(uint32 Count)
	{
		if (Count > 0)
		{
			Count = FMath::RoundUpToPowerOfTwo(Count);
			Bits.Init(false, Count);
			Counter = 0;
			Mask = Count - 1;
		}
	}

	TBitArray<> Bits;
	uint32 Counter = 0;
	uint32 Mask = 0;
};

struct FIASHostGroup::FImpl
{
	FImpl() = default;
	FImpl(FName InName, FAnsiStringView InTestPath)
		: TestPath(InTestPath)
		, Name(InName)
	{
	}
	FImpl(FName InName, FOnDemandHostGroup& InHostGroup)
		: HostGroup(InHostGroup)
		, Name(InName)
		{
		}
	~FImpl() = default;

	void Reset(FOnDemandHostGroup&& InHostGroup)
	{
		HostGroup = MoveTemp(InHostGroup);
		HttpErrorHistory.Reset();
		bHttpEnabled = true;
	}

	FOnDemandHostGroup HostGroup;
	FBitWindow HttpErrorHistory;
	FAnsiString TestPath;
	FName Name;
	bool bHttpEnabled = true;
};

TIoStatusOr<FIASHostGroup> FIASHostGroup::Create(FName Name, TConstArrayView<FAnsiString> HostUrls)
{
	TIoStatusOr<FOnDemandHostGroup> Result = FOnDemandHostGroup::Create(HostUrls);

	if (!Result.IsOk())
	{
		return Result.Status();
	}

	FIASHostGroup HostGroup(Name, Result.ConsumeValueOrDie());
	HostGroup.Impl->HttpErrorHistory.Reset();

	return HostGroup;
}

TIoStatusOr<FIASHostGroup> FIASHostGroup::Create(FName Name, TConstArrayView<FString> HostUrls)
{
	TIoStatusOr<FOnDemandHostGroup> Result = FOnDemandHostGroup::Create(HostUrls);

	if (!Result.IsOk())
	{
		return Result.Status();
	}

	FIASHostGroup HostGroup(Name, Result.ConsumeValueOrDie());
	HostGroup.Impl->HttpErrorHistory.Reset();

	return HostGroup;
}

FIASHostGroup::FIASHostGroup()
	: Impl(MakeShared<FImpl>())
{
}

FIASHostGroup::FIASHostGroup(FName Name, FAnsiStringView TestPath)
	: Impl(MakeShared<FImpl>(Name, TestPath))
{
}

FIASHostGroup::FIASHostGroup(FName Name, FOnDemandHostGroup&& HostGroup)
	: Impl(MakeShared<FImpl>(Name, HostGroup))
{
}

FIASHostGroup::FIASHostGroup(FIASHostGroup::FSharedImpl&& InImpl)
	: Impl(MoveTemp(InImpl))
{
}

FName FIASHostGroup::GetName() const
{
	return Impl->Name;
}

const FAnsiString& FIASHostGroup::GetTestPath() const
{
	return Impl->TestPath;
}

bool FIASHostGroup::IsResolved() const
{
	return !Impl->HostGroup.IsEmpty();
}

bool FIASHostGroup::IsConnected() const
{
	return Impl->HostGroup.PrimaryHostIndex() != INDEX_NONE;
}

FIoStatus FIASHostGroup::Resolve(TConstArrayView<FAnsiString> HostUrls)
{
	if (IsResolved())
	{
		return FIoStatus(EIoErrorCode::InvalidCode, TEXT("Host group is already resolved"));
	}

	TIoStatusOr<FOnDemandHostGroup> Result = FOnDemandHostGroup::Create(HostUrls);

	if (!Result.IsOk())
	{
		return Result.Status();
	}

	Impl->Reset(Result.ConsumeValueOrDie());

	return FIoStatus(EIoErrorCode::Ok);
}

FIoStatus FIASHostGroup::Resolve(TConstArrayView<FString> HostUrls)
{
	if (IsResolved())
	{
		return FIoStatus(EIoErrorCode::InvalidCode, TEXT("Host group is already resolved"));
	}

	TIoStatusOr<FOnDemandHostGroup> Result = FOnDemandHostGroup::Create(HostUrls);

	if (!Result.IsOk())
	{
		return Result.Status();
	}

	Impl->Reset(Result.ConsumeValueOrDie());

	return FIoStatus(EIoErrorCode::Ok);
}


void FIASHostGroup::Connect(int32 HostIndex)
{
	Impl->bHttpEnabled = true;
	Impl->HttpErrorHistory.Reset();

	SetPrimaryHost(HostIndex);
}

bool FIASHostGroup::Disconnect()
{
	// Never disconnect when the registered test URL is empty
	if (Impl->TestPath.IsEmpty())
	{
		Impl->HttpErrorHistory.Reset();
		return false;
	}

	Impl->bHttpEnabled = false;
	Impl->HttpErrorHistory.Reset();

	SetPrimaryHost(INDEX_NONE);

	FHostGroupManager::Get().OnHostGroupDisconncted().Broadcast();

	return true;
}

FIASHostGroup::EReconnectionResult FIASHostGroup::AttemptReconnection(bool bForce, uint32 TimeoutMs, std::atomic_bool& CancellationToken)
{
	if (!IsConnected() || bForce)
	{
		if (Impl->TestPath.IsEmpty())
		{
			Connect(0);
			return EReconnectionResult::Reconnected;
		}

		UE_LOGFMT(LogIas, Log, "[{HostName}] Trying to reconnect to any available endpoint...", GetName());

		if (const int32 Idx = ConnectionTest(GetHostUrls(), Impl->TestPath, TimeoutMs, CancellationToken); Idx != INDEX_NONE)
		{
			Connect(Idx);

			UE_LOGFMT(LogIas, Log, "[{HostName}] Successfully reconnected to '{Url}'", GetName(), GetPrimaryHostUrl());
			return EReconnectionResult::Reconnected;
		}
		else
		{
			return EReconnectionResult::FailedToConnect;
		}
	}
	else if (GetPrimaryHostIndex() != 0)
	{
		UE_LOGFMT(LogIas, Log, "[{HostName}] Trying to reconnect to primary endpoint...", GetName()); 
		if (const int32 Idx = ConnectionTest(GetHostUrls().Left(1), Impl->TestPath, TimeoutMs, CancellationToken); Idx != INDEX_NONE)
		{
			SetPrimaryHost(Idx);
			UE_LOGFMT(LogIas, Log, "[{HostName}] Reconnected to primary host '{Url}'", GetName(), GetPrimaryHostUrl());
		}
		else
		{
			UE_LOGFMT(LogIas, Log, "[{HostName}] Failed to reconnect to primary host, connection remains '{Url}'", GetName(), GetPrimaryHostUrl());
		}
	}

	return EReconnectionResult::AlreadyConnected;
}

void FIASHostGroup::OnSuccessfulResponse()
{
	Impl->HttpErrorHistory.Add(false);
}

bool FIASHostGroup::OnFailedResponse()
{
	Impl->HttpErrorHistory.Add(true);

	const float Average = Impl->HttpErrorHistory.AvgSetBits();
	const bool bAboveHighWaterMark = Average > GIasHttpErrorHighWater;

	UE_LOGF(LogIas, Verbose, "[%ls] %.2f%% the last %d HTTP requests failed", *Impl->Name.ToString(),  Average * 100.0f, GIasHttpErrorSampleCount);

	if (bAboveHighWaterMark && IsConnected())
	{
		if (Disconnect())
		{
			UE_LOGF(LogIas, Warning, "[%ls] Host group disabled due to high water mark of %.2f of the last %d requests reached",
				*Impl->Name.ToString(), GIasHttpErrorHighWater * 100.0f, GIasHttpErrorSampleCount);

			return true;
		}
	}

	return false;
}

const FOnDemandHostGroup& FIASHostGroup::GetUnderlyingHostGroup() const
{
	return Impl->HostGroup;
}

void FIASHostGroup::SetPrimaryHost(int32 Index) const
{
	Impl->HostGroup.SetPrimaryHost(Index);
}

void FIASHostGroup::TrySetTestPath(FAnsiStringView TestPath)
{
	if (Impl->TestPath.IsEmpty())
	{
		Impl->TestPath = TestPath;
	}
}

FAnsiStringView FIASHostGroup::GetPrimaryHostUrl() const
{
	return Impl->HostGroup.PrimaryHost();
}

int32 FIASHostGroup::GetPrimaryHostIndex() const
{
	return Impl->HostGroup.PrimaryHostIndex();
}

TConstArrayView<FAnsiString> FIASHostGroup::GetHostUrls() const
{
	return Impl->HostGroup.Hosts();
}

FHostGroupManager& FHostGroupManager::Get()
{
	static FHostGroupManager Instance;
	return Instance;
}

TIoStatusOr<FIASHostGroup> FHostGroupManager::Register(FName Name, FAnsiStringView TestPath)
{
	FIASHostGroup HostGroup(Name, TestPath);

	{
		UE::TWriteScopeLock _(Mutex);
		HostGroups.Add(HostGroup);
	}

	return MoveTemp(HostGroup);
}

TIoStatusOr<FIASHostGroup> FHostGroupManager::Register(FName Name, TConstArrayView<FAnsiString> HostUrls)
{
	TIoStatusOr<FIASHostGroup> Result = FIASHostGroup::Create(Name, HostUrls);

	if (Result.IsOk())
	{
		UE::TWriteScopeLock _(Mutex);
		HostGroups.Add(Result.ValueOrDie());
	}

	return Result;
}

FIASHostGroup FHostGroupManager::Find(FName Name)
{
	UE::TReadScopeLock _(Mutex);

	FIASHostGroup* FoundHostGroup = Algo::FindByPredicate(HostGroups, [Name](const FIASHostGroup& HostGroup)->bool
	{
		return HostGroup.GetName() == Name;
	});

	if (FoundHostGroup != nullptr)
	{
		return *FoundHostGroup;
	}
	else
	{
		return FIASHostGroup();
	}
}

void FHostGroupManager::TrySetTestPath(FName Name, FAnsiStringView TestPath)
{
	UE::TWriteScopeLock _(Mutex);

	FIASHostGroup* Found = HostGroups.FindByPredicate([Name](const FIASHostGroup& Hg)
	{
		return Hg.GetName() == Name;
	});

	if (Found != nullptr)
	{
		Found->TrySetTestPath(TestPath);
	}
}

bool FHostGroupManager::ResolveInitialConnection(uint32 TimeoutMs, std::atomic_bool& CancellationToken)
{
	UE::TReadScopeLock _(Mutex);
	if (!HostGroups.IsEmpty())
	{
		for (FIASHostGroup& HostGroup : HostGroups)
		{
			if (HostGroup.AttemptReconnection(true, TimeoutMs, CancellationToken) == FIASHostGroup::EReconnectionResult::Reconnected)
			{
				FOnDemandIoBackendStats::Get()->OnHttpConnected();
			}
		}

		return true;
	}
	else
	{
		return false;
	}
}

void FHostGroupManager::ForEachHostGroup(TFunctionRef<void(const FIASHostGroup&)> Callback) const
{
	UE::TReadScopeLock _(Mutex);

	for (const FIASHostGroup& Host : HostGroups)
	{
		Callback(Host);
	}
}

void FHostGroupManager::Tick(uint32 TimeoutMs, std::atomic_bool& CancellationToken)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FHostGroupManager::Tick);

	UE::TReadScopeLock _(Mutex);

	for (FIASHostGroup& HostGroup : HostGroups)
	{
		if (HostGroup.AttemptReconnection(false, TimeoutMs, CancellationToken) == FIASHostGroup::EReconnectionResult::Reconnected)
		{
			FOnDemandIoBackendStats::Get()->OnHttpConnected(); // TODO: Try to avoid singleton access somehow
		}
	}
}

void FHostGroupManager::DisconnectAll()
{
	UE::TReadScopeLock _(Mutex);
	for (FIASHostGroup& HostGroup : HostGroups)
	{
		HostGroup.Disconnect();
	}
}

uint32 FHostGroupManager::GetNumDisconnctedHosts() const
{
	uint32 NumDisconnectedHosts = 0;

	UE::TReadScopeLock _(Mutex);
	for (const FIASHostGroup& HostGroup : HostGroups)
	{
		NumDisconnectedHosts += HostGroup.IsConnected() == false ? 1 : 0;
	}

	return NumDisconnectedHosts;
}

} //namespace UE::IoStore
