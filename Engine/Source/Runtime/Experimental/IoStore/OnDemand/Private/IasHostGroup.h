// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "IO/IoStatus.h"
#include "OnDemandBackendStatus.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

namespace UE::IoStore
{

class FOnDemandHostGroup;

class FIASHostGroup final
{
	struct FImpl;
	using FSharedImpl = TSharedPtr<FImpl, ESPMode::ThreadSafe>;

public:
	static TIoStatusOr<FIASHostGroup> Create(FName Name, TConstArrayView<FAnsiString> HostUrls);
	static TIoStatusOr<FIASHostGroup> Create(FName Name, TConstArrayView<FString> HostUrls);

	FIASHostGroup();
	FIASHostGroup(FName Name, FAnsiStringView TestPath);
	FIASHostGroup(FName Name, FOnDemandHostGroup&& HostGroup);
	~FIASHostGroup() = default;

	FName GetName() const;
	const FAnsiString& GetTestPath() const;

	bool IsResolved() const;
	bool IsConnected() const;

	FIoStatus Resolve(TConstArrayView<FAnsiString> HostUrls);
	FIoStatus Resolve(TConstArrayView<FString> HostUrls);
	
	void Connect(int32 HostIndex);
	bool Disconnect();

	enum class EReconnectionResult : uint8
	{
		AlreadyConnected = 0,
		Reconnected,
		FailedToConnect,
	};

	EReconnectionResult AttemptReconnection(bool bForce, uint32 TimeoutMs, std::atomic_bool& CancellationToken);

	void OnSuccessfulResponse();
	bool OnFailedResponse();

	FAnsiStringView GetPrimaryHostUrl() const;
	int32 GetPrimaryHostIndex() const;

	const FOnDemandHostGroup& GetUnderlyingHostGroup() const;

	TConstArrayView<FAnsiString> GetHostUrls() const;
private:
	friend class FHostGroupManager;
	FIASHostGroup(FSharedImpl&& Impl);

	void SetPrimaryHost(int32 Index) const;
	void TrySetTestPath(FAnsiStringView TestPath);

	FSharedImpl Impl;
};

class FHostGroupManager final
{
	FHostGroupManager() = default;
	~FHostGroupManager() = default;

public:
	DECLARE_TS_MULTICAST_DELEGATE(FOnHostGroupDisconnectedDelegate);

	static FHostGroupManager& Get(); 

	// TODO: Note that currently we have no way to remove FIASHostGroups

	TIoStatusOr<FIASHostGroup> Register(FName Name, FAnsiStringView TestPath); // <- Unresolved
	TIoStatusOr<FIASHostGroup> Register(FName Name, TConstArrayView<FAnsiString> HostUrls); // <- Resolved

	FIASHostGroup Find(FName Name);
	void TrySetTestPath(FName Name, FAnsiStringView TestPath);

	/** Returns true if we have tried to resolve at least one hostgroup and false if there are currently no hostgroups */
	bool ResolveInitialConnection(uint32 TimeoutMs, std::atomic_bool& CancellationToken);

	void ForEachHostGroup(TFunctionRef<void (const FIASHostGroup&)> Callback) const;

	void Tick(uint32 TimeoutMs, std::atomic_bool& CancellationToken);

	void DisconnectAll(); 
	uint32 GetNumDisconnctedHosts() const;

	FOnHostGroupDisconnectedDelegate& OnHostGroupDisconncted()
	{
		return HostGroupDisconnectedDelegate;
	}

private:

	TArray<FIASHostGroup> HostGroups;

	FOnHostGroupDisconnectedDelegate HostGroupDisconnectedDelegate;

	mutable FRWLock Mutex;
};

} //namespace UE::IoStore
