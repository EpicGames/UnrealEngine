// Copyright Epic Games, Inc. All Rights Reserved.

#include "IasCache.h"
#include "DiskCacheGovernor.h"
#include "IO/IoStoreOnDemand.h"
#include "Statistics.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/FileManager.h"
#include "HAL/LowLevelMemTracker.h" 
#include "HAL/PlatformFile.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Hash/CityHash.h"
#include "IO/IoBuffer.h"
#include "IO/IoHash.h"
#include "IO/IoStatus.h"
#include "Math/UnrealMath.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/StringBuilder.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Tasks/Pipe.h"
#include "Tasks/Task.h"
#include "Templates/UniquePtr.h"

#include <atomic>

/*
DiskQuota is the maximum bytes on disk the cache will use. This includes the
JournalQuota (available_data_bytes = diskq - jrnq). JournalQuota should be
chosen such that it holds at least one wrap such that overwrites can be
detected. For example, assuming an average size of cache items of 2KiB (very
conservative), a 512MiB cache can hold 256K items. Journal entries are 16
bytes, so a 256K * 16 is sufficient; 256K * 16 = 4MiB.

DemandThreshold, DemandBoost, and DemandSuperBoost slow down or speed up the
writing of data (and journal) to disk depending on how full the memory cache
is. They are expressed as percentages.
*/

namespace UE::IoStore::JournaledCache
{

// {{{1 misc ...................................................................

////////////////////////////////////////////////////////////////////////////////
TRACE_DECLARE_INT_COUNTER(IasMemDemand, TEXT("Ias/CacheMemDemand"));
TRACE_DECLARE_INT_COUNTER(IasAllowance, TEXT("Ias/CacheAllowance"));
TRACE_DECLARE_INT_COUNTER(IasOpCount,   TEXT("Ias/CacheOpCount"));
TRACE_DECLARE_INT_COUNTER(IasReadCursor,TEXT("Ias/CacheReadCursor"));

////////////////////////////////////////////////////////////////////////////////
int32 CloneCachedIOBufferData = 0;
static FAutoConsoleVariableRef CloneCachedIOBufferDataCVar(
	TEXT("ias.CloneCachedIOBufferData"),
	CloneCachedIOBufferData,
	TEXT("Clone cached io buffer data, speculative fix for a read after free happening in FIoChunkEncoding::Decode. Disabled by default"),
	ECVF_Default
);

////////////////////////////////////////////////////////////////////////////////
static int32 LoadCache(class FDiskCache&);

////////////////////////////////////////////////////////////////////////////////
enum class EAilments
{
	NoJrnHandle		= 1 << 0,
	NoDataHandle	= 1 << 1,
};

////////////////////////////////////////////////////////////////////////////////
#if !defined(IAS_HAS_WRITE_COMMIT_THRESHOLD)
#	define IAS_HAS_WRITE_COMMIT_THRESHOLD 0
#endif

#if IAS_HAS_WRITE_COMMIT_THRESHOLD
	int32 GetWriteCommitThreshold(FStringView);
#else
	static int32 GetWriteCommitThreshold(FStringView)
	{
		return 0;
	}
#endif

////////////////////////////////////////////////////////////////////////////////
struct FDebugCacheEntry
{
	using Callback = void(void* Param, const FDebugCacheEntry&);
	uint64 Key;
	uint32 Size;
	uint32 IsMemCache : 1;
	uint32 _Unused : 31;
};

////////////////////////////////////////////////////////////////////////////////
static FStringView GetCacheFsDir()
{ 
	if (GIsEditor)
	{
		return FStringView(TEXT("iaseditor"));
	}
	return FStringView(TEXT("ias"));
}
static constexpr FStringView GetCacheFsSuffix()  { return FStringView(TEXT(".cache.0")); }
static constexpr FStringView GetCacheJrnSuffix() { return FStringView(TEXT(".jrn")); }



// {{{1 mem-cache ..............................................................

////////////////////////////////////////////////////////////////////////////////
class FMemCache
{
public:
	struct FItem
	{
		uint64		Key;
		FIoBuffer	Data;
	};

	using ItemArray = TArray<FItem>;
	using PeelItems = ItemArray;

					FMemCache(uint32 InMaxSize=64 << 10);
	uint32			GetDemand() const;
	uint32			GetCount() const	{ return Items.Num(); }
	uint32			GetUsed() const		{ return UsedSize; }
	uint32			GetMax() const		{ return MaxSize; }
	const FIoBuffer*Get(uint64 Key) const;
	bool			Put(uint64 Key, FIoBuffer&& Data);
	bool			Evict(uint64 Key, bool bAlways);
	void			Drop();
	int32			Peel(int32 PeelThreshold, PeelItems& Out);
	uint32			DebugVisit(void* Param, FDebugCacheEntry::Callback* Callback);

private:
	template <typename Lambda>
	int32			DropImpl(uint32 Size, Lambda&& Callback);
	int32			Drop(uint32 Size);
	uint32			MaxSize;
	uint32			UsedSize = 0;
	ItemArray		Items;
};

////////////////////////////////////////////////////////////////////////////////
FMemCache::FMemCache(uint32 InMaxSize)
: MaxSize(InMaxSize)
{
}

////////////////////////////////////////////////////////////////////////////////
uint32 FMemCache::GetDemand() const
{
	return MaxSize > 0 ? (GetUsed() * 100) / MaxSize : 0;
}

////////////////////////////////////////////////////////////////////////////////
const FIoBuffer* FMemCache::Get(uint64 Key) const
{
	for (auto& Item : Items)
	{
		if (Item.Key == Key)
		{
			return (Item.Data.GetSize() > 0) ? &(Item.Data) : nullptr;
		}
	}

	return nullptr;
}

////////////////////////////////////////////////////////////////////////////////
bool FMemCache::Put(uint64 Key, FIoBuffer&& Data)
{
	if (Get(Key) != nullptr)
	{
		uint32 DataSize = uint32(Data.GetSize());
		FOnDemandIoBackendStats::Get()->OnCachePutExisting(DataSize);
		return true;
	}

	uint32 Size = uint32(Data.GetSize());
	if (Size == 0 || MaxSize < Size)
	{
		return false;
	}

	if (UsedSize + Size > MaxSize)
	{
		int32 DroppedSize = Drop(Size);
		FOnDemandIoBackendStats::Get()->OnCachePutReject(DroppedSize);
	}

	Items.Add({ Key, MoveTemp(Data) });
	UsedSize += Size;

	FOnDemandIoBackendStats::Get()->OnCachePut();
	FOnDemandIoBackendStats::Get()->OnCachePendingBytes(UsedSize);
	return true;
}

////////////////////////////////////////////////////////////////////////////////
bool FMemCache::Evict(uint64 Key, bool bAlways)
{
	bool bEviction = false;
	for (uint32 i = 0, n = Items.Num(); i < n; ++i)
	{
		FItem& Item = Items[i];
		if (Item.Key != Key)
		{
			continue;
		}

		UsedSize -= int32(Item.Data.GetSize());
		bEviction = true;

		Item = MoveTemp(Items[n - 1]);
		Items.Pop();
		break;
	}

	// Even if there was no item cached in memory we may want to flow a dataless
	// item through such that a record is made of the eviction.
	if (bAlways)
	{
		Items.Add({ Key });
	}

	return bEviction;
}

////////////////////////////////////////////////////////////////////////////////
void FMemCache::Drop()
{
	UE_LOGF(LogIas, Display, "Dropping FMemCache");

	Items.Empty();
	UsedSize = 0;
}

////////////////////////////////////////////////////////////////////////////////
int32 FMemCache::Peel(int32 PeelThreshold, PeelItems& Out)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IasCache::Peel);

	// Items are sorted so that those we want to peel first are at the end. At
	// the time of writing, largest items peel first. The exception is evicted
	// items which have no data. Said evictions peeling priority.
	Algo::Sort(Items, [] (const FItem& Lhs, const FItem& Rhs)
	{
		uint32 OpL = uint32(Lhs.Data.GetSize());
		uint32 OpR = uint32(Rhs.Data.GetSize());
		if (!OpL || !OpR)
		{
			return OpL > OpR;
		}
		return OpL < OpR;
	});

	// Add large items
	int32 NumItems = Items.Num();
	int32 DropSize = 0;
	for (int32 i = NumItems - 1; i >= 0 && DropSize < PeelThreshold; --i)
	{
		FItem& Item = Items[i];

		DropSize += int32(Item.Data.GetSize());

		Out.Add(MoveTemp(Item));
		--NumItems;
	}

	Items.SetNum(NumItems);

	UsedSize -= DropSize;
	FOnDemandIoBackendStats::Get()->OnCachePendingBytes(UsedSize);

	return DropSize;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FMemCache::DebugVisit(void* Param, FDebugCacheEntry::Callback* Callback)
{
	FDebugCacheEntry Out = {};
	Out.IsMemCache = 1;
	for (auto& Item : Items)
	{
		Out.Key = Item.Key;
		Out.Size = uint32(Item.Data.GetSize());
		Callback(Param, Out);
	}
	return Items.Num();
}

////////////////////////////////////////////////////////////////////////////////
template <typename Lambda>
int32 FMemCache::DropImpl(uint32 Size, Lambda&& Callback)
{
	int32 DropSize = 0;
	int32 TargetSize = FMath::Min<int32>(Size, UsedSize);
	for (int32 n = Items.Num(); --n >= 0;)
	{
		if (DropSize >= TargetSize)
		{
			break;
		}

		uint32 Index = n ? (Size * 0x0'a9e0'493) % n : 0;

		Size = uint32(Items[Index].Data.GetSize());
		DropSize += Size;

		Callback(MoveTemp(Items[Index]));

		Items[Index] = MoveTemp(Items.Last());
		Items.Pop();
	}

	UsedSize -= DropSize;
	FOnDemandIoBackendStats::Get()->OnCachePendingBytes(UsedSize);

	return DropSize;
}

////////////////////////////////////////////////////////////////////////////////
int32 FMemCache::Drop(uint32 Size)
{
	return DropImpl(Size, [] (FItem&&) {});
}

// {{{1 phrase .................................................................

////////////////////////////////////////////////////////////////////////////////
static const uint32 SIZE_BITS = 25;
static const uint32 MARKER_MAX = 0x3fffffff;
static const uint32	HASH_CHECKSUM_SIZE = 64;

using EntryHash = uint32;

struct FDataEntry
{
	uint64		Key;
	uint64		Offset : 23;
	uint64		Size : SIZE_BITS;
	uint64		EntryCount : 16;
};

struct FPhraseDesc
{
	uint32		Magic;
	EntryHash	Hash;
	uint64		Marker : 30; // Safely store ~1G ops (1<<30)/2 operations (with wrap)
	uint64		DataCursor : 34; // Allows for cache size up to 31 Gb
	FDataEntry	Entries[];
};

static_assert(sizeof(FPhraseDesc) == 16);
static_assert(sizeof(FPhraseDesc) == sizeof(FDataEntry));

////////////////////////////////////////////////////////////////////////////////
struct FDiskPhrase
{
						FDiskPhrase(TArray<FDataEntry>& InEntries, int32 InMaxEntries, uint32 DataSize);
						FDiskPhrase(FDiskPhrase&&) = default;
	bool				Add(uint64 Key, FIoBuffer&& Data, uint32 PartialBias=0);
	void				Drop()					{ return Entries.SetNumUninitialized(Index); }
	const FDataEntry*	GetEntries() const		{ return Entries.GetData() + Index; }
	int32				GetEntryCount() const	{ return Entries.Num() - Index; }
	uint8*				GetPhraseData() const	{ return Buffer.Get(); }
	uint32				GetDataSize() const		{ return Cursor; }
	int32				GetRemainingEntries() const { return MaxEntries; }

private:
	TUniquePtr<uint8[]>	Buffer;
	TArray<FDataEntry>&	Entries;
	uint32				Cursor = 0;
	uint32				Index;
	int32				MaxEntries;

private:
						FDiskPhrase(const FDiskPhrase&) = delete;
	FDiskPhrase&		operator = (const FDiskPhrase&) = delete;
	FDiskPhrase&		operator = (FDiskPhrase&&) = delete;
};

////////////////////////////////////////////////////////////////////////////////
FDiskPhrase::FDiskPhrase(TArray<FDataEntry>& InEntries, int32 InMaxEntries, uint32 DataSize)
: Entries(InEntries)
, Index(Entries.Num())
, MaxEntries(InMaxEntries)
{
	Buffer = TUniquePtr<uint8[]>(new uint8[DataSize]);
}

////////////////////////////////////////////////////////////////////////////////
bool FDiskPhrase::Add(uint64 Key, FIoBuffer&& Data, uint32 PartialBias)
{
	check(MaxEntries > 0);
	const uint32 DataSize = uint32(Data.GetSize());
	check(DataSize < (1 << SIZE_BITS));
	Entries.Add(FDataEntry{
		.Key = Key,
		.Offset = Cursor + PartialBias,
		.Size = DataSize + PartialBias,
		.EntryCount = 0,
	});
	--MaxEntries;

	std::memcpy(Buffer.Get() + Cursor, Data.GetData(), DataSize);
	Cursor += DataSize;

	return MaxEntries > 0;
}



// {{{1 journal ................................................................

////////////////////////////////////////////////////////////////////////////////
class FDiskJournal
{
public:
							FDiskJournal(FStringView InRootPath, uint32 InMaxSize, uint32 MagicSeed = 0);
	uint32					GetAilments() const;
	void					Drop();
	int32					Flush();
	FDiskPhrase				OpenPhrase(uint32 DataSize);
	void					ClosePhrase(FDiskPhrase&& Phrase, uint64 DataCursor);
	uint32					GetMaxSize() const	{ return MaxSize; }
	uint32					GetCursor() const	{ return Cursor; }
	uint32					GetMarker() const	{ return Marker; }

	bool					DoesMagicSeedMatch(uint32 MagicSeed) const;

private:
	friend int32			LoadCache(FDiskCache&);
	uint32					GetMagic() const { return Magic; }
	void					GetPath(TStringBuilder<64>& Out);
	void					OpenJrnFile();

	static uint32			ProcessMagicSeed(uint32 MagicSeed);
	static uint32			HashBytes(const uint8* Data, uint32 Size, uint32 Seed);

	TArray<FDataEntry>		Entries;
	FStringView				RootPath;
	TUniquePtr<IFileHandle>	JrnHandle;
	uint32					Marker = 0;
	uint32					Cursor = 0;
	uint32					MaxSize;
	uint32					Magic;
};

////////////////////////////////////////////////////////////////////////////////
FDiskJournal::FDiskJournal(FStringView InRootPath, uint32 InMaxSize, uint32 MagicSeed)
: RootPath(InRootPath)
, MaxSize(InMaxSize)
{
	// Align down to keep to some assumptions
	MaxSize &= ~(sizeof(FDataEntry) - 1);
	Magic = ProcessMagicSeed(MagicSeed);

	OpenJrnFile();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FDiskJournal::GetAilments() const
{
	uint32 Ret = 0;
	if (!JrnHandle.IsValid())
	{
		Ret |= uint32(EAilments::NoJrnHandle);
	}
	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
void FDiskJournal::Drop()
{
	JrnHandle.Reset();

	TStringBuilder<64> JrnPath;
	GetPath(JrnPath);

	IPlatformFile& Ipf = IPlatformFile::GetPlatformPhysical();
	if (!Ipf.DeleteFile(*JrnPath))
	{
		const uint32 SystemError = FPlatformMisc::GetLastError();
		TCHAR SystemErrorMsg[MAX_SPRINTF] = { 0 };
		FPlatformMisc::GetSystemErrorMessage(SystemErrorMsg, UE_ARRAY_COUNT(SystemErrorMsg), SystemError);

		UE_LOGF(LogIas, Error, "Failed to delete '%ls' for FDiskJournal ('%d' - '%ls'", *JrnPath, SystemError, SystemErrorMsg);
	}

	Cursor = 0;
	Entries.Reset();

	OpenJrnFile();
}

////////////////////////////////////////////////////////////////////////////////
void FDiskJournal::OpenJrnFile()
{
	TStringBuilder<64> JrnPath;
	GetPath(JrnPath);

	IPlatformFile& Ipf = IPlatformFile::GetPlatformPhysical();

	FFileOpenResult Result = Ipf.OpenWrite(*JrnPath, IPlatformFile::EOpenWriteFlags::Append | IPlatformFile::EOpenWriteFlags::AllowRead);
	if (Result.IsValid())
	{
		JrnHandle = Result.StealValue();
	}
	else
	{
		UE_LOGF(LogIas, Error, "Failed to open '%ls' for FDiskJournal (%ls)", *JrnPath, *Result.GetError().GetMessage());
	}
}

////////////////////////////////////////////////////////////////////////////////
uint32 FDiskJournal::ProcessMagicSeed(uint32 MagicSeed)
{
	// High word is constant to identify the magic value, lower word mixes in the
	// seed to afford some control over validation. The value "3" is for backwards
	// compatibilty; it comes from the hardcoded magic value already in use.
	uint32 Magic  = 0x0493'0003;
	Magic += ((Magic + MagicSeed) & 0xffff);

	return Magic;
}
////////////////////////////////////////////////////////////////////////////////

uint32 FDiskJournal::HashBytes(const uint8* Data, uint32 Size, uint32 Seed)
{
	return Seed + CityHash32((const char*)Data, Size);
}

////////////////////////////////////////////////////////////////////////////////
void FDiskJournal::GetPath(TStringBuilder<64>& Out)
{
	Out << RootPath;
	Out << GetCacheJrnSuffix();
}

////////////////////////////////////////////////////////////////////////////////
FDiskPhrase FDiskJournal::OpenPhrase(uint32 DataSize)
{
	// We may have ended up exactly on the end of the journal file, immediately
	// wrap in that case.
	if (Cursor >= MaxSize)
	{
		Cursor = 0;
	}

	check((Cursor & (sizeof(FDataEntry) - 1)) == 0);

	Entries.Add(FDataEntry{});
	int32 MaxEntries = int32((MaxSize - Cursor) / sizeof(FDataEntry)) - Entries.Num();

	FDiskPhrase Ret(Entries, FMath::Min(MaxEntries, int32(UINT16_MAX)), DataSize);

	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
void FDiskJournal::ClosePhrase(FDiskPhrase&& Phrase, uint64 DataCursor)
{
	uint32 EntryCount = Phrase.GetEntryCount();
	if (EntryCount == 0)
	{
		Entries.Pop();
		return;
	}
	
	// Since we minimally need two entries to write a complete phrase, if there
	// is only room for one more entry, pad the phrase with an identity entry.
	const uint32 Size = Entries.Num() * sizeof(Entries[0]);
	const int32 PhraseEnd = Cursor + Size;
	if (MaxSize - PhraseEnd == sizeof(FDataEntry))
	{
		Phrase.Add(0, FIoBuffer());
		++EntryCount;
	}

	// Write the number of data entries in the first and last entry
	check(EntryCount <= UINT16_MAX);
	FDataEntry& FirstEntry = (FDataEntry&)(Phrase.GetEntries()[0]);
	FDataEntry& LastEntry = (FDataEntry&) Entries.Last();
	FirstEntry.EntryCount = uint16(EntryCount);
	LastEntry.EntryCount = uint16(EntryCount);

	auto& Desc = (FPhraseDesc&)(Phrase.GetEntries()[-1]);
	Desc.Magic = GetMagic();
	Desc.Marker = Marker;
	Desc.DataCursor = DataCursor;

	const uint32 HashSize = FMath::Min(HASH_CHECKSUM_SIZE, Phrase.GetDataSize());
	Desc.Hash = HashBytes(Phrase.GetPhraseData(), HashSize, Desc.Marker);

	// Increment and wrap marker
	if (++Marker > MARKER_MAX)
	{
		Marker = 0;
	}
}

////////////////////////////////////////////////////////////////////////////////
bool FDiskJournal::DoesMagicSeedMatch(uint32 MagicSeed) const
{
	return Magic == ProcessMagicSeed(MagicSeed);
}

////////////////////////////////////////////////////////////////////////////////
int32 FDiskJournal::Flush()
{
	if (Entries.IsEmpty())
	{
		return 0;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(IasCache::Flush_DiskCache);

	uint32 Size = Entries.Num() * sizeof(Entries[0]);

	check(Cursor + Size <= MaxSize);

	if (JrnHandle.IsValid())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IasCache::JournalWrite);

		JrnHandle->Seek(Cursor);
		JrnHandle->Write((uint8*)(Entries.GetData()), Size);
		JrnHandle.Reset();
		Cursor += Size;
	}

	Entries.Reset();
	OpenJrnFile();

	return Size;
}



// {{{1 disk-cache .............................................................

////////////////////////////////////////////////////////////////////////////////
class FDiskCache
{
public:
							FDiskCache(FString&& Path, uint64 InMaxDataSize, uint32 InJournalSize, uint32 InJournalMagic);
	uint32					GetAilments() const;
	FDiskPhrase				OpenPhrase(uint32 DataSize);
	void					ClosePhrase(FDiskPhrase&& Phrase);
	bool					Has(uint64 Key) const;
	bool					Evict(uint64 Key);
	EIoErrorCode			Materialize(uint64 Key, FIoBuffer& Out, uint32 Offset=0) const;
	int32					Flush();
	void					Drop();
	uint64					RemainingUntilWrap();
	uint32					DebugVisit(void* Param, FDebugCacheEntry::Callback* Callback);

	static void				UpdateMagicValue(FDiskCache* Cache, uint32 MagicValue);

private:
	struct FMapEntry
	{
		uint64				DataCursor : 39;
		uint64				Size : SIZE_BITS;
	};
	static_assert(sizeof(FMapEntry) == sizeof(uint64));

	using					FDataMap = TMap<uint64, FMapEntry>;

	friend int32			LoadCache(FDiskCache&);
	void					OpenDataFile();
	void					Wrap();
	void					Spam();
	int64					Insert(uint64 DataBase, const FDataEntry& Entry);
	uint64					Insert(uint64 DataBase, const FDataEntry* Entries, uint32 EntryCount);
	void					Prune(uint64 DataBase, uint32 Size);
	mutable FRWLock			Lock;
	FString					BinPath;
	FDataMap				DataMap;
	uint64					MappedBytes = 0;
	uint64					MaxDataSize;
	uint64					DataCursor = 0;
	TUniquePtr<IFileHandle>	DataHandle;
	uint32					OverRemoval = 0;
	FDiskJournal			Journal;
};

////////////////////////////////////////////////////////////////////////////////
FDiskCache::FDiskCache(FString&& Path, uint64 InMaxDataSize, uint32 InJournalSize, uint32 InJournalMagic)
: BinPath(MoveTemp(Path))
, MaxDataSize(InMaxDataSize)
, Journal(BinPath, InJournalSize, InJournalMagic)
{
	check(MaxDataSize >= HASH_CHECKSUM_SIZE);

	// Remove the space reserved for the journal and align down to the nearest MiB  keep to some assumptions
	MaxDataSize = (MaxDataSize - Journal.GetMaxSize()) & ~((1ull << 20) - 1);

	OpenDataFile();
	
	FOnDemandIoBackendStats::Get()->OnCacheSetMaxBytes(MaxDataSize);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FDiskCache::GetAilments() const
{
	uint32 Ret = Journal.GetAilments();
	if (!DataHandle.IsValid())
	{
		Ret |= uint32(EAilments::NoDataHandle);
	}
	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
void FDiskCache::OpenDataFile()
{
	IPlatformFile& Ipf = IPlatformFile::GetPlatformPhysical();

	IFileHandle* Handle(Ipf.OpenWrite(*BinPath, true, true));
	DataHandle.Reset(Handle);
}

////////////////////////////////////////////////////////////////////////////////
FDiskPhrase FDiskCache::OpenPhrase(uint32 DataSize)
{
	return Journal.OpenPhrase(DataSize);
}

////////////////////////////////////////////////////////////////////////////////
void FDiskCache::Wrap()
{
	check(DataCursor >= MaxDataSize);
	OverRemoval = 0;
	DataCursor = 0;
}

////////////////////////////////////////////////////////////////////////////////
void FDiskCache::ClosePhrase(FDiskPhrase&& Phrase)
{
	int32 EntryCount = Phrase.GetEntryCount();
	if (Phrase.GetDataSize() == 0)
	{
		Journal.ClosePhrase(MoveTemp(Phrase), 0);
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(IasCache::ClosePhrase);

	if (!DataHandle.IsValid())
	{
		Phrase.Drop();
		Journal.ClosePhrase(MoveTemp(Phrase), 0);
		return;
	}

	uint32 WriteSize = Phrase.GetDataSize();
	check(DataCursor + WriteSize <= MaxDataSize);

	bool bWriteOk;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IasCache::DataWrite);
		const uint8* Buffer = Phrase.GetPhraseData();
		DataHandle->Seek(DataCursor);
		bWriteOk = DataHandle->Write(Buffer, WriteSize);
	}

	if (!bWriteOk)
	{
		Phrase.Drop();
		Journal.ClosePhrase(MoveTemp(Phrase), 0);
		return;
	}

	FOnDemandIoBackendStats::Get()->OnCacheWriteBytes(WriteSize);

	{
		FWriteScopeLock _(Lock);
		Prune(DataCursor, WriteSize);
		Insert(DataCursor, Phrase.GetEntries(), EntryCount);
	}

	Journal.ClosePhrase(MoveTemp(Phrase), DataCursor);
	DataCursor += WriteSize;
}

////////////////////////////////////////////////////////////////////////////////
bool FDiskCache::Has(uint64 Key) const
{
	FReadScopeLock _(Lock);
	return (DataMap.Find(Key) != nullptr);
}

////////////////////////////////////////////////////////////////////////////////
bool FDiskCache::Evict(uint64 Key)
{
	FWriteScopeLock _(Lock);

	FSetElementId Id = DataMap.FindId(Key);
	if (!Id.IsValidId())
	{
		return false;
	}

	const TPair<uint64, FMapEntry>& Element = DataMap.Get(Id);
	MappedBytes -= Element.Value.Size;

	FOnDemandIoBackendStats::Get()->OnCacheEvictedBytes(Element.Value.Size);

	DataMap.Remove(Id);

	FOnDemandIoBackendStats::Get()->OnCachePersistedBytes(MappedBytes, DataMap.Num());

	return true;
}

////////////////////////////////////////////////////////////////////////////////
EIoErrorCode FDiskCache::Materialize(uint64 Key, FIoBuffer& Out, uint32 Offset) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IasCache::Materialize_Disk);

	if (!DataHandle.IsValid())
	{
		return EIoErrorCode::FileNotOpen;
	}

	uint32 ReadSize;
	uint64 EntryDataCursor;
	{
		FReadScopeLock _(Lock);

		const FMapEntry* Entry = DataMap.Find(Key);
		if (Entry == nullptr)
		{
			return EIoErrorCode::NotFound;
		}

		if (Offset > uint32(Entry->Size))
		{
			return EIoErrorCode::InvalidParameter;
		}

		ReadSize = uint32(Entry->Size) - Offset;
		EntryDataCursor = Entry->DataCursor;
	}

	if (Out.GetData() == nullptr)
	{
		Out = FIoBuffer(ReadSize);
	}

	ReadSize = FMath::Min<uint32>(uint32(Out.GetSize()), ReadSize);

	if (EntryDataCursor + Offset + ReadSize > uint64(DataHandle->Size()))
	{
		return EIoErrorCode::ReadError;
	}

	TRACE_COUNTER_SET(IasReadCursor, EntryDataCursor + Offset);

	DataHandle->Seek(EntryDataCursor + Offset);
	bool bOk = DataHandle->Read(Out.GetData(), ReadSize);
	return bOk ? EIoErrorCode::Ok : EIoErrorCode::ReadError;
}

////////////////////////////////////////////////////////////////////////////////
int64 FDiskCache::Insert(uint64 DataBase, const FDataEntry& Entry)
{
	if (Entry.Key == 0)
	{
		return 0;
	}

	if (Entry.Size == 0)
	{
		int64 Ret = 0;
		if (const FMapEntry* Existing = DataMap.Find(Entry.Key); Existing != nullptr)
		{
			Ret = -int64(Existing->Size);
		}

		DataMap.Remove(Entry.Key);
		return Ret;
	}

	FMapEntry Value;
	Value.DataCursor = DataBase + Entry.Offset;
	check(Value.DataCursor < MaxDataSize);
	Value.Size = Entry.Size;
	DataMap.Add(Entry.Key, Value);
	return Entry.Size;
}

////////////////////////////////////////////////////////////////////////////////
uint64 FDiskCache::Insert(uint64 DataBase, const FDataEntry* Entries, uint32 EntryCount)
{
	uint32 PartialBias = 0;
	if (EntryCount > 0)
	{
		PartialBias = Entries[0].Offset;
	}

	int64 TotalSize = 0;
	for (uint32 i = 0; i < EntryCount; ++i)
	{
		check(int64(DataBase - PartialBias) >= 0);

		FDataEntry Entry = Entries[i];
		Entry.Offset -= PartialBias;

		TotalSize += Insert(DataBase - PartialBias, Entry);
		PartialBias = 0;
	}

	MappedBytes += TotalSize;

	// DataMap is updated in the ::Insert call above
	FOnDemandIoBackendStats::Get()->OnCachePersistedBytes(MappedBytes, DataMap.Num());

	return MappedBytes;
}

////////////////////////////////////////////////////////////////////////////////
void FDiskCache::Prune(uint64 DataBase, uint32 Size)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IasCache::Prune);

	int32 BytesRemoved = OverRemoval;
	if (BytesRemoved >= int32(Size))
	{
		OverRemoval -= Size;
		return;
	}
	OverRemoval = 0;

	int64 A[] = { int64(DataBase), int64(DataBase + Size) };

	int64 Overage = 0;
	for (auto Iter = DataMap.CreateIterator(); Iter; ++Iter)
	{
		const FMapEntry& Candidate = Iter.Value();

		int64 B[] = {
			int64(Candidate.DataCursor),
			int64(Candidate.DataCursor + Candidate.Size)
		};

		int32 Outside = (B[0] >= A[1]) | (B[1] <= A[0]);
		if (Outside)
		{
			continue;
		}

		const int64 CandidateSize = Candidate.Size;
		Iter.RemoveCurrent();

		MappedBytes -= CandidateSize;
		FOnDemandIoBackendStats::Get()->OnCacheEvictedBytes(CandidateSize);

		Overage = FMath::Max(Overage, B[1] - A[1]);
		BytesRemoved += int32(B[1] - B[0]);
		if (BytesRemoved - Overage >= int32(Size))
		{
			OverRemoval = int32(Overage);
			break;
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
int32 FDiskCache::Flush()
{
	DataHandle.Reset();
	int32 Ret = Journal.Flush();
	Spam();
	OpenDataFile();
	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
void FDiskCache::Drop()
{
	UE_LOGF(LogIas, Display, "Dropping FDiskCache");

	DataHandle.Reset();

	IPlatformFile& Ipf = IPlatformFile::GetPlatformPhysical();
	Ipf.DeleteFile(*BinPath);

	Journal.Drop();

	DataCursor = 0;
	OverRemoval = 0;
	MappedBytes = 0;
	DataMap.Reset();

	FOnDemandIoBackendStats::Get()->OnCachePersistedBytes(MappedBytes, DataMap.Num());

	OpenDataFile();
}

////////////////////////////////////////////////////////////////////////////////
uint64 FDiskCache::RemainingUntilWrap()
{
	if (DataCursor >= MaxDataSize)
	{
		Wrap();
	}

	return MaxDataSize - DataCursor;
}

////////////////////////////////////////////////////////////////////////////////
void FDiskCache::Spam()
{
	FReadScopeLock _(Lock);

	UE_LOGF(LogIas, VeryVerbose,
		"JournaledCache: MappedKiB=%llu Entries=%d DataCur=%llu JournalCur=%u Marker=%u)",
		(MappedBytes >> 10),
		DataMap.Num(),
		DataCursor,
		Journal.GetCursor(),
		Journal.GetMarker()
	);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FDiskCache::DebugVisit(void* Param, FDebugCacheEntry::Callback* Callback)
{
	FReadScopeLock _(Lock);

	FDebugCacheEntry Out = {};
	for (const auto& Entry : DataMap)
	{
		Out.Key = Entry.Key;
		Out.Size = Entry.Value.Size;
		Callback(Param, Out);
	}
	return DataMap.Num();
}

////////////////////////////////////////////////////////////////////////////////
void FDiskCache::UpdateMagicValue(FDiskCache* Cache, uint32 MagicValue)
{
	check(Cache != nullptr);

	// Early out if the magic value matches what the journal already has
	if (Cache->Journal.DoesMagicSeedMatch(MagicValue))
	{
		return;
	}

	UE_LOGF(LogIas, Display, "JournaledCache: A new magic value (%u) has been set, reloading the cache from disk...", MagicValue);

	// Preserve the values we will need to reload the cache
	FString Path = MoveTemp(Cache->BinPath);
	const uint64 MaxDataSize = Cache->MaxDataSize;
	const uint32 JournalMaxSize = Cache->Journal.GetMaxSize();

	// Unload the old cache
	Cache->~FDiskCache();

	// Inform the statistics system that the cache has been emptied
	FOnDemandIoBackendStats::Get()->OnCachePersistedBytes(0, 0);

	// Load the cache fresh off disk, this will take the new magic value into account
	new (Cache) FDiskCache(MoveTemp(Path), MaxDataSize, JournalMaxSize, MagicValue);
	LoadCache(*Cache);

	UE_LOGF(LogIas, Display, "JournaledCache: Cache has been reloaded with the new magic value");
}

// {{{1 loader .................................................................

////////////////////////////////////////////////////////////////////////////////
static int32 LoadCache(FDiskCache& DiskCache)
{
	FWriteScopeLock _(DiskCache.Lock);

	FDiskJournal& Journal = DiskCache.Journal;

	uint32 DataSize = 0;
	TUniquePtr<uint8[]> Data;

	if (auto& Handle = Journal.JrnHandle; Handle.IsValid())
	{
		DataSize = uint32(Handle->Size());
		if (DataSize == 0)
		{
			return 0;
		}

		Data = TUniquePtr<uint8[]>(new uint8[DataSize]);
		Handle->Seek(0);
		if (!Handle->Read(Data.Get(), DataSize))
		{
			UE_LOGF(LogIas, Error, "JournaledCache: failed reading journal");
		}
	}

	if (DataSize == 0)
	{
		return 0;
	}

	UE_LOGF(LogIas, VeryVerbose, "JournaledCache: %u byte journal found", DataSize);
	
	uint64 BootTimestamp = FPlatformTime::Cycles64();
	ON_SCOPE_EXIT
	{
		BootTimestamp = FPlatformTime::Cycles64() - BootTimestamp;
		uint64 TimeMs = uint64(FPlatformTime::ToMilliseconds64(BootTimestamp));
		FOnDemandIoBackendStats::Get()->OnCacheBootMs(TimeMs);
	};

	struct FParagraph
	{
		const FPhraseDesc*	Phrase;
		uint32				EntryCount;
		uint32				DataSize;
	};

	auto IsOob = [&Data, DataSize] (const void* Address)
	{
		return (UPTRINT(Address) - UPTRINT(Data.Get())) > DataSize;
	};

	uint32 Magic = Journal.GetMagic();
	auto ReadPhrases = [&IsOob, Magic] (const uint8* Cursor, FParagraph& Out) -> const uint8*
	{
		// Only proceed if we can read at least three integers
		if (IsOob(Cursor + sizeof(FPhraseDesc)))
		{
			return nullptr;
		}

		const auto* Header = (FPhraseDesc*)Cursor;
		if (Header->Magic != Magic)
		{
			return nullptr;
		}

		Cursor += sizeof(FPhraseDesc);

		const auto* FirstEntry = Header->Entries;
		if (IsOob(Cursor + sizeof(FDataEntry)))
		{
			return nullptr;
		}

		const uint32 BytesToConsume = sizeof(FDataEntry) * (FirstEntry->EntryCount);
		if (IsOob(Cursor + BytesToConsume))
		{
			return nullptr;
		}

		Cursor += BytesToConsume;

		const auto* LastEntry = (FDataEntry*)Cursor - 1;
		if (LastEntry->EntryCount != FirstEntry->EntryCount)
		{
			return nullptr;
		}

		Out.Phrase = Header;
		Out.EntryCount = uint32(LastEntry->EntryCount);
		Out.DataSize = uint32(LastEntry->Offset + LastEntry->Size);

		return Cursor;
	};

	TArray<FParagraph> Paragraphs;
	FParagraph Paragraph;

	// Read from the front
	const uint8* Left = Data.Get();
	for (const uint8* Next; ; Left = Next)
	{
		Next = ReadPhrases(Left, Paragraph);
		if (Next == nullptr)
		{
			break;
		}

		Paragraphs.Add(Paragraph);
	}

	// Read from the back
	const uint8* Right = Data.Get() + DataSize;
	while (Right >= (Left + (sizeof(FDataEntry) * 2)))
	{
		const auto* Entry = (FDataEntry*)Right - 1;

		Entry -= Entry->EntryCount;
		if (IsOob(Entry))
		{
			break;
		}

		const auto* Next = (uint8*)Entry;
		if (ReadPhrases(Next, Paragraph) == nullptr)
		{
			break;
		}

		Paragraphs.Add(Paragraph);
		Right = Next;
	}

	UE_LOGF(LogIas, VeryVerbose, "JournaledCache: %d paragraphs discovered", Paragraphs.Num());

	if (Paragraphs.IsEmpty())
	{
		Journal.Marker = 0;
		Journal.Cursor = 0;

		return 0;
	}

	auto LessWithWrap = [] (
		const FParagraph& Lhs,
		const FParagraph& Rhs)
	{
        const auto L = Lhs.Phrase->Marker;
        const auto R = Rhs.Phrase->Marker;
        enum : uint32 { LowQuarter = 1u << 30, HighQuarter = 3u << 30 };
        int32 Wrap = (L < LowQuarter) & (R >= HighQuarter);
        Wrap |= (R < LowQuarter) & (L >= HighQuarter);
        return (L < R) != Wrap;
	};
	Algo::Sort(Paragraphs, LessWithWrap);

	// Eliminate any discontinuities and find where data wrapped
	int32 BasisIndex = 0;
	int64 Remaining = DiskCache.MaxDataSize;
	for (int32 i = Paragraphs.Num() - 2; i >= 0; BasisIndex = i--)
	{
		const FParagraph& Newer = Paragraphs[i + 1];

		int32 PhraseDataSize = Newer.DataSize;
		Remaining -= PhraseDataSize;
		if (Remaining < 0)
		{
			break;
		}

		const FParagraph& Older = Paragraphs[i];
		if (Newer.Phrase->Marker != Older.Phrase->Marker + 1)
		{
			break;
		}
	}

	if (!DiskCache.DataHandle.IsValid())
	{
		UE_LOGF(LogIas, VeryVerbose, "JournaledCache: unable to open '%ls'", *DiskCache.BinPath);
		return -1;
	}

	IFileHandle* File = DiskCache.DataHandle.Get();
	if (uint64(File->Size()) > DiskCache.MaxDataSize)
	{
		UE_LOGF(LogIas, VeryVerbose,
			"JournaledCache: Dropping - existing cache to bi; %llu/%llu",
			uint64(File->Size()), DiskCache.MaxDataSize
		);
		return -1;
	}

	// Detect data writes that are newer than any journal flushes.
	auto ReadHash = [File] (uint64 Cursor, uint32& OutHash, uint32 MaxHashSize, uint32 Seed)
	{
		if (Cursor + MaxHashSize > uint64(File->Size()))
		{
			return false;
		}

		File->Seek(Cursor);
		
		const uint32 HashSize = FMath::Min(HASH_CHECKSUM_SIZE, MaxHashSize);
		uint8 Buffer[HASH_CHECKSUM_SIZE];
		if (const bool Result = File->Read(Buffer, HashSize))
		{
			OutHash = FDiskJournal::HashBytes(Buffer, HashSize, Seed);
			return true;
		}
		return false;
	};

	// Find the first good paragraph.
	{
		enum { SearchDepth = 14 };
		int32 Last = BasisIndex + (1 << SearchDepth);
		Last = FMath::Min(Last, Paragraphs.Num());

		for (int32 End = Last; BasisIndex < End;)
		{
			int32 Mid = BasisIndex + ((End - BasisIndex) >> 1);

			const FParagraph& Stock = Paragraphs[Mid];

			uint64 DataBase = Stock.Phrase->DataCursor;
			if (DataBase + Stock.DataSize > DiskCache.MaxDataSize)
			{
				DataBase = 0;
			}

			uint32 Hash;
			const uint32 Seed = uint32(Stock.Phrase->Marker);
			if (ReadHash(DataBase, Hash, Stock.DataSize, Seed) && (Hash == Stock.Phrase->Hash))
			{
				End = Mid;
				continue;
			}

			BasisIndex = Mid + 1;
		}

		if (BasisIndex == Last)
		{
			BasisIndex = Paragraphs.Num();
		}
	}

	// Add known entries into the tree.
	uint64 MappedBytes = 0;
	uint32 MappedItems = 0;
	for (uint32 i = BasisIndex, n = Paragraphs.Num(); i < n; ++i)
	{
		const FPhraseDesc& Holm = Paragraphs[i].Phrase[0];
		uint32 EntryCount = Paragraphs[i].EntryCount;

		const FDataEntry* LastEntry = Holm.Entries + (EntryCount - 1);
		if (IsOob(LastEntry))
		{
			return -1;
		}

		MappedItems += EntryCount;
		MappedBytes += DiskCache.Insert(Holm.DataCursor, Holm.Entries, EntryCount);
	}
	
	UE_LOGF(LogIas, VeryVerbose, "JournaledCache: Mapped %u items with %llu bytes", MappedItems, MappedBytes);
	
	// Prime the journal's state
	const FParagraph& LastPara = Paragraphs.Last();
	const FPhraseDesc* LastPhrase = LastPara.Phrase;
	Journal.Marker = uint32(LastPara.Phrase->Marker) + 1;

	if (DataSize <= Journal.MaxSize)
	{
		Journal.Cursor = uint32(UPTRINT(LastPhrase + LastPara.EntryCount + 1) - UPTRINT(Data.Get()));
	}
	else
	{
		UE_LOGF(LogIas, VeryVerbose,
			"JournaledCache: Journal exceeds given size - dropping; %u/%u",
			DataSize, Journal.MaxSize
		);
		return -1;
	}

	// Prime the disk-cache's state
	DiskCache.DataCursor = LastPhrase->DataCursor + LastPara.DataSize;
	if (DiskCache.DataCursor > DiskCache.MaxDataSize)
	{
		UE_LOGF(LogIas, VeryVerbose,
			"JournaledCache: Dropping - DataCursor too big; %llu/%llu",
			DiskCache.DataCursor, DiskCache.MaxDataSize
		);
		return -1;
	}

	return 1;
}

// {{{1 cache ..................................................................

////////////////////////////////////////////////////////////////////////////////
class FCache
{
public:
	struct FConfig
		: public FIasCacheConfig
	{
		FString	Path;
	};

	using FGetToken = UPTRINT;

					FCache(FConfig&& Config);
	uint32			GetAilments() const;
	bool			Load();
	void			Drop();
	uint32			GetDemand() const;
	bool			Has(uint64 Key) const;
	FGetToken		Get(uint64 Key, FIoBuffer& OutData) const;
	bool			Put(uint64 Key, FIoBuffer& Data);
	bool			Evict(uint64 Key);
	EIoErrorCode	Materialize(FGetToken Token, FIoBuffer& OutData, uint32 Offset=0) const;
	uint32			Flush();
	uint32			WriteMemToDisk(int32 Allowance);
	uint32			DebugVisit(void* Param, FDebugCacheEntry::Callback* Callback);

	void			UpdateMagicValue(uint32 MagicValue);

private:
	struct FPartial
	{
		uint64		Key = 0;
		FIoBuffer	Data;
		uint32		Cursor;
	};
	
	mutable FRWLock	MemLock;
	FMemCache		MemCache;
	FDiskCache		DiskCache;
	std::atomic_int	Demand;
	FPartial		Partial;
};

////////////////////////////////////////////////////////////////////////////////
FCache::FCache(FConfig&& Config)
: MemCache(Config.MemoryQuota)
, DiskCache(MoveTemp(Config.Path), Config.DiskQuota, Config.JournalQuota, Config.JournalMagic)
{
	if (Config.DropCache)
	{
		DiskCache.Drop();
	}
}

////////////////////////////////////////////////////////////////////////////////
uint32 FCache::GetAilments() const
{
	uint32 Ret = 0;
	Ret |= DiskCache.GetAilments();
	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
bool FCache::Load()
{
	int32 Result = LoadCache(DiskCache);
	if (Result < 0)
	{
		Drop();
	}
	
	return Result > 0;
}

////////////////////////////////////////////////////////////////////////////////
void FCache::Drop()
{
	DiskCache.Drop();

	{
		FWriteScopeLock _(MemLock);
		MemCache.Drop();
	}

	Partial = FPartial();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FCache::GetDemand() const
{
	return Demand.load(std::memory_order_relaxed);
}

////////////////////////////////////////////////////////////////////////////////
bool FCache::Has(uint64 Key) const
{
	check(Key);

	if (DiskCache.Has(Key))
	{
		return true;
	}

	FReadScopeLock _(MemLock);
	return (MemCache.Get(Key) != nullptr) || (Partial.Key == Key);
}

////////////////////////////////////////////////////////////////////////////////
FCache::FGetToken FCache::Get(uint64 Key, FIoBuffer& OutData) const
{
	check(Key);

	// Disk first as that will have more data and is more likely to hit
	if (DiskCache.Has(Key))
	{
		return FGetToken(Key);
	}

	{
		// Nothing is on disk, so lets try the memory cache
		FReadScopeLock _(MemLock);

		if (CloneCachedIOBufferData > 0)
		{
			// Speculative fix for a read after free happening in FIoChunkEncoding::Decode
			if (Partial.Key == Key)
			{
				OutData = FIoBuffer(FIoBuffer::Clone, Partial.Data.Data(), Partial.Data.GetSize());
			}
			else if (const FIoBuffer* Data = MemCache.Get(Key); Data != nullptr)
			{
				OutData = FIoBuffer(FIoBuffer::Clone, Data->Data(), Data->GetSize());
				FOnDemandIoBackendStats::Get()->OnCacheGet(OutData.GetSize(), ECacheType::Memory);
			}
		}
		else
		{
			if (const FIoBuffer* Data = MemCache.Get(Key); Data != nullptr)
			{
				OutData = *Data;
				FOnDemandIoBackendStats::Get()->OnCacheGet(OutData.GetSize(), ECacheType::Memory);
			}

			if (Partial.Key == Key)
			{
				OutData = Partial.Data;
			}
		}
	}

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
bool FCache::Put(uint64 Key, FIoBuffer& Data)
{
	FIoBuffer Cloned = Data;
	FWriteScopeLock _(MemLock);
	bool Ok = MemCache.Put(Key, MoveTemp(Cloned));
	if (Ok)
	{
		uint32 NewDemand = MemCache.GetDemand();
		Demand.store(NewDemand, std::memory_order_relaxed);
	}
	return Ok;
}

////////////////////////////////////////////////////////////////////////////////
bool FCache::Evict(uint64 Key)
{
	bool Ok = DiskCache.Evict(Key);
	{
		FWriteScopeLock _(MemLock);
		Ok |= MemCache.Evict(Key, Ok);
	}

	if (Partial.Key == Key)
	{
		Partial = FPartial();
		Ok = true;
	}

	return (Ok != 0);
}

////////////////////////////////////////////////////////////////////////////////
EIoErrorCode FCache::Materialize(FGetToken Token, FIoBuffer& OutData, uint32 Offset) const
{
	uint64 Key = Token;
	EIoErrorCode Ret = DiskCache.Materialize(Key, OutData, Offset);
	if (Ret == EIoErrorCode::Ok)
	{
		FOnDemandIoBackendStats::Get()->OnCacheGet(OutData.GetSize(), ECacheType::Disk);
	}

	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FCache::Flush()
{
	return DiskCache.Flush();
}

////////////////////////////////////////////////////////////////////////////////
uint32 FCache::WriteMemToDisk(int32 Allowance)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IasCache::Flush_MemCache);

	bool bEof = false;
	if (int64 UntilWrap = DiskCache.RemainingUntilWrap(); UntilWrap <= int64(Allowance))
	{
		bEof = true;
		Allowance = int32(UntilWrap);
	}

	uint32 PartialBias = 0;
	FMemCache::PeelItems PeelItems;
	int32 WriteSize = 0;
	{
		FWriteScopeLock _(MemLock);

		// If we have any partials that was previously written process that first
		// and peel off as much of that buffer as possible.
		if (Partial.Key)
		{
			FMemoryView View = Partial.Data.GetView();
			View = View.Mid(Partial.Cursor, Allowance);

			uint32 ViewSize = uint32(View.GetSize());
			Partial.Cursor += ViewSize;
			WriteSize += ViewSize;

			FMemCache::FItem PeelItem = {
				.Key	= 0,
				.Data	= FIoBuffer(View, Partial.Data),
			};
			if (Partial.Cursor >= uint32(Partial.Data.GetSize()))
			{
				PeelItem.Key = Partial.Key;
				PartialBias = uint32(Partial.Data.GetSize()) - ViewSize;
				Partial = FPartial();
			}
			else if (bEof)
			{
				Partial.Cursor = 0;
			}
			PeelItems.Push(PeelItem);
		}

		// If there is any allowance left start peeling of buffers from the memcache
		if (WriteSize < Allowance)
		{
			WriteSize += MemCache.Peel(Allowance - WriteSize, PeelItems);

			uint32 NewDemand = MemCache.GetDemand();
			Demand.store(NewDemand, std::memory_order_relaxed);
		}

		// Finally split any overshooting buffers into a partial slice and save the
		// buffer in the Partial member. While being dropped this buffer exists neither
		// in the memcache or the disk cache. The partial fragment needs to be at least
		// large enough for the hash checksum.
		if (int32 Overshoot = WriteSize - Allowance; Overshoot > 0)
		{
			auto [Key, Data] = PeelItems.Pop();

			int32 PartialSize = int32(Data.GetSize()) - Overshoot;
			const FMemoryView PartialSlice = Data.GetView().Left(PartialSize);
			PeelItems.Push(FMemCache::FItem{0, FIoBuffer(PartialSlice, Data)});

			WriteSize -= Overshoot;

			uint32 Cursor = bEof ? 0 : PartialSize;
			Partial = FPartial{Key, MoveTemp(Data), uint32(Cursor)};
		}
	}

	check(WriteSize >= 0 && WriteSize <= Allowance);
	FDiskPhrase Phrase = DiskCache.OpenPhrase(WriteSize);
	int32 PeelIndex = -1;
	for (int32 i = 0, n = PeelItems.Num(); i < n; ++i)
	{
		auto& [Key, Data] = PeelItems[i];
		check(Key || i == (PeelItems.Num() - 1)); // Partials must be last
		if (Phrase.GetRemainingEntries() < 1 || !Phrase.Add(Key, MoveTemp(Data), PartialBias))
		{
			PeelIndex = i;
			break;
		}
		PartialBias = 0;
	}

	if (PeelIndex >= 0)
	{
		/* end of journal reached so not all peeled items could be added, may
		 * we can re-add leftover peeled items back to mem-cache? */
		WriteSize = Phrase.GetDataSize();
	}

	DiskCache.ClosePhrase(MoveTemp(Phrase));

	return WriteSize;
}

////////////////////////////////////////////////////////////////////////////////
uint32 FCache::DebugVisit(void* Param, FDebugCacheEntry::Callback* Callback)
{
	FReadScopeLock _(MemLock);
	uint32 Count = 0;
	Count += MemCache.DebugVisit(Param, Callback);
	Count += DiskCache.DebugVisit(Param, Callback);
	if (Partial.Key)
	{
		Count++;
		Callback(Param, {Partial.Key, uint32(Partial.Data.GetSize()), 1});
	}
	return Count;
}

////////////////////////////////////////////////////////////////////////////////
void FCache::UpdateMagicValue(uint32 MagicValue)
{
	FDiskCache::UpdateMagicValue(&DiskCache, MagicValue);
}

// {{{1 service-thread .........................................................

////////////////////////////////////////////////////////////////////////////////
class FServiceThread
	: public FRunnable
{
public:
	struct FReadSink
	{
		struct FReadResult	{ uint16 ReadId; uint16 Status; };
		virtual void		OnRead(const FReadResult* Results, int32 Num) = 0;
	};

	struct FReadRequest
	{
		uint64				Key;
		FIoBuffer*			Dest;
		FReadSink*			Sink;
		uint32				ReadId;
		uint32				Offset = 0;
	};

	static FServiceThread&	Get();
							~FServiceThread();
	void					RegisterCache(TUniquePtr<FCache> Cache);
	void					UnregisterCache(FCache* Cache);
	void					SetGovernor(FDiskCacheGovernor* Governor);
	uint32					ClaimReadId();
	void					BeginRead(const FCache* Cache, const FReadRequest& Request);
	void					CancelRead(const FIoBuffer* GivenDest);
	void					Drop();
	void					UpdateMagicValue(FCache* Cache, uint32 MagicValue);

private:
	union FWork
	{
		enum {
			Work_Register,
			Work_Unregister,
			Work_Gov,
			Work_Read,
			Work_Cancel,
			Work_Drop,
			Work_UpdateMagicValue
		};

		template <typename Type>
		struct TField
		{
			void	operator = (auto In)	{ ::memcpy(Value, &In, sizeof(Value)); }
					operator Type () const	{ Type Ret; ::memcpy(&Ret, Value, sizeof(Ret)); return Ret; }
			uint16	Value[sizeof(Type) / sizeof(uint16)];
		};

		struct {				// reg / unreg / read / cancel
			uint16				What;
			TField<FCache*>		Cache;
			TField<uint64>		Key;
		};
		struct {				// read.param
			uint16				ReadId;
			TField<FReadSink*>	Sink;
			TField<FIoBuffer*>	Dest;
		};
		struct {				// governor
			uint16				_What0;
			TField<FDiskCacheGovernor*> Gov;
		};
	};
	static_assert(sizeof(FWork) == sizeof(uint16) * 9);

	void						StartThread();
	int32						Update();
	uint32						UpdateCache(FCache* Cache);
	virtual uint32				Run() override;
	virtual void				Stop() override;
	void						SubmitWork(const FWork* Work, uint32 Num);
	void						ReceiveWork();
	TUniquePtr<FRunnableThread> Thread;
	FEventRef					WakeEvent;
	FDiskCacheGovernor*			Governor = nullptr;
	TArray<TUniquePtr<FCache>>	Caches;
	std::atomic_int				RunCount = 0;
	FCriticalSection			Lock;
	TArray<FWork>				ActiveReads;
	TArray<FWork>				PendingWork;
	std::atomic_int				PendingCount = 0;
	int32						PendingPrev = -1;
	uint32						ReadIdCounter = 0;
};

////////////////////////////////////////////////////////////////////////////////
FServiceThread& FServiceThread::Get()
{
	static FServiceThread* Ptr;
	if (Ptr != nullptr)
	{
		return *Ptr;
	}

	static FServiceThread Instance;
	Ptr = &Instance;
	return *Ptr;
};

////////////////////////////////////////////////////////////////////////////////
FServiceThread::~FServiceThread()
{
	Thread.Reset();
}

////////////////////////////////////////////////////////////////////////////////
void FServiceThread::SubmitWork(const FWork* Work, uint32 Num)
{
	FScopeLock _(&Lock);
	PendingWork.Append(Work, Num);
	PendingCount.fetch_add(1, std::memory_order_relaxed);
	WakeEvent->Trigger();
}

////////////////////////////////////////////////////////////////////////////////
void FServiceThread::RegisterCache(TUniquePtr<FCache> Cache)
{
	uint32 PrevRunCount = RunCount.fetch_add(1, std::memory_order_relaxed);

	FCache* RawPtr = Cache.Release();

	FWork Work;
	Work.What = FWork::Work_Register;
	Work.Cache = RawPtr;
	SubmitWork(&Work, 1);

	if (PrevRunCount == 0)
	{
		StartThread();
	}
}

////////////////////////////////////////////////////////////////////////////////
void FServiceThread::UnregisterCache(FCache* Cache)
{
	FWork Work;
	Work.What = FWork::Work_Unregister;
	Work.Cache = Cache;
	SubmitWork(&Work, 1);

	int32 PrevRunCount = RunCount.fetch_sub(1, std::memory_order_relaxed);
	if (PrevRunCount == 1)
	{
		/* ideally we'd shut down the thread here as there are no active caches
		 * that need servicing. But this is involved so we'll leave it up for
		 * now. See "THREAD_ALIVE" comments for add/subs keeping thread up */
		//Thread.Reset();
	}
}

////////////////////////////////////////////////////////////////////////////////
void FServiceThread::SetGovernor(FDiskCacheGovernor* Gov)
{
	FWork Work;
	Work.What = FWork::Work_Gov;
	Work.Gov = Gov;
	SubmitWork(&Work, 1);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FServiceThread::ClaimReadId()
{
	++ReadIdCounter;
	return uint16(ReadIdCounter);
}

////////////////////////////////////////////////////////////////////////////////
void FServiceThread::BeginRead(const FCache* Cache, const FReadRequest& Request)
{
	FWork Works[2] = {};

	Works[0].Cache = Cache;
	Works[0].What = FWork::Work_Read;
	Works[0].Key = Request.Key;

	Works[1].ReadId = uint16(Request.ReadId);
	Works[1].Sink = Request.Sink;
	Works[1].Dest = Request.Dest;

	SubmitWork(Works, 2);
}

////////////////////////////////////////////////////////////////////////////////
void FServiceThread::CancelRead(const FIoBuffer* GivenData)
{
	FWork Work;
	Work.What = FWork::Work_Cancel;
	Work.Dest = GivenData;
	SubmitWork(&Work, 1);
}

////////////////////////////////////////////////////////////////////////////////
void FServiceThread::Drop()
{
	FWork Work;
	Work.What = FWork::Work_Drop;
	SubmitWork(&Work, 1);
}

////////////////////////////////////////////////////////////////////////////////
void FServiceThread::UpdateMagicValue(FCache* Cache, uint32 MagicValue)
{
	FWork Work[1] = {};

	Work[0].Cache = Cache;
	Work[0].What = FWork::Work_UpdateMagicValue;
	Work[0].Key = MagicValue;

	SubmitWork(Work, 1);
}

////////////////////////////////////////////////////////////////////////////////
void FServiceThread::StartThread()
{
	RunCount.fetch_add(1, std::memory_order_relaxed); // THREAD_ALIVE

	auto* Inst = FRunnableThread::Create(this, TEXT("Ias.CacheIo"), 0, TPri_BelowNormal);
	Thread.Reset(Inst);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FServiceThread::Run()
{
	LLM_SCOPE_BYTAG(Ias);

	int64 CycleFreq	= int64(1.0 / FPlatformTime::GetSecondsPerCycle64());

	while (RunCount.load(std::memory_order_relaxed))
	{
		ReceiveWork();

		int32 WaitCycles = Update();
		if (WaitCycles < 0)
		{
			continue;
		}

		uint32 WaitMs = MAX_uint32;
		if (WaitCycles != MAX_int32)
		{
			WaitMs = uint32((int64(WaitCycles) * 1000) / CycleFreq);
		}

		WakeEvent->Wait(WaitMs);
	}

	// Loop can exist while there's at least one unregister work to do.
	ReceiveWork();
	check(Caches.Num() == 0);

	return 0;
}

////////////////////////////////////////////////////////////////////////////////
void FServiceThread::Stop()
{
	RunCount.fetch_sub(1, std::memory_order_relaxed); // THREAD_ALIVE
	WakeEvent->Trigger();
}

////////////////////////////////////////////////////////////////////////////////
int32 FServiceThread::Update()
{
	if (Caches.Num() == 0)
	{
		return MAX_int32;
	}

	// Update caches
	uint32 CycleSlice = MAX_uint32;
	if (ensureMsgf(Governor != nullptr, TEXT("Trying to update the streaming cache without a valid disk governor")))
	{
		for (TUniquePtr<FCache>& Item : Caches)
		{
			FCache* Cache = Item.Get();
			uint32 CyclesTillActive = UpdateCache(Cache);
			CycleSlice = FMath::Min(CyclesTillActive, CycleSlice);
		}
	}

	// Early out
	if (ActiveReads.Num() == 0)
	{
		return CycleSlice;
	}

	// Now we've a slice of time to process reads until caches need another tick
	int64 Cycle = FPlatformTime::Cycles64();
	int64 StopReadsCycle = Cycle + CycleSlice;

	TRACE_CPUPROFILER_EVENT_SCOPE(IasCache::ProcessReads);

	uint32 Index = 0;
	for (uint32 n = ActiveReads.Num(); Index < n;)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(IasCache::ActiveRead);

		// Lets always do at least one to make progress.
		FWork& Work = ActiveReads[Index];
		FWork& Param = ActiveReads[Index + 1];
		Index += 2;

		// A read is marked as cancelled by setting its destination to a nullptr
		EIoErrorCode Status = EIoErrorCode::Cancelled;
		if (Param.Dest != nullptr)
		{
			FCache* Cache = Work.Cache;
			Status = Cache->Materialize(Work.Key, *Param.Dest);
		}

		auto Sink = (FReadSink*)(Param.Sink);
		FReadSink::FReadResult Result = { uint16(Param.ReadId), uint16(Status) };
		Sink->OnRead(&Result, 1);

		Cycle = FPlatformTime::Cycles64();
		if (Cycle >= StopReadsCycle)
		{
			break;
		}
	}

	check(Index > 0);
	ActiveReads.RemoveAt(0, Index);

	// StopReadsCycle is where the CycleSlice would expire, while Cycle is where
	// in time we have got to. The difference is how much we need to wait.
	return int32(StopReadsCycle - Cycle);
}

////////////////////////////////////////////////////////////////////////////////
void FServiceThread::ReceiveWork()
{
	int32 PendingLoad = PendingCount.load(std::memory_order_relaxed);
	if (PendingLoad == PendingPrev)
	{
		return;
	}

	Lock.Lock();
	TArray<FWork> InboundWork = MoveTemp(PendingWork);
	Lock.Unlock();
	PendingPrev = PendingLoad;

	// Unregisters first
	bool bDropCache = false;

	for (uint32 i = 0, n = InboundWork.Num(); i < n; ++i)
	{
		const FWork& Work = InboundWork[i];

		if (Work.What == FWork::Work_Drop)
		{
			bDropCache = true;
		}

		if (Work.What != FWork::Work_Unregister)
		{
			i += (Work.What == FWork::Work_Read); // Consume parameter work item too.
			continue;
		}

		FCache* CachePtr = Work.Cache;
		bool bFound = false;
		for (int32 j = 0, m = Caches.Num(); j < m; ++j)
		{
			if (Caches[j].Get() != CachePtr)
			{
				continue;
			}

			Caches.RemoveAtSwap(j);
			bFound = true;
			break;
		}

		// It is possible that the cache's register operation is part of the inbound work.
		// In that case re-submit operation to pending work and process next pass
		if (!bFound)
		{
			for(const FWork& RegWork : InboundWork)
			{
				if (RegWork.What == FWork::Work_Register && RegWork.Cache == CachePtr)
				{
					SubmitWork(&Work, 1);
				}
			}
		}
	}

	// ...then the rest
	for (uint32 i = 0, n = InboundWork.Num(); i < n; ++i)
	{
		const FWork& Work = InboundWork[i];

		if (Work.What == FWork::Work_Unregister || Work.What == FWork::Work_Cancel || Work.What == FWork::Work_Drop)
		{
			continue;
		}

		if (Work.What == FWork::Work_Register)
		{
			FCache* Cache = Work.Cache;
			Cache->Load();
			Caches.Add(TUniquePtr<FCache>(Cache));
			continue;
		}

		if (Work.What == FWork::Work_UpdateMagicValue)
		{
			FCache* Cache = Work.Cache;
			Cache->UpdateMagicValue((uint32)Work.Key);
			continue;
		}

		if (Work.What == FWork::Work_Gov)
		{
			Governor = Work.Gov;
			continue;
		}

		check(Work.What == FWork::Work_Read);
		ActiveReads.Add(Work);
		ActiveReads.Add(InboundWork[++i]); // the read's params
	}

	// ...and finally the cancels
	for (uint32 i = 0, n = InboundWork.Num(); i < n; ++i)
	{
		const FWork& Work = InboundWork[i];

		if (Work.What != FWork::Work_Cancel)
		{
			i += (Work.What == FWork::Work_Read); // Consume parameter work item too.
			continue;
		}

		const void* ToCancel = Work.Dest;
		for (uint32 j = 1, m = ActiveReads.Num(); j < m; j += 2)
		{
			if (ActiveReads[j].Dest == ToCancel)
			{
				ActiveReads[j].Dest = nullptr;
				break;
			}
		}
	}

	// lastly if there were one ore more requests to drop the cache
	if (bDropCache)
	{
		for (TUniquePtr<FCache>& Cache : Caches)
		{
			Cache->Drop();
		}

		// cancel all active reads
		for (uint32 j = 1, m = ActiveReads.Num(); j < m; j += 2)
		{
			ActiveReads[j].Dest = nullptr;
		}
	}

	check((ActiveReads.Num() & 1) == 0);
}

////////////////////////////////////////////////////////////////////////////////
uint32 FServiceThread::UpdateCache(FCache* Cache)
{
	uint32 Demand = Cache->GetDemand();
	int32 Allowance = Governor->TryBeginWriteToStreamingCache(Demand);
	if (Allowance <= 0)
	{
		return -Allowance;
	}

	TRACE_COUNTER_SET(IasMemDemand, Demand);
	TRACE_COUNTER_SET(IasAllowance, 0);
	TRACE_COUNTER_SET(IasAllowance, Allowance);

	TRACE_CPUPROFILER_EVENT_SCOPE(IasCache::Update);

	uint32 AllowanceUsed = Cache->WriteMemToDisk(Allowance);
	uint32 Unused = Allowance - AllowanceUsed;
	TRACE_COUNTER_SET(IasAllowance, Unused);
	TRACE_COUNTER_ADD(IasOpCount, 1);

	int32 WaitCycles = Governor->EndWriteToStreamingCache(Unused);
	if (WaitCycles < 0)
	{
		WaitCycles = -WaitCycles;

		TRACE_COUNTER_ADD(IasOpCount, 1); // flush from closing .bin file - we can remove this if we use a single file for .jrn and .bin
		TRACE_COUNTER_ADD(IasOpCount, 3); // write-flush-commit from .jrn.
		Cache->Flush();
	}

	TRACE_COUNTER_SET(IasAllowance, 0);

	return WaitCycles;
}

// }}}

} // namespace UE::IoStore::JournaledCache



namespace UE::IoStore {

// {{{1 journaled-cache ........................................................

////////////////////////////////////////////////////////////////////////////////
class FJournaledCache final
	: public IIasCache
	, public JournaledCache::FServiceThread::FReadSink
{
public:
								FJournaledCache(FDiskCacheGovernor& DiskCacheGovernor);
								~FJournaledCache();
	bool						Initialize(const TCHAR* RootDir, const FIasCacheConfig& Config);
	virtual void				Abandon() override;
	virtual void				Drop() override;
	virtual bool				ContainsChunk(const FIoHash& Key) const override;
	virtual EIoErrorCode		Get(const FIoHash& Key, FIoBuffer& OutData) override;
	virtual void				Materialize(const FIoHash& Key, FIoBuffer& Dest, EIoErrorCode& Status, UE::Tasks::FTaskEvent DoneEvent) override;
	virtual void				Cancel(FIoBuffer& GivenDest) override;
	virtual FIoStatus			Put(const FIoHash& Key, FIoBuffer& Data) override;
	virtual FIoStatus			Evict(const FIoHash& Key) override;
	virtual void				UpdateMagicValue(uint32 MagicValue) override;
private:
	struct FMaterialOp
	{
		UE::Tasks::FTaskEvent	DoneEvent;
		EIoErrorCode*			Status;
	};

	virtual void				OnRead(const FReadResult* Results, int32 Num) override;
	static uint64				ReduceKey(const FIoHash& Key);
	FDiskCacheGovernor&			DiskCacheGovernor;
	JournaledCache::FCache*		Cache = nullptr;
	UE::Tasks::FPipe			GetPipe = UE::Tasks::FPipe(TEXT("IasCacheGetPipe"));
	FCriticalSection			Lock;
	TMap<int32, FMaterialOp>	PendingMaterializes;
};

////////////////////////////////////////////////////////////////////////////////
FJournaledCache::FJournaledCache(FDiskCacheGovernor& InDiskCacheGovernor)
	: DiskCacheGovernor(InDiskCacheGovernor)
{
}

////////////////////////////////////////////////////////////////////////////////
FJournaledCache::~FJournaledCache()
{
	if (Cache != nullptr)
	{
		JournaledCache::FServiceThread::Get().UnregisterCache(Cache);
	}
	JournaledCache::FServiceThread::Get().SetGovernor(nullptr);
}

////////////////////////////////////////////////////////////////////////////////
bool FJournaledCache::Initialize(const TCHAR* RootDir, const FIasCacheConfig& Config)
{
	using namespace JournaledCache;

	// Filesystem setup
	FStringView Name = Config.Name;
	check(Name.Len() > 0 && !Name.EndsWith('/') && !Name.EndsWith('\\'));

	TStringBuilder<256> CachePath;
	CachePath << RootDir;
	FPathViews::Append(CachePath, GetCacheFsDir());
	FPathViews::Append(CachePath, FPathViews::GetPath(Name));

	if (IFileManager& Ifm = IFileManager::Get(); !Ifm.MakeDirectory(CachePath.ToString(), true))
	{
		UE_LOGF(LogIas, Error, "JournaledCache: Unable to create directory '%ls'", CachePath.ToString());
		return false;
	}

	FPathViews::Append(CachePath, FPathViews::GetBaseFilename(Name));
	CachePath << GetCacheFsSuffix();

	// Inner cache
	FCache::FConfig EventualConfig;
	static_cast<FIasCacheConfig&>(EventualConfig) = Config;
	EventualConfig.Path = CachePath;

	const FIasCacheConfig::FRate& WriteRate = Config.WriteRate;
	const FIasCacheConfig::FDemand& Demand = Config.Demand;

	// Configure the governor
	FDiskCacheGovernor::FConfig Cfg;
	Cfg.Allowance.Bytes		= WriteRate.Allowance;
	Cfg.Allowance.Ops		= WriteRate.Ops;
	Cfg.Allowance.Seconds	= WriteRate.Seconds;
	Cfg.Demand.Threshold	= Demand.Threshold;
	Cfg.Demand.Boost		= Demand.Boost;
	Cfg.Demand.SuperBoost	= Demand.SuperBoost;
	Cfg.CommitBufferSize	= JournaledCache::GetWriteCommitThreshold(CachePath);
	DiskCacheGovernor.ConfigureStreamingCache(Cfg);

	TUniquePtr<FCache> NewCache = MakeUnique<FCache>(MoveTemp(EventualConfig));

	if (uint32 Ailments = NewCache->GetAilments(); Ailments != 0)
	{
		UE_LOGF(LogIas, Error, "JournaledCache: Error initialising inner cache '%x'", Ailments);
		return false;
	}

	Cache = NewCache.Get();

	FServiceThread& ServiceThread = FServiceThread::Get();
	ServiceThread.SetGovernor(&DiskCacheGovernor);
	ServiceThread.RegisterCache(MoveTemp(NewCache));

	return true;
}

////////////////////////////////////////////////////////////////////////////////
void FJournaledCache::Abandon()
{
	LLM_SCOPE_BYTAG(Ias);

	Cache->Drop();
	delete this;
}

////////////////////////////////////////////////////////////////////////////////
void FJournaledCache::Drop()
{
	JournaledCache::FServiceThread::Get().Drop();
}

////////////////////////////////////////////////////////////////////////////////
bool FJournaledCache::ContainsChunk(const FIoHash& Key) const
{
	uint64 InnerKey = ReduceKey(Key);
	return Cache->Has(InnerKey);
}

////////////////////////////////////////////////////////////////////////////////
EIoErrorCode FJournaledCache::Get(const FIoHash& Key, FIoBuffer& OutData)
{
	using namespace JournaledCache;

	check(OutData.GetData() == nullptr);

	uint64 InnerKey = ReduceKey(Key);

	uint64 GetKey = Cache->Get(InnerKey, OutData);
	if (OutData.GetData() != nullptr)
	{
		return EIoErrorCode::Ok;
	}

	// "File not open" to indicate that we have Key, just not to hand.
	return (GetKey == 0) ? EIoErrorCode::NotFound : EIoErrorCode::FileNotOpen;
}

////////////////////////////////////////////////////////////////////////////////
void FJournaledCache::Materialize(
	const FIoHash& Key,
	FIoBuffer& Dest,
	EIoErrorCode& Status,
	UE::Tasks::FTaskEvent DoneEvent)
{
	using namespace JournaledCache;

	FServiceThread& ServiceThread = FServiceThread::Get();

	uint32 ReadId;
	{
		FScopeLock _(&Lock);
		ReadId = ServiceThread.ClaimReadId();
		PendingMaterializes.Add(ReadId, { MoveTemp(DoneEvent), &Status });
	}

	FServiceThread::FReadRequest Request = {
		.Key = ReduceKey(Key),
		.Dest = &Dest,
		.Sink = this,
		.ReadId = ReadId,
	};
	ServiceThread.BeginRead(Cache, Request);
}

////////////////////////////////////////////////////////////////////////////////
void FJournaledCache::Cancel(FIoBuffer& GivenDest)
{
	JournaledCache::FServiceThread::Get().CancelRead(&GivenDest);
}

////////////////////////////////////////////////////////////////////////////////
void FJournaledCache::OnRead(const FReadResult* Results, int32 Num)
{
	FScopeLock _(&Lock);

	for (; Num-- > 0; ++Results)
	{
		const FReadResult& Result = Results[0];

		FMaterialOp* Pend = PendingMaterializes.Find(Result.ReadId);
		check(Pend != nullptr);

		Pend->Status[0] = EIoErrorCode(Result.Status);
		Pend->DoneEvent.Trigger();

		PendingMaterializes.Remove(Result.ReadId);
	}
}

////////////////////////////////////////////////////////////////////////////////
FIoStatus FJournaledCache::Put(const FIoHash& Key, FIoBuffer& Data)
{
	uint64 InnerKey = ReduceKey(Key);
	bool Ok = Cache->Put(InnerKey, Data);
	return Ok ? FIoStatus::Ok : FIoStatus(EIoErrorCode::Unknown);
}

////////////////////////////////////////////////////////////////////////////////
FIoStatus FJournaledCache::Evict(const FIoHash& Key)
{
	uint64 InnerKey = ReduceKey(Key);
	bool Ok = Cache->Evict(InnerKey);
	return Ok ? FIoStatus::Ok : FIoStatus(EIoErrorCode::Unknown);
}

////////////////////////////////////////////////////////////////////////////////
void FJournaledCache::UpdateMagicValue(uint32 MagicValue)
{
	JournaledCache::FServiceThread::Get().UpdateMagicValue(Cache, MagicValue);
}

////////////////////////////////////////////////////////////////////////////////
uint64 FJournaledCache::ReduceKey(const FIoHash& Key)
{
	const uint8* Bytes = Key.GetBytes();
	uint64 Ret[3] = {};
	std::memcpy(Ret + 0, Bytes +  0, sizeof(uint64));
	std::memcpy(Ret + 1, Bytes +  8, sizeof(uint64));
	std::memcpy(Ret + 2, Bytes + 16, sizeof(uint32));
	return (Ret[0] + Ret[2]) ^ Ret[1];
}



////////////////////////////////////////////////////////////////////////////////
TUniquePtr<IIasCache> MakeIasCache(const TCHAR* RootPath, const FIasCacheConfig& Config, FDiskCacheGovernor& Governor)
{
	LLM_SCOPE_BYTAG(Ias);

	FJournaledCache* Cache = new FJournaledCache(Governor);
	if (Cache->Initialize(RootPath, Config))
	{
		return TUniquePtr<IIasCache>(Cache);
	}

	delete Cache;
	return nullptr;
}



// {{{1 test ...................................................................

#if IS_PROGRAM

namespace IasJournaledFileCacheTest
{

////////////////////////////////////////////////////////////////////////////////
static constexpr uint64 operator ""_Ki (unsigned long long value) { return value << 10; }
static constexpr uint64 operator ""_Mi (unsigned long long value) { return value << 20; }

////////////////////////////////////////////////////////////////////////////////
static uint64 KeyGen(const uint8* Data, uint32 Size)
{
	uint64 Ret = 0x0'a9e0'493;
	for (; Size; Ret = (Data[--Size] + Ret) * 0x369dea0f31a53f85ull);
	return Ret;
}

static uint64 KeyGen(const FIoBuffer& Data)
{
	return KeyGen(Data.GetData(), uint32(Data.GetSize()));
}

////////////////////////////////////////////////////////////////////////////////
struct FSupport
{
	FSupport(const TCHAR* InCacheDir=nullptr)
	{
		for (uint32 i = 0; i < WorkingSize; i += 8)
		{
			*(uint64*)(Working + i) = Mix();
		}

		TestDir = (InCacheDir != nullptr) ? FString(InCacheDir) : FPaths::ProjectPersistentDownloadDir();
		TestDir /= TEXT("ias_cache_test");

		CleanFs();
	}

	~FSupport()
	{
		CleanFs();
	}

	void CleanFs()
	{
		IFileManager& Ifm = IFileManager::Get();
		if (Ifm.DirectoryExists(*TestDir))
		{
			// Windows doesn't delete directories immediately and subsequent make
			// dirs can fail. So we rename first then delete.
			FString TempDir = TestDir + "~";
			check(Ifm.Move(*TempDir, *TestDir, false));
			check(Ifm.DeleteDirectory(*TempDir, false, true));
		}
		check(Ifm.MakeDirectory(*TestDir, true));
	}

	auto DummyData(uint64 Size)
	{
		uint64 Offset = Mix() % (WorkingSize - Size);
		return FIoBuffer(FIoBuffer::Wrap, Working + Offset, Size);
	}

	uint64				Mix() { return Th *= 0x369dea0f31a53f85ull; }
	uint64				Th = 0x0'a9e0'493; // prime!
	const uint64		WorkingSize = 1_Mi;
	TUniquePtr<uint8[]> WorkingScope = TUniquePtr<uint8[]>(new uint8[WorkingSize]);
	uint8*				Working = WorkingScope.Get();
	FString				TestDir;
};

////////////////////////////////////////////////////////////////////////////////
static void MemCacheTests(FSupport& Support)
{
	using namespace JournaledCache;

	struct {
		int32 Size;
		int32 Expected;
	} TestCases[] = {
		{ 0, 0 },
		{ 10, 0 },
		{ 1023, 511 },
		{ 1024, 1024 },
		{ 1025, 1024 },
	};

	for (auto& [Size, Expected] : TestCases)
	{
		FMemCache MemCache(Size);

		MemCache.Put(0x493, FIoBuffer());
		MemCache.Put(0x493, Support.DummyData(0));
		check(MemCache.GetCount() == 0);

		MemCache.Put(0x493, Support.DummyData(513));
		MemCache.Put(0xa9e, Support.DummyData(511));
		check(MemCache.GetUsed() == Expected);

		MemCache.Put(0x49e, Support.DummyData(11));
		Expected = (Expected == 0) ? 0 : (511 + 11);
		check(MemCache.GetUsed() == Expected);
	}

	FMemCache::PeelItems Peeled;

	FMemCache MemCache(64);
	MemCache.Put(1, Support.DummyData(1));
	check(MemCache.Peel(0, Peeled) == 0);
	check(Peeled.Num() == 0);
	check(MemCache.Peel(64, Peeled) == 1);
	check(Peeled.Num() == 1);
	check(MemCache.GetUsed() == 0);
	Peeled.Reset();

	MemCache = FMemCache(64);
	for (int32 i = 0; i < 64; ++i)
	{
		MemCache.Put(i + 1, Support.DummyData(1));
	}

	check(MemCache.Peel(32, Peeled) == 32);
	check(Peeled.Num() == 32);
	check(MemCache.GetUsed() == 32);
	for (auto& [Key, _] : Peeled)
	{
		FIoBuffer Data;
		check(MemCache.Get(Key) == 0);
	}
	Peeled.Reset();
}

////////////////////////////////////////////////////////////////////////////////
static void BigCache(FSupport& Support)
{
	using namespace JournaledCache;

	FCache::FConfig Config;
	Config.Path = Support.TestDir / "big_cache";
	Config.MemoryQuota = uint32(2_Mi);
	Config.DiskQuota = 512_Mi;
	Config.JournalQuota = uint32(32_Ki);
	Config.DropCache = false;
	FCache Cache(MoveTemp(Config));

	uint32 FlushPeriod = 3;
	for (uint32 Countdown = 1171; Countdown-- != 0;)
	{
		for (uint64 Num = (Support.Mix() % 26) + 1; Num-- != 0;)
		{
			uint64 Size = Support.Mix() & ((128_Ki) - 1);
			FIoBuffer Data = Support.DummyData(Size);
			Cache.Put(KeyGen(Data), Data);
		}

		Cache.WriteMemToDisk(uint32(768_Ki));
		if ((Countdown % FlushPeriod) == 0)
		{
			Cache.Flush();
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
static void CacheTests(FSupport& Support)
{
	using namespace JournaledCache;

	// Small structure used to verify stored data
	struct FStoredData
	{
		uint64 Key;
		uint64 Size;
		uint64 Hash;
	};
	
	FCache::FConfig DefaultSettings;
	DefaultSettings.Path = Support.TestDir / "cache_tests";
	DefaultSettings.MemoryQuota = uint32(512_Ki);
	DefaultSettings.DiskQuota = 8_Mi;
	DefaultSettings.JournalQuota = uint32(7_Ki);

	FCache* Cache = nullptr;
	auto NewCache = [&Cache, &Support, DefaultSettings] (bool Drop=false, const FCache::FConfig* Settings = nullptr)
	{
		delete Cache;
		FCache::FConfig Config = Settings ? *Settings : DefaultSettings;
		Config.DropCache = Drop;
		Cache = new FCache(MoveTemp(Config));
	};

	auto PrimePuts = [&] (int64 PutMax) {
		TMap<uint64, FIoBuffer> Ret;
		while (true)
		{
			uint64 Size = Support.Mix() & ((128_Ki) - 1);
			if ((PutMax -= Size) < 0)
			{
				break;
			}
			FIoBuffer Data = Support.DummyData(Size);
			uint64 Key = KeyGen(Data);
			Cache->Put(Key, Data);
			Ret.Add(Key, Data);
		}
		return Ret;
	};

	uint32 WriteAllowance;

	// no-op
	NewCache(true);
	WriteAllowance = uint32(1_Ki);
	check(Cache->WriteMemToDisk(WriteAllowance) == 0);
	check(Cache->Flush() == 0);

	NewCache(true);
	WriteAllowance = uint32(512_Ki);
	PrimePuts(WriteAllowance);
	check(Cache->WriteMemToDisk(0) == 0);
	check(Cache->WriteMemToDisk(WriteAllowance) > 0);
	check(Cache->Flush() > 0);

	auto Validate = [&] () {
		struct FVisitState {
			FCache& Cache;
			const uint8* WorkRange[2];
		};
		auto Visitor = [] (void* Param, const FDebugCacheEntry& Entry) {
			auto& State = *(FVisitState*)Param;
			
			FIoBuffer Data;
			auto Token = State.Cache.Get(Entry.Key, Data);
			if (const uint8* Ptr = Data.GetData(); Ptr == nullptr)
			{
				check(Token == Entry.Key);
				check(State.Cache.Materialize(Token, Data, 0) == EIoErrorCode::Ok);
			}

			check(Data.GetSize() == Entry.Size);
			check(KeyGen(Data) == Entry.Key);

			const uint8* Ptr = Data.GetData();
			int32 IsFromDisk = 0;
			IsFromDisk |= Ptr >= State.WorkRange[1];
			IsFromDisk |= (Ptr + Data.GetSize()) <= State.WorkRange[0];
			check(Entry.IsMemCache != IsFromDisk);
		};
		FVisitState State = {*Cache, {Support.Working, Support.Working + Support.WorkingSize}};
		return Cache->DebugVisit(&State, Visitor);
	};
	
	NewCache();
	check(Validate() == 0);

	WriteAllowance = uint32(512_Ki);

	// simple
	for (uint32 i : {1, 2, 4, 7, 11})
	{
		for (uint32 j = 0; j < i; ++j)
		{
			FIoBuffer Data = Support.DummyData(32);
			Cache->Put(KeyGen(Data), Data);
		}
		Cache->WriteMemToDisk(WriteAllowance);
		check(Validate() == i);

		NewCache(false);
		check(Cache->Load());
		check(Validate() == 0); // no flushes to write a journal
	}

	// general
	for (int32 i : {1, 4, 136, 137})
	{
		enum { Jfi = 4 };
		for (int32 j = 0; j < i; ++j)
		{
			PrimePuts(WriteAllowance);
			Cache->WriteMemToDisk(WriteAllowance);
			if (j % Jfi)
			{
				Cache->Flush();
			}
		}
		uint32 PreCount = Validate();

		NewCache();
		check(Cache->Load());

		uint32 PostCount = Validate();
		check(!PostCount == !(i / Jfi));
		check(PostCount <= PreCount);
 	}
	NewCache(true);

	// power 2
	for (int32 i : {74, 75})
	{
		while (i--)
		{
			for (int32 j = 0; j < 3; ++j)
			{
				FIoBuffer Data = Support.DummyData(64_Ki);
				Cache->Put(KeyGen(Data), Data);
				Cache->WriteMemToDisk(WriteAllowance);
			}
			Cache->Flush();
		}
		Validate();
		NewCache(true);
	}

	// partial get
	{
		const uint32 Key = 0x493;
		const uint32 DataSize = uint32(64_Ki);
		FIoBuffer Data = Support.DummyData(DataSize);
		Cache->Put(Key, Data);
		for (int32 i : {0, 1})
		{
			Cache->WriteMemToDisk(uint32(16_Ki + (16_Ki * i)));
			Data = FIoBuffer();
			Cache->Get(Key, Data);
			check(Data.GetData() != nullptr);
			check(Data.GetSize() == DataSize);
		}

		Cache->WriteMemToDisk(uint32(16_Ki));
		Data = FIoBuffer();
		check(Cache->Get(Key, Data) == Key);
		check(Data.GetData() == nullptr);

		Cache->Flush();
		NewCache();
		Cache->Load();
		check(Cache->Get(Key, Data) == Key);
		check(Data.GetData() == nullptr);

		NewCache(true);
	}

	// little phrases
	{
		FIoBuffer Data;
		Data = Support.DummyData(2);	Cache->Put(1, Data);
		Data = Support.DummyData(126);	Cache->Put(2, Data);

		Cache->WriteMemToDisk(32);
		Cache->WriteMemToDisk(32);
		Cache->WriteMemToDisk(64);
		Cache->Flush();

		NewCache();
		Cache->Load();
		Data = FIoBuffer();
		check(Cache->Get(1, Data) == 1); check(Data.GetData() == nullptr);
		check(Cache->Get(2, Data) == 2); check(Data.GetData() == nullptr);

		NewCache(true);
	}

	// eviction
	{
		FIoBuffer Data;
		Data = Support.DummyData(5); Cache->Put(5, Data);
		Data = Support.DummyData(6); Cache->Put(6, Data);
		Data = Support.DummyData(7); Cache->Put(7, Data);
		Data = Support.DummyData(8); Cache->Put(8, Data);

		// mem-cache evict
		{
			Data = FIoBuffer();
			Cache->Get(6, Data); check(Data.GetData() != nullptr);
			Cache->Evict(6);
			Data = FIoBuffer();
			check(!Cache->Has(6));
		}

		// disk-cache evict
		{
			Cache->WriteMemToDisk(uint32(1_Ki));
			Cache->Flush();

			Data = FIoBuffer();
			check(Cache->Get(7, Data) != 0);
			Cache->Evict(7);
			Data = FIoBuffer();
			check(!Cache->Has(7));

			Cache->WriteMemToDisk(uint32(1_Ki));
			Cache->Flush();

			NewCache();
			Cache->Load();
			check( Cache->Has(5));
			check(!Cache->Has(6));
			check(!Cache->Has(7));
			check( Cache->Has(8));
		}

		// partial evict
		{
			Data = Support.DummyData(2_Ki); Cache->Put(9, Data);
			Cache->WriteMemToDisk(uint32(1_Ki));
			Cache->Flush();
			check(Cache->Has(9));
			Cache->Evict(9);
			check(!Cache->Has(9));
		}

		NewCache(true);
	}

	// journal wrap
	{
		FCache::FConfig Config;
		Config.Path = Support.TestDir / "cache_jrn_wrap";
		Config.MemoryQuota = uint32(4_Mi);
		Config.DiskQuota = 16_Mi;
		Config.JournalQuota = uint32(2_Ki - 1);
		
		NewCache(true, &Config);

		auto PutAndCommit = [&] (int64 FillCnt, uint64 SizeMax, TArrayView<int32> Allowance) {
			TArray<FStoredData> Ret;
			for(uint32 i = 0; i < FillCnt; ++i)
			{
				const uint64 Size = Support.Mix() & (SizeMax - 1);
				FIoBuffer Data = Support.DummyData(Size);
				const uint64 Key = KeyGen(Data);
				Ret.Push(FStoredData{Key, Size, CityHash64((const char*)Data.GetData(), uint32(Data.GetSize()))});
				Cache->Put(Key, Data);
				const uint32 AllowanceIdx = i % Allowance.Num();
				if (Allowance[AllowanceIdx])
				{
					Cache->WriteMemToDisk(Allowance[AllowanceIdx]);
					if (i % 3 == 0)
					{
						Cache->Flush();
					}
				}
			}
			Cache->Flush();
			return Ret;
		};

		int32 Allowances[] = { int32(1_Mi) };
		auto CommittedBuffers = PutAndCommit(2048, 16_Ki, Allowances);

		NewCache(false, &Config);
		Cache->Load();

		uint32 Found(0), Lost(0);

		for (const FStoredData& Committed : CommittedBuffers)
		{
			FIoBuffer Data;
			if (const auto Token = Cache->Get(Committed.Key, Data); Token == Committed.Key)
			{
				check(Cache->Materialize(Token, Data, 0) == EIoErrorCode::Ok);
			}
			// Some items have been lost by this point, as expected
			if (Data.GetSize())
			{
				check(Data.DataSize() == Committed.Size);
				const uint64 Hash = CityHash64((const char*)Data.GetData(), uint32(Data.GetSize()));
				check(Committed.Hash == Hash);
				++Found;
			}
			else
			{
				++Lost;
			}
		}

		UE_LOGF(LogIas, Display, "Journal wrap test found %u correct entries. %u entries were lost.", Found, Lost);
		check(Found > 40);
	}
	
	// journal wrap 2
	{
		FCache::FConfig Config;
		Config.Path = Support.TestDir / "cache_jrn_wrap2";
		Config.MemoryQuota = uint32(4_Mi);
		Config.DiskQuota = uint64(16_Ki*819);
		Config.JournalQuota = uint32(2_Ki);
		
		NewCache(true, &Config);

		auto PutAndCommit = [&] (int64 FillCnt, uint64 SizeMax, TArrayView<int32> Allowance) {
			TArray<FStoredData> Ret;
			for(uint32 i = 0; i < FillCnt; ++i)
			{
				const uint64 Size = SizeMax;
				FIoBuffer Data = Support.DummyData(Size);
				const uint64 Key = KeyGen(Data);
				Ret.Push(FStoredData{Key, Size, CityHash64((const char*)Data.GetData(), uint32(Data.GetSize()))});
				Cache->Put(Key, Data);
				const uint32 AllowanceIdx = i % Allowance.Num();
				if (Allowance[AllowanceIdx])
				{
					Cache->WriteMemToDisk(Allowance[AllowanceIdx]);
					if (i % 3 == 0)
					{
						Cache->Flush();
					}
				}
			}
			Cache->Flush();
			return Ret;
		};

		int32 Allowances[] = { int32(1_Mi), 0, 0, 0, 0, int32(2_Mi), 0, 0, 0, int32(500_Ki), 0 };
		auto CommittedBuffers = PutAndCommit(2048, 16_Ki, Allowances);

		NewCache(false, &Config);
		Cache->Load();

		uint32 Found(0), Lost(0);

		for (const FStoredData& Committed : CommittedBuffers)
		{
			FIoBuffer Data;
			if (const auto Token = Cache->Get(Committed.Key, Data); Token == Committed.Key)
			{
				check(Cache->Materialize(Token, Data, 0) == EIoErrorCode::Ok);
			}
			// Some items have been lost by this point, as expected
			if (Data.GetSize())
			{
				check(Data.DataSize() == Committed.Size);
				const uint64 Hash = CityHash64((const char*)Data.GetData(), uint32(Data.GetSize()));
				check(Committed.Hash == Hash);
				++Found;
			}
			else
			{
				++Lost;
			}
		}

		UE_LOGF(LogIas, Display, "Journal wrap test 2 found %u correct entries. %u entries were lost.", Found, Lost);
		check(Found > 40);
	}

	// random writes and allowances, wrap both cache and journal
	{
		constexpr uint64 AllowanceMax = 1_Mi;
		int32 Allowances[] = {
			int32(Support.Mix() & (AllowanceMax-1)),
			int32(Support.Mix() & (AllowanceMax-1)),
			int32(Support.Mix() & (AllowanceMax-1)),
			int32(Support.Mix() & (AllowanceMax-1)),
			0, 0,
			int32(Support.Mix() & (AllowanceMax-1)),
			int32(Support.Mix() & (AllowanceMax-1)),
			int32(Support.Mix() & (AllowanceMax-1)),
			int32(Support.Mix() & (AllowanceMax-1)),
			int32(Support.Mix() & (AllowanceMax-1)),
			0,
			int32(Support.Mix() & (AllowanceMax-1)),
			int32(Support.Mix() & (AllowanceMax-1)),
		};
		
		auto PutAndCommit = [&] (int64 FillCnt, uint64 SizeMax, TArrayView<int32> Allowance) {
			TArray<FStoredData> Ret;
			for(uint32 i = 0; i < FillCnt; ++i)
			{
				const uint64 Size = Support.Mix() & (SizeMax - 1);
				FIoBuffer Data = Support.DummyData(Size);
				const uint64 Key = KeyGen(Data);
				Ret.Push(FStoredData{Key, Size, CityHash64((const char*)Data.GetData(), uint32(Data.GetSize()))});
				Cache->Put(Key, Data);
				const uint32 AllowanceIdx = i % Allowance.Num();
				if (Allowance[AllowanceIdx])
				{
					Cache->WriteMemToDisk(Allowance[AllowanceIdx]);
					if (i % 3 == 0)
					{
						Cache->Flush();
					}
				}
			}
			Cache->Flush();
			return Ret;
		};

		auto CheckCommitted = [&Cache](const TArray<FStoredData>& CommittedBuffers)
		{
			uint32 Found(0);

			for (const FStoredData& Committed : CommittedBuffers)
			{
				FIoBuffer Data;
				if (const auto Token = Cache->Get(Committed.Key, Data); Token == Committed.Key)
				{
					check(Cache->Materialize(Token, Data, 0) == EIoErrorCode::Ok);
				}
				// Some items have been lost by this point, as expected
				if (Data.GetSize())
				{
					check(Data.DataSize() == Committed.Size);
					const uint64 Hash = CityHash64((const char*)Data.GetData(), uint32(Data.GetSize()));
					check(Committed.Hash == Hash);
					++Found;
				}
			}

			return Found;
		};

		auto CheckCached = [&Cache](const TArray<FStoredData>& CommittedBuffers)
		{
			uint32 Found(0);
			TArray<uint64> KnownKeys;
			Cache->DebugVisit(&KnownKeys, [](void* Param, const FDebugCacheEntry& Entry)
			{
				TArray<uint64>* KnownKeys = (TArray<uint64>*) Param;
				KnownKeys->Add(Entry.Key);
			});

			for (uint64 Key : KnownKeys)
			{
				FIoBuffer Data;
				if (const auto Token = Cache->Get(Key, Data); Token == Key)
				{
					check(Cache->Materialize(Token, Data, 0) == EIoErrorCode::Ok);
				}
				if (const FStoredData* Stored = CommittedBuffers.FindByPredicate([&](const FStoredData& Entry){ return Key == Entry.Key; }))
				{
					check(Data.DataSize() == Stored->Size);
					const uint64 Hash = CityHash64((const char*)Data.GetData(), uint32(Data.GetSize()));
					check(Stored->Hash == Hash);
					++Found;
				}
			}
			return Found;
		};

		FCache::FConfig Config;
		Config.Path = Support.TestDir / "cache_random";
		Config.MemoryQuota = uint32(4_Mi);
		Config.DiskQuota = 16_Mi;
		Config.JournalQuota = uint32(32_Ki);
		
		NewCache(true, &Config);
		
		uint32 Loops = 8;
		while (--Loops)
		{
			auto CommittedBuffers = PutAndCommit(32, 1_Mi, Allowances);

			uint32 FoundBefore = CheckCached(CommittedBuffers);

			NewCache(false, &Config);
			Cache->Load();

			uint32 FoundAfter = CheckCommitted(CommittedBuffers);

			UE_LOGF(LogIas, Display, "Random cache test found %u correct entries before reload and %u after out of %d.", FoundBefore, FoundAfter, CommittedBuffers.Num());
			check(FoundBefore > 10 && FoundAfter > 10);
		}
	}

	// Make sure we delete the cache at the end to release the file handle
	delete Cache;



	// marker wrap
	// one-phrase journal
	// journal paragraphs that are all the same size
	// phrases with no entries

	// cache items larger than pending memory
	// cache items larger than write allowance

	// journal wrapping without truncation

	// changes in max data/journal size

	// don't load-and-sort so many paragraphs (only need max-data size
}

////////////////////////////////////////////////////////////////////////////////
static void MiscTests(FSupport& Support)
{
	using namespace JournaledCache;

	FIasCacheConfig Config;
	Config.Name = TEXT("misc");

	{ // Benign
		Support.CleanFs();
		FDiskCacheGovernor Gov;
		auto Jc = MakeIasCache(*Support.TestDir, Config, Gov);
		check(Jc.IsValid());
		Jc.Reset();
	}

	{ // Ensure path creation
		Support.CleanFs();
		for (int32 i : { 0, 1 })
		{
			const TCHAR* TestName = TEXT("m/i/s/c");
			Config.Name = TestName + i;
			FDiskCacheGovernor Gov;
			auto Jc = MakeIasCache(*Support.TestDir, Config, Gov);
			check(Jc.IsValid());
			Jc.Reset();
		}
		Config.Name = TEXT("misc");
	}

	{ // Unable to create files
		Config.Name = TEXT("Blocked");

		for (int32 i : { 0, 1 })
		{
			Support.CleanFs();

			FString Blocker = Support.TestDir;
			Blocker /= GetCacheFsDir();
			Blocker /= FString(Config.Name);
			Blocker += GetCacheFsSuffix();
			if (i == 1)
			{
				Blocker += GetCacheJrnSuffix();
			}

			IFileManager& Ifm = IFileManager::Get();
			check(Ifm.MakeDirectory(*Blocker, true));

			FDiskCacheGovernor Gov;
			auto Jc = MakeIasCache(*Support.TestDir, Config, Gov);
			check(!Jc.IsValid())
		}
	}
}

////////////////////////////////////////////////////////////////////////////////
static void Truncation(FSupport& Support)
{
	// To really stress-test truncated .jrn file, use -stompmalloc

	using namespace JournaledCache;

	const ANSICHAR* CacheName = "truncate";
	FString CachePath = Support.TestDir / CacheName;

	for (uint32 i = 0; i < 512; ++i)
	{
		Support.CleanFs();

		auto NewCache = [&CachePath] {
			FCache::FConfig Config;
			Config.Path = CachePath;
			Config.MemoryQuota = uint32(2_Mi);
			Config.DiskQuota = 512_Mi;
			Config.JournalQuota = uint32(128_Ki);
			Config.DropCache = false;
			return new FCache(MoveTemp(Config));
		};

		// Fill
		FCache* Cache = NewCache();
		enum { PhraseNum = 31, EntryNum = 13 };
		for (uint32 p = 0; p < PhraseNum; ++p)
		{
			for (uint32 e = 0; e < EntryNum; ++e)
			{
				uint32 Size = (e & 1) ? 37 : 11;
				FIoBuffer Data = Support.DummyData(Size);
				Cache->Put(KeyGen(Data), Data);
				Cache->WriteMemToDisk(1 << 30);
			}
			Cache->Flush();
		}
		delete Cache;

		// Ordinary load
		Cache = NewCache();
		Cache->Load();
		delete Cache;

		// Truncate .jrn file
		FString JrnPath = Support.TestDir;
		JrnPath /= FString(CacheName);
		JrnPath += GetCacheJrnSuffix();

		IPlatformFile& Ipf = IPlatformFile::GetPlatformPhysical();
		IFileHandle* JrnFile = Ipf.OpenWrite(*JrnPath, true, true);
		uint64 JrnSize = JrnFile->Size();
		check(JrnSize > i);
		JrnSize -= i;
		JrnFile->Truncate(JrnSize);
		delete JrnFile;

		// Survive the truncation
		Cache = NewCache();
		Cache->Load();
		delete Cache;
	}
}

////////////////////////////////////////////////////////////////////////////////
IOSTOREONDEMAND_API void Tests(const TCHAR* CacheDir=nullptr)
{
	FSupport Support(CacheDir);
	MiscTests(Support);
	MemCacheTests(Support);
	CacheTests(Support);
	BigCache(Support);
	Truncation(Support);
}

} // namespace IasJournaledFileCacheTest

#endif // IS_PROGRAM

// }}}

} // namespace UE::IoStore

/* vim: set noet foldlevel=1 : */
