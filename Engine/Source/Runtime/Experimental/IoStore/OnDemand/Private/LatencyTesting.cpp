// Copyright Epic Games, Inc. All Rights Reserved.

#include "LatencyTesting.h"

#include "Containers/StringConv.h"
#include "Containers/StringView.h"
#include "HAL/PlatformTime.h"
#include "IO/Http/Client.h"
#include "IO/IoStoreOnDemand.h"
#include "Logging/StructuredLog.h"
#include "Misc/StringBuilder.h"
#include "Templates/Function.h"

namespace UE::IoStore
{

extern int32 GIaxHttpVersion;

void LatencyTest(FAnsiStringView InUrl, FAnsiStringView InPath, uint32 TimeOutMs, TArrayView<int32> OutResults)
{
	using namespace HTTP;

	FConnectionPool::FParams PoolParams;
	PoolParams.SetHostFromUrl(InUrl);
	PoolParams.ConnectionCount = 1;
	PoolParams.HttpVersion = (GIaxHttpVersion == 2) ? EHttpVersion::Two : EHttpVersion::One;
	if (PoolParams.HttpVersion == EHttpVersion::Two)
	{
		PoolParams.VerifyCert = FCertRoots::Default();
	}
	
	FConnectionPool Pool(PoolParams);
	Pool.Resolve();

	TAnsiStringBuilder<512> ConnectionDesc;
	Pool.Describe(ConnectionDesc);
	UE_LOGFMT(LogIas, Log, "Testing endpoint {Url}", ConnectionDesc.ToString());

	TAnsiStringBuilder<256> AnsiPath;
	if (!InPath.StartsWith(TEXT('/')))
	{
		AnsiPath << '/';
	}
	AnsiPath << InPath;

	FEventLoop Loop;
	Loop.SetFailTimeout(TimeOutMs);

	for (int32& Result : OutResults)
	{
		bool Ok = false;

		FRequest Request = Loop.Request("HEAD", AnsiPath, Pool);
		Loop.Send(MoveTemp(Request), [&](const FTicketStatus& Status)
			{
				if (Status.GetId() == FTicketStatus::EId::Error)
				{
					FTicketStatus::FError Error = Status.GetError();
					UE_LOGFMT(LogIas, Warning, "LatencyTest Error: 'HEAD {Url}{Path}' {ErrorReason} ({ErrorCode})", InUrl, AnsiPath, Error.Reason, Error.Code);
					return;
				}
				else if (Status.GetId() != FTicketStatus::EId::Response)
				{
					return;
				}

				const FResponse& Response = Status.GetResponse();
				Ok = (Response.GetStatus() == EStatusCodeClass::Successful);

				if (Response.GetStatus() == EStatusCodeClass::Successful)
				{
					Ok = true;
				}
				else
				{
					UE_LOGFMT(LogIas, Warning, "LatencyTest Failed: 'HEAD {Url}{Path}' HTTP response ({ResponseCode})", InUrl, AnsiPath, Response.GetStatusCode());
				}
			});

		const double StartTime = FPlatformTime::Seconds();
		
		while (Loop.Tick(TimeOutMs) != 0)
		{
			// Do nothing, we want this loop to be as tight as possible.
		}

		const int32 TimeMs = FMath::RoundToInt32((FPlatformTime::Seconds() - StartTime) * 1000.0);
		Result = Ok ? TimeMs : -1;
	}
}

bool ConnectionTest(FAnsiStringView Url, FAnsiStringView Path, uint32 TimeoutMs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IasBackend::ConnectionTest);

	int32 Results[4] = {};
	LatencyTest(Url, Path, TimeoutMs, MakeArrayView(Results));

	if (Results[0] >= 0 || Results[1] >= 0 || Results[2] >= 0 || Results[3] >= 0)
	{
#if !UE_BUILD_SHIPPING
		UE_LOGFMT(LogIas, Display, "Endpoint '{Url}' latency test (ms): {Result0} {Result1} {Result2} {Result3}",
			Url, Results[0], Results[1], Results[2], Results[3]);
#endif // !UE_BUILD_SHIPPING

		return true;
	}
	else
	{
		return false;
	}
}

int32 ConnectionTest(TConstArrayView<FAnsiString> Urls, FAnsiStringView Path, uint32 TimeoutMs, std::atomic_bool& bCancel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(IasBackend::ConnectionTest);

	for (int32 Idx = 0; Idx < Urls.Num() && !bCancel.load(std::memory_order_relaxed); ++Idx)
	{
		int32 LatencyMs = -1;
		LatencyTest(Urls[Idx], Path, TimeoutMs, MakeArrayView(&LatencyMs, 1));
		if (LatencyMs >= 0)
		{
			return Idx;
		}
	}

	return INDEX_NONE;
}

} // namespace UE::IoStore
