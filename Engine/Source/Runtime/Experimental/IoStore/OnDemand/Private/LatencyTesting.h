// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"

namespace UE::IoStore
{
/**
 * Tests a single url.
 * 
 * @param Url			The url to be tested
 * @param Path			The path on the CDN to be tested
 * @param TimeoutMs		How long to wait for a response (in ms) before timing out
 * @param OutResults	A pre-sized array to hold the results. One latency test will be made per element in this container.
 */
void LatencyTest(FAnsiStringView Url, FAnsiStringView Path, uint32 TimeoutMs, TArrayView<int32> OutResults);

/**
 * An easy way to test if a valid connection can be made to a CDN or not. Instead of returning the latency to the CDN this
 * function returns if the CDN can be contacted at all.
 * 
 * @param Url			The url to be tested
 * @param Path			The path on the CDN to be tested
 * @param TimeoutMs		How long to wait for a response (in ms) before timing out
 * 
 * @return True if the CDN can be reached and false if it cannot.
 */
bool ConnectionTest(FAnsiStringView Url, FAnsiStringView Path, uint32 TimeoutMs);

/**
 * Runs latency tests on a list of urls to find the first CDN that can be reached.
 * 
 * @param Urls			A list of urls that should be tested
 * @param Path			The path on the CDN(s) to be tested
 * @param TimeoutMs		How long to wait for a response (in ms) before timing out
 * @param bCancel		Provides a way for the caller to cancel the tests. When this bool is set to true the test will
 *						attempt to exit as fast as possible, usually when the next latency test has completed.
 * @return				The index of the CDN that was successfully reached. INDEX_NONE will be returned if no CDN could be reached.
 */
int32 ConnectionTest(TConstArrayView<FAnsiString> Urls, FAnsiStringView Path, uint32 TimeoutMs, std::atomic_bool& bCancel);

} // namespace UE::IoStore
