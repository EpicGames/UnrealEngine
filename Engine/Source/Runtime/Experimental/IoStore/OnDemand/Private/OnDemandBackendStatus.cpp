// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnDemandBackendStatus.h"

#include "IO/IoStoreOnDemand.h"
#include "Logging/StructuredLog.h"

namespace UE::IoStore
{

extern bool GIasHttpEnabled;
extern bool GIasHttpOptionalBulkDataEnabled;

bool FBackendStatus::IsHttpEnabled(EIoChunkType ChunkType) const
{
	const uint8 CurrentFlags = Flags.load(std::memory_order_relaxed);
	return IsHttpEnabled(CurrentFlags) &&
		(ChunkType != EIoChunkType::OptionalBulkData ||
		((CurrentFlags & uint8(EFlags::HttpBulkOptionalDisabled)) == 0 && GIasHttpOptionalBulkDataEnabled));
}

void FBackendStatus::ToString(FStringBuilderBase& Builder) const
{
	if (IsCacheEnabled())
	{
		Builder << TEXT("Caching - Enabled");
	}
	else
	{
		Builder << TEXT("Caching - Disabled");
	}

	Builder << TEXT(" | ");

	if (IsHttpEnabled())
	{
		Builder << TEXT("Http - Enabled");
	}
	else
	{
		Builder << TEXT("Http - Disabled");
	}

	Builder << TEXT(" | ");

	if (IsHttpEnabled(EIoChunkType::OptionalBulkData))
	{
		Builder << TEXT("Optional Mips Enabled");
	}
	else
	{
		Builder << TEXT("Optional Mips Disabled");
	}
}

bool FBackendStatus::IsHttpEnabled(uint8 FlagsToTest)
{
	constexpr uint8 HttpFlags = uint8(EFlags::HttpEnabled);
	return ((FlagsToTest & HttpFlags) == uint8(EFlags::HttpEnabled)) && GIasHttpEnabled;
}

void FBackendStatus::AddOrRemoveFlags(EFlags FlagsToAddOrRemove, bool bValue, const TCHAR* DebugText)
{
	const uint8 PrevFlags = AddOrRemoveFlags(FlagsToAddOrRemove, bValue);
	
	TStringBuilder<128> Sb;

	Sb.Append(DebugText)
		<< TEXT(" '");

	Sb.Append(bValue ? TEXT("true") : TEXT("false"))
		<< TEXT("', backend status '(")
		<< EFlags(PrevFlags)
		<< TEXT(") -> (")
		<< EFlags(Flags.load(std::memory_order_relaxed))
		<< TEXT(")'");

	UE_LOGFMT(LogIas, Log, "{Message}", Sb);
}

FStringBuilderBase& operator<<(FStringBuilderBase& Sb, FBackendStatus::EFlags StatusFlags)
{
	if (StatusFlags == FBackendStatus::EFlags::None)
	{
		Sb.Append(TEXT("None"));
		return Sb;
	}

	bool bFirst = true;
	auto AppendIf = [StatusFlags, &Sb, &bFirst](FBackendStatus::EFlags Contains, const TCHAR* Str)
		{
			if (uint8(StatusFlags) & uint8(Contains))
			{
				if (!bFirst)
				{
					Sb.AppendChar(TEXT('|'));
				}
				Sb.Append(Str);
				bFirst = false;
			}
		};

	AppendIf(FBackendStatus::EFlags::CacheEnabled, TEXT("CacheEnabled"));
	AppendIf(FBackendStatus::EFlags::HttpEnabled, TEXT("HttpEnabled"));
	AppendIf(FBackendStatus::EFlags::HttpBulkOptionalDisabled, TEXT("HttpBulkOptionalDisabled"));

	return Sb;
}

} // namespace UE::IoStore