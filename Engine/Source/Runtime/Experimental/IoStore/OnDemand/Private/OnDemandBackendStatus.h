// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "IO/IoChunkId.h"
#include "Misc/StringBuilder.h"

#include <atomic>

namespace UE::IoStore
{

class FBackendStatus
{
public:

	enum class EFlags : uint8
	{
		None						= 0,
		CacheEnabled				= 1 << 0,
		HttpEnabled					= 1 << 1,
		HttpBulkOptionalDisabled	= 1 << 2,

		// When adding new values here, remember to update operator<<(FStringBuilderBase& Sb, EFlags StatusFlags) in the cpp!
	};

	bool IsHttpEnabled() const
	{
		return IsHttpEnabled(Flags.load(std::memory_order_relaxed));
	}

	bool IsHttpEnabled(EIoChunkType ChunkType) const;

	bool IsCacheEnabled() const
	{
		return HasAnyFlags(EFlags::CacheEnabled);
	}

	bool IsCacheWriteable() const
	{
		const uint8 CurrentFlags = Flags.load(std::memory_order_relaxed);
		return (CurrentFlags & uint8(EFlags::CacheEnabled)) && IsHttpEnabled(CurrentFlags);
	}

	bool IsCacheReadOnly() const
	{
		const uint8 CurrentFlags = Flags.load(std::memory_order_relaxed);
		return (CurrentFlags & uint8(EFlags::CacheEnabled)) && !IsHttpEnabled(CurrentFlags);
	}

	void SetHttpEnabled(bool bEnabled)
	{
		AddOrRemoveFlags(EFlags::HttpEnabled, bEnabled, TEXT("HTTP streaming enabled"));
		FGenericCrashContext::SetEngineData(TEXT("IAS.Enabled"), bEnabled ? TEXT("true") : TEXT("false"));
	}

	void SetHttpOptionalBulkEnabled(bool bEnabled)
	{
		AddOrRemoveFlags(EFlags::HttpBulkOptionalDisabled, bEnabled == false, TEXT("HTTP streaming of optional bulk data disabled"));
	}

	void SetCacheEnabled(bool bEnabled)
	{
		AddOrRemoveFlags(EFlags::CacheEnabled, bEnabled, TEXT("Cache enabled"));
	}

	void ToString(FStringBuilderBase& Builder) const;

private:

	static bool IsHttpEnabled(uint8 FlagsToTest);

	bool HasAnyFlags(uint8 Contains) const
	{
		return (Flags.load(std::memory_order_relaxed) & Contains) != 0;
	}

	bool HasAnyFlags(EFlags Contains) const
	{
		return HasAnyFlags(uint8(Contains));
	}

	uint8 AddFlags(EFlags FlagsToAdd)
	{
		return Flags.fetch_or(uint8(FlagsToAdd));
	}

	uint8 RemoveFlags(EFlags FlagsToRemove)
	{
		return Flags.fetch_and(static_cast<uint8>(~uint8(FlagsToRemove)));
	}

	uint8 AddOrRemoveFlags(EFlags FlagsToAddOrRemove, bool bValue)
	{
		return bValue ? AddFlags(FlagsToAddOrRemove) : RemoveFlags(FlagsToAddOrRemove);
	}

	void AddOrRemoveFlags(EFlags FlagsToAddOrRemove, bool bValue, const TCHAR* DebugText);

	friend FStringBuilderBase& operator<<(FStringBuilderBase& Sb, EFlags StatusFlags);

	std::atomic<uint8> Flags{ 0 };
};

} // namespace UE::IoStore
