// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnDemandConfig.h"

#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "IO/IoStatus.h"
#include "IO/IoStoreOnDemand.h"
#include "IO/IoStoreOnDemandInternals.h"
#include "IO/Serialization/OnDemandContainerToc.h"
#include "IasCache.h"
#include "Misc/Base64.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CoreMisc.h"
#include "Misc/EncryptionKeyManager.h"
#include "Misc/Fork.h"
#include "Misc/Parse.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "OnDemandInstallCache.h"
#include "String/LexFromString.h"

#if WITH_IOSTORE_ONDEMAND_TESTS
#include "TestHarness.h"
#include "TestMacros/Assertions.h"
#include <catch2/generators/catch_generators.hpp>
#endif

#ifndef UE_ALL_ONDEMAND_TOCS_WILDCARD
#define UE_ALL_ONDEMAND_TOCS_WILDCARD ""
#endif

#ifndef UE_MOUNT_STARTUP_ONDEMAND_TOCS_WILDCARD
#define UE_MOUNT_STARTUP_ONDEMAND_TOCS_WILDCARD UE_ALL_ONDEMAND_TOCS_WILDCARD
#endif

namespace UE::IoStore::Config
{

////////////////////////////////////////////////////////////////////////////////
bool GIoStoreOnDemandInstallCacheEnabled = true;
static FAutoConsoleVariableRef CVar_IoStoreOnDemandInstallCacheEnabled(
	TEXT("iostore.OnDemandInstallCacheEnabled"),
	GIoStoreOnDemandInstallCacheEnabled,
	TEXT("Whether the on-demand install cache is enabled."),
	ECVF_ReadOnly
);

////////////////////////////////////////////////////////////////////////////////
static bool ParseEncryptionKeyParam(const FString& Param, FGuid& OutKeyGuid, FAES::FAESKey& OutKey)
{
	TArray<FString> Tokens;
	Param.ParseIntoArray(Tokens, TEXT(":"), true);

	if (Tokens.Num() == 2)
	{
		TArray<uint8> KeyBytes;
		if (FGuid::Parse(Tokens[0], OutKeyGuid) && FBase64::Decode(Tokens[1], KeyBytes))
		{
			if (OutKeyGuid != FGuid() && KeyBytes.Num() == FAES::FAESKey::KeySize)
			{
				FMemory::Memcpy(OutKey.Key, KeyBytes.GetData(), FAES::FAESKey::KeySize);
				return true;
			}
		}
	}
	
	return false;
}

////////////////////////////////////////////////////////////////////////////////
static bool ApplyEncryptionKeyFromString(const FString& GuidKeyPair)
{
	FGuid KeyGuid;
	FAES::FAESKey Key;

	if (ParseEncryptionKeyParam(GuidKeyPair, KeyGuid, Key))
	{
		// TODO: PAK and I/O store should share key manager
		FEncryptionKeyManager::Get().AddKey(KeyGuid, Key);
		FCoreDelegates::GetRegisterEncryptionKeyMulticastDelegate().Broadcast(KeyGuid, Key);

		return true;
	}
	else
	{
		return false;
	}
}

////////////////////////////////////////////////////////////////////////////////
int64 ParseSizeParam(FStringView Value)
{
	Value = Value.TrimStartAndEnd();

	int64 Size = 0;
	LexFromString(Size, Value);

	if (Size > 0)
	{
		// Now we have a valid integer value we should check to see if any size extension was in the string

		if (Value.EndsWith(TEXT("GB")) || Value.EndsWith(TEXT("GiB")))
		{
			return Size << 30;
		}

		if (Value.EndsWith(TEXT("MB")) || Value.EndsWith(TEXT("MiB")))
		{
			return Size << 20;
		}

		if (Value.EndsWith(TEXT("KB")) || Value.EndsWith(TEXT("KiB")))
		{
			return Size << 10;
		}
	}
	return Size;
}

////////////////////////////////////////////////////////////////////////////////
int64 ParseSizeParam(const TCHAR* CommandLine, const TCHAR* Param)
{
	FString ParamValue;
	if (!FParse::Value(CommandLine, Param, ParamValue))
	{
		return -1;
	}

	return ParseSizeParam(ParamValue);
}

///////////////////////////////////////////////////////////////////////////////
static void FindTocFiles(const FString& WildCard, TArray<FString>& OutFiles)
{
	if (WildCard.IsEmpty())
	{
		return;
	}

	TStringBuilder<128> Sb;
	Sb << FPaths::ProjectContentDir() << TEXT("Paks/") << WildCard;

	TArray<FString> FileNames;
	IFileManager::Get().FindFiles(FileNames, Sb.ToString(), true, false);

	const FStringView TocExt = FStringView(UE::IoStore::Serialization::FOnDemandFileExt::Toc);
	for (const FString& FileName : FileNames)
	{
		FStringView Ext = FPathViews::GetExtension(FileName, UE::Paths::EFlags::IncludeDot);
		if (Ext != TocExt)
		{
			continue;
		}

		Sb.Reset();
		Sb << FPaths::ProjectContentDir() << TEXT("Paks/") << FileName;
		OutFiles.Add(Sb.ToString());
	}
}

////////////////////////////////////////////////////////////////////////////////
FIasCacheConfig GetStreamingCacheConfig(const TCHAR* CommandLine)
{
	FIasCacheConfig Ret;

	// Fetch values from .ini files
	auto GetConfigIntImpl = [CommandLine] (const TCHAR* ConfigKey, const TCHAR* ParamName, auto& Out)
	{
		int64 Value = -1;
		if (FString Temp; GConfig->GetString(TEXT("Ias"), ConfigKey, Temp, GEngineIni))
		{
			Value = ParseSizeParam(Temp);
		}
#if !UE_BUILD_SHIPPING
		if (int64 Override = ParseSizeParam(CommandLine, ParamName); Override >= 0)
		{
			Value = Override;
		}
#endif

		if (Value >= 0)
		{
			Out = decltype(Out)(Value);
		}

		return true;
	};

#define GetConfigInt(Name, Dest) \
	do { GetConfigIntImpl(TEXT("FileCache.") Name, TEXT("Ias.FileCache.") Name TEXT("="), Dest); } while (false)
	GetConfigInt(TEXT("WritePeriodSeconds"),	Ret.WriteRate.Seconds);
	GetConfigInt(TEXT("WriteOpsPerPeriod"),		Ret.WriteRate.Ops);
	GetConfigInt(TEXT("WriteBytesPerPeriod"),	Ret.WriteRate.Allowance);
	GetConfigInt(TEXT("DiskQuota"),				Ret.DiskQuota);
	GetConfigInt(TEXT("MemoryQuota"),			Ret.MemoryQuota);
	GetConfigInt(TEXT("JournalQuota"),			Ret.JournalQuota);
	GetConfigInt(TEXT("DemandThreshold"),		Ret.Demand.Threshold);
	GetConfigInt(TEXT("DemandBoost"),			Ret.Demand.Boost);
	GetConfigInt(TEXT("DemandSuperBoost"),		Ret.Demand.SuperBoost);
#undef GetConfigInt

#if !UE_BUILD_SHIPPING
	if (FParse::Param(CommandLine, TEXT("Ias.DropCache")))
	{
		Ret.DropCache = true;
	}
	if (FParse::Param(CommandLine, TEXT("Ias.NoCache")))
	{
		Ret.DiskQuota = 0;
	}
#endif

	return Ret;
}

////////////////////////////////////////////////////////////////////////////////
TIoStatusOr<FOnDemandIoStoreConfig> TryParseConfig(const TCHAR* CommandLine)
{
	{
		FString EncryptionKey;
		if (FParse::Value(CommandLine, TEXT("Ias.EncryptionKey="), EncryptionKey))
		{
			if (!ApplyEncryptionKeyFromString(EncryptionKey))
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to parse encryption key from cmdline 'Ias.EncryptionKey'");
			}
		}
	}

	FOnDemandIoStoreConfig OutConfig;

	const FString GlobalTocPath = FString::Printf(TEXT("%sPaks/global%s"),
		*FPaths::ProjectContentDir(), UE::IoStore::Serialization::FOnDemandFileExt::Toc);

	//NOTE: Look for the global on-demand TOC file (Legacy)
	if (IFileManager::Get().FileExists(*GlobalTocPath))
	{
		OutConfig.StartupMountArgs.Add(FOnDemandMountArgs
		{
			.MountId	= GlobalTocPath,
			.FilePath	= GlobalTocPath
		});
	}
	else
	{
		const FString StartupWildCard = TEXT(UE_MOUNT_STARTUP_ONDEMAND_TOCS_WILDCARD);
		TArray<FString> FilePaths;
		FindTocFiles(StartupWildCard, FilePaths);

		for (const FString& FilePath : FilePaths)
		{
			OutConfig.StartupMountArgs.Add(FOnDemandMountArgs
			{
				.MountId	= FilePath,
				.FilePath	= FilePath
			});
		}
	}

	//TODO: Read host groups from config

#if !UE_BUILD_SHIPPING
	FString HostsOverride;
	if (FParse::Value(CommandLine, TEXT("Iax.DefaultHostGroup="), HostsOverride))
	{
		FOnDemandHostGroupConfig& HostConfig = OutConfig.HostConfigs.AddDefaulted_GetRef();
		HostConfig.HostGroupName = FOnDemandHostGroup::DefaultName;
		TArray<FString> Urls;
		HostsOverride.ParseIntoArray(Urls, TEXT(","), true);
		for (FString& Url : Urls)
		{
			Url.TrimStartAndEndInline();
			if (!Url.StartsWith(TEXT("http")))
			{
				// Default to HTTPS if no scheme specified
				Url = TEXT("https://") + Url;
			}
			HostConfig.Urls.Add(*Url);
		}
	}
	else if (FParse::Param(CommandLine, TEXT("Iax.Zen")))
	{
		FOnDemandHostGroupConfig& HostConfig = OutConfig.HostConfigs.AddDefaulted_GetRef();
		HostConfig.HostGroupName = FOnDemandHostGroup::DefaultName;
		HostConfig.Urls.Add(TEXT("http://localhost:8558"));
	}
#endif

	if (GConfig)
	{
		GConfig->GetBool(TEXT("Ias"), TEXT("UsePerContainerTocs"), OutConfig.bUsePerContainerTocs, GEngineIni);
	}

#if !UE_BUILD_SHIPPING
	if (FParse::Param(CommandLine, TEXT("Ias.UsePerContainerTocs")))
	{
		OutConfig.bUsePerContainerTocs = true;
	}
#endif

	return OutConfig;
}

///////////////////////////////////////////////////////////////////////////////
FString GetInstallCacheDirectory(const TCHAR* CommandLine)
{
	FString DirName;

	if (IsRunningDedicatedServer())
	{
		if (!FForkProcessHelper::IsForkRequested())
		{
			DirName = TEXT("InstallCacheServer");
		}
		else
		{
			if (!FForkProcessHelper::IsForkedChildProcess())
			{
				UE_LOGF(LogIoStoreOnDemand, Fatal, "Attempting to create install cache before forking!");
			}

			FString CommandLineDir;
			bool bUsePathFromCommandLine = FParse::Value(CommandLine, TEXT("ServerIOInstallCacheDir="), CommandLineDir);
			if (bUsePathFromCommandLine)
			{
				FPaths::NormalizeDirectoryName(CommandLineDir);

				if (!FPaths::ValidatePath(CommandLineDir))
				{
					bUsePathFromCommandLine = false;
					UE_LOGF(LogIoStoreOnDemand, Error, "Invalid ServerIOInstallCacheDir from command line: %ls", *CommandLineDir);
				}
				else if (!FPaths::CollapseRelativeDirectories(CommandLineDir))
				{
					bUsePathFromCommandLine = false;
					UE_LOGF(LogIoStoreOnDemand, Error, "ServerIOInstallCacheDir from command line would end up outside of the PersistentDownloadDirectory: %ls", *CommandLineDir);
				}
				else if (!FPaths::IsRelative(CommandLineDir))
				{
					bUsePathFromCommandLine = false;
					UE_LOGF(LogIoStoreOnDemand, Error, "ServerIOInstallCacheDir from command line is not relative: %ls", *CommandLineDir);
				}

				if (bUsePathFromCommandLine)
				{
					return FPaths::ProjectPersistentDownloadDir() / CommandLineDir;
				}
			}

			DirName = FString::Printf(TEXT("InstallCacheServer-%u"), FPlatformProcess::GetCurrentProcessId());
		}
	}
#if WITH_EDITOR
	else if (GIsEditor)
	{
		DirName = TEXT("InstallCacheEditor");
	}
#endif //if WITH_EDITOR
	else
	{
		DirName = TEXT("InstallCache");
	}

	return FPaths::ProjectPersistentDownloadDir() / TEXT("IoStore") / DirName;
}

///////////////////////////////////////////////////////////////////////////////
TIoStatusOr<FOnDemandInstallCacheConfig> TryParseInstallCacheConfig(const TCHAR* CommandLine)
{
	bool bUseInstallCache = GIoStoreOnDemandInstallCacheEnabled;
#if !UE_BUILD_SHIPPING
	bUseInstallCache = FParse::Param(CommandLine, TEXT("NoIAD")) == false;
#endif
	if (bUseInstallCache == false)
	{
		FIoStatus Status = FIoStatusBuilder(EIoErrorCode::Disabled) << TEXT("Disabled");
		return Status;
	}

	if (FPaths::HasProjectPersistentDownloadDir() == false)
	{
		FIoStatus Status = FIoStatusBuilder(EIoErrorCode::Disabled) << TEXT("Persistent storage not configured");
		return Status;
	}

	FOnDemandInstallCacheConfig OutConfig;

	const FString ICSName = [CommandLine]
	{
		if (FString ParamValue; FParse::Value(CommandLine, TEXT("-Iad.IndexedCacheStorage.IndexName="), ParamValue))
		{
			return ParamValue;
		}
		else if (FString ValueStr; GConfig->GetString(TEXT("OnDemandInstall"), TEXT("IndexedCacheStorage.IndexName"), ValueStr, GEngineIni))
		{
			return ValueStr;
		}
		return FString();
	}();

	if (ICSName.IsEmpty() == false)
	{
		OutConfig.IndexedCacheStorageName = ICSName;
	}

	const int64 JournalMaxSize = [CommandLine]
	{
		if (FString ParamValue; FParse::Value(CommandLine, TEXT("-Iad.Journal.MaxSize="), ParamValue))
		{
			return Config::ParseSizeParam(ParamValue);
		}
		else if (FString ValueStr; GConfig->GetString(TEXT("OnDemandInstall"), TEXT("Journal.MaxSize"), ValueStr, GEngineIni))
		{
			return Config::ParseSizeParam(ValueStr);
		}
		return int64(-1);
	}();

	if (JournalMaxSize > 0)
	{
		OutConfig.JournalMaxSize = JournalMaxSize;
	}

#if !UE_BUILD_SHIPPING
	OutConfig.bDropCache = FParse::Param(CommandLine, TEXT("Iad.DropCache"));
#endif

	const TCHAR* ConfigPrefixByType[EOnDemandInstallCasType::Count];
	ConfigPrefixByType[EOnDemandInstallCasType::General] = TEXT("FileCache");
	ConfigPrefixByType[EOnDemandInstallCasType::MMap] = TEXT("MMapCache");
	
	auto TryParseCasConfig = [&ConfigPrefixByType, CommandLine](FOnDemandInstallCasConfig& OutConfig, EOnDemandInstallCasType CasType)
	{
		const TCHAR* ConfigPrefix = ConfigPrefixByType[CasType];

		const int64 DiskQuota = [ConfigPrefix, CommandLine]
		{
			const FString Param = FString::Printf(TEXT("-Iad.%s.DiskQuota="), ConfigPrefix);
			const FString ConfigKey = FString::Printf(TEXT("%s.DiskQuota"), ConfigPrefix);

			if (FString ParamValue; FParse::Value(CommandLine, *Param, ParamValue))
			{
				return Config::ParseSizeParam(ParamValue);
			}
			else if (FString ValueStr; GConfig->GetString(TEXT("OnDemandInstall"), *ConfigKey, ValueStr, GEngineIni))
			{
				return Config::ParseSizeParam(ValueStr);
			}
			return int64(-1);
		}();

		if (DiskQuota > 0)
		{
			OutConfig.DiskQuota = DiskQuota;
		}

		const int64 MinBlockSize = [ConfigPrefix, CommandLine]
		{
			const FString Param = FString::Printf(TEXT("-Iad.%s.MinBlockSize="), ConfigPrefix);
			const FString ConfigKey = FString::Printf(TEXT("%s.MinBlockSize"), ConfigPrefix);

			if (FString ParamValue; FParse::Value(CommandLine, *Param, ParamValue))
			{
				return Config::ParseSizeParam(ParamValue);
			}
			else if (FString ValueStr; GConfig->GetString(TEXT("OnDemandInstall"), *ConfigKey, ValueStr, GEngineIni))
			{
				return Config::ParseSizeParam(ValueStr);
			}
			return int64(-1);
		}();

		if (MinBlockSize > 0)
		{
			OutConfig.MinBlockSize = MinBlockSize;
		}

		const int64 MaxBlockSize = [ConfigPrefix, CommandLine]
		{
			const FString Param = FString::Printf(TEXT("-Iad.%s.MaxBlockSize="), ConfigPrefix);
			const FString ConfigKey = FString::Printf(TEXT("%s.MaxBlockSize"), ConfigPrefix);

			if (FString ParamValue; FParse::Value(CommandLine, *Param, ParamValue))
			{
				return Config::ParseSizeParam(ParamValue);
			}
			else if (FString ValueStr; GConfig->GetString(TEXT("OnDemandInstall"), *ConfigKey, ValueStr, GEngineIni))
			{
				return Config::ParseSizeParam(ValueStr);
			}
			return int64(-1);
		}();

		if (MaxBlockSize > 0)
		{
			OutConfig.MaxBlockSize = MaxBlockSize;
		}

		const double LastAccessGranularitySeconds = [ConfigPrefix, CommandLine]
		{
			const FString Param = FString::Printf(TEXT("-Iad.%s.LastAccessGranularitySeconds="), ConfigPrefix);
			const FString ConfigKey = FString::Printf(TEXT("%s.LastAccessGranularitySeconds"), ConfigPrefix);

			if (double ParamValue; FParse::Value(CommandLine, *Param, ParamValue))
			{
				return ParamValue;
			}
			else if (GConfig->GetDouble(TEXT("OnDemandInstall"), *ConfigKey, ParamValue, GEngineIni))
			{
				return ParamValue;
			}
			return double(-1.0);
		}();

		if (LastAccessGranularitySeconds >= 0)
		{
			OutConfig.LastAccessGranularitySeconds = LastAccessGranularitySeconds;
		}

		const uint32 EagerDefragBlockCount = [ConfigPrefix, CommandLine]() -> uint32
		{
			const FString Param = FString::Printf(TEXT("-Iad.%s.EagerDefragBlockCount="), ConfigPrefix);
			const FString ConfigKey = FString::Printf(TEXT("%s.EagerDefragBlockCount"), ConfigPrefix);

			if (FString ParamValue; FParse::Value(CommandLine, *Param, ParamValue))
			{
				uint32 Count = 0;
				LexFromString(Count, ParamValue);
				return Count;
			}
			else if (int32 Count; GConfig->GetInt(TEXT("OnDemandInstall"), *ConfigKey, Count, GEngineIni))
			{
				if (Count > 0)
				{
					return static_cast<uint32>(Count);
				}
			}
			return 0u;
		}();

		if (EagerDefragBlockCount > 0)
		{
			OutConfig.EagerDefragBlockCount = EagerDefragBlockCount;
		}
	};

	// Always make the general CAS
	FOnDemandInstallCasConfig& GeneralCasConfig = OutConfig.CasConfig.Emplace_GetRef();
	GeneralCasConfig.Type = EOnDemandInstallCasType::General;
	GeneralCasConfig.CasSubdirectory = TEXT("blocks");

	TryParseCasConfig(GeneralCasConfig, GeneralCasConfig.Type);

	// Conditionally make MMap CAS
	bool bUseMMapCache = false;
	GConfig->GetBool(TEXT("OnDemandInstall"), TEXT("MMapCache.Enabled"), bUseMMapCache, GEngineIni);
	if (bUseMMapCache)
	{
		constexpr bool bPlatformSupportsMmap = FPlatformProperties::SupportsMemoryMappedFiles();
		UE_CLOGF(!bPlatformSupportsMmap, LogIoStoreOnDemand, Error, "MMap cache cannot be enabled on a platform that does not support memory mapping!");

		if constexpr (bPlatformSupportsMmap)
		{
			ensure(bPlatformSupportsMmap);
			FOnDemandInstallCasConfig& CasConfig = OutConfig.CasConfig.Emplace_GetRef();
			CasConfig.Type = EOnDemandInstallCasType::MMap;
			CasConfig.CasSubdirectory = TEXT("mmap");
			CasConfig.EagerDefragBlockCount = 2; // Default for mmap because long lived mmap handles can prevent the usual lazy purge/defrag

			TryParseCasConfig(CasConfig, CasConfig.Type);
		}
	}

	return OutConfig;
}

#if WITH_IOSTORE_ONDEMAND_TESTS

TEST_CASE("IoStore::OnDemand::ParseSizeParam", "[IoStoreOnDemand][Ias]")
{
	SECTION("ParseFromString")
	{
		CHECK(ParseSizeParam(TEXT("0")) == 0);
		CHECK(ParseSizeParam(TEXT("1")) == 1);
		CHECK(ParseSizeParam(TEXT("1")) != 2);

		CHECK(ParseSizeParam(TEXT("1024")) == 1024);
		CHECK(ParseSizeParam(TEXT("1KB")) == 1024);
		CHECK(ParseSizeParam(TEXT("1KiB")) == 1024);
		CHECK(ParseSizeParam(TEXT("2KB")) != 1024);
		CHECK(ParseSizeParam(TEXT("2KiB")) != 1024);

		CHECK(ParseSizeParam(TEXT("1048576")) == 1048576);
		CHECK(ParseSizeParam(TEXT("1MB")) == 1048576);
		CHECK(ParseSizeParam(TEXT("1MiB")) == 1048576);
		CHECK(ParseSizeParam(TEXT("2MB")) != 1048576);
		CHECK(ParseSizeParam(TEXT("2MiB")) != 1048576);

		CHECK(ParseSizeParam(TEXT("1073741824")) == 1073741824);
		CHECK(ParseSizeParam(TEXT("1GB")) == 1073741824);
		CHECK(ParseSizeParam(TEXT("1GiB")) == 1073741824);
		CHECK(ParseSizeParam(TEXT("2GB")) != 1073741824);
		CHECK(ParseSizeParam(TEXT("2GiB")) != 1073741824);

		CHECK(ParseSizeParam(TEXT("Invalid")) == 0);
	}

	SECTION("ParseFromCommandLine")
	{
		const TCHAR* CommandLine = TEXT("-BasicTestValue=1092387456 -SmallTestValue=1KB -MediumTestValue=2MiB -LargeTestValue=3GiB");

		CHECK(ParseSizeParam(CommandLine, TEXT("MissingValue=")) == -1);
		CHECK(ParseSizeParam(CommandLine, TEXT("BasicTestValue=")) == 1092387456);
		CHECK(ParseSizeParam(CommandLine, TEXT("SmallTestValue=")) == 1024);
		CHECK(ParseSizeParam(CommandLine, TEXT("MediumTestValue=")) == 2097152);
		CHECK(ParseSizeParam(CommandLine, TEXT("LargeTestValue=")) == 3221225472);
	}
}

#endif // WITH_IOSTORE_ONDEMAND_TESTS

} // namespace UE::IoStore
