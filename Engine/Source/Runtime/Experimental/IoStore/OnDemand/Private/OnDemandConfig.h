// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/AnsiString.h"
#include "Containers/StringView.h"
#include "IO/IoStatus.h"
#include "UObject/NameTypes.h"

namespace UE::IoStore
{

struct FIasCacheConfig;
struct FOnDemandInstallCacheConfig;
struct FOnDemandMountArgs;

struct FOnDemandHostGroupConfig
{
	FName				HostGroupName;
	TArray<FAnsiString> Urls;
};

struct FOnDemandIoStoreConfig
{
	TArray<FOnDemandHostGroupConfig>	HostConfigs;
	TArray<FOnDemandMountArgs>			StartupMountArgs;
	bool								bUsePerContainerTocs = false;
};

namespace Config
{

/*
 * Attempts to convert the provided string to an integer.
 * Unlike LexFromString this method supports postfixing the number with one of the following units of measurement:
 * KB/KiB
 * MB/MiB
 * GB/GiB
 * For example "1KB" would return 1024 or "1MiB" would return 1048576.
 * Note that the SI units (KB) are treated as the IEC version (KiB).
 * 
 * @param	Value	The string to parse the integer from.
 * 
 * @return	The integer in the string, 0 if no valid integer was found.
 */
int64 ParseSizeParam(FStringView Value);

/*
 * Attempt to find the given Parameter in the provided CommandLine and return it's value as an integer.
 * Unlike using FParse::Value directly this method supports postfixing the number with one of the following
 * units of measurement:
 * KB/KiB
 * MB/MiB
 * GB/GiB
 * For example "-Param=1KB" would return 1024 or "-Param=1MiB" would return 1048576.
 * Note that the SI units (KB) are treated as the IEC version (KiB).
 * 
 * @param	CommandLine	The commandline to parse from.
 * @param	Param		The parameter on the commandline to parse.
 * 
 * @return	The parameter value as an integer, -1 if the parameter could not be found on the commandline
 *			and 0 if the parameter was not a valid integer.
 */
int64 ParseSizeParam(const TCHAR* CommandLine, const TCHAR* Param);

TIoStatusOr<FOnDemandIoStoreConfig> TryParseConfig(const TCHAR* CommandLine);

FIasCacheConfig GetStreamingCacheConfig(const TCHAR* CommandLine);

FString GetInstallCacheDirectory(const TCHAR* CommandLine);

TIoStatusOr<FOnDemandInstallCacheConfig> TryParseInstallCacheConfig(const TCHAR* CommandLine);

} // namespace UE::IoStore::Config

} // namespace UE::IoStore
