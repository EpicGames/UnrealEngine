// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnDemandContentInstallReplay.h"

#include "Async/UniqueLock.h"
#include "HAL/FileManager.h"

#if WITH_IOSTORE_ONDEMAND_TESTS
#include "TestHarness.h"
#include "TestMacros/Assertions.h"
#include <catch2/generators/catch_generators.hpp>
#endif

namespace UE::IoStore
{

////////////////////////////////////////////////////////////////////////////////
#if !UE_BUILD_SHIPPING
const FGuid FOnDemandContentInstallReplay::CustomVersionGuid(0x945E86CB, 0x453E4132, 0x9F036DF2, 0x0D9A930E);

const uint8 FOnDemandContentInstallReplay::FHeader::MagicSequence[16] = { 'R', 'E', 'P', 'L', 'A', 'Y', 'I', 'N', 'S', 'T', 'A', 'L', 'L', 'H', 'D' };
const uint8 FOnDemandContentInstallReplay::FFooter::MagicSequence[16] = { 'R', 'E', 'P', 'L', 'A', 'Y', 'I', 'N', 'S', 'T', 'A', 'L', 'L', 'F', 'T' };

bool FOnDemandContentInstallReplay::FHeader::IsValid() const
{
	if (FMemory::Memcmp(&Magic, &FHeader::MagicSequence, sizeof(FHeader::MagicSequence)) != 0)
	{
		return false;
	}

	if (static_cast<uint32>(Version) > static_cast<uint32>(EVersion::Latest))
	{
		return false;
	}

	return true;
}

bool FOnDemandContentInstallReplay::FFooter::IsValid() const
{
	return FMemory::Memcmp(Magic, FFooter::MagicSequence, sizeof(FFooter::MagicSequence)) == 0;
}

FArchive& operator<<(FArchive& Ar, FOnDemandContentInstallReplay::FContentHandleEventData& EventData)
{
	Ar << EventData.DebugName;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FOnDemandContentInstallReplay::FContentHandleDestroyedEventData& EventData)
{
	Ar << EventData.ContentHandleEventId;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FOnDemandContentInstallReplay::FInstallEventData& EventData)
{
	Ar << EventData.MountId;
	Ar << EventData.TagSets;
	Ar << EventData.PackageIds;
	Ar << EventData.ContentHandleEventId;
	Ar << EventData.Priority;
	Ar << EventData.Options;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FOnDemandContentInstallReplay::FCancelInstallEventData& EventData)
{
	Ar << EventData.InstallEventId;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FOnDemandContentInstallReplay::FUpdatePriorityEventData& EventData)
{
	Ar << EventData.InstallEventId;
	Ar << EventData.NewPriority;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FOnDemandContentInstallReplay::FInstallDestroyedEventData& EventData)
{
	Ar << EventData.InstallEventId;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FOnDemandContentInstallReplay::FEvent& Event)
{
	Ar << Event.TimeOffset;
	Ar << Event.EventData;

	return Ar;
}

#if WITH_IOSTORE_ONDEMAND_TESTS
struct FOnDemandContentInstallReplayEventDataEqualityVistor
{
	const FOnDemandContentInstallReplay::FEventVariant& EventData;

	bool operator()(const FEmptyVariantState& OtherData) const
	{
		CHECK(false);
		ensure(false);
		return true;
	}

	template<typename TVistedType>
	bool operator()(const TVistedType& OtherData) const
	{
		return EventData.Get<TVistedType>() == OtherData;
	}
};

bool FOnDemandContentInstallReplay::operator==(const FOnDemandContentInstallReplay& Other) const
{
	if (Events.Num() != Other.Events.Num())
	{
		return false;
	}

	for (int i = 0; const FEvent& Event : Events)
	{
		const FEvent& OtherEvent = Other.Events[i++];

		if (Event.TimeOffset != OtherEvent.TimeOffset)
		{
			return false;
		}

		if (Event.EventData.GetIndex() != OtherEvent.EventData.GetIndex())
		{
			return false;
		}

		FOnDemandContentInstallReplayEventDataEqualityVistor Vistor{ .EventData = Event.EventData };
		bool bEventDataResult = Visit(Vistor, OtherEvent.EventData);
		if (bEventDataResult == false)
		{
			return false;
		}
	}

	return true;
}
#endif // WITH_IOSTORE_ONDEMAND_TESTS

TResult<FOnDemandContentInstallReplay> FOnDemandContentInstallReplay::Load(const FString& Filename, int64* OutFileSize)
{
	using namespace UE::IoStore::OnDemand;

	IFileManager& Ifm = IFileManager::Get();

	TUniquePtr<FArchive> Ar(Ifm.CreateFileReader(*Filename));
	if (Ar.IsValid() == false)
	{
		return MakeError(InstallReplayLoadError());
	}

	FHeader Header;
	Ar->Serialize(reinterpret_cast<uint8*>(&Header), FHeader::Size());
	if (Ar->IsError() || Header.IsValid() == false)
	{
		return MakeError(InstallReplayLoadError());
	}

	Ar->SetCustomVersion(CustomVersionGuid, static_cast<int32>(Header.Version), "InstallReplayVersion");

	FOnDemandContentInstallReplay Replay;
	*Ar << Replay.Events;

	FFooter Footer;
	Ar->Serialize(reinterpret_cast<uint8*>(&Footer), FFooter::Size());
	if (Ar->IsError() || Footer.IsValid() == false)
	{
		return MakeError(InstallReplayLoadError());
	}

	if (OutFileSize != nullptr)
	{
		*OutFileSize = Ar->Tell();
	}

	return MakeValue(MoveTemp(Replay));
}

TResult<int64> FOnDemandContentInstallReplay::Save(const FOnDemandContentInstallReplay& Replay, const FString& Filename)
{
	using namespace UE::IoStore::OnDemand;

	IFileManager& Ifm = IFileManager::Get();

	TUniquePtr<FArchive> Ar(Ifm.CreateFileWriter(*Filename));
	if (Ar.IsValid() == false)
	{
		return MakeError(InstallReplaySaveError());
	}

	FHeader Header;
	FMemory::Memcpy(&Header.Magic, &FHeader::MagicSequence, sizeof(FHeader::MagicSequence));
	Header.Version = EVersion::Latest;

	Ar->SetCustomVersion(CustomVersionGuid, static_cast<int32>(Header.Version), "InstallReplayVersion");

	Ar->Serialize(reinterpret_cast<uint8*>(&Header), FHeader::Size());
	if (Ar->IsError())
	{
		const uint32 LastError = FPlatformMisc::GetLastError();
		Ar.Reset();
		Ifm.Delete(*Filename);
		return MakeError(InstallReplaySaveError());
	}

	FOnDemandContentInstallReplay& NonConst = const_cast<FOnDemandContentInstallReplay&>(Replay);
	*Ar << NonConst.Events;

	if (Ar->IsError())
	{
		Ar.Reset();
		Ifm.Delete(*Filename);
		return MakeError(InstallReplaySaveError());
	}

	FFooter Footer;
	FMemory::Memcpy(&Footer.Magic, &FFooter::MagicSequence, sizeof(FFooter::MagicSequence));
	Ar->Serialize(reinterpret_cast<uint8*>(&Footer), FFooter::Size());
	if (Ar->IsError())
	{
		const uint32 LastError = FPlatformMisc::GetLastError();
		Ar.Reset();
		Ifm.Delete(*Filename);
		return MakeError(InstallReplaySaveError());
	}

	const int64 FileSize = Ar->TotalSize();
	if (Ar->Close() == false)
	{
		const uint32 LastError = FPlatformMisc::GetLastError();
		Ar.Reset();
		Ifm.Delete(*Filename);
		return MakeError(InstallReplaySaveError());
	}

	return MakeValue(FileSize);
}

FOnDemandContentInstallReplayRecorder* FOnDemandContentInstallReplayRecorder::Instance = nullptr;

FOnDemandContentInstallReplayRecorder* FOnDemandContentInstallReplayRecorder::Get()
{
	return FOnDemandContentInstallReplayRecorder::Instance;
}

void FOnDemandContentInstallReplayRecorder::StartRecording(FOnDemandContentInstallReplayRecorder* InInstance)
{
	// This should only be called by init, so no need to take the lock yet
	check(Instance == nullptr);
	Instance = InInstance;

	Instance->StartRecordingInternal();
}

void FOnDemandContentInstallReplayRecorder::StartRecordingInternal()
{
	TUniqueLock Lock(Mutex);

	constexpr bool bRecorderValid = true;
	Reset(bRecorderValid);
	StartTime = FDateTime::UtcNow();
}

TValueOrError<FOnDemandContentInstallReplay, void> FOnDemandContentInstallReplayRecorder::StopRecording()
{
	TUniqueLock Lock(Mutex);

	if (!IsValid())
	{
		Reset();
		return MakeError();
	}

	FOnDemandContentInstallReplay Replay;
	MoveTemp(Events).ToArray(Replay.Events);

	Reset();
	return MakeValue(MoveTemp(Replay));
}

void FOnDemandContentInstallReplayRecorder::RecordContentHandle(UPTRINT HandleId, FString&& DebugName)
{
	TUniqueLock Lock(Mutex);

	if (ensure(HandleId) == false)
	{
		bValid = false;
	}

	if (!IsValid())
	{
		return;
	}

	const int64 EventIndex = AddEvent(
		FOnDemandContentInstallReplay::FContentHandleEventData{ MoveTemp(DebugName) });

	ContentHandleEventToIndex.Add(HandleId, EventIndex);
}

void FOnDemandContentInstallReplayRecorder::RecordContentHandleDestroyed(UPTRINT HandleId)
{
	TUniqueLock Lock(Mutex);

	if (!IsValid())
	{
		return;
	}

	int64 ContentHandleEventIndex = -1;
	if (ensure(ContentHandleEventToIndex.RemoveAndCopyValue(HandleId, ContentHandleEventIndex)) == false)
	{
		bValid = false;
		return;
	}

	AddEvent(FOnDemandContentInstallReplay::FContentHandleDestroyedEventData{ ContentHandleEventIndex });
}

void FOnDemandContentInstallReplayRecorder::RecordInstall(
	UPTRINT RequestId,
	UPTRINT HandleId,
	const FString& MountId,
	const TArray<FString>& TagSets,
	const TArray<FPackageId>& PackageIds,
	int32 Priority,
	EOnDemandInstallOptions Options)
{
	RecordInstall(
		RequestId,
		HandleId,
		CopyTemp(MountId),
		CopyTemp(TagSets),
		CopyTemp(PackageIds),
		Priority,
		Options
	);
}

void FOnDemandContentInstallReplayRecorder::RecordInstall(
	UPTRINT RequestId,
	UPTRINT HandleId,
	FString&& MountId,
	TArray<FString>&& TagSets,
	TArray<FPackageId>&& PackageIds,
	int32 Priority,
	EOnDemandInstallOptions Options)
{
	TUniqueLock Lock(Mutex);

	if (ensure(RequestId) == false)
	{
		bValid = false;
	}

	if (!IsValid())
	{
		return;
	}

	const int64 ContentHandleEventIndex = ContentHandleEventToIndex.FindRef(HandleId, -1);
	if (ensure(ContentHandleEventIndex >= 0) == false)
	{
		bValid = false;
		return;
	}

	const int64 EventIndex = AddEvent(
		FOnDemandContentInstallReplay::FInstallEventData
		{
			.MountId = MoveTemp(MountId),
			.TagSets = MoveTemp(TagSets),
			.PackageIds = MoveTemp(PackageIds),
			.ContentHandleEventId = ContentHandleEventIndex,
			.Priority = Priority,
			.Options = Options
		}
	);

	InstallEventToIndex.Add(RequestId, EventIndex);
}

void FOnDemandContentInstallReplayRecorder::RecordCancel(UPTRINT RequestId)
{
	TUniqueLock Lock(Mutex);

	if (!IsValid())
	{
		return;
	}

	const int64 InstallEventIndex = InstallEventToIndex.FindRef(RequestId, -1);
	if (ensure(InstallEventIndex >= 0) == false)
	{
		bValid = false;
		return;
	}

	AddEvent(FOnDemandContentInstallReplay::FCancelInstallEventData{ InstallEventIndex });
}

void FOnDemandContentInstallReplayRecorder::RecordUpdatePriority(UPTRINT RequestId, int32 NewPriority)
{
	TUniqueLock Lock(Mutex);

	if (!IsValid())
	{
		return;
	}

	const int64 InstallEventIndex = InstallEventToIndex.FindRef(RequestId, -1);
	if (ensure(InstallEventIndex >= 0) == false)
	{
		bValid = false;
		return;
	}

	AddEvent(FOnDemandContentInstallReplay::FUpdatePriorityEventData
		{
			.InstallEventId = InstallEventIndex,
			.NewPriority = NewPriority
		});
}

void FOnDemandContentInstallReplayRecorder::RecordInstallDestroyed(UPTRINT RequestId)
{
	TUniqueLock Lock(Mutex);

	if (!IsValid())
	{
		return;
	}

	int64 InstallEventIndex = -1;
	if (ensure(InstallEventToIndex.RemoveAndCopyValue(RequestId, InstallEventIndex)) == false)
	{
		bValid = false;
		return;
	}

	AddEvent(FOnDemandContentInstallReplay::FInstallDestroyedEventData{ InstallEventIndex });
}

void FOnDemandContentInstallReplayRecorder::Reset(bool bInValid /*= false*/)
{
	ContentHandleEventToIndex.Empty();
	InstallEventToIndex.Empty();
	Events.Empty();
	StartTime = FDateTime();
	bValid = bInValid;
}

int64 FOnDemandContentInstallReplayRecorder::GetTimeOffset() const
{
	const int64 TimeOffset = (FDateTime::UtcNow() - StartTime).GetTicks();
	return TimeOffset;
}

#endif // !UE_BUILD_SHIPPING

} // namespace UE::IoStore

////////////////////////////////////////////////////////////////////////////////
#if WITH_IOSTORE_ONDEMAND_TESTS && !UE_BUILD_SHIPPING

TEST_CASE("IoStore::OnDemand::InstallReplay", "[IoStoreOnDemand][InstallReplay]")
{
	class FTmpDirectoryScope
	{
	public:
		explicit FTmpDirectoryScope(const FString& InDir)
			: Ifm(IFileManager::Get())
			, Dir(InDir)
		{
			const bool bTree = true;
			const bool bRequireExists = false;
			Ifm.DeleteDirectory(*Dir, bRequireExists, bTree);
			Ifm.MakeDirectory(*Dir, bTree);
		}

		~FTmpDirectoryScope()
		{
			const bool bTree = true;
			const bool bRequireExists = false;
			Ifm.DeleteDirectory(*Dir, bRequireExists, bTree);
		}
	private:
		IFileManager& Ifm;
		FString Dir;
	};

	const FString TestBaseDir = "TestTmpDir";

	SECTION("SaveLoadRoundtrip")
	{
		using namespace UE::IoStore;

		FOnDemandContentInstallReplay ExpectedReplay;
		ExpectedReplay.Events.Emplace(FOnDemandContentInstallReplay::FEvent
			{
				.TimeOffset = 1,
				.EventData{
					TInPlaceType<FOnDemandContentInstallReplay::FContentHandleEventData>(),
					FOnDemandContentInstallReplay::FContentHandleEventData
					{
						.DebugName = TEXT("ContentHandle")
					}
				}
			});
		ExpectedReplay.Events.Emplace(FOnDemandContentInstallReplay::FEvent
			{
				.TimeOffset = 2,
				.EventData{
					TInPlaceType<FOnDemandContentInstallReplay::FInstallEventData>(),
					FOnDemandContentInstallReplay::FInstallEventData
					{
						.MountId = TEXT("MountId"),
						.TagSets{ TEXT("TagA"), TEXT("TagB") },
						.PackageIds{ FPackageId::FromValue(0xABAB) },
						.ContentHandleEventId = 0,
						.Priority = 1,
						.Options = EOnDemandInstallOptions::InstallSoftReferences
					}
				}
			});
		ExpectedReplay.Events.Emplace(FOnDemandContentInstallReplay::FEvent
			{
				.TimeOffset = 3,
				.EventData{
					TInPlaceType<FOnDemandContentInstallReplay::FUpdatePriorityEventData>(),
					FOnDemandContentInstallReplay::FUpdatePriorityEventData
					{
						.InstallEventId = 1,
						.NewPriority = 3
					}
				}
			});
		ExpectedReplay.Events.Emplace(FOnDemandContentInstallReplay::FEvent
			{
				.TimeOffset = 4,
				.EventData{
					TInPlaceType<FOnDemandContentInstallReplay::FCancelInstallEventData>(),
					FOnDemandContentInstallReplay::FCancelInstallEventData
					{
						.InstallEventId = 1
					}
				}
			});
		ExpectedReplay.Events.Emplace(FOnDemandContentInstallReplay::FEvent
			{
				.TimeOffset = 5,
				.EventData{
					TInPlaceType<FOnDemandContentInstallReplay::FInstallDestroyedEventData>(),
					FOnDemandContentInstallReplay::FInstallDestroyedEventData
					{
						.InstallEventId = 1
					}
				}
			});
		ExpectedReplay.Events.Emplace(FOnDemandContentInstallReplay::FEvent
			{
				.TimeOffset = 6,
				.EventData{
					TInPlaceType<FOnDemandContentInstallReplay::FContentHandleDestroyedEventData>(),
					FOnDemandContentInstallReplay::FContentHandleDestroyedEventData
					{
						.ContentHandleEventId = 0
					}
				}
			});

		FTmpDirectoryScope _(TestBaseDir);
		const FString Filename = TestBaseDir / TEXT("test.installreplay");

		TResult<int64> SaveResult = FOnDemandContentInstallReplay::Save(ExpectedReplay, Filename);
		CHECK(SaveResult.HasValue());

		int64 BytesLoaded = 0;
		TResult<FOnDemandContentInstallReplay> LoadResult = FOnDemandContentInstallReplay::Load(Filename, &BytesLoaded);
		CHECK(LoadResult.HasValue());

		CHECK(SaveResult.GetValue() == BytesLoaded);

		CHECK(ExpectedReplay == LoadResult.GetValue());
	}
}

#endif // WITH_IOSTORE_ONDEMAND_TESTS && !UE_BUILD_SHIPPING
