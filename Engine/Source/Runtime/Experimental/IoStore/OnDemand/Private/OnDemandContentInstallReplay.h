// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "Containers/PagedArray.h"
#include "HAL/PlatformTime.h"
#include "IO/IoStoreOnDemand.h"
#include "IO/IoStoreOnDemandInternals.h"
#include "IO/OnDemandError.h"
#include "IO/PackageId.h"

namespace UE::IoStore
{

#if !UE_BUILD_SHIPPING

	struct FOnDemandContentInstallReplay
	{
		// TODO: record mounting/unmounting containers

		// TODO: Record when installs are processed and exectuted ?

		// TODO: Record an event for file last access

		enum class EVersion : int32
		{
			Invalid = 0,
			Initial,

			LatestPlusOne,
			Latest = LatestPlusOne - 1
		};

		static const FGuid CustomVersionGuid;

		struct FHeader
		{
			static const uint8 MagicSequence[16];

			bool			IsValid() const;
			static int64	Size() { return sizeof(FHeader); }

			uint8		Magic[16] = { 0 };
			EVersion	Version = EVersion::Invalid;
			uint8		Pad[12] = { 0 };
		};
		static_assert(sizeof(FHeader) == 32);

		struct FFooter
		{
			static const uint8 MagicSequence[16];

			static int64	Size() { return sizeof(FFooter); }
			bool			IsValid() const;

			uint8 Magic[16] = { 0 };
		};
		static_assert(sizeof(FFooter) == 16);

		struct FContentHandleEventData
		{
			FString DebugName;

			bool operator==(const FContentHandleEventData& Other) const = default;
		};

		struct FContentHandleDestroyedEventData
		{
			int64 ContentHandleEventId = -1;

			bool operator==(const FContentHandleDestroyedEventData& Other) const = default;
		};

		struct FInstallEventData
		{
			FString MountId;
			TArray<FString> TagSets;
			TArray<FPackageId> PackageIds;
			int64 ContentHandleEventId = -1;
			int32 Priority = 0;
			EOnDemandInstallOptions Options = EOnDemandInstallOptions::None;

			bool operator==(const FInstallEventData& Other) const = default;
		};

		struct FCancelInstallEventData
		{
			int64 InstallEventId = -1;

			bool operator==(const FCancelInstallEventData& Other) const = default;
		};

		struct FUpdatePriorityEventData
		{
			int64 InstallEventId = -1;
			int32 NewPriority = 0;

			bool operator==(const FUpdatePriorityEventData& Other) const = default;
		};

		struct FInstallDestroyedEventData
		{
			int64 InstallEventId = -1;

			bool operator==(const FInstallDestroyedEventData& Other) const = default;
		};

		using FEventVariant = TVariant<
			FEmptyVariantState,
			FContentHandleEventData,
			FContentHandleDestroyedEventData,
			FInstallEventData,
			FCancelInstallEventData,
			FUpdatePriorityEventData,
			FInstallDestroyedEventData>;

		struct FEvent
		{
			int64 TimeOffset = 0; // Time since start of replay
			FEventVariant EventData;
		};

#if WITH_IOSTORE_ONDEMAND_TESTS
		bool operator==(const FOnDemandContentInstallReplay& Other) const;
#endif // WITH_IOSTORE_ONDEMAND_TESTS

		static TResult<FOnDemandContentInstallReplay>	Load(const FString& Filename, int64* OutFileSize = nullptr);
		static TResult<int64>							Save(const FOnDemandContentInstallReplay& Replay, const FString& Filename);

		TArray64<FEvent> Events;
	};

	class FOnDemandContentInstallReplayRecorder
	{
	private:
		static FOnDemandContentInstallReplayRecorder* Instance;

	public:
		static FOnDemandContentInstallReplayRecorder* Get();

		static void StartRecording(FOnDemandContentInstallReplayRecorder* InInstance);

		// TODO: return TResult
		TValueOrError<FOnDemandContentInstallReplay, void> StopRecording();

		// Records the public content handle, no the internal implementation
		void RecordContentHandle(UPTRINT HandleId, FString&& DebugName);

		void RecordContentHandleDestroyed(UPTRINT HandleId);

		void RecordInstall(
			UPTRINT RequestId,
			UPTRINT HandleId,
			const FString& MountId,
			const TArray<FString>& TagSets,
			const TArray<FPackageId>& PackageIds,
			int32 Priority,
			EOnDemandInstallOptions Options);

		void RecordInstall(
			UPTRINT RequestId,
			UPTRINT HandleId,
			FString&& MountId,
			TArray<FString>&& TagSets,
			TArray<FPackageId>&& PackageIds,
			int32 Priority,
			EOnDemandInstallOptions Options);

		void RecordCancel(UPTRINT RequestId);

		void RecordUpdatePriority(UPTRINT RequestId, int32 NewPriority);

		void RecordInstallDestroyed(UPTRINT RequestId);

		bool IsValid() const { return bValid; }

	private:
		void StartRecordingInternal();

		void Reset(bool bInValid = false);

		int64 GetTimeOffset() const;

		template<typename EventType>
		int64 AddEvent(EventType&& InEvent)
		{
			const int64 TimeOffset = GetTimeOffset();
			const int64 EventIndex = Events.Add(
				FOnDemandContentInstallReplay::FEvent
				{
					.TimeOffset = TimeOffset,
					.EventData = FOnDemandContentInstallReplay::FEventVariant(
						TInPlaceType<EventType>(), MoveTemp(InEvent))
				}
			);
			return EventIndex;
		}

	private:
		FMutex Mutex;
		TMap<UPTRINT, int64> ContentHandleEventToIndex;
		TMap<UPTRINT, int64> InstallEventToIndex;
		TPagedArray<FOnDemandContentInstallReplay::FEvent, 16384, FDefaultAllocator64> Events;
		FDateTime StartTime;
		bool bValid = true;
	};

#endif // !UE_BUILD_SHIPPING

} // namespace UE::IoStore
