// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnDemandContentInstaller.h"

#include "Algo/Accumulate.h"
#include "Algo/AnyOf.h"
#include "Algo/Find.h"
#include "Async/UniqueLock.h"
#include "Containers/RingBuffer.h"
#include "Experimental/UnifiedError/CoreErrorTypes.h"
#include "HAL/Platform.h"
#include "IO/IoBuffer.h"
#include "IO/IoChunkId.h"
#include "IO/IoDispatcher.h"
#include "IO/IoStatus.h"
#include "IO/IoStoreOnDemand.h"
#include "IO/OnDemandError.h"
#include "Logging/StructuredLog.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h" // USE_MMAPPED_SHADERARCHIVE
#include "Misc/CoreDelegatesInternal.h"
#include "Misc/Timespan.h"
#include "OnDemandContentInstallReplay.h"
#include "OnDemandInstallCache.h"
#include "OnDemandIoStore.h"
#include "OnDemandMisc.h"
#include "OnDemandPackageStoreBackend.h"
#include "Serialization/PackageStoreInternal.h"
#include "Statistics.h"
#include <atomic>

TRACE_DECLARE_INT_COUNTER(OnDemandContentInstaller_HttpRequestsIssued, TEXT("OnDemandContentInstaller/HttpRequestsIssued"));
TRACE_DECLARE_INT_COUNTER(OnDemandContentInstaller_HttpRequestsTryCanceled, TEXT("OnDemandContentInstaller/HttpRequestsTryCanceled"));
TRACE_DECLARE_INT_COUNTER(OnDemandContentInstaller_HttpRequestsSuccess, TEXT("OnDemandContentInstaller/HttpRequestsSuccess"));
TRACE_DECLARE_INT_COUNTER(OnDemandContentInstaller_HttpRequestsFail, TEXT("OnDemandContentInstaller/HttpRequestsFail"));
TRACE_DECLARE_INT_COUNTER(OnDemandContentInstaller_HttpRequestsActuallyCanceled, TEXT("OnDemandContentInstaller/HttpRequestsActuallyCanceled"));

namespace UE::IoStore
{
	namespace CVars
	{
#if !UE_BUILD_SHIPPING
		static FString IoStoreErrorOnRequest;
		static FAutoConsoleVariableRef CVar_IoStoreErrorOnRequest(
			TEXT("iostore.ErrorOnRequest"),
			IoStoreErrorOnRequest,
			TEXT("When the request with a debug name partially matching this cvar is found iostore will error with a debug 'InstallArgs' error.")
			);

		FString IoStoreDebugRequest;
		static FAutoConsoleVariableRef CVar_IoStoreDebugRequest(
				TEXT("iostore.DebugRequest"),
				IoStoreDebugRequest,
				TEXT("When the request with a debug name partially matching this cvar is found iostore will print a message to the log on which a breakpoint can be placed.")
			);

		int32 IoStoreSimulateHttpError = -1;
		static FAutoConsoleVariableRef CVar_IoStoreSimulateHttpError(
				TEXT("iostore.SimulateHttpError"),
				IoStoreSimulateHttpError,
				TEXT("Simulate http error code in iostore.")
			);

		float IoStoreChunkMissingErrorChance = 0.0f;
		static FAutoConsoleVariableRef CVar_IoStoreChunkMissingErrorChange(
			TEXT("iostore.ChunksMissingRandomErrorChance"),
			IoStoreChunkMissingErrorChance,
			TEXT("Simulate chunk missing error based on chance 0 - 100.")
			);
#endif // !UE_BUILD_SHIPPING

		static bool bAllowAllMissingDependencies = false;
		static FAutoConsoleVariableRef CVar_AllowAllMissingDependencies(
			TEXT("iostore.AllowAllMissingDependencies"),
			bAllowAllMissingDependencies,
			TEXT("Allow missing dependencies when attempting to download assets, can cause silent failures.")
			);

		static bool bContentInstallerEnableHttpCancel = true;
		static FAutoConsoleVariableRef CVar_ContentInstallerEnableHttpCancel(
			TEXT("iostore.ContentInstallerEnableHttpCancel"),
			bContentInstallerEnableHttpCancel,
			TEXT("Enable canceling HTTP requests when an install request is canceled")
			);

		static bool bReleaseContentAsync = true;
		static FAutoConsoleVariableRef CVar_ReleaseContentAsync(
			TEXT("iostore.ReleaseContentAsync"),
			bReleaseContentAsync,
			TEXT("When enabled, releasing content handles is scheduled asynchronously on the installer pipe instead of running inline on the calling thread.")
			);

		static bool bPinDownloadedChunksImmediately = true;
		static FAutoConsoleVariableRef CVar_PinDownloadedChunksImmediately(
			TEXT("iostore.PinDownloadedChunksImmediately"),
			bPinDownloadedChunksImmediately,
			TEXT("When enabled, chunks are pinned as soon as they are downloaded instead of waiting until the request is complete.")
			);

	}

namespace Private
{

////////////////////////////////////////////////////////////////////////////////
int32 SecondsToMillicseconds(double Seconds)
{
	return Seconds > 0.0 ? int32(Seconds * 1000.0) : 0;
}

////////////////////////////////////////////////////////////////////////////////
static EPackageStoreEntryStatus GetPackageStoreEntry(
	FPackageStore& PackageStore,
	IOnDemandPackageStoreBackend& OnDemandPackageStore,
	const FPackageId& PackageId,
	FPackageStoreEntry& OutPackageStoreEntry,
	bool& bOnDemand)
{
	EPackageStoreEntryStatus EntryStatus = EPackageStoreEntryStatus::Missing;
	bOnDemand = false;

	FPackageStoreInternal::IterateBackends(
		PackageStore,
		[&EntryStatus, &OutPackageStoreEntry, &bOnDemand, &OnDemandPackageStore, &PackageId](IPackageStoreBackend& Backend)
		{
			const bool bOnDemandPackageStore = &OnDemandPackageStore == &Backend;
			if (bOnDemandPackageStore)
			{
				// When running with -iax.devmode, the on-demand package store reports
				// unreferenced entries as 'Missing' so that lower-priority backends
				// (e.g., the file backend) can attempt to load the package.
				// Query the on-demand package store directly to bypass the development mode check.
				EntryStatus = OnDemandPackageStore.GetPackageStoreEntryInternal(PackageId, OutPackageStoreEntry);
			}
			else
			{
				EntryStatus = Backend.GetPackageStoreEntry(PackageId, NAME_None, OutPackageStoreEntry);
			}

			const bool bMissing = EntryStatus == EPackageStoreEntryStatus::None || EntryStatus == EPackageStoreEntryStatus::Missing;
			bOnDemand = !bMissing && bOnDemandPackageStore;
			return bMissing;
		});

	return EntryStatus;
}

////////////////////////////////////////////////////////////////////////////////
void VisitPackageDependencies(
	const TSharedPtr<IOnDemandPackageStoreBackend>& OnDemandPackageStore,
	const TSet<FPackageId>& PackageIds, 
	bool bIncludeSoftReferences, 
	FPackageDependencyVisitor Visitor, 
	FPackageStoreReadScope* InReadScope)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandContentInstaller::TVisitPackageDependencies);

	TRingBuffer<FPackageId> Queue;
	TSet<FPackageId>		Visitied;

	Visitied.Reserve(PackageIds.Num());
	Queue.Reserve(PackageIds.Num());
	for (const FPackageId& PackageId : PackageIds)
	{
		Queue.Add(PackageId);
	}

	FPackageStore& PackageStore = FPackageStore::Get();
	TOptional<FPackageStoreReadScope> ReadScope;
	if (InReadScope == nullptr)
	{
		ReadScope.Emplace(PackageStore);
	}

	while (!Queue.IsEmpty())
	{
		FPackageId PackageId = Queue.PopFrontValue();
		{
			FName		SourcePackageName;
			FPackageId	RedirectedToPackageId;
			if (PackageStore.GetPackageRedirectInfo(PackageId, SourcePackageName, RedirectedToPackageId))
			{
				PackageId = RedirectedToPackageId;
			}
		}

		bool bIsAlreadyInSet = false;
		Visitied.Add(PackageId, &bIsAlreadyInSet);
		if (bIsAlreadyInSet)
		{
			continue;
		}

		FPackageStoreEntry PackageStoreEntry;
		bool bOnDemand = false;
		const EPackageStoreEntryStatus EntryStatus = GetPackageStoreEntry(
			PackageStore,
			*OnDemandPackageStore,
			PackageId,
			PackageStoreEntry,
			bOnDemand);

		const bool bContinue = Visitor(PackageId, EntryStatus, PackageStoreEntry, bOnDemand);
		if (bContinue == false)
		{
			break;
		}

		if (EntryStatus != EPackageStoreEntryStatus::Missing)
		{
			for (const FPackageId& ImportedPackageId : PackageStoreEntry.ImportedPackageIds)
			{
				if (!Visitied.Contains(ImportedPackageId))
				{
					Queue.Add(ImportedPackageId);
				}
			}

			if (bIncludeSoftReferences)
			{
				TConstArrayView<FPackageId> SoftReferences;
				TConstArrayView<uint32> Indices = PackageStore.GetSoftReferences(PackageId, SoftReferences);
				for (uint32 Idx : Indices)
				{
					const FPackageId& SoftRef = SoftReferences[Idx];
					if (!Visitied.Contains(SoftRef))
					{
						Queue.Add(SoftRef);
					}
				}
			}
		}
	}
}
////////////////////////////////////////////////////////////////////////////////
static bool IsPackageStoreEntryMissing(EPackageStoreEntryStatus EntryStatus)
{
#if !UE_BUILD_SHIPPING
	const float ErrorChance = UE::IoStore::CVars::IoStoreChunkMissingErrorChance;
	if (ErrorChance > 0.0f && (FMath::RandRange(0.0f, 100.0f) < ErrorChance))
	{
		EntryStatus = EPackageStoreEntryStatus::Missing;
	}
#endif
	return EntryStatus == EPackageStoreEntryStatus::Missing;
}

////////////////////////////////////////////////////////////////////////////////
void ResolveChunksToInstall(
	const TSharedPtr<IOnDemandPackageStoreBackend>& OnDemandPackageStore,
	const TSet<FSharedOnDemandContainer>& Containers,
	const TSet<FPackageId>& PackageIds,
	bool bIncludeSoftReferences,
	bool bIncludeOptionalBulkData,
	TArray<FOnDemandChunkInfoList>& OutResolvedContainerChunks,
	TSet<FIoChunkId>& OutMissing)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandContentInstaller::ResolveChunksToInstall);

	for (const FSharedOnDemandContainer& Container : Containers)
	{
		FOnDemandChunkInfoList& ResolvedChunks = OutResolvedContainerChunks.Emplace_GetRef(Container);
	}

	TSet<FPackageId> ResolvedPackageIds;
	TSet<FIoChunkId> ShaderMapChunkIds;
	VisitPackageDependencies(
		OnDemandPackageStore,
		PackageIds,
		bIncludeSoftReferences,
		[&ResolvedPackageIds, &ShaderMapChunkIds, &OutMissing](
			FPackageId PackageId,
			EPackageStoreEntryStatus EntryStatus,
			const FPackageStoreEntry& PackageStoreEntry,
			bool bOnDemand)
			{
				if (IsPackageStoreEntryMissing(EntryStatus))
				{
					OutMissing.Add(CreatePackageDataChunkId(PackageId));
				}
				else if (bOnDemand)
				{
					ResolvedPackageIds.Add(PackageId);
					if (PackageStoreEntry.ShaderMapHashes.IsEmpty() == false)
					{
						FCoreDelegates::ResolvePackageShaderMaps.ExecuteIfBound(
							PackageStoreEntry.ShaderMapHashes,
							[&ShaderMapChunkIds](const FIoChunkId& ChunkId)
							{
								ShaderMapChunkIds.Add(ChunkId);
							});
					}
				}

				return true;
			});

	auto FindChunkEntry = [&OutResolvedContainerChunks](const FIoChunkId& ChunkId, int32& OutIndex) -> FOnDemandChunkInfoList*
	{
		for (FOnDemandChunkInfoList& ContainerChunks : OutResolvedContainerChunks)
		{
			if (OutIndex = ContainerChunks.SharedContainer->FindChunkEntryIndex(ChunkId); OutIndex != INDEX_NONE)
			{
				return &ContainerChunks;
			}
		}
		return nullptr;
	};

	// Resolve shader map chunks
	for (const FIoChunkId& ChunkId : ShaderMapChunkIds)
	{
		int32 EntryIndex = INDEX_NONE;
		if (FOnDemandChunkInfoList* ResolvedChunks = FindChunkEntry(ChunkId, EntryIndex))
		{
			check(EntryIndex != INDEX_NONE);
			ResolvedChunks->Indices.Emplace(EntryIndex);
		}
		else
		{
			OutMissing.Add(ChunkId);
		}
	}

	// Resolve all chunk entries from the resolved package ID's
	for (const FPackageId& PackageId : ResolvedPackageIds)
	{
		const FIoChunkId PackageChunkId				= CreatePackageDataChunkId(PackageId);
		int32 EntryIndex							= INDEX_NONE;
		FOnDemandChunkInfoList* ResolvedChunks	= FindChunkEntry(PackageChunkId, EntryIndex);

		if (ResolvedChunks == nullptr)
		{
			UE_LOGF(LogIoStoreOnDemand, Warning, "Failed to resolve I/O store chunk for package ID '%ls'", *LexToString(PackageId));
			continue;
		}

		check(EntryIndex != INDEX_NONE);
		ResolvedChunks->Indices.Emplace(EntryIndex);

		static constexpr const EIoChunkType RequiredChunkTypes[] =
		{
			EIoChunkType::BulkData,
			EIoChunkType::MemoryMappedBulkData 
		};

		static constexpr const EIoChunkType RequiredAndOptionalChunkTypes[] =
		{
			EIoChunkType::BulkData,
			EIoChunkType::OptionalBulkData,
			EIoChunkType::MemoryMappedBulkData 
		};

		TConstArrayView<EIoChunkType> AdditionalChunkTypes = bIncludeOptionalBulkData
			? MakeArrayView<const EIoChunkType>(RequiredAndOptionalChunkTypes, UE_ARRAY_COUNT(RequiredAndOptionalChunkTypes))
			: MakeArrayView<const EIoChunkType>(RequiredChunkTypes, UE_ARRAY_COUNT(RequiredChunkTypes));

		for (EIoChunkType ChunkType : AdditionalChunkTypes)
		{
			// TODO: For Mutable we need to traverse all possible bulk data chunk indices?
			const FIoChunkId ChunkId = CreateBulkDataIoChunkId(PackageId.Value(), 0, 0, ChunkType);
			if (ResolvedChunks = FindChunkEntry(ChunkId, EntryIndex); ResolvedChunks != nullptr)
			{
				check(EntryIndex != INDEX_NONE);
				ResolvedChunks->Indices.Emplace(EntryIndex);
			}
		}
	}

	for (FOnDemandChunkInfoList& ResolvedChunks : OutResolvedContainerChunks)
	{
		ResolvedChunks.Indices.Sort();
	}
}

} // namespace Private

////////////////////////////////////////////////////////////////////////////////
uint32 FOnDemandContentInstaller::FRequest::NextSeqNo = 0;

TOptional<UE::UnifiedError::FError> FOnDemandContentInstaller::FRequest::ConsumeError()
{
	if (Result.HasError())
	{
		return TOptional<UE::UnifiedError::FError>(Result.StealError());
	}
	else if (IsCancelled())
	{
		return TOptional<UE::UnifiedError::FError>(UE::Core::CancellationError());
	}

	return TOptional<UE::UnifiedError::FError>();
}

////////////////////////////////////////////////////////////////////////////////
FOnDemandContentInstaller::FRequest::FInstall::FInstall(
	UPTRINT InstallerRequest, 
	FOnDemandInstallArgs&& InArgs, 
	FOnDemandInstallCompleted&& InOnCompleted, 
	FOnDemandInstallProgressed&& InOnProgress)
	: Args(MoveTemp(InArgs))
	, OnCompleted(MoveTemp(InOnCompleted))
	, OnProgress(MoveTemp(InOnProgress)) 
	, Request(MakeShared<FOnDemandInternalInstallRequest, ESPMode::ThreadSafe>(InstallerRequest))
{
}

FOnDemandContentInstaller::FRequest::FInstall::~FInstall() = default;

FOnDemandContentInstaller::FOnDemandContentInstaller(FOnDemandIoStore& InIoStore)
	: IoStore(InIoStore)
	, InstallerPipe(TEXT("IoStoreOnDemandInstallerPipe"))
{
}

FOnDemandContentInstaller::~FOnDemandContentInstaller()
{
	Shutdown();
}

FSharedInternalInstallRequest FOnDemandContentInstaller::EnqueueInstallRequest(
	FOnDemandInstallArgs&& Args,
	FOnDemandInstallCompleted&& OnCompleted,
	FOnDemandInstallProgressed&& OnProgress)
{
	FRequest* Request = nullptr;
	{
		TUniqueLock Lock(Mutex);
		Request = RequestAllocator.Alloc();
	}

	new(Request) FRequest(MoveTemp(Args), MoveTemp(OnCompleted), MoveTemp(OnProgress));
	FRequest::FInstall& InstallData = Request->AsInstall();
	InstallData.Request->IoStore = IoStore.AsWeak();
	InstallData.CacheHandle = IoStore.InstallCache->BeginInstall();

	FOnDemandContentInstallerStats::OnRequestEnqueued();

	InstallerPipe.Launch(
		TEXT("ProcessIoStoreOnDemandInstallRequest"),
		[this, Request] { ProcessInstallRequest(*Request); },
		UE::Tasks::ETaskPriority::BackgroundLow);

	return InstallData.Request;
}

void FOnDemandContentInstaller::EnqueuePurgeRequest(FOnDemandPurgeArgs&& Args, FOnDemandPurgeCompleted&& OnCompleted)
{
	{
		TUniqueLock Lock(Mutex);
		RequestQueue.HeapPush(RequestAllocator.Construct(MoveTemp(Args), MoveTemp(OnCompleted)), RequestSortPredicate);
	}

	TryExecuteNextRequest();
}

void FOnDemandContentInstaller::EnqueueDefragRequest(FOnDemandDefragArgs&& Args, FOnDemandDefragCompleted&& OnCompleted)
{
	{
		TUniqueLock Lock(Mutex);
		RequestQueue.HeapPush(RequestAllocator.Construct(MoveTemp(Args), MoveTemp(OnCompleted)), RequestSortPredicate);
	}

	TryExecuteNextRequest();
}

void FOnDemandContentInstaller::EnqueueVerifyRequest(FOnDemandVerifyCacheCompleted&& OnCompleted)
{
	{
		TUniqueLock Lock(Mutex);
		RequestQueue.HeapPush(RequestAllocator.Construct(MoveTemp(OnCompleted)), RequestSortPredicate);
	}

	TryExecuteNextRequest();
}

void FOnDemandContentInstaller::EnqueueFlushLastAccessRequest(FOnDemandFlushLastAccessArgs&& Args, FOnDemandFlushLastAccessCompleted&& OnCompleted)
{
	{
		TUniqueLock Lock(Mutex);
		RequestQueue.HeapPush(RequestAllocator.Construct(MoveTemp(Args), MoveTemp(OnCompleted)), RequestSortPredicate);
	}

	TryExecuteNextRequest();
}

void FOnDemandContentInstaller::ScheduleReleaseContentRequestTask(UPTRINT HandleId)
{
	auto Task = [this, HandleId]
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandContentInstaller::ReleaseContentTask);

		FOnDemandFlushLastAccessArgs FlushLastAccessArgs;
		FlushLastAccessArgs.bForceLastAccessDirty = false;

		{
			UE::TUniqueLock Lock(IoStore.ContainerMutex);

			for (FSharedOnDemandContainer& Container : IoStore.Containers)
			{
				if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::StreamOnDemand))
				{
					continue;
				}

				TUniqueLock RefsLock(Container->ReferencesMutex);

				const int32 RemoveIndex = Container->ChunkEntryReferences.IndexOfByPredicate(
					[HandleId](const FOnDemandChunkEntryReferences& Refs)
					{
						return Refs.ContentHandleId == HandleId;
					}
				);

				if (RemoveIndex == INDEX_NONE)
				{
					continue;
				}

				// Note: A referencer is only added to the first container found that has a chunk, and the
				// referencer is not added the same container more than once. So once the referencer is removed,
				// we only need to inspect the current container for unreffed chunks.

				FlushLastAccessArgs.RemovedChunkEntryReferences.Add(MakeTuple(
					Container,
					MoveTemp(Container->ChunkEntryReferences[RemoveIndex].Indices)
				));
				Container->ChunkEntryReferences.RemoveAtSwap(RemoveIndex);
			}
		}

		// Package store needs to be aware of install state
		if (ensureMsgf(IoStore.PackageStoreBackend, TEXT("PackageStoreBackend is null, is this a prefork server?")))
		{
			IoStore.PackageStoreBackend->NeedsUpdate(EOnDemandPackageStoreUpdateMode::ReferencedPackages);
		}

		if (FlushLastAccessArgs.RemovedChunkEntryReferences.IsEmpty() == false)
		{
			LLM_SCOPE_BYNAME(TEXT("FileSystem/OnDemandIoStore"));
			EnqueueFlushLastAccessRequest(MoveTemp(FlushLastAccessArgs), nullptr);
		}
	};

	if (CVars::bReleaseContentAsync)
	{
		InstallerPipe.Launch(
			TEXT("OnDemandIoStore::CompleteReleaseContentRequest"),
			MoveTemp(Task),
			UE::Tasks::ETaskPriority::BackgroundLow);
	}
	else
	{
		Task();
	}
}

void FOnDemandContentInstaller::CancelInstallRequest(FSharedInternalInstallRequest InstallRequest)
{
	InstallerPipe.Launch(
		TEXT("CancelIoStoreOnDemandInstallRequest"),
		[this, InstallRequest]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandContentInstaller::CancelIoStoreOnDemandInstallRequest);

			FRequest* ToComplete = nullptr;
			{
				if (InstallRequest->InstallerRequest == 0)
				{
					return;
				}

				FRequest* Request = reinterpret_cast<FRequest*>(InstallRequest->InstallerRequest);
				FRequest::FInstall& Install = Request->AsInstall();

				if (Install.bHttpRequestsIssued)
				{
					if (Request->TryCancel() == false)
					{
						return;
					}

#if !UE_BUILD_SHIPPING
					if (FOnDemandContentInstallReplayRecorder* Recorder = FOnDemandContentInstallReplayRecorder::Get())
					{
						Recorder->RecordCancel(InstallRequest->InstallerRequest);
					}
#endif // !UE_BUILD_SHIPPING

					UE_LOGF(LogIoStoreOnDemand, Verbose, "Cancelling install request, ContentHandle=(%ls)",
						*LexToString(Install.Args.ContentHandle));

					int32 NumCancelled = 0;
					TryCancelHttpRequestsForInstallRequest(Install, NumCancelled);
					UE_LOGF(LogIoStoreOnDemand, Verbose, "Cancelled %d HTTP request(s) due to install request cancellation", NumCancelled);
				}
				else
				{
					if (Request->TryCancel() == false)
					{
						return;
					}

#if !UE_BUILD_SHIPPING
					if (FOnDemandContentInstallReplayRecorder* Recorder = FOnDemandContentInstallReplayRecorder::Get())
					{
						Recorder->RecordCancel(InstallRequest->InstallerRequest);
					}
#endif // !UE_BUILD_SHIPPING

					UE_LOGF(LogIoStoreOnDemand, Verbose, "Cancelling install request, ContentHandle=(%ls)",
						*LexToString(Install.Args.ContentHandle));

					TUniqueLock Lock(Mutex);

					if (RequestQueue.Remove(Request) > 0)
					{
						ToComplete = Request;
						RequestQueue.Heapify(RequestSortPredicate);
					}
				}
			}

			if (ToComplete != nullptr)
			{
				CompleteInstallRequest(*ToComplete);
			}
		});
}

void FOnDemandContentInstaller::UpdateInstallRequestPriority(FSharedInternalInstallRequest InstallRequest, const int32 NewPriority)
{
	InstallerPipe.Launch(
		TEXT("UpdateIoStoreOnDemandInstallRequestPriority"),
		[this, InstallRequest, NewPriority]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandContentInstaller::UpdateInstallRequestPriority);

			if (InstallRequest->InstallerRequest == 0)
			{
				return;
			}

#if !UE_BUILD_SHIPPING
			if (FOnDemandContentInstallReplayRecorder* Recorder = FOnDemandContentInstallReplayRecorder::Get())
			{
				Recorder->RecordUpdatePriority(InstallRequest->InstallerRequest, NewPriority);
			}
#endif // !UE_BUILD_SHIPPING

			FRequest&			Request	= *reinterpret_cast<FRequest*>(InstallRequest->InstallerRequest);
			FRequest::FInstall& Install	= Request.AsInstall();

			UE_LOGF(LogIoStoreOnDemand, VeryVerbose, "Updating install request priority, SeqNo=%u, Priority=%d, NewPriority=%d, ContentHandle=(%ls)",
				Request.SeqNo, Request.Priority, NewPriority, *LexToString(Install.Args.ContentHandle));

			if (Install.bHttpRequestsIssued)
			{
				// The request has definitely left the RequestQueue 

				Request.Priority = NewPriority;

				for (FChunkHttpRequestHandle& PendingHttpRequest : Install.HttpRequestHandles)
				{
					const FOnDemandChunkInfoList& Container = Install.ResolvedChunks[PendingHttpRequest.ContainerIndex];
					const FOnDemandChunkHash& ChunkHash = Container.Hash(PendingHttpRequest.EntryIndex);

					for (auto It = PendingChunkDownloads.CreateKeyIterator(ChunkHash); It; ++It)
					{
						FChunkDownloadRequest& ChunkDownload = It.Value();
						if (ChunkDownload.bChunkCanceled)
						{
							continue;
						}

						// Use the maximum priority of all requests waiting on this chunk
						int32 MaxPriority = std::numeric_limits<int32>::min();
						for (FChunkHttpRequestHandle& Handle : ChunkDownload.ChunkRequestHttpHandles)
						{
							const int32 RequestPriority = Handle.OwnerRequest->Priority;
							if (RequestPriority > MaxPriority)
							{
								MaxPriority = RequestPriority;
							}
						}

						ChunkDownload.HttpHandle.UpdatePriorty(MaxPriority);
						break;
					}
				}
			}
			else
			{
				// The request may or may not still be in the RequestQueue
				TUniqueLock Lock(Mutex);
				Request.Priority = NewPriority;
				RequestQueue.Heapify(RequestSortPredicate);
			}
		});
}

void FOnDemandContentInstaller::ReportAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const
{
	FOnDemandContentInstallerStats::ReportAnalytics(OutAnalyticsArray);
}

bool FOnDemandContentInstaller::CanExecuteRequest(FRequest& Request) const
{
	// Allow multiple install requests XOR any other request
	bool bExecute = RunningRequests.IsEmpty() ||
		(
			Request.IsInstall() &&
			(RunningRequests.Num() > 1 || RunningRequests[0]->IsInstall())
		);

	return bExecute;
}

void FOnDemandContentInstaller::TryExecuteNextRequest()
{
	if (bShuttingDown.load(std::memory_order_relaxed))
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandContentInstaller::TryExecuteNextRequest);

	bool bRequestQueueIsEmpty = false;
	while (bRequestQueueIsEmpty == false)
	{
		FRequest* NextRequest = nullptr;
		{
			TUniqueLock Lock(Mutex);
			bRequestQueueIsEmpty = RequestQueue.IsEmpty();
			if (bRequestQueueIsEmpty == false)
			{
				if (CanExecuteRequest(*RequestQueue.HeapTop()))
				{
					RequestQueue.HeapPop(NextRequest, RequestSortPredicate, EAllowShrinking::No);
					RunningRequests.Add(NextRequest);
				}
			}
		}

		if (NextRequest == nullptr)
		{
			break;
		}

		InstallerPipe.Launch(
			TEXT("OnDemandContentInstaller::ExecuteRequest"),
			[this, NextRequest] { ExecuteRequest(*NextRequest); },
			UE::Tasks::ETaskPriority::BackgroundLow);

		// If we aren't doing an install request, we only allow one running request so 
		// stop trying to dequeue requests.
		if (NextRequest->IsInstall() == false)
		{
			break;
		}
	}
}

void FOnDemandContentInstaller::ExecuteRequest(FRequest& Request)
{
	struct FVisitor
	{
		void operator()(FEmptyVariantState& Empty)
		{
			ensure(false);
		}

		void operator()(FRequest::FInstall&)
		{
			Installer.ExecuteInstallRequest(Request);
		}

		void operator()(FRequest::FPurge&)
		{
			Installer.ExecutePurgeRequest(Request);
		}

		void operator()(FRequest::FDefrag&)
		{
			Installer.ExecuteDefragRequest(Request);
		}

		void operator()(FRequest::FVerify&)
		{
			Installer.ExecuteVerifyRequest(Request);
		}

		void operator()(FRequest::FFlushLastAccess&)
		{
			Installer.ExectuteFlushLastAccessRequest(Request);
		}

		FOnDemandContentInstaller&	Installer;
		FRequest&					Request;
	};

	FVisitor Visitor { .Installer = *this, .Request = Request };
	Visit(Visitor, Request.Variant);
}

void FOnDemandContentInstaller::PinCachedChunks(
	FOnDemandContentInstaller::FRequest::FInstall& InstallRequest,
	TFunctionRef<void(int32, int32, bool)> OnChunkFound) const
{
	const TSharedPtr<FOnDemandInternalContentHandle>& ContentHandle = InstallRequest.Args.ContentHandle.Handle;

	for (int32 ContainerIndex = 0; FOnDemandChunkInfoList& ResolvedChunks : InstallRequest.ResolvedChunks)
	{
		IoStore.InstallCache->PinCachedChunks(*InstallRequest.CacheHandle, *ContentHandle, ResolvedChunks,
			[OnChunkFound, ContainerIndex](int32 EntryIndex, bool bCached)
			{
				OnChunkFound(ContainerIndex, EntryIndex, bCached);
			}
		);

		ContainerIndex++;
	}
}

void FOnDemandContentInstaller::ProcessInstallRequest(FRequest& Request)
{
	using namespace UE::IoStore::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandContentInstaller::ProcessInstallRequest);

	FRequest::FInstall& InstallRequest	= Request.AsInstall();
	Request.Priority					= InstallRequest.Args.Priority;

	UE_LOGF(LogIoStoreOnDemand, Verbose, "Processing install request, SeqNo=%u, Priority=%d, ContentHandle=(%ls)",
		Request.SeqNo, Request.Priority, *LexToString(InstallRequest.Args.ContentHandle));

	if (InstallRequest.Args.ContentHandle.IsValid() == false)
	{
		Request.Result = MakeError(UE::Core::ArgumentError(TEXT("ContentHandle"), TEXT("Invalid content handle")));
		return CompleteInstallRequest(Request);
	}

	const TSharedPtr<FOnDemandInternalContentHandle>& ContentHandle = InstallRequest.Args.ContentHandle.Handle;
	if (ContentHandle->IoStore.IsValid() == false)
	{
		// First time this content handle is used
		// Is this threadsafe? This is read in the ContentHandle destructor which is invoked by the shard ptr
		// ref controller which uses release/aquire semantics. So I believe it is safe as long as its only read 
		// in the destructor.
		ContentHandle->IoStore = IoStore.AsWeak();

#if !UE_BUILD_SHIPPING		
		if (FOnDemandContentInstallReplayRecorder* Recorder = FOnDemandContentInstallReplayRecorder::Get())
		{
			const FSharedString& DebugName = ContentHandle->DebugName;
			Recorder->RecordContentHandle(ContentHandle->HandleId(), FString::ConstructFromPtrSize(*DebugName, DebugName.Len()));
		}
#endif // !UE_BUILD_SHIPPING
	}

#if !UE_BUILD_SHIPPING	
	if (FOnDemandContentInstallReplayRecorder* Recorder = FOnDemandContentInstallReplayRecorder::Get())
	{
		const FOnDemandInstallArgs& Args = InstallRequest.Args;
		Recorder->RecordInstall(
			InstallRequest.Request->InstallerRequest,
			ContentHandle->HandleId(),
			Args.MountId,
			Args.TagSets,
			Args.PackageIds,
			Args.Priority,
			Args.Options);
	}
#endif // !UE_BUILD_SHIPPING

	TSet<FSharedOnDemandContainer> ContainersForInstallation;
	TSet<FPackageId> PackageIdsToInstall;
	if (FIoStatus Status = IoStore.GetContainersAndPackagesForInstall(
		InstallRequest.Args.MountId,
		InstallRequest.Args.TagSets,
		InstallRequest.Args.PackageIds,
		ContainersForInstallation,
		PackageIdsToInstall); !Status.IsOk())
	{
		Request.Result = MakeError(UE::Core::ArgumentError(TEXT("InstallArgs"), Status.ToString()));
		return CompleteInstallRequest(Request);
	}

#if !UE_BUILD_SHIPPING
	if (!UE::IoStore::CVars::IoStoreErrorOnRequest.IsEmpty())
	{
		if (FCString::Stristr(*ContentHandle->DebugName, *UE::IoStore::CVars::IoStoreErrorOnRequest) != nullptr || 
			FCString::Stristr(*InstallRequest.Args.DebugName, *UE::IoStore::CVars::IoStoreErrorOnRequest) != nullptr)
		{
			Request.Result = MakeError(
				UE::Core::ArgumentError(
					TEXT("InstallArgs"),
					FString::Printf(TEXT("Debug error requested on debug name %s"), *UE::IoStore::CVars::IoStoreErrorOnRequest)));
			return CompleteInstallRequest(Request);
		}
	}

	if (!UE::IoStore::CVars::IoStoreDebugRequest.IsEmpty())
	{
		if (FCString::Strstr(*ContentHandle->DebugName, *UE::IoStore::CVars::IoStoreDebugRequest) != nullptr ||
			FCString::Strstr(*InstallRequest.Args.DebugName, *UE::IoStore::CVars::IoStoreDebugRequest) != nullptr)
		{
			UE_LOGF(LogIoStoreOnDemand, Log, "Place breakpoint here to debug FOnDemandContentInstaller::ProcessInstallRequest for asset %ls", *ContentHandle->DebugName);
		}
	}
#endif

	if (Request.IsCancelled())
	{
		return CompleteInstallRequest(Request);
	}

	TSet<FIoChunkId> Missing;
	const bool bIncludeSoftReferences	= EnumHasAnyFlags(InstallRequest.Args.Options, EOnDemandInstallOptions::InstallSoftReferences);
	const bool bIncludeOptionalBulkData = EnumHasAnyFlags(InstallRequest.Args.Options, EOnDemandInstallOptions::InstallOptionalBulkData);
	Private::ResolveChunksToInstall(
		IoStore.PackageStoreBackend,
		ContainersForInstallation,
		PackageIdsToInstall,
		bIncludeSoftReferences,
		bIncludeOptionalBulkData,
		InstallRequest.ResolvedChunks,
		Missing);

	// Check the other I/O backends for missing package chunks
	if (Missing.IsEmpty() == false)
	{
		FIoDispatcher& IoDispatcher = FIoDispatcher::Get();
		TArray<FIoChunkId> MissingChunkIds;
		TUtf8StringBuilder<128> Filename;
		TStringBuilder<128> Sb;

		for (const FIoChunkId& ChunkId : Missing)
		{
			if (IoDispatcher.DoesChunkExist(ChunkId) == false)
			{
				UE_CLOGF(MissingChunkIds.Num() == 0, LogIoStoreOnDemand, Warning, "Failed to resolve the following chunk(s) for content handle '%ls':",
					*LexToString(InstallRequest.Args.ContentHandle));
				
				Sb.Reset();
				Filename.Reset();

				Sb << TEXT("PackageName='");
				TryConvertChunkIdToPackageName(ChunkId, Filename, Sb);
				Sb << TEXT("' Filename='") << Filename << TEXT("'");
				Sb << TEXT(" ChunkId='") << ChunkId << TEXT("' (") << LexToString(ChunkId.GetChunkType()) << TEXT(")");
				UE_LOGFMT(LogIoStoreOnDemand, Warning, "\t{Msg}", Sb);
				MissingChunkIds.Add(ChunkId);
			}
		}

		if (MissingChunkIds.IsEmpty() == false && !EnumHasAnyFlags(InstallRequest.Args.Options, EOnDemandInstallOptions::AllowMissingDependencies) && !IoStore::CVars::bAllowAllMissingDependencies)
		{
			Request.Result = MakeError(UE::IoStore::OnDemand::ChunkMissingError(
				UE::IoStore::OnDemand::FChunkMissingError
				{
					.MissingChunkIds = MoveTemp(MissingChunkIds),
					.DiscoveredChunksCount = InstallRequest.ResolvedChunks.Num()
				}));
			return CompleteInstallRequest(Request);
		}
	}

	if (Request.IsCancelled())
	{
		return CompleteInstallRequest(Request);
	}

	bool bExecuteRequest = false;
	{
		TUniqueLock Lock(Mutex);
		if (CanExecuteRequest(Request))
		{
			RunningRequests.Add(&Request);
			bExecuteRequest = true;
		}
	}

	if (bExecuteRequest)
	{
		// Execute immediately, let ExecuteInstallRequest handle chunk pinning
		ExecuteInstallRequest(Request);
		return;
	}

	uint64 TotalContentSize	= 0;
	uint64 TotalInstallSize	= 0;

	// Pin any already cached chunks while waiting for execution
	PinCachedChunks(
		InstallRequest, 
		[&InstallRequest, &TotalContentSize, &TotalInstallSize](const int32 ContainerIndex, const int32 EntryIndex, const bool bCached)
		{
			const uint32 DiskSize = InstallRequest.ResolvedChunks[ContainerIndex].DiskSize(EntryIndex);
			TotalContentSize += DiskSize;

			if (bCached)
			{
				return;
			}
		
			TotalInstallSize += DiskSize;
		}
	);

	InstallRequest.Progress.TotalContentSize	= TotalContentSize;
	InstallRequest.Progress.TotalInstallSize	= TotalInstallSize;
	InstallRequest.Progress.CurrentInstallSize	= 0;

	if (TotalInstallSize == 0)
	{
		if (!IoStore.InstallCache->IsEagerDefragRequired(*InstallRequest.CacheHandle))
		{
			return CompleteInstallRequest(Request);
		}
	}

	if (Request.IsCancelled())
	{
		return CompleteInstallRequest(Request);
	}

	{
		TUniqueLock Lock(Mutex);
		RequestQueue.HeapPush(&Request, RequestSortPredicate);
	}
}

void FOnDemandContentInstaller::ExecuteInstallRequest(FRequest& Request)
{
	check(Request.IsInstall());
	check(RunningRequests.Contains(&Request));

	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandContentInstaller::ExecuteInstallRequest);

	FRequest::FInstall& InstallRequest = Request.AsInstall();
	check(InstallRequest.HttpRequestHandles.IsEmpty());

	UE_LOGF(LogIoStoreOnDemand, Verbose, "Executing install request, SeqNo=%u, Priority=%d, ContentHandle=(%ls)",
		Request.SeqNo, Request.Priority, *LexToString(InstallRequest.Args.ContentHandle));

	if (Request.IsCancelled())
	{
		return CompleteInstallRequest(Request);
	}

	uint64 TotalContentSize = 0;
	uint64 TotalInstallSize	= 0;

	PinCachedChunks(
		InstallRequest, [&Request, &TotalContentSize, &TotalInstallSize](const int32 ContainerIndex, const int32 EntryIndex, const bool bCached)
		{
			FRequest::FInstall& InstallRequest = Request.AsInstall();
			const uint32 DiskSize = InstallRequest.ResolvedChunks[ContainerIndex].DiskSize(EntryIndex);
			TotalContentSize += DiskSize;

			if (bCached)
			{
				return;
			}

			InstallRequest.HttpRequestHandles.Add(FChunkHttpRequestHandle
			{
				.OwnerRequest	= &Request,
				.ContainerIndex = ContainerIndex,
				.EntryIndex		= EntryIndex
			});
			TotalInstallSize += DiskSize;
		}
	);

	// TotalInstallSize may be different now
	InstallRequest.Progress.TotalContentSize = TotalContentSize;
	InstallRequest.Progress.TotalInstallSize = TotalInstallSize;
	InstallRequest.Progress.CurrentInstallSize = 0;

	if (InstallRequest.HttpRequestHandles.IsEmpty())
	{
		IoStore.InstallCache->EagerDefrag(*InstallRequest.CacheHandle);
		return CompleteInstallRequest(Request);
	}

	if (EnumHasAnyFlags(InstallRequest.Args.Options, EOnDemandInstallOptions::DoNotDownload))
	{
		Request.Result = MakeError(UE::IoStore::OnDemand::DownloadRequired());
		return CompleteInstallRequest(Request);
	}

	if (Request.IsCancelled())
	{
		return CompleteInstallRequest(Request);
	}

	NotifyInstallProgress(Request);

	FIoHttpBatch Batch = FHttpIoDispatcher::NewBatch();

	constexpr const int32 HttpRetryCount = 2;
	for (FChunkHttpRequestHandle& HttpRequest : InstallRequest.HttpRequestHandles)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandContentInstaller::IssueRequest);
		FOnDemandChunkInfoList& ChunkInfo		= InstallRequest.ResolvedChunks[HttpRequest.ContainerIndex];
		const FIoChunkId& ChunkId				= ChunkInfo.Id(HttpRequest.EntryIndex);
		const FOnDemandChunkHash& ChunkHash		= ChunkInfo.Hash(HttpRequest.EntryIndex);
		const uint8 HttpCategory				= 1; // see FOnDemandHttpThread

		FChunkDownloadRequest* PendingChunkRequest = nullptr;
		for (auto It = PendingChunkDownloads.CreateKeyIterator(ChunkHash); It; ++It)
		{
			FChunkDownloadRequest& ChunkDownload = It.Value();
			if (ChunkDownload.bChunkCanceled == false)
			{
				PendingChunkRequest = &ChunkDownload;
				break; // Should be a max of one active request for any chunk
			}
		}

		if (PendingChunkRequest)
		{
			PendingChunkRequest->ChunkRequestHttpHandles.AddHead(&HttpRequest);
		}
		else
		{
			PendingChunkRequest = &PendingChunkDownloads.Add(ChunkHash);
			PendingChunkRequest->RequestId = Request.SeqNo;
			PendingChunkRequest->ChunkRequestHttpHandles.AddHead(&HttpRequest);

			// Setting HTTP flags to None to not use the HTTP cache
			FIoHttpOptions HttpOptions = FIoHttpOptions(Request.Priority, HttpRetryCount, EIoHttpFlags::None);
			HttpOptions.SetCategory(HttpCategory);
			HttpOptions.SetRange(FIoHttpRange(
				ChunkInfo.PartitionOffset(HttpRequest.EntryIndex), 
				ChunkInfo.PartitionOffset(HttpRequest.EntryIndex) + ChunkInfo.DiskSize(HttpRequest.EntryIndex)));
			HttpOptions.SetMetadata(FIoHttpMetadata(ChunkId));

			UE_LOGF(LogIoStoreOnDemand, VeryVerbose, "Created request for chunk: ChunkHash=%ls, RequestId=%u",
				*LexToString(ChunkHash), PendingChunkRequest->RequestId);

			TRACE_COUNTER_INCREMENT(OnDemandContentInstaller_HttpRequestsIssued);
			PendingChunkRequest->HttpHandle = Batch.Get(
				ChunkInfo.HostGroupName(),
				ChunkInfo.RelativeUrl(),
				FIoHttpHeaders(),
				HttpOptions,
				ChunkInfo.PartitionHash(HttpRequest.EntryIndex),
				[this, ChunkHash, RequestId = PendingChunkRequest->RequestId](FIoHttpResponse&& HttpResponse)
				{
					InstallerPipe.Launch(
						TEXT("ProcessIoStoreOnDemandDownloadedChunk"),
						[this, ChunkHash, RequestId, HttpResponse = MoveTemp(HttpResponse)]() mutable
						{
							FIoBuffer Chunk = HttpResponse.GetBody();
							ProcessDownloadedChunk(ChunkHash, RequestId, HttpResponse.GetErrorCode(), HttpResponse.GetStatusCode(), MoveTemp(Chunk));
						},
						UE::Tasks::ETaskPriority::BackgroundLow);
				});
		}

	}

	Batch.Issue();

	InstallRequest.bHttpRequestsIssued = true;

	IoStore.InstallCache->EagerDefrag(*InstallRequest.CacheHandle);
}

void FOnDemandContentInstaller::ExecutePurgeRequest(FRequest& Request)
{
	check(Request.IsPurge());
	check(RunningRequests.Num() == 1 && RunningRequests.Contains(&Request));

	FRequest::FPurge& PurgeRequest			= Request.AsPurge();
	const bool bDefrag						= EnumHasAnyFlags(PurgeRequest.Args.Options, EOnDemandPurgeOptions::Defrag);
	const uint64* BytesToPurge				= PurgeRequest.Args.BytesToPurge.GetPtrOrNull();
	const EOnDemandInstallCasType CasType	= PurgeRequest.Args.CasType;

	UE_LOGF(LogIoStoreOnDemand, Log, "Executing purge request, CasType=%ls, BytesToPurge=%llu, Defrag='%ls'",
		LexToString(CasType), BytesToPurge != nullptr ? *BytesToPurge : -1, bDefrag ? TEXT("True") : TEXT("False"));

	if (Request.IsCancelled() == false)
	{
		Request.Result = IoStore.InstallCache->PurgeAllUnreferenced(CasType, bDefrag, BytesToPurge);
	}

	CompletePurgeRequest(Request);
}

void FOnDemandContentInstaller::ExecuteDefragRequest(FRequest& Request)
{
	check(Request.IsDefrag());
	check(RunningRequests.Num() == 1 && RunningRequests.Contains(&Request));

	FRequest::FDefrag& DefragRequest		= Request.AsDefrag();
	const uint64* BytesToFree				= DefragRequest.Args.BytesToFree.GetPtrOrNull();
	const EOnDemandInstallCasType CasType	= DefragRequest.Args.CasType;

	UE_LOGF(LogIoStoreOnDemand, Log, "Executing defrag request, CasType=%ls, BytesToFree=%llu",
		LexToString(CasType), BytesToFree != nullptr ? *BytesToFree : -1);

	if (Request.IsCancelled() == false)
	{
		Request.Result = IoStore.InstallCache->DefragAll(CasType, BytesToFree);
	}

	CompleteDefragRequest(Request);
}

void FOnDemandContentInstaller::ExecuteVerifyRequest(FRequest& Request)
{
	check(Request.IsVerify());
	check(RunningRequests.Num() == 1 && RunningRequests.Contains(&Request));

	UE_LOGF(LogIoStoreOnDemand, Log, "Executing verify cache request");

	if (Request.IsCancelled() == false)
	{
		Request.Result = IoStore.InstallCache->Verify();
	}

	CompleteVerifyRequest(Request); 
}

void FOnDemandContentInstaller::ExectuteFlushLastAccessRequest(FRequest& Request)
{
	check(Request.IsFlushLastAccess());
	check(RunningRequests.Num() == 1 && RunningRequests.Contains(&Request));

	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandContentInstaller::ExectuteFlushLastAccessRequest);

	const FRequest::FFlushLastAccess& FlushLastAccessReqest = Request.AsFlushLastAccess();

	UE_LOGF(LogIoStoreOnDemand, VeryVerbose, "Executing flush last access request");

	if (Request.IsCancelled())
	{
		CompleteFlushLastAccessRequest(Request);
		return;
	}

	TArray<FOnDemandChunkHash> TouchedChunks;
	for (const TPair<FSharedOnDemandContainer, TBitArray<>>& Pair : FlushLastAccessReqest.Args.RemovedChunkEntryReferences)
	{
		const FSharedOnDemandContainer& Container = Pair.Key;
		const TBitArray<>& RemovedRefs = Pair.Value;
		check(RemovedRefs.Num() == Container->ChunkEntries.Num());

		for (TBitArray<>::FConstIterator It(RemovedRefs); It; ++It)
		{
			if (It.GetValue())
			{
				TouchedChunks.Add(Container->ChunkEntries[It.GetIndex()].Hash);
			}
		}
	}

	TStaticArray<bool, EOnDemandInstallCasType::Count> bLastAccessDirty(EInPlace::InPlace, false);
	IoStore.InstallCache->UpdateLastAccess(TouchedChunks, bLastAccessDirty);
	for (bool& bCasLastAccessDirty : bLastAccessDirty)
	{
		bCasLastAccessDirty = bCasLastAccessDirty || FlushLastAccessReqest.Args.bForceLastAccessDirty;
	}
	
	if (Algo::AnyOf(bLastAccessDirty))
	{
		UE_LOGF(LogIoStoreOnDemand, Log, "Flushing last access to journal");
		Request.Result = IoStore.InstallCache->FlushLastAccess(bLastAccessDirty);
	}

	CompleteFlushLastAccessRequest(Request);
}

void FOnDemandContentInstaller::ProcessDownloadedChunk(
	const FOnDemandChunkHash& ChunkHash, uint32 RequestId, EIoErrorCode InErrorCode, uint32 HttpStatusCode, FIoBuffer&& Chunk)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandContentInstaller::ProcessDownloadedChunk);

	// Find the pending download for this chunk
	bool bFoundRequest = false;
	auto It = PendingChunkDownloads.CreateKeyIterator(ChunkHash);
	for (; It; ++It)
	{
		FChunkDownloadRequest& ChunkRequest = It.Value();
		if (ChunkRequest.RequestId == RequestId)
		{
			bFoundRequest = true;
			break;
		}
	}
	check(bFoundRequest);
	FChunkDownloadRequest& ChunkRequest = It.Value();

	FResult Result = MakeValue();

	const bool bHttpCancelled = (InErrorCode == EIoErrorCode::Cancelled);
	check(ChunkRequest.bChunkCanceled || !bHttpCancelled);

	const bool bHttpOk = HttpStatusCode > 199 && HttpStatusCode < 300 && Chunk.GetSize() > 0;
	if (bHttpOk == false)
	{
#if COUNTERSTRACE_ENABLED
		if (InErrorCode == EIoErrorCode::Cancelled)
		{
			TRACE_COUNTER_INCREMENT(OnDemandContentInstaller_HttpRequestsActuallyCanceled);
		}
		else
		{
			TRACE_COUNTER_INCREMENT(OnDemandContentInstaller_HttpRequestsFail);
		}
#endif

		Result = MakeError(IoStore::OnDemand::HttpError(HttpStatusCode));
	}
	else if (ChunkRequest.bChunkCanceled)
	{
		Result = MakeError(Core::CancellationError());
	}
	else
	{
		const FOnDemandChunkHash VerifyChunkHash = FOnDemandChunkHash::HashBuffer(Chunk.GetView());
		if (ChunkHash == VerifyChunkHash)
		{
			TRACE_COUNTER_INCREMENT(OnDemandContentInstaller_HttpRequestsSuccess);

			// Peek the first request waiting on the chunk to determine chunk type
			EIoChunkType ChunkType = EIoChunkType::Invalid;
			{
				const FChunkHttpRequestHandle& HttpRequest = *ChunkRequest.ChunkRequestHttpHandles.PeekHead();
				const FRequest& Request = *HttpRequest.OwnerRequest;
				const FRequest::FInstall& InstallRequest = Request.AsInstall();
				const FOnDemandChunkInfoList& Container = InstallRequest.ResolvedChunks[HttpRequest.ContainerIndex];
				const FIoChunkId& ChunkId = Container.Id(HttpRequest.EntryIndex);
				ChunkType = ChunkId.GetChunkType();
			}

			using namespace UE::IoStore::OnDemand;

			FResult PutResult = IoStore.InstallCache->PutChunk(ChunkType, MoveTemp(Chunk), ChunkHash);
			if (PutResult.HasError() && PutResult.GetError() != InstallerCacheFullError)
			{
				Result = MoveTemp(PutResult);
			}
			else if (PutResult.HasError() && PutResult.GetError() == InstallerCacheFullError)
			{
				FInstallCachePurgeHandle PurgeHandle = IoStore.InstallCache->BeginPurge(ChunkType);
				for (const TPair<FOnDemandChunkHash, FChunkDownloadRequest>& Kv : PendingChunkDownloads)
				{
					const FChunkDownloadRequest& PendingChunkRequest = Kv.Value;
					if (PendingChunkRequest.bChunkCanceled)
					{
						continue;
					}

					// Only need to check the first request waiting on the chunk.
					// The size can come from any container with the chunk.
					const FChunkHttpRequestHandle* ChunkHttpRequestHandle = PendingChunkRequest.ChunkRequestHttpHandles.PeekHead();
					if (!ensure(ChunkHttpRequestHandle))
					{
						continue;
					}

					const FRequest& Request = *ChunkHttpRequestHandle->OwnerRequest;
					const FRequest::FInstall& InstallRequest = Request.AsInstall();
					const FOnDemandChunkInfoList& Container = InstallRequest.ResolvedChunks[ChunkHttpRequestHandle->ContainerIndex];
					const FIoChunkId& ChunkId = Container.Id(ChunkHttpRequestHandle->EntryIndex);
					IoStore.InstallCache->AddToPurge(PurgeHandle, ChunkId.GetChunkType(), Container.DiskSize(ChunkHttpRequestHandle->EntryIndex));
				}

				Result = IoStore.InstallCache->Purge(PurgeHandle);
				if (Result.HasValue())
				{
					PutResult = IoStore.InstallCache->PutChunk(ChunkType, MoveTemp(Chunk), ChunkHash);
					if (PutResult.HasError())
					{
						if (PutResult.GetError() == InstallerCacheFullError)
						{
							UE_LOGF(LogIoStoreOnDemand, Error, "Insuffucient cache space after purge, this should never happen. ChunkType: %ls", *LexToString(ChunkType));
							ensure(false);
						}

						Result = MoveTemp(PutResult);
					}
				}
			}
		}
		else
		{
			TRACE_COUNTER_INCREMENT(OnDemandContentInstaller_HttpRequestsFail);

			Result = MakeError(IoStore::OnDemand::ChunkHashError(
				IoStore::OnDemand::FChunkHashError
				{
					.ChunkId = FIoChunkId::InvalidChunkId,
					.ExpectedHash = LexToString(ChunkHash),
					.ActualHash = LexToString(VerifyChunkHash)
				}));
		}
	}

	static constexpr const TCHAR LogFmt[] = TEXT("ProcessDownloadedChunk: ChunkHash=%s, RequestId=%u, DownloadResult=%s");
	if (ChunkRequest.bChunkCanceled)
	{
		UE_LOG(LogIoStoreOnDemand, Verbose, LogFmt, *LexToString(ChunkHash), ChunkRequest.RequestId, TEXT("Cancelled"));
	}
	else if (Result.HasError())
	{
		UE_LOG(LogIoStoreOnDemand, Warning, LogFmt, *LexToString(ChunkHash), ChunkRequest.RequestId, *Result.GetError().GetErrorMessage(true).ToString());
	}
	else
	{
		UE_LOG(LogIoStoreOnDemand, VeryVerbose, LogFmt, *LexToString(ChunkHash), ChunkRequest.RequestId, TEXT("OK"));
	}

	while (FChunkHttpRequestHandle* ChunkHttpRequestHandle = ChunkRequest.ChunkRequestHttpHandles.PopHead())
	{
		OnProcessDownloadedChunkNotifyRequest(*ChunkHttpRequestHandle, Result, ChunkRequest.bChunkCanceled);
	}

	// Delete from pending downloads
	It.RemoveCurrent();
}

void FOnDemandContentInstaller::OnProcessDownloadedChunkNotifyRequest(
	const FChunkHttpRequestHandle& HttpRequest, const FResult& ChunkDownloadResult, bool bChunkCancelled)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandContentInstaller::OnProcessDownloadedChunkNotifyRequest);

	FRequest& Request = *HttpRequest.OwnerRequest;
	FRequest::FInstall& InstallRequest = Request.AsInstall();
	FOnDemandChunkInfoList& Container = InstallRequest.ResolvedChunks[HttpRequest.ContainerIndex];
	const FOnDemandChunkEntry& ChunkEntry = Container.ChunkEntry(HttpRequest.EntryIndex);
	const FIoChunkId& ChunkId = Container.Id(HttpRequest.EntryIndex);

	bool bSetErrorOnRequest = false;
	if (Request.IsOk())
	{
		if (!ensureMsgf(!bChunkCancelled, TEXT("Received cancelled http callback for active Request %s ! This should never happen."), *InstallRequest.Args.DebugName))
		{
			bSetErrorOnRequest = true;
			Request.Result = ChunkDownloadResult;
		}
		else if (const UnifiedError::FError* Error = ChunkDownloadResult.TryGetError())
		{
			bSetErrorOnRequest = true;

			// Set the chunk Id from this container if needed
			const IoStore::OnDemand::FChunkHashError* ErrorContext =
				Error->GetErrorContext<IoStore::OnDemand::ChunkHashError>();
			if (ErrorContext)
			{
				// FError does not deep copy by design.
				// We need a different error per-request so we need to re-instance it.
				Request.Result = MakeError(IoStore::OnDemand::ChunkHashError(
					IoStore::OnDemand::FChunkHashError
					{
						.ChunkId = ChunkId,
						.ExpectedHash = ErrorContext->ExpectedHash,
						.ActualHash = ErrorContext->ActualHash
					}));
			}
			else
			{
				Request.Result = ChunkDownloadResult;
			}
		}
		else if (CVars::bPinDownloadedChunksImmediately)
		{
			// TODO: refactor chunk pinning so we don't need to this here
			FOnDemandContentHandle& ContentHandle = InstallRequest.Args.ContentHandle;
			FOnDemandChunkInfoList& ChunkInfoList = InstallRequest.ResolvedChunks[HttpRequest.ContainerIndex];

			{
				TUniqueLock Lock(ChunkInfoList.SharedContainer->ReferencesMutex);
				const FSharedOnDemandContainer& C = ChunkInfoList.SharedContainer;
				FOnDemandChunkEntryReferences& References = C->FindOrAddChunkEntryReferences(*ContentHandle.Handle);
				References.Indices[HttpRequest.EntryIndex] = true;
			}
		}
	}

	static constexpr const TCHAR LogFmt[] = TEXT("Install progress %.2lf/%.2lf MiB, SeqNo=%u, Priority=%d, ContentHandle=(%s), ChunkId='%s', ChunkSize=%.2lf KiB, DownloadResult=%s");
	if (bChunkCancelled)
	{
		UE_LOG(LogIoStoreOnDemand, VeryVerbose, LogFmt,
			double(InstallRequest.Progress.CurrentInstallSize) / 1024.0 / 1024.0, double(InstallRequest.Progress.TotalInstallSize) / 1024.0 / 1024.0,
			Request.SeqNo, Request.Priority, *LexToString(InstallRequest.Args.ContentHandle), *LexToString(ChunkId), double(ChunkEntry.GetDiskSize()) / 1024.0,
			TEXT("Cancelled"));
	}
	else if (ChunkDownloadResult.HasError())
	{
		UE_LOG(LogIoStoreOnDemand, Warning, LogFmt,
			double(InstallRequest.Progress.CurrentInstallSize) / 1024.0 / 1024.0, double(InstallRequest.Progress.TotalInstallSize) / 1024.0 / 1024.0,
			Request.SeqNo, Request.Priority, *LexToString(InstallRequest.Args.ContentHandle), *LexToString(ChunkId), double(ChunkEntry.GetDiskSize()) / 1024.0,
			*ChunkDownloadResult.GetError().GetErrorMessage(true).ToString());
	}
	else
	{
		InstallRequest.Progress.CurrentInstallSize += ChunkEntry.GetDiskSize();

		UE_LOG(LogIoStoreOnDemand, Verbose, LogFmt,
			double(InstallRequest.Progress.CurrentInstallSize) / 1024.0 / 1024.0, double(InstallRequest.Progress.TotalInstallSize) / 1024.0 / 1024.0,
			Request.SeqNo, Request.Priority, *LexToString(InstallRequest.Args.ContentHandle), *LexToString(ChunkId), double(ChunkEntry.GetDiskSize()) / 1024.0,
			TEXT("OK"));
	}

	IoStore.InstallCache->PostPutChunk(*InstallRequest.CacheHandle, ChunkId.GetChunkType());
	const bool bCompleted = ++InstallRequest.DownloadedChunkCount >= InstallRequest.HttpRequestHandles.Num();
	if (bCompleted)
	{
		CompleteInstallRequest(Request);
	}
	else
	{
		NotifyInstallProgress(Request);

		// Only try to cancel requests the first time an error is set
		if (bSetErrorOnRequest) 
		{
			int32 NumCancelled = 0;
			TryCancelHttpRequestsForInstallRequest(InstallRequest, NumCancelled);
			UE_CLOGF(NumCancelled == 0, LogIoStoreOnDemand, Warning, "Cancelled %d HTTP request(s) due to install error", NumCancelled);
		}
	}
}

void FOnDemandContentInstaller::TryCancelHttpRequestsForInstallRequest(FRequest::FInstall& InstallRequest, int32& OutNumCancelled)
{
	int32 NumCancelled = 0;

	if (CVars::bContentInstallerEnableHttpCancel == false)
	{
		OutNumCancelled = NumCancelled;
		return;
	}

	for (FChunkHttpRequestHandle& PendingHttpRequest : InstallRequest.HttpRequestHandles)
	{
		const FOnDemandChunkInfoList& PendingContainer = InstallRequest.ResolvedChunks[PendingHttpRequest.ContainerIndex];
		const FOnDemandChunkHash& PendingChunkHash = PendingContainer.Hash(PendingHttpRequest.EntryIndex);

		FChunkDownloadRequest* ChunkDownload = nullptr;
		for (auto It = PendingChunkDownloads.CreateKeyIterator(PendingChunkHash); It; ++It)
		{
			FChunkDownloadRequest& FoundChunkDownload = It.Value();
			if (FoundChunkDownload.bChunkCanceled == false)
			{
				ChunkDownload = &FoundChunkDownload;
				break; // Should be a max of one active request for any chunk
			}
		}

		if (ChunkDownload)
		{
			bool bCancelHttpRequest = true;
			for (FChunkHttpRequestHandle& Handle : ChunkDownload->ChunkRequestHttpHandles)
			{
				if (Handle.OwnerRequest->IsOk())
				{
					bCancelHttpRequest = false;
					break;
				}
			}

			if (bCancelHttpRequest)
			{
				ChunkDownload->HttpHandle.Cancel();
				ChunkDownload->bChunkCanceled = true;
				++NumCancelled;
				UE_LOGF(LogIoStoreOnDemand, Verbose, "Cancelled chunk download: hash - %ls, Id - %u", *LexToString(PendingChunkHash), ChunkDownload->RequestId);

				// ProcessDownloadedChunk will remove the chunk request
			}
		}
	}

	TRACE_COUNTER_ADD(OnDemandContentInstaller_HttpRequestsTryCanceled, NumCancelled);

	OutNumCancelled = NumCancelled;
}

void FOnDemandContentInstaller::NotifyInstallProgress(FRequest& Request)
{
	ensure(Request.IsInstall());

	FRequest::FInstall& InstallRequest = Request.AsInstall();

	if (!InstallRequest.OnProgress)
	{
		return;
	}

	const uint64 Cycles = FPlatformTime::Cycles64();
	const double SecondsSinceLastProgress = FPlatformTime::ToSeconds64(Cycles - InstallRequest.LastProgressCycles);
	if (InstallRequest.bNotifyingProgressOnGameThread.load(std::memory_order_seq_cst) || SecondsSinceLastProgress < .25)
	{
		return;
	}
	InstallRequest.LastProgressCycles = Cycles;

	//TODO: Remove support for notifying progress on the game thread
	FOnDemandInstallProgress Progress = InstallRequest.Progress;
	if (EnumHasAnyFlags(InstallRequest.Args.Options, EOnDemandInstallOptions::CallbackOnGameThread))
	{
		InstallRequest.bNotifyingProgressOnGameThread = true;
		ExecuteOnGameThread(
			UE_SOURCE_LOCATION,
			[&InstallRequest, Progress]()
			{
				InstallRequest.OnProgress(InstallRequest.Progress);
				InstallRequest.bNotifyingProgressOnGameThread = false;
			});
	}
	else
	{
		InstallRequest.OnProgress(Progress);
	}
}

void FOnDemandContentInstaller::DestroyRequest(FRequest& Request)
{
	// Call destructor outside the lock to avoid double locking
	Request.~FRequest();

	TUniqueLock Lock(Mutex);
	RequestAllocator.Free(&Request);
}

void FOnDemandContentInstaller::CompleteInstallRequest(FRequest& Request)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandContentInstaller::CompleteInstallRequest);

	FRequest::FInstall& InstallRequest = Request.AsInstall();

	// This must be called in CompleteInstallRequest so it gets called 
	// even if a reqeust doesn't queue up any downloads because some chunks may still be
	// in the pending buffer.
	if (Request.IsOk())
	{
		Request.Result = IoStore.InstallCache->ConditionallyFlushInstall(*InstallRequest.CacheHandle);
	}

#if !UE_BUILD_SHIPPING
	TArray<FString> DebugInfo;
#endif

	const uint64 ResolvedChunkCount = Algo::TransformAccumulate(
		InstallRequest.ResolvedChunks,
		[](const FOnDemandChunkInfoList& ContainerChunks)
		{
			return ContainerChunks.Indices.Num();
		},
		uint64(0)
	);

	// Mark all resolved chunk(s) as referenced by the content handle and notify the package store to update
	if (Request.IsOk() && ResolvedChunkCount > 0)
	{
		FOnDemandContentHandle& ContentHandle = InstallRequest.Args.ContentHandle;

		for (const FOnDemandChunkInfoList& ResolvedChunks : InstallRequest.ResolvedChunks)
		{
#if !UE_BUILD_SHIPPING
			const FSharedOnDemandContainer& Container = ResolvedChunks.SharedContainer;
			IoStore.InstallCache->PinChunks(*InstallRequest.CacheHandle, *ContentHandle.Handle, ResolvedChunks,
				[&DebugInfo, &Container](int32 EntryIndex)
				{
					if (UE_LOG_ACTIVE(LogIoStoreOnDemand, Verbose))
					{
						DebugInfo.Add(FString::Printf(TEXT("ID:%s, Size:%d"), *LexToString(Container->ChunkEntries[EntryIndex].Hash), Container->ChunkEntries[EntryIndex].GetDiskSize()));
					}
				}
			);
#else
			IoStore.InstallCache->PinChunks(*InstallRequest.CacheHandle, *ContentHandle.Handle, ResolvedChunks, [](int32){});
#endif
		}

		IoStore.PackageStoreBackend->NeedsUpdate(EOnDemandPackageStoreUpdateMode::ReferencedPackages);
	}

	const bool bCancelled			= Request.IsCancelled();
	const uint64 DurationCycles		= FPlatformTime::Cycles64() - Request.StartTimeCycles;
	const double DurationInSeconds	= FPlatformTime::ToSeconds64(DurationCycles);
	const double CacheHitRatio		= InstallRequest.Progress.TotalContentSize > 0
		? double(InstallRequest.Progress.TotalContentSize - InstallRequest.Progress.TotalInstallSize) / double(InstallRequest.Progress.TotalContentSize)
		: 1.0;

#if !UE_BUILD_SHIPPING
	FString DebugInfoString = FString::Join(DebugInfo, TEXT(","));
	UE_LOGF(LogIoStoreOnDemand, Verbose, "ContentHandle=(%ls), ResolvedChunks(%ls)", *LexToString(InstallRequest.Args.ContentHandle), *DebugInfoString);
#endif
	UE_LOGF(LogIoStoreOnDemand, Verbose, "Install request completed, Result='%ls', SeqNo=%u, Priority=%d, ContentHandle=(%ls), ContentSize=%.2lf MiB, InstallSize=%.2lf MiB, CacheHitRatio=%d%%, Duration=%dms",
		*Request.GetErrorString(), Request.SeqNo, Request.Priority, *LexToString(InstallRequest.Args.ContentHandle), double(InstallRequest.Progress.TotalContentSize) / 1024.0 / 1024.0,
		double(InstallRequest.Progress.TotalInstallSize) / 1024.0 / 1024.0, int32(CacheHitRatio * 100), Private::SecondsToMillicseconds(DurationInSeconds));

	FOnDemandContentInstallerStats::OnRequestCompleted(
		Request.Result,
		ResolvedChunkCount,
		InstallRequest.Progress.TotalContentSize,
		static_cast<uint64>(InstallRequest.HttpRequestHandles.Num()),
		InstallRequest.Progress.CurrentInstallSize,
		CacheHitRatio,
		DurationCycles);

	FOnDemandInstallResult InstallResult;
	InstallResult.Progress			= InstallRequest.Progress;
	InstallResult.DurationInSeconds	= DurationInSeconds;
	InstallResult.Error				= Request.ConsumeError();

#if !UE_BUILD_SHIPPING	
	if (FOnDemandContentInstallReplayRecorder* Recorder = FOnDemandContentInstallReplayRecorder::Get())
	{
		Recorder->RecordInstallDestroyed(InstallRequest.Request->InstallerRequest);
	}
#endif // !UE_BUILD_SHIPPING

	{
		TUniqueLock Lock(Mutex);

		InstallRequest.Request->InstallerRequest = 0;
		RunningRequests.RemoveSingleSwap(&Request);
	}

	TryExecuteNextRequest();

	const FOnDemandInstallRequest::EStatus FinalRequestStatus = bCancelled
		? FOnDemandInstallRequest::EStatus::Cancelled
		: InstallResult.Error.IsSet()
			? FOnDemandInstallRequest::EStatus::Error
			: FOnDemandInstallRequest::Ok;

	if (!InstallRequest.OnCompleted)
	{
		InstallRequest.Request->Status.store(FinalRequestStatus);
		DestroyRequest(Request);
		return;
	}

	// Do this before the callback in case the callback triggers any further API calls
	InstallRequest.Request->Status.store(FOnDemandInstallRequest::PendingCallbacks);

	const bool bCallbackOnGameThread = EnumHasAnyFlags(InstallRequest.Args.Options, EOnDemandInstallOptions::CallbackOnGameThread);
	if (bCallbackOnGameThread)
	{
		ExecuteOnGameThread(
			UE_SOURCE_LOCATION,
			[this, &Request, InstallResult = MoveTemp(InstallResult), FinalRequestStatus]() mutable
			{
				FRequest::FInstall& InstallRequest		= Request.AsInstall();
				FOnDemandInstallCompleted OnCompleted	= MoveTemp(InstallRequest.OnCompleted);

				ensure(InstallRequest.bNotifyingProgressOnGameThread == false);
				OnCompleted(MoveTemp(InstallResult));
				InstallRequest.Request->Status.store(FinalRequestStatus);
				DestroyRequest(Request);
			});
	}
	else
	{
		FOnDemandInstallCompleted OnCompleted = MoveTemp(InstallRequest.OnCompleted);
		OnCompleted(MoveTemp(InstallResult));
		InstallRequest.Request->Status.store(FinalRequestStatus);
		DestroyRequest(Request);
	}
}

void FOnDemandContentInstaller::CompletePurgeRequest(FRequest& Request)
{
	check(RunningRequests.Num() == 1 && RunningRequests.Contains(&Request));

	const double DurationInSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - Request.StartTimeCycles);
	
	UE_LOGF(LogIoStoreOnDemand, Log, "Purge request completed, Result='%ls', Duration=%d ms",
		*Request.GetErrorString(), Private::SecondsToMillicseconds(DurationInSeconds));
	
	FOnDemandPurgeResult PurgeResult;
	PurgeResult.DurationInSeconds	= DurationInSeconds;
	PurgeResult.Error				= Request.ConsumeError();
	
	const bool bCallbackOnGameThread	= EnumHasAnyFlags(Request.AsPurge().Args.Options, EOnDemandPurgeOptions::CallbackOnGameThread);
	FOnDemandPurgeCompleted OnCompleted = MoveTemp(Request.AsPurge().OnCompleted);
	{
		TUniqueLock Lock(Mutex);
		RunningRequests.RemoveSingleSwap(&Request);
	}
	DestroyRequest(Request);

	TryExecuteNextRequest();

	if (!OnCompleted)
	{
		return;
	}

	if (bCallbackOnGameThread)
	{
		ExecuteOnGameThread(
			UE_SOURCE_LOCATION,
			[OnCompleted = MoveTemp(OnCompleted), PurgeResult = MoveTemp(PurgeResult)]() mutable
			{
				OnCompleted(MoveTemp(PurgeResult));
			});
	}
	else
	{
		OnCompleted(MoveTemp(PurgeResult));
	}
}

void FOnDemandContentInstaller::CompleteDefragRequest(FRequest& Request)
{
	check(RunningRequests.Num() == 1 && RunningRequests.Contains(&Request));

	const double DurationInSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - Request.StartTimeCycles);

	UE_LOGF(LogIoStoreOnDemand, Log, "Defrag request completed, Result='%ls', Duration=%d ms",
		*Request.GetErrorString(), Private::SecondsToMillicseconds(DurationInSeconds));

	FOnDemandDefragResult DefragResult;
	DefragResult.DurationInSeconds	= DurationInSeconds;
	DefragResult.Error				= Request.ConsumeError();

	const bool bCallbackOnGameThread		= EnumHasAnyFlags(Request.AsDefrag().Args.Options, EOnDemandDefragOptions::CallbackOnGameThread);
	FOnDemandDefragCompleted OnCompleted	= MoveTemp(Request.AsDefrag().OnCompleted);
	{
		TUniqueLock Lock(Mutex);
		RunningRequests.RemoveSingleSwap(&Request);
	}
	DestroyRequest(Request);

	TryExecuteNextRequest();

	if (!OnCompleted)
	{
		return;
	}

	if (bCallbackOnGameThread)
	{
		ExecuteOnGameThread(
			UE_SOURCE_LOCATION,
			[OnCompleted = MoveTemp(OnCompleted), DefragResult = MoveTemp(DefragResult)]() mutable
			{
				OnCompleted(MoveTemp(DefragResult));
			});
	}
	else
	{
		OnCompleted(MoveTemp(DefragResult));
	}
}

void FOnDemandContentInstaller::CompleteVerifyRequest(FRequest& Request)
{
	const double DurationInSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - Request.StartTimeCycles);

	UE_LOGF(LogIoStoreOnDemand, Log, "Verify request completed, Result='%ls', Duration=%d ms",
		*Request.GetErrorString(), Private::SecondsToMillicseconds(DurationInSeconds));

	FOnDemandVerifyCacheResult VerifyResult;
	VerifyResult.DurationInSeconds	= DurationInSeconds;
	VerifyResult.Error				= Request.ConsumeError();

	FOnDemandVerifyCacheCompleted OnCompleted = MoveTemp(Request.AsVerify().OnCompleted);
	{
		TUniqueLock Lock(Mutex);
		RunningRequests.RemoveSingleSwap(&Request);
	}
	DestroyRequest(Request);

	TryExecuteNextRequest();

	if (!OnCompleted)
	{
		return;
	}

	OnCompleted(MoveTemp(VerifyResult));
}

void FOnDemandContentInstaller::CompleteFlushLastAccessRequest(FRequest& Request)
{
	const double DurationInSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - Request.StartTimeCycles);

	static constexpr const TCHAR FmtStr[] = TEXT("Flush last access request completed, Result='%s', Duration=%d ms");
	if (Request.Result.HasError())
	{
		UE_LOG(LogIoStoreOnDemand, Error, FmtStr, *Request.GetErrorString(), Private::SecondsToMillicseconds(DurationInSeconds));
	}
	else
	{
		UE_LOG(LogIoStoreOnDemand, VeryVerbose, FmtStr, *Request.GetErrorString(), Private::SecondsToMillicseconds(DurationInSeconds));
	}

	FOnDemandFlushLastAccessResult FlushResult;
	FlushResult.DurationInSeconds	= DurationInSeconds;
	FlushResult.Error				= Request.ConsumeError();

	FOnDemandFlushLastAccessCompleted OnCompleted = MoveTemp(Request.AsFlushLastAccess().OnCompleted);
	{
		TUniqueLock Lock(Mutex);
		RunningRequests.RemoveSingleSwap(&Request);
	}
	DestroyRequest(Request);

	TryExecuteNextRequest();

	if (!OnCompleted)
	{
		return;
	}

	OnCompleted(MoveTemp(FlushResult));
}

void FOnDemandContentInstaller::Shutdown()
{
	bShuttingDown					= true;
	const double WaitTimeoutSeconds	= 5.0;
	const uint64 StartTimeCycles	= FPlatformTime::Cycles64();

	// Cancel current requests
	{
		TUniqueLock Lock(Mutex);
		for (FRequest* Request : RunningRequests)
		{
			Request->TryCancel();
		}
	}

	// Need special handling to cancel install requests
	InstallerPipe.Launch(TEXT("OnDemandContentInstallerShutdown"),
		[this]
		{
			TUniqueLock Lock(Mutex);
			for (FRequest* Request : RunningRequests)
			{
				if (Request->IsInstall() == false)
				{
					continue;
				}

				FRequest::FInstall& Install = Request->AsInstall();

				if (Install.bHttpRequestsIssued)
				{
					int32 NumCancelled = 0;
					TryCancelHttpRequestsForInstallRequest(Install, NumCancelled);
					UE_LOGF(LogIoStoreOnDemand, Verbose, "Cancelled %d HTTP request(s) due to install request cancellation", NumCancelled);
				}
			}
		}
	);

	// Wait for the current request(s) to finish
	for (;;)
	{
		double WaitTimeSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTimeCycles);
		if (WaitTimeSeconds > WaitTimeoutSeconds)
		{
			UE_LOGF(LogIoStoreOnDemand, Warning, "Content installer shutdown cancelled after %.2lf", WaitTimeSeconds);
			break;
		}

		InstallerPipe.WaitUntilEmpty(FTimespan::FromSeconds(1.0));
		{
			TUniqueLock Lock(Mutex);
			if (RunningRequests.IsEmpty())
			{
				UE_LOGF(LogIoStoreOnDemand, Display, "Content installer shutdown flushed requests after %.2lf", WaitTimeSeconds);
				break;
			}
		}
	}

	{
		TUniqueLock Lock(Mutex);
		UE_CLOGF(RunningRequests.IsEmpty() == false, LogIoStoreOnDemand, Error, "Content installer has still inflight request(s) while shutting down");
		RunningRequests.Reset();
	}

	// Cancel all remaining request(s)
	for (;;)
	{
		FRequest* NextRequest = nullptr;
		{
			TUniqueLock Lock(Mutex);
			if (RequestQueue.IsEmpty() == false)
			{
				RequestQueue.HeapPop(NextRequest, RequestSortPredicate, EAllowShrinking::No);
				RunningRequests.Add(NextRequest);
			}
		}

		if (NextRequest == nullptr)
		{
			break;
		}

		NextRequest->TryCancel();
		ExecuteRequest(*NextRequest);
		check(RunningRequests.IsEmpty());
	}

	if (FApp::IsUnattended())
	{
		FOnDemandContentInstallerStats::LogAnalytics();
	}
}

} // namespace UE::IoStore
