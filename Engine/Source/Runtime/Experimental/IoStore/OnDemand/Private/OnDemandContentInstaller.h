// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "Containers/Set.h"
#include "Experimental/UnifiedError/CoreErrorTypes.h"
#include "IO/IoStoreOnDemand.h"
#include "IO/OnDemandError.h"
#include "IO/PackageId.h"
#include "IO/IoStoreOnDemandInternals.h"
#include "IO/IoAllocators.h"
#include "IO/IoContainers.h"
#include "IO/HttpIoDispatcher.h"
#include "Misc/TVariant.h"
#include "OnDemandIoStore.h"
#include "Serialization/PackageStore.h"
#include "Tasks/Pipe.h"

#include <atomic>

class FIoBuffer;
class FPackageStoreReadScope;
struct FAnalyticsEventAttribute;

namespace UE::IoStore
{

class FInstallCacheHandle;
class FOnDemandHttpThread;
class FOnDemandIoStore;

using FSharedOnDemandContainer = TSharedPtr<struct FOnDemandContainer, ESPMode::ThreadSafe>;

using FEagerDefragChunkList = TOnDemandChunkInfoList<TInlineAllocator<64>>;

namespace Private
{

////////////////////////////////////////////////////////////////////////////////
/** Returns true to continue visiting dependencies */
using FPackageDependencyVisitor = TFunctionRef<
	bool(FPackageId /*PackageId*/,
		EPackageStoreEntryStatus /*EntryStatus*/,
		const FPackageStoreEntry& /*PackageStoreEntry*/,
		bool /*bOnDemand*/
	)>;

void VisitPackageDependencies(
	const TSharedPtr<IOnDemandPackageStoreBackend>& OnDemandPackageStore,
	const TSet<FPackageId>& PackageIds, 
	bool bIncludeSoftReferences, 
	FPackageDependencyVisitor Visitor, 
	FPackageStoreReadScope* InReadScope = nullptr);

////////////////////////////////////////////////////////////////////////////////
void ResolveChunksToInstall(
	const TSharedPtr<IOnDemandPackageStoreBackend>& OnDemandPackageStore,
	const TSet<FSharedOnDemandContainer>& Containers,
	const TSet<FPackageId>& PackageIds,
	bool bIncludeSoftReferences,
	bool bIncludeOptionalBulkData,
	TArray<FOnDemandChunkInfoList>& OutResolvedContainerChunks,
	TSet<FIoChunkId>& OutMissing);

} // namespace Private

////////////////////////////////////////////////////////////////////////////////
class FOnDemandContentInstaller
{
	struct FRequest;

	struct FChunkHttpRequestHandle : public TIntrusiveListElement<FChunkHttpRequestHandle>
	{
		FChunkHttpRequestHandle*	Next = nullptr;
		FRequest*					OwnerRequest = nullptr;

		int32						ContainerIndex = INDEX_NONE;
		int32						EntryIndex = INDEX_NONE;
	};

	struct FRequest
	{
		struct FInstall
		{
			FInstall(
				UPTRINT InstallerRequest,
				FOnDemandInstallArgs&& InArgs,
				FOnDemandInstallCompleted&& InOnCompleted,
				FOnDemandInstallProgressed&& InOnProgress);

			~FInstall();

			FOnDemandInstallArgs						Args;
			FOnDemandInstallCompleted					OnCompleted;
			FOnDemandInstallProgressed					OnProgress;
			FSharedInternalInstallRequest				Request;
			TArray<FOnDemandChunkInfoList>				ResolvedChunks;
			TArray<FChunkHttpRequestHandle>				HttpRequestHandles;
			FOnDemandInstallProgress					Progress;
			uint64										DownloadedChunkCount = 0;
			uint64										LastProgressCycles = 0;
			TUniquePtr<FInstallCacheHandle>				CacheHandle;
			bool										bHttpRequestsIssued = false;
			std::atomic_bool							bNotifyingProgressOnGameThread{false};
		};

		struct FPurge
		{
			FPurge(FOnDemandPurgeArgs&& InArgs, FOnDemandPurgeCompleted&& InOnCompleted)
				: Args(MoveTemp(InArgs))
				, OnCompleted(MoveTemp(InOnCompleted)) { }

			FOnDemandPurgeArgs		Args;
			FOnDemandPurgeCompleted	OnCompleted;
		};

		struct FDefrag
		{
			FDefrag(FOnDemandDefragArgs&& InArgs, FOnDemandDefragCompleted&& InOnCompleted)
				: Args(MoveTemp(InArgs))
				, OnCompleted(MoveTemp(InOnCompleted)) { }

			FOnDemandDefragArgs			Args;
			FOnDemandDefragCompleted	OnCompleted;
		};

		struct FVerify
		{
			FVerify(FOnDemandVerifyCacheCompleted&& InOnCompleted)
				: OnCompleted(MoveTemp(InOnCompleted)) { }

			FOnDemandVerifyCacheCompleted	OnCompleted;
		};

		struct FFlushLastAccess
		{
			FFlushLastAccess(FOnDemandFlushLastAccessArgs&& InArgs, FOnDemandFlushLastAccessCompleted&& InOnCompleted)
				: Args(MoveTemp(InArgs))
				, OnCompleted(MoveTemp(InOnCompleted)) { }

			FOnDemandFlushLastAccessArgs		Args;
			FOnDemandFlushLastAccessCompleted	OnCompleted;
		};

		using FRequestVariant = TVariant<FEmptyVariantState, FInstall, FPurge, FDefrag, FVerify, FFlushLastAccess>;

		FRequest(FOnDemandInstallArgs&& Args, FOnDemandInstallCompleted&& OnCompleted, FOnDemandInstallProgressed&& OnProgress)
		{
			Variant.Emplace<FInstall>(UPTRINT(this), MoveTemp(Args), MoveTemp(OnCompleted), MoveTemp(OnProgress));
		}

		FRequest(FOnDemandPurgeArgs&& Args, FOnDemandPurgeCompleted&& OnCompleted)
		{
			Variant.Emplace<FPurge>(MoveTemp(Args), MoveTemp(OnCompleted));
		}

		FRequest(FOnDemandDefragArgs&& Args, FOnDemandDefragCompleted&& OnCompleted)
		{
			Variant.Emplace<FDefrag>(MoveTemp(Args), MoveTemp(OnCompleted));
		}

		FRequest(FOnDemandVerifyCacheCompleted&& OnCompleted)
		{
			Variant.Emplace<FVerify>(MoveTemp(OnCompleted));
		}

		FRequest(FOnDemandFlushLastAccessArgs&& Args, FOnDemandFlushLastAccessCompleted&& OnCompleted)
		{
			Variant.Emplace<FFlushLastAccess>(MoveTemp(Args), MoveTemp(OnCompleted));
		}

		bool				IsCancelled() const			{ return bCancelled.load(std::memory_order_relaxed); }
		bool				TryCancel()
		{ 
			bool bExpected = false; 
			return bCancelled.compare_exchange_strong(bExpected, true);
		}
		bool				IsOk() const				{ return !IsCancelled() && !Result.HasError(); }
		FString				GetErrorString() const		{ return IsOk() ? TEXT("Ok") : IsCancelled() ? TEXT("Cancelled") : LexToString(Result.GetError()); }

		bool				IsInstall() const			{ return Variant.IsType<FInstall>(); }
		bool				IsPurge() const				{ return Variant.IsType<FPurge>(); }
		bool				IsDefrag() const			{ return Variant.IsType<FDefrag>(); }
		bool				IsVerify() const			{ return Variant.IsType<FVerify>(); }
		bool				IsFlushLastAccess() const	{ return Variant.IsType<FFlushLastAccess>(); }

		FInstall&			AsInstall()			{ return Variant.Get<FInstall>(); }
		FPurge&				AsPurge()			{ return Variant.Get<FPurge>(); }
		FDefrag&			AsDefrag()			{ return Variant.Get<FDefrag>(); }
		FVerify&			AsVerify()			{ return Variant.Get<FVerify>(); }
		FFlushLastAccess&	AsFlushLastAccess()	{ return Variant.Get<FFlushLastAccess>(); }

		const FInstall&		AsInstall() const { return Variant.Get<FInstall>(); }
		const FPurge&		AsPurge() const { return Variant.Get<FPurge>(); }
		const FDefrag&		AsDefrag() const { return Variant.Get<FDefrag>(); }
		const FVerify&		AsVerify() const { return Variant.Get<FVerify>(); }
		const FFlushLastAccess& AsFlushLastAccess() const { return Variant.Get<FFlushLastAccess>(); }

		TOptional<UE::UnifiedError::FError> ConsumeError();

		static uint32	NextSeqNo;

		uint32						SeqNo = NextSeqNo++;
		int32						Priority = 0;
		uint64						StartTimeCycles = FPlatformTime::Cycles64();
		FResult						Result = MakeValue();
		std::atomic_bool			bCancelled{ false };
		FRequestVariant				Variant;
	};

	struct FChunkDownloadRequest
	{
		FIoHttpRequest							HttpHandle;
		TIntrusiveList<FChunkHttpRequestHandle> ChunkRequestHttpHandles;
		// Should only be used to compare two requests for the same iochunk, otherwise RequestId is not unique
		uint32									RequestId = 0;
		bool									bChunkCanceled = false;
	};

	static bool RequestSortPredicate(const FRequest& LHS, const FRequest& RHS)
	{
		if (LHS.Variant.GetIndex() == RHS.Variant.GetIndex())
		{
			if (LHS.Priority == RHS.Priority)
			{
				return LHS.SeqNo < RHS.SeqNo;
			}

			return LHS.Priority > RHS.Priority;
		}

		return LHS.SeqNo < RHS.SeqNo;
	}

	using FRequestAllocator	= TSingleThreadedSlabAllocator<FRequest, 32>;


public:
									FOnDemandContentInstaller(FOnDemandIoStore& IoStore);
									~FOnDemandContentInstaller();

	FSharedInternalInstallRequest	EnqueueInstallRequest(
										FOnDemandInstallArgs&& Args,
										FOnDemandInstallCompleted&& OnCompleted,
										FOnDemandInstallProgressed&& OnProgress);
	void							EnqueuePurgeRequest(FOnDemandPurgeArgs&& Args, FOnDemandPurgeCompleted&& OnCompleted);
	void							EnqueueDefragRequest(FOnDemandDefragArgs&& Args, FOnDemandDefragCompleted&& OnCompleted);
	void							EnqueueVerifyRequest(FOnDemandVerifyCacheCompleted&& OnCompleted);
	void							EnqueueFlushLastAccessRequest(FOnDemandFlushLastAccessArgs&& Args, FOnDemandFlushLastAccessCompleted&& OnCompleted);
	void							ScheduleReleaseContentRequestTask(UPTRINT HandleId);
	void							CancelInstallRequest(FSharedInternalInstallRequest InstallRequest);
	void							UpdateInstallRequestPriority(FSharedInternalInstallRequest InstallRequest, int32 NewPriority);
	void							ReportAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const;

private:
	bool							CanExecuteRequest(FRequest& Request) const;
	void							TryExecuteNextRequest();
	void							ExecuteRequest(FRequest& Request);
	void							PinCachedChunks(
										FRequest::FInstall& InstallRequest,
										TFunctionRef<void(int32, int32, bool)> OnChunkFound) const;
	void							ProcessInstallRequest(FRequest& Request);
	void							ExecuteInstallRequest(FRequest& Request);
	void							ExecutePurgeRequest(FRequest& Request);
	void							ExecuteDefragRequest(FRequest& Request);
	void							ExecuteVerifyRequest(FRequest& Request);
	void							ExectuteFlushLastAccessRequest(FRequest& Request);
	void							ProcessDownloadedChunk(
										const FOnDemandChunkHash& Hash,
										uint32 RequestId, 
										EIoErrorCode InErrorCode, 
										uint32 HttpStatusCode, 
										FIoBuffer&& Chunk);
	void							OnProcessDownloadedChunkNotifyRequest(const FChunkHttpRequestHandle& HttpRequest, const FResult& ChunkDownloadResult, bool bChunkCancelled);
	void							TryCancelHttpRequestsForInstallRequest(FRequest::FInstall& InstallRequest, int32& OutNumCancelled);
	void							NotifyInstallProgress(FRequest& Request);
	void							DestroyRequest(FRequest& Request);
	void							CompleteInstallRequest(FRequest& Request);
	void							CompletePurgeRequest(FRequest& Request);
	void							CompleteDefragRequest(FRequest& Request);
	void							CompleteVerifyRequest(FRequest& Request);
	void							CompleteFlushLastAccessRequest(FRequest& Request);
	void							Shutdown();

	FOnDemandIoStore&				IoStore;
	UE::Tasks::FPipe				InstallerPipe;

	FMutex							Mutex;
	FRequestAllocator				RequestAllocator;
	TArray<FRequest*>				RequestQueue;
	TArray<FRequest*>				RunningRequests;
	// This needs to be a multimap to support cancelation. At any time, there may be multiple
	// requests for the same iochunk but only one of them should be active with the remainder
	// waiting on final callbacks after being cancelled. See FChunkDownloadRequest.
	TMultiMap<FOnDemandChunkHash, FChunkDownloadRequest> PendingChunkDownloads;

	std::atomic_bool				bShuttingDown{false};
};

} // namespace UE::IoStore
