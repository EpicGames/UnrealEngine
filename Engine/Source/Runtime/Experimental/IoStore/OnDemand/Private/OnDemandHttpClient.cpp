// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnDemandHttpClient.h"

#include "Async/UniqueLock.h"
#include "Containers/StringConv.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "IO/HttpIoDispatcher.h"
#include "IO/IoStoreOnDemandInternals.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "Statistics.h"

namespace UE::IoStore
{

extern int32 GIaxHttpOneRequestsPerConnection;
extern int32 GIaxHttpVersion;

/** Not intended as a long term cvar, only including so that we can revert the new behavior easily if needed */
bool GIaxWarnOnFailure = true;
static FAutoConsoleVariableRef CVar_IaxWarnOnFailure(
	TEXT("iax.HttpWarnOnFailure"),
	GIaxWarnOnFailure,
	TEXT("When enabled FMultiEndpointHttpClient will log failed requests as warnings instead of the config verbosity")
);

/**
 * When enabled, http requests will make a partial content request by adding Range: bytes=<offset>-<offset+length> to the request if a valid
 * FIoOffsetAndLength is provided with the request. When disabled we will always request the full chunk and then use the FIoOffsetAndLength to
 * return a partial slice of the data to the caller.
 */
bool GAllowRangeRequests = true;
static FAutoConsoleVariableRef CVar_IaxAllowPartialContentRequests(
	TEXT("iax.HttpAllowPartialContentRequests"),
	GAllowRangeRequests,
	TEXT("Enable/disable the use of partial content http requests")
);

bool GHttpClientOptimizeRanges = true;
static FAutoConsoleVariableRef CVar_IaxHttpClientOptimizeRanges(
	TEXT("iax.HttpClientOptimizeRanges"),
	GHttpClientOptimizeRanges,
	TEXT("Should the http client attempt an optimization pass on ranged requests or process the requests as provided")
);

/**
 * Use CA root certificates from the system store if available.
 */
bool GUseSystemCaStore = false;
static FAutoConsoleVariableRef CVar_IaxUseSystemCaStore(
	TEXT("iax.HttpUseSystemCaStore"),
	GUseSystemCaStore,
	TEXT("Load root certificates from the system store (if available)")
);

///////////////////////////////////////////////////////////////////////////////

static FIoBuffer JoinIoBuffers(const FIoBuffer& LHS, const FIoBuffer& RHS)
{
	const uint64 TotalSize = LHS.GetSize() + RHS.GetSize();
	FIoBuffer Output(TotalSize);

	Output.GetMutableView().CopyFrom(LHS.GetView()).CopyFrom(RHS.GetView());

	return Output;
}

static int8 TrackCdnCacheStats(const HTTP::FResponse& Response)
{
	int8 Result = -1;

#if UE_TRACK_CDN_HIT_STATUS
	// All header fields are considered, with the assumption that later fields are
	// more accurate than prior ones (e.g. a caching proxy sits infront if CDNs)
	Response.ReadHeaders([&Result] (FAnsiStringView Key, FAnsiStringView Value)
	{
		if (Key == "CF-Cache-Status" || (Key.StartsWith("X-") && Key.EndsWith("-Cache")))
		{
			Result = Value.Find("HIT", 0, ESearchCase::IgnoreCase) >= 0 ? 1 : 0;
		}
		return true;
	});
#endif //UE_TRACK_CDN_HIT_STATUS

	return Result;
}

/**
 * Attempts to parse the multirange boundary marker from a http response header.
 * This marker will be found on the 'Content-Type:' line if a multirange request
 * was made.
 * If no boundary marker can be found we can just return an empty string.
 */
static FAnsiString ParseMultiRangeBoundary(UE::IoStore::HTTP::FResponse& Response)
{
	FAnsiStringView ContentType = Response.GetHeader(ANSITEXTVIEW("content-type"));
	if (ContentType.IsEmpty())
	{
		return FAnsiString();
	}

	constexpr FAnsiStringView MultipartType = ANSITEXTVIEW("multipart/byteranges;");
	if (!ContentType.StartsWith(MultipartType))
	{
		return FAnsiString();
	}

	// Search for the first instance of 'boundary=' after 'multipart/byteranges;'
	constexpr FAnsiStringView BoundaryKey = ANSITEXTVIEW("boundary=");
	int32 StartIndex = ContentType.Find(BoundaryKey, MultipartType.Len(), ESearchCase::IgnoreCase);
	if (StartIndex == INDEX_NONE)
	{
		return FAnsiString();
	}

	StartIndex += BoundaryKey.Len(); // Advance to the start of the value

	const int32 EndIndex = ContentType.Find(";", StartIndex);
	const int32 NumChars = (EndIndex != INDEX_NONE ? EndIndex : ContentType.Len()) - StartIndex;

	FAnsiStringView BoundaryValue = ContentType.SubStr(StartIndex, NumChars);
	BoundaryValue.TrimStartAndEndInline();

	// When the boundary is used it will always be prefixed with '--' so we can add that prefix now.
	constexpr FAnsiStringView BoundaryPrefix = ANSITEXTVIEW("--");

	FAnsiString Output;
	Output.Reserve(BoundaryPrefix.Len() + BoundaryValue.Len());
	Output.Append(BoundaryPrefix);
	Output.Append(BoundaryValue);

	return Output;
}

/**
 * Searches for the first occurrence of Sequence bytes in View, starting at Offset.
 * Returns INDEX_NONE if not found.
 */
static uint64 FindSequence(FMemoryView View, FAnsiStringView Sequence, uint64 Offset = 0)
{
	if (Sequence.IsEmpty())
	{
		return Offset;
	}

	const uint64 DataSize = View.GetSize();
	const uint64 SequenceLen = (uint64)Sequence.Len();

	if (DataSize < Offset + SequenceLen)
	{
		return INDEX_NONE;
	}

	const uint8* Data = static_cast<const uint8*>(View.GetData());
	const uint8* SequenceData = reinterpret_cast<const uint8*>(Sequence.GetData());

	for (uint64 Index = Offset; Index + SequenceLen <= DataSize; ++Index)
	{
		if (FMemory::Memcmp(Data + Index, SequenceData, SequenceLen) == 0)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

FMemoryView SkipToNextLine(FMemoryView View, uint64 Offset = 0)
{
	constexpr FAnsiStringView CRLF = ANSITEXTVIEW("\r\n");

	const uint64 LineEndingIndex = FindSequence(View, CRLF, Offset);
	if (LineEndingIndex != INDEX_NONE)
	{
		return View.RightChop(LineEndingIndex + CRLF.Len());
	}
	else
	{
		return FMemoryView();
	}
}

TIoStatusOr<FAnsiStringView> ExtractTextLine(FMemoryView View)
{
	constexpr FAnsiStringView CRLF = ANSITEXTVIEW("\r\n");

	const uint64 LineEndingIndex = FindSequence(View, CRLF, 0);
	if (LineEndingIndex != INDEX_NONE)
	{
		View = View.Left(LineEndingIndex);
	}

	if (View.GetSize() > (uint64)MAX_int32)
	{
		return FIoStatus(EIoErrorCode::HttpReadError, TEXT("Multirange request: Oversized line indicates malformed data"));
	}

	return FAnsiStringView(static_cast<const char*>(View.GetData()), (int32)View.GetSize());
}

FMemoryView ExtractData(FMemoryView View, FAnsiStringView Boundary, TPair<uint32, uint32> ByteRange)
{
	// We know that ByteRange.Value >= ByteRange.Key as that was validated (in ParseContentRange) when the byte range was parsed
	const uint64 DataLength = (uint64)(ByteRange.Value - ByteRange.Key) + 1;
	return View.Left(DataLength);
}

// TODO - LexFromString (need FAnsiStringView support)
void ExtractFromString(uint64& OutValue, const FAnsiStringView& InString)
{
	TAnsiStringBuilder<64> Builder;
	LexFromString(OutValue, *(Builder << InString));
}

FIoStatus ParseContentRange(FAnsiStringView Line, TPair<uint32,uint32>& ByteRange)
{
	constexpr FAnsiStringView Bytes = ANSITEXTVIEW("bytes ");

	int32 BytesIndex = Line.Find(Bytes);
	if (BytesIndex == INDEX_NONE)
	{
		return FIoStatus(EIoErrorCode::HttpReadError, TEXT("Multirange request: Unable to find bytes for 'Content-Range'"));
	}

	FAnsiStringView DataView = Line.RightChop(BytesIndex + Bytes.Len());
	DataView.TrimEndInline();

	uint64 StartByte = 0;
	ExtractFromString(StartByte, DataView);

	if (StartByte > MAX_uint32)
	{
		return FIoStatus(EIoErrorCode::HttpReadError, *WriteToString<32>(TEXT("Multirange request: Start byte is too large: "), StartByte));
	}

	int32 SecondByteOffset = 0;
	if (!DataView.FindChar('-', SecondByteOffset))
	{
		return FIoStatus(EIoErrorCode::HttpReadError, TEXT("Multirange request: Malformed byte range for 'Content-Range'"));
	}

	DataView.RightChopInline(SecondByteOffset + 1);

	uint64 EndByte = 0;
	ExtractFromString(EndByte, DataView);

	if (EndByte > MAX_uint32)
	{
		return FIoStatus(EIoErrorCode::HttpReadError, *WriteToString<32>(TEXT("Multirange request: End byte is too large: "), EndByte));
	}

	if (EndByte < StartByte)
	{
		return FIoStatus(EIoErrorCode::HttpReadError, TEXT("Multirange request: Malformed byte range for 'Content-Range'"));
	}

	ByteRange.Key = (uint32)StartByte;
	ByteRange.Value = (uint32)EndByte;

	return EIoErrorCode::Ok;
}

static FIoStatus ParseHeader(FMemoryView& BodyView, FAnsiStringView Boundary, TPair<uint32, uint32>& OutByteRange)
{
	constexpr FAnsiStringView ContentRange = ANSITEXTVIEW("Content-Range");

	bool bReadRange = false;
	while (!BodyView.IsEmpty())
	{
		TIoStatusOr<FAnsiStringView> LineResult = ExtractTextLine(BodyView);
		if (!LineResult.IsOk())
		{
			return LineResult.Status();
		}
		FAnsiStringView Line = LineResult.ConsumeValueOrDie();
		BodyView = SkipToNextLine(BodyView, (uint64)Line.Len());

		// TODO: Clean up the parsing, should be less logic to determine that we read the correct format.
		if (Line.StartsWith(ContentRange))
		{
			if (FIoStatus Status = ParseContentRange(Line.RightChop(ContentRange.Len()), OutByteRange); !Status.IsOk())
			{
				return Status;
			}

			bReadRange = true;
		}
		else if (Line.IsEmpty())
		{
			if (bReadRange)
			{
				return EIoErrorCode::Ok;
			}
			else
			{
				return FIoStatus(EIoErrorCode::HttpReadError, TEXT("Multirange request:Ranged header did not contain a 'Content-Range'"));
			}
			
		}
		else if (Line.StartsWith(Boundary))
		{
			return FIoStatus(EIoErrorCode::HttpReadError, TEXT("Multirange request: End of ranged header was unexpectedly reached"));
		}
	}

	return FIoStatus(EIoErrorCode::HttpReadError, TEXT("Multirange request: Ranged header was truncated"));
}

FResponseBody::FResponseBody(FIoBuffer&& InBody, const FIoOffsetAndLength& Range)
	: Body(MoveTemp(InBody))
{
	// TODO: Ignore range if whole body?
	// TODO: Improve casting/truncation.
	DataRanges.Add(FResponseBody::FRange
	{
		.BodyOffset = 0,
		.StartByte = (uint32)Range.GetOffset(),
		.EndByte = (uint32)(Range.GetOffset() + Range.GetLength() - 1),
	});
}

FIoStatus FResponseBody::Parse(FIoBuffer&& InBody, FAnsiStringView Boundary)
{
	constexpr FAnsiStringView DelimEnd = ANSITEXTVIEW("--");

	FMemoryView BodyView(InBody.GetData(), InBody.GetSize());

	while (!BodyView.IsEmpty())
	{
		TIoStatusOr<FAnsiStringView> LineResult = ExtractTextLine(BodyView);
		if (!LineResult.IsOk())
		{
			Reset();
			return LineResult.Status();
		}
		FAnsiStringView Line = LineResult.ConsumeValueOrDie();
		BodyView = SkipToNextLine(BodyView, (uint64)Line.Len());

		if (Line.StartsWith(Boundary))
		{
			FAnsiStringView PostBoundary = Line.SubStr(Boundary.Len(), 2);
			if (PostBoundary == DelimEnd)
			{
				Body = MoveTemp(InBody);
				return EIoErrorCode::Ok;
			}

			TPair<uint32, uint32> ByteRange;
			if (FIoStatus Status = ParseHeader(BodyView, Boundary, ByteRange); !Status.IsOk())
			{
				Reset();
				return Status;
			}

			// Now parse the data
			FMemoryView DataView = ExtractData(BodyView, Boundary, ByteRange);
			BodyView = SkipToNextLine(BodyView, DataView.GetSize());

			// TODO: Safer cast
			const uint32 DataOffset = (uint32)((const uint8*)DataView.GetData() - InBody.GetData());

			// TODO: Validate the boundary and data with ByteRange?
			DataRanges.Add(FResponseBody::FRange
				{
					.BodyOffset = DataOffset,
					.StartByte = ByteRange.Key,
					.EndByte = ByteRange.Value,
				});
		}
		else if (!Line.IsEmpty())
		{
			Reset();
			return FIoStatus(EIoErrorCode::HttpReadError, TEXT("Multirange request: Unexpected data found in the body"));
		}
	}

	Reset();
	return FIoStatus(EIoErrorCode::HttpReadError, TEXT("Multirange request: Truncated data"));
}

FIoBuffer FResponseBody::GetData(const FIoOffsetAndLength& Range) const
{
	if (!Range.IsValid())
	{
		return Body;
	}
	else if (DataRanges.IsEmpty())
	{
		return FIoBuffer(Body.GetView().Mid(Range.GetOffset(), Range.GetLength()), Body);
	}
	else
	{
		for (const FRange& DataRange : DataRanges)
		{
			if (Range.GetOffset() >= DataRange.StartByte && Range.GetOffset() <= DataRange.EndByte) // Does the requested range start within the range
			{
				const uint64 EndByte = (Range.GetOffset() + Range.GetLength()) - 1;
				if (EndByte >= DataRange.StartByte && EndByte <= DataRange.EndByte) // Does the requested range end within the range
				{
					// TODO - Worth returning DataRange.DataView if exact match?
					const uint64 Offset = Range.GetOffset() - DataRange.StartByte;
					return FIoBuffer(Body.GetView().Mid(DataRange.BodyOffset + Offset, Range.GetLength()), Body);
				}
			}
		}
	}

	checkNoEntry();

	// TODO - Error
	return FIoBuffer();
}

static const TCHAR* CDNCacheStatusToString(int8 Status)
{
#if UE_TRACK_CDN_HIT_STATUS
	return Status > 0 ? TEXT("HIT") : Status == 0 ? TEXT("MISS") : TEXT("???");
#else
	return TEXT("???");
#endif //UE_TRACK_CDN_HIT_STATUS
}

// Assumes sorted order
bool MergeOverlappingRanges(FIoOffsetAndLength& MergeDst, const FIoOffsetAndLength& PotentialOverlap)
{
	const uint64 DstLastByte = MergeDst.GetOffset() + MergeDst.GetLength();
	if (PotentialOverlap.GetOffset() <= DstLastByte)
	{
		const uint64 PotentialLastByte = PotentialOverlap.GetOffset() + PotentialOverlap.GetLength();
		if (DstLastByte > PotentialLastByte)
		{
			return true;
		}

		const uint64 MergedLength = PotentialLastByte - MergeDst.GetOffset();
		MergeDst.SetLength(MergedLength);

		return true;
	}
	else
	{
		return false;
	}
}

TArray<FIoOffsetAndLength> SortAndMergeRanges(TArray<FIoOffsetAndLength>& RangesToMerge)
{
	if (!GHttpClientOptimizeRanges || RangesToMerge.IsEmpty())
	{
		return RangesToMerge;
	}

	RangesToMerge.Sort([](const FIoOffsetAndLength& LHS, const FIoOffsetAndLength& RHS)
	{
		return LHS.GetOffset() < RHS.GetOffset();
	});

	TArray<FIoOffsetAndLength> SortedRanges;
	SortedRanges.Add(RangesToMerge[0]);

	for (int32 Index = 1; Index < RangesToMerge.Num(); ++Index)
	{
		if (!MergeOverlappingRanges(SortedRanges.Last(), RangesToMerge[Index]))
		{
			SortedRanges.Add(RangesToMerge[Index]);
		}
	}
	return SortedRanges;
}

FIoStatus LoadDefaultHttpCertificates(bool& bWasLoaded)
{
	using namespace UE::IoStore::HTTP;
	
	bWasLoaded = false;

	static struct FDefaultCerts
	{
		FDefaultCerts(bool& bWasLoadedd)
		{
			bWasLoadedd = true;
			Status = FIoStatus::Ok;

			// The following config option is used when staging to copy root certs PEM
			const TCHAR* CertSection = TEXT("/Script/Engine.NetworkSettings");
			const TCHAR* CertKey = TEXT("n.VerifyPeer");

			bool bVerifyPeer = false;
			if (GConfig != nullptr)
			{
				GConfig->GetBool(CertSection, CertKey, bVerifyPeer, GEngineIni);
			}

			if (GUseSystemCaStore)
			{
				FCertRoots CaRoots(FMemoryView{});
				if (CaRoots.IsValid())
				{
					uint32 NumCerts = CaRoots.Num();
					FCertRoots::SetDefault(MoveTemp(CaRoots));
					UE_LOGF(LogIoStoreOnDemand, Display, "Loaded %u certificates from system", NumCerts);
					return;
				}
			}

			// Open the certs file
			IFileManager& Ifm = IFileManager::Get();
			const FString PemPath = FPaths::EngineContentDir() / TEXT("Certificates/ThirdParty/cacert.pem");
			TUniquePtr<FArchive> Reader(Ifm.CreateFileReader(*PemPath));

			if (Reader.IsValid())
			{
				// Buffer certificate data
				const uint64 Size = Reader->TotalSize();
				FIoBuffer PemData(Size);
				FMutableMemoryView PemView = PemData.GetMutableView();
				Reader->Serialize(PemView.GetData(), Size);

				// Load the certs
				FCertRoots CaRoots(PemData.GetView());

				const uint32 NumCerts = CaRoots.Num();
				FCertRoots::SetDefault(MoveTemp(CaRoots));

				UE_LOGF(LogIoStoreOnDemand, Display, "Loaded %u certificates from '%ls'", NumCerts, *PemPath);
			}
			else if (bVerifyPeer)
			{
				Status = FIoStatusBuilder(EIoErrorCode::FileNotOpen) << TEXT("Failed to open certificates file '") << PemPath << TEXT("'");
			}
		}

		FIoStatus Status;
	} DefaultCerts(bWasLoaded);

	return DefaultCerts.Status;
}

/**
 * Generates a new FHttpTicketId when called.
 * 
 * Calling this should be thread safe and will always return a new unique non zero identifier.
 * In theory identifiers may be reused once the internal counter has overflowed but in practical
 * terms this should never cause us a problem as the identifiers are only used for a relatively
 * short lifespan.
 */
FMultiEndpointHttpClient::FHttpTicketId GenerateTicketId()
{
	static std::atomic<FMultiEndpointHttpClient::FHttpTicketId> NextFreeId = 1;

	FMultiEndpointHttpClient::FHttpTicketId NewId = 0;

	// To protect against the unlikely scenario where we somehow assign all of the ids and end up
	// overflowing we need to make sure that we don't return 0 as an Id.
	do 
	{
		NewId = NextFreeId++;
	} while (NewId == 0);
	
	return NewId;
}

///////////////////////////////////////////////////////////////////////////////

uint64 FMultiEndpointHttpClient::FRequest::GetTotalRangeLength() const
{
	uint64 TotalLength = 0;

	for (const FIoOffsetAndLength& Range : Ranges)
	{
		if (Range.IsValid())
		{
			TotalLength += Range.GetLength();
		}
	}

	return TotalLength;
}

bool FMultiEndpointHttpClient::FRequest::BuildRangeHeaderValue(FAnsiStringBuilderBase& OutString) const
{
	OutString = "bytes=";

	bool bHasAddedRange = false;

	for (const FIoOffsetAndLength& Range : Ranges)
	{
		if (Range.IsValid())
		{
			if (bHasAddedRange)
			{
				OutString << ",";
			}

			OutString << Range.GetOffset() << "-" << (Range.GetOffset() + Range.GetLength() - 1);
			bHasAddedRange = true;
		}
	}

	return bHasAddedRange;
}

///////////////////////////////////////////////////////////////////////////////

FMultiEndpointHttpClient::FMultiEndpointHttpClient(const FMultiEndpointHttpClientConfig& InConfig)
	: Config(InConfig)
{
	EventLoop.SetFailTimeout(Config.TimeoutMs);
}

FMultiEndpointHttpClient::~FMultiEndpointHttpClient()
{
	checkf(EventLoop.IsIdle(), TEXT("FMultiEndpointHttpClient still has active requests on shutdown"));
	check(TicketLookupMap.IsEmpty());
}

TUniquePtr<FMultiEndpointHttpClient> FMultiEndpointHttpClient::Create(const FMultiEndpointHttpClientConfig& Config)
{
	return TUniquePtr<FMultiEndpointHttpClient>(new FMultiEndpointHttpClient(Config));
}

TIoStatusOr<FMultiEndpointHttpClientResponse> FMultiEndpointHttpClient::Get(FAnsiStringView Url, const FMultiEndpointHttpClientConfig& Config)
{
	using namespace UE::IoStore::HTTP;

	FEventLoop::FRequestParams Params = FEventLoop::FRequestParams
	{
		.bAutoRedirect = Config.Redirects == EHttpRedirects::Follow,
		.HttpVersion = (GIaxHttpVersion == 2) ? EHttpVersion::Two : EHttpVersion::One,
	};
	if (Params.HttpVersion == EHttpVersion::Two)
	{
		Params.VerifyCert = FCertRoots::Default();
	}

	FEventLoop Loop;
	FIoBuffer Body;
	TStringBuilder<128> Reason;
	HTTP::FTicketPerf::FSample PerfSample;
	uint32 StatusCode = 0;

	const uint32 MaxAttempts = Config.MaxRetryCount == -1 ? 3u : static_cast<uint32>(Config.MaxRetryCount);
	for (uint32 Attempt = 0; Attempt <= MaxAttempts; ++Attempt)
	{
		Loop.Send(Loop.Request("GET", Url, &Params), [&Body, &Reason, &PerfSample, &StatusCode](const FTicketStatus& Status)
			{
				if (Status.GetId() == FTicketStatus::EId::Response)
				{
					Status.GetResponse().SetDestination(&Body);
					StatusCode = Status.GetResponse().GetStatusCode();
					PerfSample = Status.GetPerf().GetSample();
					return;
				}

				if (Status.GetId() == FTicketStatus::EId::Content)
				{
					PerfSample = Status.GetPerf().GetSample();
				}

				if (Status.GetId() == FTicketStatus::EId::Error)
				{
					Reason << Status.GetError().Reason;
				}
			});

		while (Loop.Tick(-1))
		{
			// Busy loop
		}

		if (IsHttpStatusOk(StatusCode))
		{
			FMultiEndpointHttpClientResponse Response
			{
				.Body = FResponseBody(MoveTemp(Body)),
				.Sample = PerfSample,
				.StatusCode = StatusCode,
				.RetryCount = Attempt,
			};

			return Response;
		}
	}

	if (Reason.Len() == 0)
	{
		Reason << TEXT("StatusCode: ") << StatusCode;
	}

	return FIoStatus(EIoErrorCode::ReadError, Reason.ToView());
}

FMultiEndpointHttpClient::FHttpTicketId FMultiEndpointHttpClient::Get(const FOnDemandHostGroup& HostGroup, FAnsiStringView RelativeUrl, FOnHttpResponse&& OnResponse)
{
	return Get(HostGroup, RelativeUrl, TArray<FIoOffsetAndLength>(), MoveTemp(OnResponse));
}

FMultiEndpointHttpClient::FHttpTicketId FMultiEndpointHttpClient::Get(const FOnDemandHostGroup& HostGroup, FAnsiStringView RelativeUrl, TArray<FIoOffsetAndLength>&& ChunkRanges, FOnHttpResponse&& OnResponse)
{
	return Get(HostGroup, RelativeUrl, MoveTemp(ChunkRanges), TArray<FAnsiString>(), EMultiEndpointRequestFlags::None, MoveTemp(OnResponse));
}

FMultiEndpointHttpClient::FHttpTicketId FMultiEndpointHttpClient::Get(
	const FOnDemandHostGroup& HostGroup,
	FAnsiStringView RelativeUrl,
	TArray<FIoOffsetAndLength>&& ChunkRanges,
	TArray<FAnsiString>&& Headers,
	EMultiEndpointRequestFlags Flags,
	FOnHttpResponse&& OnResponse)
{
	FConnection& Connection = GetConnection(HostGroup);

	TArray<FIoOffsetAndLength> SortedRanged = SortAndMergeRanges(ChunkRanges);
	
	FHttpTicketId TicketId = IssueRequest(FRequest
	{
		.OnResponse		= MoveTemp(OnResponse),
		.RequestHeaders	= MoveTemp(Headers),
		.RelativeUrl	= FAnsiString(RelativeUrl),
		.Ranges			= MoveTemp(SortedRanged),
		.Connection		= Connection,
		.StartTime		= FPlatformTime::Cycles64(),
		.Host			= Connection.CurrentHost,
		.Flags			= Flags
	});

	return TicketId;
}

bool FMultiEndpointHttpClient::Tick(int32 WaitTimeMs, uint32 MaxKiBPerSecond)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMultiEndpointHttpClient::Tick);
	EventLoop.Throttle(MaxKiBPerSecond);

	const uint32 TicketCount = EventLoop.Tick(WaitTimeMs);

	ProcessFailedRequests();
	ProcessRetryAttempts(TicketCount);

	const bool bIsIdle = EventLoop.IsIdle();
	if (bIsIdle)
	{
		// Destroy all non active connection pool(s)
		for (TUniquePtr<FConnection>& Connection : Connections)
		{
			for (int32 Idx = 0, Count = Connection->Pools.Num(); Idx < Count; ++Idx)
			{
				if (Idx != Connection->CurrentHost)
				{
					Connection->Pools[Idx].Reset();
				}
			}
		}
	}

	return bIsIdle == false;
}

void FMultiEndpointHttpClient::CancelRequest(FHttpTicketId TicketId)
{
	if (TicketId == 0)
	{
		return;
	}

	UE::TUniqueLock _(TicketLookupMutex);

	// Note that normally we would expect to find an entry for a valid FHttpTicketId
	// but it is possible that the request just completed and was removed from the
	//  map. See the HttpSink in ::IssueRequest.
	if (FTicketInfo* Ticket = TicketLookupMap.Find(TicketId))
	{
		Ticket->bCancelRequested = true;

		if (Ticket->HttpTicket != 0)
		{
#if !NO_LOGGING
			UE_LOGF(LogHttpIoDispatcher, Verbose, "Canceling FHttpTicketId %u with FTicket: %llu", TicketId, Ticket->HttpTicket);
#endif // !NO_LOGGING

			EventLoop.Cancel(Ticket->HttpTicket);
		}
	}
}

void FMultiEndpointHttpClient::UpdateConnections()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMultiEndpointHttpClient::UpdateConnections);

	for (TUniquePtr<FConnection>& Connection : Connections)
	{
		check(Connection.IsValid());

		Connection->CurrentHost = Connection->HostGroup.PrimaryHostIndex();

		if (Connection->CurrentHost != INDEX_NONE)
		{
			if (Connection->Pools[Connection->CurrentHost].IsValid() == false)
			{
				Connection->Pools[Connection->CurrentHost] = CreateConnection(Connection->HostGroup.PrimaryHost());
			}
		}
	}
}

void FMultiEndpointHttpClient::GetHttpStats(FOnDemandHttpStats& Out) const
{
	using namespace HTTP;

	struct FLoopHttpStats
		: public IOnDemandInternalHttpStats
	{
		static_assert(SAMPLE_COUNT == FLoopPerf::RecvBucketCount);
		virtual uint32 GetRecvKiBps() const { return Perf.GetRecvKiBps(); }
		virtual void GetRecvKiBps(uint32 (&Out)[SAMPLE_COUNT]) const { return Perf.GetRecvKiBps(Out); }
		virtual uint32 GetTotalRecvKiB() const { return Perf.GetTotalRecvKiB(); }
		virtual uint32 GetTimeToFirstByteMs() const { return Perf.GetTimeToFirstByteMs(); }
		FLoopPerf Perf;
	};

	if (!Out.Internal.IsValid())
	{
		Out.Internal = MakeUnique<FLoopHttpStats>();
	}

	auto* HttpStats = static_cast<FLoopHttpStats*>(Out.Internal.Get());
	EventLoop.SamplePerf(HttpStats->Perf);
}

FMultiEndpointHttpClient::FHttpTicketId FMultiEndpointHttpClient::IssueRequest(FRequest&& Request)
{
	using namespace UE::IoStore::HTTP;

	if (Request.Connection.CurrentHost == INDEX_NONE)
	{
		// Store the request response so we can invoke it in the next tick.
		// This way the calling code does not need to handle multiple paths.
		FailedRequests.Emplace(MoveTemp(Request));

		return 0;
	}

	check(Request.Connection.HostGroup.IsEmpty() == false);
	check(Request.Connection.Pools[Request.Connection.CurrentHost].IsValid());

	FAnsiStringView Url				= Request.RelativeUrl;
	FConnectionPool& ConnectionPool = *Request.Connection.Pools[Request.Connection.CurrentHost];

	FEventLoop::FRequestParams RequestParams = FEventLoop::FRequestParams
	{
		.ContentSizeEst	= (uint32)Request.GetTotalRangeLength(), // TODO - Safer cast?
		.bAutoRedirect	= Config.Redirects == EHttpRedirects::Follow,
		.bAllowChunked	= Config.bAllowChunkedTransfer
	};

	UE::IoStore::HTTP::FRequest HttpRequest = EventLoop.Get(Url, ConnectionPool, &RequestParams);

#if UE_TRACK_CDN_HIT_STATUS
	HttpRequest.Header("pragma", "akamai-x-cache-on");
#endif // UE_TRACK_CDN_HIT_STATUS

	check(Request.RequestHeaders.IsEmpty() || ((Request.RequestHeaders.Num() % 2) == 0));
	if (!Request.RequestHeaders.IsEmpty())
	{
		for (int32 Idx = 0; Idx < Request.RequestHeaders.Num(); Idx += 2)
		{
			HttpRequest.Header(Request.RequestHeaders[Idx], Request.RequestHeaders[Idx + 1]);
		}
	}

	if (GAllowRangeRequests)
	{
		TAnsiStringBuilder<128> HeaderValue;

		if (Request.BuildRangeHeaderValue(HeaderValue))
		{
			HttpRequest.Header(ANSITEXTVIEW("range"), *HeaderValue);
		}
	}

	// If the request is new then we need to assign an identifier
	if (Request.TicketId == 0)
	{
		Request.TicketId = GenerateTicketId();
	}

	FHttpTicketId RequestId = Request.TicketId;

	auto HttpSink = [this, Request = MoveTemp(Request)](const FTicketStatus& TicketStatus) mutable
	{
		bool bCompleted	= false;
		bool bCanceled	= false;
		bool bError		= false;

		switch (TicketStatus.GetId())
		{
			case FTicketStatus::EId::Response:
			{
				FResponse& HttpResponse		= TicketStatus.GetResponse();
				Request.StatusCode			= HttpResponse.GetStatusCode();
				Request.bIsChunkedTransfer	= HttpResponse.GetContentLength() == -1;
				
				if(Request.Ranges.Num() > 1)
				{
					Request.MultiRangeBoundary = ParseMultiRangeBoundary(HttpResponse); // TODO - Only do this is we sent multi-range?
				}

				if (EnumHasAnyFlags(Request.Flags, EMultiEndpointRequestFlags::ResponseHeaders))
				{
					Request.ResponseHeaders.Empty(16);
					HttpResponse.ReadHeaders([&Request] (FAnsiStringView Key, FAnsiStringView Value)
					{
						//TODO: Can we get all headers in one buffer?
						Request.ResponseHeaders.Add(FAnsiString(Key));
						Request.ResponseHeaders.Add(FAnsiString(Value));
						return true;
					});
				}

				if (IsHttpStatusOk(Request.StatusCode))
				{
					Request.CDNCacheStatus = TrackCdnCacheStats(HttpResponse);
				}
				else
				{
					Request.ResponseMessage = FString(HttpResponse.GetStatusMessage().TrimEnd());
				}

				// Returns true if the response has no content, i.e. 204, 205 and 304
				bCompleted = TicketStatus.IsComplete();
				if (!bCompleted)
				{
					HttpResponse.SetDestination(Request.bIsChunkedTransfer == false ? &Request.Body : &Request.Chunk);
				}
				break;
			}
			case FTicketStatus::EId::Content:
			{
				bCompleted = true;
				if (Request.bIsChunkedTransfer)
				{
					// If the current chunk size is zero that means all the chunks have been transfered and FRequest::Body should be complete
					bCompleted = Request.Chunk.GetSize() == 0;
					if (!bCompleted)
					{
						if (IsHttpStatusOk(Request.StatusCode) || Config.bResponseBodyOnError)
						{
							// Could consider using FRequest::Range to presize FRequest::Body and copy the chunks
							// into it rather than resizing each time.
							Request.Body = JoinIoBuffers(Request.Body, Request.Chunk);
						}
					}
				}
				break;
			}
			case FTicketStatus::EId::Cancelled:
			{
				bCompleted = bCanceled = true;
				break;
			}
			case FTicketStatus::EId::Error:
			{
				bCompleted = bError = true;
				break;
			}
			default:
			{
				checkNoReentry();
				bCompleted = bCanceled = true;
				UE_LOGF(LogHttpIoDispatcher, Error, "Unexpected HTTP ticket status");
				break;
			}
		}

		// The callback is triggered multiple times, i.e. first the response and then the content
		if (!bCompleted)
		{
			return;
		}

		// Only retry if there was a connection error or a server error
		bool bRetry = false;
		if (!bCanceled && (bError || IsHttpServerError(Request.StatusCode)))
		{
			bRetry = Request.RetryCount < GetRetryLimitForRequest(Request);
		}

		if (bRetry)
		{
#if DO_CHECK
			Request.SinkCounter++;
#endif
			RetryRequest(MoveTemp(Request));
		}
		else
		{
			CompleteRequest(MoveTemp(Request), TicketStatus);
		}
	};

	FTicket RequestTicket = EventLoop.Send(MoveTemp(HttpRequest), MoveTemp(HttpSink));
	check(RequestTicket != 0);

	{
		UE::TUniqueLock _(TicketLookupMutex);
		TicketLookupMap.Add(RequestId, FTicketInfo{ .HttpTicket = RequestTicket});
	}

	return RequestId;
}

void FMultiEndpointHttpClient::CompleteRequest(FRequest&& Request, const UE::IoStore::HTTP::FTicketStatus& TicketStatus)
{
	using namespace UE::IoStore::HTTP;

#if DO_CHECK
	check(Request.SinkCounter == Request.RetryCount);
#endif // DO_CHECK

	FMultiEndpointHttpClientResponse Response
	{
		.Headers				= MoveTemp(Request.ResponseHeaders),
		.StatusCode				= Request.StatusCode,
		.RetryCount				= Request.RetryCount,
		.HostIndex				= Request.Host,
		.CDNCacheStatus			= Request.CDNCacheStatus
	};

	if (IsHttpStatusOk(Request.StatusCode) || Config.bResponseBodyOnError)
	{
		if (Request.StatusCode == 206)
		{
			if (Request.MultiRangeBoundary.Len() > 0)
			{
				if (FIoStatus Status = Response.Body.Parse(MoveTemp(Request.Body), Request.MultiRangeBoundary); !Status.IsOk())
				{
					Response.StatusCode = 0;
					Response.Reason = Status.ToString();
				}

				//((char*)Response.Body.GetData())[104] = '\0'; // <- Test that the string view parsing code is safe
			}
			else
			{
				// TODO - Validate the range returned?
				check(!Request.Ranges.IsEmpty());
				Response.Body = FResponseBody(MoveTemp(Request.Body), Request.Ranges[0]);
			}
		}
		else
		{
			Response.Body = FResponseBody(MoveTemp(Request.Body));
		}
	}
	else if (Config.bResponseBodyOnError)
	{
		Response.Body = FResponseBody(MoveTemp(Request.Body));
	}

	if (TicketStatus.GetId() == FTicketStatus::EId::Content)
	{
		Response.Sample = TicketStatus.GetPerf().GetSample();
	}
	
	if (TicketStatus.GetId() == FTicketStatus::EId::Error)
	{
		Response.Reason = WriteToString<128>(TicketStatus.GetError().Reason, TEXT(" ("), TicketStatus.GetError().Code, TEXT(")"));
	}
	else if (TicketStatus.GetId() == FTicketStatus::EId::Cancelled)
	{
		Response.Reason = TEXT("Canceled");
		Response.bCanceled = true;
	}
	else if (!Request.ResponseMessage.IsEmpty())
	{
		Response.Reason = MoveTemp(Request.ResponseMessage);
	}

	{
		UE::TUniqueLock _(TicketLookupMutex);
		TicketLookupMap.Remove(Request.TicketId);
	}

	Log(Response, Request);

	FOnHttpResponse OnResponse = MoveTemp(Request.OnResponse);
	OnResponse(MoveTemp(Response));
}

void FMultiEndpointHttpClient::RetryRequest(FRequest&& Request)
{
	FConnection& Connection = Request.Connection;

	// Try a different host URL after the first retry
	if (Request.RetryCount > 0 && Request.Host == Request.Connection.CurrentHost)
	{
		FAnsiStringView HostUrl = Request.Connection.HostGroup.CycleHost(Connection.CurrentHost);
		if (Connection.Pools[Connection.CurrentHost].IsValid() == false)
		{
			Connection.Pools[Connection.CurrentHost] = CreateConnection(HostUrl);
		}
	}

	// Clear the ticket association, note that we don't check if the request is canceled as we'd have to
	// check it again anyway when the retry attempts are being reissued so we might as well wait and do
	// it in one place.
	{
		UE::TUniqueLock _(TicketLookupMutex);
		if (FTicketInfo* TicketInfo = TicketLookupMap.Find(Request.TicketId))
		{
			TicketInfo->HttpTicket = 0;
		}
		else
		{
			checkNoEntry();
		}
	}

	Request.StatusCode = 0;
	Request.RetryCount++;
	Request.Host = Connection.CurrentHost;
	Request.Body = FIoBuffer();
	Retries.Emplace(MoveTemp(Request));
}

void FMultiEndpointHttpClient::ProcessFailedRequests()
{
	if (FailedRequests.IsEmpty())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FMultiEndpointHttpClient::ProcessFailedRequests);

	for (FRequest& Request :FailedRequests)
	{
#if DO_CHECK
		check(Request.SinkCounter == 0);
#endif // DO_CHECK

		FMultiEndpointHttpClientResponse Response
		{
			.Reason = TEXT("Hostgroup had no valid urls for the request")
		};

		LogError(Request, Response.Reason);

		Request.OnResponse(MoveTemp(Response));
	}

	FailedRequests.Empty();
}

void FMultiEndpointHttpClient::ProcessRetryAttempts(uint32 TicketCount)
{
	if (Retries.IsEmpty())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FMultiEndpointHttpClient::ProcessRetryAttempts);

	auto IsTicketCanceled = [this](FHttpTicketId TicketId) -> bool
	{
		UE::TUniqueLock _(TicketLookupMutex);

		FTicketInfo* TicketInfo = TicketLookupMap.Find(TicketId);
		if (!ensureMsgf(TicketInfo != nullptr, TEXT("Could not find a valid FTicketInfo for FHttpTicketId %d"), TicketId))
		{
			// I think we can survive at this point by assuming that the request just needs to be retried.
			return false;
		}

		checkf(TicketInfo->HttpTicket == 0, TEXT("FHttpTicketId %d is still in progress!"), TicketId);
		return TicketInfo->bCancelRequested;
	};

	const int32 RequestCount = FMath::Min(Retries.Num(), int32(HTTP::FEventLoop::MaxActiveTickets - TicketCount));
	for (int32 Idx = 0; Idx < RequestCount; Idx++)
	{
		FRequest& Request = Retries[Idx];

		if (IsTicketCanceled(Request.TicketId) == false)
		{
			IssueRequest(MoveTemp(Request));
		}
		else
		{
			FMultiEndpointHttpClientResponse Response
			{
				.Reason = TEXT("Canceled"),
				.StatusCode = Request.StatusCode,
				.RetryCount = Request.RetryCount - 1, // Reduce retry count by 1 as this retry attempt didn't really do anything
				.HostIndex = Request.Host,
				.bCanceled = true,
				.CDNCacheStatus = Request.CDNCacheStatus
			};

			// The request is completed so we can remove it from the lookup map
			{
				UE::TUniqueLock _(TicketLookupMutex);
				TicketLookupMap.Remove(Request.TicketId);
			}

			FOnHttpResponse OnResponse = MoveTemp(Request.OnResponse);
			OnResponse(MoveTemp(Response));
		}
	}

	Retries.RemoveAtSwap(0, RequestCount);
}

TUniquePtr<HTTP::FConnectionPool> FMultiEndpointHttpClient::CreateConnection(FAnsiStringView HostUrl) const
{
	using namespace HTTP;

	FConnectionPool::FParams Params;
	ensure(Params.SetHostFromUrl(HostUrl) >= 0);

	if (Config.ReceiveBufferSize >= 0)
	{
		Params.RecvBufSize = Config.ReceiveBufferSize;
	}

	if (Config.SendBufferSize >= 0)
	{
		Params.SendBufSize = Config.SendBufferSize;
	}

	bool bUseHttpTwo = (GIaxHttpVersion == 2);
	bUseHttpTwo &= HostUrl.StartsWith("https://");
	Params.HttpVersion = bUseHttpTwo ? EHttpVersion::Two : EHttpVersion::One;

	if (Params.HttpVersion == EHttpVersion::Two)
	{
		Params.ConnectionCount = 1; // rfc9113 conformance
		Params.VerifyCert = HTTP::FCertRoots::Default();
		Params.MaxInflight = HTTP::FEventLoop::MaxActiveTickets;
	}
	else
	{
		Params.ConnectionCount = uint16(Config.MaxConnectionCount);
		Params.MaxInflight = (uint8)FMath::Clamp(GIaxHttpOneRequestsPerConnection, 1, MAX_uint8);
	}

	return MakeUnique<HTTP::FConnectionPool>(Params);
}

FMultiEndpointHttpClient::FConnection& FMultiEndpointHttpClient::GetConnection(const FOnDemandHostGroup& HostGroup)
{
	for (TUniquePtr<FConnection>& Conn : Connections)
	{
		check(Conn.IsValid());
		if (Conn->HostGroup == HostGroup)
		{
			return *Conn;
		}
	}

	FConnection& Conn = *Connections.Emplace_GetRef(new FConnection
		{
			.HostGroup = HostGroup,
		});

	Conn.Pools.SetNum(HostGroup.Hosts().Num());
	Conn.CurrentHost = HostGroup.PrimaryHostIndex();

	if (Conn.CurrentHost != INDEX_NONE)
	{
		Conn.Pools[Conn.CurrentHost] = CreateConnection(HostGroup.PrimaryHost());
	}

	return Conn;
}

FMultiEndpointHttpClient::FConnection* FMultiEndpointHttpClient::FindConnection(const FOnDemandHostGroup& HostGroup)
{
	for (TUniquePtr<FConnection>& Conn : Connections)
	{
		check(Conn.IsValid());
		if (Conn->HostGroup == HostGroup)
		{
			return Conn.Get();
		}
	}

	return nullptr;
}

uint32 FMultiEndpointHttpClient::GetRetryLimitForRequest(const FRequest& Request) const
{
	return Config.MaxRetryCount == -1 ? Request.Connection.HostGroup.Hosts().Num() : Config.MaxRetryCount;
}

void FMultiEndpointHttpClient::Log(const FMultiEndpointHttpClientResponse& Response, const FRequest& Request) const
{
#if !NO_LOGGING

	using namespace UE::IoStore::HTTP;

	ELogVerbosity::Type Verbosity = ELogVerbosity::VeryVerbose;
	// We count  a request as failed if it's status code is bad AND the request was not canceled
	if (GIaxWarnOnFailure && !IsHttpStatusOk(Response.StatusCode) && !Response.bCanceled)
	{
		Verbosity = ELogVerbosity::Type::Warning;
	}

	if (LogHttpIoDispatcher.IsSuppressed(Verbosity))
	{
		return;
	}

	const uint32 DurationMs = Response.Sample.GetTotalMs();
	const uint64 Size = Response.Body.GetSize() >> 10;

	TStringBuilder<256> Reason;
	if (!Response.Reason.IsEmpty())
	{
		Reason << TEXT(" ") << Response.Reason;
	}
	else if (Response.bCanceled)
	{
		Reason << TEXT(" Canceled");
	}

	FMsg::Logf(__FILE__, __LINE__, LogHttpIoDispatcher.GetCategoryName(), Verbosity,
		TEXT("http-%3u: %5dms %5" UINT64_FMT "KiB [%4s] %s (Attempt %u/%u)%s"),
		Response.StatusCode,
		DurationMs,
		Size,
		CDNCacheStatusToString(Request.CDNCacheStatus),
		WriteToString<512>(Request.Connection.HostGroup.Host(Request.Host), Request.RelativeUrl).ToString(),
		Request.RetryCount,
		GetRetryLimitForRequest(Request),
		*Reason
	);

#endif //!NO_LOGGING
}

void FMultiEndpointHttpClient::LogError(const FRequest& Request, const FString& ErrorReason) const
{
	// Attempt to keep roughly the same format as ::Log where is makes sense.
	UE_LOGF(LogHttpIoDispatcher, Error, "http-  0:     0ms      0KiB [ ???] %hs (Attempt 0/0) %ls", *Request.RelativeUrl, *ErrorReason);
}

} // namespace UE::IoStore
