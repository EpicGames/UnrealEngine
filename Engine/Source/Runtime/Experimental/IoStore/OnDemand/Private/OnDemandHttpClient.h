// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/RecursiveMutex.h"
#include "Containers/AnsiString.h"
#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "IO/Http/Client.h"
#include "IO/IoBuffer.h"
#include "IO/IoOffsetLength.h"
#include "IO/IoStatus.h"
#include "IO/OnDemandHostGroup.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"

// When NO_LOGGING is enabled, then log categories will change type to 'FNoLoggingCategory' which does
// not inherit from 'FLogCategoryBase' so in order to keep code compiling we need to be able to switch
// types as well.
#if !NO_LOGGING
	#define UE_LOG_CATEGORY_TYPE FLogCategoryBase
#else
	#define UE_LOG_CATEGORY_TYPE FNoLoggingCategory
#endif //!NO_LOGGING

namespace UE::IoStore
{

class FOnDemandHttpStats;

FIoStatus LoadDefaultHttpCertificates(bool& bWasLoaded);

/** Returns if a Http status code is okay (2XX) or not */
inline bool IsHttpStatusOk(uint32 StatusCode)
{
	return StatusCode >= 200 && StatusCode < 300;
}

/** Returns if a Http status code indicates a server error (5XX) or not */
inline bool IsHttpServerError(uint32 StatusCode)
{
	return StatusCode >= 500 && StatusCode < 600;
}

enum class EHttpRedirects
{
	/** Redirects will be rejected and handled as failed requests. */
	Disabled,
	/** Follow redirects automatically. */
	Follow
};

struct FMultiEndpointHttpClientConfig
{
	int32						MaxConnectionCount = 4;
	int32						ReceiveBufferSize = -1;
	int32						SendBufferSize = -1;
	int32						MaxRetryCount = -1;	// Positive: The number of times to retry a failed request
													// Zero: Failed requests will not be retried
													// Negative: A failed request will retry once per provided host url
	int32						TimeoutMs = 0;
	EHttpRedirects				Redirects = EHttpRedirects::Follow;
	bool						bEnableThreadSafetyChecks = false;
	bool						bAllowChunkedTransfer = true;
	/** When true the client will return the response body for requests outside of the 2XX range. When false only requests within the 2XX range will return the response body */
	bool						bResponseBodyOnError = false;
};

enum class EMultiEndpointRequestFlags : uint8
{
	None,
	ResponseHeaders
};
ENUM_CLASS_FLAGS(EMultiEndpointRequestFlags);

struct FResponseBody
{
public:
	FResponseBody() = default;

	FResponseBody(const FIoBuffer& InBody)
		: Body(InBody)
	{
	}

	FResponseBody(FIoBuffer&& InBody)
		: Body(MoveTemp(InBody))
	{
	}

	FResponseBody(FIoBuffer&& InBody, const FIoOffsetAndLength& Range);

	FIoStatus Parse(FIoBuffer&& Body, FAnsiStringView Boundary);

	/** 
	 * Returns the total length of the body that was downloaded.
	 * 
	 * Note that if the request was multi-part range then the body will include additional
	 * overhead as each range has it's own small header and so the  size will be larger
	 * than the sum of the requested ranged.
	 */
	uint64 GetSize() const
	{
		return Body.GetSize();
	}

	/**
	 * Returns the entire body in a single buffer as it was returned from the CDN.
	 * 
	 * Note that if the request was multi-part range then this buffer will include
	 * the headers for each range. It is up to the caller to manage that.
	 */
	FIoBuffer GetBody() const
	{
		return Body;
	}

	/**
	 * Returns a view of the data for the given range.
	 * 
	 * The requested range must be within the ranges for the initial request.
	 */
	FIoBuffer GetData(const FIoOffsetAndLength& Range) const;

private:

	void Reset()
	{
		Body = FIoBuffer();
		DataRanges.Empty();
	}

	FIoBuffer					Body;		// The full body from the response

	/**
	 * 
	 *
	 */
	struct FRange
	{
		/* Offset into the Body buffer to where this range begins */
		uint32 BodyOffset	= MAX_uint32;

		/** */
		uint32 StartByte	= MAX_uint32;
		/** */
		uint32 EndByte		= 0;
	};

	TArray<FRange> DataRanges; // Views into the body of the originally requested ranges
};

struct FMultiEndpointHttpClientResponse
{
	bool IsOk() const
	{
		return IsHttpStatusOk(StatusCode);
	}

	bool IsCanceled() const
	{
		return bCanceled;
	}

	FResponseBody				Body;

	TArray<FAnsiString>			Headers;
	FString						Reason;
	HTTP::FTicketPerf::FSample	Sample;
	uint32						StatusCode = 0;
	uint32						RetryCount = 0;
	int32						HostIndex = INDEX_NONE;
	bool						bCanceled = false;
	int8						CDNCacheStatus = -1;
};

/**
 * Todo - More documentation on the client behavior.
 * 
 * Retry policy:
 * If a request fails then the client will retry it up to FMultiEndpointHttpClientConfig::MaxRetryCount
 * times. The first retry attempt will use the primary host with each subsequent attempt cycling to the
 * next host in the FOnDemandHostGroup. If the end of the group is reached with retries remaining then
 * the cycle will begin again at the start of the group.
 * @see FOnDemandHostGroup for more info on how the host cycling works.
 */
class FMultiEndpointHttpClient
{
public:
	UE_NONCOPYABLE(FMultiEndpointHttpClient);

	using FHttpTicketId = uint32;
	using FOnHttpResponse = TFunction<void(FMultiEndpointHttpClientResponse&&)>;

	~FMultiEndpointHttpClient();

	[[nodiscard]] static TUniquePtr<FMultiEndpointHttpClient> Create(const FMultiEndpointHttpClientConfig& Config);

	/** Blocking method */
	[[nodiscard]] static TIoStatusOr<FMultiEndpointHttpClientResponse> Get(FAnsiStringView Url, const FMultiEndpointHttpClientConfig& Config);

	[[nodiscard]] FHttpTicketId Get(const FOnDemandHostGroup& HostGroup, FAnsiStringView RelativeUrl, FOnHttpResponse&& OnResponse);
	[[nodiscard]] FHttpTicketId Get(const FOnDemandHostGroup& HostGroup, FAnsiStringView RelativeUrl, TArray<FIoOffsetAndLength>&& ChunkRanges, FOnHttpResponse&& OnResponse);
	[[nodiscard]] FHttpTicketId Get(const FOnDemandHostGroup& HostGroup, FAnsiStringView RelativeUrl, TArray<FIoOffsetAndLength>&& ChunkRanges, TArray<FAnsiString>&& Headers, EMultiEndpointRequestFlags Flags, FOnHttpResponse&& OnResponse);

	/**
	 * Process the underlying http client.
	 * 
	 * @param WaitTimeMs		How long (in milliseconds) should the underlying http client wait for poll events before returning.
	 * @param MaxKiBPerSecond	The max amount of bandwidth that the underlying http client should use per second. A value of
	 *							zero indicates that the bandwidth use should not be capped.
	 * 
	 * @return True if the client still has work pending and more Tick calls will be needed, false if all work has been completed.
	 */
	bool Tick(int32 WaitTimeMs, uint32 MaxKiBPerSecond);
	bool Tick()
	{
		return Tick(-1, 0);
	}

	void CancelRequest(FHttpTicketId TicketId);

	void UpdateConnections();

	/**
	 * Samples the underlying http client for performance statistics and stores them in the provided FOnDemandHttpStats.
	 * Can be called from any thread, not just the http thread.
	 */
	void GetHttpStats(FOnDemandHttpStats& Out) const;

private:
	using FHttpConnectionPools = TArray<TUniquePtr<HTTP::FConnectionPool>, TInlineAllocator<4>>;

	struct FConnection
	{
		FOnDemandHostGroup		HostGroup;
		FHttpConnectionPools	Pools;
		int32					CurrentHost = INDEX_NONE;
	};

	struct FRequest
	{
		FOnHttpResponse				OnResponse;
		TArray<FAnsiString>			RequestHeaders;
		TArray<FAnsiString>			ResponseHeaders;
		FAnsiString					RelativeUrl;
		TArray<FIoOffsetAndLength>	Ranges; // Consider inline storage?
		FAnsiString					MultiRangeBoundary;
		FHttpTicketId				TicketId = 0;
		FConnection&				Connection;
		/** Stored the response body */
		FIoBuffer					Body;
		/**
		 * Stores each chunk being sent if the response uses chunked transfer, the chunks will be combined
		 * together so that the final result will be in Body once all chunks are processed.
		 */
		FIoBuffer					Chunk;
		/** Stores the response message if the StatusCode indicates a problem */
		FString						ResponseMessage;
		uint64						StartTime = 0;
		uint32						RetryCount = 0;
		uint32						StatusCode = 0;
		int32						Host = INDEX_NONE;
		EMultiEndpointRequestFlags	Flags = EMultiEndpointRequestFlags::None;
		bool						bIsChunkedTransfer = false;
		int8						CDNCacheStatus = -1;

#if DO_CHECK
		/** Used to make sure that the sink is only called with a completed state one per request attempt */
		int32						SinkCounter = 0;
#endif // DO_CHECK

		/** Returns the total length of all the ranges in the provided request */
		uint64 GetTotalRangeLength() const;

		/**
		 * Builds the value for a http 'range' header based on the ranges from the provided request.
		 * Returns true if a valid range exists and false if not.
		 */
		bool BuildRangeHeaderValue(FAnsiStringBuilderBase& OutString)const;
	};

										FMultiEndpointHttpClient(const FMultiEndpointHttpClientConfig& Config);

	FHttpTicketId						IssueRequest(FRequest&& Request);
	void								CompleteRequest(FRequest&& Request, const UE::IoStore::HTTP::FTicketStatus& TicketStatus);
	void								RetryRequest(FRequest&& Request);

	void								ProcessFailedRequests();
	void								ProcessRetryAttempts(uint32 TicketCount);

	TUniquePtr<HTTP::FConnectionPool>	CreateConnection(FAnsiStringView HostUrl) const;
	FConnection&						GetConnection(const FOnDemandHostGroup& HostGroup);
	FConnection*						FindConnection(const FOnDemandHostGroup& HostGroup);

	uint32								GetRetryLimitForRequest(const FRequest& Request) const;

	void								Log(const FMultiEndpointHttpClientResponse& Response, const FRequest& Request) const;
	void								LogError(const FRequest& Request, const FString& ErrorReason) const;

	FMultiEndpointHttpClientConfig		Config;
	TArray<TUniquePtr<FConnection>>		Connections;
	HTTP::FEventLoop					EventLoop;
	TArray<FRequest>					Retries;
	/** Stores the response for requests that failed to be issued, to be flushed in the next Tick */
	TArray<FRequest>					FailedRequests;

	struct FTicketInfo
	{
		HTTP::FTicket HttpTicket;
		bool bCancelRequested = false;
	};

	UE::FRecursiveMutex					TicketLookupMutex;
	TMap<FHttpTicketId, FTicketInfo>	TicketLookupMap;

};

} // namespace UE::IoStore

#undef UE_LOG_CATEGORY_TYPE
