// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnDemandHttpIoDispatcher.h"

#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "AtomicFlags.h"
#include "Containers/AnsiString.h"
#include "CVarUtilities.h"
#include "DownlinkBandwidthManager.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "HAL/Platform.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "IO/HttpIoDispatcher.h"
#include "IO/IoContainers.h"
#include "IO/IoStoreOnDemand.h"
#include "IO/Serialization/OnDemandContainerToc.h"
#include "IasCache.h"
#include "IasHostGroup.h"
#include "Logging/LogVerbosity.h"
#include "Logging/StructuredLog.h"
#include "Misc/CommandLine.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Fork.h"
#include "Misc/SingleThreadRunnable.h"
#include "OnDemandHttpClient.h"
#include "OnDemandMisc.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "ProfilingDebugging/HttpTrace.h"
#include "Statistics.h"

#include <atomic>

#if UE_TRACE_HTTP_ENABLED
#include "IO/IoChunkId.h"
#endif

#if UE_TRACE_HTTP_METADATA_ENABLED
#include "IO/IoDispatcherInternal.h"
#include "IO/PackageId.h"
#include "Misc/PackageName.h"
#endif

// When enabled it will be possible to induce errors to requests in FHttpDispatcher for testing.
#define UE_ALLOW_ERROR_SIMULATION (1 && !NO_CVARS && !UE_BUILD_SHIPPING)
// When enabled it is possible to poison data loaded from the IAS cache via the cvar 'ias.PoisonCache'
#define UE_ALLOW_CACHE_POISONING !UE_BUILD_SHIPPING
// When enabled it is possible to pause the http requests via the cvar 'iax.HttpPause'
#define UE_ALLOW_HTTP_PAUSE (1 && !NO_CVARS && !UE_BUILD_SHIPPING)
// When enabled it is possible to disable all http bandwidth limits via the 'iax.HttpNoRateLimits' cvar
#define UE_ALLOW_DISABLE_HTTP_LIMITS !UE_BUILD_SHIPPING

/** Wrapper around FOutputDevice so we can use LogHttpIoDispatcher category when logging is enabled */
#if NO_LOGGING
	#define UE_CVAR_ERROR_LOG(Msg) Ar.Log(Msg);
#else
	#define UE_CVAR_ERROR_LOG(Msg) Ar.Log(LogHttpIoDispatcher.GetCategoryName(), ELogVerbosity::Error, Msg);
#endif // NO_LOGGING

////////////////////////////////////////////////////////////////////////////////
const TCHAR* LexToString(EThreadPriority Priority)
{
	switch (Priority)
	{
		case EThreadPriority::TPri_Normal:
			return TEXT("TPri_Normal");
		case EThreadPriority::TPri_AboveNormal:
			return TEXT("TPri_AboveNormal");
		case EThreadPriority::TPri_BelowNormal:
			return TEXT("TPri_BelowNormal");
		case EThreadPriority::TPri_Highest:
			return TEXT("TPri_Highest");
		case EThreadPriority::TPri_Lowest:
			return TEXT("TPri_Lowest");
		case EThreadPriority::TPri_SlightlyBelowNormal:
			return TEXT("TPri_SlightlyBelowNormal");
		case EThreadPriority::TPri_TimeCritical:
			return TEXT("TPri_TimeCritical");
		case EThreadPriority::TPri_Num:
		default:
			return TEXT("TPri_Undefined");
	};
}

////////////////////////////////////////////////////////////////////////////////
#if UE_ALLOW_HTTP_PAUSE || UE_ALLOW_ERROR_SIMULATION
void FConsoleCommandUnregister::operator()(IConsoleCommand* ConsoleCommandPtr) const
{
	IConsoleManager::Get().UnregisterConsoleObject(ConsoleCommandPtr);
};
#endif // UE_ALLOW_HTTP_PAUSE || UE_ALLOW_ERROR_SIMULATION

////////////////////////////////////////////////////////////////////////////////
namespace UE::IoStore
{

/** Allows requests to be filtered by type */
enum class EHttpRequestTypeFilter : uint8
{
	None  = 0,

	Streaming = 1 << 0,	/** IAS */
	Installed = 1 << 1,	/** IAD */

	All = Streaming | Installed
};
ENUM_CLASS_FLAGS(EHttpRequestTypeFilter);

////////////////////////////////////////////////////////////////////////////////
DECLARE_MULTICAST_DELEGATE(FOnRecreateHttpClient);
FOnRecreateHttpClient OnRecreateHttpClient;

///////////////////////////////////////////////////////////////////////////////
void OnHttpClientCVarChanged(IConsoleVariable* CVar)
{
	if (ensure(CVar != nullptr))
	{
		const FString CVarName = IConsoleManager::Get().FindConsoleObjectName(CVar);
		UE_LOGFMT(LogIas, Log, "Existing http client config invalidated as cvar '{0}' has changed", CVarName);

		OnRecreateHttpClient.Broadcast();
	}
}

////////////////////////////////////////////////////////////////////////////////
int32 GHttpIoDispatcherMaxRangeKiB = 4096;
static FAutoConsoleVariableRef CVar_HttpIoDispatcherMaxRangeKiB(
	TEXT("iax.HttpIoDispatcherMaxRangeKiB"),
	GHttpIoDispatcherMaxRangeKiB,
	TEXT("Sets the max HTTP range request range according to:\n")
	TEXT("	- a negative value merges all currently queued request(s) for a given resource\n")
	TEXT("	- a zero or positive value will merge requests for a given resource up until the total range goes above this value."),
	ECVF_Default
);

int32 GHttpIoDispatcherMaxMergeGapKiB = 128;
static FAutoConsoleVariableRef CVar_HttpIoDispatcherMaxMergeGapKiB(
	TEXT("iax.HttpIoDispatcherMaxMergeGapKiB"),
	GHttpIoDispatcherMaxMergeGapKiB,
	TEXT("Maximum gap (in KiB) between two chunk ranges before they are no longer eligible to be merged into a single HTTP request.\n")
	TEXT("	- a zero or negative value disables the gap limit (any gap is allowed up to iax.HttpIoDispatcherMaxRangeKiB)."),
	ECVF_Default
);

int32 GIaxHttpSortRequestsByType = -1;
static FAutoConsoleVariableRef CVar_IaxHttpSortRequestsByType(
	TEXT("iax.HttpSortRequestsByType"),
	GIaxHttpSortRequestsByType,
	TEXT("-1 = Favor IAS Request, 0 = Treat all requests equally, 1 = Favor IAD requests")
);

float GHttpIoDispatcherShutdownTimeoutSeconds = 5.0;
static FAutoConsoleVariableRef CVar_HttpIoDispatcherShutdownTimeoutSeconds(
	TEXT("iax.HttpIoDispatcherShutdownTimeoutSeconds"),
	GHttpIoDispatcherShutdownTimeoutSeconds,
	TEXT("Sets the maximum time the dispatcher will wait for outstanding request(s) when shutting down."),
	ECVF_Default
);

/** Note that GIasHttpPrimaryEndpoint has no effect after initial start up */
int32 GIasHttpPrimaryEndpoint = 0;
static FAutoConsoleVariableRef CVar_IasHttpPrimaryEndpoint(
	TEXT("ias.HttpPrimaryEndpoint"),
	GIasHttpPrimaryEndpoint,
	TEXT("Primary endpoint to use returned from the distribution endpoint")
);

int32 GIasHttpTimeOutMs = 10 * 1000;
static FAutoConsoleVariableRef CVar_IasHttpTimeOutMs(
	TEXT("ias.HttpTimeOutMs"),
	GIasHttpTimeOutMs,
	TEXT("Time out value for HTTP requests in milliseconds")
);

bool GIasHttpEnabled = true;
static FAutoConsoleVariableRef CVar_IasHttpEnabled(
	TEXT("ias.HttpEnabled"),
	GIasHttpEnabled,
	TEXT("Enables individual asset streaming via HTTP")
);

bool GIasHttpOptionalBulkDataEnabled = true;
static FAutoConsoleVariableRef CVar_IasHttpOptionalBulkDataEnabled(
	TEXT("ias.HttpOptionalBulkDataEnabled"),
	GIasHttpOptionalBulkDataEnabled,
	TEXT("Enables optional bulk data via HTTP")
);

bool GIasReportAnalyticsEnabled = true;
static FAutoConsoleVariableRef CVar_IoReportAnalytics(
	TEXT("ias.ReportAnalytics"),
	GIasReportAnalyticsEnabled,
	TEXT("Enables reporting statics to the analytics system")
);

static int32 GDistributedEndpointRetryWaitTime = 15;
static FAutoConsoleVariableRef CVar_DistributedEndpointRetryWaitTime(
	TEXT("ias.DistributedEndpointRetryWaitTime"),
	GDistributedEndpointRetryWaitTime,
	TEXT("How long to wait (in seconds) after failing to resolve a distributed endpoint before retrying")
);

static int32 GDistributedEndpointAttemptCount = 5;
static FAutoConsoleVariableRef CVar_DistributedEndpointAttemptCount(
	TEXT("ias.DistributedEndpointAttemptCount"),
	GDistributedEndpointAttemptCount,
	TEXT("Number of times we should try to resolve a distributed endpoint befor eusing the fallback url (if there is one)")
);

bool GIasEnableRequestHandleClear = true;
static FAutoConsoleVariableRef CVar_EnableRequestHandleClear(
	TEXT("ias.EnableRequestHandleClear"),
	GIasEnableRequestHandleClear,
	TEXT("When enabled FChunkRequest::HttpRequest will be cleared when the request callback is triggered")
);

#if UE_ALLOW_CACHE_POISONING
bool GIasPoisonCache = false;
static FAutoConsoleVariableRef CVar_IasPoisonCache(
	TEXT("ias.PoisonCache"),
	GIasPoisonCache,
	TEXT("Fills all data materialized from the cache with 0x4d")
);
#endif // UE_ALLOW_CACHE_POISONING

int32 GIasHttpHealthCheckWaitTime = 3000;
static FAutoConsoleVariableRef CVar_IasHttpHealthCheckWaitTime(
	TEXT("ias.HttpHealthCheckWaitTime"),
	GIasHttpHealthCheckWaitTime,
	TEXT("Number of milliseconds to wait before reconnecting to avaiable endpoint(s)")
);

int32 GOnDemandBackendThreadPriorityIndex = 4; // EThreadPriority::TPri_AboveNormal
FAutoConsoleVariableRef CVarOnDemandBackendThreadPriority(
	TEXT("ias.onDemandBackendThreadPriority"),
	GOnDemandBackendThreadPriorityIndex,
	TEXT("Thread priority of the on demand backend thread: 0=Lowest, 1=BelowNormal, 2=SlightlyBelowNormal, 3=Normal, 4=AboveNormal\n")
	TEXT("Note that this is switchable at runtime"),
	ECVF_Default);

int32 GIaxHttpVersion = 2;
static FAutoConsoleVariableRef CVar_IaxHttpVersion(
	TEXT("iax.HttpVersion"),
	GIaxHttpVersion,
	TEXT("Protocol version to prefer, either '1' or '2'. The latter requires https."),
	FConsoleVariableDelegate::CreateStatic(&OnHttpClientCVarChanged)
);

int32 GIasHttpConnectionCount = 4;
static FAutoConsoleVariableRef CVar_IasHttpConnectionCount(
	TEXT("ias.HttpConnectionCount"),
	GIasHttpConnectionCount,
	TEXT("Number of open HTTP connections to the on demand endpoint(s)."),
	FConsoleVariableDelegate::CreateStatic(&OnHttpClientCVarChanged)
);

/* Note that the value of this cvar does not affect throttling. Setting it too low
 * will prevent the CDN peer from reordering cache hits ahead of misses. Latency
 * will also start to become apparent and negatively effect user experience. */
int32 GIaxHttpMaxInflight = 8;
static FAutoConsoleVariableRef CVar_IaxHttpMaxInflight(
	TEXT("iax.HttpMaxInflight"),
	GIaxHttpMaxInflight,
	TEXT("Number of requests to issue at once"),
	FConsoleVariableDelegate::CreateStatic(&OnHttpClientCVarChanged)
);

int32 GIaxHttpOneRequestsPerConnection = 1;
static FAutoConsoleVariableRef CVar_IaxHttpOneRequestsPerConnection(
	TEXT("iax.Http1RequestPerConnection"),
	GIaxHttpOneRequestsPerConnection,
	TEXT("The number of requests to allow on each connection when GIaxHttpVersion == 1"),
	FConsoleVariableDelegate::CreateStatic(&OnHttpClientCVarChanged)
);

int32 GIasHttpRecvBufKiB = -1;
static FAutoConsoleVariableRef CVar_GIasHttpRecvBufKiB(
	TEXT("ias.HttpRecvBufKiB"),
	GIasHttpRecvBufKiB,
	TEXT("Recv buffer size"),
	FConsoleVariableDelegate::CreateStatic(&OnHttpClientCVarChanged)
);

int32 GIasHttpSendBufKiB = -1;
static FAutoConsoleVariableRef CVar_GIasHttpSendBufKiB(
	TEXT("ias.HttpSendBufKiB"),
	GIasHttpSendBufKiB,
	TEXT("Send buffer size"),
	FConsoleVariableDelegate::CreateStatic(&OnHttpClientCVarChanged)
);

int32 GIasHttpRetryCount = -1;
static FAutoConsoleVariableRef CVar_IasHttpRetryCount(
	TEXT("ias.HttpRetryCount"),
	GIasHttpRetryCount,
	TEXT("Number of times that a request should be retried before being considered failed. A negative value will use the default behaviour, which is one retry per host url provided."),
	FConsoleVariableDelegate::CreateStatic(&OnHttpClientCVarChanged)
);

int32 GIasHttpFailTimeOutMs = 4 * 1000;
static FAutoConsoleVariableRef CVar_IasHttpFailTimeOutMs(
	TEXT("ias.HttpFailTimeOutMs"),
	GIasHttpFailTimeOutMs,
	TEXT("Fail infinite network waits that take longer than this (in ms, a value of zero will use the default)"),
	FConsoleVariableDelegate::CreateStatic(&OnHttpClientCVarChanged)
);

bool GIasHttpAllowChunkedXfer = true;
static FAutoConsoleVariableRef CVar_IasHttpAllowChunkedXfer(
	TEXT("ias.HttpAllowChunkedXfer"),
	GIasHttpAllowChunkedXfer,
	TEXT("Enable/disable IAS' support for chunked transfer encoding"),
	FConsoleVariableDelegate::CreateStatic(&OnHttpClientCVarChanged)
);

int32 GIasHttpRateLimitKiBPerSecond = 0;
static FAutoConsoleVariableRef CVar_GIasHttpRateLimitKiBPerSecond(
	TEXT("ias.HttpRateLimitKiBPerSecond"),
	GIasHttpRateLimitKiBPerSecond,
	TEXT("Http throttle limit in KiBPerSecond")
);

#if UE_ALLOW_DISABLE_HTTP_LIMITS
bool GIasHttpNoRateLimits = false;
static FAutoConsoleVariableRef CVar_GIasHttpNoRateLimit(
	TEXT("iax.HttpNoRateLimits"),
	GIasHttpNoRateLimits,
	TEXT("When true, we ignore all bandwidth limits and completely unthrottle the connection. Not avaliable in UE_BUILD_SHIPPING.")
);
#endif // UE_ALLOW_DISABLE_HTTP_LIMITS

int32 GIasHttpPollTimeoutMs = 17;
static FAutoConsoleVariableRef CVar_GIasHttpPollTimeoutMs(
	TEXT("ias.HttpPollTimeoutMs"),
	GIasHttpPollTimeoutMs,
	TEXT("Http tick poll timeout in milliseconds")
);

bool GIaxHttpEnableInflightCancellation = true;
static FAutoConsoleVariableRef CVar_IaxHttpEnableInflightCancellation(
	TEXT("iax.HttpEnableInflightCancellation"),
	GIaxHttpEnableInflightCancellation,
	TEXT("When enabled the system will attempt to cancel inflight http requests")
);

bool GHttpCacheEnabled = true;
static FAutoConsoleVariableRef CVar_HttpCacheEnabled(
	TEXT("iax.HttpCacheEnabled"),
	GHttpCacheEnabled,
	TEXT("When set to false the IAS cache will be ignored. No new data will be written to it and no data will be read from it.")
);

int32 GIasHttpCacheJournalMagic = 0;
static FAutoConsoleVariableRef CVar_IasHttpCacheJournalMagic(
	TEXT("ias.HttpCacheJournalMagic"),
	GIasHttpCacheJournalMagic,
	TEXT("A value used to mark entries in the IAS cache. Change the value to invalidate existing entries without deleting the cache on disk.")
);

/** Note that this will most likely be used before config hotfixs are applied */
bool GIaxHttpForceInitialConnectionCheck = true;
static FAutoConsoleVariableRef CVar_IaxHttpForceInitialConnectionCheck(
	TEXT("ias.IaxHttpForceInitialConnectionCheck"),
	GIaxHttpForceInitialConnectionCheck,
	TEXT("When true the IAX dispatcher will be forced to attempt to connect to at least one hostgroup before servicing requests")
);

bool GUseDownlinkBandwidthManager = false;
static FAutoConsoleVariableRef CVar_UseDLBWManager(
	TEXT("iax.UseDLBWManager"),
	GUseDownlinkBandwidthManager,
	TEXT("Toggle state for whether or not the HttpDispatcher will use the Downlink Bandwidth Manager.")
);

bool GUseDelayedDownlinkBandwidthManagerSubscription = true;
static FAutoConsoleVariableRef CVar_UseDLBWManagerDelayedSubscription(
	TEXT("iax.UseDelayedDLBWManager"),
	GUseDelayedDownlinkBandwidthManagerSubscription,
	TEXT("Toggle state for whether or not the HttpDispatcher sign up for a delayed CVar triggered subscription for the Bandwidth Manager.")
);

bool GLegacyBatchSorting = false;
static FAutoConsoleVariableRef CVar_UseLegacyBatchSorting(
	TEXT("iax.LegacyBatchSorting"),
	GLegacyBatchSorting,
	TEXT("Toggle between legacy or new batch sorting.")
);

////////////////////////////////////////////////////////////////////////////////
int32 CalculateRequestQueueLengthMax()
{
	// Keeping the loop loaded with work to do is important to cover round-trip
	// latency and the effect of CDN cache misses. Reducing what is in the queue
	// DOES NOT have any effect on throughput throttling.
	int32 ConnNum = (GIaxHttpVersion == 2) ? 1 : GIasHttpConnectionCount;
	int32 InflightNum = (GIaxHttpVersion == 2) ? GIaxHttpMaxInflight : GIaxHttpOneRequestsPerConnection;
	int32 Value = FMath::Max((ConnNum * InflightNum) + 2, 1);

	return FMath::Min(Value, int32(HTTP::FEventLoop::MaxActiveTickets));
}

////////////////////////////////////////////////////////////////////////////////

// TODO: Should be a member of FHttpDispatcher but needs to be global so that we can expose GetHttpRateLimitKiBPerSecond to FOnDemandIoBackendStats for now
std::atomic<int32> DLBWHttpRateLimitKiBPerSecond = 0;

int32 GetHttpRateLimitKiBPerSecond()
{
#if UE_ALLOW_DISABLE_HTTP_LIMITS
	if (GIasHttpNoRateLimits)
	{
		return 0;
	}
#endif // UE_ALLOW_DISABLE_HTTP_LIMITS

	return (GUseDownlinkBandwidthManager && FDownlinkBandwidthManager::HasValueEnforcementPassed()) ? DLBWHttpRateLimitKiBPerSecond.load() : GIasHttpRateLimitKiBPerSecond;
}

////////////////////////////////////////////////////////////////////////////////
static constexpr int32 GThreadPriorities[5] =
{
	EThreadPriority::TPri_Lowest,
	EThreadPriority::TPri_BelowNormal,
	EThreadPriority::TPri_SlightlyBelowNormal,
	EThreadPriority::TPri_Normal,
	EThreadPriority::TPri_AboveNormal
};

////////////////////////////////////////////////////////////////////////////////
namespace HttpIoDispatcher
{

#if UE_ALLOW_ERROR_SIMULATION

/** Used to help induce errors into our system for testing purposes */
class FRequestErrorSimulator
{
public:
	FRequestErrorSimulator()
	{
		SimulateError.Bind(TEXT("iax.HttpErrorChance"), TEXT("Chance that a request will fail with an error (0-100%) and optional request type filter [IAD/IAS/All]. By default the request filter will be set to All."));
		SimulateInvalidUrl.Bind(TEXT("iax.InvalidUrlChance"), TEXT("Chance that a url for a GET request will be invalid (0-100%) and optional request type filter [IAD/IAS/All]. By default the request filter will be set to All."));
	}

	~FRequestErrorSimulator() = default;

	bool ShouldSimulateError(EHttpRequestType RequestType)
	{
		return SimulateError.TriggerForRequestType(RequestType);
	}

	bool ShouldSimulateInvalidUrl(EHttpRequestType RequestType)
	{
		return SimulateInvalidUrl.TriggerForRequestType(RequestType);
	}

private:

	struct FChance
	{
		void Bind(const TCHAR* Name, const TCHAR* HelpTip)
		{
			ConsoleCommandHandle = FConsoleCommandPtr(IConsoleManager::Get().RegisterConsoleCommand(
				Name, HelpTip,
				FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateRaw(this, &FChance::OnSet),
				ECVF_Default));
		}

		bool TriggerForRequestType(EHttpRequestType RequestType)
		{
			if (Chance <= 0.0f)
			{
				return false;
			}

			if (!EnumHasAnyFlags(Filter, ToFilter(RequestType)))
			{
				return false;
			}

			return (FMath::FRand() * 100.0f) < Chance;
		}

		enum class ERequestFilter : uint8
		{
			Invalid = 0, // We must have at least one request type, so this is our error state
			Installed = 1 << 0,
			Streaming = 1 << 1,

			All = Installed | Streaming
		};

	private:
		ERequestFilter ToFilter(EHttpRequestType Type)
		{
			switch (Type)
			{
				case EHttpRequestType::Installed:
					return ERequestFilter::Installed;
				case EHttpRequestType::Streaming:
					return ERequestFilter::Streaming;
				default:
					checkNoEntry();
					return ERequestFilter::All;
			}
		}

		void OnSet(const TArray<FString>& Args, FOutputDevice& Ar)
		{
			if (Args.Num() < 1 || Args.Num() > 2)
			{
				UE_CVAR_ERROR_LOG(TEXT("Invalid args! Expected '<command> [chance (float)] [IAD/IAS/All (string)]'"));
				return;
			}

			float ParsedChance = 0.0f;
			if (!LexTryParseString(ParsedChance, *Args[0]) || ParsedChance < 0.0f || ParsedChance > 100.0f)
			{
				UE_CVAR_ERROR_LOG(TEXT("Invalid arg!, first arg should be a number between 0.0 and 100.0"));
				return;
			}

			ERequestFilter ParsedFilter = ParseFilter(Args, Ar);
			if (ParsedFilter == ERequestFilter::Invalid)
			{
				return;
			}

			Chance = ParsedChance;
			Filter = ParsedFilter;
		}

		ERequestFilter ParseFilter(const TArray<FString>& Args, FOutputDevice& Ar)
		{
			if (Args.Num() == 2)
			{
				if (Args[1] == TEXT("IAD"))
				{
					return ERequestFilter::Installed;
				}
				else if (Args[1] == TEXT("IAS"))
				{
					return ERequestFilter::Streaming;
				}
				else if (Args[1] == TEXT("All"))
				{
					return ERequestFilter::All;
				}
				else
				{
					UE_CVAR_ERROR_LOG(TEXT("Unrecognized arg!, filter arg should be [IAD/IAS/All]"));
					return ERequestFilter::Invalid;
				}
			}

			return ERequestFilter::All;
		}

		FConsoleCommandPtr	ConsoleCommandHandle;
		float				Chance = 0.0f;
		ERequestFilter		Filter = ERequestFilter::All;
	};

	FChance SimulateError;
	FChance SimulateInvalidUrl;
};

#endif // UE_ALLOW_ERROR_SIMULATION

/**
 * Tracks the lifespan of a http request for as long as we want to keep it active. When the token is
 * destroyed we will try to cancel the request if it is still in progress.
 */
class FCancellationToken
{
public:
	FCancellationToken(FMultiEndpointHttpClient& InClient, FMultiEndpointHttpClient::FHttpTicketId InTicketId)
		: Client(InClient)
		, TicketId(InTicketId)
	{
	}

	~FCancellationToken()
	{
		if (TicketId != 0)
		{
			Client.CancelRequest(TicketId);
		}
	}

	void OnRequestCompleted()
	{
		TicketId = 0;
	}

private:
	FMultiEndpointHttpClient& Client;
	FMultiEndpointHttpClient::FHttpTicketId TicketId;
};

////////////////////////////////////////////////////////////////////////////////
static FIoHash GetCacheKey(const FIoHash& ChunkHash, const FIoHttpRange& Range)
{
	FIoHashBuilder HashBuilder;
	HashBuilder.Update(ChunkHash.GetBytes(), sizeof(FIoHash::ByteArray));
	HashBuilder.Update(&Range, sizeof(FIoHttpRange));

	return HashBuilder.Finalize();
}

////////////////////////////////////////////////////////////////////////////////
static FIoHash GetCacheKey(const FName& HostGroup, const FIoRelativeUrl Url, const FIoHttpHeaders& Headers, const FIoHttpOptions& Options)
{
	FIoHashBuilder HashBuilder;

	TCHAR NameString[NAME_SIZE];
	const uint32 NameLength = HostGroup.GetPlainNameString(NameString);
	HashBuilder.Update(NameString, NameLength * sizeof(TCHAR));
	HashBuilder.Update(Url.ToString(), Url.Len() * sizeof(FIoRelativeUrl::ElementType));

	TConstArrayView<FAnsiString> HeadersView = Headers.ToArrayView();
	check(HeadersView.IsEmpty() || ((HeadersView.Num() % 2) == 0));
	for (int32 Idx = 0; Idx < HeadersView.Num(); Idx += 2)
	{
		const FAnsiString Name	= HeadersView[Idx];
		const FAnsiString Value = HeadersView[Idx + 1];
		HashBuilder.Update(*Name, Name.Len());
		HashBuilder.Update(*Value, Value.Len());
	}

	HashBuilder.Update(&Options.GetRange(), sizeof(FIoHttpRange));
	return HashBuilder.Finalize();
}

////////////////////////////////////////////////////////////////////////////////
struct FHttpRequestBlockAllocationTag : FDefaultBlockAllocationTag
{
	static constexpr uint32			BlockSize = 4 * 1024;
	static constexpr bool			AllowOversizedBlocks = false;
	static constexpr bool			RequiresAccurateSize = false;
	static constexpr bool			InlineBlockAllocation = true;
	static constexpr const char*	TagName = "IoHttpRequestLinear";

	using Allocator = TBlockAllocationCache<BlockSize, FAlignedAllocator>;
};

////////////////////////////////////////////////////////////////////////////////
enum class EHttpRequestFlags : uint32
{
	None				= 0,
	Issued				= (1 << 0),
	CacheInflight		= (1 << 1),
	HttpQueued			= (1 << 2),
	HttpInflight		= (1 << 3),
	CancelRequested		= (1 << 4),
	Completing			= (1 << 5),
	Completed			= (1 << 6)
};
ENUM_CLASS_FLAGS(EHttpRequestFlags);

////////////////////////////////////////////////////////////////////////////////
struct FHttpRequest final
	: TConcurrentLinearObject<FHttpRequest, FHttpRequestBlockAllocationTag>
	, TIntrusiveListElement<FHttpRequest>
	, IIoHttpRequestHandle
{
	using FRefCountValue			= FReturnedRefCountValue;
	using FFlags					= TAtomicFlags<EHttpRequestFlags>;
	static uint32					NextSeqNo;

	EHttpRequestType				GetHttpRequestType() const		{ return Options.GetCategory() == 1 ? EHttpRequestType::Installed : EHttpRequestType::Streaming; }
	virtual EIoErrorCode			GetStatus() const override		{ return CompletionStatus.load(std::memory_order_acquire); }
	uint32							GetRangeSize() const			{ return Options.GetRange().IsValid() ? Options.GetRange().GetSize() : 0; }

	virtual void					AddRef() const override			{ RefCount.fetch_add(1, std::memory_order_acq_rel); }
	virtual FRefCountValue			GetRefCount() const override	{ return FRefCountValue(RefCount.load(std::memory_order_relaxed)); }
	virtual inline FRefCountValue	Release() const override;

	FIoHash							ChunkHash;
	FIoHttpRequestCompleted 		OnCompleted;
	FIoHttpHeaders					Headers;
	FIoHttpOptions					Options;
	FIoBuffer						Buffer;
	FIoRelativeUrl					RelativeUrl;
	FHttpRequest*					Next = nullptr;
	FName							HostGroupName;
	EIoErrorCode					CacheStatus = EIoErrorCode::Unknown;
	FFlags							Flags;
	mutable std::atomic<int32>		RefCount{0};
	std::atomic<EIoErrorCode>		CompletionStatus{EIoErrorCode::Unknown};
	uint32							SeqNo = ++NextSeqNo;

	TSharedPtr<FCancellationToken>	CancellationToken;
};
using FHttpRequestList = TIntrusiveList<FHttpRequest>;

////////////////////////////////////////////////////////////////////////////////
FReturnedRefCountValue FHttpRequest::Release() const
{
	const int32 PrevRefs = RefCount.fetch_sub(1, std::memory_order_acq_rel);
	if (PrevRefs == 1)
	{
		ensureMsgf(Next == nullptr, TEXT("The HTTP request must not be released while in a batch"));
#if !UE_BUILD_SHIPPING 
		const EHttpRequestFlags CurrentFlags = Flags.Get(std::memory_order_acq_rel);
		ensureMsgf(CurrentFlags == EHttpRequestFlags::None || EnumHasAnyFlags(CurrentFlags, EHttpRequestFlags::Completing | EHttpRequestFlags::Completed),
			TEXT("The HTTP request handle must not be released before the completion callback has been triggered."));
#endif
		delete this;
	}

	check(PrevRefs > 0);
	return FReturnedRefCountValue(PrevRefs - 1);
}

////////////////////////////////////////////////////////////////////////////////
static bool HttpRequestSortPredicate(const FHttpRequest& LHS, const FHttpRequest& RHS)
{
	// 'iax.HttpSortRequestsByType' must be enabled and the requests need to be of different types
	// Note this assumes that all requests in the chain have the same type which is currently true.
	if (GIaxHttpSortRequestsByType != 0 && LHS.GetHttpRequestType() != RHS.GetHttpRequestType())
	{
		if (GIaxHttpSortRequestsByType < 0) // Negative value favors streaming (IAS)
		{
			return LHS.GetHttpRequestType() == EHttpRequestType::Streaming;
		}

		if (GIaxHttpSortRequestsByType > 0) // Positive value favors installed (IAD)
		{
			return LHS.GetHttpRequestType() == EHttpRequestType::Installed;
		}
	}

	if (LHS.Options.GetPriority() == RHS.Options.GetPriority())
	{
		return LHS.SeqNo < RHS.SeqNo;
	}

	return LHS.Options.GetPriority() > RHS.Options.GetPriority();
}

uint32 FHttpRequest::NextSeqNo = 0;

////////////////////////////////////////////////////////////////////////////////
TArray<FIoOffsetAndLength> GetTotalRange(FHttpRequest* Request)
{
	TArray<FIoOffsetAndLength> Ranges;

	if (GHttpIoDispatcherMaxRangeKiB < 0)
	{
		// If GHttpIoDispatcherMaxRangeKiB is negative then we will make multipart range request
		// covering each request in the list. We can rely on the lower level systems to optimize
		// the requests later.
		for (FHttpRequest* It = Request; It != nullptr; It = It->Next)
		{
			// If the range is invalid we fetch the entire resource
			if (It->Options.GetRange().IsValid() == false)
			{
				return TArray<FIoOffsetAndLength>();
			}

			Ranges.Add(It->Options.GetRange().ToOffsetAndLength());
		}
	}
	else
	{
		// If GHttpIoDispatcherMaxRangeKiB is 0 or positive then we want to make a partial range request
		// and should only produce a single range to read.
		FIoHttpRange TotalRange;
		for (FHttpRequest* It = Request; It != nullptr; It = It->Next)
		{
			// If the range is invalid we fetch the entire resource
			if (It->Options.GetRange().IsValid() == false)
			{
				return TArray<FIoOffsetAndLength>();
			}
			TotalRange += It->Options.GetRange();
		}

		Ranges.Add(TotalRange.ToOffsetAndLength());
	}

	return Ranges;
}

////////////////////////////////////////////////////////////////////////////////
FHttpRequestList CreateBatch(
	FHttpRequest& Request,
	uint32 MaxRangeSize,
	uint32 MaxGapSize,
	FHttpRequestList&& List,
	FHttpRequestList& OutRemaining)
{
	// If any request didn't specify a range we fetch the whole resource 
	for (const FHttpRequest& Existing : List)
	{
		if (Existing.Options.GetRange().IsValid() == false)
		{
			return MoveTemp(List);
		}
	}

	if (GLegacyBatchSorting)
	{
		// The request is expected to be in the list
		const bool bRemoved = List.Remove(&Request);
		check(bRemoved);

		FIoHttpRange		Range = Request.Options.GetRange();
		FHttpRequestList	Batch(&Request);

		while (FHttpRequest* NextRequest = List.PopHead())
		{
			FIoHttpRange NextRange = Range + NextRequest->Options.GetRange();
			if (NextRange.GetSize() > MaxRangeSize)
			{
				OutRemaining.AddTail(NextRequest);
			}
			else
			{
				Range = NextRange;
				Batch.AddTail(NextRequest);
			}
		}

		return Batch;
	}
	else
	{
		auto AddRange = [](const FIoHttpRange& Lhs, const FIoHttpRange& Rhs, uint32& OutGap) -> FIoHttpRange
		{
			const FIoHttpRange Sum	= Lhs + Rhs;
			const uint32 Growth		= Sum.GetSize() - Lhs.GetSize();
			const uint32 Useful		= Rhs.GetSize();
			OutGap					= Growth > Useful ? Growth - Useful : 0;

			return Sum;
		};

		OutRemaining = MoveTemp(List);

		TArray<FHttpRequest*, TInlineAllocator<32>> SortedRequests;
		for (FHttpRequest& R : OutRemaining)
		{
			SortedRequests.Add(&R);
		}
		SortedRequests.Sort([](const FHttpRequest& Lhs, const FHttpRequest& Rhs)
		{
			return Lhs.Options.GetRange().GetMin() < Rhs.Options.GetRange().GetMin();
		});

		const bool bRemoved = OutRemaining.Remove(&Request);
		check(bRemoved);

		FHttpRequestList	Batch(&Request);
		FIoHttpRange		Range = Request.Options.GetRange();

		const int32 Count	= SortedRequests.Num();
		const int32 Index	= SortedRequests.Find(&Request);
		int32 LhsIdx		= Index - 1;
		int32 RhsIdx		= Index + 1;

		for (;;)
		{
			FHttpRequest* Lhs		= LhsIdx > INDEX_NONE ? SortedRequests[LhsIdx] : nullptr;
			FHttpRequest* Rhs		= RhsIdx < Count ? SortedRequests[RhsIdx] : nullptr;
			FHttpRequest* Candidate	= nullptr;

			if (Lhs == nullptr)
			{
				Candidate = Rhs;
			}
			else if (Rhs == nullptr)
			{
				Candidate = Lhs;
			}
			else
			{
				uint32 LhsGap = 0;
				uint32 RhsGap = 0;
				const FIoHttpRange LhsRange = AddRange(Range, Lhs->Options.GetRange(), LhsGap);
				const FIoHttpRange RhsRange = AddRange(Range, Rhs->Options.GetRange(), RhsGap);

				const bool bLhsValid = LhsRange.GetSize() <= MaxRangeSize && (MaxGapSize == MAX_uint32 || LhsGap <= MaxGapSize);
				const bool bRhsValid = RhsRange.GetSize() <= MaxRangeSize && (MaxGapSize == MAX_uint32 || RhsGap <= MaxGapSize);

				if (!bLhsValid && !bRhsValid)
				{
					Candidate = nullptr;
				}
				else if (!bLhsValid)
				{
					Candidate = Rhs;
				}
				else if (!bRhsValid)
				{
					Candidate = Lhs;
				}
				else
				{
					if (LhsGap != RhsGap)
					{
						Candidate = LhsGap < RhsGap ? Lhs : Rhs;
					}
					else
					{
						Candidate = LhsRange.GetSize() < RhsRange.GetSize() ? Lhs : Rhs;
					}
				}
			}

			if (Candidate == nullptr)
			{
				break;
			}

			uint32 Gap = 0;
			const FIoHttpRange NextRange = AddRange(Range, Candidate->Options.GetRange(), Gap);
			if ((NextRange.GetSize() > MaxRangeSize) || (MaxGapSize < MAX_uint32 && Gap > MaxGapSize))
			{
				break;
			}

			Range = NextRange;
			OutRemaining.Remove(Candidate);
			Batch.AddTail(Candidate);

			if (Candidate == Lhs)
			{
				LhsIdx--;
			}
			else
			{
				RhsIdx++;
			}
		}

		return Batch;
	}
}

////////////////////////////////////////////////////////////////////////////////
#if UE_TRACE_HTTP_ENABLED

class FTraceContext
{
public:
	void					Init(void* Dispatcher);
	void					TraceRequestStarted(FHttpRequest* Request, const ANSICHAR* Url);
	void					TraceRequestCompleted(
								const FHttpRequest* Request,
								const FMultiEndpointHttpClientResponse& Response,
								const FIASHostGroup& HostGroup);
	void					TracePackageName(const FIoChunkId& ChunkId);
	static FTraceContext&	Get();

private:
	struct FPackageNameCache
	{
		static constexpr int32 Slack = 256;

		void Reset()
		{
			PackageName.Reset();
			Filename.Reset();
		}

		TUtf8StringBuilder<Slack>	Filename;
		TStringBuilder<Slack>		PackageName;
	};

	uint64				Dispatcher = 0;
	FPackageNameCache	PackageNameCache;
};

////////////////////////////////////////////////////////////////////////////////
void FTraceContext::Init(void* InDispatcher)
{
	Get().Dispatcher = uint64(InDispatcher);
	TRACE_HTTP_DISPATCHER_CREATED(uint64(InDispatcher), "HttpIoDispatcher");
	TRACE_HTTP_CATEGORY_CREATED(uint8(EHttpRequestType::Streaming), "Streaming");
	TRACE_HTTP_CATEGORY_CREATED(uint8(EHttpRequestType::Installed), "Install");
}

FTraceContext& FTraceContext::Get()
{
	static FTraceContext Ctx;
	return Ctx;
}

void FTraceContext::TraceRequestStarted(FHttpRequest* Request, const ANSICHAR* Url)
{
	TRACE_HTTP_REQUEST_STARTED(Dispatcher, uint64(Request), Url, Request->Options.GetPriority(), Request->Options.GetCategory());

	if (Request->ChunkHash.IsZero())
	{
		// Generic HTTP request
		return;
	}

	for (const FHttpRequest* It = Request; It != nullptr; It = It->Next)
	{
		const FIoChunkId&	ChunkId	= It->Options.GetMetadata().GetChunkId();
		const FIoHttpRange	Range	= It->Options.GetRange();
		TracePackageName(ChunkId);
		TRACE_HTTP_CHUNK_RANGE_ADDED(uint64(Request), ChunkId, Range.GetMin(), Range.GetMax());
	}
}

void FTraceContext::TraceRequestCompleted(
	const FHttpRequest* Request,
	const FMultiEndpointHttpClientResponse& Response,
	const FIASHostGroup& HostGroup)
{
	FAnsiStringView Host = Response.HostIndex != INDEX_NONE ? HostGroup.GetHostUrls()[Response.HostIndex] : HostGroup.GetPrimaryHostUrl();
	TRACE_HTTP_REQUEST_COMPLETED(uint64(Request), Host.GetData(), Response.StatusCode, uint32(Response.Body.GetSize()));
}

void FTraceContext::TracePackageName(const FIoChunkId& ChunkId)
{
#if UE_TRACE_HTTP_METADATA_ENABLED
	const bool bTraceHttpChannelEnabled = UE_TRACE_CHANNELEXPR_IS_ENABLED(HttpChannel);
	if (bTraceHttpChannelEnabled == false)
	{
		return;
	}

	PackageNameCache.Reset();
	if (FStringView PackageName = UE::IoStore::TryConvertChunkIdToPackageName(
		ChunkId,
		PackageNameCache.Filename,
		PackageNameCache.PackageName); !PackageName.IsEmpty())
	{
		FPackageId PackageId;
		FMemory::Memcpy(&PackageId, &ChunkId, sizeof(FPackageId));
		UE_TRACE_PACKAGE_NAME(PackageId, FName(PackageNameCache.PackageName));
	}

#endif // UE_TRACE_HTTP_METADATA_ENABLED
}

#define TRACE_INIT(Dispatcher) \
	FTraceContext::Get().Init(Dispatcher)
#define TRACE_REQUEST_STARTED(Request, Url) \
	FTraceContext::Get().TraceRequestStarted(Request, Url)
#define TRACE_REQUEST_COMPLETED(Request, Response, HostGroup) \
	FTraceContext::Get().TraceRequestCompleted(Request, Response, HostGroup)

////////////////////////////////////////////////////////////////////////////////
#else

#define TRACE_INIT(...)
#define TRACE_REQUEST_STARTED(...)
#define TRACE_REQUEST_COMPLETED(...)

#endif // UE_TRACE_HTTP_ENABLED

////////////////////////////////////////////////////////////////////////////////
class FHttpQueue
{
public:
	FHttpQueue();
	~FHttpQueue() = default;

	void			Reprioritize(FHttpRequest* Request, int32 NewPriority);
	bool			Cancel(FHttpRequest* Request);
	void			Enqueue(FHttpRequest* Request);
	/**
	* Returns one or more requests. Multiple requests will returned as a linked list (FHttpRequest::Next)
	* and will all be for the same chunk.
	*/
	FHttpRequest*	Dequeue();

#if UE_ALLOW_HTTP_PAUSE
	void			OnTogglePause(bool bPause, EHttpRequestTypeFilter Filter);
	bool			IsPaused(FHttpRequest* Request) const;
#endif // UE_ALLOW_HTTP_PAUSE

private:

	void			EnqueueInternal(FHttpRequest* Request);

	TArray<FHttpRequest*>			Heap;
	TMap<FIoHash, FHttpRequestList>	ByChunkKey;
	FMutex							Mutex;
#if UE_ALLOW_HTTP_PAUSE
	EHttpRequestTypeFilter			PauseFilter = EHttpRequestTypeFilter::None;
	TArray<FHttpRequest*>			PausedRequests;
#endif // UE_ALLOW_HTTP_PAUSE
	bool							bReprioritize = false;

	/** Slack to leave in containers when emptying them */
	static constexpr int32			ContainerSlackSize = 16;
};

////////////////////////////////////////////////////////////////////////////////

FHttpQueue::FHttpQueue()
{
	Heap.Reserve(ContainerSlackSize);
	ByChunkKey.Reserve(ContainerSlackSize);
}

void FHttpQueue::Reprioritize(FHttpRequest* Request, int32 NewPriority)
{
	TUniqueLock Lock(Mutex);

#if UE_ALLOW_HTTP_PAUSE
	if (IsPaused(Request))
	{
		Request->Options.SetPriority(NewPriority);
		return;
	}
#endif //UE_ALLOW_HTTP_PAUSE

	const bool bGenericHttpRequest = Request->ChunkHash.IsZero();
	if (bGenericHttpRequest)
	{
		if (int32 Index = Heap.Find(Request); Index != INDEX_NONE)
		{
			Request->Options.SetPriority(NewPriority);
			bReprioritize = true;
		}
		return;
	}

	FHttpRequestList* List = ByChunkKey.Find(Request->ChunkHash);
	if (List == nullptr)
	{
		ensure(Heap.Find(Request) == INDEX_NONE);
		return;
	}

	for (FHttpRequest& Existing : *List)
	{
		if (NewPriority > Existing.Options.GetPriority())
		{
			Existing.Options.SetPriority(NewPriority);
			bReprioritize = true;
		}
	}
}

bool FHttpQueue::Cancel(FHttpRequest* Request)
{
	TUniqueLock Lock(Mutex);

#if UE_ALLOW_HTTP_PAUSE
	if (IsPaused(Request))
	{
		if(PausedRequests.Remove(Request) != 0)
		{
			// Cancel stats assume that the request was enqueued, so we have to add that back
			FOnDemandIoBackendStats::Get()->OnHttpEnqueued(Request->GetHttpRequestType(), 0);
			FOnDemandIoBackendStats::Get()->OnHttpDequeued(Request->GetHttpRequestType(), 0, /*bWasCanceled=*/true);
			FOnDemandIoBackendStats::Get()->OnHttpUnpaused(Request->GetHttpRequestType());
			return true;
		}
	}
#endif //UE_ALLOW_HTTP_PAUSE

	const bool bGenericHttpRequest = Request->ChunkHash.IsZero();
	if (bGenericHttpRequest)
	{
		ensure(Request->Next == nullptr);
		if (int32 Index = Heap.Find(Request); Index != INDEX_NONE)
		{
			Heap.HeapRemoveAt(Index, HttpRequestSortPredicate);
			const bool bWasCanceled = true;
			FOnDemandIoBackendStats::Get()->OnHttpDequeued(Request->GetHttpRequestType(), Request->GetRangeSize(), bWasCanceled);
			return true;
		}
		return false;
	}

	// Note: Even if a batch exists for the given chunk hash, it doesn't necessarily mean this request is included in that batch.
	FHttpRequestList* List = ByChunkKey.Find(Request->ChunkHash);
	if ((List == nullptr) || (List->Remove(Request) == false))
	{
		ensure(Heap.Find(Request) == INDEX_NONE);
		return false;
	}

	// If the request was the first for this chunk we push any of the remaining requests to the heap
	if (int32 Index = Heap.Find(Request); Index != INDEX_NONE)
	{
		Heap.HeapRemoveAt(Index, HttpRequestSortPredicate);

		if (!List->IsEmpty())
		{
			FHttpRequest* Head = List->PeekHead();
			ensure(Heap.Find(Head) == INDEX_NONE);
			Heap.HeapPush(Head, HttpRequestSortPredicate);
		}
	}

	if (List->IsEmpty())
	{
		ByChunkKey.Remove(Request->ChunkHash);
	}

	const bool bWasCanceled = true;
	FOnDemandIoBackendStats::Get()->OnHttpDequeued(Request->GetHttpRequestType(), Request->GetRangeSize(), bWasCanceled);

	return true;
}

void FHttpQueue::Enqueue(FHttpRequest* Request)
{
	TUniqueLock Lock(Mutex);

#if UE_ALLOW_HTTP_PAUSE
	if (IsPaused(Request))
	{
		PausedRequests.Add(Request);
		FOnDemandIoBackendStats::Get()->OnHttpPaused(Request->GetHttpRequestType());

		return;
	}
#endif //UE_ALLOW_HTTP_PAUSE

	EnqueueInternal(Request);
}

void FHttpQueue::EnqueueInternal(FHttpRequest* Request)
{
	check(Request->Next == nullptr);

	Request->Flags.Remove(EHttpRequestFlags::CacheInflight);
	FOnDemandIoBackendStats::Get()->OnHttpEnqueued(Request->GetHttpRequestType(), Request->GetRangeSize());

	const bool bGenericHttpRequest = Request->ChunkHash.IsZero();
	if (bGenericHttpRequest)
	{
		Heap.HeapPush(Request, HttpRequestSortPredicate);
		Request->Flags.Add(EHttpRequestFlags::HttpQueued);
		return;
	}

	if (FHttpRequestList* List = ByChunkKey.Find(Request->ChunkHash))
	{
		ensure(List->IsEmpty() == false);
		FHttpRequest& Head = *List->PeekHead();
		if (Request->Options.GetPriority() > Head.Options.GetPriority())
		{
			Head.Options.SetPriority(Request->Options.GetPriority());
			bReprioritize = true;
		}
		else
		{
			Request->Options.SetPriority(Head.Options.GetPriority());
		}

		List->AddTail(Request);
	}
	else
	{
		Heap.HeapPush(Request, HttpRequestSortPredicate);
		ByChunkKey.Add(Request->ChunkHash, FHttpRequestList(Request));
	}

	Request->Flags.Add(EHttpRequestFlags::HttpQueued);
}

FHttpRequest* FHttpQueue::Dequeue()
{
	TUniqueLock Lock(Mutex);

	if (Heap.IsEmpty())
	{
		ensure(ByChunkKey.IsEmpty());
		return nullptr;
	}

	if (bReprioritize)
	{
		bReprioritize = false;
		Heap.Heapify(HttpRequestSortPredicate);
	}

	FHttpRequest* Next = nullptr;
	Heap.HeapPop(Next, HttpRequestSortPredicate, EAllowShrinking::No);
	if (Heap.IsEmpty())
	{
		Heap.Empty(ContainerSlackSize);
	}

	if (Next == nullptr)
	{
		return Next;
	}

	if (FHttpRequestList* List = ByChunkKey.Find(Next->ChunkHash))
	{
		// The request on the heap is expected to be the first in the list
		ensure(Next == List->PeekHead());

		if (GHttpIoDispatcherMaxRangeKiB < 0)
		{
			// Fetch the entire batch
			Next = List->PeekHead();
			ByChunkKey.Remove(Next->ChunkHash);
		}
		else
		{
			FHttpRequestList	Requests(MoveTemp(*List));
			*List								= FHttpRequestList();
			const uint32		MaxRangeSize	= uint32(GHttpIoDispatcherMaxRangeKiB) << 10;
			const uint32		MaxGapSize		= GHttpIoDispatcherMaxMergeGapKiB > 0 ? uint32(GHttpIoDispatcherMaxMergeGapKiB) << 10 : MAX_uint32;
			FHttpRequestList	Batch			= CreateBatch(*Next, MaxRangeSize, MaxGapSize, MoveTemp(Requests), *List);

			if (List->IsEmpty())
			{
				ByChunkKey.Remove(Next->ChunkHash);
			}
			else
			{
				// Push the first request in the remaining list to the heap when there's more requests for the same chunk hash
				FHttpRequest* Head = List->PeekHead();
				ensure(Heap.Find(Head) == INDEX_NONE);
				Heap.HeapPush(Head, HttpRequestSortPredicate);
			}

			Next = Batch.PeekHead();
		}
	}

	if (ByChunkKey.IsEmpty())
	{
		ByChunkKey.Empty(ContainerSlackSize);
	}

	for (FHttpRequest* It = Next; It != nullptr; It = It->Next)
	{
		It->Flags.Add(EHttpRequestFlags::HttpInflight);
		It->Flags.Remove(EHttpRequestFlags::HttpQueued);
		bool bWasCanceled = false;
		FOnDemandIoBackendStats::Get()->OnHttpDequeued(It->GetHttpRequestType(), It->GetRangeSize(), bWasCanceled);
	}

	return Next;
}

#if UE_ALLOW_HTTP_PAUSE

void FHttpQueue::OnTogglePause(bool bPause, EHttpRequestTypeFilter Filter)
{
	UE::TUniqueLock _(Mutex);

	if (bPause)
	{
		EnumAddFlags(PauseFilter, Filter);

		// Remove any queued requests which now be paused
		TArray<FHttpRequest*> NewHeap;
		for (FHttpRequest* HeadRequest :Heap)
		{
			if (FHttpRequestList* List = ByChunkKey.Find(HeadRequest->ChunkHash))
			{
				bool bRemovedHeadRequest = false;

				// Strip paused requests from the list
				FHttpRequest* Iterator = List->PeekHead();
				while (Iterator != nullptr)
				{
					FHttpRequest* ListRequest = Iterator;
					Iterator = ListRequest->Next;

					if (IsPaused(ListRequest))
					{
						if (ListRequest == HeadRequest)
						{
							bRemovedHeadRequest = true;
						}

						List->Remove(ListRequest);

						check(ListRequest->Next == nullptr);
						PausedRequests.Add(ListRequest);
						FOnDemandIoBackendStats::Get()->OnHttpPaused(ListRequest->GetHttpRequestType());
					}
				}

				// If the list is now empty we can remove the entire request from Heap
				if (List->IsEmpty())
				{
					FOnDemandIoBackendStats::Get()->OnHttpOnRemovedPending(HeadRequest->GetHttpRequestType());
					ByChunkKey.Remove(HeadRequest->ChunkHash);
				}
				// If we removed the head request we need to replace it in Heap with a non paused request
				else if (bRemovedHeadRequest)
				{
					FOnDemandIoBackendStats::Get()->OnHttpOnRemovedPending(HeadRequest->GetHttpRequestType());

					FHttpRequest* NewHeadRequest = List->PeekHead();
					NewHeap.HeapPush(NewHeadRequest, HttpRequestSortPredicate);
					FOnDemandIoBackendStats::Get()->OnHttpEnqueued(NewHeadRequest->GetHttpRequestType(), NewHeadRequest->GetRangeSize());
				}
			}
			else if (IsPaused(HeadRequest))
			{
				check(HeadRequest->Next == nullptr);
				PausedRequests.Add(HeadRequest);

				FOnDemandIoBackendStats::Get()->OnHttpOnRemovedPending(HeadRequest->GetHttpRequestType());
				FOnDemandIoBackendStats::Get()->OnHttpPaused(HeadRequest->GetHttpRequestType());
			}
			else
			{
				NewHeap.HeapPush(HeadRequest, HttpRequestSortPredicate);
			}
		}

		Heap = MoveTemp(NewHeap);
	}
	else
	{
		EnumRemoveFlags(PauseFilter, Filter);

		for (int32 Index = 0; Index < PausedRequests.Num(); ++Index)
		{
			FHttpRequest* Request = PausedRequests[Index];

			if (IsPaused(Request) == false)
			{
				// Add the newly unpaused request back to the queue
				EnqueueInternal(Request);
				FOnDemandIoBackendStats::Get()->OnHttpUnpaused(Request->GetHttpRequestType());

				PausedRequests.RemoveAt(Index);
				--Index;
			}
		}
	}
}

bool FHttpQueue::IsPaused(FHttpRequest* Request) const
{
	check(Request != nullptr);

	switch (Request->GetHttpRequestType())
	{
		case EHttpRequestType::Installed:
		{
			return EnumHasAnyFlags(PauseFilter, EHttpRequestTypeFilter::Installed);
		}
		case EHttpRequestType::Streaming:
		{
			return EnumHasAnyFlags(PauseFilter, EHttpRequestTypeFilter::Streaming);
		}
		default:
		{
			checkNoEntry();
			return false;
		}
	}
}

#endif // UE_ALLOW_HTTP_PAUSE

////////////////////////////////////////////////////////////////////////////////
class FHttpDispatcher final
	: public FRunnable
	, public FSingleThreadRunnable
	, public IOnDemandHttpIoDispatcher
{
public:
									FHttpDispatcher(TUniquePtr<IIasCache>&& Cache);
									~FHttpDispatcher();

private:
	// FRunnable
	virtual bool					Init() override;
	virtual uint32					Run() override;
	virtual void					Stop() override;
	virtual void					Exit() override;
	virtual FSingleThreadRunnable*	GetSingleThreadInterface() override
	{
		return this;
	}

	//FSingleThreadRunnable
	virtual void					Tick() override;
	// IHttpIoDispatcher
	virtual void					Shutdown() override;
	virtual void					GetHttpStats(FOnDemandHttpStats& Out) const override;
	virtual FIoStatus				RegisterHostGroup(const FName& HostGroupName, TConstArrayView<FAnsiString> HostNames, FAnsiStringView TestUrl) override;
	virtual bool					IsHostGroupRegistered(const FName& HostGroup) override;
	virtual bool					IsHostGroupOk(const FName& HostGroup) override;
	virtual FHostGroupRegistered&	OnHostGroupRegistered() override { return HostGroupRegisterd; }
	virtual FIoHttpRequest			CreateRequest(
										IIoHttpRequestHandle*& First,
										IIoHttpRequestHandle*& Last,
										const FName& HostGroupName,
										const FIoRelativeUrl& RelativeUrl,
										const FIoHttpOptions& Options,
										FIoHttpHeaders&& Headers,
										FIoHttpRequestCompleted&& OnCompleted,
										const FIoHash* ChunkHash) override;
	virtual void					IssueRequest(IIoHttpRequestHandle& Handle) override;
	virtual void					CancelRequest(FIoHttpRequest Handle) override;
	virtual void					UpdateRequestPriority(FIoHttpRequest Handle, int32 NewPriority) override;

	virtual FIoStatus				CacheResponse(const FIoHttpResponse& Response) override;
	virtual FIoStatus				EvictFromCache(const FIoHttpResponse& Response) override;

	void							UpdateThreadPriorityIfNeeded();

	bool							TryCreateHttpClient();
	void							RecreateHttpClientIfNeeded();

	bool							TryReadFromCache(FHttpRequest* Request);
	
	bool							ProcessHttpRequests();
	void							CancelRemainingHttpRequests();
	FHttpRequest*					DequeueHttpRequests();
	void							CompleteHttpRequest(FHttpRequest& Request, FMultiEndpointHttpClientResponse&& HttpResponse);
	void							CompleteRequest(FHttpRequest& Request, TArray<FAnsiString>&& Headers, const FResponseBody& Data, uint32 StatusCode, bool bCached);
	void							CompleteRequest(FHttpRequest& Request, const FIoBuffer& Body, uint32 StatusCode, bool bCached)
									{
										CompleteRequest(Request, TArray<FAnsiString>(), FResponseBody(Body), StatusCode, bCached);
									}

	void							ApplyCancellationToken(FHttpRequest* RequestChain, FMultiEndpointHttpClient::FHttpTicketId TicketId);
	void							CompleteCancellationToken(FHttpRequest* RequestChain);

	void							UpdateHostGroup(FIASHostGroup& Hostgroup, const FMultiEndpointHttpClientResponse& Response) const;

	void							SubscribeToBandwidthManager();
	bool							ShouldSimulateError(EHttpRequestType RequestType);

#if UE_ALLOW_HTTP_PAUSE
	void							OnPauseCommand(bool bShouldPause, const TArray<FString>& Args, FOutputDevice& Ar);
#endif //UE_ALLOW_HTTP_PAUSE

	TUniquePtr<IIasCache>					Cache;
	TUniquePtr<FMultiEndpointHttpClient>	HttpClient;
	TUniquePtr<FRunnableThread>				Thread;
	EThreadPriority							ThreadPriority = EThreadPriority::TPri_Normal;
	FHttpQueue								HttpQueue;
	FHostGroupRegistered					HostGroupRegisterd;
	FEventRef								WakeUp;
	/** Keeps assignment of the CancellationToken safe between ::CancelRequest and ::ProcessHttpRequests */
	FMutex									CancellationMutex;
	FDelegateHandle							OnRecreateHttpClientHandle;
#if !NO_CVARS
	FDelegateHandle							OnJournalMagicChangedHandle;
#endif // !NO_CVARS
	std::atomic_bool						bRecreateHttpClient = false;
	std::atomic_bool						bStopRequested = false;
	std::atomic_int32_t						InflightRequestCount{0};
	
	UE::BandwidthManager::FServiceHandleID	BandwidthManagerServiceHandle = UE_INVALID_BANDWIDTH_SERVICE_HANDLE;
	FDelegateHandle							BandwidthManagerDelayedSetHandle;

#if UE_ALLOW_HTTP_PAUSE
	FConsoleCommandPtr						PauseCommand;
	FConsoleCommandPtr						UnpauseCommand;
#endif // UE_ALLOW_HTTP_PAUSE

#if UE_ALLOW_DROP_CACHE
	FConsoleCommandPtr						DropCacheCommand;
#endif // UE_ALLOW_DROP_CACHE

#if UE_ALLOW_ERROR_SIMULATION
	FRequestErrorSimulator					RequestErrorSimulator;
#endif // UE_ALLOW_ERROR_SIMULATION
};

////////////////////////////////////////////////////////////////////////////////
FHttpDispatcher::FHttpDispatcher(TUniquePtr<IIasCache>&& InCache)
	: Cache(MoveTemp(InCache))
{
#if !NO_CVARS
	if (Cache)
	{
		OnJournalMagicChangedHandle = CVar_IasHttpCacheJournalMagic->OnChangedDelegate().AddLambda([this](IConsoleVariable* Cvar)
			{
				Cache->UpdateMagicValue(GIasHttpCacheJournalMagic);
			});
	}
#endif // !NO_CVARS

#if UE_ALLOW_HTTP_PAUSE
	PauseCommand = FConsoleCommandPtr(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("iax.HttpPause"),
		TEXT("Pause all http requests. Passing in 'IAD' or 'IAS' as an arg to only pause requests of that type"),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([this](const TArray<FString>& Args, FOutputDevice& Ar) -> void
			{
				OnPauseCommand(/*bShouldPause*/ true, Args, Ar);
			}),
		ECVF_Default));

	UnpauseCommand = FConsoleCommandPtr(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("iax.HttpUnpause"),
		TEXT("Unpause all http requests. Passing in 'IAD' or 'IAS' as an arg to only unpause requests of that type"),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([this](const TArray<FString>& Args, FOutputDevice& Ar) -> void
			{
				OnPauseCommand(/*bShouldPause*/ false, Args, Ar);
			}),
		ECVF_Default));
#endif // UE_ALLOW_HTTP_PAUSE

#if UE_ALLOW_DROP_CACHE
	DropCacheCommand = FConsoleCommandPtr(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("ias.DropCache"),
		TEXT("Resets the IAS cache and deletes the data from disk."),
		FConsoleCommandDelegate::CreateLambda([this]() -> void
			{
				if (Cache)
				{
					Cache->Drop();
				}
			}),
		ECVF_Default));
#endif // UE_ALLOW_DROP_CACHE

	const int32 ThreadPriorityIndex			= FMath::Clamp(GOnDemandBackendThreadPriorityIndex, 0, (int32)UE_ARRAY_COUNT(GThreadPriorities) - 1);
	EThreadPriority DesiredThreadPriority	= (EThreadPriority)GThreadPriorities[ThreadPriorityIndex];
	ThreadPriority							= DesiredThreadPriority;

	const uint32 StackSize					= 0; // Use default stack size
	const uint64 ThreadAffinityMask			= FGenericPlatformAffinity::GetNoAffinityMask();
	const EThreadCreateFlags CreateFlags	= EThreadCreateFlags::None;
	const bool bAllowPreFork				= FParse::Param(FCommandLine::IsInitialized() ? FCommandLine::Get() : TEXT(""), TEXT("-Ias.EnableHttpThreadPreFork"));

	Thread.Reset(FForkProcessHelper::CreateForkableThread(this, TEXT("IoService.Http"), StackSize, ThreadPriority, ThreadAffinityMask, CreateFlags, bAllowPreFork));
	TRACE_INIT(this);
};

FHttpDispatcher::~FHttpDispatcher()
{
	if (OnRecreateHttpClientHandle.IsValid())
	{
		OnRecreateHttpClient.Remove(OnRecreateHttpClientHandle);
		OnRecreateHttpClientHandle.Reset();
	}

#if !NO_CVARS
	if (OnJournalMagicChangedHandle.IsValid())
	{
		CVar_IasHttpCacheJournalMagic->OnChangedDelegate().Remove(OnJournalMagicChangedHandle);
		OnJournalMagicChangedHandle.Reset();
	}
#endif // !NO_CVARS

	FDownlinkBandwidthManager& DBManager = FDownlinkBandwidthManager::Get();
	DBManager.UnregisterMonitoredService(BandwidthManagerServiceHandle);
	DBManager.UnbindToCVarSubscription(BandwidthManagerDelayedSetHandle);

	Thread.Reset();
}

bool FHttpDispatcher::Init()
{
	OnRecreateHttpClientHandle = OnRecreateHttpClient.AddLambda([this]()
		{
			bRecreateHttpClient = true;
			WakeUp->Trigger();
		});

	// Sets an initial value in case of delay for bandwidth manager registration
	DLBWHttpRateLimitKiBPerSecond = static_cast<int32>(GIasHttpRateLimitKiBPerSecond);
	return TryCreateHttpClient();
}

void FHttpDispatcher::SubscribeToBandwidthManager()
{
	FDownlinkBandwidthManager& DBManager = FDownlinkBandwidthManager::Get();
	UE::BandwidthManager::FManagedServiceConfigurableData BandwidthConfig;

	BandwidthConfig.IdleGameplayPriority = UE::BandwidthManager::EBandwidthPriority::HighPriority;
	BandwidthConfig.InGameplayPriority = UE::BandwidthManager::EBandwidthPriority::LowPriority;

	BandwidthManagerServiceHandle = DBManager.RegisterMonitoredService(TEXT("HttpDispatcher"), BandwidthConfig, true,
		[this](int32 AllottedBandwidthKbps)
		{
			// 1 Kilobyte is equal to (1000 / 1024) kibibytes
			constexpr float KilobyteToKibibyte = (1000.0f / 1024.0f);
			DLBWHttpRateLimitKiBPerSecond = static_cast<int32>((float)AllottedBandwidthKbps * KilobyteToKibibyte);
		});
}

uint32 FHttpDispatcher::Run()
{
	check(HttpClient.IsValid());

	if (GIaxHttpForceInitialConnectionCheck)
	{
		bool bFirstAttempt = false;
		if (FIoStatus CertStatus = LoadDefaultHttpCertificates(bFirstAttempt); !CertStatus.IsOk())
		{
			UE_LOGF(LogHttpIoDispatcher, Error, "Failed to load certificates, reason '%ls'", *CertStatus.ToString());
		}
	
		for (;;)
		{
			if (FHostGroupManager::Get().ResolveInitialConnection(GIasHttpTimeOutMs, bStopRequested))
			{
				break;
			}

			if (bStopRequested.load(std::memory_order_relaxed))
			{
				break;
			}

			WakeUp->Wait(GIasHttpHealthCheckWaitTime);
		}
	}

	for (;;)
	{
		if (FDownlinkBandwidthManager::HasLocalUserPassedRolloutCheck() && BandwidthManagerServiceHandle == UE_INVALID_BANDWIDTH_SERVICE_HANDLE)
		{
			SubscribeToBandwidthManager();
		}
		else if (!BandwidthManagerDelayedSetHandle.IsValid() && BandwidthManagerServiceHandle == UE_INVALID_BANDWIDTH_SERVICE_HANDLE)
		{
			FDownlinkBandwidthManager& DBManager = FDownlinkBandwidthManager::Get();
			BandwidthManagerDelayedSetHandle = DBManager.BindToCVarSubscription([this]()
				{
					if (GUseDelayedDownlinkBandwidthManagerSubscription && FDownlinkBandwidthManager::HasLocalUserPassedRolloutCheck() && BandwidthManagerServiceHandle == UE_INVALID_BANDWIDTH_SERVICE_HANDLE)
					{
						SubscribeToBandwidthManager();
					}
				});
		}
		else if (FDownlinkBandwidthManager::HasLocalUserPassedRolloutCheck())
		{
			FDownlinkBandwidthManager& DBManager = FDownlinkBandwidthManager::Get();
			DBManager.SetActivationForService(BandwidthManagerServiceHandle, true);
		}

		UpdateThreadPriorityIfNeeded();

		Tick();

		if (bStopRequested.load(std::memory_order_relaxed))
		{
			break;
		}

		WakeUp->Wait(FHostGroupManager::Get().GetNumDisconnctedHosts() > 0 ? GIasHttpHealthCheckWaitTime : MAX_uint32);
	}

	CancelRemainingHttpRequests();

	{
		const float SleepTimeSeconds	= 0.5f;
		const double WaitTimeoutSeconds	= GHttpIoDispatcherShutdownTimeoutSeconds;
		const double StartTimeSeconds	= FPlatformTime::Seconds();
		int32 Remaining					= InflightRequestCount.load(std::memory_order_seq_cst);

		UE_CLOGF(Remaining > 0, LogHttpIoDispatcher, Log, "Waiting on %d request(s) before shutting down...", Remaining);

		while (Remaining > 0)
		{
			if ((FPlatformTime::Seconds() - StartTimeSeconds) > WaitTimeoutSeconds)
			{
				UE_LOGF(LogHttpIoDispatcher, Warning, "Shutting down with %d inflight request(s) after waiting %.2lf second(s)", Remaining, WaitTimeoutSeconds);
				break;
			}

			FPlatformProcess::SleepNoStats(SleepTimeSeconds);
			Remaining = InflightRequestCount.load(std::memory_order_seq_cst);
		}
	}

	return 0;
}

void FHttpDispatcher::Stop()
{
	bool bExpected = false;
	if (bStopRequested.compare_exchange_strong(bExpected, true))
	{
		WakeUp->Trigger();
	}

	if (FDownlinkBandwidthManager::HasLocalUserPassedRolloutCheck())
	{
		FDownlinkBandwidthManager& DBManager = FDownlinkBandwidthManager::Get();
		DBManager.SetActivationForService(BandwidthManagerServiceHandle, false);
	}
}

void FHttpDispatcher::Exit()
{
	FDownlinkBandwidthManager& DBManager = FDownlinkBandwidthManager::Get();
	DBManager.UnregisterMonitoredService(BandwidthManagerServiceHandle);
	DBManager.UnbindToCVarSubscription(BandwidthManagerDelayedSetHandle);
}

void FHttpDispatcher::Tick()
{
	FHostGroupManager::Get().Tick(GIasHttpTimeOutMs, bStopRequested);

	// TODO: It would be better to only update connections as they need it, consider doing this on
	// hostgroup connect/disconnect events.
	HttpClient->UpdateConnections();

	ProcessHttpRequests();
}

void FHttpDispatcher::Shutdown()
{
	bool bExpected = false;
	if (bStopRequested.compare_exchange_strong(bExpected, true))
	{
		WakeUp->Trigger();
	}
}

void FHttpDispatcher::GetHttpStats(FOnDemandHttpStats& Out) const
{
	return HttpClient->GetHttpStats(Out);
}

FIoStatus FHttpDispatcher::RegisterHostGroup(const FName& HostGroupName, TConstArrayView<FAnsiString> HostNames, FAnsiStringView TestUrl)
{
	FIASHostGroup Existing = FHostGroupManager::Get().Find(HostGroupName);
	if (Existing.GetName() == HostGroupName)
	{
		return FIoStatus::Ok;
	}

	TIoStatusOr<FIASHostGroup> RegisterResult = FHostGroupManager::Get().Register(HostGroupName, TestUrl);
	if (RegisterResult.IsOk() == false)
	{
		return RegisterResult.Status();
	}

	FIASHostGroup NewGroup = RegisterResult.ConsumeValueOrDie();

	if (FIoStatus Status = NewGroup.Resolve(HostNames); !Status.IsOk())
	{
		UE_LOGF(LogHttpIoDispatcher, Warning, "Failed to create host group '%ls'", *HostGroupName.ToString());
		return Status;
	}

	UE_LOGF(LogHttpIoDispatcher, Log, "Registered new host group '%ls'", *HostGroupName.ToString());
	for (const FAnsiString& HostName : NewGroup.GetHostUrls())
	{
		UE_LOGF(LogHttpIoDispatcher, Log, "\t* %ls", *FString(HostName));
	}

	if (HostGroupRegisterd.IsBound())
	{
		HostGroupRegisterd.Broadcast(HostGroupName);
	}

	// Trigger a wakeup if we're waiting on FHostGroupManager::ResolveInitialConnection in ::Run()
	WakeUp->Trigger();

	return FIoStatus::Ok;
}

bool FHttpDispatcher::IsHostGroupRegistered(const FName& HostGroup)
{
	FIASHostGroup Existing = FHostGroupManager::Get().Find(HostGroup);
	return Existing.GetName() == HostGroup;
}

bool FHttpDispatcher::IsHostGroupOk(const FName& HostGroup)
{
	FIASHostGroup Existing = FHostGroupManager::Get().Find(HostGroup);
	return Existing.IsConnected();
}

FIoHttpRequest FHttpDispatcher::CreateRequest(
	IIoHttpRequestHandle*& First,
	IIoHttpRequestHandle*& Last,
	const FName& HostGroupName,
	const FIoRelativeUrl& RelativeUrl,
	const FIoHttpOptions& Options,
	FIoHttpHeaders&& Headers,
	FIoHttpRequestCompleted&& OnCompleted,
	const FIoHash* ChunkHash)
	
{
	FHttpRequest* Request	= new FHttpRequest();
	Request->OnCompleted	= MoveTemp(OnCompleted);
	Request->Headers		= MoveTemp(Headers);
	Request->Options		= Options;
	Request->RelativeUrl	= RelativeUrl;
	Request->HostGroupName	= HostGroupName;
	check(Request->Next == nullptr);

	if (ChunkHash != nullptr)
	{
		Request->ChunkHash = *ChunkHash;
		if (Options.GetCacheKey().IsZero())
		{
			Request->Options.SetCacheKey(GetCacheKey(*ChunkHash, Options.GetRange()));
		}
	}
	else
	{
		Request->ChunkHash = FIoHash::Zero;
		if (Options.GetCacheKey().IsZero())
		{
			Request->Options.SetCacheKey(GetCacheKey(HostGroupName, Request->RelativeUrl, Request->Headers, Request->Options));
		}
	}

	if (Last == nullptr)
	{
		ensure(First == nullptr);
		First = Request;
	}
	else
	{
		static_cast<FHttpRequest*>(Last)->Next = Request;
	}
	Last = Request;

	return FIoHttpRequest(Request);
}

void FHttpDispatcher::IssueRequest(IIoHttpRequestHandle& Handle)
{
	bool bWakeUp				= false;
	FHttpRequest* NextRequest	= static_cast<FHttpRequest*>(&Handle);

	while (NextRequest != nullptr)
	{
		FHttpRequest* Request	= NextRequest;
		NextRequest				= Request->Next;
		Request->Next			= nullptr;

		// The dispatcher holds a reference to the request until completed
		Request->AddRef();
		Request->Flags.Add(EHttpRequestFlags::Issued);
		if (TryReadFromCache(Request) == false)
		{
			HttpQueue.Enqueue(Request);
			bWakeUp = true;
		}
	}

	if (bWakeUp)
	{
		WakeUp->Trigger();
	}
}

void FHttpDispatcher::CancelRequest(FIoHttpRequest Handle)
{
	FHttpRequest* Request = static_cast<FHttpRequest*>(Handle.GetHandle());
	if (Request == nullptr)
	{
		return;
	}

	const EHttpRequestFlags CurrentFlags = Request->Flags.Get();
	if (EnumHasAnyFlags(CurrentFlags, EHttpRequestFlags::CancelRequested | EHttpRequestFlags::Completing | EHttpRequestFlags::Completed))
	{
		return;
	}
	Request->Flags.Add(EHttpRequestFlags::CancelRequested);

	//UE_LOGF(LogHttpIoDispatcher, Verbose, "Cancelling request, SeqNo=%u", Request->SeqNo);

	if (EnumHasAnyFlags(CurrentFlags, EHttpRequestFlags::CacheInflight))
	{
		Cache->Cancel(Request->Buffer);
	}

	if (HttpQueue.Cancel(Request))
	{
		InflightRequestCount.fetch_add(1, std::memory_order_relaxed);
		UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, Request]
		{
			CompleteRequest(*Request, Request->Buffer, 0, false);
		}, UE::Tasks::ETaskPriority::BackgroundLow);
	}

	if (GIaxHttpEnableInflightCancellation)
	{
		TUniqueLock _(CancellationMutex);
		Request->CancellationToken.Reset();
	}
}

void FHttpDispatcher::UpdateRequestPriority(FIoHttpRequest Handle, int32 NewPriority)
{
	FHttpRequest* Request = static_cast<FHttpRequest*>(Handle.GetHandle());
	if (Request == nullptr)
	{
		return;
	}

	UE_LOGF(LogHttpIoDispatcher, Verbose, "Updating request priority, SeqNo=%u, Priority=%d, NewPriority=%d",
		Request->SeqNo, Request->Options.GetPriority(), NewPriority);
	HttpQueue.Reprioritize(Request, NewPriority);
}

FIoStatus FHttpDispatcher::CacheResponse(const FIoHttpResponse& Response)
{
	if (Response.IsCached())
	{
		return EIoErrorCode::Ok;
	}

	// If the cache is disabled we consider the caching to be a success, otherwise the calling code will
	// have to special case this.
	if (!GHttpCacheEnabled || !Cache.IsValid())
	{
		return EIoErrorCode::Ok;
	}

	if (Response.GetErrorCode() != EIoErrorCode::Ok || Response.GetCacheKey().IsZero() || Response.GetBody().GetSize() == 0)
	{
		return EIoErrorCode::InvalidParameter;
	}

	FIoBuffer Copy(Response.GetBody());
	return Cache->Put(Response.GetCacheKey(), Copy);
}

void FHttpDispatcher::UpdateThreadPriorityIfNeeded()
{
	// TODO - We should wake up the thread when 'ias.onDemandBackendThreadPriority' is updated
	int32 ThreadPriorityIndex = FMath::Clamp(GOnDemandBackendThreadPriorityIndex, 0, (int32)UE_ARRAY_COUNT(GThreadPriorities) - 1);
	EThreadPriority DesiredThreadPriority = (EThreadPriority)GThreadPriorities[ThreadPriorityIndex];
	if (DesiredThreadPriority != ThreadPriority)
	{
		UE_LOGFMT(LogHttpIoDispatcher, Log, "Updated IoService.Http thread priority to '{0}'", LexToString(DesiredThreadPriority));

		FPlatformProcess::SetThreadPriority(DesiredThreadPriority);
		ThreadPriority = DesiredThreadPriority;
	}
}

bool FHttpDispatcher::TryCreateHttpClient()
{
	// Make sure that 'ias.HttpConnectionCount' is within a range that the client will accept
	const int32 MaxNumberOfConnections = FMath::Clamp(GIasHttpConnectionCount, 1, (int32)MAX_uint16);
	UE_CLOGF(MaxNumberOfConnections != GIasHttpConnectionCount, LogHttpIoDispatcher, Error, "ias.HttpConnectionCount (%d) outside of valid range 1-%d", GIasHttpConnectionCount, MAX_uint16);

	HttpClient = FMultiEndpointHttpClient::Create(FMultiEndpointHttpClientConfig
		{
			.MaxConnectionCount = MaxNumberOfConnections,
			.ReceiveBufferSize = GIasHttpRecvBufKiB >= 0 ? GIasHttpRecvBufKiB << 10 : -1,
			.SendBufferSize = GIasHttpSendBufKiB >= 0 ? GIasHttpSendBufKiB << 10 : -1,
			.MaxRetryCount = GIasHttpRetryCount,
			.TimeoutMs = GIasHttpFailTimeOutMs,
			.Redirects = EHttpRedirects::Follow,
			.bAllowChunkedTransfer = true
		});

	return HttpClient.IsValid();
}

void FHttpDispatcher::RecreateHttpClientIfNeeded()
{
	if (bRecreateHttpClient.exchange(false))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FHttpDispatcher::RecreateHttpClientIfNeeded);

		// Keep the old client at first in case we fail to create the new one so that we can restore it
		// this way the game can keep on functioning correctly.
		TUniquePtr<FMultiEndpointHttpClient> OldHttpClient = MoveTemp(HttpClient);
		if (TryCreateHttpClient())
		{
			OldHttpClient.Reset();
			UE_LOGFMT(LogIas, Display, "FHttpDispatcher: Successfully created a new http client for use");
		}
		else
		{
			HttpClient = MoveTemp(OldHttpClient);
			UE_LOGFMT(LogIas, Warning, "FHttpDispatcher: Failed to create a new http client, the existing client will continue to be used");
		}
	}
}

FIoStatus FHttpDispatcher::EvictFromCache(const FIoHttpResponse& Response)
{
	// If the cache is disabled we consider the eviction to be a success, otherwise the calling code will
	// have to special case this.
	if (!GHttpCacheEnabled || !Cache.IsValid())
	{
		return EIoErrorCode::Ok;
	}

	if (Response.GetErrorCode() != EIoErrorCode::Ok || Response.GetCacheKey().IsZero())
	{
		return EIoErrorCode::InvalidParameter;
	}

	return Cache->Evict(Response.GetCacheKey());
}

bool FHttpDispatcher::TryReadFromCache(FHttpRequest* Request)
{
	if (!GHttpCacheEnabled || !Cache.IsValid())
	{
		return false;
	}

	if (EnumHasAnyFlags(Request->Options.GetFlags(), EIoHttpFlags::ReadCache) == false)
	{
		return false;
	}

	Request->Flags.Add(EHttpRequestFlags::CacheInflight);
	const EIoErrorCode CacheStatus = Cache->Get(Request->Options.GetCacheKey(), Request->Buffer);
	if (CacheStatus == EIoErrorCode::Ok)
	{
		InflightRequestCount.fetch_add(1, std::memory_order_relaxed);

		UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, Request]
		{
			CompleteRequest(*Request, Request->Buffer, 200, true);
		}, UE::Tasks::ETaskPriority::BackgroundLow);

		return true;
	}

	if (CacheStatus == EIoErrorCode::FileNotOpen)
	{
		InflightRequestCount.fetch_add(1, std::memory_order_relaxed);
		UE::Tasks::FTaskEvent OnReadyEvent(TEXT("HttpCacheReadComplete"));

		Launch(UE_SOURCE_LOCATION, [this, Request]
		{
			if (Request->CacheStatus == EIoErrorCode::Ok)
			{
				CompleteRequest(*Request, Request->Buffer, 200, true);
			}
			else if ((Request->CacheStatus == EIoErrorCode::Cancelled) || Request->Flags.HasAny(EHttpRequestFlags::CancelRequested) || bStopRequested.load(std::memory_order_relaxed))
			{
				CompleteRequest(*Request, Request->Buffer, 0, false);
			}
			else
			{
				if (Request->CacheStatus == EIoErrorCode::ReadError)
				{
					FOnDemandIoBackendStats::Get()->OnCacheError();
				}

				ensure(InflightRequestCount.fetch_sub(1, std::memory_order_relaxed) > 0);
				HttpQueue.Enqueue(Request);
				WakeUp->Trigger();
			}
		}, OnReadyEvent, UE::Tasks::ETaskPriority::BackgroundLow);

		Cache->Materialize(Request->Options.GetCacheKey(), Request->Buffer, Request->CacheStatus, MoveTemp(OnReadyEvent));

		return true;
	}

	return false;
}

bool FHttpDispatcher::ProcessHttpRequests()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FHttpDispatcher::ProcessHttpRequests);

	RecreateHttpClientIfNeeded();

	const int32 MaxConcurrentRequests = CalculateRequestQueueLengthMax();
	int32		NumConcurrentRequests = 0;

	FHttpRequest* NextHttpRequest = DequeueHttpRequests();
	if (NextHttpRequest == nullptr)
	{
		return false;
	}

	TAnsiStringBuilder<128> Url;
	TAnsiStringBuilder<41>	HashString;

	while (NextHttpRequest)
	{
		while (NextHttpRequest)
		{
			FHttpRequest* HttpRequest	= NextHttpRequest;
			NextHttpRequest				= nullptr;

			ensure(!HttpRequest->Flags.HasAny(EHttpRequestFlags::HttpQueued | EHttpRequestFlags::CacheInflight));
			FOnDemandIoBackendStats::Get()->OnHttpStarted(HttpRequest->GetHttpRequestType());

			FIASHostGroup HostGroup = FHostGroupManager::Get().Find(HttpRequest->HostGroupName);
			if (!HostGroup.IsConnected() || ShouldSimulateError(HttpRequest->GetHttpRequestType()))
			{
				// If the hostgroup is disconnected then we are canceling the request as we cannot process it. If the hostgroup is connected
				// then we are simulating an error.
				const bool bWasCanceled = !HostGroup.IsConnected();
				FOnDemandIoBackendStats::Get()->OnHttpCompleted(HttpRequest->GetHttpRequestType(), 0, 0, 0, 0, 0, bWasCanceled);
				CompleteRequest(*HttpRequest, HttpRequest->Buffer, 0, false);

				check(NextHttpRequest == nullptr);
				NextHttpRequest = DequeueHttpRequests();
				continue;
			}

			Url.Reset();
			HashString.Reset();

			TArray<FAnsiString> Headers;
			TArray<FIoOffsetAndLength> Ranges;

			const bool bGenericHttpRequest = HttpRequest->ChunkHash.IsZero();
			if (bGenericHttpRequest)
			{
				ensure(HttpRequest->Next == nullptr);
				Url << *HttpRequest->RelativeUrl;
				Headers = MoveTemp(HttpRequest->Headers).ToArray();
			}
			else
			{
				// I/O store chunk request 
				HashString << HttpRequest->ChunkHash;
				Url << *HttpRequest->RelativeUrl
					<< "/" << HashString.ToView().Left(2)
					<< "/" << HashString
					<< UE::IoStore::Serialization::FOnDemandFileExt::Partition; 
				Ranges = GetTotalRange(HttpRequest);
			}

#if UE_ALLOW_ERROR_SIMULATION
			if (RequestErrorSimulator.ShouldSimulateInvalidUrl(HttpRequest->GetHttpRequestType()))
			{
				Url << "-DebugInvalidUrl";
			}
#endif // UE_ALLOW_ERROR_SIMULATION

			NumConcurrentRequests++;
			EMultiEndpointRequestFlags Flags = EnumHasAnyFlags(HttpRequest->Options.GetFlags(), EIoHttpFlags::ResponseHeaders)
				? EMultiEndpointRequestFlags::ResponseHeaders
				: EMultiEndpointRequestFlags::None;

			// Note that the requests issued here must be completed before this method can return so the capturing of locals is safe.
			TRACE_REQUEST_STARTED(HttpRequest, Url.ToString());
			FMultiEndpointHttpClient::FHttpTicketId TicketId = HttpClient->Get(HostGroup.GetUnderlyingHostGroup(), Url, MoveTemp(Ranges), MoveTemp(Headers), Flags,
				[this, HttpRequest, &NumConcurrentRequests, HostGroup = MoveTemp(HostGroup)]
				(FMultiEndpointHttpClientResponse&& HttpResponse) mutable
				{
					TRACE_REQUEST_COMPLETED(HttpRequest, HttpResponse, HostGroup);
					FOnDemandIoBackendStats::Get()->OnHttpCompleted(
						HttpRequest->GetHttpRequestType(),
						HttpResponse.StatusCode,
						HttpResponse.Body.GetSize(),
						HttpResponse.Sample.GetTotalMs(),
						HttpResponse.RetryCount,
						HttpResponse.CDNCacheStatus,
						HttpResponse.IsCanceled());
					NumConcurrentRequests--;

					CompleteCancellationToken(HttpRequest);
					UpdateHostGroup(HostGroup, HttpResponse);

					UE::Tasks::Launch(
						UE_SOURCE_LOCATION,
						[this, HttpRequest, HttpResponse = MoveTemp(HttpResponse)]() mutable
						{
							CompleteHttpRequest(*HttpRequest, MoveTemp(HttpResponse));
						}, UE::Tasks::ETaskPriority::BackgroundLow);
				});

			ApplyCancellationToken(HttpRequest, TicketId);

			if (NumConcurrentRequests >= MaxConcurrentRequests)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FHttpDispatcher::TickHttpSaturated);

				while (NumConcurrentRequests >= MaxConcurrentRequests) //-V654
				{
					HttpClient->Tick(MAX_uint32, GetHttpRateLimitKiBPerSecond());
				}
			}

			check(NextHttpRequest == nullptr);
			NextHttpRequest = DequeueHttpRequests();
		} // Inner

		{
			check(NextHttpRequest == nullptr);

			// Keep processing pending connections until all requests are completed or a new one is issued
			TRACE_CPUPROFILER_EVENT_SCOPE(FHttpDispatcher::TickHttp);
			while (NextHttpRequest == nullptr && HttpClient->Tick(GIasHttpPollTimeoutMs, GetHttpRateLimitKiBPerSecond()))
			{
				NextHttpRequest = DequeueHttpRequests();
			}
		}
	} // Outer

	// All requests from HttpClient should be completed at this point and all callbacks processed!.
	check(NumConcurrentRequests == 0);

	return true;
}

void FHttpDispatcher::CancelRemainingHttpRequests()
{
	while (FHttpRequest* Requests = HttpQueue.Dequeue())
	{
		for (FHttpRequest* It = Requests; It != nullptr; It = It->Next)
		{
			It->Flags.Add(EHttpRequestFlags::CancelRequested);
		}

		InflightRequestCount.fetch_add(1, std::memory_order_relaxed);
		FIoBuffer Buffer;
		CompleteRequest(*Requests, Buffer, 0, false);
	}
}

FHttpRequest* FHttpDispatcher::DequeueHttpRequests()
{
	// Return nullptr to break the process loop if we need to re-create the HTTP client or we are shutting down
	if (bRecreateHttpClient.load(std::memory_order_relaxed) || bStopRequested.load(std::memory_order_relaxed))
	{
		return nullptr;
	}

	if (FHttpRequest* Requests = HttpQueue.Dequeue())
	{
		InflightRequestCount.fetch_add(1, std::memory_order_relaxed);
		return Requests;
	}

	return nullptr;
}

void FHttpDispatcher::CompleteHttpRequest(FHttpRequest& Request, FMultiEndpointHttpClientResponse&& HttpResponse)
{
	CompleteRequest(Request, MoveTemp(HttpResponse.Headers), HttpResponse.Body, HttpResponse.StatusCode, false);
}

void FHttpDispatcher::CompleteRequest(FHttpRequest& Request, TArray<FAnsiString>&& Headers, const FResponseBody& ResponseBody, const uint32 StatusCode, bool bCached)
{
#if UE_ALLOW_CACHE_POISONING
	if (GIasPoisonCache && bCached)
	{
		FIoBuffer Body = ResponseBody.GetBody();
		for (uint64 Index = 0; Index < Body.GetSize(); Index++)
		{
			Body.GetData()[Index] = 0x4d;
		}
	}
#endif // UE_ALLOW_CACHE_POISONING

	FHttpRequest* NextRequest = &Request;

	while (NextRequest != nullptr)
	{
		FHttpRequest* ToComplete	= NextRequest;
		NextRequest					= ToComplete->Next;
		ToComplete->Next			= nullptr;

		uint32 RequestStatusCode = StatusCode; // The status code may be modified per request, so take a copy.

		ensure(!ToComplete->Flags.HasAny(EHttpRequestFlags::Completing | EHttpRequestFlags::Completed));
		ToComplete->Flags.Add(EHttpRequestFlags::Completing);
		ToComplete->Flags.Remove(EHttpRequestFlags::CacheInflight | EHttpRequestFlags::HttpInflight);

		const FIoHttpRange& Range		= ToComplete->Options.GetRange();
		EIoErrorCode CompletionStatus	= IsHttpStatusOk(StatusCode) ? EIoErrorCode::Ok : EIoErrorCode::ReadError;
		
		FIoBuffer Body;

		if (ToComplete->Flags.HasAny(EHttpRequestFlags::CancelRequested))
		{
			CompletionStatus	= EIoErrorCode::Cancelled;
			RequestStatusCode = 0; // Mark as a none http response related error
		}
		else if (bCached)
		{
			Body = ResponseBody.GetData(FIoOffsetAndLength());
		}
		else
		{
			Body = ResponseBody.GetData(Range.ToOffsetAndLength());
		}

		const EIoHttpResponseFlags Flags	= bCached ? EIoHttpResponseFlags::Cached : EIoHttpResponseFlags::None;
		FIoHttpRequestCompleted OnCompleted = MoveTemp(ToComplete->OnCompleted);
		TArray<FAnsiString> HeadersCopy		= Headers; // TODO: Only used for generic HTTP request but still bad 

		OnCompleted(FIoHttpResponse(
			ToComplete->Options.GetCacheKey(),
			FIoHttpHeaders::Create(MoveTemp(HeadersCopy)),
			MoveTemp(Body),
			CompletionStatus,
			RequestStatusCode,
			Flags));

		ToComplete->Flags.Add(EHttpRequestFlags::Completed);
		ToComplete->CompletionStatus.store(CompletionStatus, std::memory_order_seq_cst);
		ToComplete->Release();
	}

	ensure(InflightRequestCount.fetch_sub(1, std::memory_order_relaxed) > 0);
}

#if UE_ALLOW_HTTP_PAUSE

void FHttpDispatcher::OnPauseCommand(bool bShouldPause, const TArray<FString>& Args, FOutputDevice& Ar)
{
	if (Args.Num() > 1)
	{
		UE_CVAR_ERROR_LOG(TEXT("Too many args for command, either 0 (all) or 1 (IAD/IAS) expected"));
		return;
	}

	EHttpRequestTypeFilter Filter = EHttpRequestTypeFilter::All;

	if (!Args.IsEmpty())
	{
		if (Args[0] == TEXT("IAD"))
		{
			Filter = EHttpRequestTypeFilter::Installed;
		}
		else if (Args[0] == TEXT("IAS"))
		{
			Filter = EHttpRequestTypeFilter::Streaming;
		}
		else
		{
			UE_CVAR_ERROR_LOG(TEXT("Invalid arg for command, expecting 'IAD' or 'IAS'"));
			return;
		}
	}

	HttpQueue.OnTogglePause(bShouldPause, Filter);

	if (!bShouldPause)
	{
		// Since we are un-pausing we might have moved previously paused requests back to the active queue
		// so we need the thread to start running again.
		WakeUp->Trigger();
	}
}

#endif // UE_ALLOW_HTTP_PAUSE

bool FHttpDispatcher::ShouldSimulateError(EHttpRequestType RequestType)
{
#if UE_ALLOW_ERROR_SIMULATION
	return RequestErrorSimulator.ShouldSimulateError(RequestType);
#else
	return false;
#endif
}
void FHttpDispatcher::ApplyCancellationToken(FHttpRequest* RequestChain, FMultiEndpointHttpClient::FHttpTicketId TicketId)
{
	TUniqueLock _(CancellationMutex);

	TSharedPtr<FCancellationToken> Token = MakeShared<FCancellationToken>(*HttpClient.Get(), TicketId);

	while (RequestChain != nullptr)
	{
		if (!RequestChain->Flags.HasAny(EHttpRequestFlags::CancelRequested))
		{
			RequestChain->CancellationToken = Token;
		}

		RequestChain = RequestChain->Next;
	}
}

void FHttpDispatcher::CompleteCancellationToken(FHttpRequest* RequestChain)
{
	TUniqueLock _(CancellationMutex);

	while (RequestChain != nullptr)
	{
		if (RequestChain->CancellationToken)
		{
			RequestChain->CancellationToken->OnRequestCompleted();
			// Every request in the chain shares the same token (unless null) so we can early out at this point
			return;
		}

		RequestChain = RequestChain->Next;
	}
}

void FHttpDispatcher::UpdateHostGroup(FIASHostGroup& Hostgroup, const FMultiEndpointHttpClientResponse& Response) const
{
	if (Response.IsOk())
	{
		Hostgroup.OnSuccessfulResponse();
	}
	else if (!Response.IsCanceled())
	{
		if (Hostgroup.OnFailedResponse())
		{
			FOnDemandIoBackendStats::Get()->OnHttpDisconnected(); // A disconnect was triggered
		}
	}
}

} // namespace UE::IoStore::HttpIoDispatcher

///////////////////////////////////////////////////////////////////////////////
TSharedPtr<IOnDemandHttpIoDispatcher> MakeOnDemanHttpIoDispatcher(TUniquePtr<class IIasCache>&& Cache)
{
	return MakeShared<HttpIoDispatcher::FHttpDispatcher>(MoveTemp(Cache));
}

} // UE::IoStore

#undef UE_CVAR_ERROR_LOG

#undef UE_ALLOW_DISABLE_HTTP_LIMITS
#undef UE_ALLOW_HTTP_PAUSE
#undef UE_ALLOW_CACHE_POISONING
#undef UE_ALLOW_ERROR_SIMULATION

#undef TRACE_INIT
#undef TRACE_REQUEST_STARTED
#undef TRACE_REQUEST_COMPLETED
