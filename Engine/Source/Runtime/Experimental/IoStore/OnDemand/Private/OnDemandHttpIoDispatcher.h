// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/HttpIoDispatcher.h"
#include "Templates/SharedPointer.h"

#define UE_API IOSTOREONDEMANDCORE_API

namespace UE::IoStore
{

/** Describes the system making the request */
enum class EHttpRequestType : uint8
{
	Streaming = 0,	/** IAS */
	Installed,		/** IAD */

	NUM_SOURCES
};

class IOnDemandHttpIoDispatcher
	: public IHttpIoDispatcher
{
public:
	virtual ~IOnDemandHttpIoDispatcher() = default;
};

TSharedPtr<IOnDemandHttpIoDispatcher> MakeOnDemanHttpIoDispatcher(TUniquePtr<class IIasCache>&& Cache);

} // namespace UE::IoStore

#undef UE_API
