// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnDemandHttpIoDispatcherBackend.h"

#include "Containers/AnsiString.h"
#include "HAL/PlatformTime.h"
#include "IO/HttpIoDispatcher.h"
#include "IO/IoAllocators.h"
#include "IO/IoHash.h"
#include "IO/IoOffsetLength.h"
#include "IO/IoStatus.h"
#include "IO/IoStoreOnDemand.h"
#include "OnDemandBackendStatus.h"
#include "OnDemandIoStore.h"
#include "OnDemandHttpIoDispatcher.h" // EHttpRequestType
#include "Statistics.h"

#include <atomic>

namespace UE::IoStore
{
////////////////////////////////////////////////////////////////////////////////
int32 GIasHttpRangeRequestMinSizeKiB = 128;
static FAutoConsoleVariableRef CVar_IasHttpRangeRequestMinSizeKiB(
	TEXT("ias.HttpRangeRequestMinSizeKiB"),
	GIasHttpRangeRequestMinSizeKiB,
	TEXT("Minimum chunk size for partial chunk request(s)")
);

bool GIasEnableWriteOnlyDecoding = false;
static FAutoConsoleVariableRef CVar_WriteOnlyDecoding(
	TEXT("ias.EnableWriteOnlyDecoding"),
	GIasEnableWriteOnlyDecoding,
	TEXT("Enables the use of 'WriteOnly' flag when decoding to buffers with the 'HardwareTargetBuffer' flag")
);

bool GIasFastCancelEnabled = true;
static FAutoConsoleVariableRef CVar_IasFastCancelEnabled(
	TEXT("ias.FastCancelEnabled"),
	GIasFastCancelEnabled,
	TEXT("Enables canceling IAS requests before the completion callback has been triggered.")
);

////////////////////////////////////////////////////////////////////////////////
static FIoHash GetHttpCacheKey(const FOnDemandChunkHash& ChunkHash, const FIoHttpRange& Range)
{
	FIoHashBuilder HashBuilder;
	HashBuilder.Update(ChunkHash.GetData(), sizeof(FOnDemandChunkHash::ByteArray));
	HashBuilder.Update(&Range, sizeof(FIoHttpRange));

	return HashBuilder.Finalize();
}

////////////////////////////////////////////////////////////////////////////////
class FHttpIoDispatcherBackend
	: public IOnDemandHttpIoDispatcherBackend
{
	using FSharedBackendCtx = TSharedPtr<const FIoDispatcherBackendContext>;

	struct FResolvedRequest
	{
		FResolvedRequest(FIoRequestImpl& InRequest, FOnDemandChunkInfo&& InChunkInfo, const FIoOffsetAndLength& InChunkRange)
			: DispatcherRequest(&InRequest)
			, ChunkInfo(MoveTemp(InChunkInfo))
			, ChunkRange(InChunkRange)
			, StartTimeCycles(FPlatformTime::Cycles64())
		{
			check(DispatcherRequest->BackendData == nullptr);
			DispatcherRequest->BackendData = this;
		}

		static FResolvedRequest* Get(FIoRequestImpl& DispatcherRequest)
		{
			return reinterpret_cast<FResolvedRequest*>(DispatcherRequest.BackendData);
		}

		static FResolvedRequest* Create(FIoRequestImpl& DispatcherRequest, FOnDemandChunkInfo&& ChunkInfo, const FIoOffsetAndLength& ChunkRange)
		{
			TUniqueLock Lock(AllocatorMutex);
			return Allocator.Construct(DispatcherRequest, MoveTemp(ChunkInfo), ChunkRange);
		}

		static void Release(FResolvedRequest* ResolvedRequest)
		{
			if (1 == ResolvedRequest->RefCount.fetch_sub(1, std::memory_order_acq_rel))
			{
				ResolvedRequest->~FResolvedRequest();
				TUniqueLock Lock(AllocatorMutex);
				Allocator.Free(ResolvedRequest);
			}
		}

		using FAllocator		= TSingleThreadedSlabAllocator<FResolvedRequest, 64>;
		static FAllocator		Allocator;
		static FMutex			AllocatorMutex;

		FIoRequestImpl*			DispatcherRequest;
		FOnDemandChunkInfo		ChunkInfo;
		FIoOffsetAndLength		ChunkRange;
		FIoHttpRequest			HttpRequest;
		uint64					StartTimeCycles;
		std::atomic_int32_t		RefCount{2};
		std::atomic_bool		bCompleted{false};
	};

public:
											FHttpIoDispatcherBackend(FOnDemandIoStore& IoStore);
	//IOnDemandHttpIoDispatcherBackend
	virtual void							SetOptionalBulkDataEnabled(bool bEnabled) override;
	virtual void 							ReportAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const override;
	// I/O dispatcher backend
	virtual void							Initialize(TSharedRef<const FIoDispatcherBackendContext> Context) override;
	virtual void							Shutdown() override;
	virtual void							ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved) override;
	virtual FIoRequestImpl*					GetCompletedIoRequests() override;
	virtual void							CancelIoRequest(FIoRequestImpl* Request) override;
	virtual void							UpdatePriorityForIoRequest(FIoRequestImpl* Request) override;
	virtual bool							DoesChunkExist(const FIoChunkId& ChunkId) const override;
	virtual bool							DoesChunkExist(const FIoChunkId& ChunkId, const FIoOffsetAndLength& ChunkRange) const override; 
	virtual TIoStatusOr<uint64> 			GetSizeForChunk(const FIoChunkId& ChunkId) const override;
	virtual TIoStatusOr<FIoMappedRegion>	OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options) override { return FIoStatus(EIoErrorCode::NotFound); }
	virtual const TCHAR*					GetName() const { return TEXT("HttpIoDispatcherBackend"); }

private:
	void									CompleteRequest(FResolvedRequest& ResolvedRequst, FIoHttpResponse&& HttpResponse);
	static void								LogRequest(FResolvedRequest& ResolvedRequest, FIoRequestImpl& DispacherRequest, bool bDecoded, bool bCached, bool bCancelled, uint64 DurationMs);

	FBackendStatus			BackendStatus;
	FOnDemandIoBackendStats	Stats;
	FOnDemandIoStore&		IoStore;
	FIoRequestList			CompletedRequests;
	FMutex					CompletedRequestMutex;
	FSharedBackendCtx		BackendContext;
};

FHttpIoDispatcherBackend::FResolvedRequest::FAllocator FHttpIoDispatcherBackend::FResolvedRequest::Allocator;
FMutex FHttpIoDispatcherBackend::FResolvedRequest::AllocatorMutex;

////////////////////////////////////////////////////////////////////////////////
FHttpIoDispatcherBackend::FHttpIoDispatcherBackend(FOnDemandIoStore& InIoStore)
	: Stats(BackendStatus)
	, IoStore(InIoStore)
{
	BackendStatus.SetHttpEnabled(true);
	BackendStatus.SetCacheEnabled(true);
}

void FHttpIoDispatcherBackend::SetOptionalBulkDataEnabled(bool bEnabled)
{
	BackendStatus.SetHttpOptionalBulkEnabled(bEnabled);
}

void FHttpIoDispatcherBackend::ReportAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const
{
	Stats.ReportGeneralAnalytics(OutAnalyticsArray);
	Stats.ReportEndPointAnalytics(OutAnalyticsArray);
}

void FHttpIoDispatcherBackend::Initialize(TSharedRef<const FIoDispatcherBackendContext> Context)
{
	BackendContext = Context;
}

void FHttpIoDispatcherBackend::Shutdown()
{
	BackendContext.Reset();
}

void FHttpIoDispatcherBackend::ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved)
{
	const int32 HttpRetryCount = 2;

	FIoHttpBatch Batch = FHttpIoDispatcher::NewBatch();
	while (FIoRequestImpl* Request = Requests.PopHead())
	{
		FOnDemandChunkInfo ChunkInfo = IoStore.GetStreamingChunkInfo(Request->ChunkId);
		if (!ChunkInfo.IsValid())
		{
			OutUnresolved.AddTail(Request);
			continue;
		}

		if (FHttpIoDispatcher::IsHostGroupOk(ChunkInfo.HostGroupName()) == false)
		{
			OutUnresolved.AddTail(Request);
			continue;
		}

		if (BackendStatus.IsHttpEnabled(Request->ChunkId.GetChunkType()) == false)
		{
			OutUnresolved.AddTail(Request);
			continue;
		}

		if (Request->Options.GetOffset() > ChunkInfo.RawSize())
		{
			// Mark the request as failed (FIoRequestImpl::SetFailed) but with a specific error code.
			Request->SetLastBackendError(EIoErrorCode::InvalidParameter);

			{
				TUniqueLock _(CompletedRequestMutex);
				CompletedRequests.AddTail(Request);
			}

			BackendContext->WakeUpDispatcherThreadDelegate.Execute();
			continue;
		}

		const uint64 ResolvedSize = FMath::Min<uint64>(Request->Options.GetSize(), ChunkInfo.RawSize() - Request->Options.GetOffset());
		Request->Options.SetRange(Request->Options.GetOffset(), ResolvedSize);
		
		FIoOffsetAndLength ChunkRange;
		const uint64 ChunkSize = ChunkInfo.ChunkEntry().GetDiskSize(); // AES aligned encoded size
		if (ChunkSize <= (uint64(GIasHttpRangeRequestMinSizeKiB) << 10))
		{
			ChunkRange = FIoOffsetAndLength(0, ChunkSize);
		}
		else
		{
			ChunkRange = FIoChunkEncoding::GetChunkRange(
				ChunkInfo.RawSize(),
				ChunkInfo.BlockSize(),
				ChunkInfo.Blocks(),
				Request->Options.GetOffset(),
				ResolvedSize).ConsumeValueOrDie();
		}

		FResolvedRequest*			ResolvedRequest = FResolvedRequest::Create(*Request, MoveTemp(ChunkInfo), ChunkRange);
		const FIoHash				CacheKey = GetHttpCacheKey(ResolvedRequest->ChunkInfo.ChunkEntry().Hash, FIoHttpRange::FromOffsetAndLength(ChunkRange));
		const FIoOffsetAndLength	PartitionRange(ChunkInfo.PartitionOffset() + ChunkRange.GetOffset(), ChunkRange.GetLength());
		const FIoHttpRange			HttpRange = FIoHttpRange::FromOffsetAndLength(PartitionRange);
		const FIoHash&				ResourceHash = ResolvedRequest->ChunkInfo.PartitionHash(); // Hash of the CDN resource
		const EIoHttpFlags			Flags = EIoHttpFlags::Default; // Default flags will fetch the request from the local cache

		ResolvedRequest->HttpRequest = Batch.Get(
			ResolvedRequest->ChunkInfo.HostGroupName(),
			ResolvedRequest->ChunkInfo.RelativeUrl(),
			FIoHttpHeaders(),
			FIoHttpOptions(Request->Priority, HttpRetryCount, Flags, HttpRange, CacheKey, FIoHttpMetadata(Request->ChunkId)),
			ResourceHash,
			[this, ResolvedRequest](FIoHttpResponse&& HttpResponse)
			{
				// Callbacks are always triggered from the task pool
				CompleteRequest(*ResolvedRequest, MoveTemp(HttpResponse));
			});
	}

	Batch.Issue();
}

FIoRequestImpl* FHttpIoDispatcherBackend::GetCompletedIoRequests()
{
	FIoRequestList Out;
	{
		TUniqueLock Lock(CompletedRequestMutex);
		Out = MoveTemp(CompletedRequests);
		CompletedRequests = FIoRequestList();
	}

	for (FIoRequestImpl& R : Out)
	{
		if (R.BackendData == nullptr)
		{
			continue;
		}

		FResolvedRequest::Release(reinterpret_cast<FResolvedRequest*>(R.BackendData));
		R.BackendData = nullptr;
	}

	return Out.GetHead();
}

void FHttpIoDispatcherBackend::CancelIoRequest(FIoRequestImpl* Request)
{
	if (FResolvedRequest* Resolved = FResolvedRequest::Get(*Request))
	{
		Resolved->HttpRequest.Cancel();

		// Try and complete the request before the callback to cancel faster
		if (GIasFastCancelEnabled)
		{
			bool bExpected = false;
			if (Resolved->bCompleted.compare_exchange_strong(bExpected, true))
			{
				Resolved->DispatcherRequest = nullptr;
				{
					TUniqueLock Lock(CompletedRequestMutex);
					CompletedRequests.AddTail(Request);
				}
				BackendContext->WakeUpDispatcherThreadDelegate.Execute();
			}
		}
	}
}

void FHttpIoDispatcherBackend::UpdatePriorityForIoRequest(FIoRequestImpl* Request)
{
	if (FResolvedRequest* Resolved = FResolvedRequest::Get(*Request))
	{
		Resolved->HttpRequest.UpdatePriorty(Request->Priority);
	}
}

bool FHttpIoDispatcherBackend::DoesChunkExist(const FIoChunkId& ChunkId) const
{
	const TIoStatusOr<uint64> ChunkSize = GetSizeForChunk(ChunkId);
	return ChunkSize.IsOk();
}

bool FHttpIoDispatcherBackend::DoesChunkExist(const FIoChunkId& ChunkId, const FIoOffsetAndLength& ChunkRange) const
{
	return DoesChunkExist(ChunkId);
}

TIoStatusOr<uint64> FHttpIoDispatcherBackend::GetSizeForChunk(const FIoChunkId& ChunkId) const
{
	const FOnDemandChunkInfo ChunkInfo = IoStore.GetStreamingChunkInfo(ChunkId);
	if (!ChunkInfo.IsValid() || !BackendStatus.IsHttpEnabled(ChunkId.GetChunkType()))
	{
		return FIoStatus(EIoErrorCode::UnknownChunkID);
	}

	return ChunkInfo.RawSize();
}

void FHttpIoDispatcherBackend::CompleteRequest(FResolvedRequest& ResolvedRequest, FIoHttpResponse&& HttpResponse)
{
	// Check whether the I/O dispatcher request has been canceled and added to the completed request(s)
	if (GIasFastCancelEnabled)
	{
		bool bExpected = false;
		if (ResolvedRequest.bCompleted.compare_exchange_strong(bExpected, true) == false)
		{
			return FResolvedRequest::Release(&ResolvedRequest);
		}
	}

	check(ResolvedRequest.DispatcherRequest != nullptr);
	FIoBuffer Chunk						= HttpResponse.GetBody();
	FOnDemandChunkInfo& ChunkInfo		= ResolvedRequest.ChunkInfo;
	FIoRequestImpl& DispatcherRequest	= *ResolvedRequest.DispatcherRequest;

	const bool bCancelled				= DispatcherRequest.IsCancelled();
	bool bDecoded						= false;

	if (!bCancelled && HttpResponse.IsOk() && Chunk.GetSize() > 0)
	{
		FIoChunkDecodingParams Params;
		Params.EncryptionKey				= ChunkInfo.EncryptionKey();
		Params.CompressionFormat			= ChunkInfo.CompressionFormat();
		Params.BlockSize					= ChunkInfo.BlockSize();
		Params.TotalRawSize					= ChunkInfo.RawSize();
		Params.EncodedBlockSize				= ChunkInfo.Blocks();
		Params.BlockHash					= ChunkInfo.BlockHashes();
		Params.EncodedOffset				= ResolvedRequest.ChunkRange.GetOffset();
		Params.RawOffset					= DispatcherRequest.Options.GetOffset();

		DispatcherRequest.CreateBuffer(DispatcherRequest.Options.GetSize());

		const EIoDecodeFlags Options = GIasEnableWriteOnlyDecoding && EnumHasAnyFlags(DispatcherRequest.Options.GetFlags(), EIoReadOptionsFlags::HardwareTargetBuffer)
			? EIoDecodeFlags::WriteOnly : EIoDecodeFlags::None;

		bDecoded = FIoChunkEncoding::Decode(Params, Chunk.GetView(), DispatcherRequest.GetBuffer().GetMutableView(), Options);

		if (!bDecoded)
		{
			if (HttpResponse.IsCached())
			{
				Stats.OnCacheDecodeError();

				if (FIoStatus Status = FHttpIoDispatcher::EvictFromCache(HttpResponse); !Status.IsOk())
				{
					UE_LOGF(LogHttpIoDispatcher, Error, "Evict HTTP cache failed, reason '%ls'", *Status.ToString());
				}
			}
			else
			{
				Stats.OnHttpDecodeError(EHttpRequestType::Streaming);
			}
		}
	}

	const uint64 DurationMs = ResolvedRequest.StartTimeCycles > 0
		? uint64(FPlatformTime::ToMilliseconds64(FPlatformTime::Cycles64() - ResolvedRequest.StartTimeCycles))
		: 0;

	if (bDecoded)
	{
		if (!HttpResponse.IsCached())
		{
			FIoStatus Status = FHttpIoDispatcher::CacheResponse(HttpResponse);
			UE_CLOGF(!Status.IsOk(), LogHttpIoDispatcher, Error, "Put HTTP cache failed, reason '%ls'", *Status.ToString());
		}
	}
	else if (!bCancelled)
	{
		DispatcherRequest.SetFailed();
	}
	
	LogRequest(ResolvedRequest, DispatcherRequest, bDecoded, HttpResponse.IsCached(), bCancelled, DurationMs);
	FResolvedRequest::Release(&ResolvedRequest);

	{
		TUniqueLock Lock(CompletedRequestMutex);
		CompletedRequests.AddTail(&DispatcherRequest);
	}

	BackendContext->WakeUpDispatcherThreadDelegate.Execute();
}

void FHttpIoDispatcherBackend::LogRequest(FResolvedRequest& ResolvedRequest, FIoRequestImpl& DispacherRequest, bool bDecoded, bool bCached, bool bCancelled, uint64 DurationMs)
{
	if (bCancelled)
	{
		return;
	}

	const TCHAR* Prefix = [bDecoded, bCached]() -> const TCHAR*
	{
		if (bDecoded)
		{
			return bCached ? TEXT("io-cache") : TEXT("io-http ");
		}
		return bCached ? TEXT("io-cache-error") : TEXT("io-http-error ");
	}();

	auto PrioToString = [](int32 Prio) -> const TCHAR*
	{
		if (Prio < IoDispatcherPriority_Low)
		{
			return TEXT("Min");
		}
		if (Prio < IoDispatcherPriority_Medium)
		{
			return TEXT("Low");
		}
		if (Prio < IoDispatcherPriority_High)
		{
			return TEXT("Medium");
		}
		if (Prio < IoDispatcherPriority_Max)
		{
			return TEXT("High");
		}

		return TEXT("Max");
	};

	FOnDemandChunkInfo& ChunkInfo			= ResolvedRequest.ChunkInfo;
	const uint64 UncompressedSize			= bDecoded ? DispacherRequest.GetBuffer().GetSize() : 0;
	const FIoOffsetAndLength& ChunkRange	= ResolvedRequest.ChunkRange;
	const uint64 ChunkSize					= ChunkInfo.ChunkEntry().GetDiskSize();

	UE_LOG(LogIas, VeryVerbose, TEXT("%s: %5" UINT64_FMT "ms %5" UINT64_FMT "KiB[%8" UINT64_FMT "] % s: % s | Range: %" UINT64_FMT "-%" UINT64_FMT "/%" UINT64_FMT " (%.2f%%) | Prio: %s"),
		Prefix,
		DurationMs,
		UncompressedSize >> 10,
		DispacherRequest.Options.GetOffset(),
		*LexToString(DispacherRequest.ChunkId),
		*LexToString(ChunkInfo.PartitionHash()),
		ChunkRange.GetOffset(), (ChunkRange.GetOffset() + ChunkRange.GetLength() - 1), ChunkSize,
		100.0f * (float(ChunkRange.GetLength()) / float(ChunkSize)),
		PrioToString(DispacherRequest.Priority));
}

////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IOnDemandHttpIoDispatcherBackend> MakeOnDemandHttpIoDispatcherBackend(FOnDemandIoStore& IoStore)
{
	return TSharedPtr<IOnDemandHttpIoDispatcherBackend>(new FHttpIoDispatcherBackend(IoStore));
}

} // namespace UE::IoStore
