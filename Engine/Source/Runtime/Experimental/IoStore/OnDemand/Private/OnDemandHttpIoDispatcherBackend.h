// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoDispatcherBackend.h"
#include "Templates/SharedPointer.h"

struct FAnalyticsEventAttribute;

namespace UE::IoStore
{

class IOnDemandHttpIoDispatcherBackend
	: public IIoDispatcherBackend
{
public:
	virtual ~IOnDemandHttpIoDispatcherBackend() = default;
	virtual void SetOptionalBulkDataEnabled(bool bEnabled) = 0;
	virtual void ReportAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const = 0;
};

TSharedPtr<IOnDemandHttpIoDispatcherBackend> MakeOnDemandHttpIoDispatcherBackend(class FOnDemandIoStore& IoStore);

} // namespace UE::IoStore
