// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnDemandInstallCache.h"
#include "DiskCacheGovernor.h"
#include "OnDemandHttpClient.h"
#include "OnDemandIoStore.h"
#include "OnDemandMisc.h"
#include "Statistics.h"

#include "Algo/Accumulate.h"
#include "Algo/Find.h"
#include "Algo/Transform.h"
#include "Async/Mutex.h"
#include "Async/MappedFileHandle.h"
#include "Async/SharedMutex.h"
#include "Async/UniqueLock.h"
#include "Async/SharedLock.h"
#include "Async/AsyncFileHandle.h"
#include "Containers/UnrealString.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "IO/IoContainerHeader.h"
#include "IO/IoChunkId.h"
#include "IO/IoChunkEncoding.h"
#include "Logging/StructuredLog.h"
#include "IO/OnDemandError.h"
#include "Misc/DateTime.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/ScopeExit.h"
#include "Misc/StringBuilder.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"
#include "String/ParseTokens.h"
#include "String/Split.h"
#include "Tasks/Task.h"
#include "Templates/Projection.h"
#include "ProfilingDebugging/IoStoreTrace.h"

#if !UE_BUILD_SHIPPING
#include "IO/OnDemandDevelopmentExtension.h"
#endif

#if WITH_IOSTORE_ONDEMAND_TESTS
#include "Algo/Find.h"
#include "TestHarness.h"
#include "TestMacros/Assertions.h"
#include <catch2/generators/catch_generators.hpp>
#endif

#ifndef UE_ONDEMANDINSTALLCACHE_EXCLUSIVE_WRITE
#define UE_ONDEMANDINSTALLCACHE_EXCLUSIVE_WRITE (0)
#endif

#ifndef UE_ONDEMANDINSTALLCACHE_BAD_JOURNAL_VERSIONS
#define UE_ONDEMANDINSTALLCACHE_BAD_JOURNAL_VERSIONS EVersion::Invalid
#endif

#ifndef UE_ONDEMANDINSTALLCACHE_BAD_SNAPSHOT_VERSIONS
#define UE_ONDEMANDINSTALLCACHE_BAD_SNAPSHOT_VERSIONS EVersion::Invalid
#endif

#ifndef UE_ONDEMANDINSTALLCACHE_USE_MODTIME
#define UE_ONDEMANDINSTALLCACHE_USE_MODTIME (1)
#endif

#if UE_ONDEMANDINSTALLCACHE_EXCLUSIVE_WRITE
#include "Tasks/Pipe.h"
#endif // UE_ONDEMANDINSTALLCACHE_EXCLUSIVE_WRITE

#ifndef UE_IAD_DEBUG_CONSOLE_CMDS
#define UE_IAD_DEBUG_CONSOLE_CMDS (1 && !NO_CVARS && !UE_BUILD_SHIPPING)
#endif

namespace UE::IoStore
{
///////////////////////////////////////////////////////////////////////////////
namespace CVars
{
	static bool GIoStoreOnDemandEnableDefrag = true;
	static FAutoConsoleVariableRef CVar_IoStoreOnDemandEnableDefrag(
		TEXT("iostore.EnableDefrag"),
		GIoStoreOnDemandEnableDefrag,
		TEXT("Whether to enable defrag when purging")
	);

#if DO_ENSURE
	static bool GIoStoreOnDemandVerifyReferencedChunksOnPurge = true;
#else
	static bool GIoStoreOnDemandVerifyReferencedChunksOnPurge = false;
#endif
	static FAutoConsoleVariableRef CVar_IoStoreOnDemandVerifyReferencedChunksOnPurge(
		TEXT("iostore.VerifyReferencedChunksOnPurge"),
		GIoStoreOnDemandVerifyReferencedChunksOnPurge,
		TEXT("Verify all referenced chunks are in the cache when purging")
	);

	static TAutoConsoleVariable<FString> CVar_IoStoreOnDemandEagerDefragBlockCount(
		TEXT("iostore.EagerDefragBlockCount"),
		FString(),
		TEXT("Format: 'CacheName1=2;CacheName2=3;...'")
	);

	static const TStaticArray<int32, EOnDemandInstallCasType::Count>& EagerDefragBlockCount()
	{
		auto GetFromCVar = []() -> TStaticArray<int32, EOnDemandInstallCasType::Count>
		{
			using namespace UE::String;

			TStaticArray<int32, EOnDemandInstallCasType::Count> ReturnValue(InPlace, -1);

			FString Str = CVar_IoStoreOnDemandEagerDefragBlockCount.GetValueOnAnyThread();
			ParseTokens(Str, TEXT(';'),
				[&ReturnValue](FStringView Token)
				{
					FStringView Key;
					FStringView Val;
					if (!SplitFirstChar(Token, TEXT('='), Key, Val))
					{
						UE_LOGFMT(LogIoStoreOnDemand, Error, "CVar iostore.EagerDefragBlockCount contains bad token: {Token}, skipping", Token);
						return;
					}

					EOnDemandInstallCasType CasType = EOnDemandInstallCasType::None;
					LexFromString(CasType, Key);
					if (CasType == EOnDemandInstallCasType::None)
					{
						UE_LOGFMT(LogIoStoreOnDemand, Error, "CVar iostore.EagerDefragBlockCount contains unknown EOnDemandInstallCasType: {Key}, skipping", Key);
						return;
					}

					int32 Count = 0;
					if (!LexTryParseString(Count, *TStringBuilder<16>(InPlace, Val))) // Needs a CStr, bleh
					{
						UE_LOGFMT(LogIoStoreOnDemand, Error, "CVar iostore.EagerDefragBlockCount contains non-numeric Count value: {Val}, skipping", Val);
						return;
					}

					ReturnValue[CasType] = Count;
				},
				EParseTokensOptions::SkipEmpty | EParseTokensOptions::Trim
			);

			return ReturnValue;
		};

		static const TStaticArray<int32, EOnDemandInstallCasType::Count> Values = GetFromCVar();
		return Values;
	}

	static TAutoConsoleVariable<bool> CVar_MMapShadersEnabled(
		TEXT("iostore.MMapShadersEnabled"),
		false,
		TEXT("Whether shader code and shader libraries should be stored in the memory mapped CAS backend."),
		ECVF_SaveForNextBoot
	);

	static bool MemoryMappedShadersEnabled()
	{
#if USE_MMAPPED_SHADERARCHIVE
		static bool bMmapShadersEnabled = CVar_MMapShadersEnabled.GetValueOnAnyThread();
		return bMmapShadersEnabled;
#else
		return false;
#endif
	}
}

///////////////////////////////////////////////////////////////////////////////
double ToKiB(uint64 Value)
{
	return double(Value) / 1024.0;
}

///////////////////////////////////////////////////////////////////////////////
double ToMiB(uint64 Value)
{
	return double(Value) / 1024.0 / 1024.0;
}

///////////////////////////////////////////////////////////////////////////////
using FUniqueFileHandle				= TUniquePtr<IFileHandle>;
using FSharedFileHandle				= TSharedPtr<IFileHandle>;

using FSharedAsyncFileHandle		= TSharedPtr<IAsyncReadFileHandle>;
using FWeakAsyncFileHandle			= TWeakPtr<IAsyncReadFileHandle>;

using FSharedMappedFileHandle		= TSharedPtr<IMappedFileHandle>;
using FSharedMappedFileHandleRef	= TSharedRef<IMappedFileHandle>;
using FWeakMappedFileHandle			= TWeakPtr<IMappedFileHandle>;

///////////////////////////////////////////////////////////////////////////////
class FSharedMappedFileHandleProxy final : public IMappedFileHandle
{
public:
	FSharedMappedFileHandleProxy(EOnDemandInstallCasType InCasType, FSharedMappedFileHandleRef&& InSharedMappedFileHandle)
		: IMappedFileHandle(InSharedMappedFileHandle->GetFileSize())
		, SharedMappedFileHandle(MoveTemp(InSharedMappedFileHandle))
	{}

	virtual IMappedFileRegion* MapRegion(int64 Offset = 0, int64 BytesToMap = MAX_int64, FFileMappingFlags Flags = EMappedFileFlags::ENone) override
	{
		// GetFileSize() is the size when the file was opened. Some platforms use current file size instead, so enforce BytesToMap < GetFileSize() here.
		check(Offset < GetFileSize()); // don't map zero bytes and don't map off the end of the file
		BytesToMap = FMath::Min<int64>(BytesToMap, GetFileSize() - Offset);
		check(BytesToMap > 0); // don't map zero bytes

		if (EnumHasAnyFlags(Flags.Flags, EMappedFileFlags::EFileWritable))
		{
			FOnDemandInstallCacheStats::OnReadCompleted(CasType, EIoErrorCode::ReadError, 0);

			// Only allow read access through the mapped handle.
			checkf(false, TEXT("Writes to the OnDeamnd cache are only allowed from the OnDemandContentInstaller!"));
			return nullptr;
		}

		IMappedFileRegion* Region = SharedMappedFileHandle->MapRegion(Offset, BytesToMap, Flags);

		FOnDemandInstallCacheStats::OnReadCompleted(CasType, Region ? EIoErrorCode::Ok : EIoErrorCode::ReadError, BytesToMap);

		return Region;
	}

	virtual void Flush() override
	{
		SharedMappedFileHandle->Flush();
	}

private:
	FSharedMappedFileHandleRef SharedMappedFileHandle;
	EOnDemandInstallCasType CasType = EOnDemandInstallCasType::None;
};

///////////////////////////////////////////////////////////////////////////////
struct FCasBlockId
{
	FCasBlockId() = default;
	explicit FCasBlockId(uint32 InId)
		: Id(InId) { }

	bool IsValid() const { return Id != 0; }

	friend inline bool operator==(FCasBlockId LHS, FCasBlockId RHS)
	{
		return LHS.Id == RHS.Id;
	}

	friend inline uint32 GetTypeHash(FCasBlockId BlockId)
	{
		return GetTypeHash(BlockId.Id);
	}

	friend FArchive& operator<<(FArchive& Ar, FCasBlockId& BlockId)
	{
		Ar << BlockId.Id;
		return Ar;
	}

	static const FCasBlockId Invalid;

	uint32 Id = 0;
};

const FCasBlockId FCasBlockId::Invalid = FCasBlockId();

///////////////////////////////////////////////////////////////////////////////
struct FCasLocation
{
	bool IsValid() const { return BlockId.IsValid() && BlockOffset != MAX_uint32; }

	friend inline bool operator==(FCasLocation LHS, FCasLocation RHS)
	{
		return LHS.BlockId == RHS.BlockId && LHS.BlockOffset == RHS.BlockOffset;
	}

	friend inline uint32 GetTypeHash(FCasLocation Loc)
	{
		return HashCombine(GetTypeHash(Loc.BlockId), GetTypeHash(Loc.BlockOffset));
	}

	friend FArchive& operator<<(FArchive& Ar, FCasLocation& Loc)
	{
		Ar << Loc.BlockId; 
		Ar << Loc.BlockOffset;
		return Ar;
	}

	static const FCasLocation Invalid;

	FCasBlockId	BlockId;
	uint32		BlockOffset = MAX_uint32;
};

const FCasLocation FCasLocation::Invalid = FCasLocation();

///////////////////////////////////////////////////////////////////////////////
struct FCasBlockInfo
{
	uint64	FileSize = 0;
	int64	LastAccess = 0;
	int64	LastModification = 0;
	uint64	RefSize = 0;

	bool operator==(const FCasBlockInfo& Other) const
	{
		return
			FileSize == Other.FileSize && 
			LastAccess == Other.LastAccess && 
			LastModification == Other.LastModification &&
			RefSize == Other.RefSize;
	}
};

using FCasBlockInfoMap = TMap<FCasBlockId, FCasBlockInfo>;

///////////////////////////////////////////////////////////////////////////////
enum class ECasTrackAccessType : uint8
{
	Always,
	Newer,
	Granular
};

///////////////////////////////////////////////////////////////////////////////
enum class ECasTrackModificationType : uint8
{
	Always,
	Newer
};

///////////////////////////////////////////////////////////////////////////////
struct FCasJournal
{
	enum class EVersion : uint32
	{
		Invalid = 0,
		Initial,

		FixSwitchBuild,

		LatestPlusOne,
		Latest = LatestPlusOne - 1
	};

	enum class EErrorCode : uint32
	{
		None = 0,
		Simulated = 1,
		DefragOutOfDiskSpace = 2,
		DefragHashMismatch = 3
	};

	struct FHeader
	{
		static const inline uint8 MagicSequence[16] = { 'C', 'A', 'S', 'J', 'O', 'U', 'R', 'N', 'A', 'L', 'H', 'E', 'A', 'D', 'E', 'R' };

		bool			IsValid() const;
		static int64	Size() { return sizeof(FHeader); }

		uint8		Magic[16] = { 0 };
		EVersion	Version = EVersion::Invalid;
		uint8		Pad[12] = { 0 };
	};
	static_assert(sizeof(FHeader) == 32);

	struct FFooter
	{
		static const inline uint8 MagicSequence[16] = { 'C', 'A', 'S', 'J', 'O', 'U', 'R', 'N', 'A', 'L', 'F', 'O', 'O', 'T', 'E', 'R' };

		bool			IsValid() const;
		static int64	Size() { return sizeof(FFooter); }

		uint8 Magic[16] = { 0 };
	};
	static_assert(sizeof(FFooter) == 16);

	struct FEntry
	{
		enum class EType : uint8
		{
			None = 0,
			ChunkLocation,
			BlockCreated,
			BlockDeleted,
			BlockAccess,
			CriticalError,
			BlockModification,
			// New values must be added at the end
		};

		struct FChunkLocation
		{
			EType					Type = EType::ChunkLocation;
			EOnDemandInstallCasType	CasType = EOnDemandInstallCasType::General;
			ANSICHAR				Pad[2] = { 0 };
			FCasLocation			CasLocation;
			FCasAddr				CasAddr;
		};
		static_assert(sizeof(FChunkLocation) == 24);

		struct FBlockOperation
		{
			EType					Type = EType::None;
			EOnDemandInstallCasType	CasType = EOnDemandInstallCasType::General;
			uint8					Pad[2] = { 0 };
			FCasBlockId				BlockId;
			int64					UtcTicks = 0;
			uint8					Pad1[8] = { 0 };
		};
		static_assert(sizeof(FBlockOperation) == 24);

		struct FCriticalError
		{
			EType					Type = EType::CriticalError;
			EOnDemandInstallCasType	CasType = EOnDemandInstallCasType::General;
			uint8					Pad[2] = { 0 };
			EErrorCode				ErrorCode = EErrorCode::None;
			uint8					Pad1[16] = { 0 };
		};
		static_assert(sizeof(FCriticalError) == 24);

		union
		{
			FChunkLocation	ChunkLocation;
			FBlockOperation	BlockOperation;
			FCriticalError	CriticalError;
		};

		EType			Type() const { return *reinterpret_cast<const EType*>(this); }
		EOnDemandInstallCasType CasType() const
		{
			const uint8* pThis = reinterpret_cast<const uint8*>(this);
			return *reinterpret_cast<const EOnDemandInstallCasType*>(pThis + sizeof(EType));
		}
		static int64	Size() { return sizeof(FEntry); }

		FEntry() {}
	};
	static_assert(sizeof(FEntry) == 24);

	struct FTransaction
	{
		void			ChunkLocation(const FCasLocation& Location, const FCasAddr& Addr);
		void			BlockCreated(FCasBlockId BlockId);
		void			BlockDeleted(FCasBlockId BlockId);
		void			BlockAccess(FCasBlockId BlockId, int64 UtcTicks);
		void			BlockModification(FCasBlockId BlockId, int64 UtcTicks);
		void			CriticalError(FCasJournal::EErrorCode ErrorCode);

		FString					JournalFile;
		TArray<FEntry>			Entries;
		EOnDemandInstallCasType	CasType = EOnDemandInstallCasType::General;
	};

	using FEntryHandler = TFunction<FResult(const FEntry&)>;

	static FResult			Replay(const FString& JournalFile, FEntryHandler&& Handler);
	static FResult			Create(const FString& JournalFile);
	static FTransaction		Begin(EOnDemandInstallCasType InCasType, FString&& JournalFile);
	static FTransaction		Begin(EOnDemandInstallCasType InCasType, const FString& JournalFile) { return Begin(InCasType, FString(JournalFile)); }
	static FResult			Commit(FTransaction&& Transaction);
	static FResult			Commit(FTransaction&& Transaction, uint64& OutByteCount, uint32& OutOpCount);
};

///////////////////////////////////////////////////////////////////////////////
static const TCHAR* GetErrorText(FCasJournal::EErrorCode ErrorCode)
{
	switch (ErrorCode)
	{
	case FCasJournal::EErrorCode::None:
		return TEXT("None");
	case FCasJournal::EErrorCode::Simulated:
		return TEXT("Simulated error");
	case FCasJournal::EErrorCode::DefragOutOfDiskSpace:
		return TEXT("Defrag failed due to out of disk space");
	case FCasJournal::EErrorCode::DefragHashMismatch:
		return TEXT("Found corrupt chunk while defragging");
	}

	return TEXT("Unknown");
}

///////////////////////////////////////////////////////////////////////////////
bool FCasJournal::FHeader::IsValid() const
{
	if (FMemory::Memcmp(&Magic, &FHeader::MagicSequence, sizeof(FHeader::MagicSequence)) != 0)
	{
		return false;
	}

	if (static_cast<uint32>(Version) > static_cast<uint32>(EVersion::Latest))
	{
		return false;
	}

	// Versions that can no longer be loaded. The cache must be dropped instead.
	const EVersion BadVersions[] = { UE_ONDEMANDINSTALLCACHE_BAD_JOURNAL_VERSIONS };
	for (EVersion BadVersion : BadVersions)
	{
		if (Version == BadVersion)
		{
			return false;
		}
	}

	return true;
}

bool FCasJournal::FFooter::IsValid() const
{
	return FMemory::Memcmp(Magic, FFooter::MagicSequence, sizeof(FFooter::MagicSequence)) == 0;
}

FResult FCasJournal::Replay(const FString& JournalFile, FEntryHandler&& Handler)
{
	using namespace UE::IoStore::OnDemand;

	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();

	if (Ipf.FileExists(*JournalFile) == false)
	{
		return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::NotFound, FString::Printf(TEXT("Failed to find '%s'"), *JournalFile));
	}

	FFileOpenResult OpenReadResult = Ipf.OpenRead(*JournalFile, IPlatformFile::EOpenReadFlags::None);
	if (OpenReadResult.HasError())
	{
		return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::FileOpenFailed, FString::Printf(TEXT("Failed to open '%s'"), *JournalFile), OpenReadResult.StealError());
	}

	TUniquePtr<IFileHandle> FileHandle = OpenReadResult.StealValue();

	FHeader Header;
	if ((FileHandle->Read(reinterpret_cast<uint8*>(&Header), FHeader::Size()) == false) || (Header.IsValid() == false))
	{
		return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::ReadError, FString::Printf(TEXT("Failed to validate journal header in '%s'"), *JournalFile));
	}

	const int64 FileSize = FileHandle->Size();
	const int64 EntryCount = (FileSize - FHeader::Size() - FFooter::Size()) / FEntry::Size();

	if (EntryCount < 0)
	{
		return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::FileCorrupt, FString::Printf(TEXT("Invalid journal file '%s'"), *JournalFile));
	}

	if (EntryCount == 0)
	{
		return MakeValue();
	}

	const int64 FooterPos = FileSize - FFooter::Size();
	if (FooterPos < 0)
	{
		return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::FileCorrupt, FString::Printf(TEXT("Invalid journal footer in '%s'"), *JournalFile));
	}

	const int64 EntriesPos = FileHandle->Tell();
	if (FileHandle->Seek(FooterPos) == false)
	{
		return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::FileSeekFailed, FString::Printf(TEXT("Failed to seek to footer offset %" INT64_FMT " in '%s'"), FooterPos, *JournalFile));
	}

	FFooter Footer;
	if ((FileHandle->Read(reinterpret_cast<uint8*>(&Footer), FFooter::Size()) == false) || (Footer.IsValid() == false))
	{
		return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::ReadError, FString::Printf(TEXT("Failed to validate journal footer in '%s'"), *JournalFile));
	}

	if (FileHandle->Seek(EntriesPos) == false)
	{
		return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::FileSeekFailed, FString::Printf(TEXT("Failed to seek to entries offset %" INT64_FMT " in '%s'"), EntriesPos, *JournalFile));
	}

	TArray<FEntry> Entries;
	Entries.SetNumZeroed(IntCastChecked<int32>(EntryCount));

	if (FileHandle->Read(reinterpret_cast<uint8*>(Entries.GetData()), FEntry::Size() * EntryCount) == false)
	{
		return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::ReadError, FString::Printf(TEXT("Failed to read journal entries in '%s'"), *JournalFile));
	}

	UE_LOG(LogIoStoreOnDemand, Log, TEXT("Replaying %" INT64_FMT " CAS journal entries of total %.2lf KiB from '%s'"),
		EntryCount, ToKiB(FEntry::Size() * EntryCount), *JournalFile);

	for (const FEntry& Entry : Entries)
	{
		if (Entry.Type() == FEntry::EType::CriticalError)
		{
			const FEntry::FCriticalError& Error = Entry.CriticalError;
			UE_LOGF(LogIoStoreOnDemand, Warning, "Found critical error entry '%ls' (%d) in journal '%ls'",
				GetErrorText(Error.ErrorCode), EnumToUnderlyingType(Error.ErrorCode), *JournalFile);

			// We append "critical error" entries to the journal when we endup in an unrecoverable error state. This will cause the cache to be reset
			// at startup
			return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::InvalidCode, FString::Printf(TEXT("Found critical error journal entry in '%s'"), *JournalFile));
		}

		FResult HandlerResult = Handler(Entry);
		if (HandlerResult.HasError())
		{
			return HandlerResult;
		}
	}

	return MakeValue();
}

FResult FCasJournal::Create(const FString& JournalFile)
{
	using namespace UE::IoStore::OnDemand;

	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();
	Ipf.DeleteFile(*JournalFile);

	FFileOpenResult OpenWriteResult(Ipf.OpenWrite(*JournalFile, IPhysicalPlatformFile::EOpenWriteFlags::None));
	if (OpenWriteResult.HasError())
	{
		return MakeJournalError(
			ECasErrorCode::CreateJournalFailed, EIoErrorCode::FileOpenFailed, 
			FString::Printf(TEXT("Failed to create journal '%s'"), *JournalFile),
			OpenWriteResult.StealError());
	}

	TUniquePtr<IFileHandle> FileHandle = OpenWriteResult.StealValue();

	FHeader Header;
	FMemory::Memcpy(&Header.Magic, &FHeader::MagicSequence, sizeof(FHeader::MagicSequence));
	Header.Version = EVersion::Latest;

	if (FileHandle->Write(reinterpret_cast<uint8*>(&Header), FHeader::Size()) == false)
	{
		return MakeJournalError(ECasErrorCode::CreateJournalFailed, EIoErrorCode::WriteError, FString::Printf(TEXT("Failed to write journal header in '%s'"), *JournalFile));
	}

	FFooter Footer;
	FMemory::Memcpy(&Footer.Magic, &FFooter::MagicSequence, sizeof(FFooter::MagicSequence));
	if (FileHandle->Write(reinterpret_cast<uint8*>(&Footer), FFooter::Size()) == false)
	{
		return MakeJournalError(ECasErrorCode::CreateJournalFailed, EIoErrorCode::WriteError, FString::Printf(TEXT("Failed to write journal footer in '%s'"), *JournalFile));
	}

	return MakeValue();
}

FCasJournal::FTransaction FCasJournal::Begin(EOnDemandInstallCasType InCasType, FString&& JournalFile)
{
	return FTransaction
	{
		.JournalFile = MoveTemp(JournalFile),
		.CasType = InCasType
	};
}

FResult FCasJournal::Commit(FTransaction&& Transaction, uint64& OutByteCount, uint32& OutOpCount)
{
	using namespace UE::IoStore::OnDemand;

	OutByteCount = 0;
	OutOpCount = 0;

	if (Transaction.Entries.IsEmpty())
	{
		return MakeValue();
	}

	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();

	// Validate header and footer
	{
		FFileOpenResult OpenReadResult = Ipf.OpenRead(*Transaction.JournalFile, IPlatformFile::EOpenReadFlags::None);
		if (OpenReadResult.HasError())
		{
			FResult Result = MakeJournalError(
				ECasErrorCode::CommitJournalFailed, EIoErrorCode::FileOpenFailed, 
				FString::Printf(TEXT("Failed to open journal '%s'"), *Transaction.JournalFile),
				OpenReadResult.StealError());
			FOnDemandInstallCacheStats::OnJournalCommit(Transaction.CasType, Result, 0);
			return Result;
		}

		TUniquePtr<IFileHandle> FileHandle = OpenReadResult.StealValue();

		const int64 FileSize = FileHandle->Size();
		if (FileSize < FHeader::Size())
		{
			FResult Result = MakeJournalError(
				ECasErrorCode::CommitJournalFailed, EIoErrorCode::FileCorrupt, FString::Printf(TEXT("Failed to validate journal header in '%s'"), *Transaction.JournalFile));
			FOnDemandInstallCacheStats::OnJournalCommit(Transaction.CasType, Result, 0);
			return Result;
		}

		FHeader Header;
		if ((FileHandle->Read(reinterpret_cast<uint8*>(&Header), FHeader::Size()) == false) || (Header.IsValid() == false))
		{
			FResult Result = MakeJournalError(
				ECasErrorCode::CommitJournalFailed, EIoErrorCode::ReadError, FString::Printf(TEXT("Failed to validate journal header in '%s'"), *Transaction.JournalFile));
			FOnDemandInstallCacheStats::OnJournalCommit(Transaction.CasType, Result, 0);
			return Result;
		}

		const int64 FooterPos = FileSize - FFooter::Size();
		if (FileHandle->Seek(FooterPos) == false)
		{
			FResult Result = MakeJournalError(
				ECasErrorCode::CommitJournalFailed, EIoErrorCode::FileSeekFailed, FString::Printf(TEXT("Failed to validate journal footer in '%s'"), *Transaction.JournalFile));
			FOnDemandInstallCacheStats::OnJournalCommit(Transaction.CasType, Result, 0);
			return Result;
		}

		FFooter Footer;
		if ((FileHandle->Read(reinterpret_cast<uint8*>(&Footer), FFooter::Size()) == false) || (Footer.IsValid() == false))
		{
			FResult Result = MakeJournalError(
				ECasErrorCode::CommitJournalFailed, EIoErrorCode::ReadError, FString::Printf(TEXT("Found to validate journal footer in '%s'"), *Transaction.JournalFile));
			FOnDemandInstallCacheStats::OnJournalCommit(Transaction.CasType, Result, 0);
			return Result;
		}
	}

	// Append entries
	{
		FFileOpenResult			OpenWriteResult(Ipf.OpenWrite(*Transaction.JournalFile, IPhysicalPlatformFile::EOpenWriteFlags::Append));
		TUniquePtr<IFileHandle>	FileHandle = OpenWriteResult.HasValue() ? OpenWriteResult.StealValue() : nullptr;
		const int64				FileSize = FileHandle.IsValid() ? FileHandle->Size() : -1;
		const int64				EntriesPos = FileSize > 0 ? FileSize - FFooter::Size() : -1;

		if (EntriesPos < 0)
		{
			FResult Result = MakeJournalError(
				ECasErrorCode::CommitJournalFailed, EIoErrorCode::FileOpenFailed, 
				FString::Printf(TEXT("Failed to open journal '%s'"), *Transaction.JournalFile),
				OpenWriteResult.HasError() ? OpenWriteResult.StealError() : FFileSystemError(FString()));
			FOnDemandInstallCacheStats::OnJournalCommit(Transaction.CasType, Result, 0);
			return Result;
		}

		if (FileHandle->Seek(EntriesPos) == false)
		{
			FResult Result = MakeJournalError(
				ECasErrorCode::CommitJournalFailed, EIoErrorCode::FileSeekFailed, FString::Printf(TEXT("Failed to seek to journal entries offset %" INT64_FMT " in '%s'"), EntriesPos, *Transaction.JournalFile));
			FOnDemandInstallCacheStats::OnJournalCommit(Transaction.CasType, Result, 0);
			return Result;
		}

		const int64 TotalEntrySize = Transaction.Entries.Num() * FEntry::Size();
		if (FileHandle->Write(
			reinterpret_cast<const uint8*>(Transaction.Entries.GetData()),
			TotalEntrySize) == false)
		{
			FResult Result = MakeJournalError(
				ECasErrorCode::CommitJournalFailed, EIoErrorCode::WriteError, FString::Printf(TEXT("Failed to write journal entries to '%s'"), *Transaction.JournalFile));
			FOnDemandInstallCacheStats::OnJournalCommit(Transaction.CasType, Result, 0);
			return Result;
		}
		OutOpCount++;
		OutByteCount += TotalEntrySize;

		FFooter Footer;
		FMemory::Memcpy(&Footer.Magic, &FFooter::MagicSequence, sizeof(FFooter::MagicSequence));
		if (FileHandle->Write(reinterpret_cast<uint8*>(&Footer), FFooter::Size()) == false)
		{
			FResult Result = MakeJournalError(
				ECasErrorCode::CommitJournalFailed, EIoErrorCode::WriteError, FString::Printf(TEXT("Failed to write journal footer to '%s'"), *Transaction.JournalFile));
			FOnDemandInstallCacheStats::OnJournalCommit(Transaction.CasType, Result, 0);
			return Result;
		}
		OutOpCount++;
		OutByteCount += uint64(FFooter::Size());

		if (FileHandle->Flush() == false)
		{
			FResult Result = MakeJournalError(
				ECasErrorCode::CommitJournalFailed, EIoErrorCode::FileFlushFailed, FString::Printf(TEXT("Failed to flush journal entries to '%s'"), *Transaction.JournalFile));
			FOnDemandInstallCacheStats::OnJournalCommit(Transaction.CasType, Result, 0);
			return Result;
		}
		OutOpCount++;

		UE_LOGF(LogIoStoreOnDemand, Log, "Committed %d CAS journal entries of total %.2lf KiB to '%ls'",
			Transaction.Entries.Num(), ToKiB(TotalEntrySize), *Transaction.JournalFile);

		FResult Result = MakeValue();
		FOnDemandInstallCacheStats::OnJournalCommit(Transaction.CasType, Result, TotalEntrySize);
		return Result;
	}
}

FResult FCasJournal::Commit(FTransaction&& Transaction)
{
	uint64 ByteCount = 0;
	uint32 OpCount = 0;
	return Commit(MoveTemp(Transaction), ByteCount, OpCount);
}

void FCasJournal::FTransaction::ChunkLocation(const FCasLocation& Location, const FCasAddr& Addr)
{
	new(&Entries.AddDefaulted_GetRef().ChunkLocation) FEntry::FChunkLocation
	{
		.CasType = CasType,
		.CasLocation = Location,
		.CasAddr = Addr
	};
}

void FCasJournal::FTransaction::BlockCreated(FCasBlockId BlockId)
{
	new(&Entries.AddDefaulted_GetRef().BlockOperation) FEntry::FBlockOperation
	{
		.Type = FEntry::EType::BlockCreated,
		.CasType = CasType,
		.BlockId = BlockId,
		.UtcTicks = FDateTime::UtcNow().GetTicks()
	};
}

void FCasJournal::FTransaction::BlockDeleted(FCasBlockId BlockId)
{
	new(&Entries.AddDefaulted_GetRef().BlockOperation) FEntry::FBlockOperation
	{
		.Type = FEntry::EType::BlockDeleted,
		.CasType = CasType,
		.BlockId = BlockId,
		.UtcTicks = FDateTime::UtcNow().GetTicks()
	};
}

void FCasJournal::FTransaction::BlockAccess(FCasBlockId BlockId, int64 UtcTicks)
{
	new(&Entries.AddDefaulted_GetRef().BlockOperation) FEntry::FBlockOperation
	{
		.Type = FEntry::EType::BlockAccess,
		.CasType = CasType,
		.BlockId = BlockId,
		.UtcTicks = UtcTicks
	};
}

void FCasJournal::FTransaction::BlockModification(FCasBlockId BlockId, int64 UtcTicks)
{
	new(&Entries.AddDefaulted_GetRef().BlockOperation) FEntry::FBlockOperation
	{
		.Type = FEntry::EType::BlockModification,
		.CasType = CasType,
		.BlockId = BlockId,
		.UtcTicks = UtcTicks
	};
}

void FCasJournal::FTransaction::CriticalError(FCasJournal::EErrorCode ErrorCode)
{
	new(&Entries.AddDefaulted_GetRef().CriticalError) FEntry::FCriticalError
	{
		.Type = FEntry::EType::CriticalError,
		.CasType = CasType,
		.ErrorCode = ErrorCode
	};
}

///////////////////////////////////////////////////////////////////////////////
struct FCasSnapshot
{
	enum class EVersion : uint32
	{
		Invalid = 0,
		Initial,

		AddModificationTime,
		FixSwitchBuild,
		FixLostSnapshots,
		AddCasType,

		LatestPlusOne,
		Latest = LatestPlusOne - 1
	};

	static const FGuid CustomVersionGuid;

	struct FHeader
	{
		static const inline uint8 MagicSequence[16] = {'+', 'S', 'N', 'A', 'P', 'S', 'H', 'O', 'T', 'H', 'E', 'A', 'D', 'E', 'R', '+'};

		bool			IsValid() const;
		static int64	Size() { return sizeof(FHeader); }

		uint8		Magic[16] = {0};
		EVersion	Version = EVersion::Invalid;
		uint8		Pad[12] = {0};
	};
	static_assert(sizeof(FHeader) == 32);

	struct FFooter
	{
		static const inline uint8 MagicSequence[16] = {'+', 'S', 'N', 'A', 'P', 'S', 'H', 'O', 'T', 'F', 'O', 'O', 'T', 'E', 'R', '+'};

		static int64	Size() { return sizeof(FFooter); }
		bool			IsValid() const;

		uint8 Magic[16] = {0};
	};
	static_assert(sizeof(FFooter) == 16);

	struct FBlock
	{
		friend FArchive& operator<<(FArchive& Ar, FBlock& Block)
		{
			Ar << Block.BlockId;
			Ar << Block.LastAccess;

			if (Ar.CustomVer(FCasSnapshot::CustomVersionGuid) >= static_cast<int32>(EVersion::AddModificationTime))
			{
				Ar << Block.ModTime;
			}

			return Ar;
		}

		FCasBlockId BlockId;
		int64		LastAccess = 0;
		int64		ModTime = 0;
	};

	using FChunkLocation = TPair<FCasAddr, FCasLocation>;

	struct FCasState
	{
		friend FArchive& operator<<(FArchive& Ar, FCasState& InCasState)
		{
			Ar << InCasState.Blocks;
			Ar << InCasState.ChunkLocations;
			Ar << InCasState.CurrentBlockId;

			if (Ar.CustomVer(FCasSnapshot::CustomVersionGuid) >= static_cast<int32>(EVersion::AddCasType))
			{
				using UnderType = std::underlying_type<EOnDemandInstallCasType>::type;
				UnderType Val = static_cast<UnderType>(InCasState.CasType);
				if (Ar.IsLoading())
				{
					Ar << Val;
					InCasState.CasType = static_cast<EOnDemandInstallCasType>(Val);
				}
				else
				{
					Ar << Val;
				}
			}

			return Ar;
		}

		TArray<FBlock>						Blocks;
		TArray<FChunkLocation>				ChunkLocations;
		FCasBlockId							CurrentBlockId;
		EOnDemandInstallCasType				CasType{ EOnDemandInstallCasType::None };
	};

	static TResult<FCasSnapshot>		FromJournal(const FString& JournalFile, FCasSnapshot&& BaseSnapshot);
	static TResult<FCasSnapshot>		Load(const FString& SnapshotFile, int64* OutFileSize = nullptr);
	static TResult<int64>				Save(const FCasSnapshot& Snapshot, const FString& SnapshotFile);
	/** 
	 * Append the journal file to the snapshot and reset the journal file.
	 *
	 * @param SnapshotFile	The snapshot file to appened the journal to. May not exist yet.
	 * @param JournalFile	The journal file to append to the snapshot, and then reset.
	 *
	 * Note: If this operation fails, it will leave behind temporary files, call TempFilesExist() check for them.
	 * There is no provided method to cleanup temporary files. If they are discovered, the
	 * cache is in an inconsistent state and should be reset.
	 */
	static TResult<int64>				AppendAndResetJournal(
											const FString& SnapshotFile, 
											const FString& JournalFile);

	/**
	 * There is no provided method to cleanup temporary files. If they are discovered, the
	 * cacche is in an inconsistent state and should be reset.
	 */
	static bool TempFilesExist(const FString& SnapshotFile, const FString& JournalFile);

	static FString MakeTempFileName(const FString& FileName)
	{
		const bool bIncludeDot = true;
		const FString Extension = FPaths::GetExtension(FileName, bIncludeDot) + TEXT("tmp");
		return FPaths::ChangeExtension(FileName, Extension);
	}

	TArray<FCasState> CasState;

private:
	FCasState& FindOrAddCasState(EOnDemandInstallCasType CasType)
	{
		FCasState* Found = Algo::FindBy(CasState, CasType, &FCasState::CasType);
		if (Found)
		{
			return *Found;
		}

		CasState.Emplace_GetRef().CasType = CasType;

		Algo::SortBy(CasState, &FCasState::CasType);

		Found = Algo::FindBy(CasState, CasType, &FCasState::CasType);
		return *Found;
	}
};

///////////////////////////////////////////////////////////////////////////////
struct FCasBlockState
{
	uint32 Size = 0;
	uint32 HandleCount = 0;
	int64 LastAccessUtcTicks = 0;
	int64 LastModificationUtcTicks = 0;
	TVariant<FWeakAsyncFileHandle, FWeakMappedFileHandle> WeakHandle; // TODO: Possibly make two flavors of FCasBlockState to avoid TVariant overhead
	TOptional<FSharedEventRef> PendingDefragEvent;	// Currently only used for MMap CAS
};

struct FCas
{
	static constexpr uint32		DeleteBlockMaxWaitTimeMs = 10000;
	static constexpr int64		DirtyTimestampMask = std::numeric_limits<int64>::lowest(); // sign bit

	using FLookup				= TMap<FCasAddr, FCasLocation>;
	using FReadHandles			= TMap<FCasBlockId, FWeakAsyncFileHandle>;
	using FLastAccess			= TMap<FCasBlockId, int64>;
	using FBlockStates			= TMap<FCasBlockId, FCasBlockState>;
	using FBlockFileSizes		= TMap<FCasBlockId, uint64>;

	struct FMaybeLock
	{
		TDynamicUniqueLock<const FCas> Lock;

		FMaybeLock(const FCas* Cas, const UE::FDeferLock* SkipLock)
			: Lock(SkipLock ? TDynamicUniqueLock(*Cas, *SkipLock) : TDynamicUniqueLock(*Cas))
		{}
	};

	FCas(EOnDemandInstallCasType InType);

	void						Lock() const		{ Mutex.Lock(); }
	void						Unlock() const		{ Mutex.Unlock(); }

	FResult						Configure(const FOnDemandInstallCasConfig& Config);
	FResult						Initialize(FStringView Directory, bool bDeleteExisting = false);
	FCasLocation				FindChunk(const FCasAddr& ChunkAddr, const UE::FDeferLock* SkipLock = nullptr) const;
	FCasBlockId					CreateBlock();
	bool						ContainsBlock(FCasBlockId BlockId) const;
	FResult						DeleteBlock(FCasBlockId BlockId, TFunctionRef<void(const FCasAddr&)> OutAddrs);
	FString						GetBlockFilename(FCasBlockId BlockId) const;
	TResult<FSharedFileHandle>	OpenRead(FCasBlockId BlockId, bool bAllowReadFromMMapCas = false);
	TResult<FSharedAsyncFileHandle>	OpenAsyncRead(FCasBlockId BlockId);
	TResult<FSharedMappedFileHandle> OpenMapped(FCasBlockId BlockId, int64 MaxSize, TOptional<FSharedEventRef>* OutBlockDeletedEvent = nullptr, const UE::FDeferLock* SkipLock = nullptr);
	void						OnFileHandleDeleted(FCasBlockId BlockId);
	bool						SetPendingDefrag(FCasBlockId BlockId, const UE::FDeferLock* SkipLock = nullptr);
	void						ResetAllPendingDefrag(const UE::FDeferLock* SkipLock = nullptr);
	TResult<FUniqueFileHandle>	OpenWrite(FCasBlockId BlockId, bool bAppend) const;
	bool						TrackAccessIf(ECasTrackAccessType Type, FCasBlockId BlockId, int64 UtcTicks, bool bDirty, const UE::FDeferLock* SkipLock = nullptr);
	bool						TrackAccessIf(ECasTrackAccessType Type, FCasBlockId BlockId, bool bDirty, const UE::FDeferLock* SkipLock = nullptr)
	{
		return TrackAccessIf(Type, BlockId, FDateTime::UtcNow().GetTicks(), bDirty, SkipLock);
	};
	bool						UnlockedTrackAccessIf(ECasTrackAccessType Type, uint32 BlockIdHash, FCasBlockId BlockId, int64 UtcTicks, bool bDirty);
	bool						TrackModificationIf(ECasTrackModificationType Type, FCasBlockId BlockId, int64 UtcTicks);
	bool						TrackModificationIf(ECasTrackModificationType Type, FCasBlockId BlockId)
	{
		return TrackModificationIf(Type, BlockId, FDateTime::UtcNow().GetTicks());
	};
	bool						UnlockedTrackModificationIf(ECasTrackModificationType Type, uint32 BlockIdHash, FCasBlockId BlockId, int64 UtcTicks);
	void						UpdateBlock(
									FCasBlockId BlockId, 
									uint32 BytesWritten, 
									TConstArrayView<FCasAddr> ChunkAddrs,
									TConstArrayView<int64> ChunkOffsets,
									TFunctionRef<void(const FCasAddr&, const FCasLocation&)> OutAddrs);
	uint64						GetBlockInfo(FCasBlockInfoMap& OutBlockInfo) const;
	uint64						GetTotalBlockSize() const;
	FBlockFileSizes				FindAllBlockFiles() const;
	void						Compact();
	FResult						Verify(TArray<FCasAddr>& OutRemovedChunksAddrs, TMap<FCasBlockId, uint64>& OutOverBudgetBlocks);
	void						LoadSnapshot(FCasSnapshot::FCasState&& Snapshot);
	FResult						ApplyJournalEntry(const FCasJournal::FEntry& JournalEntry);

	// Returns the timestamps that are "dirty" and need to be flushed to disk and clears dirty flag
	FLastAccess					GetAndClearDirtyLastAccess();

	// Returns the place holder timestamp to use if a timestamp is not found in the LastAccess table
	static int64				GetTimestampForMissingLastAccess();

	// Return the minimum value for a valid timestamp
	static int64				GetMinimumTimestamp();

	uint32						GetMinBlockSize()
	{
		return MinBlockSize;
	}
	uint32						GetMaxBlockSize()
	{
		return MaxBlockSize;
	}

	const FString&				GetRootDirectory() const { return RootDirectory; }

	EOnDemandInstallCasType		GetType() const { return CasType; }

private:
	FString					RootDirectory;
public: // TODO: FIXME: encapsulate these members
	FLookup					Lookup;
private:
	FBlockStates			BlockIds;
	FEventRef				BlockReadsDoneEvent;
	int64					LastAccessGranularityTicks = 0;
	uint32					MinBlockSize = 32 << 19;
	uint32					MaxBlockSize = 32 << 20;
	EOnDemandInstallCasType	CasType{ EOnDemandInstallCasType::None };
	mutable UE::FMutex		Mutex;
};

const FGuid FCasSnapshot::CustomVersionGuid(0x217bee7d, 0x23a54a23, 0x9ef080d2, 0x3df768b1);

///////////////////////////////////////////////////////////////////////////////
bool FCasSnapshot::FHeader::IsValid() const
{
	if (FMemory::Memcmp(&Magic, &FHeader::MagicSequence, sizeof(FHeader::MagicSequence)) != 0)
	{
		return false;
	}

	if (static_cast<uint32>(Version) > static_cast<uint32>(EVersion::Latest))
	{
		return false;
	}

	// Drop all snapshots from before we fixed snapshots not being propogated
	if (static_cast<uint32>(Version) < static_cast<uint32>(EVersion::FixLostSnapshots))
	{
		return false;
	}

	// Versions that can no longer be loaded. The cache must be dropped instead.
	const EVersion BadVersions[] = { UE_ONDEMANDINSTALLCACHE_BAD_SNAPSHOT_VERSIONS };
	for (EVersion BadVersion : BadVersions)
	{
		if (Version == BadVersion)
		{
			return false;
		}
	}

	return true;
}

bool FCasSnapshot::FFooter::IsValid() const
{
	return FMemory::Memcmp(Magic, FFooter::MagicSequence, sizeof(FFooter::MagicSequence)) == 0;
}

TResult<FCasSnapshot> FCasSnapshot::FromJournal(const FString& JournalFile, FCasSnapshot&& BaseSnapshot)
{
	TOptional<FCas::FLookup>				CasLookupByType[EOnDemandInstallCasType::Count];
	TOptional<TMap<FCasBlockId, FBlock>>	BlockMapByType[EOnDemandInstallCasType::Count];
	TOptional<FCasBlockId>					CurrentBlockIdByType[EOnDemandInstallCasType::Count];

	for (FCasState& BaseState : BaseSnapshot.CasState)
	{
		FCas::FLookup&				CasLookup = CasLookupByType[BaseState.CasType].Emplace();
		TMap<FCasBlockId, FBlock>&	BlockMap = BlockMapByType[BaseState.CasType].Emplace();
		FCasBlockId&				CurrentBlockId = CurrentBlockIdByType[BaseState.CasType].Emplace();

		CasLookup.Reserve(BaseState.ChunkLocations.Num());
		for (TPair<FCasAddr, FCasLocation>& Kv : BaseState.ChunkLocations)
		{
			CasLookup.Add(MoveTemp(Kv));
		}

		BlockMap.Reserve(BaseState.Blocks.Num());
		for (const FCasSnapshot::FBlock& Block : BaseState.Blocks)
		{
			BlockMap.Add(Block.BlockId, Block);
		}

		CurrentBlockId = BaseState.CurrentBlockId;
	}

	FResult ReplayResult = FCasJournal::Replay(
		JournalFile,
		[&CasLookupByType, &BlockMapByType, &CurrentBlockIdByType](const FCasJournal::FEntry& JournalEntry) -> FResult
		{
			EOnDemandInstallCasType CasType = JournalEntry.CasType();
			FCas::FLookup&				CasLookup = CasLookupByType[CasType].IsSet() ? 
										CasLookupByType[CasType].GetValue() : 
										CasLookupByType[CasType].Emplace();
			TMap<FCasBlockId, FBlock>&	BlockMap = BlockMapByType[CasType].IsSet() ? 
										BlockMapByType[CasType].GetValue() : 
										BlockMapByType[CasType].Emplace();
			FCasBlockId&				CurrentBlockId = CurrentBlockIdByType[CasType].IsSet() ?
										CurrentBlockIdByType[CasType].GetValue() : 
										CurrentBlockIdByType[CasType].Emplace();

			switch(JournalEntry.Type())
			{
			case FCasJournal::FEntry::EType::ChunkLocation:
			{
				const FCasJournal::FEntry::FChunkLocation& ChunkLocation = JournalEntry.ChunkLocation;
				if (ChunkLocation.CasLocation.IsValid())
				{
					FCasLocation& Loc = CasLookup.FindOrAdd(ChunkLocation.CasAddr);
					Loc = ChunkLocation.CasLocation;
				}
				else
				{
					CasLookup.Remove(ChunkLocation.CasAddr);
				}
				break;
			}
			case FCasJournal::FEntry::EType::BlockCreated:
			{
				const FCasJournal::FEntry::FBlockOperation& Op = JournalEntry.BlockOperation;
				CurrentBlockId = Op.BlockId;
				BlockMap.Emplace(Op.BlockId, FBlock{ .BlockId = Op.BlockId });
				break;
			}
			case FCasJournal::FEntry::EType::BlockDeleted:
			{
				const FCasJournal::FEntry::FBlockOperation& Op = JournalEntry.BlockOperation;
				BlockMap.Remove(Op.BlockId);
				if (CurrentBlockId == Op.BlockId)
				{
					CurrentBlockId = FCasBlockId::Invalid; 
				}
				break;
			}
			case FCasJournal::FEntry::EType::BlockAccess:
			{
				const FCasJournal::FEntry::FBlockOperation& Op = JournalEntry.BlockOperation;
				if (FBlock* Block = BlockMap.Find(Op.BlockId))
				{
					Block->LastAccess = Op.UtcTicks;
				}
				else
				{
					using namespace UE::IoStore::OnDemand;
					return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::FileCorrupt, TEXT("Journal BlockAccess Op references an invalid block"));
				}
				break;
			}
			case FCasJournal::FEntry::EType::BlockModification:
			{
				const FCasJournal::FEntry::FBlockOperation& Op = JournalEntry.BlockOperation;
				if (FBlock* Block = BlockMap.Find(Op.BlockId))
				{
					Block->ModTime = Op.UtcTicks;
				}
				else
				{
					using namespace UE::IoStore::OnDemand;
					return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::FileCorrupt, TEXT("Journal BlockModification Op references an invalid block"));
				}
				break;
			}
			};

			return MakeValue();
		});

	if (ReplayResult.HasError())
	{
		return MakeError(ReplayResult.StealError());
	}

	FCasSnapshot Snapshot;

	for (EOnDemandInstallCasType CasType : TEnumRange<EOnDemandInstallCasType>())
	{
		if (BlockMapByType[CasType].IsSet())
		{
			BlockMapByType[CasType].GetValue().GenerateValueArray(Snapshot.FindOrAddCasState(CasType).Blocks);
		}

		if (CasLookupByType[CasType].IsSet())
		{
			Snapshot.FindOrAddCasState(CasType).ChunkLocations = CasLookupByType[CasType].GetValue().Array();
		}

		if (CurrentBlockIdByType[CasType].IsSet())
		{
			Snapshot.FindOrAddCasState(CasType).CurrentBlockId = CurrentBlockIdByType[CasType].GetValue();
		}
	}

	return MakeValue(Snapshot);
}

TResult<int64> FCasSnapshot::Save(const FCasSnapshot& Snapshot, const FString& SnapshotFile)
{
	using namespace UE::IoStore::OnDemand;

	IFileManager& Ifm = IFileManager::Get();

	TUniquePtr<FArchive> Ar(Ifm.CreateFileWriter(*SnapshotFile));
	if (Ar.IsValid() == false)
	{
		return MakeSnapshotError<int64>(ECasErrorCode::SaveSnapshotFailed, EIoErrorCode::FileOpenFailed, FString::Printf(TEXT("Failed to open file '%s' for writing"), *SnapshotFile));
	}

	FHeader Header;
	FMemory::Memcpy(&Header.Magic, &FHeader::MagicSequence, sizeof(FHeader::MagicSequence));
	Header.Version = EVersion::Latest;

	Ar->SetCustomVersion(CustomVersionGuid, static_cast<int32>(Header.Version), "CasSnapshotVersion");

	Ar->Serialize(reinterpret_cast<uint8*>(&Header), FHeader::Size());
	if (Ar->IsError())
	{
		const uint32 LastError = FPlatformMisc::GetLastError();
		Ar.Reset();
		return MakeSnapshotError<int64>(ECasErrorCode::SaveSnapshotFailed, EIoErrorCode::WriteError, FString::Printf(TEXT("Failed to write snapshot header to '%s'"), *SnapshotFile), LastError);
	}

	FCasSnapshot& NonConst = *const_cast<FCasSnapshot*>(&Snapshot);
	*Ar << NonConst.CasState;

	if (Ar->IsError())
	{
		const uint32 LastError = FPlatformMisc::GetLastError();
		Ar.Reset();
		return MakeSnapshotError<int64>(ECasErrorCode::SaveSnapshotFailed, EIoErrorCode::WriteError, FString::Printf(TEXT("Failed to write snapshot to '%s'"), *SnapshotFile), LastError);
	}

	FFooter Footer;
	FMemory::Memcpy(&Footer.Magic, &FFooter::MagicSequence, sizeof(FFooter::MagicSequence));
	Ar->Serialize(reinterpret_cast<uint8*>(&Footer), FFooter::Size());
	if (Ar->IsError())
	{
		const uint32 LastError = FPlatformMisc::GetLastError();
		Ar.Reset();
		return MakeSnapshotError<int64>(ECasErrorCode::SaveSnapshotFailed, EIoErrorCode::WriteError, FString::Printf(TEXT("Failed to write snapshot footer to '%s'"), *SnapshotFile), LastError);
	}

	const int64 FileSize = Ar->TotalSize();
	if (Ar->Close() == false)
	{
		const uint32 LastError = FPlatformMisc::GetLastError();
		Ar.Reset();
		return MakeSnapshotError<int64>(ECasErrorCode::SaveSnapshotFailed, EIoErrorCode::WriteError, FString::Printf(TEXT("Failed to close snapshot footer to '%s'"), *SnapshotFile), LastError);
	}

	return MakeValue(FileSize);
}

TResult<FCasSnapshot> FCasSnapshot::Load(const FString& SnapshotFile, int64* OutFileSize)
{
	using namespace UE::IoStore::OnDemand;

	IFileManager& Ifm = IFileManager::Get();

	if (Ifm.FileExists(*SnapshotFile) == false)
	{
		return MakeSnapshotError<FCasSnapshot>(ECasErrorCode::LoadSnapshotFailed, EIoErrorCode::NotFound, FString::Printf(TEXT("Failed to find snapshot '%s'"), *SnapshotFile));
	}

	TUniquePtr<FArchive> Ar(Ifm.CreateFileReader(*SnapshotFile));
	if (Ar.IsValid() == false)
	{
		return MakeSnapshotError<FCasSnapshot>(ECasErrorCode::LoadSnapshotFailed, EIoErrorCode::FileOpenFailed, FString::Printf(TEXT("Failed open snapshot '%s'"), *SnapshotFile));
	}

	FHeader Header;
	Ar->Serialize(reinterpret_cast<uint8*>(&Header), FHeader::Size());
	if (Ar->IsError() || Header.IsValid() == false)
	{
		return MakeSnapshotError<FCasSnapshot>(ECasErrorCode::LoadSnapshotFailed, EIoErrorCode::ReadError, FString::Printf(TEXT("Failed to validate snapshot header in '%s'"), *SnapshotFile));
	}

	Ar->SetCustomVersion(CustomVersionGuid, static_cast<int32>(Header.Version), "CasSnapshotVersion");

	FCasSnapshot Snapshot;

	if (Ar->CustomVer(FCasSnapshot::CustomVersionGuid) < static_cast<int32>(EVersion::AddCasType))
	{
		*Ar << Snapshot.FindOrAddCasState(EOnDemandInstallCasType::General);
	}
	else
	{
		*Ar << Snapshot.CasState;
	}

	FFooter Footer;
	Ar->Serialize(reinterpret_cast<uint8*>(&Footer), FFooter::Size());
	if (Ar->IsError() || Footer.IsValid() == false)
	{
		return MakeSnapshotError<FCasSnapshot>(ECasErrorCode::LoadSnapshotFailed, EIoErrorCode::ReadError, FString::Printf(TEXT("Failed to validate snapshot footer in '%s'"), *SnapshotFile));
	}

	if (OutFileSize != nullptr)
	{
		*OutFileSize = Ar->Tell();
	}

	return MakeValue(MoveTemp(Snapshot));
}

TResult<int64> FCasSnapshot::AppendAndResetJournal(const FString& SnapshotFile, const FString& JournalFile)
{
	using namespace UE::IoStore::OnDemand;

	IFileManager& Ifm = IFileManager::Get();

	if (Ifm.FileExists(*JournalFile) == false)
	{
		return MakeSnapshotError<int64>(
			ECasErrorCode::CreateSnapshotFailed,
			EIoErrorCode::NotFound,
			FString::Printf(TEXT("Cannot create snapshot, failed to find journal file '%s'"), *JournalFile));
	}

	// Load the previous snapshot if it exists
	TResult<FCasSnapshot> SnapshotResult = FCasSnapshot::Load(SnapshotFile);
	if (SnapshotResult.HasError())
	{
		const FCasSnapshotError* Payload = SnapshotResult.GetError().GetErrorContext<CasSnapshotError>();
		if (Payload == nullptr || Payload->IoErrorCode != EIoErrorCode::NotFound)
		{
			return MakeError(SnapshotResult.StealError());
		}
	}

	if (SnapshotResult.HasValue() == false)
	{
		SnapshotResult = MakeValue(FCasSnapshot());
	}

	// Load the snapshot from the journal
	FCasSnapshot BaseSnapshot = SnapshotResult.StealValue();
	SnapshotResult = FCasSnapshot::FromJournal(JournalFile, MoveTemp(BaseSnapshot));
	if (SnapshotResult.HasError())
	{
		return MakeError(SnapshotResult.StealError());
	}

	const FString TmpSnapFile		= FCasSnapshot::MakeTempFileName(SnapshotFile);
	const FString TmpJournalFile	= FCasSnapshot::MakeTempFileName(JournalFile);
	
	// Save the snapshot
	int64 SnapshotSize			= -1;
	FCasSnapshot Snapshot		= SnapshotResult.StealValue();
	TResult<int64> SnapshotSaveResult	= FCasSnapshot::Save(Snapshot, TmpSnapFile);
	if (SnapshotSaveResult.HasValue())
	{
		SnapshotSize = SnapshotSaveResult.GetValue();
	}
	else
	{
		return SnapshotSaveResult;
	}

	// Try create a new empty journal 
	if (FResult Result = FCasJournal::Create(TmpJournalFile); Result.HasError())
	{
		return MakeError(Result.StealError());
	}

	if (Ifm.Move(*SnapshotFile, *TmpSnapFile) == false)
	{
		const uint32 LastError = FPlatformMisc::GetLastError();

		return MakeSnapshotError<int64>(
			ECasErrorCode::CreateSnapshotFailed,
			EIoErrorCode::FileMoveFailed,
			FString::Printf(TEXT("Failed to move tmp snapshot file '%s' -> '%s'"), *TmpSnapFile, *SnapshotFile),
			LastError);
	}

	if (Ifm.Move(*JournalFile, *TmpJournalFile) == false)
	{
		const uint32 LastError = FPlatformMisc::GetLastError();

		return MakeSnapshotError<int64>(
			ECasErrorCode::CreateSnapshotFailed,
			EIoErrorCode::FileMoveFailed,
			FString::Printf(TEXT("Failed to move tmp journal file '%s' -> '%s'"), *TmpJournalFile, *JournalFile),
			LastError);
	}

	return MakeValue(SnapshotSize);
}

bool FCasSnapshot::TempFilesExist(const FString& SnapshotFile, const FString& JournalFile)
{
	IFileManager& Ifm = IFileManager::Get();

	const FString TmpSnapFile = MakeTempFileName(SnapshotFile);
	const FString TmpJournalFile = MakeTempFileName(JournalFile);

	if (Ifm.FileExists(*TmpSnapFile))
	{
		return true;
	}

	if (Ifm.FileExists(*TmpJournalFile))
	{
		return true;
	}

	return false;
}

///////////////////////////////////////////////////////////////////////////////
FCas::FCas(EOnDemandInstallCasType InType)
	: CasType(InType)
{}

FResult FCas::Configure(const FOnDemandInstallCasConfig& Config)
{
	using namespace UE::IoStore::OnDemand;

	const uint32 MaxSupportedBlockSize = (256 << 20);

	if (Config.MinBlockSize > MaxSupportedBlockSize)
	{
		return MakeCasError<void>(
			CasType, ECasErrorCode::InitializeFailed, EIoErrorCode::InvalidParameter, FString::Printf(TEXT("MinBlockSize must be less than %i"), MaxSupportedBlockSize));
	}

	if (Config.MaxBlockSize > MaxSupportedBlockSize)
	{
		return MakeCasError<void>(
			CasType, ECasErrorCode::InitializeFailed, EIoErrorCode::InvalidParameter, FString::Printf(TEXT("MaxBlockSize must be less than %i"), MaxSupportedBlockSize));
	}

	if (Config.MinBlockSize > Config.MaxBlockSize)
	{
		return MakeCasError<void>(
			CasType, ECasErrorCode::InitializeFailed, EIoErrorCode::InvalidParameter, FString::Printf(TEXT("MinBlockSize cannot be greater than MaxBlockSize")));
	}

	LastAccessGranularityTicks = FTimespan::FromSeconds(Config.LastAccessGranularitySeconds).GetTicks();
	MinBlockSize = static_cast<uint32>(Config.MinBlockSize);
	MaxBlockSize = static_cast<uint32>(Config.MaxBlockSize);

	return MakeValue();
}

///////////////////////////////////////////////////////////////////////////////
FResult FCas::Initialize(FStringView Directory, bool bDeleteExisting)
{
	using namespace UE::IoStore::OnDemand;

	RootDirectory = Directory;

	Lookup.Empty();
	BlockIds.Empty();

	IFileManager& Ifm = IFileManager::Get();

	if (bDeleteExisting)
	{
		bool bRequireExists = false;
		const bool bTree	= true;

		if (Ifm.DeleteDirectory(*RootDirectory, bRequireExists, bTree) == false)
		{
			return MakeCasError<void>(
				CasType, ECasErrorCode::InitializeFailed, EIoErrorCode::DeleteError, FString::Printf(TEXT("Failed to delete CAS blocks directory '%s'"), *RootDirectory));
		}
	}

	if (Ifm.DirectoryExists(*RootDirectory) == false)
	{
		const bool bTree = true;
		if (Ifm.MakeDirectory(*RootDirectory, bTree) == false)
		{
			return MakeCasError<void>(
				CasType, ECasErrorCode::InitializeFailed, EIoErrorCode::WriteError, FString::Printf(TEXT("Failed to create CAS blocks directory '%s'"), *RootDirectory));
		}
	}

	return MakeValue();
};

FCasLocation FCas::FindChunk(const FCasAddr& ChunkAddr, const UE::FDeferLock* SkipLock) const
{
	const uint32 TypeHash	= GetTypeHash(ChunkAddr);
	{
		FMaybeLock Lock(this, SkipLock);
		if (const FCasLocation* Loc = Lookup.FindByHash(TypeHash, ChunkAddr))
		{
			return *Loc;
		}
	}

	return FCasLocation{};
}

FCasBlockId FCas::CreateBlock()
{
	IPlatformFile&	Ipf = FPlatformFileManager::Get().GetPlatformFile();
	FCasBlockId		Out = FCasBlockId::Invalid;

	UE::TUniqueLock Lock(Mutex);

	for (uint32 Id = 1; Id < MAX_uint32 && !Out.IsValid(); Id++)
	{
		const FCasBlockId BlockId(Id);
		if (BlockIds.Contains(BlockId))
		{
			continue;
		}

		const FString Filename = GetBlockFilename(BlockId);
		if (Ipf.FileExists(*Filename))
		{
			UE_LOGF(LogIoStoreOnDemand, Warning, "Unused CAS block id %u already exists on disk", BlockId.Id);
			continue;
		}

		BlockIds.Add(BlockId);
		Out = BlockId;
	}

	return Out;
}

bool FCas::ContainsBlock(FCasBlockId BlockId) const
{
	UE::TUniqueLock Lock(Mutex);

	return BlockIds.Contains(BlockId);
}

FResult FCas::DeleteBlock(FCasBlockId BlockId, TFunctionRef<void(const FCasAddr&)> OutAddrs)
{
	using namespace UE::IoStore::OnDemand;

	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();
	const FString	Filename = GetBlockFilename(BlockId);

	UE::TDynamicUniqueLock Lock(Mutex, UE::FDeferLock());

	// Wait for pending reads to flush before deleting block
	uint32			StartTimeCycles	= FPlatformTime::Cycles();
	const uint32	WaitTimeMs		= 1000;

	FCasBlockState* BlockState = nullptr;

	for (;;)
	{
		Lock.Lock();

		BlockState = BlockIds.Find(BlockId);

		if (CasType == EOnDemandInstallCasType::MMap)
		{
			if (BlockState && BlockState->HandleCount > 0)
			{
				return MakeCasError<void>(
					CasType, ECasErrorCode::DeleteBlockFailed, EIoErrorCode::FileHandleOpen,
					FString::Printf(TEXT("Cannot delete CAS block %u while it has open mapped handles"), BlockId.Id));
			}

			// Leave mutex locked until it goes out of scope
			break;
		}

		const int32 RequestCount = BlockState ? BlockState->HandleCount : 0;
		if (RequestCount)
		{
			Lock.Unlock();

			if (FPlatformTime::ToMilliseconds(FPlatformTime::Cycles() - StartTimeCycles) > DeleteBlockMaxWaitTimeMs)
			{
				return MakeCasError<void>(
					CasType, ECasErrorCode::DeleteBlockFailed, EIoErrorCode::Timeout, FString::Printf(TEXT("Timed out waiting for pending read(s) when deleting CAS block %u"), BlockId.Id));
			}

			BlockReadsDoneEvent->Wait(WaitTimeMs);
		}
		else
		{
			// Leave mutex locked until it goes out of scope
			break;
		}
	}

	UE_LOGF(LogIoStoreOnDemand, Log, "Deleting CAS block '%ls'", *Filename);
	if (Ipf.DeleteFile(*Filename) == false)
	{
		return MakeCasError<void>(
			CasType, ECasErrorCode::DeleteBlockFailed, EIoErrorCode::DeleteError, FString::Printf(TEXT("Failed to delete CAS block %u"), BlockId.Id));
	}

	if (BlockState && BlockState->PendingDefragEvent)
	{
		BlockState->PendingDefragEvent.GetValue()->Trigger();
	}

	BlockIds.Remove(BlockId);
	for (auto It = Lookup.CreateIterator(); It; ++It)
	{
		if (It->Value.BlockId == BlockId)
		{
			OutAddrs(It->Key);
			It.RemoveCurrent();
		}
	}

	return MakeValue();
}

FString FCas::GetBlockFilename(FCasBlockId BlockId) const
{
	check(BlockId.IsValid());
	const uint32 Id = NETWORK_ORDER32(BlockId.Id);
	FString Hex;
	BytesToHexLower(reinterpret_cast<const uint8*>(&Id), sizeof(int32), Hex);
	TStringBuilder<256> Path;
	FPathViews::Append(Path, RootDirectory, Hex);
	Path << TEXT(".ucas");

	return FString(Path.ToView());
}

TResult<FSharedFileHandle> FCas::OpenRead(FCasBlockId BlockId, bool bAllowReadFromMMapCas)
{
	using namespace UE::IoStore::OnDemand;

	const FString	Filename = GetBlockFilename(BlockId);
	IPlatformFile&	Ipf = FPlatformFileManager::Get().GetPlatformFile();

	if (!bAllowReadFromMMapCas && CasType == EOnDemandInstallCasType::MMap)
	{
		return MakeCasError<FSharedFileHandle>(
			CasType, ECasErrorCode::ReadBlockFailed, EIoErrorCode::InvalidCode,
			FString::Printf(TEXT("MMap CAS Cannot open CAS block '%s' for read"), *Filename));
	}

	UE::TUniqueLock Lock(Mutex);

	FCasBlockState* BlockState = BlockIds.Find(BlockId);
	if (BlockState == nullptr)
	{
		return MakeCasError<FSharedFileHandle>(
			CasType, ECasErrorCode::ReadBlockFailed, EIoErrorCode::NotFound, FString::Printf(TEXT("Could not find block state for '%s'"), *Filename));
	}

	FFileOpenResult Result = Ipf.OpenRead(*Filename, IPlatformFile::EOpenReadFlags::AllowWrite);
	if (Result.HasValue())
	{
		BlockState->HandleCount++;

		FSharedFileHandle NewHandle(
			Result.GetValue().Release(),
			[this, BlockId](IFileHandle* RawHandle)
			{
				delete RawHandle;
				OnFileHandleDeleted(BlockId);
			}
		);

		return MakeValue(MoveTemp(NewHandle));
	}

	return MakeCasError<FSharedFileHandle>(
		CasType, ECasErrorCode::ReadBlockFailed, EIoErrorCode::FileOpenFailed,
		FString::Printf(TEXT("Failed to open CAS block '%s'"), *Filename),
		Result.StealError());
}

TResult<FSharedAsyncFileHandle> FCas::OpenAsyncRead(FCasBlockId BlockId)
{
	using namespace UE::IoStore::OnDemand;

	if (CasType == EOnDemandInstallCasType::MMap)
	{
		return MakeCasError<FSharedAsyncFileHandle>(
			CasType, ECasErrorCode::ReadBlockFailed, EIoErrorCode::InvalidCode,
			FString::Printf(TEXT("MMap CAS Cannot open CAS block '%s' for async read"), *GetBlockFilename(BlockId)));
	}

	UE::TUniqueLock Lock(Mutex);

	FCasBlockState* BlockState = BlockIds.Find(BlockId);
	if (BlockState == nullptr)
	{
		return MakeCasError<FSharedAsyncFileHandle>(
			CasType, ECasErrorCode::ReadBlockFailed, EIoErrorCode::NotFound, FString::Printf(TEXT("Could not find block state for '%s'"), *GetBlockFilename(BlockId)));
	}

	if (FWeakAsyncFileHandle* MaybeHandle = BlockState->WeakHandle.TryGet<FWeakAsyncFileHandle>())
	{
		if (FSharedAsyncFileHandle Handle = MaybeHandle->Pin(); Handle.IsValid())
		{
			return MakeValue(MoveTemp(Handle));
		}
	}

	const FString			Filename = GetBlockFilename(BlockId);

	IPlatformFile&			Ipf = FPlatformFileManager::Get().GetPlatformFile();
	FFileOpenAsyncResult	HandleResult(Ipf.OpenAsyncRead(*Filename, IPlatformFile::EOpenReadFlags::AllowWrite));

	if (HandleResult.HasValue())
	{
		BlockState->HandleCount++;

		FSharedAsyncFileHandle NewHandle(
			HandleResult.GetValue().Release(),
			[this, BlockId](IAsyncReadFileHandle* RawHandle)
			{
				delete RawHandle;
				OnFileHandleDeleted(BlockId);
			}
		);
		BlockState->WeakHandle.Set<FWeakAsyncFileHandle>(NewHandle);
		
		return MakeValue(MoveTemp(NewHandle));
	}

	return MakeCasError<FSharedAsyncFileHandle>(
		CasType, ECasErrorCode::ReadBlockFailed, EIoErrorCode::FileOpenFailed,
		FString::Printf(TEXT("Failed to open async CAS block '%s'"), *Filename),
		HandleResult.StealError());
}

TResult<FSharedMappedFileHandle> FCas::OpenMapped(FCasBlockId BlockId, int64 MaxSize, TOptional<FSharedEventRef>* OutBlockDeletedEvent, const UE::FDeferLock* SkipLock)
{
	using namespace UE::IoStore::OnDemand;

	if (CasType != EOnDemandInstallCasType::MMap)
	{
		return MakeCasError<FSharedMappedFileHandle>(
			CasType, ECasErrorCode::OpenMappedFailed, EIoErrorCode::InvalidCode,
			FString::Printf(TEXT("%s CAS Cannot open CAS block '%s' for mapped read"), LexToString(CasType), *GetBlockFilename(BlockId)));
	}

	FMaybeLock Lock(this, SkipLock);

	FCasBlockState* BlockState = BlockIds.Find(BlockId);
	if (BlockState == nullptr)
	{
		return MakeCasError<FSharedMappedFileHandle>(
			CasType, ECasErrorCode::OpenMappedFailed, EIoErrorCode::NotFound, FString::Printf(TEXT("Could not find block state for '%s'"), *GetBlockFilename(BlockId)));
	}

	// We can't open a mapped block if it is being defragged
	if (BlockState->PendingDefragEvent)
	{
		if (OutBlockDeletedEvent)
		{
			*OutBlockDeletedEvent = BlockState->PendingDefragEvent;
		}

		return MakeCasError<FSharedMappedFileHandle>(
			CasType, ECasErrorCode::OpenMappedFailed, EIoErrorCode::PendingDelete,
			FString::Printf(TEXT("Cannot delete CAS block %u while it is being defragged"), BlockId.Id));
	}

	if (FWeakMappedFileHandle* MaybeHandle = BlockState->WeakHandle.TryGet<FWeakMappedFileHandle>())
	{
		if (FSharedMappedFileHandle Handle = MaybeHandle->Pin(); Handle.IsValid())
		{
			// If the block has grown since this handle was cached, open a new one
			// 
			// On windows this could be more optimal by reusing the underlying file handle and 
			// just making a new mapping, but we'd need to add platform level support for that.
			// On Nix platforms, no mapping mapping really happens until MapRegion and MaxSize is just an
			// an assert - so we could also re-use the file handle.
			//
			// We should consider adding some kind of Remap() API to reuse the same platform file handle but with
			// a new IMappedFileHandle instead of opening a new platform handle every time.
			if (Handle->GetFileSize() >= MaxSize)
			{
				return MakeValue(MoveTemp(Handle));
			}
		}
	}

	const FString			Filename = GetBlockFilename(BlockId);

	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();
	FOpenMappedResult HandleResult(Ipf.OpenMappedEx2(*Filename, 
		IPlatformFile::EOpenMappedFlags::OpenExisting | IPlatformFile::EOpenMappedFlags::AllowWrite));

	if (HandleResult.HasValue())
	{
		BlockState->HandleCount++;

		//UE_LOGF(LogIoStoreOnDemand, Warning, "IAD Mapped Handle Opened: %x, %d", HandleResult.GetValue().Get(), BlockId.Id);

		FSharedMappedFileHandle NewHandle(
			HandleResult.GetValue().Release(),
			[this, BlockId](IMappedFileHandle* RawHandle)
			{
				delete RawHandle;
				OnFileHandleDeleted(BlockId);
				//UE_LOGF(LogIoStoreOnDemand, Warning, "IAD Mapped Handle Deleted: %x, %d", RawHandle, BlockId.Id);
			}
		);

		check(NewHandle->GetFileSize() >= MaxSize);

		BlockState->WeakHandle.Set<FWeakMappedFileHandle>(NewHandle);

		return MakeValue(MoveTemp(NewHandle));
	}

	return MakeCasError<FSharedMappedFileHandle>(
		CasType, ECasErrorCode::OpenMappedFailed, EIoErrorCode::FileOpenFailed,
		FString::Printf(TEXT("Failed to open mapped CAS block '%s'"), *Filename),
		HandleResult.StealError());
}

void FCas::OnFileHandleDeleted(FCasBlockId BlockId)
{
	UE::TUniqueLock Lock(Mutex);
	const int32 Count = --BlockIds.FindChecked(BlockId).HandleCount;
	check(Count >= 0);
	if (Count == 0)
	{
		BlockReadsDoneEvent->Trigger();
	}
}

bool FCas::SetPendingDefrag(FCasBlockId BlockId, const UE::FDeferLock* SkipLock)
{
	if (CasType != EOnDemandInstallCasType::MMap)
	{
		return true;
	}

	FMaybeLock Lock(this, SkipLock);

	FCasBlockState* BlockState = BlockIds.Find(BlockId);
	if (!ensureMsgf(BlockState, TEXT("Expected block to exist during defrag!")))
	{
		return false;
	}

	if (BlockState->HandleCount > 0)
	{
		return false;
	}

	BlockState->PendingDefragEvent.Emplace(EEventMode::ManualReset);
	return true;
}

void FCas::ResetAllPendingDefrag(const UE::FDeferLock* SkipLock)
{
	if (CasType != EOnDemandInstallCasType::MMap)
	{
		return;
	}

	FMaybeLock Lock(this, SkipLock);

	for (TPair<FCasBlockId, FCasBlockState>& Kv : BlockIds)
	{
		FCasBlockState& BlockState = Kv.Value;
		if (BlockState.PendingDefragEvent)
		{
			// Make sure no thread are still waiting on this
			BlockState.PendingDefragEvent.GetValue()->Trigger();
			BlockState.PendingDefragEvent.Reset();
		}
	}
}

TResult<FUniqueFileHandle> FCas::OpenWrite(FCasBlockId BlockId, const bool bAppend) const
{
	using namespace UE::IoStore::OnDemand;

	IPlatformFile&	Ipf = FPlatformFileManager::Get().GetPlatformFile();
	const FString	Filename = GetBlockFilename(BlockId);

	IPhysicalPlatformFile::EOpenWriteFlags OpenWriteFlags = IPhysicalPlatformFile::EOpenWriteFlags::AllowRead;
	if (bAppend)
	{
		OpenWriteFlags |= IPhysicalPlatformFile::EOpenWriteFlags::Append;
	}

	FFileOpenResult OpenWriteResult(Ipf.OpenWrite(*Filename, OpenWriteFlags));
	if (OpenWriteResult.HasValue())
	{
		return MakeValue(OpenWriteResult.StealValue());
	}

	return MakeCasError<FUniqueFileHandle>(
		CasType, ECasErrorCode::WriteBlockFailed, EIoErrorCode::FileOpenFailed,
		FString::Printf(TEXT("Failed to open CAS block '%s' for writing"), *Filename),
		OpenWriteResult.StealError());
}

bool FCas::TrackAccessIf(const ECasTrackAccessType Type, const FCasBlockId BlockId, const int64 UtcTicks, const bool bDirty, const UE::FDeferLock* SkipLock)
{
	const uint32 BlockIdHash = GetTypeHash(BlockId);

	FMaybeLock Lock(this, SkipLock);

	return UnlockedTrackAccessIf(Type, BlockIdHash, BlockId, UtcTicks, bDirty);
}

bool FCas::UnlockedTrackAccessIf(const ECasTrackAccessType Type, const uint32 BlockIdHash, const FCasBlockId BlockId, int64 UtcTicks, bool bDirty)
{
	check(BlockId.IsValid());

	FCasBlockState* BlockState = BlockIds.FindByHash(BlockIdHash, BlockId);
	if (ensure(BlockState) == false)
	{
		return false;
	}

	int64& FoundTicks = BlockState->LastAccessUtcTicks;

	if (FoundTicks == 0)
	{
		if (bDirty)
		{
			UtcTicks |= DirtyTimestampMask;
		}

		FoundTicks = UtcTicks;

		return true;
	}

	const int64 PrevTicks = (FoundTicks & ~DirtyTimestampMask);
	bDirty = bDirty || (FoundTicks & DirtyTimestampMask); // Don't clear dirty flag if already set

	bool bUpdate = true;
	switch (Type)
	{
	case ECasTrackAccessType::Newer:
		bUpdate = PrevTicks < UtcTicks;
		break;

	case ECasTrackAccessType::Granular:
		bUpdate = PrevTicks < UtcTicks && (UtcTicks - PrevTicks > LastAccessGranularityTicks);
		break;
	}

	if (bUpdate)
	{
		if (bDirty)
		{
			UtcTicks |= DirtyTimestampMask;
		}

		FoundTicks = UtcTicks;
		return true;
	}

	return false;
}

bool FCas::TrackModificationIf(ECasTrackModificationType Type, FCasBlockId BlockId, int64 UtcTicks)
{
	const uint32 BlockIdHash = GetTypeHash(BlockId);

	UE::TUniqueLock Lock(Mutex);

	return UnlockedTrackModificationIf(Type, BlockIdHash, BlockId, UtcTicks);
}

bool FCas::UnlockedTrackModificationIf(ECasTrackModificationType Type, uint32 BlockIdHash, FCasBlockId BlockId, int64 UtcTicks)
{
	check(BlockId.IsValid());

	FCasBlockState* BlockState = BlockIds.FindByHash(BlockIdHash, BlockId);
	if (ensure(BlockState) == false)
	{
		return false;
	}

	int64& FoundTicks = BlockState->LastModificationUtcTicks;
	const int64 PrevTicks = FoundTicks;
	
	bool bUpdate = true;
	switch (Type)
	{
	case ECasTrackModificationType::Newer:
		bUpdate = PrevTicks < UtcTicks;
		break;
	}

	if (bUpdate)
	{
		FoundTicks = UtcTicks;
		return true;
	}

	return false;
}

void FCas::UpdateBlock(
	FCasBlockId BlockId, 
	uint32 BytesWritten, 
	TConstArrayView<FCasAddr> ChunkAddrs,
	TConstArrayView<int64> ChunkOffsets,
	TFunctionRef<void(const FCasAddr&, const FCasLocation&)> OutAddrs)
{
	check(ChunkAddrs.Num() == ChunkOffsets.Num());
	check(BlockId.IsValid());

	TUniqueLock Lock(Mutex);

	BlockIds.FindChecked(BlockId).Size += BytesWritten;

	for (int32 Idx = 0, Count = ChunkOffsets.Num(); Idx < Count; ++Idx)
	{
		const FCasAddr&	ChunkAddr = ChunkAddrs[Idx];
		const uint32	ChunkOffset = IntCastChecked<uint32>(ChunkOffsets[Idx]);

		FCasLocation& Loc = Lookup.FindOrAdd(ChunkAddr);
		Loc.BlockId = BlockId;
		Loc.BlockOffset = ChunkOffset;

		OutAddrs(ChunkAddr, Loc);
	}
}

uint64 FCas::GetBlockInfo(FCasBlockInfoMap& OutBlockInfo) const
{
	const int64	TimestampForMissingLastAccess = FCas::GetTimestampForMissingLastAccess();

	uint64 TotalSize = 0;

	{
		TUniqueLock Lock(Mutex);

		OutBlockInfo.Empty(BlockIds.Num());
		for (const TPair<FCasBlockId, FCasBlockState>& Kv : BlockIds)
		{
			const FCasBlockId BlockId = Kv.Key;
			const uint32 FileSize = Kv.Value.Size;
			const int64 LastAccessUtcTicks = Kv.Value.LastAccessUtcTicks;
			const int64 LastModification = Kv.Value.LastModificationUtcTicks;

			OutBlockInfo.Add(BlockId, FCasBlockInfo
			{
				.FileSize = uint64(FileSize),
				.LastAccess = LastAccessUtcTicks ? (LastAccessUtcTicks & ~FCas::DirtyTimestampMask) : TimestampForMissingLastAccess,
				.LastModification = LastModification
			});

			TotalSize += uint64(FileSize);
		}
	}

	return TotalSize;
}

uint64 FCas::GetTotalBlockSize() const
{
	uint64 TotalSize = 0;

	TUniqueLock Lock(Mutex);

	for (const TPair<FCasBlockId, FCasBlockState>& Kv : BlockIds)
	{
		const uint32 FileSize = Kv.Value.Size;
		TotalSize += uint64(FileSize);
	}

	return TotalSize;
}

FCas::FBlockFileSizes FCas::FindAllBlockFiles() const
{
	struct FDirectoryVisitor final
		: public IPlatformFile::FDirectoryStatVisitor
	{
		FDirectoryVisitor(FBlockFileSizes& InBlockFileSizes)
			: BlockFileSizes(InBlockFileSizes)
		{
		}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, const FFileStatData& StatData) override
		{
			if (StatData.bIsDirectory)
			{
				return true;
			}

			const FStringView Filename(FilenameOrDirectory);
			if (FPathViews::GetExtension(Filename) == TEXTVIEW("ucas") == false)
			{
				return true;
			}

			const FStringView	IndexHex = FPathViews::GetBaseFilename(Filename);
			const FCasBlockId	BlockId(FParse::HexNumber(WriteToString<128>(IndexHex).ToString()));

			if (BlockId.IsValid() == false || StatData.FileSize < 0)
			{
				UE_LOG(LogIoStoreOnDemand, Warning, TEXT("Found invalid CAS block '%s', FileSize=%" INT64_FMT),
					FilenameOrDirectory, StatData.FileSize);
				return true;
			}

			if (BlockFileSizes.Contains(BlockId))
			{
				UE_LOGF(LogIoStoreOnDemand, Warning, "Found duplicate CAS block '%ls'", FilenameOrDirectory);
				return true;
			}

			BlockFileSizes.Add(BlockId, uint64(StatData.FileSize));

			return true;
		}

		FBlockFileSizes&	BlockFileSizes;
	};

	FBlockFileSizes OutBlockFileSizes;

	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();
	FDirectoryVisitor Visitor(OutBlockFileSizes);
	Ipf.IterateDirectoryStat(*RootDirectory, Visitor);

	return OutBlockFileSizes;
}

void FCas::Compact()
{
	UE::TUniqueLock Lock(Mutex);
	Lookup.Compact();
	BlockIds.Compact();
}

FResult FCas::Verify(TArray<FCasAddr>& OutRemovedChunksAddrs, TMap<FCasBlockId, uint64>& OutOverBudgetBlocks)
{
	using namespace UE::IoStore::OnDemand;

	const FBlockFileSizes	BlockFiles = FindAllBlockFiles();
	uint64					TotalVerifiedBytes = 0;
	FResult					Result = MakeValue();

	const int64 Now = FDateTime::UtcNow().GetTicks();

	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();

	auto VerifyTimestamp = [this, &Ipf, Now](
		int64& InOutUtcTicks, const FCasBlockId BlockId, const TCHAR* TimestampName)
	{
		const int64 TimestampValueIfMissing = FCas::GetTimestampForMissingLastAccess();
		const int64 MinimumTimestamp = FCas::GetMinimumTimestamp();

		const FString Filename = FCas::GetBlockFilename(BlockId);

		bool bTimestampOk = false;
		if (InOutUtcTicks == 0)
		{
			UE_LOGF(LogIoStoreOnDemand, Warning, "Found missing %ls time for CAS block '%ls'", TimestampName, *Filename);
		}
		else if (InOutUtcTicks < 0)
		{
			UE_LOGF(LogIoStoreOnDemand, Warning, "Found negative %ls time for CAS block '%ls'", TimestampName, *Filename);
		}
		else if (InOutUtcTicks < MinimumTimestamp)
		{
			UE_LOGF(LogIoStoreOnDemand, Warning, "Found invalid %ls time for CAS block '%ls'", TimestampName, *Filename);
		}
		else if (InOutUtcTicks > Now)
		{
			UE_LOGF(LogIoStoreOnDemand, Warning, "Found future %ls time for CAS block '%ls'", TimestampName, *Filename);
			InOutUtcTicks = Now;
			bTimestampOk = true;
		}
		else
		{
			bTimestampOk = true;
		}

		if (bTimestampOk)
		{
			return;
		}

	#if UE_ONDEMANDINSTALLCACHE_USE_MODTIME
		FDateTime ModTime = Ipf.GetTimeStamp(*Filename);
		if (ensure(ModTime > FDateTime::MinValue()))
		{
			if (ModTime < MinimumTimestamp)
			{
				UE_LOGF(LogIoStoreOnDemand, Warning, "Unusable %ls time and found invalid file mod time for CAS block '%ls'", TimestampName, *Filename);
				InOutUtcTicks = TimestampValueIfMissing;
			}
			else if (ModTime > Now)
			{
				UE_LOGF(LogIoStoreOnDemand, Warning, "Unusable %ls time and found future file mod time so using now time for CAS block '%ls'", TimestampName, *Filename);
				InOutUtcTicks = Now;
			}
			else
			{
				UE_LOGF(LogIoStoreOnDemand, Warning, "Unusable %ls time so using file mod time %ls for CAS block '%ls'", TimestampName, *ModTime.ToString(), *Filename);
				InOutUtcTicks = ModTime.GetTicks();
			}
		}
		else
	#endif // UE_ONDEMANDINSTALLCACHE_USE_MODTIME
		{
			// Failed to get mod time
			UE_LOGF(LogIoStoreOnDemand, Warning, "Unusable %ls time for CAS block '%ls'", TimestampName, *Filename);
			InOutUtcTicks = TimestampValueIfMissing;
		}
	};

	UE::TUniqueLock Lock(Mutex);

	for (auto BlockIt = BlockIds.CreateIterator(); BlockIt; ++BlockIt)
	{
		const FCasBlockId BlockId = BlockIt->Key;
		FCasBlockState& BlockState = BlockIt->Value;

		FString ErrorMessage;

		if (const uint64* FileSize = BlockFiles.Find(BlockId))
		{
			VerifyTimestamp(BlockState.LastAccessUtcTicks,			BlockId, TEXT("last access"));
			VerifyTimestamp(BlockState.LastModificationUtcTicks,	BlockId, TEXT("modification"));

			if (*FileSize > GetMaxBlockSize())
			{
				OutOverBudgetBlocks.Add(BlockId, *FileSize);
			}

			if (*FileSize <= std::numeric_limits<uint32>::max())
			{
				BlockState.Size = uint32(*FileSize);
				TotalVerifiedBytes += *FileSize;
				continue;
			}

			const FString Filename = GetBlockFilename(BlockId);
			ErrorMessage = FString::Printf(TEXT("Invalid file size %" UINT64_FMT " for CAS block '%s'"), *FileSize, *Filename);
			UE_LOGF(LogIoStoreOnDemand, Warning, "%ls", *ErrorMessage);
		}
		else
		{
			const FString Filename = GetBlockFilename(BlockId);
			ErrorMessage = FString::Printf(TEXT("Missing CAS block '%s'"), *Filename);
			UE_LOGF(LogIoStoreOnDemand, Warning, "%ls", *ErrorMessage);
		}

		BlockIt.RemoveCurrent();
		Result = MakeCasError<void>(CasType, ECasErrorCode::VerifyFailed, EIoErrorCode::NotFound, MoveTemp(ErrorMessage));
	}

	UE_LOGF(LogIoStoreOnDemand, Log, "Verified %d CAS blocks of total %.2lf MiB",
		BlockIds.Num(), ToMiB(TotalVerifiedBytes));

	for (const TPair<FCasBlockId, uint64>& Kv : BlockFiles)
	{
		const FCasBlockId BlockId = Kv.Key;
		if (BlockIds.Contains(BlockId))
		{
			continue;
		}

		const FString Filename = GetBlockFilename(BlockId);
		if (Ipf.DeleteFile(*Filename))
		{
			UE_LOGF(LogIoStoreOnDemand, Warning, "Deleted orphaned CAS block '%ls'", *Filename);
		}
	}

	TSet<FString> MissingReferencedBlocks;
	for (auto It = Lookup.CreateIterator(); It; ++It)
	{
		if (!BlockIds.Contains(It->Value.BlockId))
		{
			FString Filename		= GetBlockFilename(It->Value.BlockId);
			FString ErrorMessage	= FString::Printf(TEXT("Missing CAS block '%s'"), *Filename);
			MissingReferencedBlocks.Add(MoveTemp(Filename));

			OutRemovedChunksAddrs.Add(It->Key);
			It.RemoveCurrent();

			Result = MakeCasError<void>(CasType, ECasErrorCode::VerifyFailed, EIoErrorCode::NotFound, MoveTemp(ErrorMessage));
		}
	}

	for (const FString& Filename : MissingReferencedBlocks)
	{
		UE_LOGF(LogIoStoreOnDemand, Warning, "Lookup references missing CAS block '%ls'", *Filename);
	}

	return Result;
}

FCas::FLastAccess FCas::GetAndClearDirtyLastAccess()
{
	FCas::FLastAccess DirtyLastAccess;
	{
		TUniqueLock Lock(Mutex);
		for (TPair<FCasBlockId, FCasBlockState>& Kv : BlockIds)
		{
			int64& Timestamp = Kv.Value.LastAccessUtcTicks;
			if (Timestamp & FCas::DirtyTimestampMask)
			{
				DirtyLastAccess.Add(Kv.Key, Timestamp);
				Timestamp &= ~FCas::DirtyTimestampMask;
			}
		}
	}

	return DirtyLastAccess;
}

int64 FCas::GetTimestampForMissingLastAccess()
{
	const FDateTime Now = FDateTime::UtcNow();
	const FTimespan FourWeeks = FTimespan::FromDays(4*7);
	const FDateTime FourWeeksAgo = Now - FourWeeks;
	return FourWeeksAgo.GetTicks();
}

int64 FCas::GetMinimumTimestamp()
{
	// FCas didn't exist prior to 2024.4.24 so no date before then is valid.
	FDateTime MinTime = FDateTime(2024, 4, 24);
	return MinTime.GetTicks();
}

///////////////////////////////////////////////////////////////////////////////
static void JournalLastAccess(FCasJournal::FTransaction& Transaction, const FCas::FLastAccess& LastAccess)
{
	for (const TPair<FCasBlockId, int64>& Kv : LastAccess)
	{
		int64 Timestamp = Kv.Value;
		if (Timestamp & FCas::DirtyTimestampMask)
		{
			Transaction.BlockAccess(Kv.Key, Timestamp & ~FCas::DirtyTimestampMask);
		}
	}
}

///////////////////////////////////////////////////////////////////////////////
void FCas::LoadSnapshot(FCasSnapshot::FCasState&& Snapshot)
{
	TUniqueLock Lock(Mutex);

	Lookup.Reserve(Snapshot.ChunkLocations.Num());
	for (TPair<FCasAddr, FCasLocation>& Kv : Snapshot.ChunkLocations)
	{
		Lookup.Add(MoveTemp(Kv));
	}

	BlockIds.Reserve(Snapshot.Blocks.Num());
	for (const FCasSnapshot::FBlock& Block : Snapshot.Blocks)
	{
		FCasBlockState& BlockState = BlockIds.Add(Block.BlockId);
		BlockState.LastAccessUtcTicks = Block.LastAccess;
		BlockState.LastModificationUtcTicks = Block.ModTime;
	}
}

///////////////////////////////////////////////////////////////////////////////
FResult FCas::ApplyJournalEntry(const FCasJournal::FEntry& JournalEntry)
{
	switch (JournalEntry.Type())
	{
	case FCasJournal::FEntry::EType::ChunkLocation:
	{
		const FCasJournal::FEntry::FChunkLocation& ChunkLocation = JournalEntry.ChunkLocation;
		if (ChunkLocation.CasLocation.IsValid())
		{
			FCasLocation& Loc = Lookup.FindOrAdd(ChunkLocation.CasAddr);
			Loc = ChunkLocation.CasLocation;
		}
		else
		{
			Lookup.Remove(ChunkLocation.CasAddr);
		}
		break;
	}
	case FCasJournal::FEntry::EType::BlockCreated:
	{
		const FCasJournal::FEntry::FBlockOperation& Op = JournalEntry.BlockOperation;
		BlockIds.Add(Op.BlockId);
		break;
	}
	case FCasJournal::FEntry::EType::BlockDeleted:
	{
		const FCasJournal::FEntry::FBlockOperation& Op = JournalEntry.BlockOperation;
		BlockIds.Remove(Op.BlockId);
		break;
	}
	case FCasJournal::FEntry::EType::BlockAccess:
	{
		const FCasJournal::FEntry::FBlockOperation& Op = JournalEntry.BlockOperation;
		constexpr const bool bDirty = false;

		if (ensure(TrackAccessIf(ECasTrackAccessType::Always, Op.BlockId, Op.UtcTicks, bDirty)) == false)
		{
			using namespace UE::IoStore::OnDemand;
			return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::FileCorrupt, TEXT("Journal BlockAccess Op references an invalid block"));
		}
		break;
	}
	case FCasJournal::FEntry::EType::BlockModification:
	{
		const FCasJournal::FEntry::FBlockOperation& Op = JournalEntry.BlockOperation;
		if (ensure(TrackModificationIf(ECasTrackModificationType::Always, Op.BlockId, Op.UtcTicks)) == false)
		{
			using namespace UE::IoStore::OnDemand;
			return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::FileCorrupt, TEXT("Journal BlockModification Op references an invalid block"));
		}
	}
	};

	return MakeValue();
}

///////////////////////////////////////////////////////////////////////////////
FInstallCacheHandle::~FInstallCacheHandle() = default;

class FInstallCacheHandleImpl : public FInstallCacheHandle
{
public:
	TStaticArray<TArray<FEagerDefragChunkList>, EOnDemandInstallCasType::Count> EagerDefragCandidateChunks;
	TStaticArray<bool, EOnDemandInstallCasType::Count> bUsed{ InPlace, false };
};

///////////////////////////////////////////////////////////////////////////////
class FOnDemandInstallCache final
	: public IOnDemandInstallCache 
{
	using FSharedBackendContextRef	= TSharedRef<const FIoDispatcherBackendContext>;
	using FSharedBackendContext		= TSharedPtr<const FIoDispatcherBackendContext>;

	struct FChunkRequest
	{
		explicit FChunkRequest(
			FIoRequestImpl* Request,
			FOnDemandChunkInfo&& Info,
			FIoOffsetAndLength Range,
			uint64 RequestedRawSize)
				: DispatcherRequest(Request)
				, ChunkInfo(MoveTemp(Info))
				, EncodedChunk(Range.GetLength())
				, RawSize(RequestedRawSize)
				, ChunkRange(Range)
		{
			check(DispatcherRequest != nullptr);
			check(ChunkInfo.IsValid());
			check(Request->NextRequest == nullptr);
			check(Request->BackendData == nullptr);
		}

		~FChunkRequest()
		{
			if (bMadeCompleteEvent)
			{
				using namespace UE::Tasks;
				CompleteReadRequestTrigger.~FTaskEvent();
			}
		}

		static FChunkRequest* Get(FIoRequestImpl& Request)
		{
			return static_cast<FChunkRequest*>(Request.BackendData);
		}

		static FChunkRequest& GetRef(FIoRequestImpl& Request)
		{
			check(Request.BackendData);
			return *static_cast<FChunkRequest*>(Request.BackendData);
		}

		static FChunkRequest& Attach(FIoRequestImpl& Request, FChunkRequest* ChunkRequest)
		{
			check(Request.BackendData == nullptr);
			check(ChunkRequest != nullptr);
			Request.BackendData = ChunkRequest;
			return *ChunkRequest;
		}

		static TUniquePtr<FChunkRequest> Detach(FIoRequestImpl& Request)
		{
			void* ChunkRequest = nullptr;
			Swap(ChunkRequest, Request.BackendData);
			return TUniquePtr<FChunkRequest>(static_cast<FChunkRequest*>(ChunkRequest));
		}

		UE::Tasks::FTaskEvent& GetCompleteReadRequestTrigger()
		{
			if (!bMadeCompleteEvent)
			{
				bMadeCompleteEvent = true;
				new(&CompleteReadRequestTrigger) UE::Tasks::FTaskEvent{ UE_SOURCE_LOCATION };
			}

			return CompleteReadRequestTrigger;
		}

private:
		// Don't make this until we need it - the task system will assert if an uncompleted task is destroyed
		union { UE::Tasks::FTaskEvent CompleteReadRequestTrigger; };

public:
		FSharedAsyncFileHandle			SharedFileHandle;
		TUniquePtr<IAsyncReadRequest>	FileReadRequest;
		FIoRequestImpl*					DispatcherRequest = nullptr;
		FOnDemandChunkInfo				ChunkInfo;
		FIoBuffer						EncodedChunk;
		uint64							RawSize = 0;
		FIoOffsetAndLength				ChunkRange;
		EOnDemandInstallCasType			CasType = EOnDemandInstallCasType::None;
private:
		bool							bMadeCompleteEvent = false;
	};

	struct FPendingChunks
	{
		static constexpr uint64 MaxPendingBytes = 4ull << 20;

		bool IsEmpty() const
		{
			check(Chunks.Num() == ChunkAddrs.Num());
			return TotalSize == 0 && Chunks.IsEmpty() && ChunkAddrs.IsEmpty();
		}

		void Append(FIoBuffer&& Chunk, const FCasAddr& ChunkAddr)
		{
			check(Chunks.Num() == ChunkAddrs.Num());
			TotalSize += Chunk.GetSize();
			ChunkAddrs.Add(ChunkAddr);
			Chunks.Add(MoveTemp(Chunk));
		}

		void Reset()
		{
			Chunks.Reset();
			ChunkAddrs.Reset();
			TotalSize = 0;
		}

		const FIoBuffer* FindChunk(const FCasAddr& ChunkAddr) const
		{
			const int32 PendingIdx = ChunkAddrs.IndexOfByKey(ChunkAddr);
			return (PendingIdx == INDEX_NONE) ? nullptr : &Chunks[PendingIdx];
		}

		bool ContainsChunk(const FCasAddr& ChunkAddr) const
		{
			return ChunkAddrs.Contains(ChunkAddr);
		}

		TArray<FIoBuffer>		Chunks;
		TArray<FCasAddr>		ChunkAddrs;
		uint64					TotalSize = 0;
		mutable FSharedMutex	SharedMutex;
	};

	struct FCacheState
	{
		FCacheState(EOnDemandInstallCasType InType) : Cas(InType) {}

		FCas						Cas;
		FPendingChunks				PendingChunks;
		std::atomic<FCasBlockId>	CurrentBlock{ FCasBlockId::Invalid };
		FSharedMutex				PurgeDefragMutex;
		uint64						MaxCacheSize{ 0 };
		uint32						EagerDefragBlockCount{ 0 };
#if UE_ONDEMANDINSTALLCACHE_EXCLUSIVE_WRITE
		UE::Tasks::FPipe			ExclusivePipe{ UE_SOURCE_LOCATION };
#endif // UE_ONDEMANDINSTALLCACHE_EXCLUSIVE_WRITE
	};

public:
	FOnDemandInstallCache(const FOnDemandInstallCacheConfig& Config, FString RootDirectory, FOnDemandIoStore& IoStore, FDiskCacheGovernor& Governor);
	virtual ~FOnDemandInstallCache();

	// IIoDispatcherBackend
	virtual void								Initialize(FSharedBackendContextRef Context) override;
	virtual void								Shutdown() override;
	virtual void								ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved) override;
	virtual FIoRequestImpl*						GetCompletedIoRequests() override;
	virtual void								CancelIoRequest(FIoRequestImpl* Request) override;
	virtual void								UpdatePriorityForIoRequest(FIoRequestImpl* Request) override;
	virtual bool								DoesChunkExist(const FIoChunkId& ChunkId) const override;
	virtual TIoStatusOr<uint64>					GetSizeForChunk(const FIoChunkId& ChunkId) const override;
	virtual TIoStatusOr<FIoMappedRegion>		OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options) override;
	virtual const TCHAR*						GetName() const override;

	// IOnDemandInstallCache
	virtual TUniquePtr<FInstallCacheHandle>		BeginInstall() override;
	virtual void								PinCachedChunks(FInstallCacheHandle& Handle, const FOnDemandInternalContentHandle& ContentHandle, const FOnDemandChunkInfoList& Chunks, TFunctionRef<void(int32, bool)> OnChunkFound) override;
	virtual void								PinChunks(FInstallCacheHandle& Handle, const FOnDemandInternalContentHandle& ContentHandle, const FOnDemandChunkInfoList& Chunks, TFunctionRef<void(int32)> OnChunkFound) override;
	virtual FResult								PutChunk(EIoChunkType ChunkType, FIoBuffer&& Chunk, const FCasAddr& ChunkHash) override;
	virtual void								PostPutChunk(FInstallCacheHandle& Handle, EIoChunkType ChunkType) override;
	virtual FInstallCachePurgeHandle			BeginPurge(EIoChunkType ChunkType) override;
	virtual void								AddToPurge(FInstallCachePurgeHandle& Handle, EIoChunkType ChunkType, uint64 Size) override;
	virtual FResult								Purge(const FInstallCachePurgeHandle& Handle) override;
	virtual FResult								PurgeAllUnreferenced(EOnDemandInstallCasType CasType, bool bDefrag, const uint64* BytesToPurge = nullptr) override;
	virtual FResult								DefragAll(EOnDemandInstallCasType CasType, const uint64* BytesToFree = nullptr) override;
	virtual FResult								EagerDefrag(FInstallCacheHandle& Handle) override;
	virtual bool								IsEagerDefragRequired(FInstallCacheHandle& Handle) override;
	virtual FResult								ConditionallyFlushInstall(FInstallCacheHandle& Handle) override;
	virtual FResult								Verify() override;
	virtual FResult								FlushLastAccess(const TStaticArray<bool, EOnDemandInstallCasType::Count>& bFlushCasLastAccess) override;
	virtual void								UpdateLastAccess(
													TConstArrayView<FCasAddr> ChunkAddrs,
													TStaticArray<bool, EOnDemandInstallCasType::Count>& bInOutLastAccessDirty) override;
	virtual FOnDemandInstallCacheUsage			GetCacheUsage() override;

private:
	void										RegisterConsoleCommands();
	FResult										Reset();
	FResult										InitialVerify();
	void										AddReferencesToBlocks(
													const FCacheState& CacheState,
													const TArray<FSharedOnDemandContainer>& Containers, 
													const TArray<TBitArray<>>& ChunkEntryIndices,
													FCasBlockInfoMap& BlockInfoMap,
													uint64& OutTotalReferencedBytes) const;
	FResult										PurgeInternal(
													FCacheState& CacheState, 
													FCasBlockInfoMap& BlockInfo, 
													uint64 TotalBytesToPurge, 
													uint64& OutTotalPurgedBytes);
	FResult										Defrag(
													FCacheState& CacheState,
													const TArray<FSharedOnDemandContainer>& Containers,
													const TArray<TBitArray<>>& ChunkEntryIndices,
													FCasBlockInfoMap& BlockInfo, 
													const uint64* TotalBytesToFree = nullptr,
													uint64* OutTotalFreedBytes = nullptr);
	bool										Resolve(FIoRequestImpl* Request);
	bool										ResolveFromCas(FCacheState& CacheState, FIoRequestImpl* Request);
	void										CompleteRequest(FIoRequestImpl* Request, EIoErrorCode Status);
	FResult										Flush(EOnDemandInstallCasType CasType);
	FResult										FlushPendingChunks(FCacheState& CacheState, FPendingChunks& Chunks, int64 UtcAccessTicks = 0, int64 UtcModificationTicks = 0);
	FResult										FlushPendingChunksImpl(FCacheState& CacheState, const FPendingChunks& Chunks, int64 UtcAccessTicks = 0, int64 UtcModificationTicks = 0);
	FString										GetJournalFilename() const { return CacheDirectory / TEXT("cas.jrn"); }
	FString										GetSnapshotFilename() const { return CacheDirectory / TEXT("cas.snp"); }
	FCacheState*								GetCacheState(EOnDemandInstallCasType CacheType);
	const FCacheState*							GetCacheState(EOnDemandInstallCasType CacheType) const;
	EOnDemandInstallCasType						GetCasType(EIoChunkType ChunkType) const;
	FResult										PurgeInternal(EOnDemandInstallCasType CasType, uint64 BytesToInstall);
	FResult										EagerDefragInternal(EOnDemandInstallCasType CasType, TConstArrayView<FEagerDefragChunkList> ChunkHahses);
	FResult										PutChunkInternal(EOnDemandInstallCasType CasType, FIoBuffer&& Chunk, const FCasAddr& ChunkAddr);
	UE::IoStore::OnDemand::FInstallCacheErrorState MakeInstallCacheErrorState(FCacheState& CacheState, uint64 TotalCachedBytes = 0, uint64 RequestedBytes = 0, uint32 LineNo = __builtin_LINE());

	FOnDemandIoStore&			IoStore;
	FDiskCacheGovernor&			Governor;
	FString						CacheDirectory;
	FSharedBackendContext		BackendContext;
	FIoRequestList				CompletedRequests;
	FMutex						Mutex;
	uint64						MaxJournalSize;

	TArray<FCacheState>			CacheStates;

#if UE_IAD_DEBUG_CONSOLE_CMDS
	TArray<IConsoleCommand*> ConsoleCommands;
#endif // UE_IAD_DEBUG_CONSOLE_CMDS
};

///////////////////////////////////////////////////////////////////////////////
FOnDemandInstallCache::FOnDemandInstallCache(const FOnDemandInstallCacheConfig& Config, FString RootDirectory, FOnDemandIoStore& InIoStore, FDiskCacheGovernor& InGovernor)
	: IoStore(InIoStore)
	, Governor(InGovernor)
	, CacheDirectory(MoveTemp(RootDirectory))
	, MaxJournalSize(Config.JournalMaxSize)
{
	using namespace UE::IoStore::OnDemand;

	const TStaticArray<int32, EOnDemandInstallCasType::Count>& CVarEagerDefragBlockCountByType = CVars::EagerDefragBlockCount();

	FResult InitResult = MakeValue();

	CacheStates.Empty(Config.CasConfig.Num());
	for (const FOnDemandInstallCasConfig& CasConfig : Config.CasConfig)
	{
		if (ensureAlways(CasConfig.Type != EOnDemandInstallCasType::None) == false)
		{
			continue;
		}

		FCacheState& CacheState = CacheStates.Emplace_GetRef(CasConfig.Type);

		CacheState.MaxCacheSize = CasConfig.DiskQuota;

		InitResult = CacheState.Cas.Configure(CasConfig);
		if (InitResult.HasError())
		{
			UE_LOGFMT(LogIoStoreOnDemand, Fatal, "Failed to configure install cache CAS {CasType}, error: {Error}", LexToString(CasConfig.Type), InitResult.GetError());
			return;
		}

		UE_LOGF(LogIoStoreOnDemand, Log, "Initializing install cache CAS %ls, MaxCacheSize=%.2lf MiB, MaxJournalSize=%.2lf KiB, MaxBlockSize=%.2lf MiB, MinBlockSize=%.2lf MiB",
			LexToString(CasConfig.Type), ToMiB(CacheState.MaxCacheSize), ToKiB(MaxJournalSize), ToMiB(CacheState.Cas.GetMaxBlockSize()), ToMiB(CacheState.Cas.GetMinBlockSize()));

		const uint64 MinDiskQuota = 2 * CacheState.Cas.GetMaxBlockSize();
		if (CacheState.MaxCacheSize < MinDiskQuota)
		{
			UE_LOGF(LogIoStoreOnDemand, Fatal, "Failed to initialize install cache CAS %ls - disk quota must be at least %.2lf MiB", LexToString(CasConfig.Type), ToMiB(MinDiskQuota));
			return;
		}

		// Reserve one block of space for defragmentation overhead
		CacheState.MaxCacheSize -= CacheState.Cas.GetMaxBlockSize();
		UE_LOGF(LogIoStoreOnDemand, Log, "Effective MaxCacheSize for CAS %ls without defragmentation space is MaxCacheSize=%.2lf MiB", LexToString(CasConfig.Type), ToMiB(CacheState.MaxCacheSize));

		TStringBuilder<256> CasDirectory;
		FPathViews::Append(CasDirectory, CacheDirectory, CasConfig.CasSubdirectory);

		CacheState.EagerDefragBlockCount = CasConfig.EagerDefragBlockCount;
		const int32 CVarEagerDefragBlockCount = CVarEagerDefragBlockCountByType[CasConfig.Type];

		const uint64 MaxBlockCount = CacheState.MaxCacheSize / CasConfig.MaxBlockSize;
		if (CVarEagerDefragBlockCount >= 0)
		{
			if (MaxBlockCount <= CVarEagerDefragBlockCount)
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "CAS %ls - CVar iostore.EagerDefragBlockCount must be less than the maximum number of blocks the cache can accomodate. Ignoring CVar value.", LexToString(CasConfig.Type));
			}
			else
			{
				CacheState.EagerDefragBlockCount = CVarEagerDefragBlockCount;
			}
		}
		
		if (MaxBlockCount <= CacheState.EagerDefragBlockCount)
		{
			UE_LOGF(LogIoStoreOnDemand, Fatal, "Failed to initialize install cache CAS %ls - EagerDefragBlockCount must be less than the maximum number of blocks the cache can accomodate", LexToString(CasConfig.Type));
			return;
		}

		InitResult = CacheState.Cas.Initialize(CasDirectory);
		if (InitResult.HasError())
		{
			UE_LOGFMT(LogIoStoreOnDemand, Fatal, "Failed to initialize install cache CAS {CasType}, error: {Error}", LexToString(CasConfig.Type), InitResult.GetError());
			return;
		}
	}

	if (CacheStates.IsEmpty())
	{
		UE_LOGF(LogIoStoreOnDemand, Fatal, UTF8TEXT("Failed to configure install cache CAS! No valid configurations!"));
	}

	const FString SnapshotFilename = GetSnapshotFilename();
	const FString JournalFilename = GetJournalFilename();

	// Check for any temp files that indicate we are in an incosistent state
	if (InitResult.HasValue())
	{
		if (FCasSnapshot::TempFilesExist(SnapshotFilename, JournalFilename))
		{
			InitResult = MakeSnapshotError<void>(
				ECasErrorCode::InitializeFailed, EIoErrorCode::FileCorrupt, 
				FString::Printf(TEXT("Last snapshot update is incomplete, snapshot is in an inconsistent state")));
		}
	}

	// Check if the snapshot should be updated
	if (InitResult.HasValue())
	{
		IFileManager& Ifm = IFileManager::Get();
		if (Ifm.FileSize(*JournalFilename) > int64(MaxJournalSize))
		{
			TResult<int64> SnapshotResult = FCasSnapshot::AppendAndResetJournal(SnapshotFilename, JournalFilename);
			if (SnapshotResult.HasValue())
			{
				UE_LOGF(LogIoStoreOnDemand, Log, "Saved CAS snapshot '%ls' %.2lf KiB", *SnapshotFilename, ToKiB(SnapshotResult.GetValue()));
			}
			else
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to update CAS snapshot from journal '%ls', error '%ls'",
					*JournalFilename, *SnapshotResult.GetError().GetErrorMessage(true).ToString());

				InitResult = MakeError(SnapshotResult.StealError());
			}
		}
	}

	// Try read the journal snapshot
	if (InitResult.HasValue())
	{
		int64 SnapshotSize = -1;
		TResult<FCasSnapshot> SnapshotResult = FCasSnapshot::Load(SnapshotFilename, &SnapshotSize);
		if (SnapshotResult.HasValue())
		{
			FCasSnapshot Snapshot = SnapshotResult.StealValue();
			
			UE_LOGF(LogIoStoreOnDemand, Log, "Loaded CAS snapshot '%ls' %.2lf KiB", *SnapshotFilename, ToKiB(SnapshotSize));

			for (FCacheState& CacheState : CacheStates)
			{
				FCasSnapshot::FCasState* SnapshotCasState = Algo::FindBy(Snapshot.CasState, CacheState.Cas.GetType(), &FCasSnapshot::FCasState::CasType);
				if (SnapshotCasState)
				{
					UE_LOGF(LogIoStoreOnDemand, Log, "Loaded %ls CAS state from snapshot with %d blocks and %d chunk locations",
						LexToString(CacheState.Cas.GetType()), SnapshotCasState->Blocks.Num(), SnapshotCasState->ChunkLocations.Num());

					CacheState.CurrentBlock = SnapshotCasState->CurrentBlockId;
					CacheState.Cas.LoadSnapshot(MoveTemp(*SnapshotCasState));
				}
			}
		}
		else if (const FCasSnapshotError* Payload = SnapshotResult.GetError().GetErrorContext<CasSnapshotError>())
		{
			// It's ok for the snapshot to not exist, other errors are significant
			if (Payload->IoErrorCode != EIoErrorCode::NotFound)
			{
				InitResult = MakeError(SnapshotResult.StealError());
			}
		}
	}

	if (InitResult.HasValue())
	{
		// Replay the journal 
		InitResult = FCasJournal::Replay(JournalFilename, [this](const FCasJournal::FEntry& JournalEntry)
		{
			FResult Result = MakeValue();
			FCacheState* CacheState = Algo::FindBy(CacheStates, JournalEntry.CasType(), Projection(&FCacheState::Cas, &FCas::GetType));
			if (CacheState == nullptr)
			{
				// This cache state is not currently configured and there's no way to partially cleanup the cache right now.
				// For now, just issue an error and wipe the cache.
				// TODO: IadMultiCAS: Need a way to reset a cache type if we remove it.
				//		TODO: Scan root dir for cache subdirs that are unknown and delete them.
				//		TODO: When making a snapshot, strip any cache types that aren't currently configured.
				return MakeJournalError(ECasErrorCode::ReplayJournalFailed, EIoErrorCode::InvalidParameter, 
					FString::Printf(TEXT("Journal Entry references an invalid cache type %s"), LexToString(JournalEntry.CasType())));
			}

			Result = CacheState->Cas.ApplyJournalEntry(JournalEntry);
			if (Result.HasError())
			{
				return Result;
			}

			switch(JournalEntry.Type())
			{
			case FCasJournal::FEntry::EType::BlockCreated:
			{
				const FCasJournal::FEntry::FBlockOperation& Op = JournalEntry.BlockOperation;
				CacheState->CurrentBlock = Op.BlockId;
				break;
			}
			case FCasJournal::FEntry::EType::BlockDeleted:
			{
				const FCasJournal::FEntry::FBlockOperation& Op = JournalEntry.BlockOperation;
				FCasBlockId MaybeCurrentBlock = Op.BlockId;
				CacheState->CurrentBlock.compare_exchange_strong(MaybeCurrentBlock, FCasBlockId::Invalid);
				break;
			}
			};

			return Result;
		});
	}

	// If the CAS journal is not found we assume we are initializing the cache for the first time
	if (InitResult.HasError())
	{
		if (const FCasJournalError* Payload = InitResult.GetError().GetErrorContext<CasJournalError>())
		{
			if (Payload->CasErrorCode == ECasErrorCode::ReplayJournalFailed &&
				Payload->IoErrorCode == EIoErrorCode::NotFound)
			{
				if (InitResult = FCasJournal::Create(JournalFilename); InitResult.HasValue())
				{
					UE_LOGF(LogIoStoreOnDemand, Log, "Created CAS journal '%ls'", *JournalFilename);

					for (FCacheState& CacheState : CacheStates)
					{
						// Make sure that there are no existing blocks when starting from an empty cache
						FString CasRootDir = CacheState.Cas.GetRootDirectory();
						const bool bDeleteExisting = true;
						InitResult = CacheState.Cas.Initialize(CasRootDir, bDeleteExisting);
						if (InitResult.HasError())
						{
							break;
						}
					}
				}
			}
		}
	}

	// Verify the current state of the cache
	if (InitResult.HasValue())
	{
		InitResult = InitialVerify();
	}

	// Try to reset the cache if something has gone wrong 
	if (InitResult.HasError())
	{
		UE_LOGFMT(LogIoStoreOnDemand, Error, "Resetting install cache, reason: {Error}", *InitResult.GetError().GetErrorMessage(true).ToString());

		FOnDemandInstallCacheStats::OnStartupError(InitResult);
		InitResult = Reset();
	}

	if (InitResult.HasValue())
	{
		UE_LOGF(LogIoStoreOnDemand, Log, "Install cache Ok!");
		RegisterConsoleCommands();
		for (FCacheState& CacheState : CacheStates)
		{
			CacheState.Cas.Compact();
		}
	}
	else
	{
		UE_LOGFMT(LogIoStoreOnDemand, Fatal, "Failed to initialize install cache, reason: {Error}", *InitResult.GetError().GetErrorMessage(true).ToString());
	}
}

FOnDemandInstallCache::~FOnDemandInstallCache()
{
}

void FOnDemandInstallCache::Initialize(FSharedBackendContextRef Context)
{
	BackendContext = Context;
}

void FOnDemandInstallCache::Shutdown()
{
	for (FCacheState& CacheState : CacheStates)
	{
		if (FResult Result = FlushPendingChunksImpl(CacheState, CacheState.PendingChunks); Result.HasError())
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "Failed to flush pending chunks on shutdown for %ls, reason '%ls'", 
				LexToString(CacheState.Cas.GetType()), *Result.GetError().GetErrorMessage(true).ToString());
		}
	}

	const FString JournalFile = GetJournalFilename();
	for (FCacheState& CacheState : CacheStates)
	{
		FCasJournal::FTransaction Transaction = FCasJournal::Begin(CacheState.Cas.GetType(), JournalFile);

		FCas::FLastAccess LastAccess = CacheState.Cas.GetAndClearDirtyLastAccess();
		JournalLastAccess(Transaction, LastAccess);

		if (FResult Result = FCasJournal::Commit(MoveTemp(Transaction)); Result.HasError())
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "Failed to update CAS journal '%ls' with block timestamp(s), error '%ls'",
				*JournalFile, *LexToString(Result.GetError()));
		}
	}

#if UE_IAD_DEBUG_CONSOLE_CMDS
	for (IConsoleCommand* Cmd : ConsoleCommands)
	{
		IConsoleManager::Get().UnregisterConsoleObject(Cmd);
	}
#endif // UE_IAD_DEBUG_CONSOLE_CMDS
}

void FOnDemandInstallCache::ResolveIoRequests(FIoRequestList Requests, FIoRequestList& OutUnresolved)
{
	while (FIoRequestImpl* Request = Requests.PopHead())
	{
		if (Resolve(Request) == false)
		{
			OutUnresolved.AddTail(Request);
		}
	}
}

FIoRequestImpl* FOnDemandInstallCache::GetCompletedIoRequests()
{
	FIoRequestList LocalCompleted;
	{
		UE::TUniqueLock Lock(Mutex);
		LocalCompleted = MoveTemp(CompletedRequests);
		CompletedRequests = FIoRequestList();
	}

	for (FIoRequestImpl& Completed : LocalCompleted)
	{
		TUniquePtr<FChunkRequest> Detached = FChunkRequest::Detach(Completed);
	}

	return LocalCompleted.GetHead();
}

void FOnDemandInstallCache::CancelIoRequest(FIoRequestImpl* Request)
{
	check(Request != nullptr);
	if (FChunkRequest* ChunkRequest = FChunkRequest::Get(*Request))
	{
		if (ChunkRequest->FileReadRequest.IsValid())
		{
			ChunkRequest->FileReadRequest->Cancel();
		}
	}
}

void FOnDemandInstallCache::UpdatePriorityForIoRequest(FIoRequestImpl* Request)
{
}

bool FOnDemandInstallCache::DoesChunkExist(const FIoChunkId& ChunkId) const
{
	EIoErrorCode ErrorCode = EIoErrorCode::UnknownChunkID;
	if (FOnDemandChunkInfo ChunkInfo = IoStore.GetInstalledChunkInfo(ChunkId, ErrorCode))
	{
		for (const FCacheState& CacheState : CacheStates)
		{
			const bool bAllowReadFromPendingChunks = CacheState.Cas.GetType() != EOnDemandInstallCasType::MMap;
			if (bAllowReadFromPendingChunks)
			{
				TSharedLock SharedLock(CacheState.PendingChunks.SharedMutex);

				if (CacheState.PendingChunks.ContainsChunk(ChunkInfo.Hash()))
				{
					return true;
				}
			}

			const FCasLocation CasLoc = CacheState.Cas.FindChunk(ChunkInfo.Hash());
			if (CasLoc.IsValid())
			{
				return true;
			}
		}
	}

	return false;
}

TIoStatusOr<uint64> FOnDemandInstallCache::GetSizeForChunk(const FIoChunkId& ChunkId) const
{
	EIoErrorCode ErrorCode = EIoErrorCode::UnknownChunkID;
	if (FOnDemandChunkInfo ChunkInfo = IoStore.GetInstalledChunkInfo(ChunkId, ErrorCode))
	{
		return ChunkInfo.RawSize();
	}

	return FIoStatus(ErrorCode);
}

TIoStatusOr<FIoMappedRegion> FOnDemandInstallCache::OpenMapped(const FIoChunkId& ChunkId, const FIoReadOptions& Options)
{
	if (!FPlatformProperties::SupportsMemoryMappedFiles())
	{
		return FIoStatus(EIoErrorCode::Unknown, TEXT("Platform does not support memory mapped files"));
	}

#if UE_ONDEMANDINSTALLCACHE_EXCLUSIVE_WRITE
	return FIoStatus(EIoErrorCode::Unknown, TEXT("OpenMapped is not supported with EXCLUSIVE_WRITE"));
#else

	if (Options.GetTargetVa() != nullptr)
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid read options"));
	}

	EIoErrorCode ErrorCode = EIoErrorCode::UnknownChunkID;
	FOnDemandChunkInfo ChunkInfo = IoStore.GetInstalledChunkInfo(ChunkId, ErrorCode);
	if (ChunkInfo.IsValid() == false)
	{
		if (ErrorCode == EIoErrorCode::NotInstalled)
		{
#if !UE_BUILD_SHIPPING
			if (IOnDemandDevelopmentExtension* Ext = IoStore.TryGetDevelopmentExtension())
			{
				TUtf8StringBuilder<128> Filename;
				TStringBuilder<128> Sb;
				FStringView PackageName = TryConvertChunkIdToPackageName(ChunkId, Filename, Sb);

				if (Ext->GetUnreferencedIoChunkStatus(
					ChunkId,
					Filename.ToView(),
					PackageName,
					IsDevModeEnabled()) != EIoErrorCode::NotInstalled)
				{
					return FIoStatus(EIoErrorCode::NotFound);
				}
			}
#endif
			FOnDemandInstallCacheStats::OnReadCompleted(EOnDemandInstallCasType::None, EIoErrorCode::NotInstalled, 0);
			return FIoStatus(EIoErrorCode::NotInstalled);
		}
		return FIoStatus(EIoErrorCode::NotFound);
	}

	const uint64 RequestSize = FMath::Min<uint64>(
		Options.GetSize(),
		ChunkInfo.RawSize() - Options.GetOffset());

	TIoStatusOr<FIoOffsetAndLength> MaybeChunkRange = FIoChunkEncoding::GetChunkRange(
		ChunkInfo.RawSize(),
		ChunkInfo.BlockSize(),
		ChunkInfo.Blocks(),
		Options.GetOffset(),
		RequestSize);

	if (MaybeChunkRange.IsOk() == false)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to get chunk range for OpenMapped");
		FOnDemandInstallCacheStats::OnReadCompleted(EOnDemandInstallCasType::None, MaybeChunkRange.Status().GetErrorCode(), 0);
		return MaybeChunkRange.Status();
	}

	const FIoOffsetAndLength ChunkRange = MaybeChunkRange.ConsumeValueOrDie();

	// OpenMapped is not async and may be called from any thread so it needs to hold the Cas lock from this point forward to avoid
	// racing with a possible purge or defrag being called from an installer task. Normally, DeleteBlock waits for pending reads to
	// flush but a mapped file handle may be held indefinitely so any block with open mmap handles cannot be deleted.
	TDynamicUniqueLock<FCas> CasUniqueLock;

	FCasLocation CasLoc;
	FCacheState *pCacheState = nullptr;
	for (FCacheState& CacheState : CacheStates)
	{
		TDynamicUniqueLock LocalLock(CacheState.Cas);
		CasLoc = CacheState.Cas.FindChunk(ChunkInfo.Hash(), &UE::DeferLock);
		if (CasLoc.IsValid())
		{
			pCacheState = &CacheState;
			CasUniqueLock = MoveTemp(LocalLock);
			break;
		}
	}

	TResult<FSharedMappedFileHandle> FileOpenResult = MakeError(UE::IoStore::OnDemand::CasError());

	for(;;)
	{
		if (CasLoc.IsValid() == false)
		{
			// Chunk is referenced but not in any cache. This should probably never happen currently.
			// If pinning of unreferenced chunks was allowed, then this would be expected...
			FOnDemandInstallCacheStats::OnReadCompleted(EOnDemandInstallCasType::None, EIoErrorCode::NotInstalled, 0);
			return FIoStatus(EIoErrorCode::NotInstalled, TEXT("IoChunk should be installed but was not found!"));
		}

		// Cas is now locked by CasUniqueLock so skip locking on remaining API calls

		check(pCacheState);
		FCacheState& CacheState = *pCacheState;

		constexpr const bool bDirty = true;
		if (CacheState.Cas.TrackAccessIf(ECasTrackAccessType::Granular, CasLoc.BlockId, bDirty, &UE::DeferLock))
		{
			IoStore.FlushLastAccess({}, nullptr);
		}

		const int64 MaxSize = CasLoc.BlockOffset + ChunkRange.GetOffset() + ChunkRange.GetLength();

		TOptional<FSharedEventRef> BlockDeletedEvent;
		FileOpenResult = CacheState.Cas.OpenMapped(CasLoc.BlockId, MaxSize, &BlockDeletedEvent, &UE::DeferLock);
		if (FileOpenResult.HasError())
		{
			using namespace UE::IoStore::OnDemand;

			if (BlockDeletedEvent)
			{
				CasUniqueLock.Unlock();

				// Wait for delete event from defrag
				BlockDeletedEvent.GetValue()->Wait();

				// Retry
				CasUniqueLock.Lock();
				CasLoc = CacheState.Cas.FindChunk(ChunkInfo.Hash(), &UE::DeferLock);
				continue;
			}

			const FString Filename = CacheState.Cas.GetBlockFilename(CasLoc.BlockId);
			UE_LOGF(LogIoStoreOnDemand, Error, "Failed to open CAS block '%ls' in CAS %ls for mapped reading, reason '%ls'",
				*Filename, LexToString(CacheState.Cas.GetType()), *FileOpenResult.GetError().GetErrorMessage(true).ToString());

			if (const FCasError* Payload = FileOpenResult.GetError().GetErrorContext<CasError>())
			{
				FIoStatus Status(Payload->IoErrorCode, Payload->SystemErrorCode, *LexToString(FileOpenResult.GetError()));
				FOnDemandInstallCacheStats::OnReadCompleted(CacheState.Cas.GetType(), Status.GetErrorCode(), 0);
				return Status;
			}

			FIoStatus Status(EIoErrorCode::FileOpenFailed, *LexToString(FileOpenResult.GetError()));
			FOnDemandInstallCacheStats::OnReadCompleted(CacheState.Cas.GetType(), Status.GetErrorCode(), 0);
			return Status;
		}

		break;
	}

	//UE_LOGF(LogIoStoreOnDemand, Warning, "IAD Mapped Handle Opened for chunk %ls in block %d", *LexToString(ChunkInfo.Hash()), CasLoc.BlockId.Id);

	// The file is open the and the Cas block has a non-0 handle count, so the lock can released
	CasUniqueLock.Unlock();

	const FCacheState& CacheState = *pCacheState;

	FSharedMappedFileHandle MappedHandle = FileOpenResult.StealValue();
	IMappedFileRegion* MappedRegion = MappedHandle->MapRegion(CasLoc.BlockOffset + ChunkRange.GetOffset(), ChunkRange.GetLength());
	if (MappedRegion == nullptr)
	{
		const FString Filename = CacheState.Cas.GetBlockFilename(CasLoc.BlockId);
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to map region for chunk in %ls", *Filename);
		FIoStatus Status(EIoErrorCode::ReadError, TEXT("Failed to map region"));
		FOnDemandInstallCacheStats::OnReadCompleted(CacheState.Cas.GetType(), Status.GetErrorCode(), 0);
		return Status;
	}

	FOnDemandInstallCacheStats::OnReadCompleted(CacheState.Cas.GetType(), EIoErrorCode::Ok, ChunkRange.GetLength());

	return FIoMappedRegion
	{
		.MappedFileHandle = new FSharedMappedFileHandleProxy(CacheState.Cas.GetType(), MappedHandle.ToSharedRef()),
		.MappedFileRegion = MappedRegion
	};
#endif
}

const TCHAR* FOnDemandInstallCache::GetName() const
{
	return TEXT("OnDemandInstallCache");
}

bool FOnDemandInstallCache::Resolve(FIoRequestImpl* Request)
{
	EIoErrorCode ErrorCode = EIoErrorCode::UnknownChunkID;
	FOnDemandChunkInfo ChunkInfo = IoStore.GetInstalledChunkInfo(Request->ChunkId, ErrorCode);
	if (ChunkInfo.IsValid() == false)
	{
		if (ErrorCode == EIoErrorCode::NotInstalled)
		{
#if !UE_BUILD_SHIPPING
			if (IOnDemandDevelopmentExtension* Ext = IoStore.TryGetDevelopmentExtension())
			{
				TUtf8StringBuilder<128> Filename;
				TStringBuilder<128> Sb;
				FStringView PackageName = TryConvertChunkIdToPackageName(Request->ChunkId, Filename, Sb);

				if (Ext->GetUnreferencedIoChunkStatus(
					Request->ChunkId,
					Filename.ToView(),
					PackageName,
					IsDevModeEnabled()) != EIoErrorCode::NotInstalled)
				{
					return false;
				}
			}
#endif			
			CompleteRequest(Request, EIoErrorCode::NotInstalled);
			return true;
		}
		return false;
	}

	const uint64 RequestSize = FMath::Min<uint64>(
		Request->Options.GetSize(),
		ChunkInfo.RawSize() - Request->Options.GetOffset());

	TIoStatusOr<FIoOffsetAndLength> ChunkRange = FIoChunkEncoding::GetChunkRange(
		ChunkInfo.RawSize(),
		ChunkInfo.BlockSize(),
		ChunkInfo.Blocks(),
		Request->Options.GetOffset(),
		RequestSize);

	if (ChunkRange.IsOk() == false)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to get chunk range");
		CompleteRequest(Request, ChunkRange.Status().GetErrorCode());
		return true;
	}

	// TODO: Should this support GetTargetVa() ?

	// The internal request parameters are attached/owned by the I/O request via
	// the backend data parameter.
	FChunkRequest::Attach(*Request, new FChunkRequest(
		Request,
		MoveTemp(ChunkInfo),
		ChunkRange.ConsumeValueOrDie(),
		RequestSize));

	bool bFoundInCas = false;
	for (FCacheState& CacheState : CacheStates)
	{
		if (ResolveFromCas(CacheState, Request))
		{
			bFoundInCas = true;
			break;
		}
	}

	if (!bFoundInCas)
	{
		// Chunk is referenced but not in any cache. This should probably never happen currently.
		// If pinning of unreferenced chunks was allowed, then this would be expected...

		FChunkRequest& ChunkRequest = FChunkRequest::GetRef(*Request);
		UE_LOGF(LogIoStoreOnDemand, Error, "Pinned chunk was not found in the cache, Id: %ls, Hash: %ls", 
			*LexToString(ChunkRequest.ChunkInfo.Id()), *LexToString(ChunkRequest.ChunkInfo.Hash()));
		ensure(false);

		CompleteRequest(Request, EIoErrorCode::NotInstalled);
	}

	return true;
}

bool FOnDemandInstallCache::ResolveFromCas(FCacheState& CacheState, FIoRequestImpl* Request)
{
	FChunkRequest& ChunkRequest = FChunkRequest::GetRef(*Request);

	const bool bAllowResolve = CacheState.Cas.GetType() != EOnDemandInstallCasType::MMap;

	// Must check pending chunks before CAS because otherwise
	// a chunk could be flushed after checking the CAS and before checking pending.
	// Otherise we'd need take both locks.
	{
		TDynamicSharedLock SharedLock(CacheState.PendingChunks.SharedMutex);

		const FIoBuffer* PendingChunk = CacheState.PendingChunks.FindChunk(ChunkRequest.ChunkInfo.Hash());
		if (PendingChunk != nullptr)
		{
			ChunkRequest.CasType = CacheState.Cas.GetType();

			if (bAllowResolve == false)
			{
				// Can't use the Resolve API to read this data
				CompleteRequest(Request, EIoErrorCode::FileOpenFailed);
				return true;
			}

			FMemoryView BytesToRead = PendingChunk->GetView().Mid(ChunkRequest.ChunkRange.GetOffset(), ChunkRequest.EncodedChunk.GetSize());
			check(BytesToRead.GetSize() == ChunkRequest.EncodedChunk.GetSize());
			FMemory::Memcpy(ChunkRequest.EncodedChunk.GetData(), BytesToRead.GetData(), ChunkRequest.EncodedChunk.GetSize());

			SharedLock.Unlock();

			UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, Request]
			{
				CompleteRequest(Request, EIoErrorCode::Ok);
			});

			return true;
		}
	}

	const FCasLocation CasLoc = CacheState.Cas.FindChunk(ChunkRequest.ChunkInfo.Hash());
	if (CasLoc.IsValid() == false)
	{
		return false;
	}

	ChunkRequest.CasType = CacheState.Cas.GetType();

	if (bAllowResolve == false)
	{
		// Can't use the Resolve API to read this data
		CompleteRequest(Request, EIoErrorCode::FileOpenFailed);
		return true;
	}

	TRACE_IOSTORE_BACKEND_REQUEST_STARTED(Request, this);
	constexpr const bool bDirty = true;
	if (CacheState.Cas.TrackAccessIf(ECasTrackAccessType::Granular, CasLoc.BlockId, bDirty))
	{
		IoStore.FlushLastAccess({}, nullptr);
	}

#if UE_ONDEMANDINSTALLCACHE_EXCLUSIVE_WRITE
	const bool bIsLocationInCurrentBlock = CasLoc.BlockId == CacheState.CurrentBlock;
	if (bIsLocationInCurrentBlock)
	{
		// The current block may have open writes which may cause async reads to fail
		// on some platforms. Schedule the reads to happen on the same pipe as writes
		CacheState.ExclusivePipe.Launch(UE_SOURCE_LOCATION, [this, &CacheState, Request, CasLoc]
		{
			FChunkRequest& ChunkRequest = FChunkRequest::GetRef(*Request);
			EIoErrorCode Status = EIoErrorCode::FileOpenFailed;

			const FString Filename = CacheState.Cas.GetBlockFilename(CasLoc.BlockId);

			TResult<FSharedFileHandle> FileOpenResult = CacheState.Cas.OpenRead(CasLoc.BlockId);
			if (FileOpenResult.HasValue())
			{
				Status = EIoErrorCode::ReadError;

				TSharedPtr<IFileHandle> FileHandle = FileOpenResult.StealValue();
				const int64 CasBlockOffset = CasLoc.BlockOffset + ChunkRequest.ChunkRange.GetOffset();
				if (Request->IsCancelled())
				{
					UE_LOG(LogIoStoreOnDemand, Verbose, TEXT("Cancelled request - skipped seek to offset %" INT64_FMT " in CAS block '%s' in CAS %s"), 
						CasBlockOffset, *Filename, LexToString(CacheState.Cas.GetType()));
				}
				else if (FileHandle->Seek(CasBlockOffset))
				{
					const bool bOk = FileHandle->Read(ChunkRequest.EncodedChunk.GetData(), ChunkRequest.EncodedChunk.GetSize());
					if (bOk)
					{
						Status = EIoErrorCode::Ok;
					}
					else
					{
						UE_LOG(LogIoStoreOnDemand, Error, TEXT("Failed to read %" UINT64_FMT " bytes at offset %" INT64_FMT " in CAS block '%s' in CAS %s"),
							ChunkRequest.EncodedChunk.GetSize(), CasBlockOffset, *Filename, LexToString(CacheState.Cas.GetType()));
					}
				}
				else
				{
					UE_LOG(LogIoStoreOnDemand, Error, TEXT("Failed to seek to offset %" INT64_FMT " in CAS block '%s' in CAS %s"), 
						CasBlockOffset, *Filename, LexToString(CacheState.Cas.GetType()));
				}
			}
			else
			{
				UE_LOGFMT(LogIoStoreOnDemand, Error, "Failed to open CAS block '{Filename}' in CAS {CasType} for reading ({Error})", 
					*Filename, LexToString(CacheState.Cas.GetType()), FileOpenResult.GetError().GetErrorMessage(true).ToString());
			}

			UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, Request, Status]
			{
				CompleteRequest(Request, Status);
			});
		}, UE::Tasks::ETaskPriority::BackgroundHigh);

		return true;
	}
#endif // UE_ONDEMANDINSTALLCACHE_EXCLUSIVE_WRITE


	TResult<FSharedAsyncFileHandle> FileOpenResult = CacheState.Cas.OpenAsyncRead(CasLoc.BlockId);
	if (FileOpenResult.HasError())
	{
		const FString Filename = CacheState.Cas.GetBlockFilename(CasLoc.BlockId);
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to open CAS block '%ls' in CAS %ls for async reading, reason '%ls'",
			*Filename, LexToString(CacheState.Cas.GetType()), *FileOpenResult.GetError().GetErrorMessage(true).ToString());
		CompleteRequest(Request, EIoErrorCode::FileOpenFailed);
		return true;
	}
	
	ChunkRequest.SharedFileHandle = FileOpenResult.StealValue();

	// CompleteReadTrigger keeps CompleteRequest from being called while ChunkRequest is on the stack
	UE::Tasks::FTaskEvent& CompleteReadTrigger = ChunkRequest.GetCompleteReadRequestTrigger();
	FAsyncFileCallBack Callback = [this, Request, &CompleteReadTrigger](bool bWasCancelled, IAsyncReadRequest* ReadRequest)
	{
		UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, Request, bWasCancelled]
		{
			const EIoErrorCode Status = bWasCancelled ? EIoErrorCode::ReadError : EIoErrorCode::Ok;
			CompleteRequest(Request, Status);
		}, 
		CompleteReadTrigger);
	};

	TUniquePtr<IAsyncReadRequest> ReadRequest(ChunkRequest.SharedFileHandle->ReadRequest(
		CasLoc.BlockOffset + ChunkRequest.ChunkRange.GetOffset(),
		ChunkRequest.ChunkRange.GetLength(),
		EAsyncIOPriorityAndFlags::AIOP_BelowNormal,
		&Callback,
		ChunkRequest.EncodedChunk.GetData()));

	if (ReadRequest.IsValid())
	{
		ChunkRequest.FileReadRequest = MoveTemp(ReadRequest);
		CompleteReadTrigger.Trigger();
	}
	else
	{
		CompleteReadTrigger.Trigger();

		TRACE_IOSTORE_BACKEND_REQUEST_FAILED(Request);
		CompleteRequest(Request, EIoErrorCode::ReadError);
	}

	return true;
}

EOnDemandInstallCasType FOnDemandInstallCache::GetCasType(EIoChunkType ChunkType) const
{
	if (ChunkType == EIoChunkType::MemoryMappedBulkData ||
		(CVars::MemoryMappedShadersEnabled() && (ChunkType == EIoChunkType::ShaderCode || ChunkType == EIoChunkType::ShaderCodeLibrary)))
	{
		if (GetCacheState(EOnDemandInstallCasType::MMap) != nullptr)
		{
			return EOnDemandInstallCasType::MMap;
		}
	}
	return EOnDemandInstallCasType::General;
}

TUniquePtr<FInstallCacheHandle> FOnDemandInstallCache::BeginInstall()
{
	return MakeUnique<FInstallCacheHandleImpl>();
}

void FOnDemandInstallCache::PinCachedChunks(FInstallCacheHandle& InHandle, const FOnDemandInternalContentHandle& ContentHandle, const FOnDemandChunkInfoList& Chunks, TFunctionRef<void(int32, bool)> OnChunkFound)
{
	FInstallCacheHandleImpl* Handle = static_cast<FInstallCacheHandleImpl*>(&InHandle);

	for (EOnDemandInstallCasType CasType : TEnumRange<EOnDemandInstallCasType>())
	{
		Handle->EagerDefragCandidateChunks[CasType].Reset();
	}

	TArray<int32, TInlineAllocator<64>> CachedEntryIndices;

	TStaticArray<int32, EOnDemandInstallCasType::Count> DefragContainerChunksIndex(EInPlace::InPlace, INDEX_NONE);

	for (int32 EntryIndex : Chunks.Indices)
	{
		const FOnDemandChunkHash& Hash = Chunks.Hash(EntryIndex);

		const FCacheState* FoundCacheState = nullptr;
		for (const FCacheState& CacheState : CacheStates)
		{
			// Do not check for EOnDemandInstallCasFlags::AllowReadFromPendingChunks here because if the chunk
			// is pending we don't want to make another request for it regardless of the CAS its in.

			TSharedLock SharedLock(CacheState.PendingChunks.SharedMutex);
			if (CacheState.PendingChunks.ContainsChunk(Hash))
			{
				FoundCacheState = &CacheState;
				break;
			}

			const FCasLocation Loc = CacheState.Cas.FindChunk(Hash);
			if (Loc.IsValid())
			{
				FoundCacheState = &CacheState;

				if (CacheState.EagerDefragBlockCount > 0)
				{
					const EOnDemandInstallCasType CasType = CacheState.Cas.GetType();
					int32& DefragContainerIndex = DefragContainerChunksIndex[CasType];
					if (DefragContainerIndex == INDEX_NONE)
					{
						DefragContainerIndex = Handle->EagerDefragCandidateChunks[CasType].Emplace(Chunks.SharedContainer);
					}
					Handle->EagerDefragCandidateChunks[CasType][DefragContainerIndex].Indices.Add(EntryIndex);
				}
				
				break;
			}
		}

		if (FoundCacheState)
		{
			const EOnDemandInstallCasType CasType = FoundCacheState->Cas.GetType();
			Handle->bUsed[CasType] = true;
			CachedEntryIndices.Add(EntryIndex);
		}

		const bool bCached = !!FoundCacheState;
		OnChunkFound(EntryIndex, bCached);
	}

	if (CachedEntryIndices.IsEmpty() == false)
	{
		TUniqueLock Lock(Chunks.SharedContainer->ReferencesMutex);

		FOnDemandChunkEntryReferences& References = Chunks.SharedContainer->FindOrAddChunkEntryReferences(ContentHandle);
		for (int32 EntryIndex : CachedEntryIndices)
		{
			References.Indices[EntryIndex] = true;
		}
	}
}

void FOnDemandInstallCache::PinChunks(FInstallCacheHandle& Handle, const FOnDemandInternalContentHandle& ContentHandle, const FOnDemandChunkInfoList& Chunks, TFunctionRef<void(int32)> OnChunkFound)
{
	const FSharedOnDemandContainer& Container = Chunks.SharedContainer;

	for (int32 EntryIndex : Chunks.Indices)
	{
		OnChunkFound(EntryIndex);
	}

	TUniqueLock Lock(Container->ReferencesMutex);
	FOnDemandChunkEntryReferences& References = Container->FindOrAddChunkEntryReferences(ContentHandle);
	for (int32 EntryIndex : Chunks.Indices)
	{
		References.Indices[EntryIndex] = true;
	}
}

FResult FOnDemandInstallCache::PutChunk(EIoChunkType ChunkType, FIoBuffer&& Chunk, const FCasAddr& ChunkAddr)
{
	const EOnDemandInstallCasType CasType = GetCasType(ChunkType);
	return PutChunkInternal(CasType, MoveTemp(Chunk), ChunkAddr);
}

void FOnDemandInstallCache::PostPutChunk(FInstallCacheHandle& InHandle, EIoChunkType ChunkType)
{
	FInstallCacheHandleImpl* Handle = static_cast<FInstallCacheHandleImpl*>(&InHandle);
	const EOnDemandInstallCasType CasType = GetCasType(ChunkType);
	Handle->bUsed[CasType] = true;
}

FResult FOnDemandInstallCache::PutChunkInternal(EOnDemandInstallCasType CasType, FIoBuffer&& Chunk, const FCasAddr& ChunkAddr)
{
	FCacheState* CacheState = GetCacheState(CasType);
	if (CacheState == nullptr)
	{
		using namespace UE::IoStore::OnDemand;
		return MakeCasError<void>(
			CasType, ECasErrorCode::WriteBlockFailed, EIoErrorCode::InvalidParameter, FString::Printf(TEXT("CAS %s is not configured"), LexToString(CasType)));
	}

	if (CacheState->PendingChunks.TotalSize > FPendingChunks::MaxPendingBytes)
	{
		const uint64 TotalBlockSize = CacheState->Cas.GetTotalBlockSize();
		if (TotalBlockSize + CacheState->PendingChunks.TotalSize > CacheState->MaxCacheSize)
		{
			// Special error for a full cache so the calling code can decide what to do
			return MakeError(OnDemand::InstallerCacheFullError(CasType));
		}

		if (FResult Result = FlushPendingChunks(*CacheState, CacheState->PendingChunks); Result.HasError())
		{
			return Result;
		}
		check(CacheState->PendingChunks.IsEmpty());
	}

	FOnDemandInstallCacheStats::OnPutChunk(CasType, Chunk.GetSize());

	{
		TUniqueLock Lock(CacheState->PendingChunks.SharedMutex);
		CacheState->PendingChunks.Append(MoveTemp(Chunk), ChunkAddr);
	}

	return MakeValue();
}

FInstallCachePurgeHandle FOnDemandInstallCache::BeginPurge(EIoChunkType ChunkType)
{
	const EOnDemandInstallCasType CasType = GetCasType(ChunkType);
	FInstallCachePurgeHandle Handle;
	Handle.CasType = CasType;
	return Handle;
}

void FOnDemandInstallCache::AddToPurge(FInstallCachePurgeHandle& Handle, EIoChunkType ChunkType, uint64 Size)
{
	const EOnDemandInstallCasType CasType = GetCasType(ChunkType);
	if (CasType == Handle.CasType)
	{
		Handle.BytesToPurge += Size;
	}
}

FResult FOnDemandInstallCache::Purge(const FInstallCachePurgeHandle& Handle)
{
	return PurgeInternal(Handle.CasType, Handle.BytesToPurge);
}

FResult FOnDemandInstallCache::PurgeInternal(EOnDemandInstallCasType CasType, uint64 BytesToInstall)
{
	using namespace UE::IoStore::OnDemand;

	FCacheState* CacheState = GetCacheState(CasType);
	if (CacheState == nullptr)
	{
		return MakeCasError<void>(
			CasType, ECasErrorCode::DeleteBlockFailed, EIoErrorCode::InvalidParameter, FString::Printf(TEXT("CAS %s is not configured"), LexToString(CasType)));
	}

	TUniqueLock Lock(CacheState->PurgeDefragMutex);

	FCasBlockInfoMap	BlockInfo;
	const uint64		TotalCachedBytes = CacheState->Cas.GetBlockInfo(BlockInfo);

	TArray<FSharedOnDemandContainer>	Containers;
	TArray<TBitArray<>>					ChunkEntryIndices;

	IoStore.GetReferencedContent(Containers, ChunkEntryIndices);
	check(Containers.Num() == ChunkEntryIndices.Num());

	uint64 ReferencedBytes = 0;
	uint64 FragmentedBytes = 0;
	uint64 TotalReferencedBlockBytes = 0;

	AddReferencesToBlocks(*CacheState, Containers, ChunkEntryIndices, BlockInfo, ReferencedBytes);
	for (const TPair<FCasBlockId, FCasBlockInfo>& Kv : BlockInfo)
	{
		const FCasBlockInfo& Info = Kv.Value;

		if (Info.FileSize > CacheState->Cas.GetMaxBlockSize())
		{
			UE_LOGFMT(LogIoStoreOnDemand, Error, "CAS {CasType} Block {BlockId} has total size {BlockSize} which is greater than max block size {MaxBlockSize}", 
				LexToString(CacheState->Cas.GetType()), Kv.Key.Id, Info.FileSize, CacheState->Cas.GetMaxBlockSize());
			ensure(false);
		}

		if (Info.RefSize < Info.FileSize)
		{
			FragmentedBytes += (Info.FileSize - Info.RefSize);
		}
		else if (Info.RefSize > Info.FileSize)
		{
			UE_LOGFMT(LogIoStoreOnDemand, Error, "CAS {CasType} Block {BlockId} has RefSize {RefSize} which is greater than total size {BlockSize}",
				LexToString(CacheState->Cas.GetType()), Kv.Key.Id, Info.RefSize, Info.FileSize);
			ensure(false);
		}

		if (Info.RefSize > 0)
		{
			TotalReferencedBlockBytes += Info.FileSize;
		}
	}

	FOnDemandInstallCacheStats::OnCacheUsage(
		CacheState->Cas.GetType(), CacheState->MaxCacheSize, TotalCachedBytes, TotalReferencedBlockBytes, ReferencedBytes, FragmentedBytes);

	const uint64 TotalUncachedBytes = BytesToInstall + CacheState->PendingChunks.TotalSize;
	const uint64 TotalRequiredBytes = TotalCachedBytes + TotalUncachedBytes;
	if (TotalRequiredBytes <= CacheState->MaxCacheSize)
	{
		UE_LOGF(LogIoStoreOnDemand, Verbose, "Skipping cache purge for %ls, MaxCacheSize=%.2lf MiB, CacheSize=%.2lf MiB, ReferencedBlockSize=%.2lf MiB, ReferencedSize=%.2lf MiB, FragmentedBytes=%.2lf MiB, UncachedSize=%.2lf MiB",
			LexToString(CacheState->Cas.GetType()), ToMiB(CacheState->MaxCacheSize), ToMiB(TotalCachedBytes), ToMiB(TotalReferencedBlockBytes), ToMiB(ReferencedBytes), ToMiB(FragmentedBytes), ToMiB(TotalUncachedBytes));
		return MakeValue();
	}

	UE_LOGF(LogIoStoreOnDemand, Log, "Purging install cache %ls, MaxCacheSize=%.2lf MiB, CacheSize=%.2lf MiB, ReferencedBlockSize=%.2lf MiB, ReferencedSize=%.2lf MiB, FragmentedBytes=%.2lf MiB, UncachedSize=%.2lf MiB",
		LexToString(CacheState->Cas.GetType()), ToMiB(CacheState->MaxCacheSize), ToMiB(TotalCachedBytes), ToMiB(TotalReferencedBlockBytes), ToMiB(ReferencedBytes), ToMiB(FragmentedBytes), ToMiB(TotalUncachedBytes));

	const uint64	TotalBytesToPurge	= TotalRequiredBytes - CacheState->MaxCacheSize;
	uint64			TotalPurgedBytes	= 0;
	FResult			Result				= PurgeInternal(*CacheState, BlockInfo, TotalBytesToPurge, TotalPurgedBytes);

	if (TotalPurgedBytes > 0)
	{
		UE_LOGF(LogIoStoreOnDemand, Log, "Purged %.2lf MiB (%.2lf%%) from install cache %ls",
			ToMiB(TotalPurgedBytes), 100.0 * (double(TotalPurgedBytes) / double(TotalCachedBytes)), LexToString(CacheState->Cas.GetType()));
	}

	const uint64 NewCachedBytes = TotalCachedBytes - TotalPurgedBytes;
	UE_CLOGF(NewCachedBytes > CacheState->MaxCacheSize,
		LogIoStoreOnDemand, Warning, "Max install cache size exceeded by %.2lf MiB (%.2lf%%) for cache %ls",
			ToMiB(NewCachedBytes - CacheState->MaxCacheSize), 100.0 * (double(NewCachedBytes - CacheState->MaxCacheSize) / double(CacheState->MaxCacheSize)), LexToString(CacheState->Cas.GetType()));

	uint64 DefragPurgedBytes = 0;
	if (Result.HasError() == false && TotalPurgedBytes < TotalBytesToPurge)
	{
		if (UE::IoStore::CVars::GIoStoreOnDemandEnableDefrag)
		{
			// Attempt to defrag
			const uint64 DefragBytesToPurge = TotalBytesToPurge - TotalPurgedBytes;
			Result = Defrag(*CacheState, Containers, ChunkEntryIndices, BlockInfo, &DefragBytesToPurge, &DefragPurgedBytes);
		}
		else
		{
			FInstallCacheErrorState State = MakeInstallCacheErrorState(*CacheState, TotalCachedBytes, BytesToInstall);
			State.IoError = FIoStatus(EIoErrorCode::InvalidCode, TEXT("Failed to purge required size from install cache"));
			Result = MakeError(InstallCachePurgeError(MoveTemp(State)));
		}
	}

	FOnDemandInstallCacheStats::OnPurge(CacheState->Cas.GetType(), Result, CacheState->MaxCacheSize, NewCachedBytes, TotalBytesToPurge, TotalPurgedBytes + DefragPurgedBytes);
	return Result;
}

FResult FOnDemandInstallCache::PurgeAllUnreferenced(EOnDemandInstallCasType CasType, bool bDefrag, const uint64* BytesToPurge /*= nullptr*/)
{
	using namespace UE::IoStore::OnDemand;

	FCacheState* CacheState = GetCacheState(CasType);
	if (CacheState == nullptr)
	{
		return MakeCasError<void>(
			CasType, ECasErrorCode::DeleteBlockFailed, EIoErrorCode::InvalidParameter, FString::Printf(TEXT("CAS %s is not configured"), LexToString(CasType)));
	}

	TUniqueLock Lock(CacheState->PurgeDefragMutex);

	FCasBlockInfoMap	BlockInfo;
	const uint64		TotalCachedBytes = CacheState->Cas.GetBlockInfo(BlockInfo);

	TArray<FSharedOnDemandContainer>	Containers;
	TArray<TBitArray<>>					ChunkEntryIndices;

	IoStore.GetReferencedContent(Containers, ChunkEntryIndices);
	check(Containers.Num() == ChunkEntryIndices.Num());

	uint64 ReferencedBytes = 0;
	AddReferencesToBlocks(*CacheState, Containers, ChunkEntryIndices, BlockInfo, ReferencedBytes);

	const uint64 TotalReferencedBytes = Algo::TransformAccumulate(BlockInfo,
		[](const TPair<FCasBlockId, FCasBlockInfo>& Kv) { return (Kv.Value.RefSize > 0) ? Kv.Value.FileSize : uint64(0); },
		uint64(0));

	UE_LOGF(LogIoStoreOnDemand, Log, "Purging install cache %ls, MaxCacheSize=%.2lf MiB, CacheSize=%.2lf MiB, ReferencedBytes=%.2lf MiB",
		LexToString(CacheState->Cas.GetType()), ToMiB(CacheState->MaxCacheSize), ToMiB(TotalCachedBytes), ToMiB(TotalReferencedBytes));

	// TODO: should probably default to TotalCachedBytes and not MaxCacheSize
	const uint64	TotalBytesToPurge	= BytesToPurge ? *BytesToPurge : CacheState->MaxCacheSize;
	uint64			TotalPurgedBytes	= 0;
	FResult			Result				= PurgeInternal(*CacheState, BlockInfo, TotalBytesToPurge, TotalPurgedBytes);

	if (TotalPurgedBytes > 0)
	{
		UE_LOGF(LogIoStoreOnDemand, Log, "Purged %.2lf MiB (%.2lf%%) from install cache %ls",
			ToMiB(TotalPurgedBytes), 100.0 * (double(TotalPurgedBytes) / double(TotalCachedBytes)), LexToString(CacheState->Cas.GetType()));
	}

	const uint64 NewCachedBytes = TotalCachedBytes - TotalPurgedBytes;
	UE_CLOGF(NewCachedBytes > CacheState->MaxCacheSize,
		LogIoStoreOnDemand, Warning, "Max install cache size exceeded by %.2lf MiB (%.2lf%%) for cache %ls",
			ToMiB(NewCachedBytes - CacheState->MaxCacheSize), 100.0 * (double(NewCachedBytes - CacheState->MaxCacheSize) / double(CacheState->MaxCacheSize)), LexToString(CacheState->Cas.GetType()));

	if (BytesToPurge)
	{
		if (bDefrag)
		{
			// Attempt to defrag
			const uint64 DefragBytesToPurge = TotalBytesToPurge - TotalPurgedBytes;
			Result = Defrag(*CacheState, Containers, ChunkEntryIndices, BlockInfo, &DefragBytesToPurge);
		}
		else
		{
			FInstallCacheErrorState State = MakeInstallCacheErrorState(*CacheState, TotalCachedBytes, TotalBytesToPurge);
			Result = MakeError(InstallCachePurgeError(MoveTemp(State)));
		}
	}
	else if (bDefrag)
	{
		Result = Defrag(*CacheState, Containers, ChunkEntryIndices, BlockInfo);
	}

	FOnDemandInstallCacheStats::OnPurge(CacheState->Cas.GetType(), Result, CacheState->MaxCacheSize, NewCachedBytes, TotalBytesToPurge, TotalPurgedBytes);
	return Result;
}

FResult FOnDemandInstallCache::DefragAll(EOnDemandInstallCasType CasType, const uint64* BytesToFree /*= nullptr*/)
{
	using namespace UE::IoStore::OnDemand;

	FCacheState* CacheState = GetCacheState(CasType);
	if (CacheState == nullptr)
	{
		return MakeCasError<void>(
			CasType, ECasErrorCode::DeleteBlockFailed, EIoErrorCode::InvalidParameter, FString::Printf(TEXT("CAS %s is not configured"), LexToString(CasType)));
	}

	TUniqueLock Lock(CacheState->PurgeDefragMutex);

	FCasBlockInfoMap	BlockInfo;
	const uint64		TotalCachedBytes = CacheState->Cas.GetBlockInfo(BlockInfo);

	TArray<FSharedOnDemandContainer>	Containers;
	TArray<TBitArray<>>					ChunkEntryIndices;

	IoStore.GetReferencedContent(Containers, ChunkEntryIndices);
	check(Containers.Num() == ChunkEntryIndices.Num());

	uint64 ReferencedBytes = 0;
	AddReferencesToBlocks(*CacheState, Containers, ChunkEntryIndices, BlockInfo, ReferencedBytes);

	const uint64 TotalReferencedBlockBytes = Algo::TransformAccumulate(BlockInfo,
		[](const TPair<FCasBlockId, FCasBlockInfo>& Kv) { return (Kv.Value.RefSize > 0) ? Kv.Value.FileSize : uint64(0); },
		uint64(0));

	UE_LOGF(LogIoStoreOnDemand, Log, "Defragmenting install cache %ls, MaxCacheSize=%.2lf MiB, CacheSize=%.2lf MiB, ReferencedBlockSize=%.2lf MiB, ReferencedSize=%.2lf MiB",
		LexToString(CacheState->Cas.GetType()), ToMiB(CacheState->MaxCacheSize), ToMiB(TotalCachedBytes), ToMiB(TotalReferencedBlockBytes), ToMiB(ReferencedBytes));

	return Defrag(*CacheState, Containers, ChunkEntryIndices, BlockInfo, BytesToFree);
}

FResult FOnDemandInstallCache::EagerDefrag(FInstallCacheHandle& InHandle)
{
	FInstallCacheHandleImpl* Handle = static_cast<FInstallCacheHandleImpl*>(&InHandle);

	FResult Result = MakeValue();
	for (EOnDemandInstallCasType CasType : TEnumRange<EOnDemandInstallCasType>())
	{
		const TArray<FEagerDefragChunkList>& EagerDefragCandidateChunks = Handle->EagerDefragCandidateChunks[CasType];
		if (EagerDefragCandidateChunks.IsEmpty())
		{
			continue;
		}

		if (FResult TypeResult = EagerDefragInternal(CasType, EagerDefragCandidateChunks); TypeResult.HasError())
		{
			if (!Result.HasError())
			{
				Result = MoveTemp(TypeResult);
			}
			else if (TypeResult.HasError())
			{
				// TODO: This doesn't work right. I can only have one of each context type?
				// How do I not lose other error codes?  What if I don't know the type?
				//Result.GetError().PushErrorContext(*TypeResult.GetError().GetErrorContext())
			}
		}
	}
	return Result;
}

bool FOnDemandInstallCache::IsEagerDefragRequired(FInstallCacheHandle& InHandle)
{
	FInstallCacheHandleImpl* Handle = static_cast<FInstallCacheHandleImpl*>(&InHandle);

	for (EOnDemandInstallCasType CasType : TEnumRange<EOnDemandInstallCasType>())
	{
		if (!Handle->EagerDefragCandidateChunks[CasType].IsEmpty())
		{
			return true;
		}
	}

	return false;
}

FResult FOnDemandInstallCache::EagerDefragInternal(EOnDemandInstallCasType CasType, TConstArrayView<FEagerDefragChunkList> ChunkHahses)
{
	// Move any chunks that are in the oldest EagerDefragBlockCount blocks to the current block

	using namespace UE::IoStore::OnDemand;

	if (ChunkHahses.IsEmpty())
	{
		return MakeValue();
	}

	FCacheState* CacheState = GetCacheState(CasType);
	if (CacheState == nullptr)
	{
		return MakeCasError<void>(
			CasType, ECasErrorCode::DeleteBlockFailed, EIoErrorCode::InvalidParameter, FString::Printf(TEXT("CAS %s is not configured"), LexToString(CasType)));
	}

	const uint32 EagerDefragBlockCount = CacheState->EagerDefragBlockCount;
	if (EagerDefragBlockCount == 0)
	{
		return MakeValue();
	}

	FCasBlockInfoMap BlockInfoMap;
	CacheState->Cas.GetBlockInfo(BlockInfoMap);

	if (static_cast<uint32>(BlockInfoMap.Num()) <= EagerDefragBlockCount)
	{
		return MakeValue();
	}

	// Don't consider the current block. The current block can't change while this runs because all changes happen in the same pipe.
	// Theoretically, this shouldn't be necessary because current block should always be the most recent but I want this here to be as
	// cautious as possible.
	BlockInfoMap.Remove(CacheState->CurrentBlock);

	BlockInfoMap.ValueSort([](const FCasBlockInfo& LHS, const FCasBlockInfo& RHS)
	{
		return LHS.LastModification < RHS.LastModification;
	});

	auto IsInOldBlock = [&BlockInfoMap, EagerDefragBlockCount](const FCasLocation& Loc)
	{
		uint32 i = 0;
		for (auto It = BlockInfoMap.CreateConstIterator(); It && i < EagerDefragBlockCount; ++It)
		{
			if (It.Key() == Loc.BlockId)
			{
				return true;
			}
			++i;
		}

		return false;
	};

	// TODO: PinCachedChunks already looks up the CasLocation.
	// We could avoid this double lookup but it might be less safe

	struct FDefragChunk
	{
		FOnDemandChunkHash	Hash;
		FIoChunkId			ChunkId;
		FCasLocation		CasLoc;
		uint32				DiskSize = 0;
	};

	TArray<FDefragChunk, TInlineAllocator<64>> DefragChunks;
	{
		TUniqueLock Lock(CacheState->Cas);

		for (const FEagerDefragChunkList& ChunkInfoList : ChunkHahses)
		{
			for (const int32 i : ChunkInfoList.Indices)
			{
				const FOnDemandChunkHash& Hash = ChunkInfoList.Hash(i);
				FCasLocation CasLoc = CacheState->Cas.FindChunk(Hash, &UE::DeferLock);
				if (ensure(CasLoc.IsValid()) && IsInOldBlock(CasLoc))
				{
					//UE_LOGF(LogIoStoreOnDemand, Warning, "Eager Defrag found chunk %ls in block %d", *LexToString(Hash), CasLoc.BlockId.Id);

					DefragChunks.Add(FDefragChunk
						{
							.Hash = Hash,
							.ChunkId = ChunkInfoList.Id(i),
							.CasLoc = CasLoc,
							.DiskSize = ChunkInfoList.DiskSize(i)
						});
				}
			}
		}
	}

	if (DefragChunks.IsEmpty())
	{
		return MakeValue();
	}

	DefragChunks.Sort([](const FDefragChunk& A, const FDefragChunk& B)
	{
		return 
			(A.CasLoc.BlockId.Id < B.CasLoc.BlockId.Id) || 
			(A.CasLoc.BlockId == B.CasLoc.BlockId && A.CasLoc.BlockOffset < B.CasLoc.BlockOffset);
	});

	FCasBlockId LastDefragBlock;
	FSharedFileHandle FileHandle;
	
	for (TArrayView<FDefragChunk> DefragChunksView(DefragChunks); DefragChunksView.Num() > 0; DefragChunksView.RightChopInline(1))
	{
		const FDefragChunk& Chunk = DefragChunksView[0];

		if (Chunk.CasLoc.BlockId != LastDefragBlock)
		{
			constexpr bool bAllowReadFromMMapCas = true;
			TResult<FSharedFileHandle> FileOpenResult = CacheState->Cas.OpenRead(Chunk.CasLoc.BlockId, bAllowReadFromMMapCas);
			if (FileOpenResult.HasError())
			{
				UE_LOGFMT(LogIoStoreOnDemand, Error, "Eager Defrag failed to open CAS block for reading: {Error}", FileOpenResult.GetError().GetErrorMessage(true).ToString());

				FResult Result = MakeCasError<void>(
					CacheState->Cas.GetType(),
					ECasErrorCode::ReadBlockFailed, EIoErrorCode::FileOpenFailed, FString::Printf(TEXT("Failed to open cache block %u"),
						Chunk.CasLoc.BlockId.Id));
				return Result;
			}

			LastDefragBlock = Chunk.CasLoc.BlockId;
			FileHandle = FileOpenResult.StealValue();
		}

		FIoBuffer	Buffer(Chunk.DiskSize);
		FResult		ReadResult = MakeValue();

		for (int32 Attempt = 0, MaxAttempts = 3; Attempt < MaxAttempts; ++Attempt)
		{
			if (FileHandle->Seek(Chunk.CasLoc.BlockOffset) == false)
			{
				ReadResult = MakeCasError<void>(
					CacheState->Cas.GetType(),
					ECasErrorCode::ReadBlockFailed, EIoErrorCode::FileSeekFailed, FString::Printf(TEXT("Failed to seek to offset %u in cache block %u"),
						Chunk.CasLoc.BlockOffset, Chunk.CasLoc.BlockId.Id));
				continue;
			}
			if (FileHandle->Read(Buffer.GetData(), Buffer.GetSize()) == false)
			{
				ReadResult = MakeCasError<void>(
					CacheState->Cas.GetType(),
					ECasErrorCode::ReadBlockFailed, EIoErrorCode::ReadError, FString::Printf(TEXT("Failed to read %" UINT64_FMT " bytes at offset %u in cache block %u"),
						Buffer.GetSize(), Chunk.CasLoc.BlockOffset, Chunk.CasLoc.BlockId.Id));
				continue;
			}

			ReadResult = MakeValue();
			break;
		}

		if (ReadResult.HasError())
		{
			UE_LOGFMT(LogIoStoreOnDemand, Error, "Eager Defrag failed to read CAS block: {Error}", ReadResult.GetError().GetErrorMessage(true).ToString());
			return ReadResult;
		}

		const FOnDemandChunkHash ChunkHash = FOnDemandChunkHash::HashBuffer(Buffer.GetView());
		if (ChunkHash != Chunk.Hash)
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "Found chunk with invalid hash during eager defrag, ChunkId='%ls', ExpectedHash='%ls', ActualHash='%ls', BlockId=%u, BlockOffset=%u",
				*LexToString(Chunk.ChunkId), *LexToString(Chunk.Hash), *LexToString(ChunkHash), Chunk.CasLoc.BlockId.Id, Chunk.CasLoc.BlockOffset);

			// Append a critical error entry to clear the cache at next startup
			FCasJournal::FTransaction Transaction = FCasJournal::Begin(CacheState->Cas.GetType(), GetJournalFilename());
			Transaction.CriticalError(FCasJournal::EErrorCode::DefragHashMismatch);
			if (FResult CommitResult = FCasJournal::Commit(MoveTemp(Transaction)); CommitResult.HasError())
			{
				UE_LOGFMT(LogIoStoreOnDemand, Error, "Failed to commit critical errors to CAS journal ({Error})", CommitResult.GetError());
			}

			FResult Result = MakeError(InstallCacheDefragHashMismatchError(
				FInstallCacheDefragHashMismatchError
				{
					.ChunkId = Chunk.ChunkId, // May not be unique, just return the first found for debugging purposes
					.ExpectedHash = LexToString(Chunk.Hash),
					.ActualHash = LexToString(ChunkHash),
					.CasType = CacheState->Cas.GetType()
				}));
			return Result;
		}

		
		// Let the normal PutChunk path update the CAS
		FResult Result = MakeValue();
		FResult PutResult = PutChunkInternal(CasType, MoveTemp(Buffer), Chunk.Hash);
		if (PutResult.HasError() && PutResult.GetError() != OnDemand::InstallerCacheFullError)
		{
			Result = MakeError(PutResult.StealError());
		}
		else if (PutResult.HasError() && PutResult.GetError() == OnDemand::InstallerCacheFullError)
		{
			uint64 PendingSize = 0;
			for (const FDefragChunk& PendingChunk : DefragChunksView)
			{
				PendingSize += PendingChunk.DiskSize;
			}

			// TODO: does this need to be here? Can we move the installer code that does this into the cache and just have this code once?
			Result = PurgeInternal(CasType, PendingSize);
			if (Result.HasValue())
			{
				PutResult = PutChunkInternal(CasType, MoveTemp(Buffer), Chunk.Hash);
				if (PutResult.HasError())
				{
					if (PutResult.GetError() == InstallerCacheFullError)
					{
						UE_LOGF(LogIoStoreOnDemand, Error, "Insuffucient cache space after purge, this should never happen. CasType: %ls", LexToString(CasType));
						ensure(false);
					}

					Result = MoveTemp(PutResult);
				}
			}
		}

		if (Result.HasError())
		{
			//UE_LOGF(LogIoStoreOnDemand, Warning, "Eager Defrag put chunk %ls failed", *LexToString(Chunk.Hash));
			return Result;
		}
		else
		{
			//UE_LOGF(LogIoStoreOnDemand, Warning, "Eager Defrag put chunk %ls ok", *LexToString(Chunk.Hash));
		}
	}

	return MakeValue();
}

FResult FOnDemandInstallCache::ConditionallyFlushInstall(FInstallCacheHandle& InHandle)
{
	FInstallCacheHandleImpl* Handle = static_cast<FInstallCacheHandleImpl*>(&InHandle);

	FResult Result = MakeValue();

	// Cannot read from pending buffer for MMap chunks, must flush to disk
	if (Handle->bUsed[EOnDemandInstallCasType::MMap])
	{
		// TODO: this could be a partial flush if pending chunks was sorted by request
		Result = Flush(EOnDemandInstallCasType::MMap);
	}

	return Result;
}

FResult FOnDemandInstallCache::Verify()
{
	struct FChunkLookup
	{
		TMap<FCasAddr, int32> AddrToIndex;
	};

	struct FCasAddrLocation
	{
		FCasAddr		Addr;
		FCasLocation	Location;

		bool operator<(const FCasAddrLocation& Other) const
		{
			if (Location.BlockId == Other.Location.BlockId)
			{
				return Location.BlockOffset < Other.Location.BlockOffset;
			}
			return Location.BlockId.Id < Other.Location.BlockId.Id;
		}
	};

	uint32		CorruptChunkCount = 0;
	uint32		MissingChunkCount = 0;
	uint32		ReadErrorCount = 0;
	uint64		TotalVerifiedBytes = 0;
	uint32		TotalChunkCount = 0;

	TArray<FSharedOnDemandContainer>	Containers = IoStore.GetContainers(EOnDemandContainerFlags::InstallOnDemand);
	TArray<FChunkLookup>				ChunkLookups;

	ChunkLookups.Reserve(Containers.Num());
	for (int32 Idx = 0; Idx < Containers.Num(); ++Idx)
	{
		FSharedOnDemandContainer& Container = Containers[Idx];
		FChunkLookup& Lookup				= ChunkLookups.AddDefaulted_GetRef();

		Lookup.AddrToIndex.Reserve(Container->ChunkEntries.Num());
		for (int32 EntryIndex = 0; const FOnDemandChunkEntry& Entry : Container->ChunkEntries)
		{
			const FCasAddr& Addr = Entry.Hash;
			Lookup.AddrToIndex.Add(Addr, EntryIndex++);
		}
	}

	auto FindChunkEntry = [&ChunkLookups](const FCasAddr& Addr, int32& OutContainerIndex) -> int32
	{
		OutContainerIndex = INDEX_NONE;
		for (int32 Idx = 0; Idx < ChunkLookups.Num(); ++Idx)
		{
			FChunkLookup& Lookup = ChunkLookups[Idx];
			if (const int32* EntryIndex = Lookup.AddrToIndex.Find(Addr))
			{
				OutContainerIndex = Idx;
				return *EntryIndex;
			}
		}

		return INDEX_NONE;
	};

	for (FCacheState& CacheState : CacheStates)
	{
		TArray<FCasAddrLocation>			ChunkLocations;

		{
			TUniqueLock Lock(CacheState.Cas);
			ChunkLocations.Reserve(CacheState.Cas.Lookup.Num());
			for (const TPair<FCasAddr, FCasLocation>& Kv : CacheState.Cas.Lookup)
			{
				ChunkLocations.Add(FCasAddrLocation
				{
					.Addr		= Kv.Key,
					.Location	= Kv.Value
				});
			}
		}
		ChunkLocations.Sort();

		const int32	ChunkCount = ChunkLocations.Num();
		TotalChunkCount += ChunkCount;

		if (ChunkCount == 0)
		{
			UE_LOGF(LogIoStoreOnDemand, Log, "Verify skipped CAS %ls, install cache is empty", LexToString(CacheState.Cas.GetType()));
			continue;
		}

		FIoBuffer	Chunk(1 << 20);

		UE_LOGF(LogIoStoreOnDemand, Log, "Verifying %d installed chunks in CAS %ls...", ChunkCount, LexToString(CacheState.Cas.GetType()));
		for (int32 ChunkIndex = 0; const FCasAddrLocation& ChunkLocation : ChunkLocations)
		{
			constexpr bool bAllowReadFromMMapCas = true;
			TResult<FSharedFileHandle> OpenResult = CacheState.Cas.OpenRead(ChunkLocation.Location.BlockId, bAllowReadFromMMapCas);
			if (OpenResult.HasError())
			{
				UE_LOGFMT(LogIoStoreOnDemand, Error, "Failed to open block {BlockId} for reading ({Error})", ChunkLocation.Location.BlockId.Id, OpenResult.GetError());

				ReadErrorCount++;
				ChunkIndex++;
				continue;
			}

			int32 ContainerIndex	= INDEX_NONE;
			int32 EntryIndex		= FindChunkEntry(ChunkLocation.Addr, ContainerIndex);

			if (EntryIndex == INDEX_NONE)
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to find chunk entry for CAS address '%ls'", *LexToString(ChunkLocation.Addr));

				MissingChunkCount++;
				ChunkIndex++;
				continue;
			}

			const FSharedOnDemandContainer& Container	= Containers[ContainerIndex];
			const FIoChunkId& ChunkId					= Container->ChunkIds[EntryIndex];
			const FOnDemandChunkEntry& ChunkEntry		= Container->ChunkEntries[EntryIndex];
			FSharedFileHandle FileHandle				= OpenResult.GetValue();
			const int64 ChunkSize						= ChunkEntry.GetDiskSize();
			TotalVerifiedBytes							+= ChunkSize;

			if (int64(Chunk.GetSize()) < ChunkSize)
			{
				Chunk = FIoBuffer(ChunkSize);
			}

			if (FileHandle->Seek(int64(ChunkLocation.Location.BlockOffset)) == false)
			{
				UE_LOG(LogIoStoreOnDemand, Error, TEXT("Chunk %d/%d SEEK FAILED, Container='%s', ChunkId='%s', ChunkSize=%" INT64_FMT ", Hash='%s', Block=%u, BlockOffset=%u"),
					ChunkIndex + 1, ChunkCount, *Container->Name(), *LexToString(ChunkId), ChunkSize, *LexToString(ChunkEntry.Hash),
					ChunkLocation.Location.BlockId.Id, ChunkLocation.Location.BlockOffset);

				ReadErrorCount++;
				ChunkIndex++;
				continue;
			}

			if (FileHandle->Read(reinterpret_cast<uint8*>(Chunk.GetData()), ChunkSize) == false)
			{
				UE_LOG(LogIoStoreOnDemand, Error, TEXT("Chunk %d/%d READ FAILED, Container='%s', ChunkId='%s', ChunkSize=%" INT64_FMT ", Hash='%s', Block=%u, BlockOffset=%u"),
					ChunkIndex + 1, ChunkCount, *Container->Name(), *LexToString(ChunkId), ChunkSize, *LexToString(ChunkEntry.Hash),
					ChunkLocation.Location.BlockId.Id, ChunkLocation.Location.BlockOffset);

				ReadErrorCount++;
				ChunkIndex++;
				continue;
			}

			const FOnDemandChunkHash ChunkHash = FOnDemandChunkHash::HashBuffer(Chunk.GetView().Left(ChunkSize));

			if (ChunkHash == ChunkEntry.Hash)
			{
				UE_LOG(LogIoStoreOnDemand, VeryVerbose, TEXT("Chunk %d/%d OK, Container='%s', ChunkId='%s', ChunkSize=%" INT64_FMT ", Hash='%s', Block=%u, BlockOffset=%u"),
					ChunkIndex + 1, ChunkCount, *Container->Name(), *LexToString(ChunkId), ChunkSize, *LexToString(ChunkEntry.Hash),
					ChunkLocation.Location.BlockId.Id, ChunkLocation.Location.BlockOffset);
			}
			else
			{
				UE_LOG(LogIoStoreOnDemand, Error, TEXT("Chunk %d/%d CORRUPT, Container='%s', ChunkId='%s', ChunkSize=%" INT64_FMT ", Hash='%s', ActualHash='%s', Block=%u, BlockOffset=%u"),
					ChunkIndex + 1, ChunkCount, *Container->Name(), *LexToString(ChunkId), ChunkSize, *LexToString(ChunkEntry.Hash), *LexToString(ChunkHash),
					ChunkLocation.Location.BlockId.Id, ChunkLocation.Location.BlockOffset);

				CorruptChunkCount++;
			}

			ChunkIndex++;
		}
	}

	if (CorruptChunkCount > 0 || MissingChunkCount > 0 || ReadErrorCount > 0)
	{
		const FString Reason = FString::Printf(TEXT("Verify install cache failed, Corrupt=%u, Missing=%u, ReadErrors=%u"),
			CorruptChunkCount, MissingChunkCount, ReadErrorCount);

		if (CorruptChunkCount > 0 || ReadErrorCount > 0)
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "%ls", *Reason);
		}
		else
		{
			UE_LOGF(LogIoStoreOnDemand, Warning, "%ls", *Reason);
		}

		return MakeError(UE::IoStore::OnDemand::InstallCacheVerificationError(
			UE::IoStore::OnDemand::FInstallCacheVerificationError
			{
				.CorruptChunkCount	= CorruptChunkCount,
				.MissingChunkCount	= MissingChunkCount,
				.ReadErrorCount		= ReadErrorCount
			}));
	}

	UE_LOGF(LogIoStoreOnDemand, Log, "Successfully verified %d chunk(s) of total %.2lf MiB",
		TotalChunkCount, ToMiB(TotalVerifiedBytes));

	return MakeValue();
}

void FOnDemandInstallCache::RegisterConsoleCommands()
{
#if UE_IAD_DEBUG_CONSOLE_CMDS
	ConsoleCommands.Emplace(
	IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("iostore.SimulateCriticalInstallCacheError"),
		TEXT(""),
		FConsoleCommandDelegate::CreateLambda([this]()
		{
			UE_LOGF(LogIoStoreOnDemand, Log, "Simulating critical install cache error");

			FCasJournal::FTransaction Transaction = FCasJournal::Begin(EOnDemandInstallCasType::General, GetJournalFilename());
			Transaction.CriticalError(FCasJournal::EErrorCode::Simulated);
			if (FResult Result = FCasJournal::Commit(MoveTemp(Transaction)); Result.HasError())
			{
				UE_LOGF(LogIoStoreOnDemand, Warning, "Failed to append critical error to journal, error '%ls'",
					*LexToString(Result.GetError()));
			}
		}),
		ECVF_Default)
	);
#endif // UE_IAD_DEBUG_CONSOLE_CMDS
}

FResult FOnDemandInstallCache::Reset()
{
	using namespace UE::IoStore::OnDemand;

	UE_LOGF(LogIoStoreOnDemand, Log, "Resetting install cache in directory '%ls'", *CacheDirectory);

	IFileManager& Ifm	= IFileManager::Get();
	const bool bTree	= true;

	if (Ifm.DeleteDirectory(*CacheDirectory, false, bTree) == false)
	{
		return MakeCasError<void>(
			EOnDemandInstallCasType::None, ECasErrorCode::InitializeFailed, EIoErrorCode::DeleteError, FString::Printf(TEXT("Failed to delete cache directory '%s'"), *CacheDirectory));
	}

	if (Ifm.MakeDirectory(*CacheDirectory, bTree) == false)
	{
		return MakeCasError<void>(
			EOnDemandInstallCasType::None, ECasErrorCode::InitializeFailed, EIoErrorCode::WriteError, FString::Printf(TEXT("Failed to create cache directory '%s'"), *CacheDirectory));
	}

	for (FCacheState& CacheState : CacheStates)
	{
		FString CasRootDir = CacheState.Cas.GetRootDirectory();
		if (FResult Result = CacheState.Cas.Initialize(CasRootDir); Result.HasError())
		{
			return Result;
		}

		CacheState.CurrentBlock = FCasBlockId::Invalid;
	}

	const FString JournalFile = GetJournalFilename();
	if (FResult Result = FCasJournal::Create(JournalFile); Result.HasError())
	{
		return Result;
	}

	UE_LOGF(LogIoStoreOnDemand, Log, "Created CAS journal '%ls'", *JournalFile);
	return MakeValue();
}

FResult FOnDemandInstallCache::InitialVerify()
{
	// TODO: IadMultiCAS: there's no way to verify each CAS in isolation right now, so if we detect an error everything will be reset
	for (FCacheState& CacheState : CacheStates)
	{
		// Verify the blocks on disk with the current state of the CAS
		{
			TArray<FCasAddr> RemovedChunks;
			TMap<FCasBlockId, uint64> OverBudgetBlocks;
			FResult Verify = CacheState.Cas.Verify(RemovedChunks, OverBudgetBlocks);

			uint64 TotalBlockBytesOverBudget = 0;
			for (const TPair<FCasBlockId, uint64>& Kv : OverBudgetBlocks)
			{
				if (ensure(Kv.Value > CacheState.Cas.GetMaxBlockSize()))
				{
					TotalBlockBytesOverBudget += (Kv.Value - CacheState.Cas.GetMaxBlockSize());
				}
			}

			FOnDemandInstallCacheStats::OnCasVerification(CacheState.Cas.GetType(), Verify, RemovedChunks.Num(), TotalBlockBytesOverBudget);

			if (Verify.HasError())
			{
				// Remove all entries that doesn't have a valid cache block 
				FCasJournal::FTransaction Transaction = FCasJournal::Begin(CacheState.Cas.GetType(), GetJournalFilename());
				for (const FCasAddr& Addr : RemovedChunks)
				{
					Transaction.ChunkLocation(FCasLocation::Invalid, Addr);
				}

				if (FResult Result = FCasJournal::Commit(MoveTemp(Transaction)); Result.HasError())
				{
					return Result;
				}
			}
		}

		// Check if the cache is over budget
		{
			FCasBlockInfoMap	BlockInfo;
			uint64				CacheSize = CacheState.Cas.GetBlockInfo(BlockInfo);

			if (CacheSize > CacheState.MaxCacheSize)
			{
				const uint64	TotalBytesToPurge = CacheSize - CacheState.MaxCacheSize;
				uint64			TotalPurgedBytes = 0;

				UE_LOGF(LogIoStoreOnDemand, Warning, "Cache size is greater than disk quota for CAS %ls - Purging install cache, MaxCacheSize=%.2lf MiB, TotalSize=%.2lf MiB, TotalBytesToPurge=%.2lf MiB",
					LexToString(CacheState.Cas.GetType()), ToMiB(CacheState.MaxCacheSize), ToMiB(CacheSize), ToMiB(TotalBytesToPurge));

				if (FResult Result = PurgeInternal(CacheState, BlockInfo, TotalBytesToPurge, TotalPurgedBytes); Result.HasError())
				{
					const FString ErrorMessage = FString::Printf(TEXT("Failed to purge %.2lf MiB from CAS %s reason '%s'"),
						ToMiB(TotalBytesToPurge), LexToString(CacheState.Cas.GetType()), *LexToString(Result.GetError()));

					UE_LOGF(LogIoStoreOnDemand, Error, "%ls", *ErrorMessage);
					return Result;
				}

				if (TotalPurgedBytes < TotalBytesToPurge)
				{
					// This should never happen since we don't have any referenced cache blocks at startup
					const FString ErrorMessage = FString::Printf(TEXT("Failed to purge %.2lf MiB from CAS %s. Actually purged %.2lf MiB"),
						ToMiB(TotalBytesToPurge), LexToString(CacheState.Cas.GetType()), ToMiB(TotalPurgedBytes));

					UE_LOGF(LogIoStoreOnDemand, Error, "%ls", *ErrorMessage);
					UE::IoStore::OnDemand::FInstallCacheErrorState State = MakeInstallCacheErrorState(CacheState, CacheSize, TotalBytesToPurge);
					return MakeError(UE::IoStore::OnDemand::InstallCachePurgeError(MoveTemp(State)));
				}

				UE_LOGF(LogIoStoreOnDemand, Log, "Successfully purged %.2lf MiB from CAS %ls", ToMiB(TotalPurgedBytes), LexToString(CacheState.Cas.GetType()));
			}
		}

		if (!CacheState.Cas.ContainsBlock(CacheState.CurrentBlock))
		{
			const FCasBlockId OldCurrentBlock = CacheState.CurrentBlock.exchange(FCasBlockId::Invalid);
			if (OldCurrentBlock != FCasBlockId::Invalid)
			{
				UE_LOGF(LogIoStoreOnDemand, Warning, "Current Block %u was not found for CAS %ls, resetting", OldCurrentBlock.Id, LexToString(CacheState.Cas.GetType()));
			}
		}
	}

	return MakeValue();
}

void FOnDemandInstallCache::AddReferencesToBlocks(
	const FCacheState& InCacheState,
	const TArray<FSharedOnDemandContainer>& Containers, 
	const TArray<TBitArray<>>& ChunkEntryIndices, 
	FCasBlockInfoMap& BlockInfoMap, 
	uint64& OutTotalReferencedBytes) const
{
	OutTotalReferencedBytes = 0;

	TSet<FCasAddr> VisitedReferencedChunks;
	{
		int32 ReserveSize = 0;
		for (int32 Index = 0; FSharedOnDemandContainer Container : Containers)
		{
			const TBitArray<>& IsReferenced = ChunkEntryIndices[Index++];
			ReserveSize += IsReferenced.CountSetBits();
		}

		VisitedReferencedChunks.Reserve(ReserveSize);
	}

	for (int32 Index = 0; FSharedOnDemandContainer Container : Containers)
	{
		const TBitArray<>& IsReferenced = ChunkEntryIndices[Index++];
		for (int32 EntryIndex = 0; const FOnDemandChunkEntry& Entry : Container->ChunkEntries)
		{
			const bool bIsReferenced = IsReferenced[EntryIndex];
			if (!bIsReferenced)
			{
				EntryIndex++;
				continue;
			}

			bool bAlreadyVisited = false;
			VisitedReferencedChunks.Add(Entry.Hash, &bAlreadyVisited);
			if (bAlreadyVisited)
			{
				EntryIndex++;
				continue;
			}

			const uint64 ChunkDiskSize = Entry.GetDiskSize();
			
			if (InCacheState.Cas.GetType() == GetCasType(Container->ChunkIds[EntryIndex].GetChunkType()))
			{
				OutTotalReferencedBytes += ChunkDiskSize;
			}

			auto VisitChunk = 
				[&InCacheState, &BlockInfoMap, &Container, EntryIndex, ChunkDiskSize]
				(const FCacheState& CacheState, const FOnDemandChunkHash& Hash, bool bSearchPending) -> bool
			{
				bool bFound = false;
				if (FCasLocation Loc = CacheState.Cas.FindChunk(Hash); Loc.IsValid())
				{
					bFound = true;
					if (&CacheState == &InCacheState)
					{
						FCasBlockInfo* BlockInfo = BlockInfoMap.Find(Loc.BlockId);
						if (!BlockInfo)
						{
							UE_LOGF(LogIoStoreOnDemand, Error, "Failed to find CAS block info for referenced chunk, CAS=%ls, ChunkId='%ls', Container='%ls'",
								LexToString(CacheState.Cas.GetType()), *LexToString(Container->ChunkIds[EntryIndex]), *Container->Name());
						}
						else
						{
							BlockInfo->RefSize += ChunkDiskSize;
						}
					}
				}
				else if (bSearchPending)
				{
					// Check pending list
					TSharedLock SharedLock(CacheState.PendingChunks.SharedMutex);
					bFound = CacheState.PendingChunks.ContainsChunk(Hash);
				}

				return bFound;
			};

			if (CVars::GIoStoreOnDemandVerifyReferencedChunksOnPurge)
			{
				bool bFound = false;
				constexpr bool bSearchPendingTrue = true;
				for (const FCacheState& CacheState : CacheStates)
				{
					bFound = VisitChunk(CacheState, Entry.Hash, bSearchPendingTrue);

					if (bFound)
					{
						break;
					}

				}

				UE_CLOGF(!bFound, LogIoStoreOnDemand, Error, "Failed to find CAS location or pending chunk for chunk reference, CAS=%ls, ChunkId='%ls', Container='%ls'",
					LexToString(InCacheState.Cas.GetType()), *LexToString(Container->ChunkIds[EntryIndex]), *Container->Name());

				ensure(bFound);
			}
			else
			{
				constexpr bool bSearchPendingFalse = false;
				VisitChunk(InCacheState, Entry.Hash, bSearchPendingFalse);
			}

			EntryIndex++;
		}
	}
}

FResult FOnDemandInstallCache::PurgeInternal(
	FCacheState& CacheState, 
	FCasBlockInfoMap& BlockInfo, 
	const uint64 TotalBytesToPurge, 
	uint64& OutTotalPurgedBytes)
{
	BlockInfo.ValueSort([](const FCasBlockInfo& LHS, const FCasBlockInfo& RHS)
	{
		return LHS.LastAccess < RHS.LastAccess;
	});

	OutTotalPurgedBytes = 0;

	for (auto It = BlockInfo.CreateIterator(); It; ++It)
	{
		const FCasBlockId BlockId = It->Key;
		const FCasBlockInfo& Info = It->Value;
		if (Info.RefSize > 0)
		{
			continue;
		}

		FCasJournal::FTransaction	Transaction = FCasJournal::Begin(CacheState.Cas.GetType(), GetJournalFilename());

		auto HandleChunkRemoved = [&Transaction](const FCasAddr& Addr)
		{
			Transaction.ChunkLocation(FCasLocation::Invalid, Addr);
		};

		if (FResult Result = CacheState.Cas.DeleteBlock(BlockId, HandleChunkRemoved); Result.HasError())
		{
			// Skip any blocks with open MMap handles. They can't be deleted currently.
			if (CacheState.Cas.GetType() == EOnDemandInstallCasType::MMap)
			{
				using namespace UE::IoStore::OnDemand;

				if (const FCasError* Payload = Result.GetError().GetErrorContext<CasError>();
					Payload && Payload->CasErrorCode == ECasErrorCode::DeleteBlockFailed && Payload->IoErrorCode == EIoErrorCode::FileHandleOpen)
				{
					continue;
				}
			}

			return Result;
		}

		constexpr bool bFromDefragFalse = false;
		FOnDemandInstallCacheStats::OnBlockDeleted(CacheState.Cas.GetType(), Info.LastAccess, Info.LastModification, bFromDefragFalse);

		// This should be the only thread writing to CurrentBlock
		FCasBlockId MaybeCurrentBlock = BlockId;
		CacheState.CurrentBlock.compare_exchange_strong(MaybeCurrentBlock, FCasBlockId::Invalid);

		OutTotalPurgedBytes += Info.FileSize;

		It.RemoveCurrent();
		Transaction.BlockDeleted(BlockId);

		if (FResult Result = FCasJournal::Commit(MoveTemp(Transaction)); Result.HasError())
		{
			return Result;
		}
		
		if (OutTotalPurgedBytes >= TotalBytesToPurge)
		{
			break;
		}
	}

	return MakeValue();
}

FResult FOnDemandInstallCache::Defrag(
	FCacheState& CacheState,
	const TArray<FSharedOnDemandContainer>& Containers,
	const TArray<TBitArray<>>& ChunkEntryIndices,
	FCasBlockInfoMap& BlockInfo, 
	const uint64* TotalBytesToFree /*= nullptr*/,
	uint64* OutTotalFreedBytes /*= nullptr*/)
{
	using namespace UE::IoStore::OnDemand;

	if (OutTotalFreedBytes)
	{
		*OutTotalFreedBytes = 0;
	}

	if (TotalBytesToFree && *TotalBytesToFree == 0)
	{
		return MakeValue();
	}

	uint64 TotalCachedBytes				= 0;
	uint64 TotalFragmentedBytes			= 0;
	uint64 BlocksToDefragFragmentedSize	= 0;
	uint64 BlocksToDefragTotalSize		= 0;

	struct FDefragBlockReferencedChunk
	{
		FOnDemandChunkHash	Hash;
		FIoChunkId			ChunkId; 
		uint32				BlockOffset = 0;
		uint32				DiskSize = 0;
	};

	struct FDefragBlock
	{
		FCasBlockId BlockId;
		int64 LastAccess = 0;
		int64 LastModification = 0;
		TArray<FDefragBlockReferencedChunk> ReferencedChunks;

		FDefragBlock(const FCasBlockId InBlockId, const FCasBlockInfo& InBlockInfo)
			: BlockId(InBlockId)
			, LastAccess(InBlockInfo.LastAccess)
			, LastModification(InBlockInfo.LastModification)
		{ }
	};

	// Build the list of blocks to defrag and determine if its possible to free enough data through defragging
	TArray<FDefragBlock> BlocksToDefrag;
	
	// Start with the least referenced blocks
	BlockInfo.ValueSort([](const FCasBlockInfo& LHS, const FCasBlockInfo& RHS)
	{
		return LHS.RefSize < RHS.RefSize;
	});

	// If an error occurs, cleanup any blocks marked as pending defrag
	ON_SCOPE_EXIT
	{
		CacheState.Cas.ResetAllPendingDefrag();
	};

	TDynamicUniqueLock<FCas> CasUniqueLock;
	const UE::FDeferLock* SkipLock = nullptr;
	if (CacheState.Cas.GetType() == EOnDemandInstallCasType::MMap)
	{
		// Any blocks added to BlocksToDefrag must be marked in the CAS as pending defrag,
		// OpenMapped will wait until they have been defragged.
		CasUniqueLock = TDynamicUniqueLock(CacheState.Cas);
		SkipLock = &UE::DeferLock;
	}

	if (TotalBytesToFree)
	{
		// Partial defrag
		bool bPossibleToFreeBytes = false;

		uint64 FreedBlockBytes = 0;
		uint64 NewBlockBytes = 0;

		for (const TPair<FCasBlockId, FCasBlockInfo>& Kv : BlockInfo)
		{
			const FCasBlockId BlockId = Kv.Key;
			const FCasBlockInfo& Info = Kv.Value;

			TotalCachedBytes += Info.FileSize;
			TotalFragmentedBytes += ensure(Info.RefSize <= Info.FileSize) ?
				(Info.FileSize - Info.RefSize) : Info.FileSize;

			if (!bPossibleToFreeBytes && Info.RefSize < Info.FileSize)
			{
				// Block is fragmented
				if (CacheState.Cas.SetPendingDefrag(BlockId, SkipLock))
				{
					BlocksToDefragFragmentedSize += (Info.FileSize - Info.RefSize);
					BlocksToDefragTotalSize += Info.FileSize;

					FreedBlockBytes += Info.FileSize;
					NewBlockBytes += Info.RefSize; // For now, assume that nothing will be moved to the current block

					BlocksToDefrag.Emplace(BlockId, Info);

					if (FreedBlockBytes >= NewBlockBytes && FreedBlockBytes - NewBlockBytes >= *TotalBytesToFree)
					{
						bPossibleToFreeBytes = true;
					}
				}
			}
			else if (Info.FileSize < CacheState.Cas.GetMinBlockSize())
			{
				// Block is too small whether or not its fragmented
				if (CacheState.Cas.SetPendingDefrag(BlockId, SkipLock))
				{
					if (ensure(Info.RefSize <= Info.FileSize))
					{
						BlocksToDefragFragmentedSize += (Info.FileSize - Info.RefSize);
					}

					BlocksToDefragTotalSize += Info.FileSize;

					BlocksToDefrag.Emplace(BlockId, Info);
				}
			}
		}

		if (!bPossibleToFreeBytes)
		{
			FStringView ErrorMessage(TEXTVIEW("Failed to defrag the install cache due to too much data being referenced by the game"));
			UE_LOGFMT(LogIoStoreOnDemand, Error, "{ErrorMessage}", ErrorMessage);
			FInstallCacheErrorState State = MakeInstallCacheErrorState(CacheState, TotalCachedBytes, *TotalBytesToFree);
			State.IoError = FIoStatus(EIoErrorCode::OutOfDiskSpace, ErrorMessage);
			FResult Result = MakeError(InstallCacheDefragError(MoveTemp(State)));
			FOnDemandInstallCacheStats::OnDefrag(CacheState.Cas.GetType(), Result, TotalFragmentedBytes, TotalCachedBytes, BlocksToDefragFragmentedSize, BlocksToDefragTotalSize);
			return Result;
		}
	}
	else
	{
		bool bSkippedFragmentedBlock = false;

		// Full defrag
		for (const TPair<FCasBlockId, FCasBlockInfo>& Kv : BlockInfo)
		{
			const FCasBlockId BlockId = Kv.Key;
			const FCasBlockInfo& Info = Kv.Value;

			TotalCachedBytes += Info.FileSize;
			TotalFragmentedBytes += ensure(Info.RefSize <= Info.FileSize) ?
				(Info.FileSize - Info.RefSize) : Info.FileSize;

			if (Info.RefSize < Info.FileSize)
			{
				// Block is fragmented
				if (CacheState.Cas.SetPendingDefrag(BlockId, SkipLock))
				{
					BlocksToDefragFragmentedSize += (Info.FileSize - Info.RefSize);
					BlocksToDefragTotalSize += Info.FileSize;

					BlocksToDefrag.Emplace(BlockId, Info);
				}
				else
				{
					bSkippedFragmentedBlock = true;
				}
			}
			else if (Info.FileSize < CacheState.Cas.GetMinBlockSize())
			{
				// Block is too small whether or not its fragmented
				if (CacheState.Cas.SetPendingDefrag(BlockId, SkipLock))
				{
					if (ensure(Info.RefSize <= Info.FileSize))
					{
						BlocksToDefragFragmentedSize += (Info.FileSize - Info.RefSize);
					}

					BlocksToDefragTotalSize += Info.FileSize;

					BlocksToDefrag.Emplace(BlockId, Info);
				}
				else
				{
					bSkippedFragmentedBlock = true;
				}
			}
		}

		if (BlocksToDefrag.IsEmpty())
		{
			if (bSkippedFragmentedBlock)
			{
				UE_LOGF(LogIoStoreOnDemand, Display, "No %ls cache blocks are eligable for defrag.", LexToString(CacheState.Cas.GetType()));
			}
			else
			{
				// Already defragged
				UE_LOGF(LogIoStoreOnDemand, Display, "%ls cache not fragmented.", LexToString(CacheState.Cas.GetType()));
			}
			
			FResult Result = MakeValue();
			FOnDemandInstallCacheStats::OnDefrag(CacheState.Cas.GetType(), Result, TotalFragmentedBytes, TotalCachedBytes, BlocksToDefragFragmentedSize, BlocksToDefragTotalSize);
			return Result;
		}
	}
	
	if (CasUniqueLock)
	{
		CasUniqueLock.Unlock();
		SkipLock = nullptr;
	}	

	UE_LOG(LogIoStoreOnDemand, Display, TEXT("Defrag found %" UINT64_FMT " fragmented bytes of %" UINT64_FMT " total bytes in %i blocks."), 
		BlocksToDefragFragmentedSize, BlocksToDefragTotalSize, BlocksToDefrag.Num()); 

	if (TotalCachedBytes > CacheState.MaxCacheSize)
	{
		// Ruh-Roh! There's not enough of the disk quota left to run a defrag!
		const FString ErrorMsg = FString::Printf(TEXT("Cache size is greater than disk quota for %s - Cannot Defragment!, MaxCacheSize=%.2lf MiB"),
			LexToString(CacheState.Cas.GetType()), ToMiB(CacheState.MaxCacheSize));
		UE_LOGF(LogIoStoreOnDemand, Error, "%ls", *ErrorMsg);

		FInstallCacheErrorState State = MakeInstallCacheErrorState(CacheState, TotalCachedBytes, TotalBytesToFree ? *TotalBytesToFree : TotalFragmentedBytes);
		State.IoError = FIoStatus(EIoErrorCode::OutOfDiskSpace, ErrorMsg);
		FResult Result = MakeError(InstallCacheDefragError(MoveTemp(State)));

		// Append a critical error entry to clear the cache at next startup
		FCasJournal::FTransaction Transaction = FCasJournal::Begin(CacheState.Cas.GetType(), GetJournalFilename());
		Transaction.CriticalError(FCasJournal::EErrorCode::DefragOutOfDiskSpace);
		if (FResult CommitResult = FCasJournal::Commit(MoveTemp(Transaction)); CommitResult.HasError())
		{
			UE_LOGFMT(LogIoStoreOnDemand, Error, "Failed to commit critical errors to CAS journal ({Error})", CommitResult.GetError());
		}

		FOnDemandInstallCacheStats::OnDefrag(CacheState.Cas.GetType(), Result, TotalFragmentedBytes, TotalCachedBytes, BlocksToDefragFragmentedSize, BlocksToDefragTotalSize);
		return Result;
	}

	// Right now, don't allow moving chunks to the current block for defrag. Its somewhat dangerous and hard to reason about.
	// - Currently, the slack in the current block cannot be determined without opening a write handle to the block.
	// - If we defrag the current block itself, then we would need additional tracking so we don't lose any chunks moved into it.
	// - Additionally, this would also depend on the order blocks are defragged.
	// This should be the only thread writing to CurrentBlock.
	CacheState.CurrentBlock = FCasBlockId::Invalid;

	uint64 TotalRefBytes = 0;

	// Determine chunks that need to be moved for each defrag block
	for (int32 Index = 0; FSharedOnDemandContainer Container : Containers)
	{
		const TBitArray<>& IsReferenced = ChunkEntryIndices[Index++];
		for (int32 EntryIndex = 0; const FOnDemandChunkEntry& Entry : Container->ChunkEntries)
		{
			const FIoChunkId& ChunkId = Container->ChunkIds[EntryIndex];
			if (bool bIsReferenced = IsReferenced[EntryIndex++]; bIsReferenced == false)
			{
				continue;
			}

			if (FCasLocation Loc = CacheState.Cas.FindChunk(Entry.Hash); Loc.IsValid())
			{
				if (FDefragBlock* DefragBlock = Algo::FindBy(BlocksToDefrag, Loc.BlockId, &FDefragBlock::BlockId))
				{
					// TODO: Should this be a map?
					if (nullptr == Algo::FindBy(DefragBlock->ReferencedChunks, Entry.Hash, &FDefragBlockReferencedChunk::Hash))
					{
						DefragBlock->ReferencedChunks.Add(FDefragBlockReferencedChunk
						{
							.Hash = Entry.Hash,
							.ChunkId = ChunkId,  // May not be unique, just return the first found for debugging purposes
							.BlockOffset = Loc.BlockOffset,
							.DiskSize = Entry.GetDiskSize(),
						});
						TotalRefBytes += Entry.GetDiskSize();
					}
				}
			}
		}
	}

	if (BlocksToDefragTotalSize - BlocksToDefragFragmentedSize != TotalRefBytes)
	{
		UE_LOGFMT(LogIoStoreOnDemand, Error, "Possibly corrupt CAS blocks - TotalBlockSize {TotalBlockSize}, FragmentedBytes {FragmentedBytes}, TotalRefBytes {TotalRefBytes}", 
			BlocksToDefragTotalSize, BlocksToDefragFragmentedSize, TotalRefBytes);
		ensure(false);
	}

	// Move chunks to new blocks and delete old blocks
	FPendingChunks DefragPendingChunks;
	for (const FDefragBlock& DefragBlock : BlocksToDefrag)
	{
		if (DefragBlock.ReferencedChunks.IsEmpty() == false)
		{
			constexpr bool bAllowReadFromMMapCas = true;
			TResult<FSharedFileHandle> FileOpenResult = CacheState.Cas.OpenRead(DefragBlock.BlockId, bAllowReadFromMMapCas);
			if (FileOpenResult.HasError())
			{
				UE_LOGFMT(LogIoStoreOnDemand, Error, "Defrag failed to open CAS block for reading {Error}", FileOpenResult.GetError());

				FInstallCacheErrorState State = MakeInstallCacheErrorState(CacheState, TotalCachedBytes, TotalBytesToFree ? *TotalBytesToFree : TotalFragmentedBytes);
				FileOpenResult.GetError().PushErrorContext(FInstallCacheErrorContext{.State = MoveTemp(State)});
				FResult Result = MakeError(FileOpenResult.StealError());
				FOnDemandInstallCacheStats::OnDefrag(CacheState.Cas.GetType(), Result, TotalFragmentedBytes, TotalCachedBytes, BlocksToDefragFragmentedSize, BlocksToDefragTotalSize);
				return Result;
			}

			FSharedFileHandle FileHandle = FileOpenResult.StealValue();

			Algo::SortBy(DefragBlock.ReferencedChunks, &FDefragBlockReferencedChunk::BlockOffset);

			for (const FDefragBlockReferencedChunk& ReffedChunk : DefragBlock.ReferencedChunks)
			{
				FIoBuffer	Buffer(ReffedChunk.DiskSize);
				FResult		ReadResult = MakeValue();

				for (int32 Attempt = 0, MaxAttempts = 3; Attempt < MaxAttempts; ++Attempt)
				{
					if (FileHandle->Seek(ReffedChunk.BlockOffset) == false)
					{
						ReadResult = MakeCasError<void>(
							CacheState.Cas.GetType(),
							ECasErrorCode::ReadBlockFailed, EIoErrorCode::FileSeekFailed, FString::Printf(TEXT("Failed to seek to offset %u in cache block %u"),
								ReffedChunk.BlockOffset, DefragBlock.BlockId.Id));
						continue;
					}
					if (FileHandle->Read(Buffer.GetData(), Buffer.GetSize()) == false)
					{
						ReadResult = MakeCasError<void>(
							CacheState.Cas.GetType(),
							ECasErrorCode::ReadBlockFailed, EIoErrorCode::ReadError, FString::Printf(TEXT("Failed to read %" UINT64_FMT " bytes at offset %u in cache block %u"),
								Buffer.GetSize(), ReffedChunk.BlockOffset, DefragBlock.BlockId.Id));
						continue;
					}

					ReadResult = MakeValue();
					break;
				}

				if (ReadResult.HasError())
				{
					FInstallCacheErrorState State = MakeInstallCacheErrorState(CacheState, TotalCachedBytes, TotalBytesToFree ? *TotalBytesToFree : TotalFragmentedBytes);
					ReadResult.GetError().PushErrorContext(FInstallCacheErrorContext{.State = MoveTemp(State)});
					FResult Result = MakeError(ReadResult.StealError());
					FOnDemandInstallCacheStats::OnDefrag(CacheState.Cas.GetType(), Result, TotalFragmentedBytes, TotalCachedBytes, BlocksToDefragFragmentedSize, BlocksToDefragTotalSize);
					return Result;
				}

				const FOnDemandChunkHash ChunkHash = FOnDemandChunkHash::HashBuffer(Buffer.GetView());
				if (ChunkHash != ReffedChunk.Hash)
				{
					UE_LOGF(LogIoStoreOnDemand, Error, "Found chunk with invalid hash while defragging block, ChunkId='%ls', BlockId=%u, BlockOffset=%u",
						*LexToString(ReffedChunk.ChunkId), DefragBlock.BlockId.Id, ReffedChunk.BlockOffset);

					// Append a critical error entry to clear the cache at next startup
					FCasJournal::FTransaction Transaction = FCasJournal::Begin(CacheState.Cas.GetType(), GetJournalFilename());
					Transaction.CriticalError(FCasJournal::EErrorCode::DefragHashMismatch);
					if (FResult CommitResult = FCasJournal::Commit(MoveTemp(Transaction)); CommitResult.HasError())
					{
						UE_LOGFMT(LogIoStoreOnDemand, Error, "Failed to commit critical errors to CAS journal ({Error})", CommitResult.GetError());
					}

					if (FResult Result = FlushPendingChunks(CacheState, DefragPendingChunks, DefragBlock.LastAccess, DefragBlock.LastModification); Result.HasError())
					{
						FOnDemandInstallCacheStats::OnDefrag(CacheState.Cas.GetType(), Result, TotalFragmentedBytes, TotalCachedBytes, BlocksToDefragFragmentedSize, BlocksToDefragTotalSize);
						return Result;
					}
					check(DefragPendingChunks.IsEmpty());
	
					FResult Result = MakeError(InstallCacheDefragHashMismatchError(
						FInstallCacheDefragHashMismatchError
						{
							.ChunkId = ReffedChunk.ChunkId, // May not be unique, just return the first found for debugging purposes
							.ExpectedHash = LexToString(ReffedChunk.Hash),
							.ActualHash = LexToString(ChunkHash),
							.CasType = CacheState.Cas.GetType()
						}));
					FOnDemandInstallCacheStats::OnDefrag(CacheState.Cas.GetType(), Result, TotalFragmentedBytes, TotalCachedBytes, BlocksToDefragFragmentedSize, BlocksToDefragTotalSize);
					return Result;
				}

				if (DefragPendingChunks.TotalSize > FPendingChunks::MaxPendingBytes)
				{
					if (FResult Result = FlushPendingChunks(CacheState, DefragPendingChunks, DefragBlock.LastAccess, DefragBlock.LastModification); Result.HasError())
					{
						FOnDemandInstallCacheStats::OnDefrag(CacheState.Cas.GetType(), Result, TotalFragmentedBytes, TotalCachedBytes, BlocksToDefragFragmentedSize, BlocksToDefragTotalSize);
						return Result;
					}
					check(DefragPendingChunks.IsEmpty());
				}

				DefragPendingChunks.Append(MoveTemp(Buffer), ReffedChunk.Hash);
			}

			FileHandle.Reset();

			if (FResult Result = FlushPendingChunks(CacheState, DefragPendingChunks, DefragBlock.LastAccess, DefragBlock.LastModification); Result.HasError())
			{
				FOnDemandInstallCacheStats::OnDefrag(CacheState.Cas.GetType(), Result, TotalFragmentedBytes, TotalCachedBytes, BlocksToDefragFragmentedSize, BlocksToDefragTotalSize);
				return Result;
			}
			check(DefragPendingChunks.IsEmpty());
		}

		FCasJournal::FTransaction Transaction = FCasJournal::Begin(CacheState.Cas.GetType(), GetJournalFilename());

		auto HandleChunkRemoved = [&Transaction](const FCasAddr& Addr)
		{
			Transaction.ChunkLocation(FCasLocation::Invalid, Addr);
		};

		// Flushing should overwrite the lookup info for the cas addr to point at the new block.
		// Can now remove the old block
		if (FResult Result = CacheState.Cas.DeleteBlock(DefragBlock.BlockId, HandleChunkRemoved); Result.HasError())
		{
			FOnDemandInstallCacheStats::OnDefrag(CacheState.Cas.GetType(), Result, TotalFragmentedBytes, TotalCachedBytes, BlocksToDefragFragmentedSize, BlocksToDefragTotalSize);
			return Result;
		}

		constexpr bool bFromDefragTrue = true;
		FOnDemandInstallCacheStats::OnBlockDeleted(CacheState.Cas.GetType(), DefragBlock.LastAccess, DefragBlock.LastModification, bFromDefragTrue);

		Transaction.BlockDeleted(DefragBlock.BlockId);

		if (FResult Result = FCasJournal::Commit(MoveTemp(Transaction)); Result.HasError())
		{
			FOnDemandInstallCacheStats::OnDefrag(CacheState.Cas.GetType(), Result, TotalFragmentedBytes, TotalCachedBytes, BlocksToDefragFragmentedSize, BlocksToDefragTotalSize);
			return Result;
		}
	}

	UE_LOG(LogIoStoreOnDemand, Display, TEXT("Defrag removed %" UINT64_FMT " fragmented bytes of %" UINT64_FMT " total bytes in %i blocks from CAS %s."),
		BlocksToDefragFragmentedSize, BlocksToDefragTotalSize, BlocksToDefrag.Num(), LexToString(CacheState.Cas.GetType()));

	if (OutTotalFreedBytes)
	{
		*OutTotalFreedBytes = BlocksToDefragFragmentedSize;
	}

	FResult Result = MakeValue();
	FOnDemandInstallCacheStats::OnDefrag(CacheState.Cas.GetType(), Result, TotalFragmentedBytes, TotalCachedBytes, BlocksToDefragFragmentedSize, BlocksToDefragTotalSize);
	return Result;
}

FResult FOnDemandInstallCache::FlushLastAccess(const TStaticArray<bool, EOnDemandInstallCasType::Count>& bFlushCasLastAccess)
{
	for (EOnDemandInstallCasType CasType : TEnumRange<EOnDemandInstallCasType>())
	{
		if (bFlushCasLastAccess[CasType] == false)
		{
			continue;
		}

		FCacheState* CacheState = GetCacheState(CasType);
		if (CacheState == nullptr)
		{
			continue;
		}

		const FCas::FLastAccess DirtyLastAccess = CacheState->Cas.GetAndClearDirtyLastAccess();
		if (DirtyLastAccess.IsEmpty())
		{
			continue;
		}

		const FString JournalFile = GetJournalFilename();
		FCasJournal::FTransaction Transaction = FCasJournal::Begin(CasType, JournalFile);
		JournalLastAccess(Transaction, DirtyLastAccess);

		if (FResult Result = FCasJournal::Commit(MoveTemp(Transaction)); Result.HasError())
		{
			using namespace UE::IoStore::OnDemand;
			UE_LOGF(LogIoStoreOnDemand, Error, "Failed to update CAS journal '%ls' with block timestamp(s), reason '%ls'",
				*JournalFile, *LexToString(Result.GetError()));
			Result.GetError().PushErrorContext(FInstallCacheErrorContext{.State = MakeInstallCacheErrorState(*CacheState)});
			return Result;
		}
	}

	return MakeValue();
}

void FOnDemandInstallCache::UpdateLastAccess(TConstArrayView<FCasAddr> ChunkAddrs, TStaticArray<bool, EOnDemandInstallCasType::Count>& bInOutLastAccessDirty)
{
	if (ChunkAddrs.IsEmpty())
	{
		return;
	}

	const int64 Now = FDateTime::UtcNow().GetTicks();
	constexpr const bool bDirty = true;

	for (FCacheState& CacheState : CacheStates)
	{
		const EOnDemandInstallCasType CasType = CacheState.Cas.GetType();
		TUniqueLock Lock(CacheState.Cas);

		for (const FCasAddr& Addr : ChunkAddrs)
		{
			if (FCasLocation* CasLoc = CacheState.Cas.Lookup.Find(Addr))
			{
				const FCasBlockId BlockId = CasLoc->BlockId;
				const uint32 BlockIdHash = GetTypeHash(BlockId);
				const bool bUpdatedLastAccess = CacheState.Cas.UnlockedTrackAccessIf(
					ECasTrackAccessType::Granular, BlockIdHash, BlockId, Now, bDirty);

				bInOutLastAccessDirty[CasType] = bInOutLastAccessDirty[CasType] || bUpdatedLastAccess;
			}
		}
	}
}

FOnDemandInstallCacheUsage FOnDemandInstallCache::GetCacheUsage()
{
	// If this is called from a thread other than the OnDemandIoStore tick thread
	// then its possible the block info and containers may not be in sync with each other
	// or the current state of the tick thread.
	// This should only be used for debugging and telemetry purposes.

	FOnDemandInstallCacheUsage CacheUsage;

	for (EOnDemandInstallCasType CasType : TEnumRange<EOnDemandInstallCasType>())
	{
		FOnDemandInstallCacheUsage::FCasUsage& CasUsage = CacheUsage.CasUsage[CasType];
		CasUsage.CasType = CasType;

		FCacheState* CacheState = GetCacheState(CasType);
		if (CacheState == nullptr)
		{
			continue;
		}

		FCasBlockInfoMap	BlockInfo;
		const uint64		TotalCachedBytes = CacheState->Cas.GetBlockInfo(BlockInfo);

		TArray<FSharedOnDemandContainer>	Containers;
		TArray<TBitArray<>>					ChunkEntryIndices;
		IoStore.GetReferencedContent(Containers, ChunkEntryIndices);
		check(Containers.Num() == ChunkEntryIndices.Num());

		uint64 ReferencedBytes = 0;
		AddReferencesToBlocks(*CacheState, Containers, ChunkEntryIndices, BlockInfo, ReferencedBytes);

		uint64 FragmentedBytes = 0;
		uint64 ReferencedBlockBytes = 0;
		for (const TPair<FCasBlockId, FCasBlockInfo>& Kv : BlockInfo)
		{
			const FCasBlockId BlockId = Kv.Key;
			const FCasBlockInfo& Info = Kv.Value;

			if (Info.RefSize < Info.FileSize)
			{
				FragmentedBytes += (Info.FileSize - Info.RefSize);
			}

			if (Info.RefSize > 0)
			{
				ReferencedBlockBytes += Info.FileSize;
			}
		}

		CasUsage.MaxSize = CacheState->MaxCacheSize;
		CasUsage.TotalSize = TotalCachedBytes;
		CasUsage.ReferencedBlockSize = ReferencedBlockBytes;
		CasUsage.ReferencedSize = ReferencedBytes;
		CasUsage.FragmentedChunksSize = FragmentedBytes;
	}

	return CacheUsage;
}

FResult FOnDemandInstallCache::Flush(EOnDemandInstallCasType CasType)
{
	using namespace UE::IoStore::OnDemand;
	FCacheState* CacheState = GetCacheState(CasType);
	if (CacheState == nullptr)
	{
		return MakeCasError<void>(
			CasType, ECasErrorCode::WriteBlockFailed, EIoErrorCode::InvalidParameter, FString::Printf(TEXT("CAS %s is not configured"), LexToString(CasType)));
	}

	if (FResult Result = FlushPendingChunks(*CacheState, CacheState->PendingChunks); Result.HasError())
	{
		return Result;
	}
	check(CacheState->PendingChunks.IsEmpty());

	CacheState->Cas.Compact();
	return MakeValue();
}

FResult FOnDemandInstallCache::FlushPendingChunks(FCacheState& CacheState, FPendingChunks& Chunks, int64 UtcAccessTicks, int64 UtcModificationTicks)
{
	if (Chunks.IsEmpty())
	{
		return MakeValue();
	}

#if UE_ONDEMANDINSTALLCACHE_EXCLUSIVE_WRITE
	UE::Tasks::TTask<FResult> Task = CacheState.ExclusivePipe.Launch(UE_SOURCE_LOCATION, [this, &CacheState, &Chunks, UtcAccessTicks, UtcModificationTicks]
	{
		ON_SCOPE_EXIT 
		{ 
			TUniqueLock Lock(Chunks.SharedMutex);
			Chunks.Reset();
		};

		return FlushPendingChunksImpl(CacheState, Chunks, UtcAccessTicks, UtcModificationTicks);
	}, UE::Tasks::ETaskPriority::BackgroundHigh);

	Task.Wait();

	return Task.GetResult();

#else
	ON_SCOPE_EXIT
	{
		TUniqueLock Lock(Chunks.SharedMutex);
		Chunks.Reset();
	};

	return FlushPendingChunksImpl(CacheState, Chunks, UtcAccessTicks, UtcModificationTicks);
#endif // UE_ONDEMANDINSTALLCACHE_EXCLUSIVE_WRITE
}

FResult FOnDemandInstallCache::FlushPendingChunksImpl(FCacheState& CacheState, const FPendingChunks& Chunks, int64 UtcAccessTicks, int64 UtcModificationTicks)
{
	using namespace UE::IoStore::OnDemand;

	const bool bFromDefrag = (&CacheState.PendingChunks != &Chunks);

	uint64 TotalCasBytes		= 0;
	uint64 TotalJournalBytes	= 0;
	uint32 TotalOpCount			= 0;

	ON_SCOPE_EXIT 
	{ 
		Governor.OnInstallCacheFlushed(TotalCasBytes + TotalJournalBytes, TotalOpCount);
	};

	FLargeMemoryWriter Ar(FMath::Min(Chunks.TotalSize, static_cast<uint64>(CacheState.Cas.GetMaxBlockSize())));

	// This should be the only thread writing to CurrentBlock
	FCasBlockId CurrentBlockId = CacheState.CurrentBlock;

	int32 ChunkIdx = 0;

	while (ChunkIdx < Chunks.Chunks.Num())
	{
		FCasJournal::FTransaction Transaction = FCasJournal::Begin(CacheState.Cas.GetType(), GetJournalFilename());
		
		// Only open for append if continuing a block.
		const bool bAppendToBlock = CurrentBlockId.IsValid();

		if (CurrentBlockId.IsValid() == false)
		{
			CurrentBlockId = CacheState.Cas.CreateBlock();
			ensure(CurrentBlockId.IsValid());
			CacheState.CurrentBlock = CurrentBlockId;
			Transaction.BlockCreated(CurrentBlockId);
		}

		TResult<FUniqueFileHandle> OpenWriteResult = CacheState.Cas.OpenWrite(CurrentBlockId, bAppendToBlock);
		if (OpenWriteResult.HasError())
		{
			OpenWriteResult.GetError().PushErrorContext(FInstallCacheErrorContext{.State = MakeInstallCacheErrorState(CacheState)});
			FResult Result = MakeError(OpenWriteResult.StealError());
			FOnDemandInstallCacheStats::OnFlush(CacheState.Cas.GetType(), Result, TotalCasBytes, bFromDefrag);
			return Result;
		}
		
		FUniqueFileHandle CasFileHandle = OpenWriteResult.StealValue();
		const int64 CasBlockOffset = CasFileHandle->Tell();
		
		TArray<int64>		Offsets;
		const FCasAddr*		ChunkAddrBegin = &Chunks.ChunkAddrs[ChunkIdx];

		while (ChunkIdx < Chunks.Chunks.Num())
		{
			if (CasBlockOffset > 0 && CasBlockOffset + Ar.Tell() + Chunks.Chunks[ChunkIdx].GetSize() > CacheState.Cas.GetMaxBlockSize())
			{
				break;
			}

			if (!ensure(Chunks.Chunks[ChunkIdx].GetSize() <= CacheState.Cas.GetMaxBlockSize()))
			{
				UE_LOGFMT(LogIoStoreOnDemand, Error, "{CasType} CAS - Chunk is larger then block size ({ChunkAddr})",
					("CasType", LexToString(CacheState.Cas.GetType())), 
					("ChunkAddr", LexToString(Chunks.ChunkAddrs[ChunkIdx])));
			}

			const FIoBuffer& Chunk = Chunks.Chunks[ChunkIdx];
			Offsets.Add(CasBlockOffset + Ar.Tell());
			Ar.Serialize(const_cast<uint8*>(Chunk.GetData()), Chunk.GetSize());

			++ChunkIdx;
		}

		TConstArrayView<FCasAddr> ChunkAddrs = MakeArrayView(ChunkAddrBegin, Offsets.Num());
		const int64 BytesToWrite = Ar.Tell();

		if (BytesToWrite > 0)
		{
			UE_LOGF(LogIoStoreOnDemand, Log, "Writing %.2lf MiB to %ls CAS block %u",
				ToMiB(Ar.Tell()), LexToString(CacheState.Cas.GetType()), CurrentBlockId.Id);

			check(CasBlockOffset + BytesToWrite <= CacheState.Cas.GetMaxBlockSize());

			if (CasFileHandle->Write(Ar.GetData(), BytesToWrite) == false)
			{
				FResult Result = MakeCasError<void>(
					CacheState.Cas.GetType(),
					ECasErrorCode::WriteBlockFailed, EIoErrorCode::WriteError, FString::Printf(TEXT("Failed to write %" INT64_FMT " bytes to CAS block %u"), BytesToWrite, CurrentBlockId.Id));
				Result.GetError().PushErrorContext(FInstallCacheErrorContext{.State = MakeInstallCacheErrorState(CacheState)});
				FOnDemandInstallCacheStats::OnFlush(CacheState.Cas.GetType(), Result, TotalCasBytes, bFromDefrag);
				return Result;
			}
			TotalOpCount++;
			TotalCasBytes += uint64(BytesToWrite);

			if (CasFileHandle->Flush() == false)
			{
				FResult Result = MakeCasError<void>(
					CacheState.Cas.GetType(),
					ECasErrorCode::WriteBlockFailed, EIoErrorCode::FileFlushFailed, FString::Printf(TEXT("Failed to flush %" INT64_FMT " bytes to CAS block %u"), BytesToWrite, CurrentBlockId.Id));
				Result.GetError().PushErrorContext(FInstallCacheErrorContext{.State = MakeInstallCacheErrorState(CacheState)});
				FOnDemandInstallCacheStats::OnFlush(CacheState.Cas.GetType(), Result, TotalCasBytes, bFromDefrag);
				return Result;
			}
			TotalOpCount++;

			constexpr const bool bDirty = false;
			if (UtcAccessTicks)
			{
				if (CacheState.Cas.TrackAccessIf(ECasTrackAccessType::Newer, CurrentBlockId, UtcAccessTicks, bDirty))
				{
					Transaction.BlockAccess(CurrentBlockId, UtcAccessTicks);
				}
			}
			else
			{
				const int64 Now = FDateTime::UtcNow().GetTicks();
				verify(CacheState.Cas.TrackAccessIf(ECasTrackAccessType::Always, CurrentBlockId, Now, bDirty));
				Transaction.BlockAccess(CurrentBlockId, Now);
			}

			if (UtcModificationTicks)
			{
				if (CacheState.Cas.TrackModificationIf(ECasTrackModificationType::Newer, CurrentBlockId, UtcModificationTicks))
				{
					Transaction.BlockModification(CurrentBlockId, UtcModificationTicks);
				}
			}
			else
			{
				const int64 Now = FDateTime::UtcNow().GetTicks();
				verify(CacheState.Cas.TrackModificationIf(ECasTrackModificationType::Always, CurrentBlockId, Now));
				Transaction.BlockModification(CurrentBlockId, Now);
			}

			CacheState.Cas.UpdateBlock(CurrentBlockId, IntCastChecked<uint32>(BytesToWrite), ChunkAddrs, Offsets,
				[&Transaction](const FCasAddr& CasAddr, const FCasLocation& Loc)
				{
					Transaction.ChunkLocation(Loc, CasAddr);
				}
			);

			Ar.Seek(0);
		}

		uint64 JrnlBytesWritten	= 0;
		uint32 JrnlOps			= 0;
		if (FResult Result = FCasJournal::Commit(MoveTemp(Transaction), JrnlBytesWritten, JrnlOps); Result.HasError())
		{
			Result.GetError().PushErrorContext(FInstallCacheErrorContext{.State = MakeInstallCacheErrorState(CacheState)});
			FOnDemandInstallCacheStats::OnFlush(CacheState.Cas.GetType(), Result, TotalCasBytes, bFromDefrag);
			return Result;
		}

		TotalJournalBytes	+= JrnlBytesWritten;
		TotalOpCount		+= JrnlOps;

		if (ChunkIdx < Chunks.Chunks.Num())
		{
			CurrentBlockId = FCasBlockId::Invalid;
		}
	}

	FResult Result = MakeValue();
	FOnDemandInstallCacheStats::OnFlush(CacheState.Cas.GetType(), Result, TotalCasBytes, bFromDefrag);
	return Result;
}

void FOnDemandInstallCache::CompleteRequest(FIoRequestImpl* Request, EIoErrorCode Status)
{
	uint64 EncodedSize = 0;
	if (Status == EIoErrorCode::Ok && !Request->IsCancelled())
	{
		FChunkRequest& ChunkRequest = FChunkRequest::GetRef(*Request);
		const FOnDemandChunkInfo& ChunkInfo = ChunkRequest.ChunkInfo;
		FIoBuffer EncodedChunk = MoveTemp(ChunkRequest.EncodedChunk);
		EncodedSize = EncodedChunk.GetSize();

		if (EncodedChunk.GetSize() > 0)
		{
			FIoChunkDecodingParams Params;
			Params.CompressionFormat = ChunkInfo.CompressionFormat();
			Params.EncryptionKey = ChunkInfo.EncryptionKey();
			Params.BlockSize = ChunkInfo.BlockSize();
			Params.TotalRawSize = ChunkInfo.RawSize();
			Params.RawOffset = Request->Options.GetOffset();
			Params.EncodedOffset = ChunkRequest.ChunkRange.GetOffset();
			Params.EncodedBlockSize = ChunkInfo.Blocks();
			Params.BlockHash = ChunkInfo.BlockHashes();

			Request->CreateBuffer(ChunkRequest.RawSize);
			FMutableMemoryView RawChunk = Request->GetBuffer().GetMutableView();

			if (FIoChunkEncoding::Decode(Params, EncodedChunk.GetView(), RawChunk) == false)
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to decode chunk, ChunkId='%ls'", *LexToString(Request->ChunkId));
				Status = EIoErrorCode::CompressionError;
			}
		}
	}

	if (Status != EIoErrorCode::Ok)
	{
		Request->SetLastBackendError(Status);
		Request->SetResult(FIoBuffer());
		TRACE_IOSTORE_BACKEND_REQUEST_FAILED(Request);
	}
	else
	{
		// Ensure buffer is valid, otherwise Request->GetBuffer() will crash without information about the ChunkId
		checkf(Request->HasBuffer(), TEXT("Missing buffer when IoRequest status was reported as Ok, ChunkId=%s"), *LexToString(Request->ChunkId));
		TRACE_IOSTORE_BACKEND_REQUEST_COMPLETED(Request, Request->GetBuffer().GetSize());
	}


	EOnDemandInstallCasType CasType = EOnDemandInstallCasType::None;
	if (const FChunkRequest* ChunkRequest = FChunkRequest::Get(*Request))
	{
		CasType = ChunkRequest->CasType;
	}

	{
		UE::TUniqueLock Lock(Mutex);
		CompletedRequests.AddTail(Request);
	}

	FOnDemandInstallCacheStats::OnReadCompleted(CasType, Status, EncodedSize);

	BackendContext->WakeUpDispatcherThreadDelegate.Execute();
}

FOnDemandInstallCache::FCacheState* FOnDemandInstallCache::GetCacheState(EOnDemandInstallCasType CacheType)
{
	return Algo::FindBy(CacheStates, CacheType, Projection(&FCacheState::Cas, &FCas::GetType));
}

const FOnDemandInstallCache::FCacheState* FOnDemandInstallCache::GetCacheState(EOnDemandInstallCasType CacheType) const
{
	return Algo::FindBy(CacheStates, CacheType, Projection(&FCacheState::Cas, &FCas::GetType));
}

UE::IoStore::OnDemand::FInstallCacheErrorState FOnDemandInstallCache::MakeInstallCacheErrorState(FCacheState& CacheState, uint64 TotalCachedBytes, uint64 RequestedBytes, uint32 LineNo)
{
	UE::IoStore::OnDemand::FInstallCacheErrorState OutState;

	const FString CachRootDirectory = CacheState.Cas.GetRootDirectory();
	OutState.CasType				= LexToString(CacheState.Cas.GetType());
	OutState.bDiskQuerySucceeded	= FPlatformMisc::GetDiskTotalAndFreeSpace(CachRootDirectory, OutState.DiskTotalBytes, OutState.DiskFreeBytes);
	OutState.MaxCacheSize			= CacheState.MaxCacheSize;
	OutState.CacheSize				= TotalCachedBytes;
	OutState.RequestedSize			= RequestedBytes;
	OutState.LineNo					= LineNo;

	return OutState;
}

///////////////////////////////////////////////////////////////////////////////
TSharedPtr<IOnDemandInstallCache> MakeOnDemandInstallCache(
	FOnDemandIoStore& IoStore,
	const FOnDemandInstallCacheConfig& Config,
	FString RootDirectory,
	FDiskCacheGovernor& Governor)
{
	IFileManager& Ifm = IFileManager::Get();
	if (Config.bDropCache)
	{
		UE_LOGF(LogIoStoreOnDemand, Log, "Deleting install cache directory '%ls'", *RootDirectory);
		Ifm.DeleteDirectory(*RootDirectory, false, true);
	}

	const bool bTree = true;
	if (!Ifm.MakeDirectory(*RootDirectory, bTree))
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to create directory '%ls'", *RootDirectory);
		return TSharedPtr<IOnDemandInstallCache>();
	}

	return MakeShared<FOnDemandInstallCache>(Config, MoveTemp(RootDirectory), IoStore, Governor);
}

///////////////////////////////////////////////////////////////////////////////
#if WITH_IOSTORE_ONDEMAND_TESTS

class FTmpDirectoryScope
{
public:
	explicit FTmpDirectoryScope(const FString& InDir)
		: Ifm(IFileManager::Get())
		, Dir(InDir)
	{
		const bool bTree			= true;
		const bool bRequireExists	= false;
		Ifm.DeleteDirectory(*Dir, bRequireExists, bTree);
		Ifm.MakeDirectory(*Dir, bTree);
	}

	~FTmpDirectoryScope()
	{
		const bool bTree			= true;
		const bool bRequireExists	= false;
		Ifm.DeleteDirectory(*Dir, bRequireExists, bTree);
	}
private:
	IFileManager& Ifm;
	FString Dir;
};

FCasAddr CreateCasTestAddr(uint64 Value)
{
	return FCasAddr::From(reinterpret_cast<const uint8*>(&Value), sizeof(uint64));
}

TEST_CASE("IoStore::OnDemand::InstallCache::Journal", "[IoStoreOnDemand][InstallCache]")
{
	const FString TestBaseDir = "TestTmpDir";

	SECTION("CreateJournalFile")
	{
		FTmpDirectoryScope _(TestBaseDir);
		const FString JournalFile = TestBaseDir / TEXT("test.jrn");
		FResult Result = FCasJournal::Create(JournalFile);
		CHECK(Result.HasValue());
	}

	SECTION("SimpleTransaction")
	{
		FTmpDirectoryScope _(TestBaseDir);
		const FString JournalFile = TestBaseDir / TEXT("test.jrn");
		FResult Result = FCasJournal::Create(JournalFile);
		CHECK(Result.HasValue());

		FCasJournal::FTransaction Transaction = FCasJournal::Begin(EOnDemandInstallCasType::General, JournalFile);
		Transaction.BlockCreated(FCasBlockId(1));
		Result = FCasJournal::Commit(MoveTemp(Transaction));
		CHECK(Result.HasValue());
	}

	SECTION("ReplayChunkLocations")
	{
		//Arrange
		TArray<FCasAddr>	ExpectedAddresses;
		TArray<uint32>		ExpectedBlockOffsets;
		const FCasBlockId	ExpectedBlockId(42);
		
		for (int32 Idx = 1; Idx < 33; ++Idx)
		{
			ExpectedAddresses.Add(FCasAddr::From(reinterpret_cast<const uint8*>(&Idx), sizeof(uint32)));
			ExpectedBlockOffsets.Add(Idx);
		}

		// Act
		FTmpDirectoryScope _(TestBaseDir);
		const FString JournalFile = TestBaseDir / TEXT("test.jrn");
		FResult Result = FCasJournal::Create(JournalFile);
		CHECK(Result.HasValue());

		FCasJournal::FTransaction Transaction = FCasJournal::Begin(EOnDemandInstallCasType::General, JournalFile);
		for (int32 Idx = 0; const FCasAddr& Addr : ExpectedAddresses)
		{
			Transaction.ChunkLocation(
				FCasLocation
				{
					.BlockId = ExpectedBlockId,
					.BlockOffset = ExpectedBlockOffsets[Idx]
				},
				Addr);
		}

		Result = FCasJournal::Commit(MoveTemp(Transaction));
		CHECK(Result.HasValue());

		// Assert
		TArray<FCasJournal::FEntry::FChunkLocation> Locs;
		Result = FCasJournal::Replay(
			JournalFile,
			[&Locs](const FCasJournal::FEntry& JournalEntry)
			{
				switch(JournalEntry.Type())
				{
				case FCasJournal::FEntry::EType::ChunkLocation:
				{
					Locs.Add(JournalEntry.ChunkLocation);
					break;
				}
				default:
					CHECK(false);
					break;
				};

				return MakeValue();
			});
		CHECK(Result.HasValue());
		CHECK(Locs.Num() == ExpectedAddresses.Num());
		for (int32 Idx = 0; const FCasJournal::FEntry::FChunkLocation& Loc : Locs)
		{
			const FCasLocation ExpectedLoc = FCasLocation
			{
				.BlockId = ExpectedBlockId,
				.BlockOffset = uint32(Idx + 1)
			};
			CHECK(Loc.CasLocation.BlockId == ExpectedLoc.BlockId);
			CHECK(Loc.CasLocation.BlockOffset == ExpectedLoc.BlockOffset);
		}
	}

	SECTION("ReplayBlockCreatedAndDeleted")
	{
		// Arrange
		const FCasBlockId ExpectedBlockId(42);

		// Act
		FTmpDirectoryScope _(TestBaseDir);
		const FString JournalFile = TestBaseDir / TEXT("test.jrn");

		FResult Result = FCasJournal::Create(JournalFile);
		CHECK(Result.HasValue());

		FCasJournal::FTransaction Tx = FCasJournal::Begin(EOnDemandInstallCasType::General, JournalFile);
		Tx.BlockCreated(ExpectedBlockId);
		Tx.BlockDeleted(ExpectedBlockId);

		Result = FCasJournal::Commit(MoveTemp(Tx));
		CHECK(Result.HasValue());

		// Assert
		FCasBlockId CreatedBlockId;
		FCasBlockId DeletedBlockId;

		Result = FCasJournal::Replay(
			JournalFile,
			[&CreatedBlockId, &DeletedBlockId](const FCasJournal::FEntry& JournalEntry)
			{
				switch(JournalEntry.Type())
				{
				case FCasJournal::FEntry::EType::BlockCreated:
				{
					CreatedBlockId = JournalEntry.BlockOperation.BlockId;
					break;
				}
				case FCasJournal::FEntry::EType::BlockDeleted:
				{
					DeletedBlockId = JournalEntry.BlockOperation.BlockId;
					break;
				}
				default:
					CHECK(false);
					break;
				};

				return MakeValue();
			});

		CHECK(Result.HasValue());
		CHECK(CreatedBlockId == ExpectedBlockId);
		CHECK(DeletedBlockId == ExpectedBlockId);
	}

	SECTION("ReplayBlockAccess")
	{
		// Arrange
		const FCasBlockId ExpectedBlockId(462);
		const uint64 ExpectedTicks = FDateTime::UtcNow().GetTicks();

		// Act
		FTmpDirectoryScope _(TestBaseDir);
		const FString JournalFile = TestBaseDir / TEXT("test.jrn");

		FResult Result = FCasJournal::Create(JournalFile);
		CHECK(Result.HasValue());

		FCasJournal::FTransaction Tx = FCasJournal::Begin(EOnDemandInstallCasType::General, JournalFile);
		Tx.BlockAccess(ExpectedBlockId, ExpectedTicks);

		Result = FCasJournal::Commit(MoveTemp(Tx));
		CHECK(Result.HasValue());

		// Assert
		FCasBlockId BlockId;
		uint64 Ticks = 0;

		Result = FCasJournal::Replay(
			JournalFile,
			[&BlockId, &Ticks](const FCasJournal::FEntry& JournalEntry)
			{
				switch(JournalEntry.Type())
				{
				case FCasJournal::FEntry::EType::BlockAccess:
				{
					const FCasJournal::FEntry::FBlockOperation& Op = JournalEntry.BlockOperation;
					BlockId	= Op.BlockId;
					Ticks	= Op.UtcTicks;
					break;
				}
				default:
					CHECK(false);
					break;
				};

				return MakeValue();
			});

		CHECK(Result.HasValue());
		CHECK(BlockId == ExpectedBlockId);
		CHECK(Ticks == ExpectedTicks);
	}

	SECTION("ReplayBlockModification")
	{
		// Arrange
		const FCasBlockId ExpectedBlockId(462);
		const uint64 ExpectedTicks = FDateTime::UtcNow().GetTicks();

		// Act
		FTmpDirectoryScope _(TestBaseDir);
		const FString JournalFile = TestBaseDir / TEXT("test.jrn");

		FResult Result = FCasJournal::Create(JournalFile);
		CHECK(Result.HasValue());

		FCasJournal::FTransaction Tx = FCasJournal::Begin(EOnDemandInstallCasType::General, JournalFile);
		Tx.BlockModification(ExpectedBlockId, ExpectedTicks);

		Result = FCasJournal::Commit(MoveTemp(Tx));
		CHECK(Result.HasValue());

		// Assert
		FCasBlockId BlockId;
		uint64 Ticks = 0;

		Result = FCasJournal::Replay(
			JournalFile,
			[&BlockId, &Ticks](const FCasJournal::FEntry& JournalEntry)
			{
				switch (JournalEntry.Type())
				{
				case FCasJournal::FEntry::EType::BlockModification:
				{
					const FCasJournal::FEntry::FBlockOperation& Op = JournalEntry.BlockOperation;
					BlockId = Op.BlockId;
					Ticks = Op.UtcTicks;
					break;
				}
				default:
					CHECK(false);
					break;
				};

				return MakeValue();
			});

		CHECK(Result.HasValue());
		CHECK(BlockId == ExpectedBlockId);
		CHECK(Ticks == ExpectedTicks);
	}
}

TEST_CASE("IoStore::OnDemand::InstallCache::Snapshot", "[IoStoreOnDemand][InstallCache]")
{
	const FString TestBaseDir = "TestTmpDir";

	SECTION("SaveLoadRoundtrip")
	{
		// Arrange
		FCasSnapshot ExpectedSnapshot;
		FCasSnapshot::FCasState& ExpectedCasState = ExpectedSnapshot.CasState.Emplace_GetRef();
		ExpectedCasState.CasType = EOnDemandInstallCasType::General;

		const int64 Now = FDateTime::UtcNow().GetTicks();

		for (uint32 Id = 1; Id <= 10; ++Id)
		{
			ExpectedCasState.Blocks.Add(FCasSnapshot::FBlock
			{
				.BlockId	= FCasBlockId(Id),
				.LastAccess = Now + Id,
				.ModTime = Now + 10 + Id
			});

			for (uint32 Idx = 1; Idx <= 10; ++Idx)
			{
				FCasAddr CasAddr = CreateCasTestAddr(Idx);
				FCasLocation Loc = FCasLocation
				{
					.BlockId		= FCasBlockId(Id),
					.BlockOffset	= Idx * 256
				};
				ExpectedCasState.ChunkLocations.Emplace(CasAddr, Loc);
			}
		}
		ExpectedCasState.CurrentBlockId = FCasBlockId(1);

		// Act
		FTmpDirectoryScope _(TestBaseDir);
		const FString SnapshotFile = TestBaseDir / TEXT("test.snp");
		TResult<int64> Result = FCasSnapshot::Save(ExpectedSnapshot, SnapshotFile);
		CHECK(Result.HasValue());
		const FCasSnapshot Snapshot = FCasSnapshot::Load(SnapshotFile).StealValue();

		// Assert
		CHECK(Snapshot.CasState.Num() == ExpectedSnapshot.CasState.Num());

		const FCasSnapshot::FCasState& CasState = Snapshot.CasState[0];

		CHECK(CasState.CasType == ExpectedCasState.CasType);
		CHECK(CasState.Blocks.Num() == ExpectedCasState.Blocks.Num());
		for (int32 Idx = 0; Idx < CasState.Blocks.Num(); ++Idx)
		{
			CHECK(CasState.Blocks[Idx].BlockId == ExpectedCasState.Blocks[Idx].BlockId);
			CHECK(CasState.Blocks[Idx].LastAccess == ExpectedCasState.Blocks[Idx].LastAccess);
			CHECK(CasState.Blocks[Idx].ModTime == ExpectedCasState.Blocks[Idx].ModTime);
		}
		CHECK(CasState.ChunkLocations.Num() == ExpectedCasState.ChunkLocations.Num());
		for (int32 Idx = 0; Idx < CasState.ChunkLocations.Num(); ++Idx)
		{
			CHECK(CasState.ChunkLocations[Idx].Get<0>() == ExpectedCasState.ChunkLocations[Idx].Get<0>());
			CHECK(CasState.ChunkLocations[Idx].Get<1>() == ExpectedCasState.ChunkLocations[Idx].Get<1>());
		}
		CHECK(CasState.CurrentBlockId == ExpectedCasState.CurrentBlockId);
	}

	SECTION("CreateFromJournal")
	{
		// Arrange
		FTmpDirectoryScope _(TestBaseDir);
		const FString JournalFile = TestBaseDir / TEXT("test.jrn");
		const FCasBlockId ExpectedCurrentBlockId(2);

		FResult Result = FCasJournal::Create(JournalFile);
		CHECK(Result.HasValue());

		FCasJournal::FTransaction Tx = FCasJournal::Begin(EOnDemandInstallCasType::General, JournalFile);

		const int64 Now = FDateTime::UtcNow().GetTicks();
		const int64 ExpectedModTime = Now + 2;
		const int64 ExpectedLastAccess = Now + 3;

		// Add a block and some chunk locations
		FCasBlockId Block1(1);
		Tx.BlockCreated(Block1);
		Tx.BlockModification(Block1, Now);
		Tx.BlockAccess(Block1, Now + 1);
		for (int32 Idx = 1; Idx <= 10; ++Idx)
		{
			Tx.ChunkLocation(FCasLocation
			{
				.BlockId		= FCasBlockId(1),
				.BlockOffset	= 256
			},
			CreateCasTestAddr(uint64(Idx) << 32 | 1ull));
		}

		// Remove the block and the corresponding chunk locations
		for (int32 Idx = 1; Idx <= 10; ++Idx)
		{
			Tx.ChunkLocation(FCasLocation::Invalid, CreateCasTestAddr(uint64(Idx) << 32 | 1ull));
		}
		Tx.BlockDeleted(FCasBlockId(1));

		// Add a second block and some chunk locations
		Tx.BlockCreated(ExpectedCurrentBlockId);
		Tx.BlockModification(ExpectedCurrentBlockId, ExpectedModTime);
		Tx.BlockAccess(ExpectedCurrentBlockId, ExpectedLastAccess);
		for (int32 Idx = 1; Idx <= 10; ++Idx)
		{
			Tx.ChunkLocation(FCasLocation
			{
				.BlockId		= ExpectedCurrentBlockId,
				.BlockOffset	= uint32(Idx) * 256
			},
			CreateCasTestAddr(Idx));
		}

		Result = FCasJournal::Commit(MoveTemp(Tx));
		CHECK(Result.HasValue());

		// Act
		TResult<FCasSnapshot> FromJournalResult = FCasSnapshot::FromJournal(JournalFile, FCasSnapshot());
		CHECK(FromJournalResult.HasValue());
		const FCasSnapshot Snapshot = FromJournalResult.StealValue();
		CHECK(Snapshot.CasState.Num() == 1);
		const FCasSnapshot::FCasState& CasState = Snapshot.CasState[0];
		CHECK(CasState.CasType == EOnDemandInstallCasType::General);

		// Assert
		CHECK(CasState.CurrentBlockId == ExpectedCurrentBlockId);
		CHECK(CasState.Blocks.Num() == 1);
		CHECK(CasState.Blocks[0].ModTime == ExpectedModTime);
		CHECK(CasState.Blocks[0].LastAccess == ExpectedLastAccess);
		CHECK(CasState.ChunkLocations.Num() == 10);
		for (int32 Idx = 1; Idx < CasState.ChunkLocations.Num(); ++Idx)
		{
			const FCasAddr Addr = CreateCasTestAddr(Idx);
			const FCasSnapshot::FChunkLocation* Loc =
				Algo::FindByPredicate(
					CasState.ChunkLocations,
					[&Addr](const FCasSnapshot::FChunkLocation& L) { return L.Get<0>() == Addr; });
			CHECK(Loc != nullptr);
			if (Loc != nullptr)
			{
				CHECK(Loc->Get<1>().BlockId == ExpectedCurrentBlockId);
				CHECK(Loc->Get<1>().BlockOffset == uint32(Idx) * 256); 
			}
		}
	}

	SECTION("AppendJournal")
	{
		// Arrange
		FCasSnapshot ExpectedSnapshot;
		FCasSnapshot::FCasState& ExpectedCasState = ExpectedSnapshot.CasState.Emplace_GetRef();
		ExpectedCasState.CasType = EOnDemandInstallCasType::General;

		const int64 Now = FDateTime::UtcNow().GetTicks();

		for (uint32 Id = 1; Id <= 10; ++Id)
		{
			ExpectedCasState.Blocks.Add(FCasSnapshot::FBlock
			{
				.BlockId = FCasBlockId(Id),
				.LastAccess = Now + Id,
				.ModTime = Now + 10 + Id
			});

			for (uint32 Idx = 1; Idx <= 10; ++Idx)
			{
				FCasAddr CasAddr = CreateCasTestAddr(uint64(Id * 10 + Idx));
				FCasLocation Loc = FCasLocation
				{
					.BlockId = FCasBlockId(Id),
					.BlockOffset = Idx * 256
				};
				ExpectedCasState.ChunkLocations.Emplace(CasAddr, Loc);
			}
		}
		ExpectedCasState.CurrentBlockId = FCasBlockId(1);

		FTmpDirectoryScope _(TestBaseDir);
		const FString SnapshotFile = TestBaseDir / TEXT("test.snp");
		const FString JournalFile = TestBaseDir / TEXT("test.jrn");

		TResult<int64> SaveSnapResult = FCasSnapshot::Save(ExpectedSnapshot, SnapshotFile);
		CHECK(SaveSnapResult.HasValue());

		FResult Result = FCasJournal::Create(JournalFile);
		CHECK(Result.HasValue());

		FCasJournal::FTransaction Tx = FCasJournal::Begin(EOnDemandInstallCasType::General, JournalFile);

		// Add a block and some chunk locations
		FCasBlockId Block11(11);
		Tx.BlockCreated(Block11);
		Tx.BlockModification(Block11, Now + 100);
		Tx.BlockAccess(Block11, Now + 101);
		for (uint32 Idx = 1; Idx <= 10; ++Idx)
		{
			Tx.ChunkLocation(FCasLocation
			{
				.BlockId = Block11,
				.BlockOffset = Idx * 256
			},
			CreateCasTestAddr(uint64(110 + Idx)));
		}

		Result = FCasJournal::Commit(MoveTemp(Tx));
		CHECK(Result.HasValue());

		// Act
		TResult<int64> AppendResult = FCasSnapshot::AppendAndResetJournal(SnapshotFile, JournalFile);
		CHECK(AppendResult.HasValue());

		TResult<FCasSnapshot> LoadSnapResult = FCasSnapshot::Load(SnapshotFile);
		CHECK(LoadSnapResult.HasValue());
		const FCasSnapshot Snapshot = LoadSnapResult.StealValue();
		CHECK(Snapshot.CasState.Num() == ExpectedSnapshot.CasState.Num());
		const FCasSnapshot::FCasState& CasState = Snapshot.CasState[0];
		CHECK(CasState.CasType == EOnDemandInstallCasType::General);

		// Assert
		CHECK(CasState.Blocks.Num() == ExpectedCasState.Blocks.Num() + 1);
		for (int32 Idx = 0; Idx < ExpectedCasState.Blocks.Num(); ++Idx)
		{
			CHECK(CasState.Blocks[Idx].BlockId == ExpectedCasState.Blocks[Idx].BlockId);
			CHECK(CasState.Blocks[Idx].LastAccess == ExpectedCasState.Blocks[Idx].LastAccess);
			CHECK(CasState.Blocks[Idx].ModTime == ExpectedCasState.Blocks[Idx].ModTime);
		}
		CHECK(CasState.ChunkLocations.Num() == ExpectedCasState.ChunkLocations.Num() + 10);
		for (int32 Idx = 0; Idx < ExpectedCasState.ChunkLocations.Num(); ++Idx)
		{
			CHECK(CasState.ChunkLocations[Idx].Get<0>() == ExpectedCasState.ChunkLocations[Idx].Get<0>());
			CHECK(CasState.ChunkLocations[Idx].Get<1>() == ExpectedCasState.ChunkLocations[Idx].Get<1>());
		}

		CHECK(CasState.CurrentBlockId == Block11);

		CHECK(CasState.Blocks[10].BlockId == Block11);
		CHECK(CasState.Blocks[10].ModTime == Now + 100);
		CHECK(CasState.Blocks[10].LastAccess == Now + 101);

		for (int32 Idx = 0; Idx < 10; ++Idx)
		{
			const int32 ExpectIdx = ExpectedCasState.ChunkLocations.Num() + Idx;

			FCasAddr ExpectAddr = CreateCasTestAddr(111 + Idx);

			FCasLocation ExpectLoc
			{
				.BlockId = FCasBlockId(11),
				.BlockOffset = (Idx + 1) * 256u
			};

			CHECK(CasState.ChunkLocations[ExpectIdx].Get<0>() == ExpectAddr);
			CHECK(CasState.ChunkLocations[ExpectIdx].Get<1>() == ExpectLoc);
		}
	}
}

TEST_CASE("IoStore::OnDemand::InstallCache::ErrorHandling", "[IoStoreOnDemand][InstallCache]")
{
	using namespace UE::UnifiedError;
	using namespace UE::IoStore::OnDemand;

	SECTION("YesWeNeedToTestSerializeErrorDetailsToCbToMakeSureTheGameDoesNotCrashAtRuntime")
	{
		FResult Result = MakeCasError<void>(
			EOnDemandInstallCasType::General, ECasErrorCode::InitializeFailed, EIoErrorCode::DeleteError, TEXT("Test Message"));
		UE_LOGFMT(LogIoStoreOnDemand, Display, "{Error}", Result.GetError());

		{
			FError Error = InstallCacheFlushError().PushErrorContext(FInstallCacheErrorContext{}, EDetailFilter::All);
			UE_LOGFMT(LogIoStoreOnDemand, Display, "{Short}", Error.GetModuleIdAndErrorCodeString());
			UE_LOGFMT(LogIoStoreOnDemand, Display, "{Error}", Error);
		}
		{
			FError Error = InstallCachePurgeError(FInstallCacheErrorState{}).PushErrorContext(FInstallCacheErrorContext{}, EDetailFilter::All);
			UE_LOGFMT(LogIoStoreOnDemand, Display, "{Error}", Error);
		}
	}
}

#endif // WITH_IOSTORE_ONDEMAND_TESTS

} // namespace UE::IoStore
