// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoDispatcherBackend.h"
#include "IO/IoStoreOnDemand.h"
#include "IO/OnDemandError.h"
#include "IO/GenericHash.h"
#include "Templates/SharedPointer.h"

struct FIoContainerHeader; 

namespace UE::IoStore
{

class FOnDemandContentHandle;
class FOnDemandIoStore;
struct FOnDemandInstallCacheUsage;

using FSharedOnDemandContainer	= TSharedPtr<struct FOnDemandContainer, ESPMode::ThreadSafe>;
using FCasAddr					= FHash96;

template<typename Allocator>
struct TOnDemandChunkInfoList;
using FOnDemandChunkInfoList = TOnDemandChunkInfoList<FDefaultAllocator>;
using FEagerDefragChunkList = TOnDemandChunkInfoList<TInlineAllocator<64>>;

struct FOnDemandInstallCacheStorageUsage
{
	uint64 MaxSize = 0;
	uint64 TotalSize = 0;
	uint64 ReferencedBlockSize = 0;
};

class FInstallCacheHandle
{
public:
	virtual ~FInstallCacheHandle();
};

class FInstallCachePurgeHandle
{
private:
	EOnDemandInstallCasType CasType = EOnDemandInstallCasType::None;
	uint64 BytesToPurge = 0;

	friend class FOnDemandInstallCache;
};

class IOnDemandInstallCache
	: public IIoDispatcherBackend
{
public:
	virtual										~IOnDemandInstallCache() = default;
	virtual TUniquePtr<FInstallCacheHandle>		BeginInstall() = 0;
	virtual void								PinCachedChunks(FInstallCacheHandle& Handle, const FOnDemandInternalContentHandle& ContentHandle, const FOnDemandChunkInfoList& Chunks, TFunctionRef<void(int32, bool)> OnChunkFound) = 0;
	virtual void								PinChunks(FInstallCacheHandle& Handle, const FOnDemandInternalContentHandle& ContentHandle, const FOnDemandChunkInfoList& Chunks, TFunctionRef<void(int32)> OnChunkFound) = 0;
	virtual FResult								PutChunk(EIoChunkType ChunkType, FIoBuffer&& Chunk, const FCasAddr& CasAddr) = 0;
	virtual void								PostPutChunk(FInstallCacheHandle& Handle, EIoChunkType ChunkType) = 0;
	virtual FInstallCachePurgeHandle			BeginPurge(EIoChunkType ChunkType) = 0;
	virtual void								AddToPurge(FInstallCachePurgeHandle& Handle, EIoChunkType ChunkType, uint64 Size) = 0;
	virtual FResult								Purge(const FInstallCachePurgeHandle& Handle) = 0;
	virtual FResult								PurgeAllUnreferenced(EOnDemandInstallCasType CasType, bool bDefrag, const uint64* BytesToPurge = nullptr) = 0;
	virtual FResult								DefragAll(EOnDemandInstallCasType CasType, const uint64* BytesToFree = nullptr) = 0;
	virtual FResult								EagerDefrag(FInstallCacheHandle& Handle) = 0;
	virtual bool								IsEagerDefragRequired(FInstallCacheHandle& Handle) = 0;
	virtual FResult								ConditionallyFlushInstall(FInstallCacheHandle& Handle) = 0;
	virtual FResult								Verify() = 0;
	virtual FResult								FlushLastAccess(const TStaticArray<bool, EOnDemandInstallCasType::Count>& bFlushCasLastAccess) = 0;
	virtual void								UpdateLastAccess(TConstArrayView<FCasAddr> CasAddrs, TStaticArray<bool, EOnDemandInstallCasType::Count>& bInOutLastAccessDirty) = 0;
	virtual FOnDemandInstallCacheUsage			GetCacheUsage() = 0;
};

struct FOnDemandInstallCasConfig
{
	EOnDemandInstallCasType Type = EOnDemandInstallCasType::None;
	FString CasSubdirectory;
	uint64	DiskQuota = 1ull << 30;
	uint64	MinBlockSize = 32 << 19;
	uint64	MaxBlockSize = 32 << 20;
	double	LastAccessGranularitySeconds = 60 * 60;
	uint32	EagerDefragBlockCount = 0;
};

struct FOnDemandInstallCacheConfig
{
	TArray<FOnDemandInstallCasConfig> CasConfig;

	FString IndexedCacheStorageName;
	uint64	JournalMaxSize	= 2ull << 20;
	bool	bDropCache		= false;
};

TSharedPtr<IOnDemandInstallCache> MakeOnDemandInstallCache(
	FOnDemandIoStore& IoStore,
	const FOnDemandInstallCacheConfig& Config,
	FString RootDirectory,
	class FDiskCacheGovernor& Governor);

} // namespace UE::IoStore
