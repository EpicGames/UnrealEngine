// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnDemandIoStore.h"

#include "Algo/Accumulate.h"
#include "Algo/AnyOf.h"
#include "Algo/RemoveIf.h"
#include "Algo/Transform.h"
#include "Algo/IsSorted.h"
#include "Async/UniqueLock.h"
#include "Containers/StringConv.h"
#include "Containers/Ticker.h"
#include "DebugCommands.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "HAL/LowLevelMemTracker.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "IndexedCacheStorageManager.h"
#include "IO/IoChunkEncoding.h"
#include "IO/IoContainerHeader.h"
#include "IO/IoStatus.h"
#include "IO/IoStoreOnDemand.h"
#include "Misc/PathTypes.h"
#include "Serialization/PackageStore.h"
#include "IasCache.h"
#include "Logging/StructuredLog.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/CoreDelegatesInternal.h"
#include "Misc/EncryptionKeyManager.h"
#include "Misc/Guid.h"
#include "Misc/PathViews.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "OnDemandConfig.h"
#include "OnDemandContentInstallReplay.h"
#include "OnDemandHttpClient.h"
#include "OnDemandHttpIoDispatcher.h"
#include "OnDemandHttpIoDispatcherBackend.h"
#include "OnDemandInstallCache.h"
#include "OnDemandContentInstaller.h"
#include "OnDemandPackageStoreBackend.h"
#include "ProfilingDebugging/AssetMetadataTrace.h"
#include "Serialization/MemoryReader.h"
#include "Statistics.h"

#if !(UE_BUILD_SHIPPING|UE_BUILD_TEST)
#include "String/LexFromString.h"
#endif

#ifndef UE_IOSTORE_ONDEMAND_DEVMODE_ENABLED
#define UE_IOSTORE_ONDEMAND_DEVMODE_ENABLED 0
#endif

///////////////////////////////////////////////////////////////////////////////
namespace UE::IoStore
{

bool GIaxPackageStreamingEnabled = false;
static FAutoConsoleVariableRef CVar_PackageStreamingEnabled(
	TEXT("iax.PackageStreamingEnabled"),
	GIaxPackageStreamingEnabled,
	TEXT("Enables streaming packages without pre-installing into the install cache.")
);

bool GIaxMemoryMapTocEnabled = true;
static FAutoConsoleVariableRef CVar_IaxMemoryMappedTocEnabled(
	TEXT("iax.MemoryMappedTocEnabled"),
	GIaxMemoryMapTocEnabled,
	TEXT("Whether container TOC memory-mapping is enabled.")
);

bool GIaxDevModeEnabled = UE_IOSTORE_ONDEMAND_DEVMODE_ENABLED;
static FAutoConsoleVariableRef CVar_IaxDevModeEnabled (
	TEXT("iax.DevModeEnabled"),
	GIaxDevModeEnabled,
	TEXT("Whether IAX development mode is enabled or not.")
);

extern int32 GIasHttpCacheJournalMagic;

///////////////////////////////////////////////////////////////////////////////
bool IsDevModeEnabled()
{
#if UE_BUILD_SHIPPING
	return false;
#else
	static bool bDevModeEnabled = FParse::Param(FCommandLine::Get(), TEXT("iax.DevMode"));
	static bool bDevModeDisabled = FParse::Param(FCommandLine::Get(), TEXT("iax.DisableDevMode"));

	return !bDevModeDisabled && (GIaxDevModeEnabled || bDevModeEnabled);
#endif
}

///////////////////////////////////////////////////////////////////////////////
bool IsPackageStreamingEnabled()
{
#if UE_BUILD_SHIPPING
	return false;
#else
	static bool bPackageStreamingEnabled = FParse::Param(FCommandLine::Get(), TEXT("iax.PackageStreamingEnabled"));
	return bPackageStreamingEnabled || GIaxPackageStreamingEnabled;
#endif
}

///////////////////////////////////////////////////////////////////////////////
bool IsIasEnabled()
{
#if UE_BUILD_SHIPPING
	return true;
#else
	bool bEnabled = true;
#if WITH_EDITOR
	static const bool bEnabledInEditor = []
	{
		bool bConfigValue = false;
		if (GConfig)
		{
			GConfig->GetBool(TEXT("Ias"), TEXT("EnableInEditor"), bConfigValue, GEngineIni);
		}
		return bConfigValue;
	}();
	bEnabled = bEnabledInEditor;
#endif // WITH_EDITOR
	static bool bNoIas = FParse::Param(FCommandLine::Get(), TEXT("NoIas"));
	return bEnabled && !bNoIas;
#endif
}

///////////////////////////////////////////////////////////////////////////////
namespace Private
{

///////////////////////////////////////////////////////////////////////////////
static TIoStatusOr<FIoBuffer> DecodeChunk(const FOnDemandChunkInfo& ChunkInfo, FMemoryView EncodedChunk)
{
	FIoChunkDecodingParams Params;
	Params.CompressionFormat = ChunkInfo.CompressionFormat();
	Params.EncryptionKey = ChunkInfo.EncryptionKey();
	Params.BlockSize = ChunkInfo.BlockSize();
	Params.TotalRawSize = ChunkInfo.RawSize();
	Params.RawOffset = 0;
	Params.EncodedOffset = 0;
	Params.EncodedBlockSize = ChunkInfo.Blocks();
	Params.BlockHash = ChunkInfo.BlockHashes();

	FIoBuffer OutRawChunk = FIoBuffer(ChunkInfo.RawSize());
	if (FIoChunkEncoding::Decode(Params, EncodedChunk, OutRawChunk.GetMutableView()) == false)
	{
		FIoStatus Status = FIoStatusBuilder(EIoErrorCode::ReadError)
			<< TEXT("Failed to decode container chunk");
		return Status;
	}

	return OutRawChunk;
}

///////////////////////////////////////////////////////////////////////////////
static TIoStatusOr<FSharedContainerHeader> DeserializeContainerHeader(const FOnDemandChunkInfo& ChunkInfo, FMemoryView EncodedHeaderChunk, bool bReadSoftRefs)
{
	TIoStatusOr<FIoBuffer> Chunk = DecodeChunk(ChunkInfo, EncodedHeaderChunk);
	if (Chunk.IsOk() == false)
	{
		return Chunk.Status();
	}

	FSharedContainerHeader OutHeader = MakeShared<FIoContainerHeader>();
	FMemoryReaderView Ar(Chunk.ValueOrDie().GetView());
	Ar << *OutHeader;
	if (bReadSoftRefs && OutHeader->SoftPackageReferencesSerialInfo.Size > 0)
	{
		if (OutHeader->SoftPackageReferencesSerialInfo.Offset < 0)
		{
			FIoStatus Status = FIoStatusBuilder(EIoErrorCode::ReadError)
				<< FString::Printf(TEXT("Invalid soft package reference offset '%" INT64_FMT "'"), OutHeader->SoftPackageReferencesSerialInfo.Offset);
			return Status;
		}
		if ((OutHeader->SoftPackageReferencesSerialInfo.Offset + OutHeader->SoftPackageReferencesSerialInfo.Size) > Ar.TotalSize())
		{
			FIoStatus Status = FIoStatusBuilder(EIoErrorCode::ReadError)
				<< FString::Printf(TEXT("Soft package reference offset '%" INT64_FMT "' and size '%" INT64_FMT "' will seek past the end of archive size '%" INT64_FMT "'"),
					OutHeader->SoftPackageReferencesSerialInfo.Offset,
					OutHeader->SoftPackageReferencesSerialInfo.Size,
					Ar.TotalSize());
			return Status;
		}
		Ar.Seek(OutHeader->SoftPackageReferencesSerialInfo.Offset);
		Ar << OutHeader->SoftPackageReferences;
	}
	Ar.Close();

	if (Ar.IsError() || Ar.IsCriticalError())
	{
		FIoStatus Status = FIoStatusBuilder(EIoErrorCode::FileNotOpen) << TEXT("Failed to serialize container header");
		return Status;
	}

	return OutHeader; 
}

///////////////////////////////////////////////////////////////////////////////
static TIoStatusOr<FSharedContainerHeader> DeserializeContainerHeader(FSharedOnDemandContainer Container, FMemoryView EncodedHeaderChunk)
{
	// Only serialize package dependencies for containers used for installing content or when stream-all-the-things is enabled
	const bool bShouldSerializeHeader = EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::InstallOnDemand) || IsPackageStreamingEnabled();
	if (bShouldSerializeHeader == false || EncodedHeaderChunk.IsEmpty())
	{
		FSharedContainerHeader Empty = MakeShared<FIoContainerHeader>();
		return Empty;
	}

	const FIoChunkId ChunkId			= CreateContainerHeaderChunkId(Container->ContainerId());
	const FOnDemandChunkInfo ChunkInfo	= FOnDemandChunkInfo::Find(Container, ChunkId); ChunkInfo.IsValid();

	// Create an empty header if the container does not have package dependencies
	if (ChunkInfo.IsValid() == false)
	{
		FSharedContainerHeader Empty = MakeShared<FIoContainerHeader>();
		return Empty;
	}

	return Private::DeserializeContainerHeader(
		ChunkInfo,
		EncodedHeaderChunk,
		EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::WithSoftReferences));
}

///////////////////////////////////////////////////////////////////////////////
static void LogMounted(FSharedOnDemandContainer Container)
{
	using namespace UE::IoStore::Serialization;

	TStringBuilder<128> Sb;
	Sb << Container->Flags;
	UE_LOGF(LogIoStoreOnDemand, Log, "Mounting container '%ls', Entries=%d, Flags='%ls', HostGroup='%ls', TocSize=%.2lf KiB, Storage='%ls'",
		*Container->Name(), Container->ChunkEntries.Num(), Sb.ToString(), *Container->HostGroupName.ToString(),
		double(Container->Storage.GetView().GetSize()) / 1024.0,
		Container->Storage.StorageType() == FOnDemandTocStorageType::MemoryMapped ? TEXT("MemoryMapped") : TEXT("Memory"));
}

} // namespace UE::IoStore::Private

///////////////////////////////////////////////////////////////////////////////
const FOnDemandChunkEntry FOnDemandChunkInfo::NullEntry = {};

///////////////////////////////////////////////////////////////////////////////
static FString OnDemandContainerUniqueName(FStringView MountId, FStringView Name)
{
	return FString::Printf(TEXT("%.*s-%.*s"), MountId.Len(), MountId.GetData(), Name.Len(), Name.GetData());
}

FString FOnDemandContainer::UniqueName() const
{
	return OnDemandContainerUniqueName(MountId, Name());
}

///////////////////////////////////////////////////////////////////////////////
FOnDemandIoStore::FOnDemandIoStore()
{
#if !UE_BUILD_SHIPPING
	DebugCommands = MakeUnique<FOnDemandDebugCommands>(this);
#endif // !UE_BUILD_SHIPPING

	FEncryptionKeyManager::Get().OnKeyAdded().AddRaw(this, &FOnDemandIoStore::OnEncryptionKeyAdded);
}

FOnDemandIoStore::~FOnDemandIoStore()
{
	FEncryptionKeyManager::Get().OnKeyAdded().RemoveAll(this);

	if (OnMountPakHandle.IsValid())
	{
		FCoreInternalDelegates::GetOnPakMountOperation().Remove(OnMountPakHandle);
	}

	if (OnServerPostForkHandle.IsValid())
	{
		FCoreDelegates::OnPostFork.Remove(OnServerPostForkHandle);
	}

	if (TickFuture.IsValid())
	{
		TickFuture.Wait();
	}

	Installer.Reset();
	if (FHttpIoDispatcher::IsInitialized())
	{
		FHttpIoDispatcher::OnHostGroupRegistered().RemoveAll(this);
		if (FIoStatus Status = FHttpIoDispatcher::Shutdown(); Status.IsOk() == false)
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "Failed to shutdown HTTP I/O dispatcher, reason: %ls",
				*Status.ToString());
		}
	}
}

FIoStatus FOnDemandIoStore::Initialize()
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/OnDemandIoStore"));

	TIoStatusOr<FOnDemandInstallCacheConfig> InstallCacheConfig = Config::TryParseInstallCacheConfig(FCommandLine::Get());

	FIoStatus InstallCacheStatus = InstallCacheConfig.IsOk() ?
		InitializeInstallCache(InstallCacheConfig.ValueOrDie()) : InstallCacheConfig.Status();
	if (InstallCacheStatus.IsOk())
	{
		Installer = MakeUnique<FOnDemandContentInstaller>(*this);
	}
	else if (InstallCacheStatus.GetErrorCode() == EIoErrorCode::PendingFork)
	{
		UE_LOGF(LogIoStoreOnDemand, Log, "Deferring initialization of install cache until post server fork")
		OnServerPostForkHandle = FCoreDelegates::OnPostFork.AddLambda(
			[this, InstallCacheConfig = MoveTemp(InstallCacheConfig)](EForkProcessRole ProcessRole) mutable
			{
				if (ProcessRole == EForkProcessRole::Child)
				{
					FIoStatus InstallCacheStatus = InstallCacheConfig.IsOk() ?
						InitializeInstallCache(InstallCacheConfig.ValueOrDie()) : InstallCacheConfig.Status();
					if (InstallCacheStatus.IsOk())
					{
						Installer = MakeUnique<FOnDemandContentInstaller>(*this);
					}
					else
					{
						if (InstallCacheStatus.GetErrorCode() == EIoErrorCode::Disabled)
						{
							UE_LOGF(LogIoStoreOnDemand, Log, "Install cache disabled");
						}
						else
						{
							UE_LOGF(LogIoStoreOnDemand, Error, "Failed to initialize install cache, reason '%ls'", *InstallCacheStatus.ToString());
						}
					}
				}
			});
		InstallCacheStatus = FIoStatus::Ok;
	}
	else if (InstallCacheStatus.GetErrorCode() == EIoErrorCode::Disabled)
	{
		UE_LOGF(LogIoStoreOnDemand, Log, "Install cache disabled");
	}

	{
		TUniquePtr<IIasCache> HttpCache;
		FIasCacheConfig CacheConfig = Config::GetStreamingCacheConfig(FCommandLine::Get());
		
		if (CacheConfig.DiskQuota > 0)
		{
			if (FPaths::HasProjectPersistentDownloadDir())
			{
				CacheConfig.JournalMagic = GIasHttpCacheJournalMagic;

				FString CacheDir = FPaths::ProjectPersistentDownloadDir();
				HttpCache = MakeIasCache(*CacheDir, CacheConfig, DiskCacheGovernor);

				UE_CLOGF(!HttpCache.IsValid(), LogIoStoreOnDemand, Warning, "HTTP cache disabled");
			}
			else
			{
				UE_LOGF(LogIoStoreOnDemand, Warning, "HTTP cache disabled - streaming only (project has no persistent download dir enabled for this platform)");
			}
		}
		else
		{
			UE_LOGF(LogIoStoreOnDemand, Log, "HTTP cache disabled - streaming only (zero-quota)");
		}

		TSharedPtr<IOnDemandHttpIoDispatcher> HttpDispatcher = MakeOnDemanHttpIoDispatcher(MoveTemp(HttpCache));
		if (FIoStatus Status = FHttpIoDispatcher::Initialize(HttpDispatcher); !Status.IsOk())
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "Failed to initialize HTTP I/O dispatcher");
		}
		
		int32 HttpBackendPriority = -10;
#if !UE_BUILD_SHIPPING
		if (FParse::Param(FCommandLine::Get(), TEXT("Ias")))
		{
			UE_LOGF(LogIoStoreOnDemand, Log, "Setting HTTP backend priority higher than file system backend");
			HttpBackendPriority = 10;
		}
#endif
		HttpIoBackend = MakeOnDemandHttpIoDispatcherBackend(*this);
		FIoDispatcher::Get().Mount(HttpIoBackend.ToSharedRef(), HttpBackendPriority);
		FHttpIoDispatcher::OnHostGroupRegistered().AddRaw(this, &FOnDemandIoStore::OnHostGroupRegistered);
	}

	return InstallCacheStatus;
}

FIoStatus FOnDemandIoStore::InitializePostHotfix()
{
	FOnDemandIoStoreConfig Config;
	if (TIoStatusOr<FOnDemandIoStoreConfig> Status = Config::TryParseConfig(FCommandLine::Get()); Status.IsOk())
	{
		Config = Status.ConsumeValueOrDie();
	}
	else
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to parse config, reason '%ls'", *Status.Status().ToString());
	}

	bool bFirstAttempt = false;
	if (FIoStatus Status = LoadDefaultHttpCertificates(bFirstAttempt); !Status.IsOk())
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to load certificates, reason '%ls'", *Status.ToString());
	}

	for (const FOnDemandHostGroupConfig& HostConfig : Config.HostConfigs)
	{
		if (FIoStatus Status = FHttpIoDispatcher::RegisterHostGroup(HostConfig.HostGroupName, HostConfig.Urls); !Status.IsOk())
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "Failed to register host group, reason '%ls'", *Status.ToString());
		}
	}

	UE_LOGF(LogIoStoreOnDemand, Log, "Using per container TOCs=%ls", Config.bUsePerContainerTocs ? TEXT("True") : TEXT("False"));
	if (Config.bUsePerContainerTocs)
	{
		OnMountPakHandle = FCoreInternalDelegates::GetOnPakMountOperation().AddLambda(
			[this](EMountOperation Operation, const TCHAR* ContainerPath, int32 Order) -> void
			{
				IPlatformFile& Ipf = IPlatformFile::GetPlatformPhysical();
				const FString OnDemandTocPath = FPathViews::ChangeExtension(ContainerPath, *FOnDemandToc::FileExt);

				if (Ipf.FileExists(*OnDemandTocPath) == false)
				{
					return;
				}

				switch (Operation)
				{
				case EMountOperation::Mount:
				{
					UE::FManualResetEvent DoneEvent;
					Mount(FOnDemandMountArgs
					{
						.MountId = OnDemandTocPath,
						.FilePath = OnDemandTocPath
					},
					[&DoneEvent](TIoStatusOr<FOnDemandMountResult> Result)
					{
						UE_CLOGF(!Result.IsOk(), LogIoStoreOnDemand, Error,
							"Failed to mount container, reason '%ls'", *Result.Status().ToString());
						DoneEvent.Notify();
					});
					DoneEvent.Wait();
					break;
				}
				case EMountOperation::Unmount:
				{
					const FIoStatus Status = Unmount(OnDemandTocPath);
					UE_CLOGF(!Status.IsOk(), LogIoStoreOnDemand, Error,
						"Failed to unmount container, reason '%ls'", *Status.ToString());
					break;
				}
				default:
					checkNoEntry();
				}
			}
		);

		{
			FCurrentlyMountedPaksDelegate& Delegate = FCoreInternalDelegates::GetCurrentlyMountedPaksDelegate();
			if (Delegate.IsBound())
			{
				IPlatformFile& Ipf = IPlatformFile::GetPlatformPhysical();
				TArray<FMountedPakInfo> PakInfo = Delegate.Execute();
				for (const FMountedPakInfo& Info : PakInfo)
				{
					check(Info.PakFile != nullptr);
					const FString FilePath = FPathViews::ChangeExtension(Info.PakFile->PakGetPakFilename(), *FOnDemandToc::FileExt);
					if (Ipf.FileExists(*FilePath))
					{
						Config.StartupMountArgs.Add(FOnDemandMountArgs
						{
							.MountId	= FilePath,
							.FilePath	= FilePath,
							.Options	= EOnDemandMountOptions::WithSoftReferences
						});
					}
				}
			}
		}
	}

#if !UE_BUILD_SHIPPING
	if (FParse::Param(FCommandLine::Get(), TEXT("Iad.RecordReplay")))
	{
		// If we don't start recording at startup, then we can miss content handles being created.
		// Recording should only be started via command line; however, it should be possible to stop recording at any time.
		static FOnDemandContentInstallReplayRecorder Recorder;
		FOnDemandContentInstallReplayRecorder::StartRecording(&Recorder);
	}
#endif

	for (FOnDemandMountArgs& MountArgs : Config.StartupMountArgs)
	{
		EnqueueMountRequest(MoveTemp(MountArgs), [](FOnDemandMountResult MountResult)
		{
			MountResult.LogResult();
		});
	}

	return FIoStatus::Ok;
}

FOnDemandRegisterHostGroupResult FOnDemandIoStore::RegisterHostGroup(FOnDemandRegisterHostGroupArgs&& Args)
{
	using namespace UE::Core;

	auto SanitizeHostNames = [bUseSecureHttp = Args.bUseSecureHttp](TArrayView<FString> HostNames) -> TArray<FAnsiString>
	{
		TArray<FAnsiString> OutHostNames;
		for (FString& HostName : HostNames)
		{
			if (HostName.StartsWith(TEXT("http")) == false)
			{
				HostName = TEXT("https://") + HostName;
			}
			if (bUseSecureHttp == false)
			{
				HostName.ReplaceInline(TEXT("https"), TEXT("http"));
			}
			HostName.RemoveFromEnd(TEXT("/"));
			OutHostNames.Add(FAnsiString(StringCast<ANSICHAR>(*HostName)));
		}

		return OutHostNames;
	};

	if (Args.HostGroupName.IsNone())
	{
		return FOnDemandRegisterHostGroupResult
		{
			.Error = ArgumentError(TEXT("HostGroupName"), TEXT("Host group name cannot be empty"))
		};
	}

	if (Args.HostNames.IsEmpty())
	{
		return FOnDemandRegisterHostGroupResult
		{
			.Error = ArgumentError(TEXT("HostNames"), TEXT("No host name(s) specified"))
		};
	}

	TArray<FAnsiString> HostNames	= SanitizeHostNames(Args.HostNames);
	FAnsiString AnsiTestUrl			= FAnsiString(StringCast<ANSICHAR>(*Args.TestUrl));

	if (FIoStatus Status = FHttpIoDispatcher::RegisterHostGroup(Args.HostGroupName, HostNames, AnsiTestUrl); !Status.IsOk())
	{
		return FOnDemandRegisterHostGroupResult
		{
			.Error = ArgumentError(TEXT("Args"), Status.ToString())
		};
	}

	FIASHostGroup RegisteredHostGroup = FHostGroupManager::Get().Find(Args.HostGroupName); // The HTTP I/O dispatcher currenlty uses the host group manager
	return FOnDemandRegisterHostGroupResult
	{
		.HostGroup = RegisteredHostGroup.GetUnderlyingHostGroup()
	};
}

void FOnDemandIoStore::Mount(FOnDemandMountArgs&& Args, FOnDemandMountCompleted&& OnCompleted)
{
	// The global.uondemandtoc is currently a special TOC but will go away once we start with per container TOCs
	if (FPathViews::GetBaseFilename(Args.FilePath) == TEXTVIEW("global"))
	{
		UE_LOGF(LogIoStoreOnDemand, Warning, "Trying to mount the global container, please update install bundle configuration");
		return OnCompleted(FOnDemandMountResult
		{
			.MountId	= MoveTemp(Args.MountId),
			.Status		= FIoStatus::Ok
		});
	}

	EnqueueMountRequest(MoveTemp(Args), MoveTemp(OnCompleted));
}

FOnDemandInstallRequest FOnDemandIoStore::Install(
	FOnDemandInstallArgs&& Args,
	FOnDemandInstallCompleted&& OnCompleted,
	FOnDemandInstallProgressed&& OnProgress)
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/OnDemandIoStore"));
	
	FSharedInternalInstallRequest InstallRequest = Installer->EnqueueInstallRequest(
		MoveTemp(Args),
		MoveTemp(OnCompleted),
		MoveTemp(OnProgress));

	return FOnDemandInstallRequest(InstallRequest);
}

void FOnDemandIoStore::Purge(FOnDemandPurgeArgs&& Args, FOnDemandPurgeCompleted&& OnCompleted)
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/OnDemandIoStore"));
	Installer->EnqueuePurgeRequest(MoveTemp(Args), MoveTemp(OnCompleted));
}

void FOnDemandIoStore::Defrag(FOnDemandDefragArgs&& Args, FOnDemandDefragCompleted&& OnCompleted)
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/OnDemandIoStore"));
	Installer->EnqueueDefragRequest(MoveTemp(Args), MoveTemp(OnCompleted));
}

void FOnDemandIoStore::Verify(FOnDemandVerifyCacheCompleted&& OnCompleted)
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/OnDemandIoStore"));
	Installer->EnqueueVerifyRequest(MoveTemp(OnCompleted));
}

FIoStatus FOnDemandIoStore::Unmount(FStringView MountId)
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/OnDemandIoStore"));
	UE_LOGF(LogIoStoreOnDemand, Log, "Unmounting '%ls'", *WriteToString<256>(MountId));

	bool bPendingMount = false;

	{
		UE::TUniqueLock Lock(RequestMutex);
		bPendingMount = Algo::AnyOf(MountRequests, 
			[MountId](const FSharedMountRequest& Request) { return Request->Args.MountId == MountId; });
	}

	if (bPendingMount)
	{
		return FIoStatusBuilder(EIoErrorCode::InvalidParameter) << TEXT("Mount requests pending for MountId");
	}

	{
		TUniqueLock Lock(ContainerMutex);

		bool bRemoved = false;
		Containers.SetNum(Algo::RemoveIf(Containers, [this, &MountId, &bRemoved](const FSharedOnDemandContainer& Container)
		{
			if (Container->MountId == MountId)
			{
				ensureMsgf(Container->ChunkEntryReferences.IsEmpty(), 
					TEXT("Container is still referenced when unmounting, ContainerName='%s', MountId='%s'"), 
					*Container->Name(), *Container->MountId);

				UE_LOGF(LogIoStoreOnDemand, Log, "Unmounting container, ContainerName='%ls', MountId='%ls'",
					*Container->Name(), *Container->MountId);

				bRemoved = true;
				return true;
			}

			return false;
		}));

		if (bRemoved && ensureMsgf(PackageStoreBackend, TEXT("PackageStoreBackend is null, is this a prefork server?")))
		{
			PackageStoreBackend->NeedsUpdate();
		}
	}

	return EIoErrorCode::Ok;
}

TIoStatusOr<FOnDemandInstallSizeResult> FOnDemandIoStore::GetInstallSize(const FOnDemandGetInstallSizeArgs& Args) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandIoStore::GetInstallSize);

	using namespace UE::IoStore::Private;

	TSet<FSharedOnDemandContainer> ContainersForInstallation;
	TSet<FPackageId> PackageIdsToInstall;

	if (FIoStatus Status = GetContainersAndPackagesForInstall(
		Args.MountId,
		Args.TagSets,
		Args.PackageIds,
		ContainersForInstallation,
		PackageIdsToInstall); !Status.IsOk())
	{
		return Status;
	}

	const bool bIncludeSoftReferences	= EnumHasAnyFlags(Args.Options, EOnDemandGetInstallSizeOptions::IncludeSoftReferences);
	const bool bIncludeOptionalBulkData = EnumHasAnyFlags(Args.Options, EOnDemandGetInstallSizeOptions::IncludeOptionalBulkData);
	TArray<FOnDemandChunkInfoList> ResolvedChunks;
	TSet<FIoChunkId> Missing;

	ResolveChunksToInstall(
		PackageStoreBackend,
		ContainersForInstallation,
		PackageIdsToInstall,
		bIncludeSoftReferences,
		bIncludeOptionalBulkData,
		ResolvedChunks,
		Missing);

	FOnDemandInstallSizeResult Result;

	Result.InstallSize = Algo::TransformAccumulate(
		ResolvedChunks,
		[](const FOnDemandChunkInfoList& R) { return R.TotalDiskSize(); },
		uint64(0));

	return Result;
}

FIoStatus FOnDemandIoStore::GetInstallSizesByMountId(const FOnDemandGetInstallSizeArgs& Args, TMap<FString, uint64>& OutSizesByMountId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandIoStore::GetInstallSizesByMountId);

	using namespace UE::IoStore::Private;

	TSet<FSharedOnDemandContainer> ContainersForInstallation;
	TSet<FPackageId> PackageIdsToInstall;

	if (FIoStatus Status = GetContainersAndPackagesForInstall(
		Args.MountId,
		Args.TagSets,
		Args.PackageIds,
		ContainersForInstallation,
		PackageIdsToInstall); !Status.IsOk())
	{
		return Status;
	}

	const bool bIncludeSoftReferences	= EnumHasAnyFlags(Args.Options, EOnDemandGetInstallSizeOptions::IncludeSoftReferences);
	const bool bIncludeOptionalBulkData = EnumHasAnyFlags(Args.Options, EOnDemandGetInstallSizeOptions::IncludeOptionalBulkData);
	TArray<FOnDemandChunkInfoList> ResolvedChunks;
	TSet<FIoChunkId> Missing;

	ResolveChunksToInstall(
		PackageStoreBackend,
		ContainersForInstallation,
		PackageIdsToInstall,
		bIncludeSoftReferences,
		bIncludeOptionalBulkData,
		ResolvedChunks,
		Missing);

	for (const FOnDemandChunkInfoList& R : ResolvedChunks)
	{
		OutSizesByMountId.FindOrAdd(R.SharedContainer->MountId, 0) += R.TotalDiskSize();
	}

	return EIoErrorCode::Ok;
}

FIoStatus FOnDemandIoStore::GetIsOnDemand(TConstArrayView<FOnDemandIsOnDemandArgs> InArgs, TArrayView<FOnDemandIsOnDemandResult> OutResults) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandIoStore::GetIsOnDemand);

	using namespace UE::IoStore::Private;

	if (InArgs.Num() != OutResults.Num())
	{
		FIoStatus Status = FIoStatusBuilder(EIoErrorCode::InvalidParameter)
			<< TEXT("Result count must match arg count");
		return Status;
	}

	if (InArgs.Num() == 0)
	{
		// trivial success
		return EIoErrorCode::Ok;
	}

	TSet<FSharedOnDemandContainer> ContainersForInstallation;
	if (FIoStatus Status = GetContainersForInstall({}, ContainersForInstallation);
		Status.IsOk() == false)
	{
		return Status;
	}

	FPackageStoreReadScope ReadScope(FPackageStore::Get());

	// TODO: Parallel for here? Is that safe?
	for (int Index = 0; const FOnDemandIsOnDemandArgs& Args : InArgs)
	{
		FOnDemandIsOnDemandResult& Result = OutResults[Index++];
		Result.bIsOnDemand = false;

		const bool bIncludeSoftReferences = EnumHasAnyFlags(Args.Options, EOnDemandIsOnDemandOptions::IncludeSoftReferences);
		VisitPackageDependencies(
			PackageStoreBackend,
			TSet<FPackageId>(Args.PackageIds),
			bIncludeSoftReferences,
			[&Result](
				FPackageId PackageId,
				EPackageStoreEntryStatus EntryStatus,
				const FPackageStoreEntry& PackageStoreEntry,
				bool bIsOnDemand) 
			{
				if (bIsOnDemand && (EntryStatus == EPackageStoreEntryStatus::Ok || EntryStatus == EPackageStoreEntryStatus::NotInstalled))
				{
					Result.bIsOnDemand = true;
					return false;
				}
				return true;
			},
			&ReadScope
		);
	}

	return EIoErrorCode::Ok;
}

FOnDemandChunkInfo FOnDemandIoStore::GetStreamingChunkInfo(const FIoChunkId& ChunkId)
{
	const EOnDemandContainerFlags ContainerFlags = IsPackageStreamingEnabled()
		? EOnDemandContainerFlags::Mounted
		: EOnDemandContainerFlags::Mounted | EOnDemandContainerFlags::StreamOnDemand;

	TUniqueLock Lock(ContainerMutex);

	for (FSharedOnDemandContainer& Container : Containers)
	{
		if (!EnumHasAllFlags(Container->Flags, ContainerFlags))
		{
			continue;
		}

		if (const FOnDemandChunkEntry* Entry = Container->FindChunkEntry(ChunkId))
		{
			return FOnDemandChunkInfo(Container, *Entry);
		}
	}

	return FOnDemandChunkInfo();
}

FOnDemandChunkInfo FOnDemandIoStore::GetInstalledChunkInfo(const FIoChunkId& ChunkId, EIoErrorCode& OutErrorCode)
{
	const EOnDemandContainerFlags ContainerFlags = EOnDemandContainerFlags::Mounted | EOnDemandContainerFlags::InstallOnDemand;

	TUniqueLock Lock(ContainerMutex);

	for (FSharedOnDemandContainer& Container : Containers)
	{
		if (!EnumHasAllFlags(Container->Flags, ContainerFlags))
		{
			continue;
		}

		int32 EntryIndex = INDEX_NONE;
		if (const FOnDemandChunkEntry* Entry = Container->FindChunkEntry(ChunkId, &EntryIndex))
		{
			TUniqueLock RefsLock(Container->ReferencesMutex);

			if (Container->IsReferenced(EntryIndex))
			{
				OutErrorCode = EIoErrorCode::Ok;
				return FOnDemandChunkInfo(Container, *Entry);
			}

			OutErrorCode = EIoErrorCode::NotInstalled;
			return FOnDemandChunkInfo();
		}
	}

	OutErrorCode = EIoErrorCode::UnknownChunkID;
	return FOnDemandChunkInfo();
}

#if !UE_BUILD_SHIPPING
TArray<FIoChunkId> FOnDemandIoStore::DebugFindStreamingChunkIds(int32 NumToFind)
{
	TArray<FIoChunkId> IdsContainer;
	IdsContainer.Reserve(NumToFind);

	TUniqueLock Lock(ContainerMutex);

	for (FSharedOnDemandContainer& Container : Containers)
	{
		if (EnumHasAllFlags(Container->Flags, EOnDemandContainerFlags::Mounted | EOnDemandContainerFlags::StreamOnDemand))
		{
			for(const FIoChunkId& Id : Container->ChunkIds)
			{
				if (EnumHasAllFlags(StreamingOptions, EOnDemandStreamingOptions::OptionalBulkDataDisabled) && Id.GetChunkType() == EIoChunkType::OptionalBulkData)
				{
					continue;
				}

				IdsContainer.Add(Id);

				if (IdsContainer.Num() == NumToFind)
				{
					return IdsContainer;
				}
			}
		}
	}

	return IdsContainer;
}
#endif

FIoStatus FOnDemandIoStore::InitializeInstallCache(FOnDemandInstallCacheConfig& OnDemandInstallCacheConfig)
{
	if (FForkProcessHelper::IsForkRequested() && !FForkProcessHelper::IsForkedChildProcess())
	{
		return FIoStatusBuilder(EIoErrorCode::PendingFork) << TEXT("Install cache waiting for fork");
	}

	FString RootDirectory = Config::GetInstallCacheDirectory(FCommandLine::Get());

	// Check if we use ICS for IAD and update the RootDirectory accordingly
	int32 SelectedCacheIndex = 0;
	if (OnDemandInstallCacheConfig.IndexedCacheStorageName.IsEmpty() == false)
	{
		// OnDemandInstallCacheConfig.DiskQuota is the quota to store the data, but we also need some room for metadata, see GetJournalFilename/GetSnapshotFilename
		const uint64 MetaDataSize = 2 * OnDemandInstallCacheConfig.JournalMaxSize + (2 << 20);
		uint64 OnDemandInstallRequestedSize = 
			MetaDataSize + 
			Algo::TransformAccumulate(OnDemandInstallCacheConfig.CasConfig, &FOnDemandInstallCasConfig::DiskQuota, uint64(0));

		FModuleManager::Get().LoadModule(TEXT("IndexedCacheStorage"));

		SelectedCacheIndex = Experimental::FIndexedCacheStorageManager::Get().GetStorageIndex(OnDemandInstallCacheConfig.IndexedCacheStorageName);
		if (SelectedCacheIndex > 0)
		{
			// ICS will allocate space for all early indices in one combined operation.
			// In this case, it should be configured to read the early size from the IAD config, for example Engine:[OnDemandInstall]:FileCache.DiskQuota
			// See Engine:[IndexedCacheStorage]:Storage
			const uint64 EarlyStartupSize = Experimental::FIndexedCacheStorageManager::Get().GetCacheEarlyStartupSize(SelectedCacheIndex);
			if (EarlyStartupSize > 0)
			{
				UE_CLOGF(EarlyStartupSize < OnDemandInstallRequestedSize, LogIoStoreOnDemand, Fatal, "EarlyStartupSize must be >= %" UINT64_FMT, OnDemandInstallRequestedSize);
				OnDemandInstallRequestedSize = EarlyStartupSize;
			}

			// When IAD is assigned to an indexed cache, we *have* to get enough disk space.
			while (true)
			{
				if (!Experimental::FIndexedCacheStorageManager::Get().CreateCacheStorage(OnDemandInstallRequestedSize, SelectedCacheIndex))
				{
					continue;
				}

				FString MountPath = Experimental::FIndexedCacheStorageManager::Get().MountCacheStorage(SelectedCacheIndex);
				if (MountPath.IsEmpty())
				{
					UE_LOGF(LogIoStoreOnDemand, Error, "Failed to mount indexed cache storage %d (%ls) even though it was created", SelectedCacheIndex, *OnDemandInstallCacheConfig.IndexedCacheStorageName);
					Experimental::FIndexedCacheStorageManager::Get().DestroyCacheStorage(SelectedCacheIndex);
					continue;
				}

				FString RelativeDir(MountPath / RootDirectory.Replace(*FPaths::ProjectPersistentDownloadDir(), TEXT("")));
				RootDirectory = RelativeDir;
				break;
			}
		}
	}

	InstallCache = MakeOnDemandInstallCache(*this, OnDemandInstallCacheConfig, MoveTemp(RootDirectory), DiskCacheGovernor);
	if (InstallCache.IsValid())
	{
		int32 BackendPriority = -5; // Lower than file (zero) but higher than streaming backend (-10)
#if !UE_BUILD_SHIPPING
		if (IsDevModeEnabled() || FParse::Param(FCommandLine::Get(), TEXT("Iad")))
		{
			// Bump the priority to be higher then the file system backend
			BackendPriority = 5;
		}
#endif
		FIoDispatcher::Get().Mount(InstallCache.ToSharedRef(), BackendPriority);
		PackageStoreBackend = MakeOnDemandPackageStoreBackend(AsShared().ToWeakPtr());
		FPackageStore::Get().Mount(PackageStoreBackend.ToSharedRef(), BackendPriority);
	}
	else
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to initialize install cache");

		if (SelectedCacheIndex > 0)
		{
			Experimental::FIndexedCacheStorageManager::Get().DestroyCacheStorage(SelectedCacheIndex);
		}

		return FIoStatusBuilder(EIoErrorCode::InvalidParameter) << TEXT("Failed to initialize install cache");
	}

#if !(UE_BUILD_SHIPPING|UE_BUILD_TEST)
	FString ParamValue;
	if (FParse::Value(FCommandLine::Get(), TEXT("-Iad.Fill="), ParamValue))
	{
		ParamValue.TrimStartAndEndInline();
		int64 FillSize = -1;
		LexFromString(FillSize, ParamValue);

		if (FillSize > 0)
		{
			if (ParamValue.EndsWith(TEXT("GB")))
			{
				FillSize = FillSize << 30;
			}
			if (ParamValue.EndsWith(TEXT("MB")))
			{
				FillSize = FillSize << 20;
			}

			UE_LOGF(LogIoStoreOnDemand, Log, "Filling install cache with %.2lf MiB of dummy data", double(FillSize) / 1024.0 / 1024.0);

			TUniquePtr<FInstallCacheHandle> CacheHandle = InstallCache->BeginInstall();

			FResult		Result = MakeValue();
			uint64		Seed = 1;
			while (FillSize >= 0 && Result.HasError() == false)
			{
				const uint64		ChunkSize = 256 << 10;
				FIoBuffer			Chunk(ChunkSize);
				TArrayView<uint64>	Values(reinterpret_cast<uint64*>(Chunk.GetData()), ChunkSize / sizeof(uint64));

				for (uint64& Value : Values)
				{
					Value = Seed;
				}

				const FOnDemandChunkHash ChunkAddr = FOnDemandChunkHash::HashBuffer(Chunk.GetView());
				FResult PutResult = InstallCache->PutChunk(EIoChunkType::ExportBundleData, MoveTemp(Chunk), ChunkAddr);
				if (PutResult.HasError())
				{
					Result = MoveTemp(PutResult);
				}

				InstallCache->PostPutChunk(*CacheHandle, EIoChunkType::ExportBundleData);

				Seed++;
				FillSize -= ChunkSize;
			}

			if (Result.HasError() == false)
			{
				Result = InstallCache->ConditionallyFlushInstall(*CacheHandle);
			}

			UE_CLOGF(Result.HasError(), LogIoStoreOnDemand, Warning, "Failed to fill install cache with dummy data, reason '%ls'", *LexToString(Result.GetError()));
		}
	}
#endif

	return EIoErrorCode::Ok;
}

void FOnDemandIoStore::TryEnterTickLoop()
{
	bool bEnterTickLoop = false;
	{
		UE::TUniqueLock Lock(RequestMutex);
		bTickRequested = true;
		if (bTicking == false)
		{
			bTicking = bEnterTickLoop = true;
		}
	}

	if (bEnterTickLoop == false)
	{
		UE_LOGF(LogIoStoreOnDemand, Verbose, "I/O store already ticking");
		return;
	}

	if (FPlatformProcess::SupportsMultithreading() && GIOThreadPool != nullptr)
	{
		TickFuture = AsyncPool(*GIOThreadPool, [this] { TickLoop(); }, nullptr, EQueuedWorkPriority::Low);
	}
	else
	{
		TickLoop();
	}
}

void FOnDemandIoStore::TickLoop()
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/OnDemandIoStore"));
	ON_SCOPE_EXIT { UE_LOGF(LogIoStoreOnDemand, Verbose, "Exiting I/O store tick loop"); };

	UE_LOGF(LogIoStoreOnDemand, Verbose, "Entering I/O store tick loop");
	for (;;)
	{
		const bool bTicked = Tick();
		if (bTicked == false)
		{
			UE::TUniqueLock Lock(RequestMutex);
			if (bTickRequested == false)
			{
				bTicking = false;
				break;
			}
			bTickRequested = false;
		}
	}
}

bool FOnDemandIoStore::Tick()
{
	TArray<FSharedMountRequest> LocalMountRequests;
	{
		UE::TUniqueLock Lock(RequestMutex);
		LocalMountRequests = MountRequests;
	}

	bool bTicked = LocalMountRequests.IsEmpty() == false;

	// Process mount request(s)
	for (FSharedMountRequest& Request : LocalMountRequests)
	{
		FIoStatus MountStatus = ProcessMountRequest(*Request);

		{
			UE::TUniqueLock Lock(RequestMutex);
			MountRequests.Remove(Request);
		}

		CompleteMountRequest(*Request, 
			FOnDemandMountResult
			{
				.MountId = MoveTemp(Request->Args.MountId),
				.Status = MoveTemp(MountStatus),
				.DurationInSeconds = Request->DurationInSeconds
			});
	}

	return bTicked;
}

void FOnDemandIoStore::EnqueueMountRequest(FOnDemandMountArgs&& Args, FOnDemandMountCompleted&& OnCompleted)
{
	FSharedMountRequest MountRequest = MakeShared<FMountRequest>();
	MountRequest->Args = MoveTemp(Args);
	MountRequest->OnCompleted = MoveTemp(OnCompleted);

	{
		UE::TUniqueLock Lock(RequestMutex);
		MountRequests.Add(MoveTemp(MountRequest));
	}

	TryEnterTickLoop();
}

FIoStatus FOnDemandIoStore::ProcessMountRequest(FMountRequest& MountRequest)
{
	UE_LOGF(LogIoStoreOnDemand, Verbose, "Processing mount request, MountId='%ls'", *MountRequest.Args.MountId);

	bool bWasLoaded = false;
	if (FIoStatus CertStatus = LoadDefaultHttpCertificates(bWasLoaded); CertStatus.IsOk() == false) 
	{
		if (bWasLoaded)
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "Failed to load certificates, reason '%ls'", *CertStatus.ToString());
		}
	}

	const double StartTime = FPlatformTime::Seconds();
	ON_SCOPE_EXIT
	{
		MountRequest.DurationInSeconds = FPlatformTime::Seconds() - StartTime;
	};

	FOnDemandMountArgs& Args = MountRequest.Args;

	if (Args.MountId.IsEmpty())
	{
		Args.MountId = Args.FilePath;
		if (Args.MountId.IsEmpty())
		{
			return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid mount ID"));
		}
	}

	if (Args.FilePath.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid TOC file path"));
	}

	EIoErrorCode MountResult = EIoErrorCode::Ok;
	TArray<FIoContainerId> ExistingContainers;

	// Find containers matching the mount ID
	{
		UE::TUniqueLock Lock(ContainerMutex);
		for (const FSharedOnDemandContainer& Container : Containers)
		{
			if (Container->MountId != Args.MountId)
			{
				continue;
			}
			
			ExistingContainers.Add(Container->ContainerId());
			
			if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::PendingEncryptionKey))
			{
				MountResult = EIoErrorCode::PendingEncryptionKey;
			}

			if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::PendingHostGroup))
			{
				MountResult = EIoErrorCode::PendingHostGroup;
			}
		}
	}

	TArray<FSharedOnDemandContainer> RequestedContainers;
	if (FIoStatus Status = CreateContainersFromToc(Args, ExistingContainers, RequestedContainers); !Status.IsOk())
	{
		return Status;
	}

	{
		RequestedContainers.SetNum(Algo::RemoveIf(
			RequestedContainers, [](const FSharedOnDemandContainer& Container)
			{
				if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::StreamOnDemand) && !IsIasEnabled())
				{
					UE_LOGF(LogIoStoreOnDemand, Log, "Skipping container '%ls', streaming containers disabled", *Container->Name());
					return true;
				}
				return false;
			}));
	}

	for (FSharedOnDemandContainer& Container : RequestedContainers)
	{
		if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::StreamOnDemand | EOnDemandContainerFlags::InstallOnDemand) == false)
		{
			return FIoStatusBuilder(EIoErrorCode::InvalidParameter) << TEXT("Does not have either a 'StreamOnDemand' or 'InstallOnDemand' EOnDemandContainerFlags flag");
		}

		if (EnumHasAnyFlags(Args.Options, EOnDemandMountOptions::WithSoftReferences))
		{
			EnumAddFlags(Container->Flags, EOnDemandContainerFlags::WithSoftReferences);
		}

		if (const FIoStatus Status = SetupHostGroup(Container, Args); !Status.IsOk())
		{
			if (Status.GetErrorCode() == EIoErrorCode::PendingHostGroup)
			{
				EnumAddFlags(Container->Flags, EOnDemandContainerFlags::PendingHostGroup);
				MountResult = EIoErrorCode::PendingHostGroup;
				UE_LOGF(LogIoStoreOnDemand, Log, "Deferring container '%ls' until host group '%ls' becomes available",
					*Container->Name(), *Container->HostGroupName.ToString());
			}
			else
			{
				return Status;
			}
		}

		if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::Encrypted) &&
			Container->EncryptionKey.IsValid() == false)
		{
			if (FEncryptionKeyManager::Get().TryGetKey(Container->EncryptionKeyGuid(), Container->EncryptionKey) == false)
			{
				EnumAddFlags(Container->Flags, EOnDemandContainerFlags::PendingEncryptionKey);
				MountResult = EIoErrorCode::PendingEncryptionKey;
				UE_LOGF(LogIoStoreOnDemand, Log, "Deferring container '%ls' until encryption key '%ls' becomes available",
					*Container->Name(), *LexToString(Container->EncryptionKeyGuid()));
			}
		}

		// Serialize the container header with package dependencies if the encryption key is available
		if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::PendingEncryptionKey) == false)
		{
			FIoBuffer ContainerHeaderChunk = MoveTemp(Container->ContainerHeaderChunk);
			TIoStatusOr<FSharedContainerHeader> Header = Private::DeserializeContainerHeader(Container, ContainerHeaderChunk.GetView());
			if (Header.IsOk())
			{
				Container->Header = Header.ConsumeValueOrDie();
			}
			else
			{
				return Header.Status();
			}
		}

		if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::PendingEncryptionKey | EOnDemandContainerFlags::PendingHostGroup))
		{
			continue;
		}

		Private::LogMounted(Container);
		EnumAddFlags(Container->Flags, EOnDemandContainerFlags::Mounted);
	}

	{
		UE::TUniqueLock Lock(ContainerMutex);
		Containers.Append(MoveTemp(RequestedContainers));
	}

	if (ensureMsgf(PackageStoreBackend, TEXT("PackageStoreBackend is null, is this a prefork server?")))
	{
		PackageStoreBackend->NeedsUpdate();
	}

	return MountResult;
}

void FOnDemandIoStore::CompleteMountRequest(FMountRequest& Request, FOnDemandMountResult&& MountResult)
{
	if (!Request.OnCompleted)
	{
		return;
	}

	if (EnumHasAnyFlags(Request.Args.Options, EOnDemandMountOptions::CallbackOnGameThread))
	{
		ExecuteOnGameThread(
			UE_SOURCE_LOCATION,
			[OnCompleted = MoveTemp(Request.OnCompleted), MountResult = MoveTemp(MountResult)]() mutable
			{
				OnCompleted(MoveTemp(MountResult));
			});
	}
	else
	{
		FOnDemandMountCompleted OnCompleted = MoveTemp(Request.OnCompleted);
		OnCompleted(MoveTemp(MountResult));
	}
}

void FOnDemandIoStore::OnEncryptionKeyAdded(const FGuid& Id, const FAES::FAESKey& Key)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandIoStore::OnEncryptionKeyAdded);

	TUniqueLock Lock(ContainerMutex);

	bool bAddedContainerHeaders = false;

	for (FSharedOnDemandContainer& Container : Containers)
	{
		if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::PendingEncryptionKey) == false)
		{
			continue;
		}

		if (FEncryptionKeyManager::Get().TryGetKey(Container->EncryptionKeyGuid(), Container->EncryptionKey) == false)
		{
			continue;
		}

		Private::LogMounted(Container);
		EnumRemoveFlags(Container->Flags, EOnDemandContainerFlags::PendingEncryptionKey);
		EnumAddFlags(Container->Flags, EOnDemandContainerFlags::Mounted);

		{
			FIoBuffer ContainerHeaderChunk = MoveTemp(Container->ContainerHeaderChunk);
			TIoStatusOr<FSharedContainerHeader> Header = Private::DeserializeContainerHeader(Container, ContainerHeaderChunk.GetView());
			if (Header.IsOk())
			{
				Container->Header = Header.ConsumeValueOrDie();
				bAddedContainerHeaders = true;
			}
			else
			{
				EnumRemoveFlags(Container->Flags, EOnDemandContainerFlags::Mounted);
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to deserialize header when mounting container '%ls', Entries=%d, Flags='%ls'",
					*Container->Name(), Container->ChunkEntries.Num(), *LexToString(Container->Flags));
			}
		}
	}

	if (bAddedContainerHeaders && ensureMsgf(PackageStoreBackend, TEXT("PackageStoreBackend is null, is this a prefork server?")))
	{
		PackageStoreBackend->NeedsUpdate();
	}
}

void FOnDemandIoStore::OnHostGroupRegistered(const FName& HostGroup)
{
	TUniqueLock Lock(ContainerMutex);
	for (FSharedOnDemandContainer& Container : Containers)
	{
		if (!EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::PendingHostGroup) || Container->HostGroupName != HostGroup)
		{
			continue;
		}

		Private::LogMounted(Container);
		EnumRemoveFlags(Container->Flags, EOnDemandContainerFlags::PendingHostGroup);
		EnumAddFlags(Container->Flags, EOnDemandContainerFlags::Mounted);
	}
}

FIoStatus FOnDemandIoStore::CreateContainersFromToc(
	FOnDemandMountArgs& MountArgs,
	TConstArrayView<FIoContainerId> Existing,
	TArray<FSharedOnDemandContainer>& Out)
{
	using namespace UE::IoStore::Serialization;
	using namespace UE::IoStore::Serialization::V2;

	check(!MountArgs.MountId.IsEmpty());
	check(!MountArgs.FilePath.IsEmpty());

	const static FName AssetClassName("OnDemandIoStore");
	LLM_SCOPE_BYNAME(TEXT("FileSystem/OnDemandIoStore"));
	LLM_TAGSET_SCOPE(FName(MountArgs.MountId), ELLMTagSet::Assets);
	LLM_TAGSET_SCOPE(AssetClassName, ELLMTagSet::AssetClasses);
	UE_TRACE_METADATA_SCOPE_ASSET_FNAME(FName(MountArgs.MountId), AssetClassName, FName(MountArgs.FilePath));

	UE_LOGF(LogIoStoreOnDemand, Log, "Loading TOC from file '%ls'", *MountArgs.FilePath);
	TIoStatusOr<FOnDemandTocReader> MaybeReader = FOnDemandTocReader::Read(*MountArgs.FilePath);
	if (MaybeReader.IsOk() == false)
	{
		return MaybeReader.Status();
	}

	FOnDemandTocReader Reader = MaybeReader.ConsumeValueOrDie();

	const FStringView TocPath = FPathViews::GetExtension(MountArgs.TocRelativeUrl).IsEmpty()
		? MountArgs.TocRelativeUrl
		: FPathViews::GetPath(MountArgs.TocRelativeUrl);

	TStringBuilder<128> Sb;
	FStringView ChunksDirectory;
	{
		if (TocPath.IsEmpty() == false)
		{
			FPathViews::Append(Sb, TocPath);
		}
		else
		{
			FPathViews::Append(Sb, FString(Reader.ChunksDirectory()));
		}
		FPathViews::Append(Sb, TEXT("chunks"));

		ChunksDirectory = Sb;
		if (ChunksDirectory.StartsWith('/'))
		{
			ChunksDirectory.RemovePrefix(1);
		}
		if (ChunksDirectory.EndsWith('/'))
		{
			ChunksDirectory.RemoveSuffix(1);
		}
	}

	for (const FOnDemandContainerEntry& ContainerEntry : Reader.Containers())
	{
		if (Existing.Contains(ContainerEntry.ContainerId))
		{
			continue;
		}

		FSharedOnDemandContainer				Container = MakeShared<FOnDemandContainer>();
		EOnDemandTocReaderOptions				Options = GIaxMemoryMapTocEnabled ? EOnDemandTocReaderOptions::MemoryMap : EOnDemandTocReaderOptions::None;
		TIoStatusOr<FOnDemandContainerTocView>	MaybeView = Reader.ReadContainer(ContainerEntry, Container->Storage, Options);

		if (MaybeView.IsOk() == false)
		{
			return MaybeView.Status();
		}

		Container->MountId				= MountArgs.MountId;
		FOnDemandContainerTocView View	= MaybeView.ConsumeValueOrDie();
		Container->HeaderView			= View.Header;
		Container->ContainerHeaderChunk	= MoveTemp(View.ContainerHeaderChunk);
		Container->PartitionEntries		= View.PartitionEntries;
		Container->ChunkIds				= View.ChunkIds;
		Container->ChunkEntries			= View.ChunkEntries;
		Container->BlockSizes			= View.BlockSizes;
		Container->BlockHashes			= View.BlockHashes;
		Container->TagSets				= View.TagSets;
		Container->TagSetIndices		= View.TagSetIndices;
		Container->RelativeUrl			= UE::FIoRelativeUrl::From(FAnsiString(ChunksDirectory));
		Container->CompressionFormat	= FName(FString(Reader.CompressionFormat()));
		Container->HostGroupName		= FName(Reader.HostGroupName());

		if (EnumHasAnyFlags(View.Header.FileContainerFlags(), EIoContainerFlags::Encrypted))
		{
			EnumAddFlags(Container->Flags, EOnDemandContainerFlags::Encrypted);
		}
		if (EnumHasAnyFlags(View.Header.ContainerFlags(), EOnDemandContainerEntryFlags::InstallOnDemand))
		{
			EnumAddFlags(Container->Flags, EOnDemandContainerFlags::InstallOnDemand);
		}
		else if (EnumHasAnyFlags(View.Header.ContainerFlags(), EOnDemandContainerEntryFlags::StreamOnDemand))
		{
			EnumAddFlags(Container->Flags, EOnDemandContainerFlags::StreamOnDemand);
		}

		check(Algo::IsSorted(Container->ChunkIds));

		// Do not count output container memory as being allocated for utocs
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::Assets);
		LLM_TAGSET_SCOPE_CLEAR(ELLMTagSet::AssetClasses);
		UE_TRACE_METADATA_CLEAR_SCOPE();
		Out.Add(MoveTemp(Container));
	}

	return FIoStatus::Ok;
}

FIoStatus FOnDemandIoStore::SetupHostGroup(const FSharedOnDemandContainer& Container, const FOnDemandMountArgs& MountArgs)
{
	if (Container->HostGroupName.IsNone())
	{
		Container->HostGroupName = MountArgs.HostGroupName.IsNone()
			? FOnDemandHostGroup::DefaultName
			: MountArgs.HostGroupName;
	}
	else if (!MountArgs.HostGroupName.IsNone() && MountArgs.HostGroupName != Container->HostGroupName)
	{
		UE_LOGF(LogIoStoreOnDemand, Log, "Overriding container host group '%ls' with '%ls' from mount option",
			*Container->HostGroupName.ToString(), *MountArgs.HostGroupName.ToString());
		Container->HostGroupName = MountArgs.HostGroupName;
	}

	if (FHttpIoDispatcher::IsHostGroupRegistered(Container->HostGroupName))
	{
		FHostGroupManager::Get().TrySetTestPath(Container->HostGroupName, Container->GetTestUrl());
		return FIoStatus::Ok;
	}

	if (MountArgs.HostGroup.IsEmpty() == false)
	{
		return FHttpIoDispatcher::RegisterHostGroup(Container->HostGroupName, MountArgs.HostGroup.Hosts(), Container->GetTestUrl());
	}

	return FIoStatus(EIoErrorCode::PendingHostGroup); 
}

TArray<FSharedOnDemandContainer> FOnDemandIoStore::GetContainers(EOnDemandContainerFlags ContainerFlags) const
{
	TArray<FSharedOnDemandContainer> Out;
	Out.Reserve(Containers.Num());
	{
		UE::TUniqueLock Lock(ContainerMutex);
		for (const FSharedOnDemandContainer& Container : Containers)
		{
			if (ContainerFlags == EOnDemandContainerFlags::None || Container->HasAllFlags(ContainerFlags))
			{
				Out.Add(Container);
			}
		}
	}

	return Out;
}

void FOnDemandIoStore::FlushLastAccess(FOnDemandFlushLastAccessArgs&& Args, FOnDemandFlushLastAccessCompleted&& OnCompleted)
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/OnDemandIoStore"));
	Installer->EnqueueFlushLastAccessRequest(MoveTemp(Args), MoveTemp(OnCompleted));
}

FIoStatus FOnDemandIoStore::GetContainersForInstall(
	FStringView MountId, 
	TSet<FSharedOnDemandContainer>& OutContainersForInstallation) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandIoStore::GetContainersForInstall);

	UE::TUniqueLock Lock(ContainerMutex);

	OutContainersForInstallation.Reserve(Containers.Num());
	for (const FSharedOnDemandContainer& Container : Containers)
	{
		if (Container->HasAnyFlags(EOnDemandContainerFlags::InstallOnDemand) == false)
		{
			continue;
		}

		if (Container->HasAnyFlags(EOnDemandContainerFlags::PendingEncryptionKey))
		{
			check(Container->HasAnyFlags(EOnDemandContainerFlags::Mounted) == false);
			if (MountId == Container->MountId)
			{
				check(EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::Mounted) == false);
				FIoStatus Status = FIoStatusBuilder(EIoErrorCode::PendingEncryptionKey)
					<< TEXT("Trying to install content from encrypted container '")
					<< Container->Name()
					<< TEXT("'");
				return Status;
			}

			continue;
		}

		check(Container->HasAnyFlags(EOnDemandContainerFlags::Mounted));
		OutContainersForInstallation.Add(Container);
	}

	return FIoStatus::Ok;
}

FIoStatus FOnDemandIoStore::GetContainersAndPackagesForInstall(
	FStringView MountId,
	const TArray<FString>& TagSets,
	const TArray<FPackageId>& PackageIds,
	TSet<FSharedOnDemandContainer>& OutContainersForInstallation, 
	TSet<FPackageId>& OutPackageIdsToInstall) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandIoStore::GetContainersAndPackagesForInstall);

	OutContainersForInstallation.Reset();
	if (FIoStatus Status = GetContainersForInstall(MountId, OutContainersForInstallation);
		Status.IsOk() == false)
	{
		return Status;
	}

	// It's not allowed to install all content
	if (MountId.IsEmpty() && TagSets.IsEmpty() && PackageIds.IsEmpty())
	{
		OutContainersForInstallation.Reset();
		FIoStatus Status = FIoStatusBuilder(EIoErrorCode::InvalidCode)
			<< TEXT("Trying to install content from all mounted containers");
		return Status;
	}

	const bool bInstallAllPackages = TagSets.IsEmpty() && PackageIds.IsEmpty();
	if (bInstallAllPackages)
	{
		check(MountId.IsEmpty() == false);
		for (const FSharedOnDemandContainer& Container : OutContainersForInstallation)
		{
			if (Container->Header.IsValid() == false || MountId != Container->MountId)
			{
				continue;
			}
			UE_LOGF(LogIoStoreOnDemand, Log, "Installing all %d package(s) from container '%ls'",
				Container->ChunkIds.Num(), *Container->UniqueName());

			OutPackageIdsToInstall.Append(Container->Header->PackageIds);
		}

		return FIoStatus::Ok;
	}

	// Add packages from matching tags
	for (const FSharedOnDemandContainer& Container : OutContainersForInstallation)
	{
		if (Container->Header.IsValid() == false || (!MountId.IsEmpty() && MountId != Container->MountId))
		{
			continue;
		}

		for (const FString& Tag : TagSets)
		{
			const int32 NumBeforeTag = OutPackageIdsToInstall.Num();
			for (const FOnDemandTagSet& TagSet : Container->TagSets)
			{
				//TODO: Use UTF8 strings in the public API
				const FString TagSetTag = FString(Container->HeaderView.GetString(TagSet.Tag));
				if (TagSetTag == Tag)
				{
					UE_LOGF(LogIoStoreOnDemand, Log, "Installing %u package(s) with tag '%ls' from container '%ls'",
						TagSet.Count, *Tag, *Container->UniqueName());

					OutPackageIdsToInstall.Reserve(TagSet.Count);
					TConstArrayView<uint32> PackageIndicies = Container->TagSetIndices.Mid(TagSet.Offset, TagSet.Count);
					for (const uint32 PackageIndex : PackageIndicies)
					{
						const FPackageId PackageId = Container->Header->PackageIds[IntCastChecked<int32>(PackageIndex)];
						OutPackageIdsToInstall.Add(PackageId);
					}
				}
			}
			UE_LOGF(LogIoStoreOnDemand, Log, "Installing total %d package(s) with tag '%ls'", OutPackageIdsToInstall.Num() - NumBeforeTag, *Tag)
		}
	}

	// Finally add any specified package ID(s) 
	OutPackageIdsToInstall.Append(PackageIds);

	return FIoStatus::Ok;
}

void FOnDemandIoStore::ReleaseContent(FOnDemandInternalContentHandle& ContentHandle)
{
	LLM_SCOPE_BYNAME(TEXT("FileSystem/OnDemandIoStore"));
	UE_LOGF(LogIoStoreOnDemand, Verbose, "Releasing content handle, %ls", *LexToString(ContentHandle));

#if !UE_BUILD_SHIPPING
	if (FOnDemandContentInstallReplayRecorder* Recorder = FOnDemandContentInstallReplayRecorder::Get())
	{
		Recorder->RecordContentHandleDestroyed(ContentHandle.HandleId());
	}
#endif // !UE_BUILD_SHIPPING

	if (Installer.IsValid())
	{
		Installer->ScheduleReleaseContentRequestTask(ContentHandle.HandleId());
	}
}

void FOnDemandIoStore::GetReferencedContent(TArray<FSharedOnDemandContainer>& OutContainers, TArray<TBitArray<>>& OutChunkEntryIndices, bool bPackageStore)
{
	UE::TUniqueLock Lock(ContainerMutex);
	for (FSharedOnDemandContainer& Container : Containers)
	{
		if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::StreamOnDemand) && !IsPackageStreamingEnabled())
		{
			continue;
		}

		if (Container->HasAnyFlags(EOnDemandContainerFlags::PendingEncryptionKey))
		{
			continue;
		}

		if (bPackageStore && Container->Header.IsValid() == false)
		{
			// Package store doesn't care about containers without packages
			continue;
		}

		TUniqueLock RefsLock(Container->ReferencesMutex);

		TBitArray<> Indices = Container->GetReferencedChunkEntries();
		if (Indices.IsEmpty() && bPackageStore)
		{
			// Package store wants to know about about containers with packages even if there are not references yet
			Indices.SetNum(Container->ChunkEntries.Num(), false);
		}

		if (Indices.IsEmpty() == false)
		{
			OutContainers.Add(Container);
			OutChunkEntryIndices.Add(MoveTemp(Indices));
		}
	}
}

TBitArray<> FOnDemandIoStore::GetReferencedContent(const FSharedOnDemandContainer& Container)
{
	TUniqueLock RefsLock(Container->ReferencesMutex);
	return Container->GetReferencedChunkEntries();
}

void FOnDemandIoStore::GetReferencedContentByHandle(TMap<FOnDemandWeakContentHandle, TArray<FOnDemandContainerChunkEntryReferences>>& OutReferencesByHandle) const
{
	UE::TUniqueLock Lock(ContainerMutex);
	for (const FSharedOnDemandContainer& Container : Containers)
	{
		TUniqueLock RefsLock(Container->ReferencesMutex);
		for (const FOnDemandChunkEntryReferences& Refs : Container->ChunkEntryReferences)
		{
			FOnDemandWeakContentHandle WeakHandle = FOnDemandWeakContentHandle::FromUnsafeHandle(Refs.ContentHandleId);
			if (WeakHandle.IsValid() == false)
			{
				UE_LOGF(LogIoStoreOnDemand, Warning, "Found invalid content handle in container '%ls'", *Container->Name());
				continue;
			}

			TArray<FOnDemandContainerChunkEntryReferences>& Entries = OutReferencesByHandle.FindOrAdd(WeakHandle);
			Entries.Add(FOnDemandContainerChunkEntryReferences
			{
				.Container	= Container,
				.Indices	= Refs.Indices
			});
		}
	}
}

void FOnDemandIoStore::CancelInstallRequest(FSharedInternalInstallRequest InstallRequest)
{
	check(InstallRequest.IsValid());
	Installer->CancelInstallRequest(InstallRequest);
}

void FOnDemandIoStore::UpdateInstallRequestPriority(FSharedInternalInstallRequest InstallRequest, int32 NewPriority)
{
	Installer->UpdateInstallRequestPriority(InstallRequest, NewPriority);
}

FOnDemandCacheUsage FOnDemandIoStore::GetCacheUsage(const FOnDemandGetCacheUsageArgs& Args) const
{
	FOnDemandCacheUsage CacheUsage;

	if (InstallCache.IsValid())
	{
		CacheUsage.InstallCache = InstallCache->GetCacheUsage();

		if (EnumHasAnyFlags(Args.Options, EOnDemandGetCacheUsageOptions::DumpHandlesToLog | EOnDemandGetCacheUsageOptions::DumpHandlesToResults))
		{
			TMap<FOnDemandWeakContentHandle, TArray<FOnDemandContainerChunkEntryReferences>> RefsByHandle;
			GetReferencedContentByHandle(RefsByHandle);

			if (EnumHasAnyFlags(Args.Options, EOnDemandGetCacheUsageOptions::DumpHandlesToResults))
			{
				CacheUsage.InstallCache.ReferencedBytesByHandle.Reserve(RefsByHandle.Num());
			}

			for (const TPair<FOnDemandWeakContentHandle, TArray<FOnDemandContainerChunkEntryReferences>>& Kv : RefsByHandle)
			{
				const FOnDemandWeakContentHandle& WeakHandle = Kv.Key;
				const TArray<FOnDemandContainerChunkEntryReferences>& ContainerReferences = Kv.Value;

				uint64 ReferencedBytesByHandle = 0;
				for (const FOnDemandContainerChunkEntryReferences& Refs : ContainerReferences)
				{
					for (int32 EntryIndex = 0; const FOnDemandChunkEntry & ChunkEntry : Refs.Container->ChunkEntries)
					{
						if (Refs.Indices[EntryIndex])
						{
							ReferencedBytesByHandle += ChunkEntry.GetDiskSize();
						}
						EntryIndex++;
					}
				}

				if (ReferencedBytesByHandle > 0)
				{
					if (EnumHasAnyFlags(Args.Options, EOnDemandGetCacheUsageOptions::DumpHandlesToLog))
					{
						UE_LOGF(LogIoStoreOnDemand, Display, "HandleId=0x%llX, DebugName='%ls', ReferencedBytes=%.2lf KiB",
							WeakHandle.HandleId, *WeakHandle.DebugName, double(ReferencedBytesByHandle) / 1024.0);
					}

					if (EnumHasAnyFlags(Args.Options, EOnDemandGetCacheUsageOptions::DumpHandlesToResults))
					{
						CacheUsage.InstallCache.ReferencedBytesByHandle.Add(
							FOnDemandInstallHandleCacheUsage
							{
								.HandleId = WeakHandle.HandleId,
								.DebugName = WeakHandle.DebugName,
								.ReferencedBytes = ReferencedBytesByHandle
							}
						);
					}
				}
			}
		}
	}

	FOnDemandIoBackendStats::Get()->GetIasCacheStats(CacheUsage.StreamingCache.TotalSize, CacheUsage.StreamingCache.MaxSize);

	return CacheUsage;
}

void FOnDemandIoStore::DumpMountedContainersToLog() const
{
	TStringBuilder<128> FlagsSb;

	TUniqueLock Lock(ContainerMutex);

	UE_LOGF(LogIoStoreOnDemand, Display, "Containers:");
	for (const FSharedOnDemandContainer& Container : Containers)
	{
		FlagsSb.Reset();
		bool bFirst = true;

		if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::StreamOnDemand))
		{
			FlagsSb << TEXT("StreamOnDemand");
			bFirst = false;
		}
		if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::InstallOnDemand))
		{
			if (!bFirst)
			{
				FlagsSb << TEXT(", ");
			}

			FlagsSb << TEXT("InstallOnDemand");
			bFirst = false;
		}
		if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::Mounted))
		{
			if (!bFirst)
			{
				FlagsSb << TEXT(", ");
			}

			FlagsSb << TEXT("Mounted");
			bFirst = false;
		}
		if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::PendingEncryptionKey))
		{
			if (!bFirst)
			{
				FlagsSb << TEXT(", ");
			}

			FlagsSb << TEXT("PendingEncryptionKey");
			bFirst = false;
		}
		if (EnumHasAnyFlags(Container->Flags, EOnDemandContainerFlags::PendingHostGroup))
		{
			if (!bFirst)
			{
				FlagsSb << TEXT(", ");
			}

			FlagsSb << TEXT("PendingHostGroup");
			bFirst = false;
		}

		UE_LOGFMT(LogIoStoreOnDemand, Display, "\t Container: {Name}, Flags: {Flags}", *Container->UniqueName(), FlagsSb);
	}
}

bool FOnDemandIoStore::IsOnDemandStreamingEnabled() const
{
	return HttpIoBackend.IsValid();
}

void FOnDemandIoStore::SetStreamingOptions(EOnDemandStreamingOptions Options) 
{
	StreamingOptions = Options;
	if (HttpIoBackend.IsValid())
	{
		HttpIoBackend->SetOptionalBulkDataEnabled(EnumHasAnyFlags(Options, EOnDemandStreamingOptions::OptionalBulkDataDisabled) == false);
	}
}

void FOnDemandIoStore::GetHttpStats(FOnDemandHttpStats& Out) const
{
	FHttpIoDispatcher::GetHttpStats(Out);
}

void FOnDemandIoStore::ReportAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const
{
	if (Installer.IsValid())
	{
		Installer->ReportAnalytics(OutAnalyticsArray);
	}

	if (HttpIoBackend.IsValid())
	{
		HttpIoBackend->ReportAnalytics(OutAnalyticsArray);
	}
}

TUniquePtr<IAnalyticsRecording> FOnDemandIoStore::StartAnalyticsRecording() const
{
	return TUniquePtr<IAnalyticsRecording>();
}

void FOnDemandIoStore::OnImmediateAnalytic(FOnDemandImmediateAnalyticHandler EventHandler)
{
	OnDemandSetImmediateAnalyticHandler(MoveTemp(EventHandler));
}

void FOnDemandIoStore::SetDevelopmentExtension(TSharedPtr<IOnDemandDevelopmentExtension> Ext)
{
	DevExt = Ext;
}

bool FOnDemandIoStore::IsDevelopmentModeEnabled() const
{
	return IsDevModeEnabled();
}

} // namespace UE::IoStore

////////////////////////////////////////////////////////////////////////////////
FStringBuilderBase& operator<<(FStringBuilderBase& Sb, UE::IoStore::EOnDemandContainerFlags Flags)
{
	using namespace UE::IoStore;

	static const TCHAR* Names[]
	{
		TEXT("None"),
		TEXT("PendingEncryptionKey"),
		TEXT("Mounted"),
		TEXT("StreamOnDemand"),
		TEXT("InstallOnDemand"),
		TEXT("Encrypted"),
		TEXT("WithSoftReferences"),
		TEXT("PendingHostGroup")
	};

	if (Flags == EOnDemandContainerFlags::None)
	{
		Sb << TEXT("None");
		return Sb;
	}

	constexpr uint32 BitCount = 1 + FMath::CountTrailingZeros(
		static_cast<std::underlying_type_t<EOnDemandContainerFlags>>(EOnDemandContainerFlags::Last));
	static_assert(UE_ARRAY_COUNT(Names) == BitCount + 1, "Please update names list");

	for (int32 Idx = 0; Idx < BitCount; ++Idx)
	{
		const EOnDemandContainerFlags FlagToTest = static_cast<EOnDemandContainerFlags>(1 << Idx);
		if (EnumHasAnyFlags(Flags, FlagToTest))
		{
			if (Sb.Len())
			{
				Sb << TEXT("|");
			}
			Sb << Names[Idx + 1];
		}
	}

	return Sb;
}

FString LexToString(UE::IoStore::EOnDemandContainerFlags Flags)
{
	TStringBuilder<128> Sb;
	Sb << Flags;
	return FString::ConstructFromPtrSize(Sb.ToString(), Sb.Len());
}
