// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Algo/BinarySearch.h"
#include "Async/Async.h"
#include "Async/Mutex.h"
#include "Containers/AnsiString.h"
#include "Containers/BitArray.h"
#include "DiskCacheGovernor.h"
#include "IO/HttpIoDispatcher.h"
#include "IO/IoBuffer.h"
#include "IO/IoChunkEncoding.h"
#include "IO/IoChunkId.h"
#include "IO/IoHash.h"
#include "IO/IoStoreOnDemandInternals.h"
#include "IO/OnDemandDevelopmentExtension.h"
#include "IO/Serialization/OnDemandContainerToc.h"
#include "IasHostGroup.h"
#include "Misc/AES.h"
#include "Misc/EnumClassFlags.h"

#include <atomic>

enum class EForkProcessRole : uint8;

struct FIoContainerHeader;
using FSharedContainerHeader		= TSharedPtr<FIoContainerHeader>;

namespace UE::IoStore
{

class FOnDemandDebugCommands;
class FOnDemandContentInstaller;
struct FOnDemandInstallCacheConfig;
enum class ELogOnDemandCacheUsage : uint8;

using FSharedPackageStoreBackend	= TSharedPtr<class IOnDemandPackageStoreBackend>;
using FSharedInstallCache			= TSharedPtr<class IOnDemandInstallCache>;
using FSharedHttpIoBackend			= TSharedPtr<class IOnDemandHttpIoDispatcherBackend>;

///////////////////////////////////////////////////////////////////////////////
bool IsDevModeEnabled();
bool IsPackageStreamingEnabled();

///////////////////////////////////////////////////////////////////////////////
enum class EOnDemandContainerFlags : uint8
{
	None					= 0,
	PendingEncryptionKey	= (1 << 0),
	Mounted					= (1 << 1),
	StreamOnDemand			= (1 << 2),
	InstallOnDemand			= (1 << 3),
	Encrypted				= (1 << 4),
	WithSoftReferences		= (1 << 5),
	PendingHostGroup		= (1 << 6),
	Last = PendingHostGroup
};
ENUM_CLASS_FLAGS(EOnDemandContainerFlags);

///////////////////////////////////////////////////////////////////////////////
using FOnDemandPartitionEntry			= UE::IoStore::Serialization::V2::FOnDemandPartitionEntry;
using FOnDemandChunkEntry				= UE::IoStore::Serialization::V2::FOnDemandChunkEntry;
using FOnDemandTagSet					= UE::IoStore::Serialization::V2::FOnDemandTagSetEntry;
using FOnDemandTocStorage				= UE::IoStore::Serialization::FOnDemandTocStorage;
using FOnDemandContainerTocHeaderView	= UE::IoStore::Serialization::V2::FOnDemandContainerTocHeaderView;
using FOnDemandPartitionHash			= UE::IoStore::Serialization::V2::FOnDemandPartitionHash;
using FOnDemandChunkHash				= UE::IoStore::Serialization::V2::FOnDemandChunkHash;

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandChunkEntryReferences
{
	UPTRINT		ContentHandleId = 0;
	TBitArray<>	Indices;
};

using FSharedOnDemandContainer = TSharedPtr<struct FOnDemandContainer, ESPMode::ThreadSafe>;

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandContainerChunkEntryReferences
{
	FSharedOnDemandContainer	Container;
	TBitArray<>					Indices;
};

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandContainer
{
	FString									UniqueName() const;
	inline FString							Name() const;
	FIoContainerId							ContainerId() const { return HeaderView.ContainerId(); }
	const FGuid&							EncryptionKeyGuid() const { return HeaderView.EncryptionKeyGuid(); }
	uint32									BlockSize() const { return HeaderView.BlockSize(); }
	inline int32							GetNumChunks() const;
	inline int32							FindChunkEntryIndex(const FIoChunkId& ChunkId) const;
	inline const FOnDemandChunkEntry*		FindChunkEntry(const FIoChunkId& ChunkId, int32* OutIndex = nullptr) const;
	inline FOnDemandChunkEntryReferences&	FindOrAddChunkEntryReferences(const FOnDemandInternalContentHandle& ContentHandle);
	inline TBitArray<>						GetReferencedChunkEntries() const;
	inline bool								IsReferenced(int32 ChunkEntryIndex) const;

	inline FAnsiString						GetTestUrl();
	bool									HasAnyFlags(EOnDemandContainerFlags Contains) const { return EnumHasAnyFlags(Flags, Contains); }
	bool									HasAllFlags(EOnDemandContainerFlags Contains) const { return EnumHasAllFlags(Flags, Contains); }

	FAES::FAESKey							EncryptionKey;
	FOnDemandTocStorage						Storage;
	FOnDemandContainerTocHeaderView			HeaderView;
	FIoBuffer								ContainerHeaderChunk;
	FSharedContainerHeader					Header;
	FIASHostGroup							HostGroup;
	FString									MountId;
	TConstArrayView<FOnDemandPartitionEntry> PartitionEntries;
	TConstArrayView<FIoChunkId>				ChunkIds;
	TConstArrayView<FOnDemandChunkEntry> 	ChunkEntries;
	TConstArrayView<uint32>					BlockSizes;
	TConstArrayView<FIoBlockHash>			BlockHashes;
	TConstArrayView<FOnDemandTagSet>		TagSets;
	TConstArrayView<uint32>					TagSetIndices;
	TArray<FOnDemandChunkEntryReferences>	ChunkEntryReferences;
	FIoRelativeUrl							RelativeUrl;
	FName									CompressionFormat;
	FName									HostGroupName = FOnDemandHostGroup::DefaultName;
	mutable UE::FMutex						ReferencesMutex;
	EOnDemandContainerFlags 				Flags = EOnDemandContainerFlags::None;
};

///////////////////////////////////////////////////////////////////////////////
inline FString FOnDemandContainer::Name() const
{
	return FString(HeaderView.ContainerName());
}

int32 FOnDemandContainer::GetNumChunks() const
{
	return ChunkIds.Num();
}

int32 FOnDemandContainer::FindChunkEntryIndex(const FIoChunkId& ChunkId) const
{
	if (const int32 Index = Algo::LowerBound(ChunkIds, ChunkId); Index < ChunkIds.Num())
	{
		if (ChunkIds[Index] == ChunkId)
		{
			return Index;
		}
	}

	return INDEX_NONE;
}

const FOnDemandChunkEntry* FOnDemandContainer::FindChunkEntry(const FIoChunkId& ChunkId, int32* OutIndex) const
{
	if (int32 Index = FindChunkEntryIndex(ChunkId); Index != INDEX_NONE)
	{
		if (OutIndex != nullptr)
		{
			*OutIndex = Index;
		}
		return &ChunkEntries[Index];
	}

	return nullptr;
}

FOnDemandChunkEntryReferences& FOnDemandContainer::FindOrAddChunkEntryReferences(const FOnDemandInternalContentHandle& ContentHandle)
{
	const UPTRINT ContentHandleId = ContentHandle.HandleId(); 
	check(ContentHandleId != 0);

	for (FOnDemandChunkEntryReferences& Refs : ChunkEntryReferences)
	{
		if (Refs.ContentHandleId == ContentHandleId)
		{
			return Refs;
		}
	}

	FOnDemandChunkEntryReferences& NewRef = ChunkEntryReferences.AddDefaulted_GetRef();
	NewRef.ContentHandleId = ContentHandleId;
	NewRef.Indices.SetNum(ChunkEntries.Num(), false);
	return NewRef;
}

TBitArray<> FOnDemandContainer::GetReferencedChunkEntries() const
{
	TBitArray<> Indices;
	for (const FOnDemandChunkEntryReferences& Refs : ChunkEntryReferences)
	{
		check(Refs.ContentHandleId != 0);
		check(Refs.Indices.Num() == ChunkEntries.Num());
		Indices.CombineWithBitwiseOR(Refs.Indices, EBitwiseOperatorFlags::MaxSize);
	}

	return Indices;
}

bool FOnDemandContainer::IsReferenced(int32 ChunkEntryIndex) const
{
	for (const FOnDemandChunkEntryReferences& Refs : ChunkEntryReferences)
	{
		if (Refs.Indices[ChunkEntryIndex])
		{
			return true;
		}
	}

	return false;
}

FAnsiString FOnDemandContainer::GetTestUrl()
{
	if (ChunkEntries.IsEmpty())
	{
		return FAnsiString();
	}

	TAnsiStringBuilder<41> HashString;
	HashString << PartitionEntries[ChunkEntries[0].PartitionIndex].Hash;

	TAnsiStringBuilder<256> Url;
	Url << RelativeUrl.GetView()
		<< "/" << HashString.ToView().Left(2)
		<< "/" << HashString
		<< UE::IoStore::Serialization::FOnDemandFileExt::Partition;

	return Url.ToString();
}

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandChunkInfo
{
											FOnDemandChunkInfo() = default;
											FOnDemandChunkInfo(const FOnDemandChunkInfo& Other)
												: SharedContainer(Other.SharedContainer)
												, Entry(Other.Entry) { }
											FOnDemandChunkInfo(FOnDemandChunkInfo&& Other)
												: SharedContainer(MoveTemp(Other.SharedContainer))
												, Entry(Other.Entry) { }

	const FOnDemandChunkHash&				Hash() const { return Entry.Hash; }
	inline const FIoChunkId&				Id() const;
	const FIoHash&							PartitionHash() const { return SharedContainer->PartitionEntries[Entry.PartitionIndex].Hash; }
	uint32									PartitionOffset() const { return Entry.PartitionOffset; }
	uint32									RawSize() const { return Entry.RawSize; }
	uint32									EncodedSize() const { return Entry.EncodedSize; }
	uint32									DiskSize() const { return Entry.GetDiskSize(); }
	uint32									BlockSize() const { return SharedContainer->BlockSize(); }
	FName									CompressionFormat() const { return SharedContainer->CompressionFormat; }
	FMemoryView								EncryptionKey() const { return FMemoryView(SharedContainer->EncryptionKey.Key, FAES::FAESKey::KeySize); }
	inline TConstArrayView<uint32>			Blocks() const;
	inline TConstArrayView<FIoBlockHash>	BlockHashes() const;
	FIoRelativeUrl							RelativeUrl() const { return SharedContainer->RelativeUrl; }
	const FOnDemandChunkEntry&				ChunkEntry() const { return Entry; }
	inline const FIASHostGroup&				HostGroup() const;
	FName									HostGroupName() const { return SharedContainer->HostGroupName; }

	bool									IsValid() const { return SharedContainer.IsValid(); }
	operator								bool() const { return IsValid(); }
	
	inline static FOnDemandChunkInfo		Find(FSharedOnDemandContainer Container, const FIoChunkId& ChunkId);

	static const FOnDemandChunkEntry		NullEntry;

	explicit FOnDemandChunkInfo(FSharedOnDemandContainer InContainer, const FOnDemandChunkEntry& InEntry = FOnDemandChunkInfo::NullEntry)
		: SharedContainer(InContainer)
		, Entry(InEntry)
	{
		check(
			(&Entry == &NullEntry) || (
				(SharedContainer != nullptr) &&
				(SharedContainer->ChunkEntries.Num() > 0) &&
				(&Entry >= SharedContainer->ChunkEntries.GetData()) && 
				(&Entry <= &SharedContainer->ChunkEntries.Last())
			)
		);
	}

	FSharedOnDemandContainer	SharedContainer;
	const FOnDemandChunkEntry&	Entry = NullEntry;
};

const FIoChunkId& FOnDemandChunkInfo::Id() const
{
	if (&Entry == &NullEntry)
	{
		return FIoChunkId::InvalidChunkId;
	}

	const int32 Index = IntCastChecked<int32>(&Entry - SharedContainer->ChunkEntries.GetData());
	return SharedContainer->ChunkIds[Index];
}

TConstArrayView<uint32> FOnDemandChunkInfo::Blocks() const
{
	// When the chunk entry has a single compression block the block size is inlined into the block info as the first 4 bytes
	return Entry.BlockInfo.Count() > 1
		? TConstArrayView<uint32>(SharedContainer->BlockSizes.GetData() + Entry.BlockInfo.Offset(), Entry.BlockInfo.Count())
		: TConstArrayView<uint32>(Entry.BlockInfo.Data(), 1);
}

TConstArrayView<FIoBlockHash> FOnDemandChunkInfo::BlockHashes() const
{
	// When the chunk entry has a single compression block the block hash is inlined into the block info as the second 4 bytes
	return SharedContainer->BlockHashes.IsEmpty()
		? TConstArrayView<FIoBlockHash>()
		: Entry.BlockInfo.Count() > 1
			? TConstArrayView<FIoBlockHash>(SharedContainer->BlockHashes.GetData() + Entry.BlockInfo.Offset(), Entry.BlockInfo.Count())
			: TConstArrayView<FIoBlockHash>(Entry.BlockInfo.Data() + 1, 1);
}

const FIASHostGroup& FOnDemandChunkInfo::HostGroup() const
{
	return SharedContainer->HostGroup;
}

FOnDemandChunkInfo FOnDemandChunkInfo::Find(FSharedOnDemandContainer Container, const FIoChunkId& ChunkId)
{
	check(Container.IsValid());
	if (const FOnDemandChunkEntry* Entry = Container->FindChunkEntry(ChunkId))
	{
		return FOnDemandChunkInfo(Container, *Entry);
	}

	return FOnDemandChunkInfo();
}

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandChunkInfoListBase
{
											FOnDemandChunkInfoListBase() = default;
											explicit FOnDemandChunkInfoListBase(FSharedOnDemandContainer InContainer)
												: SharedContainer(MoveTemp(InContainer))
											{}

	const FOnDemandChunkHash&				Hash(int32 Index) const { return ChunkEntry(Index).Hash; }
	const FIoChunkId&						Id(int32 Index) const { return SharedContainer->ChunkIds[Index]; }
	const FIoHash&							PartitionHash(int32 Index) const { return SharedContainer->PartitionEntries[ChunkEntry(Index).PartitionIndex].Hash; }
	uint32									PartitionOffset(int32 Index) const { return ChunkEntry(Index).PartitionOffset; }
	uint32									RawSize(int32 Index) const { return ChunkEntry(Index).RawSize; }
	uint32									EncodedSize(int32 Index) const { return ChunkEntry(Index).EncodedSize; }
	uint32									DiskSize(int32 Index) const { return ChunkEntry(Index).GetDiskSize(); }
	uint32									BlockSize() const { return SharedContainer->BlockSize(); }
	FName									CompressionFormat() const { return SharedContainer->CompressionFormat; }
	FMemoryView								EncryptionKey() const { return FMemoryView(SharedContainer->EncryptionKey.Key, FAES::FAESKey::KeySize); }
	inline TConstArrayView<uint32>			Blocks(int32 Index) const;
	inline TConstArrayView<FIoBlockHash>	BlockHashes(int32 Index) const;
	FIoRelativeUrl							RelativeUrl() const { return SharedContainer->RelativeUrl; }
	const FOnDemandChunkEntry&				ChunkEntry(int32 Index) const { return SharedContainer->ChunkEntries[Index]; }
	inline const FIASHostGroup&				HostGroup() const;
	FName									HostGroupName() const { return SharedContainer->HostGroupName; }

	bool									IsValid() const { return SharedContainer.IsValid(); }
	operator								bool() const { return IsValid(); }

	FSharedOnDemandContainer				SharedContainer;
};

TConstArrayView<uint32> FOnDemandChunkInfoListBase::Blocks(int32 Index) const
{
	const FOnDemandChunkEntry& E = ChunkEntry(Index);
	// When the chunk entry has a single compression block the block size is inlined into the block info as the first 4 bytes
	return E.BlockInfo.Count() > 1
		? TConstArrayView<uint32>(SharedContainer->BlockSizes.GetData() + E.BlockInfo.Offset(), E.BlockInfo.Count())
		: TConstArrayView<uint32>(E.BlockInfo.Data(), 1);
}

TConstArrayView<FIoBlockHash> FOnDemandChunkInfoListBase::BlockHashes(int32 Index) const
{
	const FOnDemandChunkEntry& E = ChunkEntry(Index);
	// When the chunk entry has a single compression block the block hash is inlined into the block info as the second 4 bytes
	return SharedContainer->BlockHashes.IsEmpty()
		? TConstArrayView<FIoBlockHash>()
		: E.BlockInfo.Count() > 1
			? TConstArrayView<FIoBlockHash>(SharedContainer->BlockHashes.GetData() + E.BlockInfo.Offset(), E.BlockInfo.Count())
			: TConstArrayView<uint32>(E.BlockInfo.Data() + 1, 1);
}

const FIASHostGroup& FOnDemandChunkInfoListBase::HostGroup() const
{
	return SharedContainer->HostGroup;
}

template<typename Allocator /*= FDefaultAllocator*/>
struct TOnDemandChunkInfoList : public FOnDemandChunkInfoListBase
{
	TOnDemandChunkInfoList() = default;
	template<typename OtherAllocator>
	TOnDemandChunkInfoList(const TOnDemandChunkInfoList<OtherAllocator>& Other)
		: FOnDemandChunkInfoListBase(Other.SharedContainer)
		, Indices(Other.Indices) 
	{}
	TOnDemandChunkInfoList(TOnDemandChunkInfoList&& Other)
		: FOnDemandChunkInfoListBase(MoveTemp(Other.SharedContainer))
		, Indices(MoveTemp(Other.Indices)) 
	{}

	explicit TOnDemandChunkInfoList(FSharedOnDemandContainer InContainer)
		: FOnDemandChunkInfoListBase(MoveTemp(InContainer))
	{}

	template<typename OtherAllocator>
	explicit TOnDemandChunkInfoList(FSharedOnDemandContainer InContainer, const TArray<int32, OtherAllocator>& InIndices)
		: FOnDemandChunkInfoListBase(MoveTemp(InContainer))
		, Indices(InIndices)
	{}

	template<typename OtherAllocator>
	explicit TOnDemandChunkInfoList(FSharedOnDemandContainer InContainer, TArray<int32, OtherAllocator>&& InIndices)
		: FOnDemandChunkInfoListBase(MoveTemp(InContainer))
		, Indices(MoveTemp(InIndices))
	{}

	inline uint64							TotalDiskSize() const
	{
		uint64 OutDiskSize = 0;
		for (int32 Idx : Indices)
		{
			OutDiskSize += DiskSize(Idx);
		}

		return OutDiskSize;
	}

	inline static TOnDemandChunkInfoList	Find(FSharedOnDemandContainer Container, TConstArrayView<FIoChunkId> ChunkIds)
	{
		check(Container.IsValid());
		TArray<int32, Allocator> FoundIndices;
		FoundIndices.Reserve(ChunkIds.Num());
		for (const FIoChunkId& ChunkId : ChunkIds)
		{
			if (const int32 EntryIndex = Container->FindChunkEntryIndex(ChunkId); EntryIndex != INDEX_NONE)
			{
				FoundIndices.Add(EntryIndex);
			}
		}
		FoundIndices.Sort();
		return TOnDemandChunkInfoList(MoveTemp(Container), MoveTemp(FoundIndices));
	}

	TArray<int32, Allocator>				Indices;
};

using FOnDemandChunkInfoList = TOnDemandChunkInfoList<FDefaultAllocator>;

///////////////////////////////////////////////////////////////////////////////
/** Flush last access args. */
struct FOnDemandFlushLastAccessArgs
{
	TArray<TPair<FSharedOnDemandContainer, TBitArray<>>> RemovedChunkEntryReferences;
	bool bForceLastAccessDirty = true;
};

/** Result from flushing cache last access times. */
struct FOnDemandFlushLastAccessResult
{
	/** Returns True if the request succeeded. */
	bool IsOk() const { return Error.IsSet() == false; }
	/** Duration in seconds. */
	double DurationInSeconds = 0.0;
	/** Error information about the request. */ 
	TOptional<UE::UnifiedError::FError> Error;
};

/** Flush last access completion callback. */
using FOnDemandFlushLastAccessCompleted = TUniqueFunction<void(FOnDemandFlushLastAccessResult)>;

///////////////////////////////////////////////////////////////////////////////
class FOnDemandIoStore
	: public TSharedFromThis<FOnDemandIoStore, ESPMode::ThreadSafe>
	, public IOnDemandIoStore
{
	struct FMountRequest
	{
		FOnDemandMountArgs		Args;
		FOnDemandMountCompleted	OnCompleted;
		double					DurationInSeconds = 0.0;
	};

	using FSharedMountRequest	= TSharedRef<FMountRequest>;

public:
	UE_NONCOPYABLE(FOnDemandIoStore);

										FOnDemandIoStore();
										~FOnDemandIoStore();

	FOnDemandChunkInfo					GetStreamingChunkInfo(const FIoChunkId& ChunkId);
	FOnDemandChunkInfo					GetInstalledChunkInfo(const FIoChunkId& ChunkId, EIoErrorCode& OutErrorCode);

#if !UE_BUILD_SHIPPING
	/** Used to access a number of FIoChunkId that exist in the system when running tests and is not intended for production use */
	TArray<FIoChunkId>					DebugFindStreamingChunkIds(int32 NumToFind);
#endif //!UE_BUILD_SHIPPING
	void								GetReferencedContent(TArray<FSharedOnDemandContainer>& OutContainers, TArray<TBitArray<>>& OutChunkEntryIndices, bool bPackageStore = false);
	TBitArray<>							GetReferencedContent(const FSharedOnDemandContainer& Container);
	void								GetReferencedContentByHandle(TMap<FOnDemandWeakContentHandle, TArray<FOnDemandContainerChunkEntryReferences>>& OutReferencesByHandle) const;

	// IOnDemandIoStore interface
	virtual FIoStatus					Initialize() override;
	virtual FIoStatus					InitializePostHotfix() override;
	virtual FOnDemandRegisterHostGroupResult RegisterHostGroup(FOnDemandRegisterHostGroupArgs&& Args) override;
	virtual void						Mount(FOnDemandMountArgs&& Args, FOnDemandMountCompleted&& OnCompleted) override;
	virtual FOnDemandInstallRequest 	Install(FOnDemandInstallArgs&& Args,
											FOnDemandInstallCompleted&& OnCompleted,
											FOnDemandInstallProgressed&& OnProgress = nullptr) override;
	virtual void						Purge(FOnDemandPurgeArgs&& Args, FOnDemandPurgeCompleted&& OnCompleted) override;
	virtual void						Defrag(FOnDemandDefragArgs&& Args, FOnDemandDefragCompleted&& OnCompleted) override;
	virtual void						Verify(FOnDemandVerifyCacheCompleted&& OnCompleted) override;	
	virtual FIoStatus					Unmount(FStringView MountId) override;
	virtual TIoStatusOr<FOnDemandInstallSizeResult> GetInstallSize(const FOnDemandGetInstallSizeArgs& Args) const override;
	virtual FIoStatus					GetInstallSizesByMountId(const FOnDemandGetInstallSizeArgs& Args, TMap<FString, uint64>& OutSizesByMountId) const override;
	virtual FIoStatus					GetIsOnDemand(TConstArrayView<FOnDemandIsOnDemandArgs> Args, TArrayView<FOnDemandIsOnDemandResult> OutResults) const override;
	virtual FOnDemandCacheUsage			GetCacheUsage(const FOnDemandGetCacheUsageArgs& Args) const override;
	virtual void						DumpMountedContainersToLog() const override;
	virtual bool						IsOnDemandStreamingEnabled() const override;
	virtual void						SetStreamingOptions(EOnDemandStreamingOptions Options) override;
	virtual void						GetHttpStats(FOnDemandHttpStats& Out) const override;
	virtual void						ReportAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const override;
	virtual TUniquePtr<IAnalyticsRecording> StartAnalyticsRecording() const override;
	virtual void						OnImmediateAnalytic(FOnDemandImmediateAnalyticHandler EventHandler) override;
	virtual void						SetDevelopmentExtension(FSharedOnDemandDevelopmentExtension Ext) override;
	virtual bool						IsDevelopmentModeEnabled() const override;
	virtual void						CancelInstallRequest(FSharedInternalInstallRequest InstallRequest) override;
	virtual void						UpdateInstallRequestPriority(FSharedInternalInstallRequest InstallRequest, int32 NewPriority) override;
	virtual void						ReleaseContent(FOnDemandInternalContentHandle& ContentHandle) override;

	IOnDemandDevelopmentExtension*		TryGetDevelopmentExtension() { return DevExt.Get(); }
	TArray<FSharedOnDemandContainer>	GetContainers(EOnDemandContainerFlags ContainerFlags = EOnDemandContainerFlags::None) const;
	void								FlushLastAccess(FOnDemandFlushLastAccessArgs&& Args, FOnDemandFlushLastAccessCompleted&& OnCompleted);

private:
	friend class FOnDemandContentInstaller;

	FIoStatus							GetContainersForInstall(
											FStringView MountId, 
											TSet<FSharedOnDemandContainer>& OutContainersForInstallation) const;
	FIoStatus							GetContainersAndPackagesForInstall(
											FStringView MountId,
											const TArray<FString>& TagSets,
											const TArray<FPackageId>& PackageIds,
											TSet<FSharedOnDemandContainer>& OutContainersForInstallation,
											TSet<FPackageId>& OutPackageIdsToInstall) const;
	FIoStatus							InitializeInstallCache(FOnDemandInstallCacheConfig& OnDemandInstallCacheConfig);
	void								TryEnterTickLoop();
	void								TickLoop();
	bool								Tick();
	void								EnqueueMountRequest(FOnDemandMountArgs&& Args, FOnDemandMountCompleted&& OnCompleted);
	FIoStatus							ProcessMountRequest(FMountRequest& MountRequest);
	void								CompleteMountRequest(FMountRequest& MountRequest, FOnDemandMountResult&& MountResult);
	void								OnEncryptionKeyAdded(const FGuid& Id, const FAES::FAESKey& Key);
	void								OnHostGroupRegistered(const FName& HostGroup);
	static FIoStatus					CreateContainersFromToc(
											FOnDemandMountArgs& MountArgs,
											TConstArrayView<FIoContainerId> Existing,
											TArray<FSharedOnDemandContainer>& Out);
	static FIoStatus					SetupHostGroup(const FSharedOnDemandContainer& Container, const FOnDemandMountArgs& MountArgs);

#if !UE_BUILD_SHIPPING
	TUniquePtr<FOnDemandDebugCommands>		DebugCommands;
#endif // !UE_BUILD_SHIPPING

	TUniquePtr<FOnDemandContentInstaller>	Installer;
	FSharedInstallCache						InstallCache;
	FSharedPackageStoreBackend				PackageStoreBackend;
	FSharedHttpIoBackend					HttpIoBackend;
	FDiskCacheGovernor						DiskCacheGovernor;

	FDelegateHandle							OnMountPakHandle;
	FDelegateHandle							OnServerPostForkHandle;
	TArray<FSharedOnDemandContainer>		Containers;
	mutable UE::FMutex						ContainerMutex;

	TArray<FSharedMountRequest>				MountRequests;
	UE::FMutex								RequestMutex;

	EOnDemandStreamingOptions				StreamingOptions = EOnDemandStreamingOptions::Default;
	bool									bTicking = false;
	bool									bTickRequested = false;
	TFuture<void>							TickFuture;
	FSharedOnDemandDevelopmentExtension		DevExt;
};

} // namespace UE::IoStore

////////////////////////////////////////////////////////////////////////////////
FStringBuilderBase& operator<<(FStringBuilderBase& Sb, UE::IoStore::EOnDemandContainerFlags Flags);
FString LexToString(UE::IoStore::EOnDemandContainerFlags Flags);

