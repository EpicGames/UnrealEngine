// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnDemandMisc.h"

#include "Containers/Utf8String.h"
#include "IO/IoChunkId.h"
#include "IO/IoDispatcherInternal.h"
#include "IO/PackageId.h"
#include "Misc/StringBuilder.h"
#include "Misc/PackageName.h"

#define UE_API IOSTOREONDEMAND_API

namespace UE::IoStore
{

FStringView TryConvertChunkIdToPackageName(
	const FIoChunkId& ChunkId,
	FUtf8StringBuilderBase& OutFilename,
	FStringBuilderBase& OutPackageName,
	FStringBuilderBase* OutError)
{
	FUtf8StringView ContainerName;
	FUtf8StringView Filename = FIoDispatcherInternal::GetFilename(ChunkId, OutFilename, ContainerName);

	if (Filename.IsEmpty())
	{
		if (OutError != nullptr)
		{
			*OutError << TEXT("Filename missing for chunk ID '") << LexToString(ChunkId) << TEXT("'");
		}
		return FStringView();
	}

	bool bPackageDataChunk = false;
	switch (ChunkId.GetChunkType())
	{
		case EIoChunkType::ExportBundleData:
		case EIoChunkType::BulkData:
		case EIoChunkType::OptionalBulkData:
		case EIoChunkType::MemoryMappedBulkData:
			bPackageDataChunk = true;
		default:
			break;
	};

	if (bPackageDataChunk == false)
	{
		if (OutError != nullptr)
		{
			*OutError << TEXT("Non package chunk ID '") << ChunkId << TEXT("'");
		}
		return FStringView();
	}

	FPackageId PackageId;
	FMemory::Memcpy(&PackageId, &ChunkId, sizeof(FPackageId));

	if (PackageId.IsValid() == false)
	{
		if (OutError != nullptr)
		{
			*OutError << TEXT("Invalid package ID '") << LexToString(PackageId) << TEXT("'");
		}
		return FStringView();
	}

	FString Tmp(Filename);
	if (FPackageName::TryConvertFilenameToLongPackageName(Tmp, OutPackageName, OutError))
	{
		return OutPackageName.ToView();
	}

	return FStringView();
}

FString TryConvertChunkIdToPackageName(
	const FIoChunkId& ChunkId,
	FString& OutFilename,
	FStringBuilderBase* OutError)
{
	TStringBuilder<128> Packagename;
	TUtf8StringBuilder<128> Filename;
	FStringView View = TryConvertChunkIdToPackageName(ChunkId, Filename, Packagename, OutError);
	OutFilename = FString(Filename.ToView());
	return FString(View);
}

} // namespace UE::IoStore

#undef UE_API
