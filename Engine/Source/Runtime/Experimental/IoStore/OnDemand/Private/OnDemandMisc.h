// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"

class FIoChunkId;

#define UE_API IOSTOREONDEMAND_API

namespace UE::IoStore
{

UE_API FStringView TryConvertChunkIdToPackageName(
	const FIoChunkId& ChunkId,
	FUtf8StringBuilderBase& OutFilename,
	FStringBuilderBase& OutPackageName,
	FStringBuilderBase* OutError = nullptr);

UE_API FString TryConvertChunkIdToPackageName(
	const FIoChunkId& ChunkId,
	FString& OutFilename,
	FStringBuilderBase* OutError = nullptr);

} // namespace UE::IoStore

#undef UE_API 
