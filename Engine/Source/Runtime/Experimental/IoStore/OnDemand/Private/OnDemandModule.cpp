// Copyright Epic Games, Inc. All Rights Reserved.

#include "Features/IModularFeature.h"
#include "Features/IModularFeatures.h"
#include "IO/IoStoreOnDemand.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "OnDemandIoStore.h"

#if PLATFORM_WINDOWS
#	include <Windows/AllowWindowsPlatformTypes.h>
#		include <winsock2.h>
#	include <Windows/HideWindowsPlatformTypes.h>
#endif //PLATFORM_WINDOWS

namespace UE::IoStore
{

////////////////////////////////////////////////////////////////////////////////
struct FPlatformSocketSystem
{
#if PLATFORM_WINDOWS
	// Note that we use base types not UE types to match the platform API

	void Startup()
	{
		WSADATA WSAData;
		const int Result = WSAStartup(MAKEWORD(2, 2), &WSAData);
		if (Result == 0)
		{
			bLoaded = true;
		}
		else
		{
			// If WSAStartup fails it will return it's error code
			LogError(TEXT("WSAStartup"), Result);
		}
	}

	void Cleanup()
	{
		if (!bLoaded)
		{
			return;
		}

		const int Result = WSACleanup();

		if (Result != 0)
		{
			// If WSACleanup fails it returns SOCKET_ERROR and WSAGetLastError must be called instead
			const int SystemError = WSAGetLastError();
			LogError(TEXT("WSACleanup"), SystemError);
		}
	}

private:

	void LogError(const TCHAR* ErrorFunction, int ErrorCode)
	{
		TCHAR SystemErrorMsg[MAX_SPRINTF] = { 0 };
		FPlatformMisc::GetSystemErrorMessage(SystemErrorMsg, UE_ARRAY_COUNT(SystemErrorMsg), ErrorCode);
		UE_LOGF(LogIoStoreOnDemand, Error, "%ls failed due to: %ls (%d)", ErrorFunction, SystemErrorMsg, ErrorCode);
	}

	bool bLoaded = false;

#else

	void Startup()
	{
	}
	void Cleanup()
	{
	}

#endif // PLATFORM_WINDOWS
};

////////////////////////////////////////////////////////////////////////////////
class FOnDemandIoStoreFactory final
	: public IOnDemandIoStoreFactory
{
public:
	virtual IOnDemandIoStore* CreateInstance() override
	{
		check(IoStore == nullptr);

		IoStore = MakeShared<FOnDemandIoStore>();
		return IoStore.Get();
	}

	virtual void DestroyInstance(IOnDemandIoStore* Instance)
	{
		IoStore.Reset();
	}

private:
	FSharedOnDemandIoStore IoStore;
};

////////////////////////////////////////////////////////////////////////////////
class FIoStoreOnDemandModule final
	: public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	FPlatformSocketSystem PlatformSocketSystem;
	TUniquePtr<FOnDemandIoStoreFactory> Factory = MakeUnique<FOnDemandIoStoreFactory>();
};
IMPLEMENT_MODULE(UE::IoStore::FIoStoreOnDemandModule, IoStoreOnDemand);

////////////////////////////////////////////////////////////////////////////////
void FIoStoreOnDemandModule::StartupModule()
{
	PlatformSocketSystem.Startup();
	IModularFeatures::Get().RegisterModularFeature(IOnDemandIoStoreFactory::FeatureName, Factory.Get());
}

void FIoStoreOnDemandModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(IOnDemandIoStoreFactory::FeatureName, Factory.Get());
	PlatformSocketSystem.Cleanup();
}

} // namespace UE::IoStore
