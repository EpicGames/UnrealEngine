// Copyright Epic Games, Inc. All Rights Reserved.

#include "OnDemandPackageStoreBackend.h"

#include "Algo/Accumulate.h"
#include "Algo/Find.h"
#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "CoreGlobals.h"
#include "IO/IoChunkId.h"
#include "IO/IoContainerHeader.h"
#include "IO/IoStoreOnDemand.h"
#include "IO/PackageId.h"
#include "Internationalization/PackageLocalizationManager.h"
#include "Logging/StructuredLog.h"
#include "OnDemandIoStore.h"
#include "OnDemandMisc.h"

#if !UE_BUILD_SHIPPING
#include "IO/OnDemandDevelopmentExtension.h"
#endif

namespace UE::IoStore
{

///////////////////////////////////////////////////////////////////////////////
class FOnDemandPackageStoreBackend final
	: public IOnDemandPackageStoreBackend
{
	using FSoftPackageReferenceMap = TMap<FPackageId, const FFilePackageStoreEntrySoftReferences*>;

	struct FOnDemandPackageStoreEntry
	{
		uint32 PackageEntryIndex : 31;
		uint32 IsReferenced : 1;
		int32 ChunkEntryIndex;
	};

	using FSharedBackendContext = TSharedPtr<const FPackageStoreBackendContext>;
	using FEntryMap				= TMap<FPackageId, FOnDemandPackageStoreEntry>;
	using FRedirect				= TTuple<FName, FPackageId>;
	using FLocalizedMap			= TMap<FPackageId, FName>;
	using FRedirectMap			= TMap<FPackageId, FRedirect>;

	struct FContainer
	{
		FSharedOnDemandContainer	Container;
		FEntryMap					EntryMap;
		FLocalizedMap				LocalizedMap;
		FRedirectMap				RedirectMap;
		FSoftPackageReferenceMap	SoftRefs;
	};

public:
						FOnDemandPackageStoreBackend(TWeakPtr<FOnDemandIoStore> OnDemandIoStore);
						virtual ~FOnDemandPackageStoreBackend ();

	virtual void		NeedsUpdate(EOnDemandPackageStoreUpdateMode Mode) override;

	virtual void		OnMounted(TSharedRef<const FPackageStoreBackendContext> Context) override;
	virtual void		BeginRead() override;
	virtual void		EndRead() override;

	virtual bool		GetPackageRedirectInfo(
							FPackageId PackageId,
							FName& OutSourcePackageName,
							FPackageId& OutRedirectedToPackageId) override;

	virtual EPackageStoreEntryStatus GetPackageStoreEntry(
							FPackageId PackageId,
							FName PackageName,
							FPackageStoreEntry& OutPackageStoreEntry) override;
	virtual EPackageStoreEntryStatus GetPackageStoreEntryInternal(
							FPackageId PackageId,
							FPackageStoreEntry& OutPackageStoreEntry) override;
	virtual TConstArrayView<uint32> GetSoftReferences(FPackageId PackageId, TConstArrayView<FPackageId>& OutPackageIds) override;

private:
	const FOnDemandPackageStoreEntry* FindPackageEntry(
							const FPackageId& PackageId,
							FPackageStoreEntry& OutPackageStoreEntry) const;
	void				UpdateLookupTables(const TArray<FSharedOnDemandContainer>& AllContainers, const TArray<TBitArray<>>& ReferencedChunkEntryIndices);
	void				UpdateReferencedPackages(const TArray<TBitArray<>>& ReferencedChunkEntryIndices);

	TWeakPtr<FOnDemandIoStore>			OnDemandIoStore;
	TArray<FContainer>					Containers;
	UE::FMutex							Mutex;
	std::atomic<EOnDemandPackageStoreUpdateMode> NeedsUpdateMode{ EOnDemandPackageStoreUpdateMode::Full };
};

///////////////////////////////////////////////////////////////////////////////
FOnDemandPackageStoreBackend::FOnDemandPackageStoreBackend(TWeakPtr<FOnDemandIoStore> OnDemandIoStore)
	: OnDemandIoStore(MoveTemp(OnDemandIoStore))
{
}

FOnDemandPackageStoreBackend::~FOnDemandPackageStoreBackend()
{
}

void FOnDemandPackageStoreBackend::NeedsUpdate(EOnDemandPackageStoreUpdateMode Mode)
{
	if (Mode == EOnDemandPackageStoreUpdateMode::Full)
	{
		NeedsUpdateMode.store(EOnDemandPackageStoreUpdateMode::Full);
	}
	else if (Mode == EOnDemandPackageStoreUpdateMode::ReferencedPackages)
	{
		EOnDemandPackageStoreUpdateMode Expected = EOnDemandPackageStoreUpdateMode::None;
		NeedsUpdateMode.compare_exchange_strong(Expected, EOnDemandPackageStoreUpdateMode::ReferencedPackages);
	}
}

void FOnDemandPackageStoreBackend::OnMounted(TSharedRef<const FPackageStoreBackendContext> Context)
{
}

void FOnDemandPackageStoreBackend::BeginRead()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOnDemandPackageStoreBackend::BeginRead);

	EOnDemandPackageStoreUpdateMode LocalNeedsUpdate = NeedsUpdateMode.exchange(EOnDemandPackageStoreUpdateMode::None);

	if (LocalNeedsUpdate == EOnDemandPackageStoreUpdateMode::None)
	{
		Mutex.Lock();
		return;
	}

	TArray<TBitArray<>> ReferencedChunkEntryIndices;
	TSharedPtr<FOnDemandIoStore> PinOnDemandIoStore = OnDemandIoStore.Pin();

	if (LocalNeedsUpdate == EOnDemandPackageStoreUpdateMode::Full)
	{
		TArray<FSharedOnDemandContainer> AllContainers;
		if (PinOnDemandIoStore.IsValid())
		{
			const bool bPackageStore = true;
			PinOnDemandIoStore->GetReferencedContent(AllContainers, ReferencedChunkEntryIndices, bPackageStore);
		}
		Mutex.Lock();
		UpdateLookupTables(AllContainers, ReferencedChunkEntryIndices);
	}
	else
	{
		ensure(LocalNeedsUpdate == EOnDemandPackageStoreUpdateMode::ReferencedPackages);
		if (PinOnDemandIoStore.IsValid())
		{
			for (const FContainer& Container : Containers)
			{
				ReferencedChunkEntryIndices.Add(PinOnDemandIoStore->GetReferencedContent(Container.Container));
			}
		}
		Mutex.Lock();
		UpdateReferencedPackages(ReferencedChunkEntryIndices);
	}
}

void FOnDemandPackageStoreBackend::EndRead()
{
	Mutex.Unlock();
}

EPackageStoreEntryStatus FOnDemandPackageStoreBackend::GetPackageStoreEntry(
	FPackageId PackageId,
	FName PackageName,
	FPackageStoreEntry& OutPackageStoreEntry)
{
	const FOnDemandPackageStoreEntry* PackageEntry = FindPackageEntry(PackageId, OutPackageStoreEntry);
	if (PackageEntry == nullptr)
	{
		return EPackageStoreEntryStatus::Missing;
	}

	if (PackageEntry->IsReferenced || IsPackageStreamingEnabled())
	{
		return EPackageStoreEntryStatus::Ok;
	}

	EPackageStoreEntryStatus EntryStatus = EPackageStoreEntryStatus::NotInstalled;
#if !UE_BUILD_SHIPPING
	if (TSharedPtr<FOnDemandIoStore> Pinned = OnDemandIoStore.Pin(); Pinned.IsValid())
	{
		if (IOnDemandDevelopmentExtension* Ext = Pinned->TryGetDevelopmentExtension())
		{
			if (PackageName.IsNone())
			{
				TUtf8StringBuilder<128> Filename;
				TStringBuilder<128> Sb;

				if (FStringView PackageNameView =
					TryConvertChunkIdToPackageName(CreatePackageDataChunkId(PackageId), Filename, Sb);
					PackageNameView.IsEmpty() == false)
				{
					PackageName = FName(PackageNameView);
				}
			}

			if (EPackageStoreEntryStatus Override = Ext->GetUnreferencedPackageStoreEntryStatus(
				PackageName,
				PackageId,
				OutPackageStoreEntry,
				IsDevModeEnabled()); Override == EPackageStoreEntryStatus::Missing)
			{
				EntryStatus = Override;
			}
		}
	}
#endif
	return EntryStatus;
}

EPackageStoreEntryStatus FOnDemandPackageStoreBackend::GetPackageStoreEntryInternal(
	FPackageId PackageId,
	FPackageStoreEntry& OutPackageStoreEntry)
{
	if (const FOnDemandPackageStoreEntry* PackageEntry = FindPackageEntry(PackageId, OutPackageStoreEntry))
	{
		return EPackageStoreEntryStatus::Ok;
	}

	return EPackageStoreEntryStatus::Missing;
}

bool FOnDemandPackageStoreBackend::GetPackageRedirectInfo(
	FPackageId PackageId,
	FName& OutSourcePackageName,
	FPackageId& OutRedirectedToPackageId)
{
	for (const FContainer& Container : Containers)
	{
		if (const FRedirect* Redirect = Container.RedirectMap.Find(PackageId))
		{
			OutSourcePackageName		= Redirect->Key;
			OutRedirectedToPackageId	= Redirect->Value;
			return true;
		}

		if (const FName* SourcePkgName = Container.LocalizedMap.Find(PackageId))
		{
			const FName LocalizedPkgName = FPackageLocalizationManager::Get().FindLocalizedPackageName(*SourcePkgName);
			if (LocalizedPkgName.IsNone() == false)
			{
				const FPackageId LocalizedPkgId = FPackageId::FromName(LocalizedPkgName);
				if (Container.EntryMap.Find(LocalizedPkgId))
				{
					OutSourcePackageName		= *SourcePkgName;
					OutRedirectedToPackageId	= LocalizedPkgId;
					return true;
				}
			}
		}
	}

	return false;
}

TConstArrayView<uint32> FOnDemandPackageStoreBackend::GetSoftReferences(FPackageId PackageId, TConstArrayView<FPackageId>& OutPackageIds) 
{
	for (const FContainer& Container : Containers)
	{
		if (const FFilePackageStoreEntrySoftReferences* SoftRefs = Container.SoftRefs.FindRef(PackageId))
		{
			OutPackageIds = Container.Container->Header->SoftPackageReferences.PackageIds;
			return TConstArrayView<uint32>(SoftRefs->Indices.Data(), SoftRefs->Indices.Num());
		}
	}
	return TConstArrayView<uint32>();
}

const FOnDemandPackageStoreBackend::FOnDemandPackageStoreEntry* FOnDemandPackageStoreBackend::FindPackageEntry(
	const FPackageId& PackageId,
	FPackageStoreEntry& OutPackageStoreEntry) const
{
	for (const FContainer& Container : Containers)
	{
		if (const FOnDemandPackageStoreEntry* PackageEntry = Container.EntryMap.Find(PackageId))
		{
			const FIoContainerHeader& ContainerHeader			= *Container.Container->Header;
			const FFilePackageStoreEntry* PackageStoreEntries	= reinterpret_cast<const FFilePackageStoreEntry*>(ContainerHeader.StoreEntries.GetData());
			const FFilePackageStoreEntry& PackageStoreEntry		= *(PackageStoreEntries + PackageEntry->PackageEntryIndex);

			OutPackageStoreEntry.ImportedPackageIds =
				MakeArrayView(PackageStoreEntry.ImportedPackages.Data(), PackageStoreEntry.ImportedPackages.Num());
			OutPackageStoreEntry.ShaderMapHashes =
				MakeArrayView(PackageStoreEntry.ShaderMapHashes.Data(), PackageStoreEntry.ShaderMapHashes.Num());
			
			return PackageEntry;
		}
	}
	
	return nullptr;
}

void FOnDemandPackageStoreBackend::UpdateLookupTables(const TArray<FSharedOnDemandContainer>& AllContainers, const TArray<TBitArray<>>& ReferencedChunkEntryIndices)
{
	check(Mutex.IsLocked());
	check(ReferencedChunkEntryIndices.Num() ==  AllContainers.Num());

	Containers.Empty(AllContainers.Num());

	for (int32 ContainerIdx = 0; const FSharedOnDemandContainer& SharedContainer : AllContainers)
	{
		FContainer& Container			= Containers.AddDefaulted_GetRef();
		Container.Container				= SharedContainer;
		FEntryMap& EntryMap				= Container.EntryMap;
		const FIoContainerHeader& Hdr	= *SharedContainer->Header;
	
		EntryMap.Reserve(Hdr.PackageIds.Num());

		TConstArrayView<FFilePackageStoreEntry> Entries(
			reinterpret_cast<const FFilePackageStoreEntry*>(Hdr.StoreEntries.GetData()),
			Hdr.PackageIds.Num());

		TConstArrayView<FFilePackageStoreEntrySoftReferences> AllSoftReferences;
		if (Hdr.SoftPackageReferences.bContainsSoftPackageReferences)
		{
			AllSoftReferences = MakeArrayView<const FFilePackageStoreEntrySoftReferences>(
				reinterpret_cast<const FFilePackageStoreEntrySoftReferences*>(Hdr.SoftPackageReferences.PackageIndices.GetData()),
				Hdr.PackageIds.Num());
			Container.SoftRefs.Reserve(Hdr.PackageIds.Num());
		}

		check(ReferencedChunkEntryIndices[ContainerIdx].Num() == SharedContainer->GetNumChunks());

		for (int32 EntryIdx = 0; const FFilePackageStoreEntry& Entry : Entries)
		{
			const FPackageId PkgId = Hdr.PackageIds[EntryIdx];
			const FIoChunkId PkgChunkId = CreatePackageDataChunkId(PkgId);
			const int32 PkgChunkIdx = SharedContainer->FindChunkEntryIndex(PkgChunkId);
			check(PkgChunkIdx != INDEX_NONE);
			const bool IsReferenced = ReferencedChunkEntryIndices[ContainerIdx][PkgChunkIdx];

			EntryMap.Add(PkgId, FOnDemandPackageStoreEntry
			{
				.PackageEntryIndex	= uint32(EntryIdx),
				.IsReferenced		= uint32(IsReferenced),
				.ChunkEntryIndex	= PkgChunkIdx
			});

			if (AllSoftReferences.IsEmpty() == false)
			{
				const FFilePackageStoreEntrySoftReferences& SoftRefs = AllSoftReferences[EntryIdx];
				if (SoftRefs.Indices.Num() > 0)
				{
					Container.SoftRefs.Add(PkgId, &SoftRefs);
				}
			}
			++EntryIdx;
		}
		EntryMap.Shrink();
		Container.SoftRefs.Shrink();

		FLocalizedMap& LocalizedMap = Container.LocalizedMap;
		for (const FIoContainerHeaderLocalizedPackage& Localized : Hdr.LocalizedPackages)
		{
			FName& SourcePackageName = LocalizedMap.FindOrAdd(Localized.SourcePackageId);
			if (SourcePackageName.IsNone())
			{
				FDisplayNameEntryId NameEntry = Hdr.RedirectsNameMap[Localized.SourcePackageName.GetIndex()];
				SourcePackageName = NameEntry.ToName(Localized.SourcePackageName.GetNumber());
			}
		}
		LocalizedMap.Shrink();

		FRedirectMap& RedirectMap = Container.RedirectMap;
		for (const FIoContainerHeaderPackageRedirect& Redirect : Hdr.PackageRedirects)
		{
			FRedirect& RedirectEntry = RedirectMap.FindOrAdd(Redirect.SourcePackageId);
			FName& SourcePackageName = RedirectEntry.Key;
			if (SourcePackageName.IsNone())
			{
				FDisplayNameEntryId NameEntry = Hdr.RedirectsNameMap[Redirect.SourcePackageName.GetIndex()];
				SourcePackageName = NameEntry.ToName(Redirect.SourcePackageName.GetNumber());
				RedirectEntry.Value = Redirect.TargetPackageId;
			}
		}
		RedirectMap.Shrink();

		++ContainerIdx;
	}
}

void FOnDemandPackageStoreBackend::UpdateReferencedPackages(const TArray<TBitArray<>>& ReferencedChunkEntryIndices)
{
	check(Mutex.IsLocked());

	if (ReferencedChunkEntryIndices.IsEmpty())
	{
		for (FContainer& Container : Containers)
		{
			for (TPair<FPackageId, FOnDemandPackageStoreEntry>& Kv : Container.EntryMap)
			{
				Kv.Value.IsReferenced = 0; 
			}
		}
	}
	else
	{
		check(ReferencedChunkEntryIndices.Num() == Containers.Num());
		for (int32 ContainerIdx = 0; FContainer& Container : Containers)
		{
			const TBitArray<>& Refs = ReferencedChunkEntryIndices[ContainerIdx];
			if (Refs.IsEmpty())
			{
				for (TPair<FPackageId, FOnDemandPackageStoreEntry>& Kv : Container.EntryMap)
				{
					Kv.Value.IsReferenced = false;
				}
			}
			else
			{
				ensure(Container.Container->GetNumChunks() == Refs.Num());
				for (TPair<FPackageId, FOnDemandPackageStoreEntry>& Kv : Container.EntryMap)
				{
					Kv.Value.IsReferenced = Refs[Kv.Value.ChunkEntryIndex];
				}
			}
			++ContainerIdx;
		}
	}
}

///////////////////////////////////////////////////////////////////////////////
TSharedPtr<IOnDemandPackageStoreBackend> MakeOnDemandPackageStoreBackend(TWeakPtr<FOnDemandIoStore> OnDemandIoStore)
{
	return MakeShared<FOnDemandPackageStoreBackend>(MoveTemp(OnDemandIoStore));
}

} // UE::IoStore
