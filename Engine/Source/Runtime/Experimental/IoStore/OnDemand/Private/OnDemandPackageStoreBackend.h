// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IO/IoStatus.h"
#include "Serialization/PackageStore.h"
#include "Templates/SharedPointer.h"

struct FIoContainerHeader;
class FOnDemandIoStore;
using FSharedContainerHeader = TSharedPtr<FIoContainerHeader>;

namespace UE::IoStore
{

enum class EOnDemandPackageStoreUpdateMode : uint8
{
	None,
	Full,
	ReferencedPackages,
};

class IOnDemandPackageStoreBackend
	: public IPackageStoreBackend
{
public:
	virtual ~IOnDemandPackageStoreBackend() = default;

	virtual EPackageStoreEntryStatus GetPackageStoreEntryInternal(FPackageId PackageId, FPackageStoreEntry& Out) = 0;
	virtual void NeedsUpdate(EOnDemandPackageStoreUpdateMode Mode = EOnDemandPackageStoreUpdateMode::Full) = 0;
};

TSharedPtr<IOnDemandPackageStoreBackend> MakeOnDemandPackageStoreBackend(TWeakPtr<FOnDemandIoStore> OnDemandIoStore);

} // namespace UE::IoStore
