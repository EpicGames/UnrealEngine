// Copyright Epic Games, Inc. All Rights Reserved.

#include "Statistics.h"

#include "Async/Mutex.h"
#include "Async/UniqueLock.h"
#include "Algo/MaxElement.h"
#include "AnalyticsEventAttribute.h"
#include "Experimental/UnifiedError/CoreErrorTypes.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformTime.h"
#include "IO/IoStatus.h"
#include "IO/IoStoreOnDemand.h"
#include "IasHostGroup.h"
#include "Internationalization/Internationalization.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/CoreDelegates.h"
#include "Logging/StructuredLog.h"
#include "OnDemandHttpIoDispatcher.h"
#include "Templates/Requires.h"

LLM_DEFINE_TAG(Ias);

#if IAS_WITH_STATISTICS

#define UE_ENABLE_ANALYTICS_RECORDING 0

namespace UE::IoStore
{

bool IsDevModeEnabled();

extern int32 GHttpIoDispatcherMaxRangeKiB;

int32 CalculateRequestQueueLengthMax();
int32 GetHttpRateLimitKiBPerSecond();

float GIasStatisticsLogInterval = 60.f;
static FAutoConsoleVariableRef CVar_StatisticsLogInterval(
	TEXT("ias.StatisticsLogInterval"),
	GIasStatisticsLogInterval,
	TEXT("Enables and sets interval for periodic logging of statistics"));

bool GIasReportHttpAnalyticsEnabled = true;
static FAutoConsoleVariableRef CVar_ReportHttpAnalytics(
	TEXT("ias.ReportHttpAnalytics"),
	GIasReportHttpAnalyticsEnabled,
	TEXT("Enables reporting statics on our http traffic to the analytics system"));

bool GIasReportCacheAnalyticsEnabled = true;
static FAutoConsoleVariableRef CVar_ReportCacheAnalytics(
	TEXT("ias.ReportCacheAnalytics"),
	GIasReportCacheAnalyticsEnabled,
	TEXT("Enables reporting statics on our file cache usage to the analytics system"));

bool GIadReportAnalyticsEnabled = true;
static FAutoConsoleVariableRef CVar_ReportIadAnalyticsEnabled(
	TEXT("iad.ReportAnalytics"),
	GIadReportAnalyticsEnabled,
	TEXT("Enables reporting analytics for individual asset downloads."));

float GIaxImmediateAnalyticsChance = 1.0f;
static FAutoConsoleVariableRef CVar_ImmediateAnalyticsChance(
	TEXT("iax.ImmediateAnalyticsChance"),
	GIaxImmediateAnalyticsChance,
	TEXT("Chance of sending and immediate analytics event."));

#if UE_ENABLE_ONSCREEN_STATISTICS

bool GIasDisplayOnScreenStatistics = false;
static FAutoConsoleVariableRef CVar_DisplayOnScreenStatistics(
	TEXT("ias.DisplayOnScreenStatistics"),
	GIasDisplayOnScreenStatistics,
	TEXT("Enables display of Ias on screen statistics"));

#endif // UE_ENABLE_ONSCREEN_STATISTICS

#if !UE_BUILD_SHIPPING
FAutoConsoleCommand CmdLogOnDemandContentInstallerStats(
	TEXT("iad.LogStats"), 
	TEXT("Dump IAD content installer stats to log"), 
	FConsoleCommandDelegate::CreateLambda([]()
	{
		FOnDemandContentInstallerStats::LogAnalytics();
	})
);
#endif // !UE_BUILD_SHIPPING

////////////////////////////////////////////////////////////////////////////////
static FOnDemandImmediateAnalyticHandler OnDemandImmediateAnalyticEventHandler;
static TArray<FOnDemandImmediateAnalytic> QueuedOnDemandImmediateAnalyticEventStartupEvents;
static FMutex OnDemandImmediateAnalyticEventHandlerMutex;
void OnDemandSetImmediateAnalyticHandler(FOnDemandImmediateAnalyticHandler&& EventHandler)
{
	TUniqueLock Lock(OnDemandImmediateAnalyticEventHandlerMutex);
	OnDemandImmediateAnalyticEventHandler = MoveTemp(EventHandler);

	if (OnDemandImmediateAnalyticEventHandler.EventHandler)
	{
		if (GIaxImmediateAnalyticsChance > 0.0f && FMath::FRand() <= GIaxImmediateAnalyticsChance)
		{
			for (FOnDemandImmediateAnalytic& QueuedEvent : QueuedOnDemandImmediateAnalyticEventStartupEvents)
			{
				OnDemandImmediateAnalyticEventHandler.EventHandler(MoveTemp(QueuedEvent));
			}
		}

		QueuedOnDemandImmediateAnalyticEventStartupEvents.Empty();
	}
}

static void SendImmediateAnalytic(FOnDemandImmediateAnalytic&& Event)
{
	if (GIaxImmediateAnalyticsChance > 0.0f && FMath::FRand() <= GIaxImmediateAnalyticsChance)
	{
		TUniqueLock Lock(OnDemandImmediateAnalyticEventHandlerMutex);

		if (OnDemandImmediateAnalyticEventHandler.EventHandler)
		{
			OnDemandImmediateAnalyticEventHandler.EventHandler(MoveTemp(Event));
		}
		else
		{
			if (ensure(QueuedOnDemandImmediateAnalyticEventStartupEvents.Num() < 16))
			{
				QueuedOnDemandImmediateAnalyticEventStartupEvents.Add(MoveTemp(Event));
			}
		}
	}
}

FOnDemandImmediateAnalytic MakeAnalyticsEventFromResult(const TCHAR* EventName, const FResult& Result)
{
	FString ErrorCode;
	FString ErrorJson;

	if (!Result.HasError())
	{
		ErrorCode = TEXTVIEW("Success");
		ErrorJson = TEXTVIEW("{\"Root\":{\"ErrorCodeString\":\"Success\", \"ModuleIdString\":\"UnifiedError\"}}");
	}
	else
	{
		ErrorCode = FString(Result.GetError().GetModuleIdAndErrorCodeString());
		ErrorJson = Result.GetError().SerializeToJsonForAnalytics();
	}

	return FOnDemandImmediateAnalytic
	{
		.EventName		= EventName,
		.AnalyticsArray	= MakeAnalyticsEventAttributeArray(TEXT("ErrorCode"), ErrorCode, TEXT("ErrorJson"), FJsonFragment(MoveTemp(ErrorJson)))
	};
}

////////////////////////////////////////////////////////////////////////////////
int32 TryGetLineNo(const UE::UnifiedError::FError& Error)
{
	int32 LineNo = -1;
	const UE::IoStore::OnDemand::FInstallCacheErrorContext* ErrorCtx =
		Error.GetErrorContext<UE::IoStore::OnDemand::FInstallCacheErrorContext>();
	if (ErrorCtx)
	{
		LineNo = ErrorCtx->State.LineNo;
	}
	return LineNo;
}

int32 TryGetLineNo(const FResult& Result)
{
	int32 LineNo = -1;
	if (Result.HasError())
	{
		LineNo = TryGetLineNo(Result.GetError());
	}
	return LineNo;
}

////////////////////////////////////////////////////////////////////////////////
static int32 BytesToApproxMB(uint64 Bytes) { return int32(Bytes >> 20); }
static int32 BytesToApproxKB(uint64 Bytes) { return int32(Bytes >> 10); }

/**
 * Code taken from SummarizeTraceCommandlet.cpp pending discussion on moving it
 * somewhere for general use.
 * Currently not thread safe!
 */
class FIncrementalVariance
{
public:
	FIncrementalVariance()
		: Count(0)
		, Mean(0.0)
		, VarianceAccumulator(0.0)
	{

	}

	uint64 GetCount() const
	{
		return Count;
	}

	double GetMean() const
	{
		return Mean;
	}

	/**
	* Compute the variance given Welford's accumulator and the overall count
	*
	* @return The variance in sample units squared
	*/
	double GetVariance() const
	{
		double Result = 0.0;

		if (Count > 1)
		{
			// Welford's final step, dependent on sample count
			Result = VarianceAccumulator / double(Count - 1);
		}

		return Result;
	}

	/**
	* Compute the standard deviation given Welford's accumulator and the overall count
	*
	* @return The standard deviation in sample units
	*/
	double GetDeviation() const
	{
		double Result = 0.0;

		if (Count > 1)
		{
			// Welford's final step, dependent on sample count
			double DeviationSqrd = VarianceAccumulator / double(Count - 1);

			// stddev is sqrt of variance, to restore to units (vs. units squared)
			Result = sqrt(DeviationSqrd);
		}

		return Result;
	}

	/**
	* Perform an increment of work for Welford's variance, from which we can compute variation and standard deviation
	*
	* @param InSample	The new sample value to operate on
	*/
	void Increment(const double InSample)
	{
		Count++;
		const double OldMean = Mean;
		Mean += ((InSample - Mean) / double(Count));
		VarianceAccumulator += ((InSample - Mean) * (InSample - OldMean));
	}

	/**
	* Merge with another IncrementalVariance series in progress
	*
	* @param Other	The other variance incremented from another mutually exclusive population of analogous data.
	*/
	void Merge(const FIncrementalVariance& Other)
	{
		// empty other, nothing to do
		if (Other.Count == 0)
		{
			return;
		}

		// empty this, just copy other
		if (Count == 0)
		{
			Count = Other.Count;
			Mean = Other.Mean;
			VarianceAccumulator = Other.VarianceAccumulator;
			return;
		}

		const double TotalPopulation = static_cast<double>(Count + Other.Count);
		const double MeanDifference = Mean - Other.Mean;
		const double A = (double(Count - 1) * GetVariance()) + (double(Other.Count - 1) * Other.GetVariance());
		const double B = (MeanDifference) * (MeanDifference) * (double(Count) * double(Other.Count) / TotalPopulation);
		const double MergedVariance = (A + B) / (TotalPopulation - 1);

		const uint64 NewCount = Count + Other.Count;
		const double NewMean = ((Mean * double(Count)) + (Other.Mean * double(Other.Count))) / double(NewCount);
		const double NewVarianceAccumulator = MergedVariance * double(NewCount - 1);

		Count = NewCount;
		Mean = NewMean;
		VarianceAccumulator = NewVarianceAccumulator;
	}

	/**
	* Reset state back to initialized.
	*/
	void Reset()
	{
		Count = 0;
		Mean = 0.0;
		VarianceAccumulator = 0.0;
	}

private:
	uint64 Count;
	double Mean;
	double VarianceAccumulator;
};

class FDeltaTracking
{
public:
	int64 Get(FStringView Name, int64 Value)
	{
		if (int64* PrevValue = IntTotals.FindByHash(GetTypeHash(Name), Name))
		{
			const int64 Delta = Value - *PrevValue;
			*PrevValue = Value;

			return Delta;
		}
		else
		{
			IntTotals.Add(FString(Name), Value);
			return Value;
		}
	}

	uint32 Get(FStringView Name, uint32 Value)
	{
		return static_cast<uint32>(Get(Name, static_cast<int64>(Value)));
	}

	double Get(FStringView Name, double Value)
	{
		if (double* PrevValue = RealTotals.FindByHash(GetTypeHash(Name), Name))
		{
			const double Delta = Value - *PrevValue;
			*PrevValue = Value;

			return Delta;
		}
		else
		{
			RealTotals.Add(FString(Name), 0.0);
			return Value;
		}
	}

private:

	TMap<FString, int64> IntTotals;
	TMap<FString, double> RealTotals;

} static GDeltaTracking;

class FDiskWriteRate
{
public:
	FDiskWriteRate(double InWindowSpanSeconds = 5.0, double InBucketWidthSeconds = 0.050)
		: WindowSpan(InWindowSpanSeconds)
		, BucketWidth(InBucketWidthSeconds)
		, StartTime(FPlatformTime::Seconds())
		, NumBuckets(FMath::Max(1, FMath::CeilToInt32(InWindowSpanSeconds / InBucketWidthSeconds)))
	{
		Buckets.SetNumZeroed(NumBuckets);
	}

	void SubmitWrite(uint64 Bytes)
	{
		TUniqueLock Lock(BucketCS);
		const int64 Idx = CurrentIndex();
		Advance(Idx);
		Buckets[Idx % NumBuckets] += Bytes;
		TotalBytes += Bytes;
	}

	double GetBytesPerSecond()
	{
		TUniqueLock Lock(BucketCS);
		Advance(CurrentIndex());
		return static_cast<double>(TotalBytes) / WindowSpan;
	}

	double GetMegabytesPerSecond()
	{
		return GetBytesPerSecond() / (1024.0 * 1024.0);
	}

private:
	int64 CurrentIndex() const
	{
		return static_cast<int64>((FPlatformTime::Seconds() - StartTime) / BucketWidth);
	}

	void Advance(int64 Idx)
	{
		if (Idx <= LastBucketIndex)
		{
			return;
		}

		const int64 Delta = Idx - LastBucketIndex;
		if (Delta >= NumBuckets)
		{
			FMemory::Memzero(Buckets.GetData(), Buckets.Num() * sizeof(Buckets[0]));
			TotalBytes = 0;
		}
		else
		{
			for (int64 i = LastBucketIndex + 1; i <= Idx; ++i)
			{
				const int32 Slot = i % NumBuckets;
				TotalBytes -= Buckets[Slot];
				Buckets[Slot] = 0;
			}
		}

		LastBucketIndex = Idx;
	}

private:
	double WindowSpan;
	double BucketWidth;
	double StartTime;
	int32 NumBuckets;
	int64 LastBucketIndex = 0;
	uint64 TotalBytes = 0;
	TArray<uint64> Buckets;
	FMutex BucketCS;
} static GWriteRate;

////////////////////////////////////////////////////////////////////////////////
// TRACE STATS

#if COUNTERSTRACE_ENABLED
	using FCounterInt			= FCountersTrace::FCounterInt;
	using FCounterFloat			= FCountersTrace::FCounterFloat;
	using FCounterAtomicInt		= FCountersTrace::FCounterAtomicInt;
	using FCounterAtomicFloat	= FCountersTrace::FCounterAtomicFloat;
#else
	template <typename Type>
	struct TCounterInt
	{
		TCounterInt(...)  {}
		void Set(int64 i) { V = i; }
		void Add(int64 d) { V += d; }
		void Subtract(int64 d) { V -= d; }
		void Increment() { V++; }
		void Decrement() { --V; }
		int64 Get() const { return V;}
		Type V = 0;
	};
	template <typename Type>
	struct TCounterFloat
	{
		TCounterFloat(...) {}
		void Set(Type i) { V = i; }
		void Add(Type d) { V += d; }
		void Increment() { V += 1; }
		void Decrement() { V -= 1; }
		Type Get() const { return V;}
		Type V = 0;
	};

	template<typename Type>
	struct TCounterFloat<std::atomic<Type>>
	{
		TCounterFloat(...) {}
		void Set(Type i) { V.store(i , std::memory_order_relaxed); }
		void Add(Type d) { V.fetch_add(d, std::memory_order_relaxed); }
		void Increment() { V.fetch_add(1, std::memory_order_relaxed); }
		void Decrement() { V.fetch_add(-1, std::memory_order_relaxed); }
		Type Get() const { return V.load(std::memory_order_relaxed); }
		std::atomic<Type> V{ Type(0) };
	};
	using FCounterInt		= TCounterInt<int64>;
	using FCounterFloat		= TCounterFloat<double>;
	using FCounterAtomicInt = TCounterInt<std::atomic<int64>>;
	using FCounterAtomicFloat = TCounterFloat<std::atomic<double>>;
#endif

#define UE_IAX_COUNTER(Name, Type) Type G##Name[(uint8)EHttpRequestType::NUM_SOURCES] = {{TEXT(UE_STRINGIZE(Ias/Name)), TraceCounterDisplayHint_None}, {TEXT(UE_STRINGIZE(Iad/Name)), TraceCounterDisplayHint_None}};
#define UE_IAX_MEMORY_COUNTER(Name, Type) Type G##Name[(uint8)EHttpRequestType::NUM_SOURCES] = {{TEXT(UE_STRINGIZE(Ias/Name)), TraceCounterDisplayHint_None}, {TEXT(UE_STRINGIZE(Iad/Name)), TraceCounterDisplayHint_Memory}};

// cache stats
static uint32			GCacheBootMs = 0;
FCounterAtomicInt		GCacheErrorCount(TEXT("Iax/StreamingCache/CacheErrorCount"), TraceCounterDisplayHint_None);
FCounterAtomicInt		GCacheDecodeErrorCount(TEXT("Iax/StreamingCache/DecodeErrorCount"), TraceCounterDisplayHint_None);
FCounterAtomicInt		GCacheGetCount(TEXT("Iax/StreamingCache/GetCount"), TraceCounterDisplayHint_None);
FCounterAtomicInt		GCachePutCount(TEXT("Iax/StreamingCache/PutCount"), TraceCounterDisplayHint_None);
FCounterAtomicInt		GCachePutExistingCount(TEXT("Iax/StreamingCache/PutExistingCount"), TraceCounterDisplayHint_None);
FCounterAtomicInt		GCachePutRejectCount(TEXT("Iax/StreamingCache/PutRejectCount"), TraceCounterDisplayHint_None);
FCounterAtomicInt		GCacheCachedBytes(TEXT("Iax/StreamingCache/CachedBytes"), TraceCounterDisplayHint_Memory);
FCounterAtomicInt		GCacheCachedEntries(TEXT("Iax/StreamingCache/CachedEntries"), TraceCounterDisplayHint_Memory);
FCounterAtomicInt		GCacheWriteCount(TEXT("Iax/StreamingCache/WriteCount"), TraceCounterDisplayHint_Memory);
FCounterAtomicInt		GCacheWrittenBytes(TEXT("Iax/StreamingCache/WriteBytes"), TraceCounterDisplayHint_Memory);
FCounterAtomicInt		GCacheEvictedBytes(TEXT("Iax/StreamingCache/EvictedBytes"), TraceCounterDisplayHint_Memory);
FCounterFloat			GCacheSuspendedSeconds(TEXT("Iax/StreamingCache/SuspendedSeconds"), TraceCounterDisplayHint_None);
int64					GCacheMaxBytes = 0;
FCounterAtomicInt		GCachePendingBytes(TEXT("Iax/StreamingCache/PendingBytes"), TraceCounterDisplayHint_Memory);
FCounterAtomicInt		GCacheReadBytes(TEXT("Iax/StreamingCache/ReadBytes"), TraceCounterDisplayHint_Memory);
FCounterAtomicInt		GCacheReadMemory(TEXT("Iax/StreamingCache/ReadMemoryBytes"), TraceCounterDisplayHint_Memory);
FCounterAtomicInt		GCacheReadDisk(TEXT("Iax/StreamingCache/ReadDiskBytes"), TraceCounterDisplayHint_Memory);
FCounterAtomicInt		GCacheRejectBytes(TEXT("Iax/StreamingCache/PutRejectBytes"), TraceCounterDisplayHint_Memory);
FCounterAtomicFloat		GCacheWriteRate(TEXT("Iax/DiskWriteMegabytesPerSecond"), TraceCounterDisplayHint_None);
// general stats
double					GAnalyticsStartTiming = 0.0;
// http stats
int64					GAppResumeCount = 0;
FCounterInt				GHttpConnectCount(TEXT("Ias/HttpConnectCount"), TraceCounterDisplayHint_None);
FCounterInt				GHttpDisconnectCount(TEXT("Ias/HttpDisconnectCount"), TraceCounterDisplayHint_None);
UE_IAX_COUNTER(HttpGetCount, FCounterInt);
UE_IAX_COUNTER(HttpErrorCount, FCounterInt);
UE_IAX_COUNTER(HttpDecodeErrorCount, FCounterAtomicInt);
UE_IAX_COUNTER(HttpRetryCount, FCounterInt);
UE_IAX_COUNTER(HttpCancelCount, FCounterInt);
UE_IAX_COUNTER(HttpPendingCount, FCounterAtomicInt);
UE_IAX_MEMORY_COUNTER(HttpPendingBytes, FCounterAtomicInt);
UE_IAX_COUNTER(HttpInflightCount, FCounterInt);
UE_IAX_MEMORY_COUNTER(HttpDownloadedBytes, FCounterInt);
UE_IAX_COUNTER(HttpDurationMs, FCounterInt);
UE_IAX_COUNTER(HttpBandwidthMbps, FCounterFloat);
double GHttpDurationMsAvg[(uint8)EHttpRequestType::NUM_SOURCES] = { 0.0 };
int32 GHttpDurationMsMax[(uint8)EHttpRequestType::NUM_SOURCES] = { 0 };
int64 GHttpDurationMsSum[(uint8)EHttpRequestType::NUM_SOURCES] = { 0 };
std::atomic<uint32> GHttpNumPausedRequests[(uint8)EHttpRequestType::NUM_SOURCES] = {};
std::atomic<uint32> GHttpNumConcurrentRequests = 0;

struct FHttpRecentHistoryStatistics
{
	static const int64 HistoryCount = 16;

	int64			Duration[HistoryCount] = {};
	int64			Bytes[HistoryCount] = {};
	int64			TotalDuration = 0;
	int64			TotalBytes = 0;
	int64 			Index = 0;

	void OnGet(uint64 SizeBytes, uint64 DurationMs)
	{
		int64 OldDuration = Duration[Index];
		int64 NewDuration = (int64)DurationMs;

		TotalDuration -= OldDuration;
		TotalDuration += NewDuration;
		Duration[Index] = NewDuration;

		TotalBytes -= Bytes[Index];
		TotalBytes += SizeBytes;
		Bytes[Index] = SizeBytes;

		Index = (Index + 1) % HistoryCount;
	}

	double GetBandwidthMbps() const
	{
		return double(TotalBytes * 8) / double(TotalDuration + 1) / 1000.0;
	}

	double GetAverage() const
	{
		return static_cast<double>(TotalDuration) / static_cast<double>(HistoryCount);
	}
};

FHttpRecentHistoryStatistics GHttpHistory[(uint8)EHttpRequestType::NUM_SOURCES];

// Experimental Http Stats
static uint32			GHttpCdnCacheHit = 0;
static uint32			GHttpCdnCacheMiss = 0;
static uint32			GHttpCdnCacheUnknown = 0;

////////////////////////////////////////////////////////////////////////////////
// CSV STATS

CSV_DEFINE_CATEGORY(Ias, true);
CSV_DEFINE_CATEGORY(Iad, true);

////////////////////////////////////////////////////////////////////////////////
// IAX CSV Counters
CSV_DEFINE_CATEGORY(Iax, true);

CSV_DEFINE_STAT(Iax, HttpGetCount);
CSV_DEFINE_STAT(Iax, HttpGetMiB);
CSV_DEFINE_STAT(Iax, HttpRetryCount);
CSV_DEFINE_STAT(Iax, HttpErrorCount);
CSV_DEFINE_STAT(Iax, HttpDurationAvgMs);

CSV_DEFINE_STAT(Iax, InstallerInstallCount);
CSV_DEFINE_STAT(Iax, InstallerInflightCount);
CSV_DEFINE_STAT(Iax, InstallerDownloadedMiB);
CSV_DEFINE_STAT(Iax, InstallerDurationAvgMs);
CSV_DEFINE_STAT(Iax, InstallerAvgCacheHitPct);

#define DEFINE_CAS_STATS_FOR_CSV(CasType) \
	CSV_DEFINE_STAT(Iax, CasType##_InstallCachePurgeCount) \
	CSV_DEFINE_STAT(Iax, CasType##_InstallCacheDefragCount) \
	CSV_DEFINE_STAT(Iax, CasType##_InstallCacheWriteCount) \
	CSV_DEFINE_STAT(Iax, CasType##_InstallCacheWriteMiB) \
	CSV_DEFINE_STAT(Iax, CasType##_InstallCacheDecodeErrorCount) \
	CSV_DEFINE_STAT(Iax, CasType##_InstallCacheReadMiB) \
	CSV_DEFINE_STAT(Iax, CasType##_InstallCacheReadCount)

ON_DEMAND_INSTALL_CAS_TYPE_LIST(DEFINE_CAS_STATS_FOR_CSV)
#undef DEFINE_CAS_STATS_FOR_CSV

CSV_DEFINE_STAT(Iax, StreamingCacheWriteCount);
CSV_DEFINE_STAT(Iax, StreamingCacheWriteMiB);
CSV_DEFINE_STAT(Iax, StreamingCacheDecodeErrorCount);

CSV_DEFINE_STAT(Iax, DiskWriteRateMBps);
CSV_DEFINE_STAT(Iax, StreamingCacheReadDiskMiB);
CSV_DEFINE_STAT(Iax, StreamingCacheReadMemoryMiB);
CSV_DEFINE_STAT(Iax, StreamingCacheReadMiB);

////////////////////////////////////////////////////////////////////////////////
struct FHttpCounters
{
	FHttpCounters()
		: GetCount(TEXT("Iax/Http/GetCount"), TraceCounterDisplayHint_None)
		, GetBytes(TEXT("Iax/Http/GetBytes"), TraceCounterDisplayHint_Memory)
		, CancelCount(TEXT("Iax/Http/CancelCount"), TraceCounterDisplayHint_None)
		, RetryCount(TEXT("Iax/Http/RetryCount"), TraceCounterDisplayHint_None)
		, ErrorCount(TEXT("Iax/Http/ErrorCount"), TraceCounterDisplayHint_None)
		, DurationTotalMs(TEXT("Iax/Http/DurationTotalMs"), TraceCounterDisplayHint_None)
		, DurationAvgMs(TEXT("Iax/Http/DurationAvgMs"), TraceCounterDisplayHint_None)
	{ }

	FCounterAtomicInt		GetCount;
	FCounterAtomicInt 		GetBytes;
	FCounterAtomicInt 		CancelCount;
	FCounterAtomicInt 		RetryCount;
	FCounterAtomicInt 		ErrorCount;
	FCounterAtomicInt 		DurationTotalMs;
	FCounterAtomicInt 		DurationAvgMs;
};
FHttpCounters HttpCounters;

////////////////////////////////////////////////////////////////////////////////
struct FInstallerCounters
{
	FInstallerCounters()
		: InstallCount(TEXT("Iax/Installer/InstallCount"), TraceCounterDisplayHint_None)
		, InflightCount(TEXT("Iax/Installer/InflightCount"), TraceCounterDisplayHint_None)
		, DownloadedBytes(TEXT("Iax/Installer/DownloadedBytes"), TraceCounterDisplayHint_Memory)
		, DurationAvgMs(TEXT("Iax/Installer/DurationAvgMs"), TraceCounterDisplayHint_None)
		, CacheHitAvgPct(TEXT("Iax/Installer/CacheHitAvgPct."), TraceCounterDisplayHint_None)
	{ }

	FCounterAtomicInt 		InstallCount;
	FCounterAtomicInt 		InflightCount;
	FCounterAtomicInt 		DownloadedBytes;
	std::atomic_uint64_t	DurationTotalMs{0};
	FCounterAtomicInt 		DurationAvgMs;
	FCounterAtomicInt 		CacheHitAvgPct;
	std::atomic_uint64_t	CacheHitTotalPct{0};
};
FInstallerCounters InstallerCounters;

////////////////////////////////////////////////////////////////////////////////
struct FInstallerAnalytics
{
	void Lock()		{ Mutex.Lock(); }
	void Unlock()	{ Mutex.Unlock(); }
	static FMutex Mutex; 

	uint64 InstallCount = 0;
	uint64 InstallErrorCount = 0;
	uint64 InstallCacheMissCount = 0;
	uint64 DownloadedBytes = 0;
	uint64 TotalInstallDurationMs = 0;
	double TotalCacheHitRatio = 0.0;
};
FMutex FInstallerAnalytics::Mutex;
static FInstallerAnalytics InstallerAnalytics;

////////////////////////////////////////////////////////////////////////////////
struct FInstallCacheAnalytics
{
	static FMutex Mutex; 

	// Stats not associated with a particular CAS
	struct FShared
	{
		uint64 ResolveErrorCount = 0;
	};

	static FShared Shared;

	// CAS specific stats
	uint64 VerificationRemovedBlockCount = 0;
	uint64 FlushCount = 0;
	uint64 FlushErrorCount = 0;
	uint64 FlushedBytes = 0;
	uint64 DefragFlushCount = 0;
	uint64 DefragFlushErrorCount = 0;
	uint64 DefragFlushedBytes = 0;
	uint64 PurgedBytes = 0;
	uint64 PurgeCount = 0;
	uint64 PurgeErrorCount = 0;
	uint64 DefragCount = 0;
	uint64 DefragErrorCount = 0;
	uint64 JournalCommitCount = 0;
	uint64 JournalCommitErrorCount = 0;
	uint64 ReadCount = 0;
	uint64 ReadErrorCount = 0;
	uint64 MaxCacheSize = 0;
	uint64 MaxCacheUsageSize = 0;
	uint64 MaxReferencedBlockSize = 0;
	uint64 MaxReferencedSize = 0;
	uint64 MaxFragmentedSize = 0;
	uint32 StartupErrorCode = 0;

	double TotalFragmentionRatio = 0.0;
	double DefraggedBlockFragmentionRatio = 0.0;

	EOnDemandInstallCasType CasType = EOnDemandInstallCasType::None;

	static FInstallCacheAnalytics& Get(EOnDemandInstallCasType CasType)
	{
		check(CasType != EOnDemandInstallCasType::None);

		// Please update this function if you hit this static assert
		static_assert(EOnDemandInstallCasType::Count == 2);

		if constexpr(FPlatformProperties::SupportsMemoryMappedFiles())
		{
			if (CasType == EOnDemandInstallCasType::MMap)
			{
				static FInstallCacheAnalytics MMapCacheAnalytics{ .CasType = EOnDemandInstallCasType::MMap };
				return MMapCacheAnalytics;
			}
		}
		
		static FInstallCacheAnalytics InstallCacheAnalytics{ .CasType = EOnDemandInstallCasType::General };
		return InstallCacheAnalytics;
	}
};
FMutex FInstallCacheAnalytics::Mutex;
FInstallCacheAnalytics::FShared FInstallCacheAnalytics::Shared;

////////////////////////////////////////////////////////////////////////////////
struct FInstallCacheCounters
{
	FInstallCacheCounters(EOnDemandInstallCasType InCasType)
		: MaxCacheSize(TraceCounterNameType_Dynamic,			*FString::Printf(TEXT("Iax/InstallCache/%s/MaxCacheSize"),			LexToString(InCasType)), TraceCounterDisplayHint_Memory)
		, CacheSize(TraceCounterNameType_Dynamic,				*FString::Printf(TEXT("Iax/InstallCache/%s/CacheSize"),				LexToString(InCasType)), TraceCounterDisplayHint_Memory)
		, FlushedBytes(TraceCounterNameType_Dynamic,			*FString::Printf(TEXT("Iax/InstallCache/%s/WriteBytes"),			LexToString(InCasType)), TraceCounterDisplayHint_Memory)
		, FlushCount(TraceCounterNameType_Dynamic,				*FString::Printf(TEXT("Iax/InstallCache/%s/WriteCount"),			LexToString(InCasType)), TraceCounterDisplayHint_None)
		, InMemoryBytes(TraceCounterNameType_Dynamic,			*FString::Printf(TEXT("Iax/InstallCache/%s/InMemoryBytes"),			LexToString(InCasType)), TraceCounterDisplayHint_Memory)
		, InMemoryCount(TraceCounterNameType_Dynamic,			*FString::Printf(TEXT("Iax/InstallCache/%s/InMemoryCount"),			LexToString(InCasType)), TraceCounterDisplayHint_None)
		, ReadBytes(TraceCounterNameType_Dynamic,				*FString::Printf(TEXT("Iax/InstallCache/%s/ReadBytes"),				LexToString(InCasType)), TraceCounterDisplayHint_Memory)
		, ReadCount(TraceCounterNameType_Dynamic,				*FString::Printf(TEXT("Iax/InstallCache/%s/ReadCount"),				LexToString(InCasType)), TraceCounterDisplayHint_None)
		, PurgeCount(TraceCounterNameType_Dynamic,				*FString::Printf(TEXT("Iax/InstallCache/%s/PurgeCount"),			LexToString(InCasType)), TraceCounterDisplayHint_None)
		, DefragCount(TraceCounterNameType_Dynamic,				*FString::Printf(TEXT("Iax/InstallCache/%s/DefragCount"),			LexToString(InCasType)), TraceCounterDisplayHint_None)
		, ReadErrorCount(TraceCounterNameType_Dynamic,			*FString::Printf(TEXT("Iax/InstallCache/%s/ReadErrorCount"),		LexToString(InCasType)), TraceCounterDisplayHint_None)
		, ReadDecodeErrorCount(TraceCounterNameType_Dynamic,	*FString::Printf(TEXT("Iax/InstallCache/%s/ReadDecodeErrorCount"),	LexToString(InCasType)), TraceCounterDisplayHint_None)
		, CasType(InCasType)
	{ }

	FCounterAtomicInt MaxCacheSize;
	FCounterAtomicInt CacheSize;
	FCounterAtomicInt FlushedBytes;
	FCounterAtomicInt FlushCount;
	FCounterAtomicInt InMemoryBytes;
	FCounterAtomicInt InMemoryCount;
	FCounterAtomicInt ReadBytes;
	FCounterAtomicInt ReadCount;
	FCounterAtomicInt PurgeCount;
	FCounterAtomicInt DefragCount;
	FCounterAtomicInt ReadErrorCount;
	FCounterAtomicInt ReadDecodeErrorCount;

	EOnDemandInstallCasType CasType = EOnDemandInstallCasType::None;

	// Stats not associated with a particular CAS
	struct FShared
	{
		FCounterAtomicInt ResolveErrorCount{ TEXT("Iax/InstallCache/Shared/ResolveErrorCount"), TraceCounterDisplayHint_None };
	};

	static FShared Shared;

	static FInstallCacheCounters& Get(EOnDemandInstallCasType CasType)
	{
		check(CasType != EOnDemandInstallCasType::None);

		// Please update this function if you hit this static assert
		static_assert(EOnDemandInstallCasType::Count == 2);

		if constexpr (FPlatformProperties::SupportsMemoryMappedFiles())
		{
			if (CasType == EOnDemandInstallCasType::MMap)
			{
				static FInstallCacheCounters MMapCacheCounters{ EOnDemandInstallCasType::MMap };
				return MMapCacheCounters;
			}
		}

		static FInstallCacheCounters InstallCacheCounters{ EOnDemandInstallCasType::General };
		return InstallCacheCounters;
	}
};
FInstallCacheCounters::FShared FInstallCacheCounters::Shared;

////////////////////////////////////////////////////////////////////////////////
static FOnDemandIoBackendStats* GStatistics = nullptr;

FOnDemandIoBackendStats::FOnDemandIoBackendStats(FBackendStatus& InStatus)
	: BackendStatus(InStatus)
{
	static constexpr float OneOver1024 = 1.0f / 1024.0f;

	check(GStatistics == nullptr);
	GStatistics = this;

	GAnalyticsStartTiming = FPlatformTime::Seconds();

	OnApplicationResumeHandle = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FOnDemandIoBackendStats::OnApplicationResume);

	EndFrameDelegateHandle = FCoreDelegates::OnEndFrame.AddLambda([this]()
	{
		GCacheWriteRate.Set(GWriteRate.GetMegabytesPerSecond());
		UpdateCSVValues();
		PrintPeriodicLogging();
	});

#if UE_ENABLE_ONSCREEN_STATISTICS
	OnScreenDelegateHandle = FCoreDelegates::OnGetOnScreenMessages.AddLambda(
		[this] (FCoreDelegates::FSeverityMessageMap& OutMessages)
		{
			if (!GIasDisplayOnScreenStatistics)
			{
				return;
			}

			PrintOnScreenStatistics(OutMessages);
		});

	ResetStatisticsCommand = FConsoleCommandPtr(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("iax.ResetOnScreenStatistics"),
		TEXT("Resets the values shown by 'DisplayOnScreenStatistics'"),
		FConsoleCommandDelegate::CreateLambda([this]() -> void
			{
				ResetOnScreenStatistics();
			}),
		ECVF_Default));
#endif // UE_ENABLE_ONSCREEN_STATISTICS
#if CSV_PROFILER_STATS
	CSVProfileEndRequestedHandle = FCsvProfiler::Get()->OnCSVProfileEndRequested().AddRaw(this, &FOnDemandIoBackendStats::OnCSVProfileEndRequested);
#endif
}

FOnDemandIoBackendStats::~FOnDemandIoBackendStats()
{
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(OnApplicationResumeHandle);
	FCoreDelegates::OnEndFrame.Remove(EndFrameDelegateHandle);
	FCoreDelegates::OnGetOnScreenMessages.Remove(OnScreenDelegateHandle);
#if CSV_PROFILER_STATS
	FCsvProfiler::Get()->OnCSVProfileEndRequested().Remove(CSVProfileEndRequestedHandle);
#endif

	GStatistics = nullptr;
}

FOnDemandIoBackendStats* FOnDemandIoBackendStats::Get()
{
	return GStatistics;
}

#if IAS_WITH_STATISTICS

void FOnDemandIoBackendStats::UpdateCSVValues()
{
	static constexpr float OneOver1024 = 1.0f / 1024.0f;

	CSV_CUSTOM_STAT_DEFINED(DiskWriteRateMBps, GCacheWriteRate.Get(), ECsvCustomStatOp::Set);

	// Streaming cache stats
	{
		CSV_CUSTOM_STAT_DEFINED(StreamingCacheWriteCount, int32(GCacheWriteCount.Get()), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(StreamingCacheWriteMiB, BytesToApproxMB(GCacheWrittenBytes.Get()), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(StreamingCacheDecodeErrorCount, int32(GCacheDecodeErrorCount.Get()), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(StreamingCacheReadMiB, BytesToApproxMB(GCacheReadBytes.Get()), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(StreamingCacheReadMemoryMiB, BytesToApproxMB(GCacheReadMemory.Get()), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(StreamingCacheReadDiskMiB, BytesToApproxMB(GCacheReadDisk.Get()), ECsvCustomStatOp::Set);
	}

	// Install cache stats
	{
		#define DEFINE_CUSTOM_CAS_STATS_FOR_CSV(InCasType) \
			{ \
				FInstallCacheCounters& InstallCacheCounters = FInstallCacheCounters::Get(EOnDemandInstallCasType::InCasType); \
				if (InstallCacheCounters.CasType == EOnDemandInstallCasType::InCasType) /* If a CAS type is not enabled, if will fall back to EOnDemandInstallCasType::General*/ \
				{ \
					CSV_CUSTOM_STAT_DEFINED(InCasType##_InstallCachePurgeCount, int32(InstallCacheCounters.PurgeCount.Get()), ECsvCustomStatOp::Set) \
					CSV_CUSTOM_STAT_DEFINED(InCasType##_InstallCacheDefragCount, int32(InstallCacheCounters.DefragCount.Get()), ECsvCustomStatOp::Set) \
					CSV_CUSTOM_STAT_DEFINED(InCasType##_InstallCacheWriteCount, int32(InstallCacheCounters.FlushCount.Get()), ECsvCustomStatOp::Set) \
					CSV_CUSTOM_STAT_DEFINED(InCasType##_InstallCacheWriteMiB, BytesToApproxMB(InstallCacheCounters.FlushedBytes.Get()), ECsvCustomStatOp::Set) \
					CSV_CUSTOM_STAT_DEFINED(InCasType##_InstallCacheDecodeErrorCount, int32(InstallCacheCounters.ReadDecodeErrorCount.Get()), ECsvCustomStatOp::Set) \
					CSV_CUSTOM_STAT_DEFINED(InCasType##_InstallCacheReadMiB, BytesToApproxMB(InstallCacheCounters.ReadBytes.Get()), ECsvCustomStatOp::Set) \
					CSV_CUSTOM_STAT_DEFINED(InCasType##_InstallCacheReadCount, int32(InstallCacheCounters.ReadCount.Get()), ECsvCustomStatOp::Set) \
				} \
			}

		ON_DEMAND_INSTALL_CAS_TYPE_LIST(DEFINE_CUSTOM_CAS_STATS_FOR_CSV)
		#undef DEFINE_CUSTOM_CAS_STATS_FOR_CSV
	}

	// Installer stats
	{
		const int64 InstallCount		= InstallerCounters.InstallCount.Get();
		const int64 InflightCount		= InstallerCounters.InflightCount.Get();
		const int64 DownloadedBytes		= InstallerCounters.DownloadedBytes.Get();
		const uint64 DurationTotalMs	= InstallerCounters.DurationTotalMs;
		const double DurationAvgMs		= InstallCount > 0 ? double(DurationTotalMs) / double(InstallCount) : 0;
		const int64 CacheHitTotalPct 	= InstallerCounters.CacheHitTotalPct;
		const double CacheHitAvgPct		= InstallCount > 0 ? double(CacheHitTotalPct) / double(InstallCount) : 0;
		CSV_CUSTOM_STAT_DEFINED(InstallerInstallCount, int32(InstallCount), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(InstallerInflightCount, int32(InflightCount), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(InstallerDownloadedMiB, BytesToApproxMB(DownloadedBytes), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(InstallerDurationAvgMs, int32(FMath::RoundToInt64(DurationAvgMs)), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(InstallerAvgCacheHitPct, int32(FMath::RoundToInt64(CacheHitAvgPct)), ECsvCustomStatOp::Set);
	}

	// HTTP stats
	{
		CSV_CUSTOM_STAT_DEFINED(HttpGetCount, int32(HttpCounters.GetCount.Get()), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(HttpGetMiB, BytesToApproxMB(HttpCounters.GetBytes.Get()), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(HttpRetryCount, int32(HttpCounters.RetryCount.Get()), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(HttpErrorCount, int32(HttpCounters.ErrorCount.Get()), ECsvCustomStatOp::Set);
		CSV_CUSTOM_STAT_DEFINED(HttpDurationAvgMs, int32(HttpCounters.DurationAvgMs.Get()), ECsvCustomStatOp::Set);
	}
}

void FOnDemandIoBackendStats::PrintPeriodicLogging()
{
	if (GIasStatisticsLogInterval <= 0.f)
	{
		return;
	}

	static double LastLogTime = 0.0;
	if (double Time = FPlatformTime::Seconds(); Time - LastLogTime > (double)GIasStatisticsLogInterval)
	{
		UE_LOGF(LogIoStoreOnDemand, Log, "Stats:");
		UE_LOGF(LogIoStoreOnDemand, Log, "===============");

		static constexpr float OneOver1024 = 1.0f / 1024.0f;

		int32 CGetCount = int32(GCacheGetCount.Get());
		int32 CErrorCount = int32(GCacheErrorCount.Get());
		int32 CPutCount = int32(GCachePutCount.Get());
		int32 CPutExistingCount = int32(GCachePutExistingCount.Get());
		int32 CPutRejectCount = int32(GCachePutRejectCount.Get());

		float CCachedKiB = (float)GCacheCachedBytes.Get() * OneOver1024;
		float CWrittenKiB = (float)GCacheWrittenBytes.Get() * OneOver1024;
		float CEvictedKiB = (float)GCacheEvictedBytes.Get() * OneOver1024;
		float CReadKiB = (float)GCacheReadBytes.Get() * OneOver1024;
		float CRejectedKiB = (float)GCacheRejectBytes.Get() * OneOver1024;

		if (BackendStatus.IsCacheEnabled())
		{
			UE_LOGF(LogIoStoreOnDemand, Log, "StreamingCache: Count=%d, CachedKiB=%d, WrittenKiB=%d, ReadKiB=%d, EvictedKiB=%d, RejectedKiB=%d, Get=%d, Error=%d, Put=%d, PutReject=%d, PutExisting=%d",
				(int32)GCacheCachedEntries.Get(), (int32)CCachedKiB, (int32)CWrittenKiB, (int32)CReadKiB,
				(int32)CEvictedKiB, (int32)CRejectedKiB, CGetCount, CErrorCount, CPutCount, CPutRejectCount, CPutExistingCount);
		}
		else
		{
			UE_LOGF(LogIoStoreOnDemand, Log, "StreamingCache: Disabled");
		}

		auto HttpLog = [](const TCHAR* Title, EHttpRequestType Type)
			{
				UE_LOGF(LogIoStoreOnDemand, Log, "Http: Category: %ls, DownloadedKiB=%d, Get=%d, Retry=%d, Cancel=%d, Error=%d, CurPending=%d, CurDurationMsAvg=%d, CurDurationMsMax=%d",
					Title,
					(int32)((float)GHttpDownloadedBytes[(uint8)Type].Get() * OneOver1024),
					(int32)GHttpGetCount[(uint8)Type].Get(),
					(int32)GHttpRetryCount[(uint8)Type].Get(),
					(int32)GHttpCancelCount[(uint8)Type].Get(),
					(int32)GHttpErrorCount[(uint8)Type].Get(),
					(int32)GHttpPendingCount[(uint8)Type].Get(),
					(int32)GHttpDurationMsAvg[(uint8)Type],
					GHttpDurationMsMax[(uint8)Type]);
			};

		HttpLog(TEXT("Streaming"), EHttpRequestType::Streaming);
		HttpLog(TEXT("Install"), EHttpRequestType::Installed);

		{
			const int64 InstallCount		= InstallerCounters.InstallCount.Get();
			const uint64 DurationTotalMs	= InstallerCounters.DurationTotalMs;
			const double DurationAvgMs		= InstallCount > 0 ? double(DurationTotalMs) / double(InstallCount) : 0;
			const int64 CacheHitTotalPct 	= InstallerCounters.CacheHitTotalPct;
			const double CacheHitAvgPct		= InstallCount > 0 ? double(CacheHitTotalPct) / double(InstallCount) : 0;

			UE_LOGF(LogIoStoreOnDemand, Log, "Installer: InstallCount=%lld, AvgDuration: %d, AvgCacheHitPct=%.2lf",
				InstallCount, int32(DurationAvgMs), CacheHitAvgPct);
		}

		for (EOnDemandInstallCasType CasType : TEnumRange<EOnDemandInstallCasType>())
		{
			FInstallCacheCounters& InstallCacheCounters = FInstallCacheCounters::Get(CasType);
			if (InstallCacheCounters.CasType == CasType) // If a CAS type is not enabled, if will fall back to EOnDemandInstallCasType::General
			{
				const int64 WriteCount = InstallCacheCounters.FlushCount.Get();
				const int64 WriteBytes = InstallCacheCounters.FlushedBytes.Get();
				UE_LOGF(LogIoStoreOnDemand, Log, "%ls InstallCache: WriteCount=%lld, WriteBytes=%.2lf MiB",
					LexToString(CasType), WriteCount, double(WriteBytes) / 1024.0 / 1024.0);
			}
		}

		LastLogTime = Time;
	}
}

void FOnDemandIoBackendStats::ResetOnScreenStatistics()
{
#if UE_ENABLE_ONSCREEN_STATISTICS

#define UE_IAX_RESET_COUNTER(Counter) {for(auto& Value:Counter){Value.Set(0);}}	
#define UE_IAX_RESET_VALUE(Counter) {for(auto& Value:Counter){Value = 0;}}

	UE_IAX_RESET_COUNTER(GHttpDownloadedBytes);
	UE_IAX_RESET_COUNTER(GHttpGetCount);
	UE_IAX_RESET_VALUE(GHttpDurationMsAvg);
	UE_IAX_RESET_COUNTER(GHttpRetryCount);

	GHttpCdnCacheHit = 0;
	GHttpCdnCacheMiss = 0;
	GHttpCdnCacheUnknown = 0;

	GCacheRejectBytes.Set(0);
	GCacheReadBytes.Set(0);
	GCacheWrittenBytes.Set(0);
	GCacheGetCount.Set(0);
	GCacheEvictedBytes.Set(0);

	GCacheDecodeErrorCount.Set(0);

	UE_IAX_RESET_COUNTER(GHttpDecodeErrorCount);
	UE_IAX_RESET_COUNTER(GHttpErrorCount);

	bValuesValidForAnalytics = false;

#undef UE_IAX_RESET_COUNTER

#endif // UE_ENABLE_ONSCREEN_STATISTICS
}

void FOnDemandIoBackendStats::PrintOnScreenStatistics(FCoreDelegates::FSeverityMessageMap& OutMessages)
{
#if UE_ENABLE_ONSCREEN_STATISTICS

	// Print the backend status
	{
		TStringBuilder<256> BackendStatusText;
		BackendStatusText << TEXT("IAS Backend Status: ");
		BackendStatus.ToString(BackendStatusText);

		{
			BackendStatusText.Append(" | HttpMaxRange: ");

			const int32 MaxRange = GHttpIoDispatcherMaxRangeKiB;
			if (MaxRange < 0)
			{
				BackendStatusText.Append("Unlimited");
			}
			else if (MaxRange == 0)
			{
				BackendStatusText.Append("Single");
			}
			else
			{
				BackendStatusText.Appendf(TEXT("%d KiB"), MaxRange);
			}
			
			BackendStatusText.Append(" | DevMode: ");
			BackendStatusText.Append(IsDevModeEnabled() ? TEXT("Enabled") : TEXT("Disabled"));
		}

		OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromStringView(BackendStatusText.ToView()));
	}

	// Print HostGroup info
	FHostGroupManager::Get().ForEachHostGroup([&OutMessages](const FIASHostGroup& HostGroup)
		{
			FCoreDelegates::EOnScreenMessageSeverity Verbosity = FCoreDelegates::EOnScreenMessageSeverity::Info;
			TStringBuilder<256> Text;
			Text << TEXT("HostGroup [") << HostGroup.GetName() << TEXT("] ");

			if (HostGroup.IsConnected())
			{
				Text << HostGroup.GetPrimaryHostUrl();
				Text << TEXT(" (") << HostGroup.GetPrimaryHostIndex() << TEXT("/") << HostGroup.GetHostUrls().Num() << TEXT(")");
			}
			else if (HostGroup.IsResolved())
			{
				Text << TEXT("Resolving...");
			}
			else
			{
				Text << TEXT("Disconnected");
				Verbosity = FCoreDelegates::EOnScreenMessageSeverity::Error;
			}

			OutMessages.Add(Verbosity, FText::FromStringView(Text.ToView()));
		});

	// Print Http Stats
	{
		auto HttpStats = [&OutMessages](const TCHAR* Title, EHttpRequestType Type)
			{
				TStringBuilder<256> Builder;
				Builder.Appendf(
					TEXT("%s Http: Downloaded: %s (%d) Avg %d ms | Retries: %d | Pending: %d"),
					Title,
					*FText::AsMemory(GHttpDownloadedBytes[(uint8)Type].Get()).ToString(),
					GHttpGetCount[(uint8)Type].Get(),
					(int32)GHttpDurationMsAvg[(uint8)Type],
					GHttpRetryCount[(uint8)Type].Get(),
					GHttpPendingCount[(uint8)Type].Get()
				);

				uint32 NumPaused = GHttpNumPausedRequests[(uint8)Type];
				if (NumPaused > 0)
				{
					Builder.Appendf(TEXT(" | Paused: %d"), NumPaused);
				}

				OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromStringView(Builder.ToView()));
			};

		{
			static uint32 RateKiBps = 0;
			static uint32 DownKiB = 0;
			static uint32 TtfbMs = 0;
			if (IOnDemandIoStore* Store = TryGetOnDemandIoStore(); Store != nullptr)
			{
				double Time = FPlatformTime::Seconds();
				if (static double LastSampleTime = Time; Time - LastSampleTime > 0.5)
				{
					LastSampleTime -= 0.5;
					if (LastSampleTime > 0.5)
					{
						LastSampleTime = Time;
					}

					static FOnDemandHttpStats Stats;
					Store->GetHttpStats(Stats);
					RateKiBps = Stats.GetRecvKiBps();
					DownKiB = Stats.GetTotalRecvKiB();
					TtfbMs = Stats.GetTimeToFirstByteMs();
				}
			}

			TStringBuilder<32> RateLimit;
			const int32 RateLimitKiBPerSecond = GetHttpRateLimitKiBPerSecond();
			if (RateLimitKiBPerSecond > 0)
			{
				RateLimit << RateLimitKiBPerSecond;
			}
			else
			{
				RateLimit << TEXT("\u221E"); // Infinity symbol code
			}

			TStringBuilder<256> Builder;
			Builder << TEXT("IAX Http:");
			Builder << TEXT(" - Downloaded: ") << FText::AsMemory(uint64(DownKiB) << 10).ToString();
			Builder << TEXT(" - Rate/limit: ") << RateKiBps << TEXT(" / ") << RateLimit << TEXT(" KiB/s");
			Builder << TEXT(" - TTFB: ") << TtfbMs << TEXT("ms");

			const int32 NumConcurrentRequests = GHttpNumConcurrentRequests;
			const int32 MaxConcurrentRequests = CalculateRequestQueueLengthMax();
			Builder << TEXT(" - Queue: ") << NumConcurrentRequests << TEXT("/") << MaxConcurrentRequests;
			if (NumConcurrentRequests > MaxConcurrentRequests)
			{
				Builder << TEXT("(saturated)");
			}

			OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromStringView(Builder.ToView()));
		}

		HttpStats(TEXT("IAS"), EHttpRequestType::Streaming);
		HttpStats(TEXT("IAD"), EHttpRequestType::Installed);
	}

	// Print IAX CDN Cache rates
#if UE_TRACK_CDN_HIT_STATUS
	{
		TStringBuilder<256> Builder;
		Builder << TEXT("IAX CDN: Hit/Miss/NoHdr: ");

		Builder << GHttpCdnCacheHit;
		Builder << TEXT("/") << GHttpCdnCacheMiss;
		Builder << TEXT("/") << GHttpCdnCacheUnknown;

		if (uint32 Total = GHttpCdnCacheHit + GHttpCdnCacheMiss + GHttpCdnCacheUnknown; Total)
		{
			auto AsPercent = [Total](uint32 Value) { return (Value * 100 + (Total >> 1)) / Total; };
			Builder << TEXT(" - ") << AsPercent(GHttpCdnCacheHit);
			Builder << TEXT("%/") << AsPercent(GHttpCdnCacheMiss);
			Builder << TEXT("%/") << AsPercent(GHttpCdnCacheUnknown);
			Builder << TEXT("%");
		}

		OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromStringView(Builder.ToView()));
	}
#endif //#if UE_TRACK_CDN_HIT_STATUS

	// Print Streaming Cache
	{
		TStringBuilder<256> CachingText;
		CachingText << TEXT("Streaming Cache: ");
		if (BackendStatus.IsCacheEnabled())
		{
			const float CacheUsagePercent = GCacheMaxBytes > 0 ? 100.f * (float(GCacheCachedBytes.Get()) / float(GCacheMaxBytes)) : 0.f;

			CachingText.Appendf(TEXT(" (%.1f%%) "), CacheUsagePercent);
			CachingText << FText::AsMemory(GCacheCachedBytes.Get()).ToString();
			CachingText << TEXT(" - (") << GCacheGetCount.Get() << TEXT(")");

			CachingText << TEXT(" | Write: ") << FText::AsMemory(GCacheWrittenBytes.Get()).ToString();
			CachingText << TEXT(" | Read: ") << FText::AsMemory(GCacheReadBytes.Get()).ToString();
			CachingText << TEXT(" | Evict: ") << FText::AsMemory(GCacheEvictedBytes.Get()).ToString();
			if (GCacheRejectBytes.Get() > 0)
			{
				CachingText << TEXT(" | Rejected: ") << FText::AsMemory(GCacheRejectBytes.Get()).ToString();
			}

			CachingText << TEXT(" (Boot: ") << GCacheBootMs << TEXT("ms)");

			const double SuspendedSeconds = GCacheSuspendedSeconds.Get();
			if (SuspendedSeconds > 0.0)
			{
				CachingText.Appendf(TEXT(" (Suspended: %.2lfs)"), SuspendedSeconds);
			}
		}
		else
		{
			CachingText << TEXTVIEW("Streaming Cache: Disabled");
		}

		OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromStringView(CachingText.ToView()));
	}

	// Print Streaming Cache errors
	if (GHttpDecodeErrorCount[(uint8)EHttpRequestType::Streaming].Get() > 0 || GCacheDecodeErrorCount.Get() > 0 || GHttpErrorCount[(uint8)EHttpRequestType::Streaming].Get() > 0)
	{
		TStringBuilder<256> Builder;
		Builder.Appendf(TEXT("Streaming Cache Errors: Cache Decode: %d | Http Decode: %d | Http: %d"),
			GCacheDecodeErrorCount.Get(),
			GHttpDecodeErrorCount[(uint8)EHttpRequestType::Streaming].Get(),
			GHttpErrorCount[(uint8)EHttpRequestType::Streaming].Get()
		);

		OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Error, FText::FromStringView(Builder.ToView()));
	}

	// Print Install Cache
	for (EOnDemandInstallCasType CasType : TEnumRange<EOnDemandInstallCasType>())
	{
		FInstallCacheCounters& InstallCacheCounters = FInstallCacheCounters::Get(CasType);
		if (InstallCacheCounters.CasType == CasType) // If a CAS type is not enabled, if will fall back to EOnDemandInstallCasType::General
		{
			TStringBuilder<256> Sb;
			Sb << LexToString(CasType);
			Sb << TEXT(" Install Cache: ");
			Sb << TEXT(" Memory: ") << FText::AsMemory(InstallCacheCounters.InMemoryBytes.Get()).ToString();
			Sb << TEXT(" (") << InstallCacheCounters.InMemoryCount.Get() << TEXT(")");
			Sb << TEXT(" | Write: ") << FText::AsMemory(InstallCacheCounters.FlushedBytes.Get()).ToString();
			Sb << TEXT(" (") << InstallCacheCounters.FlushCount.Get() << TEXT(")");
			Sb << TEXT(" | Read: ") << FText::AsMemory(InstallCacheCounters.ReadBytes.Get()).ToString();
			OutMessages.Add(FCoreDelegates::EOnScreenMessageSeverity::Info, FText::FromStringView(Sb.ToView()));
		}
	}

#endif // UE_ENABLE_ONSCREEN_STATISTICS
}

#endif // IAS_WITH_STATISTICS

void FOnDemandIoBackendStats::ReportGeneralAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const
{
	const double CurrentTime = FPlatformTime::Seconds();
	const int64 TimeSpent = FMath::RoundToInt64(CurrentTime - GAnalyticsStartTiming);
	GAnalyticsStartTiming = CurrentTime;

	AppendAnalyticsEventAttributeArray(OutAnalyticsArray
		, TEXT("IaxTimeSpent"), TimeSpent
		, TEXT("IasAppResumeCount"), GAppResumeCount
		, TEXT("IasHttpHasEverConnected"), GHttpConnectCount.Get() > 0 // Report if the system has ever actually managed to make a connection
	);

	GAppResumeCount = 0;
}

void FOnDemandIoBackendStats::ReportEndPointAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const
{
	if (GIasReportHttpAnalyticsEnabled && bValuesValidForAnalytics)
	{
		auto ReportHttpStats = [&OutAnalyticsArray](const TCHAR* Prefix, EHttpRequestType Type)
		{
#define UE_PREFIX(Name) WriteToString<128>(Prefix, Name)
#define UE_TRACK_DELTA(Name, Value) *Name, GDeltaTracking.Get(Name, Value)

			const int64 ByteCount = GDeltaTracking.Get(UE_PREFIX(TEXT("HttpDownloadedBytes")), GHttpDownloadedBytes[(uint8)Type].Get());
			const int64 GetCount = GDeltaTracking.Get(UE_PREFIX(TEXT("HttpGetCount")), GHttpGetCount[(uint8)Type].Get());

			const double DataRateBPS = GHttpDurationMsSum[(uint8)Type] > 0 ? static_cast<double>(ByteCount) / (static_cast<double>(GHttpDurationMsSum[(uint8)Type]) / 1000.0) : 0.0;
			const double DurationMean = GetCount ? static_cast<double>(GHttpDurationMsSum[(uint8)Type]) / static_cast<double>(GetCount) : 0.0;

			AppendAnalyticsEventAttributeArray(OutAnalyticsArray
				, UE_TRACK_DELTA(UE_PREFIX("HttpErrorCount"), GHttpErrorCount[(uint8)Type].Get())
				, UE_TRACK_DELTA(UE_PREFIX("HttpDecodeErrors"), GHttpDecodeErrorCount[(uint8)Type].Get())
				, UE_TRACK_DELTA(UE_PREFIX("HttpRetryCount"), GHttpRetryCount[(uint8)Type].Get())
				, *UE_PREFIX(TEXT("HttpGetCount")), GetCount
				, *UE_PREFIX(TEXT("HttpDownloadedBytes")), ByteCount
				, *UE_PREFIX(TEXT("HttpDurationMean")), DurationMean
				, *UE_PREFIX(TEXT("HttpDurationSum")), GHttpDurationMsSum[(uint8)Type]
				, *UE_PREFIX(TEXT("HttpDataRateMean")), DataRateBPS
			);

			// These values we can just reset as they are only being used with analytics
			GHttpDurationMsSum[(uint8)Type] = 0;
#undef UE_TRACK_DELTA
#undef UE_PREFIX
		};

		uint32 TtfbMs = 0;
		if (IOnDemandIoStore* Store = TryGetOnDemandIoStore(); Store != nullptr)
		{
			FOnDemandHttpStats Stats;
			Store->GetHttpStats(Stats);

			TtfbMs = Stats.GetTimeToFirstByteMs();
		}

		AppendAnalyticsEventAttributeArray(OutAnalyticsArray, TEXT("IaxTtfbMs"), TtfbMs);
		ReportHttpStats(TEXT("Ias"), EHttpRequestType::Streaming);
		ReportHttpStats(TEXT("Iad"), EHttpRequestType::Installed);
	}

	if (GIasReportCacheAnalyticsEnabled)
	{
#define UE_TRACK_DELTA(Name, Value) TEXT(Name), GDeltaTracking.Get(TEXTVIEW(Name), Value)
		const float CacheUsagePercent = GCacheMaxBytes > 0 ? 100.f * (float(GCacheCachedBytes.Get()) / float(GCacheMaxBytes)) : 0.f;

		AppendAnalyticsEventAttributeArray(OutAnalyticsArray

			, TEXT("IasCacheEnabled"), BackendStatus.IsCacheEnabled()

			, UE_TRACK_DELTA("IasCacheErrorCount", GCacheErrorCount.Get())
			, UE_TRACK_DELTA("IasCacheDecodeErrors", GCacheDecodeErrorCount.Get())
			, UE_TRACK_DELTA("IasCacheGetCount", GCacheGetCount.Get())
			, UE_TRACK_DELTA("IasCachePutCount", GCachePutCount.Get())

			, TEXT("IasCacheCachedBytes"), GCacheCachedBytes.Get()
			, TEXT("IasCacheCachedEntries"), GCacheCachedEntries.Get()
			, TEXT("IasCacheMaxBytes"), GCacheMaxBytes
			, TEXT("IasCacheUsagePercent"), CacheUsagePercent

			, UE_TRACK_DELTA("IasCacheWriteBytes", GCacheWrittenBytes.Get())
			, UE_TRACK_DELTA("IasCacheEvictedBytes", GCacheEvictedBytes.Get())
			, UE_TRACK_DELTA("IasCacheReadBytes", GCacheReadBytes.Get())
			, UE_TRACK_DELTA("IasCacheReadMemoryBytes", GCacheReadMemory.Get())
			, UE_TRACK_DELTA("IasCacheReadDiskBytes", GCacheReadDisk.Get())
			, UE_TRACK_DELTA("IasCacheRejectBytes", GCacheRejectBytes.Get())
		);
#undef UE_TRACK_DELTA
	}
}

#if UE_ENABLE_ANALYTICS_RECORDING

class FAnalyticsRecording : public IAnalyticsRecording
{
public:
	FAnalyticsRecording() = delete;

	FAnalyticsRecording(const FBackendStatus& InBackendStatus)
		: BackendStatus(InBackendStatus)
	{

	}

	virtual ~FAnalyticsRecording() = default;

private:

	void StopRecording() override
	{
		if (!bRecording)
		{
			return;
		}
	
		Http.ErrorCount.Stop();
		Http.DecodeErrorCount.Stop();
		Http.RetryCount.Stop();
		Http.GetCount.Stop();
		Http.DownloadedBytes.Stop();
		Http.TotalDuration.Stop();
		
		Cache.ErrorCount.Stop();
		Cache.DecodeErrorCount.Stop();
		Cache.GetCount.Stop();
		Cache.PutCount.Stop();

		Cache.WrittenBytes.Stop();
		Cache.ReadBytes.Stop();
		Cache.RejectBytes.Stop();

		bRecording = false;
	}

	void Report(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const override
	{
		// Report if the system has ever actually managed to make a connection
		AppendAnalyticsEventAttributeArray(OutAnalyticsArray
			,TEXT("IasHttpHasEverConnected"), GHttpConnectCount.Get() > 0
		);

		if (GIasReportHttpAnalyticsEnabled)
		{
			const double DataRateBPS = Http.TotalDuration.GetValue() > 0 ? static_cast<double>(Http.DownloadedBytes.GetValue()) / (static_cast<double>(Http.TotalDuration.GetValue()) / 1000.0) : 0.0;
			const double DurationMean = Http.GetCount.GetValue() ? static_cast<double>(Http.TotalDuration.GetValue()) / static_cast<double>(Http.GetCount.GetValue()) : 0.0;

			AppendAnalyticsEventAttributeArray(OutAnalyticsArray
				, TEXT("IasHttpErrorCount"), Http.ErrorCount.GetValue()
				, TEXT("IasHttpDecodeErrorCount"), Http.DecodeErrorCount.GetValue()
				, TEXT("IasHttpRetryCount"), Http.RetryCount.GetValue()
				, TEXT("IasHttpGetCount"), Http.GetCount.GetValue()
				, TEXT("IasHttpDownloadedBytes"), Http.DownloadedBytes.GetValue()
				, TEXT("IasHttpDurationMean"), DurationMean
				, TEXT("IasHttpDurationSum"), Http.TotalDuration.GetValue()
				, TEXT("IasHttpDataRateMean"), DataRateBPS
			);
		}

		if (GIasReportCacheAnalyticsEnabled)
		{
			const float CacheUsagePercent = GCacheMaxBytes > 0 ? 100.f * (float(GCacheCachedBytes.Get()) / float(GCacheMaxBytes)) : 0.f;

			AppendAnalyticsEventAttributeArray(OutAnalyticsArray

				, TEXT("IasCacheEnabled"), this->BackendStatus.IsCacheEnabled()
				, TEXT("IasCacheMaxBytes"), GCacheMaxBytes
				, TEXT("IasCacheCachedBytes"), GCacheCachedBytes.Get()
				, TEXT("IasCacheUsagePercent"), CacheUsagePercent

				, TEXT("IasCacheErrorCount"), Cache.ErrorCount.GetValue()
				, TEXT("IasCacheDecodeErrorCount"), Cache.DecodeErrorCount.GetValue()
				, TEXT("IasCacheGetCount"), Cache.GetCount.GetValue()
				, TEXT("IasCachePutCount"), Cache.PutCount.GetValue()

				, TEXT("IasCacheWriteBytes"), Cache.WrittenBytes.GetValue()
				, TEXT("IasCacheEvictedBytes"), Cache.EvictedBytes.GetValue()
				, TEXT("IasCacheReadBytes"), Cache.ReadBytes.GetValue()
				, TEXT("IasCacheReadMemoryBytes"), Cache.ReadMemoryBytes.GetValue()
				, TEXT("IasCacheReadDiskBytes"), Cache.ReadDiskBytes.GetValue()
				, TEXT("IasCacheRejectBytes"), Cache.RejectBytes.GetValue()
			);
		}
	}

private:

	/** This wrapper class allows us to treat raw integer types and FCounter types  as the same thing */
	//template<typename CounterType, CounterType& Counter, int32 INDEX = -1>
	template<typename CounterType, const CounterType& Counter, int32 INDEX = -1>
	struct TTrackedValue
	{
		TTrackedValue()
		{
			Value = GetCurrentValue();
		}

		void Stop()
		{
			Value = GetCurrentValue() - Value;
			bRecording = false;
		}

		int64 GetValue() const
		{
			if (bRecording)
			{
				return GetCurrentValue() - Value;
			}
			else
			{
				return Value;
			}
		}

	private:
		int64 GetCurrentValue() const
		{
			if constexpr (std::is_integral_v<CounterType>)
			{
				return Counter;
			}
			else
			{
				return Counter.Get();
			}
		}

		int64 Value;
		bool bRecording = true;
	};

	/** Http Stats */
	struct FHttpStats
	{
		TTrackedValue<FCounterInt, GHttpErrorCount[0]>				ErrorCount;
		TTrackedValue<FCounterAtomicInt, GHttpDecodeErrorCount[0]>	DecodeErrorCount;
		TTrackedValue<FCounterInt, GHttpRetryCount[0]>				RetryCount;
		TTrackedValue<FCounterInt, GHttpGetCount[0]>				GetCount;
		TTrackedValue<FCounterInt, GHttpDownloadedBytes[0]>			DownloadedBytes;
		TTrackedValue<int64, GHttpDurationMsSum[0]>					TotalDuration;
		
	} Http;

	/** Cache Stats */
	struct FCacheStats
	{
		TTrackedValue<FCounterAtomicInt, GCacheErrorCount>			ErrorCount;
		TTrackedValue<FCounterAtomicInt, GCacheDecodeErrorCount>	DecodeErrorCount;
		TTrackedValue<FCounterAtomicInt, GCacheGetCount>			GetCount;
		TTrackedValue<FCounterAtomicInt, GCachePutCount>			PutCount;

		TTrackedValue<FCounterAtomicInt, GCacheWrittenBytes>		WrittenBytes;
		TTrackedValue<FCounterAtomicInt, GCacheEvictedBytes>		EvictedBytes;
		TTrackedValue<FCounterAtomicInt, GCacheReadBytes>			ReadBytes;
		TTrackedValue<FCounterAtomicInt, GCacheReadMemory>			ReadMemoryBytes;
		TTrackedValue<FCounterAtomicInt, GCacheReadDisk>			ReadDiskBytes;
		TTrackedValue<FCounterAtomicInt, GCacheRejectBytes>			RejectBytes;
	} Cache;

	const FBackendStatus& BackendStatus;
	bool bRecording = true;
};

#endif //UE_ENABLE_ANALYTICS_RECORDING

TUniquePtr<IAnalyticsRecording> FOnDemandIoBackendStats::StartAnalyticsRecording() const
{
#if UE_ENABLE_ANALYTICS_RECORDING
	TUniquePtr<FAnalyticsRecording> Recording = MakeUnique<FAnalyticsRecording>(BackendStatus);
	return Recording;
#else
	return nullptr;
#endif // UE_ENABLE_ANALYTICS_RECORDING
}

void FOnDemandIoBackendStats::GetIasCacheStats(uint64& OutUsed, uint64& OutMaxSize) const
{
	OutUsed = GCacheCachedBytes.Get();
	OutMaxSize = GCacheMaxBytes;
}

void FOnDemandIoBackendStats::OnCacheBootMs(uint64 TimeMs)
{
	GCacheBootMs = uint32(TimeMs);
}

void FOnDemandIoBackendStats::OnCacheError()
{
	GCacheErrorCount.Increment();
}

void FOnDemandIoBackendStats::OnCacheDecodeError()
{
	GCacheDecodeErrorCount.Increment();
}

void FOnDemandIoBackendStats::OnCacheGet(uint64 DataSize, ECacheType Type)
{
	GCacheGetCount.Increment();
	GCacheReadBytes.Add(DataSize);

	switch(Type)
	{
	case ECacheType::Memory:
		GCacheReadMemory.Add(DataSize);
		break;
	case ECacheType::Disk:
		GCacheReadDisk.Add(DataSize);
		break;
	default:
		checkNoEntry();
	}
}

void FOnDemandIoBackendStats::OnCachePut()
{
	GCachePutCount.Increment();
}

void FOnDemandIoBackendStats::OnCachePutExisting(uint64 /*DataSize*/)
{
	GCachePutExistingCount.Increment();
}

void FOnDemandIoBackendStats::OnCachePutReject(uint64 DataSize)
{
	GCachePutRejectCount.Increment();
	GCacheRejectBytes.Add(DataSize);
}

void FOnDemandIoBackendStats::OnCachePendingBytes(uint64 TotalSize)
{
	GCachePendingBytes.Set(TotalSize);
}

void FOnDemandIoBackendStats::OnCachePersistedBytes(uint64 TotalSize, int32 NumEntries)
{
	GCacheCachedBytes.Set(TotalSize);
	GCacheCachedEntries.Set(NumEntries);
}

void FOnDemandIoBackendStats::OnCacheWriteBytes(uint64 WriteSize)
{
	GCacheWriteCount.Increment();
	GCacheWrittenBytes.Add(WriteSize);
	GWriteRate.SubmitWrite(WriteSize);
}

void FOnDemandIoBackendStats::OnCacheEvictedBytes(uint64 EvictedSize)
{
	GCacheEvictedBytes.Add(EvictedSize);
}

void FOnDemandIoBackendStats::OnCacheSetMaxBytes(uint64 TotalSize)
{
	GCacheMaxBytes = TotalSize;
}

void FOnDemandIoBackendStats::OnCacheSuspended(double Seconds)
{
	GCacheSuspendedSeconds.Set(Seconds);
}

void FOnDemandIoBackendStats::OnHttpConnected()
{
	GHttpConnectCount.Increment();
}

void FOnDemandIoBackendStats::OnHttpDisconnected()
{
	GHttpDisconnectCount.Increment();
}

void FOnDemandIoBackendStats::OnHttpEnqueued(EHttpRequestType Type, uint64 RangeSize)
{
	GHttpPendingCount[(uint8)Type].Increment();
	GHttpPendingBytes[(uint8)Type].Add(RangeSize);
}

void FOnDemandIoBackendStats::OnHttpDequeued(EHttpRequestType Type, uint64 RangeSize, bool bWasCanceled)
{
	const uint8 Index = (uint8)Type;
	if (bWasCanceled)
	{
		GHttpCancelCount[Index].Increment();
	}
	GHttpPendingCount[Index].Decrement();
	GHttpPendingBytes[(uint8)Type].Subtract(RangeSize);
}

void FOnDemandIoBackendStats::OnHttpDecodeError(EHttpRequestType Type)
{
	GHttpDecodeErrorCount[(uint8)Type].Increment();
}

void FOnDemandIoBackendStats::OnHttpStarted(EHttpRequestType Type)
{
	++GHttpNumConcurrentRequests;
	GHttpInflightCount[uint8(Type)].Increment();
}

void FOnDemandIoBackendStats::OnHttpCompleted(
	EHttpRequestType Type,
	uint32 StatusCode,
	uint64 SizeBytes,
	uint64 DurationMs,
	uint32 RetryCount,
	int8 CDNCacheStatus,
	bool bCanceled)
{
	const uint8 Index	= (uint8)Type;
	const bool bHttpOk	= StatusCode >= 200 && StatusCode < 300;

	--GHttpNumConcurrentRequests;
	GHttpInflightCount[Index].Decrement();

	if (RetryCount > 0)
	{
		HttpCounters.RetryCount.Add(RetryCount);
		GHttpRetryCount[Index].Increment();
	}

	if (bHttpOk)
	{
		HttpCounters.GetCount.Increment();
		HttpCounters.GetBytes.Add(SizeBytes);
		HttpCounters.DurationTotalMs.Add(DurationMs);
		HttpCounters.DurationAvgMs.Set(int64(double(HttpCounters.DurationTotalMs.Get()) / double(HttpCounters.GetCount.Get())));

		GHttpGetCount[Index].Increment();
		GHttpDownloadedBytes[Index].Add(SizeBytes);
		GHttpDurationMsSum[Index] += DurationMs;
		GHttpDurationMs[Index].Set(DurationMs);

		GHttpHistory[Index].OnGet(SizeBytes, DurationMs);

		GHttpBandwidthMbps[Index].Set(GHttpHistory[Index].GetBandwidthMbps());
		GHttpDurationMsAvg[Index] = GHttpHistory[Index].GetAverage();

		GHttpDurationMsMax[Index] = FMath::Max(GHttpDurationMsMax[Index], (int32)DurationMs);

		switch (CDNCacheStatus)
		{
		case -1:	GHttpCdnCacheUnknown += 1;	break;	
		case 0:		GHttpCdnCacheMiss += 1;		break;
		default:	GHttpCdnCacheHit += 1;		break;
		}
	}
	else if (bCanceled)
	{
		HttpCounters.CancelCount.Increment();
		GHttpCancelCount[Index].Increment();
	}
	else
	{
		HttpCounters.ErrorCount.Increment();
		GHttpErrorCount[Index].Increment();
	}
}

void FOnDemandIoBackendStats::OnHttpOnRemovedPending(EHttpRequestType Type)
{
	GHttpPendingCount[(uint8)Type].Decrement();
}

void FOnDemandIoBackendStats::OnHttpPaused(EHttpRequestType Type)
{
	++GHttpNumPausedRequests[(uint8)Type];
}

void FOnDemandIoBackendStats::OnHttpUnpaused(EHttpRequestType Type)
{
	--GHttpNumPausedRequests[(uint8)Type];
}

void FOnDemandIoBackendStats::OnApplicationResume()
{
	++GAppResumeCount;
}

void FOnDemandIoBackendStats::OnCSVProfileEndRequested()
{
#if CSV_PROFILER_STATS
	CSV_METADATA(TEXT("IaxDevMode"), IsDevModeEnabled() ? TEXT("1") : TEXT("0"));
#endif
}

////////////////////////////////////////////////////////////////////////////////
void FOnDemandContentInstallerStats::OnRequestEnqueued()
{
	InstallerCounters.InflightCount.Increment();
}

void FOnDemandContentInstallerStats::OnRequestCompleted(
		const FResult& Result,
		uint64 RequestedChunkCount,
		uint64 RequestedBytes,
		uint64 DownloadedChunkCount,
		uint64 DownloadedBytes,
		double CacheHitRatio,
		uint64 DurationCycles) 
{
	InstallerCounters.InflightCount.Decrement();

	if (RequestedChunkCount == 0)
	{
		return;
	}

	{
		InstallerCounters.InstallCount.Increment();
		const int64 CurrentInstallCount = InstallerCounters.InstallCount.Get();
		InstallerCounters.DownloadedBytes.Add(DownloadedBytes);
		InstallerCounters.DurationTotalMs.fetch_add(uint64(FPlatformTime::ToMilliseconds64(DurationCycles)));
		InstallerCounters.DurationAvgMs.Set(
			int64(double(InstallerCounters.DurationTotalMs) / double(CurrentInstallCount)));
		const int64 CacheHitPct = FMath::RoundToInt64(100.0 * CacheHitRatio);
		const int64 TotalCacheHitPct = CacheHitPct + InstallerCounters.CacheHitTotalPct.fetch_add(CacheHitPct);
		const double AvgCacheHitPec = double(TotalCacheHitPct) / double(CurrentInstallCount);
		InstallerCounters.CacheHitAvgPct.Set(FMath::Clamp(FMath::RoundToInt64(AvgCacheHitPec), 0, 100));
	}

	{
		TUniqueLock Lock(InstallerAnalytics);
		
		InstallerAnalytics.InstallCount++;
		if (Result.HasError() && !UE::Core::IsCancellationError(Result.GetError()))
		{
			InstallerAnalytics.InstallErrorCount++;
		}
		if (CacheHitRatio != 1.0)
		{
			InstallerAnalytics.InstallCacheMissCount++;
		}
		InstallerAnalytics.DownloadedBytes += DownloadedBytes;
		InstallerAnalytics.TotalInstallDurationMs += uint64(FPlatformTime::ToMilliseconds64(DurationCycles));
		InstallerAnalytics.TotalCacheHitRatio += CacheHitRatio;
	}
}

static void LogOrReportContentInstallerAnalytics(bool bReport, bool bLog, TArray<FAnalyticsEventAttribute>& OutAnalyticsArray)
{
	if (GIadReportAnalyticsEnabled == false)
	{
		bReport = false;
	}

	if (bReport == false && bLog == false)
	{
		return;
	}

	FInstallerAnalytics CurrentInstallerAnalytics;
	{
		TUniqueLock Lock(InstallerAnalytics);
		CurrentInstallerAnalytics = InstallerAnalytics;
		InstallerAnalytics = FInstallerAnalytics();
	}

	FInstallCacheAnalytics::FShared CurrentSharedInstallCacheAnalytics;
	TArray<FInstallCacheAnalytics, TInlineAllocator<EOnDemandInstallCasType::Count>> CurrentInstallCacheAnalytics;
	{
		TUniqueLock Lock(FInstallCacheAnalytics::Mutex);

		CurrentSharedInstallCacheAnalytics = FInstallCacheAnalytics::Shared;
		FInstallCacheAnalytics::Shared = FInstallCacheAnalytics::FShared();

		for (EOnDemandInstallCasType CasType : TEnumRange<EOnDemandInstallCasType>())
		{
			FInstallCacheAnalytics& InstallCacheAnalytics = FInstallCacheAnalytics::Get(CasType);
			if (InstallCacheAnalytics.CasType == CasType) // If a CAS type is not enabled, if will fall back to EOnDemandInstallCasType::General
			{
				CurrentInstallCacheAnalytics.Add(InstallCacheAnalytics);
				InstallCacheAnalytics = FInstallCacheAnalytics{ .CasType = CasType };
			}
		}
	}

	const double AvgInstallDurationMs = CurrentInstallerAnalytics.InstallCount > 0 ? (double)CurrentInstallerAnalytics.TotalInstallDurationMs / (double)CurrentInstallerAnalytics.InstallCount : 0.0;
	const double AvgCacheHitRatio = CurrentInstallerAnalytics.InstallCount > 0 ? CurrentInstallerAnalytics.TotalCacheHitRatio / (double)CurrentInstallerAnalytics.InstallCount : 1.0;

	#define INSTALLER_ANALYTICS_LIST(XANALYTIC) \
		XANALYTIC(IadTotalInstallCount, CurrentInstallerAnalytics.InstallCount) \
		XANALYTIC(IadTotalInstallErrorCount, CurrentInstallerAnalytics.InstallErrorCount) \
		XANALYTIC(IadTotalInstallCacheMissCount, CurrentInstallerAnalytics.InstallCacheMissCount) \
		XANALYTIC(IadTotalDownloadedBytes, CurrentInstallerAnalytics.DownloadedBytes) \
		XANALYTIC(IadTotalInstallDurationMs, CurrentInstallerAnalytics.TotalInstallDurationMs) \
		XANALYTIC(IadAvgInstallDurationMs, AvgInstallDurationMs) \
		XANALYTIC(IadAvgCacheHitRatio, AvgCacheHitRatio)

	#define SHARED_INSTALL_CACHE_ANALYTICS_LIST(XANALYTIC) \
		XANALYTIC(IadInstallCacheResolveErrorCount, CurrentSharedInstallCacheAnalytics.ResolveErrorCount)

	TArray<double, TInlineAllocator<EOnDemandInstallCasType::Count>> AvgTotalFragmentationRatio;
	TArray<double, TInlineAllocator<EOnDemandInstallCasType::Count>> AvgDefraggedBlockFragmentationRatio;
	for (FInstallCacheAnalytics& InstallCacheAnalytics : CurrentInstallCacheAnalytics)
	{
		AvgTotalFragmentationRatio.Emplace_GetRef() = InstallCacheAnalytics.DefragCount > 0 ? InstallCacheAnalytics.TotalFragmentionRatio / (double)InstallCacheAnalytics.DefragCount : 0.0;
		AvgDefraggedBlockFragmentationRatio.Emplace_GetRef() = InstallCacheAnalytics.DefragCount > 0 ? InstallCacheAnalytics.DefraggedBlockFragmentionRatio / (double)InstallCacheAnalytics.DefragCount : 0.0;
	}

	#define INSTALL_CACHE_ANALYTICS_LIST(XANALYTIC) \
		XANALYTIC(IadAvgTotalFragmentationRatio, AvgTotalFragmentationRatio[Index]) \
		XANALYTIC(IadAvgDefraggedBlockFragmentationRatio, AvgDefraggedBlockFragmentationRatio[Index]) \
		XANALYTIC(IadInstallCacheStartupErrorCode, CurrentInstallCacheAnalytics[Index].StartupErrorCode) \
		XANALYTIC(IadInstallCacheVerificationRemovedBlockCount, CurrentInstallCacheAnalytics[Index].VerificationRemovedBlockCount) \
		XANALYTIC(IadInstallCacheFlushCount, CurrentInstallCacheAnalytics[Index].FlushCount) \
		XANALYTIC(IadInstallCacheFlushErrorCount, CurrentInstallCacheAnalytics[Index].FlushErrorCount) \
		XANALYTIC(IadInstallCacheFlushedBytes, CurrentInstallCacheAnalytics[Index].FlushedBytes) \
		XANALYTIC(IadInstallCacheDefragFlushCount, CurrentInstallCacheAnalytics[Index].DefragFlushCount) \
		XANALYTIC(IadInstallCacheDefragFlushErrorCount, CurrentInstallCacheAnalytics[Index].DefragFlushErrorCount) \
		XANALYTIC(IadInstallCacheDefragFlushedBytes, CurrentInstallCacheAnalytics[Index].DefragFlushedBytes) \
		XANALYTIC(IadInstallCachePurgeCount, CurrentInstallCacheAnalytics[Index].PurgeCount) \
		XANALYTIC(IadInstallCachePurgeErrorCount, CurrentInstallCacheAnalytics[Index].PurgeErrorCount) \
		XANALYTIC(IadInstallCacheDefragCount, CurrentInstallCacheAnalytics[Index].DefragCount) \
		XANALYTIC(IadInstallCacheDefragErrorCount, CurrentInstallCacheAnalytics[Index].DefragErrorCount) \
		XANALYTIC(IadInstallCacheJournalCommitCount, CurrentInstallCacheAnalytics[Index].JournalCommitCount) \
		XANALYTIC(IadInstallCacheJournalCommitErrorCount, CurrentInstallCacheAnalytics[Index].JournalCommitErrorCount) \
		XANALYTIC(IadInstallCacheMaxSize, CurrentInstallCacheAnalytics[Index].MaxCacheSize) \
		XANALYTIC(IadInstallCacheMaxUsageSize, CurrentInstallCacheAnalytics[Index].MaxCacheUsageSize) \
		XANALYTIC(IadInstallCacheMaxReferencedBlockSize, CurrentInstallCacheAnalytics[Index].MaxReferencedBlockSize) \
		XANALYTIC(IadInstallCacheMaxReferencedSize, CurrentInstallCacheAnalytics[Index].MaxReferencedSize) \
		XANALYTIC(IadInstallCacheMaxFragmentedSize, CurrentInstallCacheAnalytics[Index].MaxFragmentedSize) \
		XANALYTIC(IadInstallCacheReadCount, CurrentInstallCacheAnalytics[Index].ReadCount) \
		XANALYTIC(IadInstallCacheReadErrorCount, CurrentInstallCacheAnalytics[Index].ReadErrorCount)

	if (bReport)
	{
		#define COUNT_ANALYTIC(inName, inValue) ++AnalyticCount;
		int32 AnalyticCount = 0;
		INSTALLER_ANALYTICS_LIST(COUNT_ANALYTIC)
		SHARED_INSTALL_CACHE_ANALYTICS_LIST(COUNT_ANALYTIC)
		for (FInstallCacheAnalytics& InstallCacheAnalytics : CurrentInstallCacheAnalytics)
		{
			INSTALL_CACHE_ANALYTICS_LIST(COUNT_ANALYTIC)
		}
		OutAnalyticsArray.Reserve(AnalyticCount);
		#undef COUNT_ANALYTIC
		

		#define REPORT_ANALYTIC(inName, inValue) , TEXT(#inName), inValue
		AppendAnalyticsEventAttributeArray(OutAnalyticsArray
			INSTALLER_ANALYTICS_LIST(REPORT_ANALYTIC)
		);
		#undef REPORT_ANALYTIC

		#define REPORT_ANALYTIC(inName, inValue) , FString::Printf(TEXT("%s%s%s"), PrefixStr, DotStr, TEXT(#inName)), inValue
		{
			const TCHAR* PrefixStr = TEXT("Shared");
			const TCHAR* DotStr = TEXT(".");
			AppendAnalyticsEventAttributeArray(OutAnalyticsArray
				SHARED_INSTALL_CACHE_ANALYTICS_LIST(REPORT_ANALYTIC)
			);
		}
		for (int32 Index = 0; Index < CurrentInstallCacheAnalytics.Num(); ++Index)
		{
			FInstallCacheAnalytics& InstallCacheAnalytics = CurrentInstallCacheAnalytics[Index];
			const TCHAR* PrefixStr = TEXT("");
			const TCHAR* DotStr = TEXT("");
			if (InstallCacheAnalytics.CasType != EOnDemandInstallCasType::General)
			{
				PrefixStr = LexToString(InstallCacheAnalytics.CasType);
				DotStr = TEXT(".");
			}

			AppendAnalyticsEventAttributeArray(OutAnalyticsArray
				INSTALL_CACHE_ANALYTICS_LIST(REPORT_ANALYTIC)
			);
		}
		#undef REPORT_ANALYTIC
	}

	if (bLog)
	{
		#define ANALYTIC_LOG_FORMAT(inName, inValue) "stat:\t" #inName " {"#inName"}\n"
		#define ANALYTIC_LOG_VALUE(inName, inValue) , UE_LOGFMT_FIELD(#inName, inValue)

		UE_LOGFMT_EX(LogIoStoreOnDemand, Log, "IAD Installer Stats:\n" INSTALLER_ANALYTICS_LIST(ANALYTIC_LOG_FORMAT) INSTALLER_ANALYTICS_LIST(ANALYTIC_LOG_VALUE));

		UE_LOGFMT_EX(LogIoStoreOnDemand, Log, "IAD Shared Cache Stats:\n" SHARED_INSTALL_CACHE_ANALYTICS_LIST(ANALYTIC_LOG_FORMAT) SHARED_INSTALL_CACHE_ANALYTICS_LIST(ANALYTIC_LOG_VALUE));
		
		for (int32 Index = 0; Index < CurrentInstallCacheAnalytics.Num(); ++Index)
		{
			FInstallCacheAnalytics& InstallCacheAnalytics = CurrentInstallCacheAnalytics[Index];
			UE_LOGFMT_EX(LogIoStoreOnDemand, Log, "IAD {CasType} Cache Stats:\n" INSTALL_CACHE_ANALYTICS_LIST(ANALYTIC_LOG_FORMAT), UE_LOGFMT_FIELD("CasType", LexToString(InstallCacheAnalytics.CasType)) INSTALL_CACHE_ANALYTICS_LIST(ANALYTIC_LOG_VALUE));
		}

		#undef ANALYTIC_LOG_FORMAT
		#undef ANALYTIC_LOG_VALUE
	}

	#undef INSTALLER_ANALYTICS_LIST
	#undef SHARED_INSTALL_CACHE_ANALYTICS_LIST
	#undef INSTALL_CACHE_ANALYTICS_LIST
}

void FOnDemandContentInstallerStats::ReportAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray)
{
	const bool bReportTrue = true;
	const bool bLogFalse = false;
	LogOrReportContentInstallerAnalytics(bReportTrue, bLogFalse, OutAnalyticsArray);
}

void FOnDemandContentInstallerStats::LogAnalytics()
{
	const bool bReportFalse = false;
	const bool bLogTrue = true;
	TArray<FAnalyticsEventAttribute> UnusedAnalyticsArray;
	LogOrReportContentInstallerAnalytics(bReportFalse, bLogTrue, UnusedAnalyticsArray);
}

////////////////////////////////////////////////////////////////////////////////
void FOnDemandInstallCacheStats::OnStartupError(const FResult& Result)
{
	if (Result.HasError())
	{
		FOnDemandImmediateAnalytic Event = MakeAnalyticsEventFromResult(TEXT("IadStartupError"), Result);
		SendImmediateAnalytic(MoveTemp(Event));
	}
}

void FOnDemandInstallCacheStats::OnPutChunk(EOnDemandInstallCasType CasType, uint64 ByteCount)
{
	FInstallCacheCounters& InstallCacheCounters = FInstallCacheCounters::Get(CasType);
	InstallCacheCounters.InMemoryBytes.Add(ByteCount);
	InstallCacheCounters.InMemoryCount.Increment();
}

void FOnDemandInstallCacheStats::OnFlush(EOnDemandInstallCasType CasType, const FResult& Result, int64 ByteCount, bool bFromDefrag)
{
	FInstallCacheCounters& InstallCacheCounters = FInstallCacheCounters::Get(CasType);
	InstallCacheCounters.InMemoryBytes.Set(0);
	InstallCacheCounters.InMemoryCount.Set(0);

	if (Result.HasError() && UE::Core::IsCancellationError(Result.GetError()))
	{
		return;
	}

	InstallCacheCounters.FlushedBytes.Add(ByteCount);
	InstallCacheCounters.FlushCount.Increment();

	GWriteRate.SubmitWrite(ByteCount);

	UE::TUniqueLock Lock(FInstallCacheAnalytics::Mutex);
	FInstallCacheAnalytics& InstallCacheAnalytics = FInstallCacheAnalytics::Get(CasType);

	if (bFromDefrag)
	{
		if (Result.HasError())
		{
			InstallCacheAnalytics.DefragFlushErrorCount++;
		}
		InstallCacheAnalytics.DefragFlushCount++;
		InstallCacheAnalytics.DefragFlushedBytes += ByteCount;
	}
	else
	{
		if (Result.HasError())
		{
			InstallCacheAnalytics.FlushErrorCount++;
		}
		InstallCacheAnalytics.FlushCount++;
		InstallCacheAnalytics.FlushedBytes += ByteCount;
	}
}

void FOnDemandInstallCacheStats::OnJournalCommit(EOnDemandInstallCasType CasType, const FResult& Result, int64 ByteCount)
{
	if (Result.HasError() && UE::Core::IsCancellationError(Result.GetError()))
	{
		return;
	}

	UE::TUniqueLock Lock(FInstallCacheAnalytics::Mutex);
	FInstallCacheAnalytics& InstallCacheAnalytics = FInstallCacheAnalytics::Get(CasType);

	if (Result.HasError())
	{
		InstallCacheAnalytics.JournalCommitErrorCount++;
	}
	InstallCacheAnalytics.JournalCommitCount++;
}

void FOnDemandInstallCacheStats::OnCasVerification(EOnDemandInstallCasType CasType, const FResult& Result, int32 RemovedChunkCount, uint64 BlockBytesOverBudget)
{
	FOnDemandImmediateAnalytic Event = MakeAnalyticsEventFromResult(TEXT("IadCasVerification"), Result);
	AppendAnalyticsEventAttributeArray(Event.AnalyticsArray,
		TEXT("RemovedChunkCount"), RemovedChunkCount,
		TEXT("BlockBytesOverBudget"), BlockBytesOverBudget,
		TEXT("CasType"), LexToString(CasType));
	SendImmediateAnalytic(MoveTemp(Event));

	UE::TUniqueLock Lock(FInstallCacheAnalytics::Mutex);
	FInstallCacheAnalytics& InstallCacheAnalytics = FInstallCacheAnalytics::Get(CasType);
	InstallCacheAnalytics.VerificationRemovedBlockCount += RemovedChunkCount;
}

void FOnDemandInstallCacheStats::OnPurge(
	EOnDemandInstallCasType CasType,
	const FResult& Result,
	uint64 MaxCacheSize,
	uint64 NewCacheSize,
	uint64 BytesToPurge,
	uint64 PurgedBytes)
{
	if (Result.HasError() && UE::Core::IsCancellationError(Result.GetError()))
	{
		return;
	}

	FInstallCacheCounters& InstallCacheCounters = FInstallCacheCounters::Get(CasType);
	InstallCacheCounters.PurgeCount.Increment();

	{
		UE::TUniqueLock Lock(FInstallCacheAnalytics::Mutex);
		FInstallCacheAnalytics& InstallCacheAnalytics = FInstallCacheAnalytics::Get(CasType);
		if (Result.HasError())
		{
			InstallCacheAnalytics.PurgeErrorCount++;
		}
		InstallCacheAnalytics.PurgeCount++;
		InstallCacheAnalytics.PurgedBytes += PurgedBytes;
	}

	const int32 LineNo = TryGetLineNo(Result);

	FOnDemandImmediateAnalytic Event = MakeAnalyticsEventFromResult(TEXT("IadCachePurge"), Result);
	AppendAnalyticsEventAttributeArray(Event.AnalyticsArray, 
		TEXT("PurgedBytes"), PurgedBytes,
		TEXT("LineNo"), LineNo,
		TEXT("CasType"), LexToString(CasType));
	SendImmediateAnalytic(MoveTemp(Event));
}

void FOnDemandInstallCacheStats::OnDefrag(
	EOnDemandInstallCasType CasType,
	const FResult& Result, 
	uint64 TotalFragmentedBytes,
	uint64 TotalCachedBytes,
	uint64 BlocksToDefragFragmentedSize, 
	uint64 BlocksToDefragTotalSize)
{
	if (Result.HasError() && UE::Core::IsCancellationError(Result.GetError()))
	{
		return;
	}

	FInstallCacheCounters& InstallCacheCounters = FInstallCacheCounters::Get(CasType);
	InstallCacheCounters.DefragCount.Increment();

	const double TotalFragmentionRatio = TotalCachedBytes > 0
		? double(TotalFragmentedBytes) / double(TotalCachedBytes)
		: 0.0;

	const double BlocksToDefragFragmentionRatio = BlocksToDefragTotalSize > 0
		? double(BlocksToDefragFragmentedSize) / double(BlocksToDefragTotalSize)
		: 0.0;

	{
		UE::TUniqueLock Lock(FInstallCacheAnalytics::Mutex);
		FInstallCacheAnalytics& InstallCacheAnalytics = FInstallCacheAnalytics::Get(CasType);
		if (Result.HasError())
		{
			InstallCacheAnalytics.DefragErrorCount++;
		}
		InstallCacheAnalytics.DefragCount++;

		InstallCacheAnalytics.TotalFragmentionRatio += TotalFragmentionRatio;
		InstallCacheAnalytics.DefraggedBlockFragmentionRatio += BlocksToDefragFragmentionRatio;
	}

	const int32 LineNo = TryGetLineNo(Result);

	FOnDemandImmediateAnalytic Event = MakeAnalyticsEventFromResult(TEXT("IadCacheDefrag"), Result);
	AppendAnalyticsEventAttributeArray(Event.AnalyticsArray, 
		TEXT("TotalFragmentedBytes"), TotalFragmentedBytes,
		TEXT("TotalCachedBytes"), TotalCachedBytes,
		TEXT("TotalFragmentionRatio"), TotalFragmentionRatio,
		TEXT("DefraggedBlocksFragmentedSize"), BlocksToDefragFragmentedSize,
		TEXT("DefraggedBlocksTotalSize"), BlocksToDefragTotalSize,
		TEXT("DefraggedBlocksFragmentionRatio"), BlocksToDefragFragmentionRatio,
		TEXT("LineNo"), LineNo,
		TEXT("CasType"), LexToString(CasType));
	SendImmediateAnalytic(MoveTemp(Event));
}

void FOnDemandInstallCacheStats::OnBlockDeleted(EOnDemandInstallCasType CasType, int64 LastAccessTicks, int64 LastModificationTicks, bool bFromDefrag)
{
	const FDateTime Now = FDateTime::UtcNow();
	const FDateTime BlockAccess(LastAccessTicks);
	const FDateTime BlockModification(LastModificationTicks);

	// Theoretically all of the following checks shouldn't be necessary, as they should be 
	// impossible. Emperical data is not fun :(
	FTimespan BlockAccessAge = FTimespan::MaxValue();
	if (ensure(BlockAccess.GetTicks() >= 0) &&
		ensure(Now.GetTicks() >= 0) &&
		ensure(Now >= BlockAccess))
	{
		BlockAccessAge = Now - BlockAccess;
	}

	FTimespan BlockModAge = FTimespan::MaxValue();
	if (ensure(BlockModification.GetTicks() >= 0) &&
		ensure(Now.GetTicks() >= 0) &&
		ensure(Now >= BlockModification))
	{
		BlockModAge = Now - BlockModification;
	}

	SendImmediateAnalytic(FOnDemandImmediateAnalytic
	{
		TEXT("IadCacheBlockDeleted"),
		MakeAnalyticsEventAttributeArray(
			TEXT("BlockAgeMinutes"), BlockAccessAge.GetTotalMinutes(),
			TEXT("BlockModAgeMinutes"), BlockModAge.GetTotalMinutes(),
			TEXT("FromDefrag"), bFromDefrag,
			TEXT("CasType"), LexToString(CasType)
		)
	});
}

void FOnDemandInstallCacheStats::OnCacheUsage(
	EOnDemandInstallCasType CasType,
	uint64 MaxCacheSize,
	uint64 CacheSize,
	uint64 ReferencedBlockSize,
	uint64 ReferencedSize,
	uint64 FragmentedSize)
{
	FInstallCacheCounters& InstallCacheCounters = FInstallCacheCounters::Get(CasType);
	InstallCacheCounters.MaxCacheSize.Set(MaxCacheSize);
	InstallCacheCounters.CacheSize.Set(CacheSize);

	UE::TUniqueLock Lock(FInstallCacheAnalytics::Mutex);
	FInstallCacheAnalytics& InstallCacheAnalytics = FInstallCacheAnalytics::Get(CasType);

	InstallCacheAnalytics.MaxCacheSize				= MaxCacheSize; 
	InstallCacheAnalytics.MaxCacheUsageSize			= FMath::Max(InstallCacheAnalytics.MaxCacheUsageSize, CacheSize);
	InstallCacheAnalytics.MaxReferencedBlockSize	= FMath::Max(InstallCacheAnalytics.MaxReferencedBlockSize, ReferencedBlockSize);
	InstallCacheAnalytics.MaxReferencedSize			= FMath::Max(InstallCacheAnalytics.MaxReferencedSize, ReferencedSize);
	InstallCacheAnalytics.MaxFragmentedSize			= FMath::Max(InstallCacheAnalytics.MaxFragmentedSize, FragmentedSize);
}

void FOnDemandInstallCacheStats::OnReadCompleted(EOnDemandInstallCasType CasType, EIoErrorCode ErrorCode, uint64 EncodedSize)
{
	if (ErrorCode == EIoErrorCode::Cancelled)
	{
		return;
	}

	if (CasType == EOnDemandInstallCasType::None)
	{
		FInstallCacheCounters::Shared.ResolveErrorCount.Increment();
	}
	else
	{
		FInstallCacheCounters& InstallCacheCounters = FInstallCacheCounters::Get(CasType);		
		
		if (ErrorCode != EIoErrorCode::Ok)
		{
			InstallCacheCounters.ReadErrorCount.Increment();
			if (ErrorCode == EIoErrorCode::CompressionError)
			{
				InstallCacheCounters.ReadDecodeErrorCount.Increment();
			}
		}
		else
		{
			InstallCacheCounters.ReadBytes.Add(EncodedSize);
		}
		InstallCacheCounters.ReadCount.Increment();
	}

	UE::TUniqueLock Lock(FInstallCacheAnalytics::Mutex);

	if (CasType == EOnDemandInstallCasType::None)
	{
		if (ErrorCode != EIoErrorCode::Ok)
		{
			++FInstallCacheAnalytics::Shared.ResolveErrorCount;
		}
	}
	else
	{
		FInstallCacheAnalytics& InstallCacheAnalytics = FInstallCacheAnalytics::Get(CasType);

		if (ErrorCode != EIoErrorCode::Ok)
		{
			++InstallCacheAnalytics.ReadErrorCount;
		}
		++InstallCacheAnalytics.ReadCount;
	}
}

} // namespace UE::IoStore

#undef UE_ENABLE_ONSCREEN_STATISTICS

#endif // IAS_WITH_STATISTICS
