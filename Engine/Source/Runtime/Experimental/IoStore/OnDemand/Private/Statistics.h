// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CVarUtilities.h"
#include "HAL/LowLevelMemTracker.h"
#include "IO/OnDemandError.h"
#include "IO/IoStoreOnDemand.h"
#include "Misc/CoreDelegates.h"
#include "OnDemandBackendStatus.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"

#if COUNTERSTRACE_ENABLED || CSV_PROFILER_STATS
#	define IAS_WITH_STATISTICS 1
#else
#	define IAS_WITH_STATISTICS 0
#endif

#ifndef UE_TRACK_CDN_HIT_STATUS
	#if UE_BUILD_SHIPPING == 0
		#define UE_TRACK_CDN_HIT_STATUS 1
	#else
		#define UE_TRACK_CDN_HIT_STATUS 0
	#endif
#endif

#define UE_ENABLE_ONSCREEN_STATISTICS !UE_BUILD_SHIPPING

struct FAnalyticsEventAttribute;
class FIoStatus;
enum class EIoErrorCode;

LLM_DECLARE_TAG(Ias);

namespace UE::IoStore
{

class IAnalyticsRecording;
enum class EHttpRequestType : uint8;
struct FAvailableEndPoints;
struct FOnDemandImmediateAnalyticHandler;

enum class ECacheType : uint8
{
	/* The data was returned immediately from the in memory cache. */
	Memory,
	/* The data was returned at a later point from the disk cache. */
	Disk,
};

#if IAS_WITH_STATISTICS
#	define IAS_STATISTICS_IMPL(...) ;
#else
#	define IAS_STATISTICS_IMPL(...) { return __VA_ARGS__; }
#endif // IAS_WITH_STATISTICS

////////////////////////////////////////////////////////////////////////////////
#if IAS_WITH_STATISTICS
extern void OnDemandSetImmediateAnalyticHandler(FOnDemandImmediateAnalyticHandler&& EventHandler);
#else
inline void OnDemandSetImmediateAnalyticHandler(FOnDemandImmediateAnalyticHandler&& EventHandler) {}
#endif // IAS_WITH_STATISTICS

////////////////////////////////////////////////////////////////////////////////
class FOnDemandIoBackendStats
{
public:
	static FOnDemandIoBackendStats* Get() IAS_STATISTICS_IMPL(nullptr)

	FOnDemandIoBackendStats(FBackendStatus& InBackendStatus) IAS_STATISTICS_IMPL()
	~FOnDemandIoBackendStats() IAS_STATISTICS_IMPL()

	/** Report analytics not directly associated with a specific endpoint */
	void ReportGeneralAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const IAS_STATISTICS_IMPL()
	/** Report analytics for the current endpoint */
	void ReportEndPointAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const IAS_STATISTICS_IMPL()

	TUniquePtr<IAnalyticsRecording> StartAnalyticsRecording() const IAS_STATISTICS_IMPL(TUniquePtr<IAnalyticsRecording>())
	void GetIasCacheStats(uint64& OutUsed, uint64& OutMaxSize) const IAS_STATISTICS_IMPL(void(OutUsed = OutMaxSize = 0);) 

	void OnCacheBootMs(uint64 TimeMs) IAS_STATISTICS_IMPL()
	void OnCacheError() IAS_STATISTICS_IMPL()
	void OnCacheDecodeError() IAS_STATISTICS_IMPL()
	void OnCacheGet(uint64 DataSize, ECacheType Type) IAS_STATISTICS_IMPL()
	void OnCachePut() IAS_STATISTICS_IMPL()
	void OnCachePutExisting(uint64 DataSize) IAS_STATISTICS_IMPL()
	void OnCachePutReject(uint64 DataSize) IAS_STATISTICS_IMPL()
	void OnCachePendingBytes(uint64 TotalSize) IAS_STATISTICS_IMPL()
	void OnCachePersistedBytes(uint64 TotalSize, int32 NumEntries) IAS_STATISTICS_IMPL()
	void OnCacheWriteBytes(uint64 WriteSize) IAS_STATISTICS_IMPL()
	void OnCacheEvictedBytes(uint64 EvictedSize) IAS_STATISTICS_IMPL()
	void OnCacheSetMaxBytes(uint64 TotalSize) IAS_STATISTICS_IMPL()
	void OnCacheSuspended(double Seconds) IAS_STATISTICS_IMPL()

	// Per hostgroup
	void OnHttpConnected() IAS_STATISTICS_IMPL()
	void OnHttpDisconnected() IAS_STATISTICS_IMPL()

	// From HTTP I/O dispatcher
	void OnHttpEnqueued(EHttpRequestType Type, uint64 RangeSize) IAS_STATISTICS_IMPL()
	void OnHttpDequeued(EHttpRequestType Type, uint64 RangeSize, bool bWasCanceled) IAS_STATISTICS_IMPL()
	void OnHttpStarted(EHttpRequestType Type) IAS_STATISTICS_IMPL()
	void OnHttpCompleted(EHttpRequestType Type, uint32 StatusCode, uint64 BodySize, uint64 DurationMs, uint32 RetryCount, int8 CDNCacheStatus, bool bCanceled) IAS_STATISTICS_IMPL()
	void OnHttpDecodeError(EHttpRequestType Type) IAS_STATISTICS_IMPL()
	void OnHttpOnRemovedPending(EHttpRequestType Type) IAS_STATISTICS_IMPL()
	void OnHttpPaused(EHttpRequestType Type) IAS_STATISTICS_IMPL()
	void OnHttpUnpaused(EHttpRequestType Type) IAS_STATISTICS_IMPL()

private:

#if IAS_WITH_STATISTICS

	void OnApplicationResume();
	void OnCSVProfileEndRequested();

	void UpdateCSVValues();
	void PrintPeriodicLogging();

	void ResetOnScreenStatistics();
	void PrintOnScreenStatistics(FCoreDelegates::FSeverityMessageMap& OutMessages);

	FDelegateHandle OnApplicationResumeHandle;
	FDelegateHandle EndFrameDelegateHandle;
	FDelegateHandle OnScreenDelegateHandle;
	FDelegateHandle CSVProfileEndRequestedHandle;

#if UE_ENABLE_ONSCREEN_STATISTICS
	FConsoleCommandPtr ResetStatisticsCommand;
#endif // UE_ENABLE_ONSCREEN_STATISTICS

	const FBackendStatus& BackendStatus;

	bool bValuesValidForAnalytics = true;
#endif //IAS_WITH_STATISTICS
};

////////////////////////////////////////////////////////////////////////////////
class FOnDemandContentInstallerStats
{
public:
	static void OnRequestEnqueued() IAS_STATISTICS_IMPL()
	static void OnRequestCompleted(
		const FResult& Result,
		uint64 RequestedChunkCount,
		uint64 RequestedBytes,
		uint64 DownloadedChunkCount,
		uint64 DownloadedBytes,
		double CacheHitRatio,
		uint64 DurationCycles) IAS_STATISTICS_IMPL()
	static void ReportAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) IAS_STATISTICS_IMPL()
	static void LogAnalytics() IAS_STATISTICS_IMPL()
};

////////////////////////////////////////////////////////////////////////////////
class FOnDemandInstallCacheStats
{
public:
	static void OnStartupError(const FResult& Result) IAS_STATISTICS_IMPL()
	static void OnPutChunk(EOnDemandInstallCasType CasType, uint64 ChunkSize) IAS_STATISTICS_IMPL()
	static void OnFlush(EOnDemandInstallCasType CasType, const FResult& Result, int64 ByteCount, bool bFromDefrag) IAS_STATISTICS_IMPL()
	static void OnJournalCommit(EOnDemandInstallCasType CasType, const FResult& Result, int64 ByteCount) IAS_STATISTICS_IMPL()
	static void OnCasVerification(EOnDemandInstallCasType CasType, const FResult& Result, int32 RemovedChunkCount, uint64 BlockBytesOverBudget) IAS_STATISTICS_IMPL()
	static void OnPurge(
		EOnDemandInstallCasType CasType,
		const FResult& Result,	
		uint64 MaxCacheSize,
		uint64 NewCacheSize,
		uint64 BytesToPurge,
		uint64 PurgedBytes) IAS_STATISTICS_IMPL()
	static void OnDefrag(
		EOnDemandInstallCasType CasType,
		const FResult& Result,
		uint64 TotalFragmentedBytes,
		uint64 TotalCachedBytes,
		uint64 BlocksToDefragFragmentedSize, 
		uint64 BlocksToDefragTotalSize) IAS_STATISTICS_IMPL()
	static void OnBlockDeleted(EOnDemandInstallCasType CasType, int64 LastAccessTicks, int64 LastModificationTicks, bool bFromDefrag) IAS_STATISTICS_IMPL()
	static void OnCacheUsage(
		EOnDemandInstallCasType CasType,
		uint64 MaxCacheSize,
		uint64 CacheSize,
		uint64 ReferencedBlockSize,
		uint64 ReferencedSize,
		uint64 FragmentedSize) IAS_STATISTICS_IMPL()
	static void OnReadCompleted(EOnDemandInstallCasType CasType, EIoErrorCode ErrorCode, uint64 EncodedSize) IAS_STATISTICS_IMPL()
};

#undef IAS_STATISTICS_IMPL

} // namespace UE::IoStore
