// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"
#include "Templates/UnrealTemplate.h"

namespace UE::IoStore
{

/**
 * Note that the type used requires the following members:
 *     NextRequest: A pointer to the same type, this is used to maintain the list.
 *     Priority: An int32 value used to store the priority of the value
 */
template<typename T>
class TThreadSafeIntrusiveQueue
{
public:
	void Enqueue(T* Request)
	{
		check(Request->NextRequest == nullptr);
		FScopeLock _(&CriticalSection);

		if (Tail)
		{
			Tail->NextRequest = Request;
		}
		else
		{
			check(Head == nullptr);
			Head = Request;
		}

		Tail = Request;

		++ListNum;
	}

	void EnqueueByPriority(T* Request)
	{
		FScopeLock _(&CriticalSection);
		EnqueueByPriorityInternal(Request);

		++ListNum;
	}

	T* Dequeue(int32 NumToRemove = MAX_int32)
	{
		FScopeLock _(&CriticalSection);

		if (NumToRemove <= 0 || ListNum == 0)
		{
			return nullptr;
		}
		else if (NumToRemove == MAX_int32)
		{
			T* Requests = Head;
			Head = Tail = nullptr;

			ListNum = 0;

			return Requests;
		}
		else
		{
			int32 NumRemoved = 1;
			T* Requests = Head;
			while(NumRemoved < NumToRemove && Requests->NextRequest != nullptr)
			{
				Requests = Requests->NextRequest;
				NumRemoved++;
			}

			// Store the new head of the list for later
			T* NewHead = Requests->NextRequest;

			// Break off the subsection we are returning
			Requests->NextRequest = nullptr;

			// Clear the tail if the entire list was dequeued
			if (Tail == Requests)
			{
				Tail = nullptr;
			}

			Swap(Head, NewHead);
			ListNum -= NumRemoved;

			return NewHead;
		}

		
	}

	void Reprioritize(T* Request, int32 NewPriority)
	{
		check(Request != nullptr);

		// Switch to double linked list/array if this gets too expensive
		FScopeLock _(&CriticalSection);
		
		Request->Priority = NewPriority;
		if (RemoveInternal(Request))
		{
			EnqueueByPriorityInternal(Request);
		}
	}

	int32 Num() const
	{
		FScopeLock _(&CriticalSection);
		return ListNum;
	}

private:
	void EnqueueByPriorityInternal(T* Request)
	{
		check(Request->NextRequest == nullptr);

		if (Head == nullptr || Request->Priority > Head->Priority)
		{
			if (Head == nullptr)
			{
				check(Tail == nullptr);
				Tail = Request;
			}

			Request->NextRequest = Head;
			Head = Request;
		}
		else if (Request->Priority <= Tail->Priority)
		{
			check(Tail != nullptr);
			Tail->NextRequest = Request;
			Tail = Request;
		}
		else
		{
			// NOTE: This can get expensive if the queue gets too long, might be better to have x number of bucket(s)
			TRACE_CPUPROFILER_EVENT_SCOPE(IasBackend::EnqueueByPriority);
			T* It = Head;
			while (It->NextRequest != nullptr && Request->Priority <= It->NextRequest->Priority)
			{
				It = It->NextRequest;
			}

			Request->NextRequest = It->NextRequest;
			It->NextRequest = Request;
		}
	}

	bool RemoveInternal(T* Request)
	{
		check(Request != nullptr);
		if (Head == nullptr)
		{
			check(Tail == nullptr);
			return false;
		}

		if (Head == Request)
		{
			Head = Request->NextRequest;
			if (Tail == Request)
			{
				check(Head == nullptr);
				Tail = nullptr;
			}

			Request->NextRequest = nullptr;
			return true;
		}
		else
		{
			T* It = Head;
			while (It->NextRequest && It->NextRequest != Request)
			{
				It = It->NextRequest;
			}

			if (It->NextRequest == Request)
			{
				It->NextRequest = It->NextRequest->NextRequest;
				Request->NextRequest = nullptr;
				if (Tail == Request)
				{
					Tail = It;
				}
				return true;
			}
		}

		return false;
	}

	mutable FCriticalSection CriticalSection;
	
	T* Head = nullptr;
	T* Tail = nullptr;

	int32 ListNum = 0;
};

} //namespace UE::IoStore
