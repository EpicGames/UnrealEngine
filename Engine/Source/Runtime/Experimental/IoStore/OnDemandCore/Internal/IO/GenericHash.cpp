// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericHash.h"

#include "Containers/ArrayView.h"
#include "Hash/Blake3.h"
#include "IO/IoHash.h"
#include "Misc/StringBuilder.h"
#include "String/BytesToHex.h"

namespace UE::GenericHash
{

static_assert(sizeof(FIoHash) == 20);

FStringBuilderBase& ToHex(FMemoryView Memory, FStringBuilderBase& Out)
{
	UE::String::BytesToHexLower(
		MakeArrayView<const uint8>(
			reinterpret_cast<const uint8*>(Memory.GetData()),
			IntCastChecked<int32>(Memory.GetSize())),
		Out);

	return Out;
}

FString ToHex(FMemoryView Memory)
{
	TStringBuilder<256> Sb;
	ToHex(Memory, Sb);
	return FString(Sb);
}

void HashBuffer(FMutableMemoryView Dst, FMemoryView Src)
{
	check(Dst.GetSize() > 0);
	check(Src.GetSize() > 0);
	FBlake3Hash		Blake = FBlake3::HashBuffer(Src);
	const uint64	BlakeSize = uint64(sizeof(FBlake3Hash::ByteArray));

	Dst.CopyFrom(FMemoryView(Blake.GetBytes(), FMath::Min(Dst.GetSize(), BlakeSize)));
}

} // namesapce UE::GenericHash
