// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "HAL/UnrealMemory.h"
#include "Math/UnrealMathUtility.h"
#include "Memory/MemoryView.h"
#include "Serialization/Archive.h"

#define UE_API IOSTOREONDEMANDCORE_API

struct FIoHash;

namespace UE
{

namespace GenericHash
{

UE_API FString				ToHex(FMemoryView Memory);
UE_API FStringBuilderBase&	ToHex(FMemoryView Memory, FStringBuilderBase& Out);
UE_API void					HashBuffer(FMutableMemoryView Dst, FMemoryView Src);

} // namespace UE::GenericHash

template<SIZE_T BitCount>
struct THash
{
	static constexpr SIZE_T Size	= BitCount >> 3;
	using ByteArray					= uint8[Size];

						THash() = default;
	const uint8*		GetData() const { return Hash; }
	SIZE_T				GetSize() const { return Size; }
	FMemoryView			GetView() const { return FMemoryView(Hash, Size); }

	static THash		HashBuffer(FMemoryView Memory);
	static void			HashBuffer(FMemoryView Memory, THash& Out);

	static THash		From(FMemoryView Memory);
	static THash		From(const uint8* Data, SIZE_T DataSize);
	static THash		From(const FIoHash& IoHash);
	static const		THash Zero;

	friend inline bool operator==(const THash& A, const THash& B)
	{
		return FMemory::Memcmp(A.Hash, B.Hash, Size) == 0;
	}

	friend inline bool operator!=(const THash& A, const THash& B)
	{
		return FMemory::Memcmp(A.Hash, B.Hash, Size) != 0;
	}

	friend inline bool operator<(const THash& A, const THash& B)
	{
		return FMemory::Memcmp(A.Hash, B.Hash, Size) < 0;
	}

	friend inline uint32 GetTypeHash(const THash& H)
	{
		return *reinterpret_cast<const uint32*>(H.Hash);
	}

	friend inline FString LexToString(const THash& H)
	{
		return GenericHash::ToHex(H.GetView());
	}

	friend inline FStringBuilderBase& LexToString(const THash& H, FStringBuilderBase& Out)
	{
		return GenericHash::ToHex(H.GetView(), Out);
	}

	friend FArchive& operator<<(FArchive& Ar, THash& H)
	{
		Ar.Serialize(const_cast<uint8*>(H.GetData()), H.GetSize());
		return Ar;
	}

private:
	FMutableMemoryView GetMutableView() { return FMutableMemoryView(Hash, Size); }

	alignas(uint32) ByteArray Hash{};
};

template<SIZE_T BitCount>
const THash<BitCount> THash<BitCount>::Zero;

template<SIZE_T BitCount>
THash<BitCount> THash<BitCount>::From(FMemoryView Memory)
{
	THash<BitCount> Hash;
	Hash.GetMutableView().CopyFrom(Memory.Left(THash<BitCount>::Size));
	return Hash;
}

template<SIZE_T BitCount>
THash<BitCount> THash<BitCount>::From(const uint8* Data, SIZE_T DataSize)
{
	return From(MakeMemoryView(Data, DataSize));
}

template<SIZE_T BitCount>
THash<BitCount> THash<BitCount>::From(const FIoHash& IoHash)
{
	THash<BitCount> Hash;
	FMemory::Memcpy(&Hash, &IoHash, FMath::Min(THash<BitCount>::Size, SIZE_T(20)));
	return Hash;
}

template<SIZE_T BitCount>
void THash<BitCount>::HashBuffer(FMemoryView Memory, THash<BitCount>& Out)
{
	GenericHash::HashBuffer(MakeMemoryView(Out.Hash, Out.GetSize()), Memory);
}

template<SIZE_T BitCount>
THash<BitCount> THash<BitCount>::HashBuffer(FMemoryView Memory)
{
	THash<BitCount> Hash;
	GenericHash::HashBuffer(MakeMemoryView(Hash.Hash, Hash.GetSize()), Memory);
	return Hash;
}

using FHash32	= THash<32>;
using FHash64	= THash<64>;
using FHash96	= THash<96>;
using FHash128	= THash<128>;
using FHash160	= THash<160>;
using FHash256	= THash<256>;

} // namespace UE

#undef UE_API
