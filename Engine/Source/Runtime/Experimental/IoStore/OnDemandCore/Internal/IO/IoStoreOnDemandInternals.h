// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SharedString.h"
#include "HAL/Platform.h"
#include "IO/IoStoreOnDemand.h"

#ifndef WITH_IOSTORE_ONDEMAND_TESTS
#define WITH_IOSTORE_ONDEMAND_TESTS 0
#endif

#define UE_API IOSTOREONDEMANDCORE_API

namespace UE::IoStore
{

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandWeakContentHandle
{
	bool IsValid() const { return HandleId != 0; }

	inline bool operator==(const FOnDemandWeakContentHandle& Other) const
	{
		return HandleId == Other.HandleId;
	}

	inline bool operator!=(const FOnDemandWeakContentHandle& Other) const
	{
		return HandleId != Other.HandleId;
	}

	inline operator bool() const { return IsValid(); }

	inline friend uint32 GetTypeHash(const FOnDemandWeakContentHandle& Handle)
	{
		return GetTypeHash(Handle.HandleId);
	}

	UE_API static FOnDemandWeakContentHandle FromUnsafeHandle(UPTRINT HandleId);

	UPTRINT			HandleId = 0;
	FSharedString	DebugName;
};

///////////////////////////////////////////////////////////////////////////////
class FOnDemandInternalContentHandle
{
public:
	FOnDemandInternalContentHandle();

	FOnDemandInternalContentHandle(FSharedString InDebugName);

	UE_API ~FOnDemandInternalContentHandle();

	FOnDemandInternalContentHandle(const FOnDemandInternalContentHandle& Other) = default;
	FOnDemandInternalContentHandle(FOnDemandInternalContentHandle&& Other) = default;

	FOnDemandInternalContentHandle& operator=(const FOnDemandInternalContentHandle& Other) = default;
	FOnDemandInternalContentHandle& operator=(FOnDemandInternalContentHandle&& Other) = default;

	UPTRINT HandleId() const { return UPTRINT(this); }

	FSharedString			DebugName;

private:
	friend class FOnDemandContentInstaller;
	FWeakOnDemandIoStore	IoStore; // This needs additional syncronization if its read outside the destructor
};

////////////////////////////////////////////////////////////////////////////////
class FOnDemandInternalInstallRequest
{
public:
	using EStatus = FOnDemandRequest::EStatus;
	
	explicit FOnDemandInternalInstallRequest(UPTRINT InInstallerRequest)
		: InstallerRequest(InInstallerRequest) { }

	FWeakOnDemandIoStore	IoStore;
	UPTRINT					InstallerRequest = 0;
	std::atomic<EStatus>	Status{EStatus::Pending};
};

////////////////////////////////////////////////////////////////////////////////
class IOnDemandInternalHttpStats
{
public:
	enum { SAMPLE_COUNT = FOnDemandHttpStats::SAMPLE_COUNT };
	virtual ~IOnDemandInternalHttpStats() = default;
	virtual uint32 GetRecvKiBps() const = 0;
	virtual void GetRecvKiBps(uint32 (&Out)[SAMPLE_COUNT]) const = 0;
	virtual uint32 GetTotalRecvKiB() const = 0;
	virtual uint32 GetTimeToFirstByteMs() const = 0;
};

} // namespace UE::IoStore

////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////////
UE_API FString LexToString(const UE::IoStore::FOnDemandInternalContentHandle& Handle);

#undef UE_API
