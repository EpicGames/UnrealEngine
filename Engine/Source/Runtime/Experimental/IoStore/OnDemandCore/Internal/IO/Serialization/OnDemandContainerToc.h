// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "Containers/StringFwd.h"
#include "IO/IoBuffer.h"
#include "IO/IoContainerId.h"
#include "IO/GenericHash.h"
#include "IO/IoHash.h"
#include "IO/IoStatus.h"
#include "Misc/AES.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Misc/TVariant.h"

#define UE_API IOSTOREONDEMANDCORE_API

class FArchive;
class IMappedFileHandle;
class IMappedFileRegion;
class FIoChunkId;
using FIoBlockHash = uint32;
 
enum class EIoContainerFlags : uint8;

namespace UE::IoStore { struct FOnDemandToc; }

namespace UE::IoStore::Serialization
{

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandFileExt
{
	static constexpr const TCHAR* Toc			= TEXT(".uondemandtoc");
	static constexpr const TCHAR* Chunk			= TEXT(".iochunk");
	static constexpr const TCHAR* PartitionToc	= TEXT(".ioparttoc");
	static constexpr const TCHAR* Partition		= TEXT(".iopart");
};

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandTocSignature
{
	static constexpr inline ANSICHAR ExpectedSignature[] = "UE ON-DEMAND TOC";

	FOnDemandTocSignature()
	{
		FMemory::Memcpy(Signature, ExpectedSignature, sizeof(Signature));
	}

	bool IsValid() const
	{
		return FMemory::Memcmp(Signature, ExpectedSignature, sizeof(Signature)) == 0;
	}

	ANSICHAR Signature[16];
};
static_assert(sizeof(FOnDemandTocSignature) == 16);

///////////////////////////////////////////////////////////////////////////////
enum class EOnDemandTocMajorVersion : uint16
{
	Invalid			= 0,
	One				= 1,

	LatestPlusOne,
	Latest			= (LatestPlusOne - 1)
};

///////////////////////////////////////////////////////////////////////////////
enum class EOnDemandTocMinorVersion : uint16
{
	Invalid			= 0,
	MemoryMapped	= 1,
	Partitions		= 2,

	LatestPlusOne,
	Latest			= (LatestPlusOne - 1)
};

///////////////////////////////////////////////////////////////////////////////
struct alignas(4) FOnDemandTocVersion
{
	bool IsValid() const { return Major > 0 && Major != ~uint16(0) && Minor > 0 && Minor != ~uint16(0); }

	uint16 Major = uint16(EOnDemandTocMajorVersion::Latest);
	uint16 Minor = uint16(EOnDemandTocMinorVersion::Latest);
};
static_assert(sizeof(FOnDemandTocVersion) == 4);

////////////////////////////////////////////////////////////////////////////////
struct FOnDemandTocFooter
{
	FOnDemandTocSignature Signature;
};
static_assert(sizeof(FOnDemandTocFooter) == 16);

////////////////////////////////////////////////////////////////////////////////
enum class FOnDemandTocStorageType : uint8
{
	Invalid = 0,
	Memory,
	MemoryMapped
};

////////////////////////////////////////////////////////////////////////////////
class FOnDemandTocStorage
{
	struct FMappedFileStorage
	{
		TUniquePtr<IMappedFileHandle> MappedFile;
		TUniquePtr<IMappedFileRegion> MappedRegion;
	};

	struct FMemoryStorage
	{
		FIoBuffer Buffer;
	};

	using FStorage = TVariant<FEmptyVariantState, FMemoryStorage, FMappedFileStorage>;

public:
	UE_API FOnDemandTocStorage();
	UE_API FOnDemandTocStorage(FIoBuffer HeaderData, FIoBuffer ContainerData);
	UE_API FOnDemandTocStorage(FIoBuffer HeaderData, TUniquePtr<IMappedFileHandle>&& File, TUniquePtr<IMappedFileRegion>&& Region);
	UE_API FOnDemandTocStorage(FOnDemandTocStorage&& Other);
	UE_API ~FOnDemandTocStorage();

	UE_API FOnDemandTocStorage& operator=(FOnDemandTocStorage&&);

	UE_API FMemoryView				GetView() const;
	UE_API FOnDemandTocStorageType	StorageType() const;
	bool							IsEmpty() const { return HeaderData.GetSize() == 0; }

private:
	FOnDemandTocStorage(const FOnDemandTocStorage&) = delete;
	FOnDemandTocStorage& operator=(const FOnDemandTocStorage&) = delete;

	FIoBuffer	HeaderData;
	FStorage	ContainerData;
};

////////////////////////////////////////////////////////////////////////////////
enum class EOnDemandContainerEntryFlags : uint32
{
	None			= 0,
	InstallOnDemand	= (1 << 0),
	StreamOnDemand	= (1 << 1),

	Last			= StreamOnDemand
};
ENUM_CLASS_FLAGS(EOnDemandContainerEntryFlags);

////////////////////////////////////////////////////////////////////////////////
class FOnDemandTocWriter
{
public:
	UE_API FOnDemandTocWriter();
	UE_API FOnDemandTocWriter(FOnDemandTocWriter&&);
	UE_API ~FOnDemandTocWriter();

	UE_API FOnDemandTocWriter& operator=(FOnDemandTocWriter&&);

	UE_API bool					IsEmpty() const;
	UE_API void					SetMetadata(
									const FString& BuildVersion,
									const FString& TargetPlatform);
	UE_API void					SetHostGroup(const FString& HostGroupName);
	UE_API void					SetChunksDirectory(const FString& ChunksDirectory);
	UE_API void					BeginContainer(
									const FString& ContainerName,
									const FIoContainerId& ContainerId,
									const FIoHash& UTocHash,
									const FGuid& EncryptionKeyGuid,
									uint32 CompressionBlockSize,
									uint32 FileContainerFlags,
									EOnDemandContainerEntryFlags ContainerFlags,
									FIoBuffer ContainerHeader);
	UE_API int32				BeginPartition();
	UE_API uint64				AddChunk(
									int32 Partition,
									const FIoChunkId& ChunkId,
									FIoBuffer EncodedChunk,
									TConstArrayView<uint32> BlockSizes,
									TConstArrayView<FIoBlockHash> BlockHashes,
									uint64 RawChunkSize);
	UE_API void					AddTagSet(FStringView Tag, TConstArrayView<uint32> PackageIndices);
	UE_API FIoBuffer			EndPartition(int32 Partition, FIoHash& OutHash);
	UE_API void 				EndContainer(FName CompressionFormat);

	UE_API TIoStatusOr<uint64>	Write(FArchive& Ar);
	UE_API TIoStatusOr<uint64>	Write(const FString& Filename);

private:
	FOnDemandTocWriter(const FOnDemandTocWriter&) = delete;
	FOnDemandTocWriter& operator=(const FOnDemandTocWriter&) = delete;

	class FImpl;
	TUniquePtr<FImpl> Impl;
};

////////////////////////////////////////////////////////////////////////////////
namespace V1
{

struct FOnDemandStringEntry
{
	uint32 Offset = 0;
	uint32 Len = 0;
};
static_assert(sizeof(FOnDemandStringEntry) == 8);

struct FOnDemandTocHeader
{
	FOnDemandTocSignature	Signature;
	FOnDemandTocVersion		Version;
	uint32					Pad = 0;
	int64					EpochTimestamp = 0;
	FOnDemandStringEntry	BuildVersion;
	FOnDemandStringEntry	TargetPlatform;
	FOnDemandStringEntry	ChunksDirectory;
	FOnDemandStringEntry	HostGroupName;
	FOnDemandStringEntry	CompressionFormat;
	uint32					StringTableLen = 0;
	uint32					ContainerCount = 0;
	uint8					Pad2[48] = {0};
};
static_assert(sizeof(FOnDemandTocHeader) == 128);

struct FOnDemandContainerEntry
{
	FGuid					EncryptionKeyGuid;
	FIoContainerId			ContainerId;
	FOnDemandStringEntry	ContainerName;
	uint32					ContainerHeaderSize = 0;
	uint32					DataOffset = 0;
	uint32					DataSize = 0;
	uint32					ChunkCount = 0;
	uint32					BlockCount = 0;
	uint32					BlockSize = 0;
	uint32					TagSetCount = 0;
	uint32					TagSetIndicesCount = 0;
	FIoHash					UTocHash = FIoHash::Zero;
	uint32					ContainerFlags = 0; // EOnDemandContainerEntryFlags
	uint32					FileContainerFlags = 0; // EIoContainerFlags
	uint32					PartitionCount = 0;
	int8					Pad[32] = {0};
};
static_assert(sizeof(FOnDemandContainerEntry) == 128);

struct FOnDemandChunkBlockInfo
{
	FOnDemandChunkBlockInfo()
	{
		FMemory::Memzero(this, sizeof(FOnDemandChunkBlockInfo));
	}

	uint32			Offset()	const { return bHasOffset ? OffsetOrSize : ~uint32(0); }
	uint32			Count()		const { return bHasOffset ? CountOrHash : OffsetOrSize > 0 ? 1 : 0; }
	uint32			Size()		const { return bHasOffset ? 0 : OffsetOrSize; }
	uint32			Hash()		const { return bHasOffset ? 0 : CountOrHash; }
	const uint32*	Data()		const { return reinterpret_cast<const uint32*>(this); } 

	static FOnDemandChunkBlockInfo FromOffsetAndCount(uint32 BlockOffset, uint32 BlockCount)
	{
		FOnDemandChunkBlockInfo Nfo;
		Nfo.OffsetOrSize	= BlockOffset;
		Nfo.bHasOffset		= 1;
		Nfo.CountOrHash		= BlockCount;
		return Nfo;
	}

	static FOnDemandChunkBlockInfo FromSizeAndHash(uint32 BlockSize, uint32 BlockHash)
	{
		FOnDemandChunkBlockInfo Nfo;
		Nfo.OffsetOrSize	= BlockSize;
		Nfo.bHasOffset		= 0;
		Nfo.CountOrHash		= BlockHash;
		return Nfo;
	}

private:
	uint32 OffsetOrSize: 31;
	uint32 bHasOffset: 1;
	uint32 CountOrHash;
};
static_assert(sizeof(FOnDemandChunkBlockInfo) == 8);

struct FOnDemandChunkEntry
{
	uint32						GetDiskSize() const { return Align(EncodedSize, FAES::AESBlockSize); }

	FIoHash						Hash = FIoHash::Zero;
	uint32						RawSize = 0;
	uint32						EncodedSize = 0;
	FOnDemandChunkBlockInfo		BlockInfo;
};
static_assert(sizeof(FOnDemandChunkEntry) == 36);

struct FOnDemandTagSetEntry
{
	FOnDemandStringEntry	Tag;
	uint32					Offset = 0;
	uint32					Count = 0;
};
static_assert(sizeof(FOnDemandTagSetEntry) == 16);

////////////////////////////////////////////////////////////////////////////////
struct FOnDemandContainerTocHeaderView
{
	using EContainerFlags = EOnDemandContainerEntryFlags;

	FOnDemandContainerTocHeaderView() = default;
	FOnDemandContainerTocHeaderView(
		const FOnDemandTocHeader* InHeader,
		const UTF8CHAR* InStringTable,
		const FOnDemandContainerEntry* InContainerEntry)
			: Header(InHeader)
			, StringTable(InStringTable)
			, ContainerEntry(InContainerEntry)
	{ }

	FUtf8StringView				ContainerName() const { return GetString(ContainerEntry->ContainerName); }
	FIoContainerId				ContainerId() const { return ContainerEntry->ContainerId; }
	EContainerFlags				ContainerFlags() const { return static_cast<EContainerFlags>(ContainerEntry->ContainerFlags); }
	UE_API EIoContainerFlags	FileContainerFlags() const;
	const FGuid&				EncryptionKeyGuid() const { return ContainerEntry->EncryptionKeyGuid; }
	uint32						BlockSize() const { return ContainerEntry->BlockSize; }
	UE_API FUtf8StringView		GetString(FOnDemandStringEntry StringEntry) const;

	bool						IsValid() const { return Header != nullptr && StringTable != nullptr && ContainerEntry != nullptr; }
								operator bool() const { return IsValid(); }

private:
	const FOnDemandTocHeader*		Header = nullptr;
	const UTF8CHAR*					StringTable = nullptr;
	const FOnDemandContainerEntry*	ContainerEntry = nullptr;
};

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandContainerTocView
{
	FOnDemandContainerTocHeaderView			Header;
	TConstArrayView<FIoChunkId>				ChunkIds;
	TConstArrayView<FOnDemandChunkEntry>	ChunkEntries;
	TConstArrayView<uint32>					BlockSizes;
	TConstArrayView<FIoBlockHash>			BlockHashes;
	TConstArrayView<FOnDemandTagSetEntry>	TagSets;
	TConstArrayView<uint32>					TagSetIndices;
	FIoBuffer								ContainerHeaderChunk; // FIoContainerHeader
};

////////////////////////////////////////////////////////////////////////////////
enum class EOnDemandTocReaderOptions : uint32
{
	None		= 0,
	MemoryMap	= (1 << 0)
};
ENUM_CLASS_FLAGS(EOnDemandTocReaderOptions);

////////////////////////////////////////////////////////////////////////////////
class FOnDemandTocReader
{
public:
	FOnDemandTocReader() = default;
	FOnDemandTocReader(FOnDemandTocReader&&) = default;
	FOnDemandTocReader& operator=(FOnDemandTocReader&&) = default;

	FDateTime				CreationTime() const { return FDateTime::FromUnixTimestamp(Header->EpochTimestamp); }
	FUtf8StringView			BuildVersion() const { return GetString(Header->BuildVersion); }
	FUtf8StringView			TargetPlatform() const { return GetString(Header->TargetPlatform); }
	FUtf8StringView			ChunksDirectory() const { return GetString(Header->ChunksDirectory); }
	FUtf8StringView			HostGroupName() const { return GetString(Header->HostGroupName); }
	FUtf8StringView			CompressionFormat() const { return GetString(Header->CompressionFormat); }
	UE_API FUtf8StringView	GetString(FOnDemandStringEntry StringEntry) const;

	TConstArrayView<FOnDemandContainerEntry> Containers() const { return ContainerEntries; }

	UE_API TIoStatusOr<FOnDemandContainerTocView> ReadContainer(
		const FOnDemandContainerEntry& Container,
		FOnDemandTocStorage& Storage,
		EOnDemandTocReaderOptions Options) const;
	UE_API static TIoStatusOr<FOnDemandTocReader> Read(const FString& Filename);

private:
	FOnDemandTocReader(const FString& Filename, FIoBuffer Buffer);
	FOnDemandTocReader(const FOnDemandTocReader&) = delete;
	FOnDemandTocReader& operator=(const FOnDemandTocReader&) = delete;

	FString										Filename;
	FIoBuffer									Buffer;
	const FOnDemandTocHeader*					Header = nullptr;
	const UTF8CHAR*								StringTable = nullptr;
	TConstArrayView<FOnDemandContainerEntry>	ContainerEntries;
};

} // V1

////////////////////////////////////////////////////////////////////////////////
namespace V2
{

////////////////////////////////////////////////////////////////////////////////
using FOnDemandStringEntry				= V1::FOnDemandStringEntry;
using FOnDemandTocHeader				= V1::FOnDemandTocHeader;
using FOnDemandContainerEntry			= V1::FOnDemandContainerEntry;
using FOnDemandChunkBlockInfo			= V1::FOnDemandChunkBlockInfo;
using FOnDemandTagSetEntry				= V1::FOnDemandTagSetEntry;
using FOnDemandContainerTocHeaderView	= V1::FOnDemandContainerTocHeaderView;
using EOnDemandTocReaderOptions			= V1::EOnDemandTocReaderOptions;
using FOnDemandChunkHash				= FHash96;
using FOnDemandPartitionHash			= FIoHash;

////////////////////////////////////////////////////////////////////////////////
struct FOnDemandPartitionEntry
{
	FIoHash	Hash = FIoHash::Zero;
	uint32	Size = 0;
};
static_assert(sizeof(FOnDemandPartitionEntry) == 24);

////////////////////////////////////////////////////////////////////////////////
struct FOnDemandChunkEntry
{
	uint32						GetDiskSize() const { return Align(EncodedSize, FAES::AESBlockSize); }

	FOnDemandChunkHash			Hash;
	uint32						PartitionIndex = 0;
	uint32						PartitionOffset = 0;
	uint32						RawSize = 0;
	uint32						EncodedSize = 0;
	FOnDemandChunkBlockInfo		BlockInfo;
};
static_assert(sizeof(FOnDemandChunkEntry) == 36);

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandContainerTocView
{
	FOnDemandContainerTocHeaderView				Header;
	TConstArrayView<FOnDemandPartitionEntry>	PartitionEntries;
	TConstArrayView<FIoChunkId>					ChunkIds;
	TConstArrayView<FOnDemandChunkEntry>		ChunkEntries;
	TConstArrayView<uint32>						BlockSizes;
	TConstArrayView<FIoBlockHash>				BlockHashes;
	TConstArrayView<FOnDemandTagSetEntry>		TagSets;
	TConstArrayView<uint32>						TagSetIndices;
	FIoBuffer									ContainerHeaderChunk; // FIoContainerHeader
};

////////////////////////////////////////////////////////////////////////////////
class FOnDemandTocReader
{
public:
	FOnDemandTocReader() = default;
	FOnDemandTocReader(FOnDemandTocReader&&) = default;
	FOnDemandTocReader& operator=(FOnDemandTocReader&&) = default;

	FDateTime				CreationTime() const { return FDateTime::FromUnixTimestamp(Header->EpochTimestamp); }
	FUtf8StringView			BuildVersion() const { return GetString(Header->BuildVersion); }
	FUtf8StringView			TargetPlatform() const { return GetString(Header->TargetPlatform); }
	FUtf8StringView			ChunksDirectory() const { return GetString(Header->ChunksDirectory); }
	FUtf8StringView			HostGroupName() const { return GetString(Header->HostGroupName); }
	FUtf8StringView			CompressionFormat() const { return GetString(Header->CompressionFormat); }
	UE_API FUtf8StringView	GetString(FOnDemandStringEntry StringEntry) const;

	TConstArrayView<FOnDemandContainerEntry> Containers() const { return ContainerEntries; }

	UE_API TIoStatusOr<FOnDemandContainerTocView> ReadContainer(
		const FOnDemandContainerEntry& Container,
		FOnDemandTocStorage& Storage,
		EOnDemandTocReaderOptions Options) const;
	UE_API static TIoStatusOr<FOnDemandTocReader> Read(const FString& Filename);

private:
	FOnDemandTocReader(const FString& Filename, FIoBuffer Buffer);
	FOnDemandTocReader(const FOnDemandTocReader&) = delete;
	FOnDemandTocReader& operator=(const FOnDemandTocReader&) = delete;

	FString										Filename;
	FIoBuffer									Buffer;
	const FOnDemandTocHeader*					Header = nullptr;
	const UTF8CHAR*								StringTable = nullptr;
	TConstArrayView<FOnDemandContainerEntry>	ContainerEntries;
};

} // namespace V2
	
} // UE::IoStore::Serialization

////////////////////////////////////////////////////////////////////////////////
UE_API FStringBuilderBase& operator<<(FStringBuilderBase& Sb, UE::IoStore::Serialization::EOnDemandContainerEntryFlags Flags);
UE_API FString LexToString(UE::IoStore::Serialization::EOnDemandContainerEntryFlags Flags);

#undef UE_API
