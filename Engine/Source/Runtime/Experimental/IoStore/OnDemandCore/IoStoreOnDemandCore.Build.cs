// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class IoStoreOnDemandCore : ModuleRules
{
	public IoStoreOnDemandCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CoreUObject",
				"Json"
			}
		);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error; 
	}
}
