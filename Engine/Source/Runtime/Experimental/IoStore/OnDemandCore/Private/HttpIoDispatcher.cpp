// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/HttpIoDispatcher.h"

#include "Containers/AnsiString.h"
#include "IO/IoChunkId.h"
#include "Misc/StringBuilder.h"

DEFINE_LOG_CATEGORY(LogHttpIoDispatcher);

namespace UE
{

////////////////////////////////////////////////////////////////////////////////
TSharedPtr<IHttpIoDispatcher> GHttpIoDispatcher;
const FIoHttpOptions FIoHttpOptions::Default = FIoHttpOptions();

////////////////////////////////////////////////////////////////////////////////
#if UE_HTTP_METADATA_ENABLED
struct FIoHttpMetadata::FMetadata
{
	FIoChunkId ChunkId = FIoChunkId::InvalidChunkId;
};
#endif

////////////////////////////////////////////////////////////////////////////////
FIoHttpMetadata::FIoHttpMetadata()
#if UE_HTTP_METADATA_ENABLED
	: Metadata(MakeShared<FMetadata>())
#endif
{
}

FIoHttpMetadata::FIoHttpMetadata(const FIoChunkId& ChunkId)
	: FIoHttpMetadata()
{
	SetChunkId(ChunkId);
}

FIoHttpMetadata::~FIoHttpMetadata()
{
}

const FIoChunkId& FIoHttpMetadata::GetChunkId() const
{
#if UE_HTTP_METADATA_ENABLED
	return Metadata->ChunkId;
#else
	return FIoChunkId::InvalidChunkId;
#endif
}

void FIoHttpMetadata::SetChunkId(const FIoChunkId& ChunkId)
{
#if UE_HTTP_METADATA_ENABLED
	Metadata->ChunkId = ChunkId;
#endif
}

////////////////////////////////////////////////////////////////////////////////
FIoHttpHeaders& FIoHttpHeaders::Add(FAnsiString&& HeaderName, FAnsiString&& HeaderValue)
{
	Headers.Add(MoveTemp(HeaderName));
	Headers.Add(MoveTemp(HeaderValue));
	return *this;
}

FIoHttpHeaders& FIoHttpHeaders::Add(FAnsiStringView HeaderName, FAnsiStringView HeaderValue)
{
	Headers.Emplace(HeaderName);
	Headers.Emplace(HeaderValue);
	return *this;
}

FAnsiStringView FIoHttpHeaders::Get(FAnsiStringView Key) const
{
	for (int32 Idx = 0, Count = Headers.Num(); Idx < Count; Idx += 2)
	{
		FAnsiStringView Header = Headers[Idx];
		if (Header.Equals(Key, ESearchCase::IgnoreCase))
		{
			return Headers[Idx + 1];
		}
	}

	return FAnsiStringView();
}

FAnsiStringView FIoHttpHeaders::Get(const FAnsiString& Key) const
{
	FAnsiStringView KeyView = Key;
	return Get(KeyView);
}

TArray<FAnsiString> FIoHttpHeaders::ToArray() &&
{
	return MoveTemp(Headers);
}

FIoHttpHeaders FIoHttpHeaders::Create(FAnsiString&& HeaderName, FAnsiString&& HeaderValue)
{
	FIoHttpHeaders Headers;
	Headers.Add(MoveTemp(HeaderName), MoveTemp(HeaderValue));
	return Headers;
}

FIoHttpHeaders FIoHttpHeaders::Create(FAnsiStringView HeaderName, FAnsiStringView HeaderValue)
{
	FIoHttpHeaders Headers;
	Headers.Add(HeaderName, HeaderValue);
	return Headers;
}

FIoHttpHeaders FIoHttpHeaders::Create(TArray<FAnsiString>&& Headers)
{
	check(Headers.IsEmpty() || ((Headers.Num() % 2) == 0));
	return FIoHttpHeaders(MoveTemp(Headers));
}

////////////////////////////////////////////////////////////////////////////////
FIoRelativeUrl FIoRelativeUrl::From(FAnsiStringView Url)
{
	if (Url.IsEmpty())
	{
		return FIoRelativeUrl();
	}

	if (Url[0] != ANSICHAR('/'))
	{
		TAnsiStringBuilder<128> Sb;
		Sb << "/" << Url;
		return FIoRelativeUrl(Sb.ToString());
	}

	return FIoRelativeUrl(Url);
}

////////////////////////////////////////////////////////////////////////////////
void FIoHttpRequest::Cancel()
{
	FIoHttpRequest ToCancel(*this);
	GHttpIoDispatcher->CancelRequest(ToCancel);
}

void FIoHttpRequest::UpdatePriorty(int32 NewPriority)
{
	FIoHttpRequest ToUpdate(*this);
	GHttpIoDispatcher->UpdateRequestPriority(ToUpdate, NewPriority);
}

EIoErrorCode FIoHttpRequest::Status() const
{
	return Handle.IsValid() ? Handle->GetStatus() : EIoErrorCode::InvalidCode;
}

FIoHttpRequest&	FIoHttpRequest::operator=(FIoHttpRequest&& Other)
{
	Handle = MoveTemp(Other.Handle);
	return *this;
}

////////////////////////////////////////////////////////////////////////////////
FIoHttpRequest FIoHttpBatch::Get(
	const FName& HostGroup,
	const FIoRelativeUrl& RelativeUrl,
	FIoHttpHeaders&& Headers,
	const FIoHttpOptions& Options,
	const FIoHash& ChunkHash,
	FIoHttpRequestCompleted&& OnCompleted)
{
	return GHttpIoDispatcher->CreateRequest(
		First,
		Last,
		HostGroup,
		RelativeUrl,
		Options,
		MoveTemp(Headers),
		MoveTemp(OnCompleted),
		&ChunkHash);
}

FIoHttpRequest FIoHttpBatch::Get(
	const FName& HostGroup,
	const FIoRelativeUrl& RelativeUrl,
	FIoHttpHeaders&& Headers,
	const FIoHttpOptions& Options,
	FIoHttpRequestCompleted&& OnCompleted)
{
	return GHttpIoDispatcher->CreateRequest(
		First,
		Last,
		HostGroup,
		RelativeUrl,
		Options,
		MoveTemp(Headers),
		MoveTemp(OnCompleted));
}

FIoHttpRequest FIoHttpBatch::Get(
	const FName& HostGroup,
	const FIoRelativeUrl& RelativeUrl,
	FIoHttpHeaders&& Headers,
	FIoHttpRequestCompleted&& OnCompleted)
{
	return Get(HostGroup, RelativeUrl, MoveTemp(Headers), FIoHttpOptions::Default, MoveTemp(OnCompleted));
}

FIoHttpRequest FIoHttpBatch::Get(
	const FName& HostGroup,
	const FIoRelativeUrl& RelativeUrl,
	FIoHttpRequestCompleted&& OnCompleted)
{
	return Get(HostGroup, RelativeUrl, FIoHttpHeaders(), FIoHttpOptions::Default, MoveTemp(OnCompleted));
}

void FIoHttpBatch::Issue()
{
	if (IIoHttpRequestHandle* ToIssue = First)
	{
		First = Last = nullptr;
		GHttpIoDispatcher->IssueRequest(*ToIssue);
	}
}

////////////////////////////////////////////////////////////////////////////////
bool FHttpIoDispatcher::IsInitialized()
{
	return GHttpIoDispatcher.IsValid();
}

FIoStatus FHttpIoDispatcher::Initialize(TSharedPtr<IHttpIoDispatcher> Dispatcher)
{
	if (GHttpIoDispatcher.IsValid())
	{
		return FIoStatus(EIoErrorCode::InvalidCode);
	}

	UE_LOGF(LogHttpIoDispatcher, Log, "Initializing HTTP I/O dispatcher");
	GHttpIoDispatcher = Dispatcher;

	return FIoStatus::Ok;
}

FIoStatus FHttpIoDispatcher::Shutdown()
{
	if (GHttpIoDispatcher.IsValid() == false)
	{
		return FIoStatus::Invalid;
	}

	UE_LOGF(LogHttpIoDispatcher, Log, "Shutting down HTTP I/O dispatcher");
	GHttpIoDispatcher->Shutdown();
	GHttpIoDispatcher.Reset();

	return FIoStatus::Ok;
}

void FHttpIoDispatcher::GetHttpStats(FOnDemandHttpStats& Out)
{
	return GHttpIoDispatcher->GetHttpStats(Out);
}

FIoStatus FHttpIoDispatcher::RegisterHostGroup(const FName& HostGroup, TConstArrayView<FAnsiString> HostNames, FAnsiStringView TestUrl)
{
	if (GHttpIoDispatcher.IsValid())
	{
		return GHttpIoDispatcher->RegisterHostGroup(HostGroup, HostNames, TestUrl);
	}

	return FIoStatus(EIoErrorCode::InvalidCode);
}

FIoStatus FHttpIoDispatcher::RegisterHostGroup(const FName& HostGroup, TConstArrayView<FAnsiString> HostNames)
{
	return RegisterHostGroup(HostGroup, HostNames, FAnsiStringView());
}

FIoStatus FHttpIoDispatcher::RegisterHostGroup(const FName& HostGroup, FAnsiStringView HostName, FAnsiStringView TestUrl)
{
	FAnsiString HostNameString(HostName);
	TConstArrayView<FAnsiString> HostNames(&HostNameString, 1);
	return RegisterHostGroup(HostGroup, HostNames, TestUrl);
}

FIoStatus FHttpIoDispatcher::RegisterHostGroup(const FName& HostGroup, FAnsiStringView HostName)
{
	return RegisterHostGroup(HostGroup, HostName, FAnsiStringView());
}

bool FHttpIoDispatcher::IsHostGroupRegistered(const FName& HostGroup)
{
	return GHttpIoDispatcher.IsValid() ? GHttpIoDispatcher->IsHostGroupRegistered(HostGroup) : false;
}

bool FHttpIoDispatcher::IsHostGroupOk(const FName& HostGroup)
{
	return GHttpIoDispatcher.IsValid() ? GHttpIoDispatcher->IsHostGroupOk(HostGroup) : false;
}

FIoHttpBatch FHttpIoDispatcher::NewBatch()
{
	check(GHttpIoDispatcher.IsValid());
	return FIoHttpBatch();
}

FIoHttpRequest FHttpIoDispatcher::Get(
	const FName& HostGroup,
	const FIoRelativeUrl& RelativeUrl,
	FIoHttpHeaders&& Headers,
	const FIoHttpOptions& Options,
	FIoHttpRequestCompleted&& OnCompleted)
{
	ensure(GHttpIoDispatcher.IsValid());

	IIoHttpRequestHandle* First = nullptr;
	IIoHttpRequestHandle* Last = nullptr;

	FIoHttpRequest Request = GHttpIoDispatcher->CreateRequest(
		First,
		Last,
		HostGroup,
		RelativeUrl,
		Options,
		MoveTemp(Headers),
		MoveTemp(OnCompleted));

	if (ensure(First != nullptr))
	{
		GHttpIoDispatcher->IssueRequest(*First);
	}

	return Request; 
}

FIoHttpRequest FHttpIoDispatcher::Get(
	const FName& HostGroup,
	const FIoRelativeUrl& RelativeUrl,
	FIoHttpHeaders&& Headers,
	FIoHttpRequestCompleted&& OnCompleted)
{
	return Get(HostGroup, RelativeUrl, MoveTemp(Headers), FIoHttpOptions::Default, MoveTemp(OnCompleted));
}

FIoHttpRequest FHttpIoDispatcher::Get(
	const FName& HostGroup,
	const FIoRelativeUrl& RelativeUrl,
	FIoHttpRequestCompleted&& OnCompleted)
{
	return Get(HostGroup, RelativeUrl, FIoHttpHeaders(), FIoHttpOptions::Default, MoveTemp(OnCompleted));
}

FIoStatus FHttpIoDispatcher::CacheResponse(const FIoHttpResponse& Response)
{
	 check(GHttpIoDispatcher.IsValid());
	 return GHttpIoDispatcher->CacheResponse(Response);
}

FIoStatus FHttpIoDispatcher::EvictFromCache(const FIoHttpResponse& Response)
{
	check(GHttpIoDispatcher.IsValid());
	return GHttpIoDispatcher->EvictFromCache(Response);
}

FHttpIoDispatcher::FHostGroupRegistered& FHttpIoDispatcher::OnHostGroupRegistered()
{
	check(GHttpIoDispatcher.IsValid());
	return GHttpIoDispatcher->OnHostGroupRegistered();
}

} // namespace UE
