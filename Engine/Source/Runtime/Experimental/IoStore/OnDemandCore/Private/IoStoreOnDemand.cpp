// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoStoreOnDemand.h"

#include "Containers/StringView.h"
#include "Features/IModularFeatures.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "IO/IoStatus.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/Parse.h"
#include "Modules/ModuleManager.h"
#include "String/Numeric.h"

#if !UE_BUILD_SHIPPING
#include "Misc/PackageName.h"
#endif //!UE_BUILD_SHIPPING

DEFINE_LOG_CATEGORY(LogIoStoreOnDemand);
DEFINE_LOG_CATEGORY(LogIas);

///////////////////////////////////////////////////////////////////////////////
const TCHAR* LexToString(UE::IoStore::EOnDemandInstallCasType CacheType)
{
	using namespace UE::IoStore;

	switch (CacheType)
	{
	case EOnDemandInstallCasType::General:
		return TEXT("General");
	case EOnDemandInstallCasType::MMap:
		return TEXT("MMap");
	}

	return TEXT("None");
	static_assert(EOnDemandInstallCasType::Count == 2); // Does not include 'None'
}

void LexFromString(UE::IoStore::EOnDemandInstallCasType& CacheType, FStringView String)
{
	using namespace UE::IoStore;

	CacheType = EOnDemandInstallCasType::None;
	for (EOnDemandInstallCasType Val : TEnumRange<EOnDemandInstallCasType>())
	{
		if (String == LexToString(Val))
		{
			CacheType = Val;
			return;
		}
	}
}

namespace UE::IoStore
{

IOnDemandIoStore* GOnDemandIoStore = nullptr;

///////////////////////////////////////////////////////////////////////////////
#if !UE_BUILD_SHIPPING
namespace Commands
{

////////////////////////////////////////////////////////////////////////////////
static TIoStatusOr<FOnDemandHostGroup> SplitHostAndTocUrl(FStringView Url, FStringView& OutTocRelativUrl)
{
	if (Url.StartsWith(TEXTVIEW("http")) == false)
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid URL protocol"));
	}

	int32 Delim = INDEX_NONE;
	if (Url.FindChar(':', Delim) == false)
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Invalid URL"));
	}

	const int32 ProtocolDelim = Delim + 3;
	if (Url.RightChop(ProtocolDelim).FindChar('/', Delim) == false)
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Failed to find host and TOC path delimiter"));
	}

	const FStringView Host	= Url.Left(ProtocolDelim + Delim);
	OutTocRelativUrl		= Url.RightChop(Host.Len());

	return FOnDemandHostGroup::Create(Host);
}

////////////////////////////////////////////////////////////////////////////////
static void PurgeInstallCache(const TArray<FString>& Args)
{
	if (UE::IoStore::IOnDemandIoStore* IoStore = UE::IoStore::TryGetOnDemandIoStore())
	{
		FOnDemandPurgeArgs PurgeArgs;
		for (const FString& Arg : Args)
		{
			EOnDemandInstallCasType CasType = EOnDemandInstallCasType::None;

			if (Arg == TEXTVIEW("defrag"))
			{
				PurgeArgs.Options |= EOnDemandPurgeOptions::Defrag;
			}
			else if (LexFromString(CasType, Arg); CasType != EOnDemandInstallCasType::None)
			{
				PurgeArgs.CasType = CasType;
			}
			else if (!PurgeArgs.BytesToPurge && UE::String::IsNumericOnlyDigits(Arg))
			{
				uint64 BytesToPurge = 0;
				if (LexTryParseString(BytesToPurge, *Arg))
				{
					PurgeArgs.BytesToPurge = BytesToPurge;
				}
			}
		}

		UE_LOGF(LogIoStoreOnDemand, Display, "Purging on demand install cache %ls", LexToString(PurgeArgs.CasType));
		IoStore->Purge(MoveTemp(PurgeArgs), [](const FOnDemandPurgeResult& Result)
		{
			if (Result.IsOk())
			{
				UE_LOGF(LogIoStoreOnDemand, Display, "Purged on demand install cache");
			}
			else
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed Purged on demand install cache: %ls", *LexToString(Result.Error.GetValue()));
			}
		});
	}
	else
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Purge install cache failed, reason 'I/O store on-demand module not initialized'");
	}
}

////////////////////////////////////////////////////////////////////////////////
static void DefragInstallCache(const TArray<FString>& Args)
{
	if (UE::IoStore::IOnDemandIoStore* IoStore = UE::IoStore::TryGetOnDemandIoStore())
	{
		FOnDemandDefragArgs DefragArgs;
		for (const FString& Arg : Args)
		{
			EOnDemandInstallCasType CasType = EOnDemandInstallCasType::None;
			if (LexFromString(CasType, Arg); CasType != EOnDemandInstallCasType::None)
			{
				DefragArgs.CasType = CasType;
			}
			else if (UE::String::IsNumericOnlyDigits(Arg))
			{
				uint64 BytesToFree = 0;
				if (LexTryParseString(BytesToFree, *Arg))
				{
					DefragArgs.BytesToFree = BytesToFree;
					break;
				}
			}
		}

		UE_LOGF(LogIoStoreOnDemand, Display, "Defragging on demand install cache %ls", LexToString(DefragArgs.CasType));
		IoStore->Defrag(MoveTemp(DefragArgs), [](const FOnDemandDefragResult& Result)
		{
			if (Result.IsOk())
			{
				UE_LOGF(LogIoStoreOnDemand, Display, "Defragmented on demand install cache");
			}
			else
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to defragment on demand install cache: %ls", *LexToString(Result.Error.GetValue()));
			}
		});
	}
	else
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Defrag install cache failed, reason 'I/O store on-demand module not initialized'");
	}
}

////////////////////////////////////////////////////////////////////////////////
static void PrintCacheUsage(const TArray<FString>& InArgs)
{
	if (UE::IoStore::IOnDemandIoStore* IoStore = UE::IoStore::TryGetOnDemandIoStore())
	{
		FOnDemandGetCacheUsageArgs UsageArgs;
		UsageArgs.Options = EOnDemandGetCacheUsageOptions::DumpHandlesToLog;

		TIoStatusOr<FOnDemandCacheUsage> MaybeUsage = IoStore->GetCacheUsage(UsageArgs);
		if (!MaybeUsage.IsOk())
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "iostore.CacheUsage failed: %ls", *MaybeUsage.Status().ToString());
			return;
		}

		const FOnDemandCacheUsage CacheUsage = MaybeUsage.ConsumeValueOrDie();

		UE_LOGF(LogIoStoreOnDemand, Display, "iostore.CacheUsage");

		TUtf8StringBuilder<256> Sb;
		Sb << UTF8TEXTVIEW("InstallCache: ") << CacheUsage.InstallCache;
		UE_LOGF(LogIoStoreOnDemand, Display, "%s", Sb.ToString());

		Sb.Reset();
		Sb << UTF8TEXTVIEW("StreamingCache: ") << CacheUsage.StreamingCache;
		UE_LOGF(LogIoStoreOnDemand, Display, "%s", Sb.ToString());

		IoStore->DumpMountedContainersToLog();
	}
	else
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Print cache usage failed, reason 'I/O store on-demand module not initialized'");
	}
}

////////////////////////////////////////////////////////////////////////////////
static void MountUrl(const TArray<FString>& Args)
{
	UE::IoStore::IOnDemandIoStore* IoStore = UE::IoStore::TryGetOnDemandIoStore();
	if (IoStore == nullptr)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to load I/O store on-demand module.");
		return;
	}

	if (Args.Num() < 1)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Not enough arguments.");
		return;
	}

	FString Url = Args[0];
	Url.TrimQuotesInline();

	FStringView TocRelativeUrl;
	TIoStatusOr<FOnDemandHostGroup> HostGroup = SplitHostAndTocUrl(Url, TocRelativeUrl);
	if (HostGroup.IsOk() == false)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "%ls", *HostGroup.Status().ToString());
		return;
	}

	const FString& MountId = Args.Num() > 1 ? Args[1] : Url; 

	IoStore->Mount(
		FOnDemandMountArgs
		{
			.MountId = MountId,
			.TocRelativeUrl = FString(TocRelativeUrl),
			.HostGroup = HostGroup.ConsumeValueOrDie()
		},
		[](FOnDemandMountResult MountResult)
		{
			MountResult.LogResult();
		});
}

////////////////////////////////////////////////////////////////////////////////
static void MountFile(const TArray<FString>& Args)
{
	UE::IoStore::IOnDemandIoStore* IoStore = UE::IoStore::TryGetOnDemandIoStore();
	if (IoStore == nullptr)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to load I/O store on-demand module.");
		return;
	}

	if (Args.Num() < 2)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Not enough arguments.");
		return;
	}

	TArray<FOnDemandMountArgs> AllMountArgs;
	FString FilenameOrWildcard = Args[0];
	FilenameOrWildcard.TrimQuotesInline();
	TArray<FString> FilePaths;

	const FString ContentDir = FString::Printf(TEXT("%sPaks/"), *FPaths::ProjectContentDir());
	IFileManager& Ifm = IFileManager::Get();
	Ifm.IterateDirectoryRecursively(
		*ContentDir,
		[&FilenameOrWildcard, &FilePaths, &ContentDir](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
		{
			if (bIsDirectory == false)
			{
				FString FilePath(FilenameOrDirectory);
				if (FPathViews::GetExtension(FilePath) == TEXT("uondemandtoc"))
				{
					FStringView Filename = FPathViews::GetBaseFilename(FilePath);
					if (Filename == FilenameOrWildcard || FString(Filename).MatchesWildcard(FilenameOrWildcard))
					{
						FilePaths.Add(FilenameOrDirectory);
					}
				}
			}
			return true;
		});

	if (FilePaths.IsEmpty())
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to find any on-demand TOC file(s) matching '%ls'", *FilenameOrWildcard);
		return;
	}

	FString HostPath = Args[1];
	HostPath.TrimQuotesInline();
	FStringView TocRelativeUrl;
	TIoStatusOr<FOnDemandHostGroup> HostResult = SplitHostAndTocUrl(HostPath, TocRelativeUrl);
	if (HostResult.IsOk() == false)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "%ls", *HostResult.Status().ToString());
		return;
	}

	FOnDemandHostGroup HostGroup = HostResult.ConsumeValueOrDie();

	const FString& MountId = Args.Num() > 2 ? Args[2] : FilePaths[0];
	for (const FString& FilePath : FilePaths)
	{
		FOnDemandMountArgs& MountArgs	= AllMountArgs.AddDefaulted_GetRef();
		MountArgs.FilePath				= FilePath;
		MountArgs.HostGroup				= HostGroup;
		MountArgs.TocRelativeUrl		= TocRelativeUrl;
		MountArgs.MountId				= MountId;
	}

	if (AllMountArgs.IsEmpty())
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to parse mount arguments");
		return;
	}

	for (FOnDemandMountArgs& MountArgs : AllMountArgs)
	{
		IoStore->Mount(
			MoveTemp(MountArgs),
			[](FOnDemandMountResult MountResult)
			{
				MountResult.LogResult();
			});
	}
}

////////////////////////////////////////////////////////////////////////////////
static void InstallPackage(const TArray<FString>& Args)
{
	UE::IoStore::IOnDemandIoStore* IoStore = UE::IoStore::TryGetOnDemandIoStore();
	if (IoStore == nullptr)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to load I/O store on-demand module.");
		return;
	}

	if (Args.Num() < 1)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Not enough arguments.");
		return;
	}

	FString PackageName = Args[0];
	PackageName.TrimQuotesInline();
	FPackageId PackageId;
	if (FPackageName::IsValidLongPackageName(PackageName))
	{
		PackageId = FPackageId::FromName(FName(*PackageName));
	}

	if (PackageId.IsValid() == false)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Invalid package name '%ls'", *PackageName);
		return;
	}

	static FOnDemandContentHandle DefaultContentHandle = FOnDemandContentHandle::Create(UE::FSharedString(TEXT("ConsoleCommand")));

	FOnDemandInstallArgs InstallArgs;
	InstallArgs.PackageIds.Add(PackageId);
	InstallArgs.ContentHandle = DefaultContentHandle;
	InstallArgs.Options = EOnDemandInstallOptions::InstallSoftReferences;

	IoStore->Install(
		MoveTemp(InstallArgs),
		[PackageName = MoveTemp(PackageName)](FOnDemandInstallResult InstallResult)
		{
			if (InstallResult.IsOk())
			{
				UE_LOGF(LogIoStoreOnDemand, Log, "Successfully installed package '%ls'", *PackageName);
			}
			else
			{
				UE_LOGF(LogIoStoreOnDemand, Error, "Failed to install package '%ls', reason '%ls'",
					*PackageName, *LexToString(InstallResult.Error.GetValue()));
			}
		});
}

////////////////////////////////////////////////////////////////////////////////
static void VerifyCache()
{
	UE::IoStore::IOnDemandIoStore* IoStore = UE::IoStore::TryGetOnDemandIoStore();
	if (IoStore == nullptr)
	{
		UE_LOGF(LogIoStoreOnDemand, Error, "Failed to load I/O store on-demand module.");
		return;
	}

	IoStore->Verify([](FOnDemandVerifyCacheResult&& VerifyResult)
	{
		if (VerifyResult.IsOk())
		{
			UE_LOGF(LogIoStoreOnDemand, Log, "Install cache verified OK!");
		}
		else
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "Verify install cache failed, reason '%ls'",
				*LexToString(VerifyResult.Error.GetValue()));
		}
	});
}

////////////////////////////////////////////////////////////////////////////////
static FAutoConsoleCommand PurgeCacheCommand(
	TEXT("iostore.PurgeInstallCache"),
	TEXT("Purge On Demand Install Cache"),
	FConsoleCommandWithArgsDelegate::CreateStatic(PurgeInstallCache),
	ECVF_Cheat);

static FAutoConsoleCommand DefragCacheCommand(
	TEXT("iostore.DefragInstallCache"),
	TEXT("Defragment On Demand Install Cache"),
	FConsoleCommandWithArgsDelegate::CreateStatic(DefragInstallCache),
	ECVF_Cheat);

static FAutoConsoleCommand PrintCacheUsageCommand(
	TEXT("iostore.CacheUsage"),
	TEXT("print cache usage"),
	FConsoleCommandWithArgsDelegate::CreateStatic(PrintCacheUsage),
	ECVF_Cheat);

static FAutoConsoleCommand MountUrlCommand(
	TEXT("iostore.MountUrl"),
	TEXT("<URL> <Install|Stream> <MountId>"),
	FConsoleCommandWithArgsDelegate::CreateStatic(MountUrl));

static FAutoConsoleCommand MountFileCommand(
	TEXT("iostore.MountFile"),
	TEXT("<Filename|Wildcard> <Install|Stream> <HostPath> <MountId>"),
	FConsoleCommandWithArgsDelegate::CreateStatic(MountFile));

static FAutoConsoleCommand InstallPackageCommand(
	TEXT("iostore.InstallPackage"),
	TEXT("<PackageName>"),
	FConsoleCommandWithArgsDelegate::CreateStatic(InstallPackage));

static FAutoConsoleCommand VerifyCacheCommand(
	TEXT("iostore.VerifyCache"),
	TEXT("Verifies the cache chunks against the mounted container TOCs"),
	FConsoleCommandDelegate::CreateStatic(VerifyCache),
	ECVF_Cheat);

} // namespace Commands
#endif // !UE_BUILD_SHIPPING

static const TCHAR* NotInitializedError = TEXT("I/O store on-demand not initialized");

////////////////////////////////////////////////////////////////////////////////
FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Sb, const FOnDemandInstallCacheUsage::FCasUsage& CasUsage)
{
	if (!CasUsage.IsValid())
	{
		return Sb;
	}

	Sb.Appendf("CAS=%ls, MaxSize=%.2lf MiB, TotalSize=%.2lf MiB, ReferencedBlockSize=%.2lf MiB, ReferencedSize=%.2lf MiB, FragmentedChunksSize=%.2lf MiB",
		LexToString(CasUsage.CasType),
		double(CasUsage.MaxSize) / 1024.0 / 1024.0,
		double(CasUsage.TotalSize) / 1024.0 / 1024.0,
		double(CasUsage.ReferencedBlockSize) / 1024.0 / 1024.0,
		double(CasUsage.ReferencedSize) / 1024.0 / 1024.0,
		double(CasUsage.FragmentedChunksSize) / 1024.0 / 1024.0);

	return Sb;
}

FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Sb, const FOnDemandInstallCacheUsage& CacheUsage)
{
	for (const FOnDemandInstallCacheUsage::FCasUsage& CasUsage : CacheUsage.CasUsage)
	{
		Sb << CasUsage << TEXTVIEW("\n");
	}

	return Sb;
}

////////////////////////////////////////////////////////////////////////////////
FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Sb, const FOnDemandStreamingCacheUsage& CacheUsage)
{
	Sb.Appendf("MaxSize=%.2lf MiB, TotalSize=%.2lf MiB",
		double(CacheUsage.MaxSize) / 1024.0 / 1024.0, double(CacheUsage.TotalSize) / 1024.0 / 1024.0);
	return Sb;
}

////////////////////////////////////////////////////////////////////////////////
class FIoStoreOnDemandCoreModule final
	: public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void HandleModularFeatureRegistered(const FName& Type, IModularFeature* ModularFeature);
};
IMPLEMENT_MODULE(UE::IoStore::FIoStoreOnDemandCoreModule, IoStoreOnDemandCore);

////////////////////////////////////////////////////////////////////////////////
void FIoStoreOnDemandCoreModule::HandleModularFeatureRegistered(const FName& Type, IModularFeature* ModularFeature)
{
	if (Type != IOnDemandIoStoreFactory::FeatureName || GOnDemandIoStore != nullptr)
	{
		return;
	}

	IOnDemandIoStoreFactory* Factory	= static_cast<IOnDemandIoStoreFactory*>(ModularFeature);
	IOnDemandIoStore* IoStore			= Factory->CreateInstance();

	if (IoStore == nullptr)
	{
		UE_LOGF(LogIoStoreOnDemand, Warning, "I/O store on-demand disabled, reason '%ls'", TEXT("Failed to create I/O store"));
		return;
	}

	if (FIoStatus Status = IoStore->Initialize(); !Status.IsOk())
	{
		Factory->DestroyInstance(IoStore);

		if (Status.GetErrorCode() == EIoErrorCode::Disabled || Status.GetErrorCode() == EIoErrorCode::NotFound)
		{
			UE_LOGF(LogIoStoreOnDemand, Log, "I/O store on-demand disabled, reason '%ls'", *Status.ToString());
		}
		else
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "I/O store on-demand disabled, reason '%ls'", *Status.ToString());
		}
		return;
	}
	#if !UE_IAS_CUSTOM_INITIALIZATION
	if (FIoStatus Status = IoStore->InitializePostHotfix(); !Status.IsOk())
	{
		if (Status.GetErrorCode() == EIoErrorCode::Disabled || Status.GetErrorCode() == EIoErrorCode::NotFound)
		{
			UE_LOGF(LogIoStoreOnDemand, Log, "I/O store post hotfix initialization failed, reason '%ls'", *Status.ToString());
		}
		else
		{
			UE_LOGF(LogIoStoreOnDemand, Error, "I/O store post hotfix initialization failed, reason '%ls'", *Status.ToString());
		}
	}
	#endif
	GOnDemandIoStore = IoStore;
	IModularFeatures::Get().OnModularFeatureRegistered().RemoveAll(this);
}

void FIoStoreOnDemandCoreModule::StartupModule()
{
	IModularFeatures& Features = IModularFeatures::Get();
	if (Features.GetModularFeatureImplementationCount(IOnDemandIoStoreFactory::FeatureName) > 0)
	{
		IModularFeature* Feature = Features.GetModularFeatureImplementation(IOnDemandIoStoreFactory::FeatureName, 0);
		HandleModularFeatureRegistered(IOnDemandIoStoreFactory::FeatureName, Feature);
	}
	else
	{
		Features.OnModularFeatureRegistered().AddRaw(this, &FIoStoreOnDemandCoreModule::HandleModularFeatureRegistered);
	}
}

void FIoStoreOnDemandCoreModule::ShutdownModule()
{
	IOnDemandIoStore* ToDestroy = nullptr;
	Swap(ToDestroy, GOnDemandIoStore);

	if (ToDestroy == nullptr)
	{
		return;
	}

	IModularFeatures& Features = IModularFeatures::Get();
	if (Features.GetModularFeatureImplementationCount(IOnDemandIoStoreFactory::FeatureName) > 0)
	{
		IModularFeature* Feature			= Features.GetModularFeatureImplementation(IOnDemandIoStoreFactory::FeatureName, 0);
		IOnDemandIoStoreFactory* Factory	= static_cast<IOnDemandIoStoreFactory*>(Feature);

		Factory->DestroyInstance(ToDestroy);
	}
	IModularFeatures::Get().OnModularFeatureRegistered().RemoveAll(this);
}

////////////////////////////////////////////////////////////////////////////////
IOnDemandIoStore* TryGetOnDemandIoStore()
{
	return GOnDemandIoStore;
}

////////////////////////////////////////////////////////////////////////////////
IOnDemandIoStore& GetOnDemandIoStore()
{
	check(GOnDemandIoStore != nullptr);
	return *GOnDemandIoStore;
}

FName IOnDemandIoStoreFactory::FeatureName = FName("OnDemandIoStoreFactory");

////////////////////////////////////////////////////////////////////////////////
TIoStatusOr<FOnDemandIsOnDemandResult> IOnDemandIoStore::GetIsOnDemand(const FOnDemandIsOnDemandArgs& Args) const
{
	FOnDemandIsOnDemandResult Result;
	FIoStatus Status = GetIsOnDemand(MakeConstArrayView(&Args, 1), MakeArrayView(&Result, 1));
	if (Status.IsOk())
	{
		return TIoStatusOr<FOnDemandIsOnDemandResult>(MoveTemp(Result));
	}

	return Status;
}

} // namespace UE::IoStore

////////////////////////////////////////////////////////////////////////////////
