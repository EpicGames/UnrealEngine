// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/IoStoreOnDemandInternals.h"

#include "Logging/StructuredLog.h"

namespace UE::IoStore
{

////////////////////////////////////////////////////////////////////////////////
void FOnDemandMountResult::LogResult()
{
	if (!Status.IsOk())
	{
		UE_CLOGFMT(Status.GetErrorCode() != EIoErrorCode::PendingHostGroup, LogIas, Error, "Failed to mount TOC for '{0}', reason '{1}'", MountId, Status);
		UE_CLOGFMT(Status.GetErrorCode() == EIoErrorCode::PendingHostGroup, LogIas, Log, "Failed to mount TOC for '{0}', reason '{1}'", MountId, Status);
	}
}

///////////////////////////////////////////////////////////////////////////////
FOnDemandWeakContentHandle FOnDemandWeakContentHandle::FromUnsafeHandle(UPTRINT HandleId)
{
	if (HandleId == 0)
	{
		return FOnDemandWeakContentHandle{};
	}

	FOnDemandInternalContentHandle& ContentHandle = *reinterpret_cast<FOnDemandInternalContentHandle*>(HandleId);
	return FOnDemandWeakContentHandle
	{
		.HandleId	= HandleId,
		.DebugName	= ContentHandle.DebugName
	};
}

///////////////////////////////////////////////////////////////////////////////
FOnDemandInternalContentHandle::FOnDemandInternalContentHandle()
	: FOnDemandInternalContentHandle(FSharedString(TEXT("NoName")))
{
}

FOnDemandInternalContentHandle::FOnDemandInternalContentHandle(FSharedString InDebugName)
	: DebugName(MoveTemp(InDebugName))
{
}

FOnDemandInternalContentHandle::~FOnDemandInternalContentHandle()
{
	if (TSharedPtr<IOnDemandIoStore, ESPMode::ThreadSafe> Pinned = IoStore.Pin(); Pinned.IsValid())
	{
		Pinned->ReleaseContent(*this);
	}
}

///////////////////////////////////////////////////////////////////////////////
FOnDemandContentHandle FOnDemandContentHandle::Create()
{
	FOnDemandContentHandle NewHandle;
	NewHandle.Handle = MakeShared<FOnDemandInternalContentHandle, ESPMode::ThreadSafe>();
	return NewHandle;
}

FOnDemandContentHandle FOnDemandContentHandle::Create(FSharedString DebugName)
{
	FOnDemandContentHandle NewHandle;
	NewHandle.Handle = MakeShared<FOnDemandInternalContentHandle, ESPMode::ThreadSafe>(DebugName);
	return NewHandle;
}

FOnDemandContentHandle FOnDemandContentHandle::Create(FStringView DebugName)
{
	return Create(FSharedString(DebugName));
}

FString LexToString(const FOnDemandContentHandle& Handle)
{
	return Handle.IsValid() ? ::LexToString(*Handle.Handle) : TEXT("Invalid");
}

///////////////////////////////////////////////////////////////////////////////
FOnDemandInstallRequest::EStatus FOnDemandInstallRequest::GetStatus() const
{
	return Request.IsValid() ? Request->Status.load(std::memory_order_relaxed) : EStatus::None; 
}

void FOnDemandInstallRequest::Cancel()
{
	if (IsPending())
	{
		if (TSharedPtr<IOnDemandIoStore, ESPMode::ThreadSafe> Pinned = Request->IoStore.Pin(); Pinned.IsValid())
		{
			Pinned->CancelInstallRequest(Request);
		}
	}
}

void FOnDemandInstallRequest::UpdatePriority(int32 NewPriority)
{
	if (IsPending())
	{
		if (TSharedPtr<IOnDemandIoStore, ESPMode::ThreadSafe> Pinned = Request->IoStore.Pin(); Pinned.IsValid())
		{
			Pinned->UpdateInstallRequestPriority(Request, NewPriority);
		}
	}
}

} // namespace UE::IoStore

////////////////////////////////////////////////////////////////////////////////
FString LexToString(const UE::IoStore::FOnDemandInternalContentHandle& Handle)
{
	return FString::Printf(TEXT("Id=0x%" UPTRINT_X_FMT " DebugName='%s'"), Handle.HandleId(), *Handle.DebugName);
}
