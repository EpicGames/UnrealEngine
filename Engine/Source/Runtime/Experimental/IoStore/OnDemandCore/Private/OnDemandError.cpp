// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/OnDemandError.h"
#include "IO/IoDispatcherInternal.h"

UE_DEFINE_ERROR_MODULE(UE::IoStore::OnDemand);
UE_DEFINE_ERROR_CONTEXT(UE::IoStore::OnDemand, InstallCacheErrorContext);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, HttpError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, ChunkMissingError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, ChunkHashError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, InstallCacheFlushError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, InstallCacheFlushLastAccessError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, InstallCachePurgeError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, InstallCacheDefragError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, InstallCacheVerificationError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, CasError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, CasJournalError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, CasSnapshotError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, InstallerCacheFullError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, InstallReplayLoadError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, InstallReplaySaveError);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, DownloadRequired);
UE_DEFINE_ERROR(UE::IoStore::OnDemand, InstallCacheDefragHashMismatchError);

////////////////////////////////////////////////////////////////////////////////
// Global-scope SerializeForLog overloads. Placed here so ADL on each argument
// type resolves them from within UE::CallSerializeForLog (which the macro-
// generated error-payload serializer invokes for each field).

void SerializeForLog(FCbWriter& Writer, const FIoChunkId& ChunkId)
{
	TUtf8StringBuilder<256> Sb;
	FUtf8StringView ContainerName;
	FUtf8StringView Filename = FIoDispatcherInternal::GetFilename(ChunkId, Sb, ContainerName);

	Writer.BeginObject();
	Writer.AddString(ANSITEXTVIEW("$type"), ANSITEXTVIEW("FIoChunkId"));
	if (Filename.IsEmpty())
	{
		Writer.AddString(ANSITEXTVIEW("$format"), TEXT("{ChunkId}"));
	}
	else
	{
		Writer.AddString(ANSITEXTVIEW("$format"), TEXT("({Filename}:{ContainerName}:{ChunkId})"));
		Writer.AddString(ANSITEXTVIEW("Filename"), Filename);
		Writer.AddString(ANSITEXTVIEW("ContainerName"), ContainerName);
	}
	Writer.AddString(ANSITEXTVIEW("ChunkId"), LexToString(ChunkId));
	Writer.EndObject();
}

void SerializeForLog(FCbWriter& Writer, EIoErrorCode ErrorCode)
{
	Writer.AddString(GetIoErrorText(ErrorCode));
}

void SerializeForLog(FCbWriter& Writer, const FFileSystemError& Error)
{
	TUtf8StringBuilder<512> Sb;
	Sb << Error;
	Writer.AddString(FUtf8StringView(Sb));
}

namespace UE::IoStore
{
	void SerializeForLog(FCbWriter& Writer, EOnDemandInstallCasType CasType)
	{
		Writer.AddString(::LexToString(CasType));
	}
}

namespace UE::IoStore::OnDemand
{
	// Mirrors the pre-refactor InstallCacheErrorContext serialization: the IoStatus
	// is decomposed into ErrorCode/ErrorMessage/SystemErrorCode rather than written
	// as a single sub-object, and the format string surfaces ErrorMessage so the
	// rendered log line stays a one-liner.
	void SerializeForLog(FCbWriter& Writer, const FInstallCacheErrorState& State)
	{
		Writer.BeginObject();
		Writer.AddString(UTF8TEXTVIEW("$format"), TEXT("({ErrorMessage})"));
		if (State.CasType != nullptr)
		{
			Writer.AddString(UTF8TEXTVIEW("CasType"), State.CasType);
		}
		Writer.AddInteger(UTF8TEXTVIEW("ErrorCode"), int32(State.IoError.GetErrorCode()));
		Writer.AddString(UTF8TEXTVIEW("ErrorMessage"), State.IoError.ToString());
		Writer.AddInteger(UTF8TEXTVIEW("SystemErrorCode"), int32(State.IoError.GetSystemErrorCode()));
		Writer.AddInteger(UTF8TEXTVIEW("MaxCacheSize"), State.MaxCacheSize);
		Writer.AddInteger(UTF8TEXTVIEW("CacheSize"), State.CacheSize);
		Writer.AddInteger(UTF8TEXTVIEW("RequestedSize"), State.RequestedSize);
		Writer.AddInteger(UTF8TEXTVIEW("DiskTotalBytes"), State.DiskTotalBytes);
		Writer.AddInteger(UTF8TEXTVIEW("DiskFreeBytes"), State.DiskFreeBytes);
		Writer.AddInteger(UTF8TEXTVIEW("LineNo"), uint32(State.LineNo));
		Writer.EndObject();
	}
}
