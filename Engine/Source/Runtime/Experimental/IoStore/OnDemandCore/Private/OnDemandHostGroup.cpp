// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/OnDemandHostGroup.h"

#include "Containers/AnsiString.h"
#include "Containers/Array.h"
#include "Containers/StringView.h"
#include "HAL/IConsoleManager.h"

#if !UE_BUILD_SHIPPING
#include "IO/IoStoreOnDemand.h"
#include "Internationalization/Regex.h"
#include "Logging/StructuredLog.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Parse.h"
#endif // !UE_BUILD_SHIPPING

namespace UE::IoStore
{

namespace Private
{

static bool ValidateUrl(FAnsiStringView Url, FString& Reason)
{
	//TODO: Add better validation
	return Url.StartsWith("http") || Url.StartsWith("https");
}

#if !UE_BUILD_SHIPPING

static bool ShouldSkipUrl(const FString& Url)
{
	struct FDevCDNCheck
	{
		FDevCDNCheck()
		{
			if (FParse::Param(FCommandLine::Get(), TEXT("Iax.EnableDevCDNs")))
			{
				return;
			}

			FString ConfigRegex;
			GConfig->GetString(TEXT("Ias"), TEXT("DevelopmentCDNPattern"), ConfigRegex, GEngineIni);

			if (!ConfigRegex.IsEmpty())
			{
				Pattern = MakeUnique<FRegexPattern>(MoveTemp(ConfigRegex));
			}
		}

		bool IsDevCNDUrl(const FString& Url) const
		{
			if (Pattern == nullptr)
			{
				return false;
			}
			else
			{
				FRegexMatcher Regex = FRegexMatcher(*Pattern, Url);
				return Regex.FindNext();
			}
		}
	private:

		TUniquePtr<FRegexPattern> Pattern;
	} static DevCDNCheck;

	return DevCDNCheck.IsDevCNDUrl(Url);
}

static bool ShouldSkipUrl(const FStringView& Url)
{
	return ShouldSkipUrl(FString(Url));
}

static bool ShouldSkipUrl(FAnsiStringView Url)
{
	return ShouldSkipUrl(FString(Url));
}

#endif // !UE_BUILD_SHIPPING

} // namespace Private

struct FOnDemandHostGroup::FImpl
{
	TArray<FAnsiString> HostUrls;
	int32				PrimaryIndex = INDEX_NONE;
};

FOnDemandHostGroup::FOnDemandHostGroup()
	: Impl(MakeShared<FImpl>())
{
}

FOnDemandHostGroup::FOnDemandHostGroup(FOnDemandHostGroup::FSharedImpl&& InImpl)
	: Impl(MoveTemp(InImpl))
{
}

FOnDemandHostGroup::~FOnDemandHostGroup()
{
}

TConstArrayView<FAnsiString> FOnDemandHostGroup::Hosts() const
{
	return Impl->HostUrls;
}

FAnsiStringView FOnDemandHostGroup::Host(int32 Index) const
{
	if (Index < 0 || Index >= Impl->HostUrls.Num())
	{
		return FAnsiStringView();
	}
	else
	{
		return Impl->HostUrls[Index];
	}
}

FAnsiStringView FOnDemandHostGroup::CycleHost(int32& InOutIndex) const
{
	if (!Impl->HostUrls.IsEmpty())
	{
		InOutIndex = (InOutIndex + 1) % Impl->HostUrls.Num();
		return Host(InOutIndex);
	}
	else
	{
		InOutIndex = INDEX_NONE;
		return FAnsiStringView();
	}
}

void FOnDemandHostGroup::SetPrimaryHost(int32 Index)
{
	check(Index >= 0 && Index < Impl->HostUrls.Num() || Index == INDEX_NONE);
	Impl->PrimaryIndex = Index;
}

FAnsiStringView FOnDemandHostGroup::PrimaryHost() const
{
	if (Impl->PrimaryIndex != INDEX_NONE)
	{
		check(Impl->PrimaryIndex >= 0 && Impl->PrimaryIndex < Impl->HostUrls.Num());
		return Impl->HostUrls[Impl->PrimaryIndex];
	}

	return FAnsiStringView();
}

int32 FOnDemandHostGroup::PrimaryHostIndex() const
{
	return Impl->PrimaryIndex;
}

bool FOnDemandHostGroup::IsEmpty() const
{
	return Impl->HostUrls.IsEmpty();
}

TIoStatusOr<FOnDemandHostGroup> FOnDemandHostGroup::Create(FStringView Url)
{
	FAnsiString AnsiUrl(StringCast<ANSICHAR>(Url.GetData(), Url.Len()));
	return Create(MakeConstArrayView(&AnsiUrl, 1));
}

TIoStatusOr<FOnDemandHostGroup> FOnDemandHostGroup::Create(TConstArrayView<FAnsiString> Urls)
{
	if (Urls.IsEmpty())
	{
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("No URLs specified"));
	}

	FSharedImpl Impl = MakeShared<FImpl>();
	FString		Reason;

	for( int32 Index = 0; Index < Urls.Num(); ++Index)
	{
		FAnsiStringView Url = Urls[Index];

		if (Url.EndsWith('/'))
		{
			Url.RemoveSuffix(1);
		}

		if (Private::ValidateUrl(Url, Reason) == false)
		{
			return FIoStatus(EIoErrorCode::InvalidParameter, Reason);
		}

#if !UE_BUILD_SHIPPING
		if (Private::ShouldSkipUrl(Url))
		{
			if (!Impl->HostUrls.IsEmpty() || Urls.Num() - Index > 1)
			{
				UE_LOGFMT(LogIas, Log, "Skipping development CDN '{Url}' when creating HostGroup", Url);
				continue;
			}
			else
			{
				UE_LOGFMT(LogIas, Warning, "Cannot skip development CDN '{Url}' as it is the last possible url to create the HostGroup", Url);
			}
		}
#endif // !UE_BUILD_SHIPPING

		Impl->HostUrls.Emplace(Url);
	}

	if (Impl->HostUrls.IsEmpty())
	{
		// This branch should not be possible but leaving it just in case
		Impl->PrimaryIndex = -1;
		return FIoStatus(EIoErrorCode::InvalidParameter, TEXT("Alls URLs were filtered out"));
	}
	
	Impl->PrimaryIndex = 0;
	return FOnDemandHostGroup(MoveTemp(Impl));
}

TIoStatusOr<FOnDemandHostGroup> FOnDemandHostGroup::Create(TConstArrayView<FString> Urls)
{
	TArray<FAnsiString> ConvertedUrls;
	for (const FString& Url : Urls)
	{
		ConvertedUrls.Emplace(FAnsiString(Url));
	}

	return Create(ConvertedUrls);
}

FName FOnDemandHostGroup::DefaultName = FName("Default");

} //namespace UE::IoStore
