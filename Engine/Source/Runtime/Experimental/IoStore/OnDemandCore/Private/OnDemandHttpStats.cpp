// Copyright Epic Games, Inc. All Rights Reserved.

#include <IO/IoStoreOnDemand.h>
#include <IO/IoStoreOnDemandInternals.h>

namespace UE
{

namespace IoStore
{

FOnDemandHttpStats::FOnDemandHttpStats() = default;
FOnDemandHttpStats::~FOnDemandHttpStats() = default;

uint32 FOnDemandHttpStats::GetRecvKiBps() const
{
	return Internal.IsValid() ? Internal->GetRecvKiBps() : 0;
}

void FOnDemandHttpStats::GetRecvKiBps(uint32 (&Out)[SAMPLE_COUNT]) const
{
	if (Internal.IsValid())
	{
		Internal->GetRecvKiBps(Out);
	}
}

uint32 FOnDemandHttpStats::GetTotalRecvKiB() const
{
	return Internal.IsValid() ? Internal->GetTotalRecvKiB() : 0;
}

uint32 FOnDemandHttpStats::GetTimeToFirstByteMs() const
{
	return Internal.IsValid() ? Internal->GetTimeToFirstByteMs() : 0;
}

} // namespace IoStore
} // namespace UE
