// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/OnDemandToc.h"

#include "HAL/FileManager.h"
#include "HAL/FileManagerGeneric.h"
#include "Memory/SharedBuffer.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Paths.h"
#include "Misc/PathViews.h"
#include "Misc/StringBuilder.h"
#include "Serialization/Archive.h"
#include "Serialization/CompactBinarySerialization.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/CustomVersion.h"

namespace UE::IoStore
{

////////////////////////////////////////////////////////////////////////////////
// 
// Utility to create a FArchive capable of reading from disk using the exact same pathing
// rules as FPlatformMisc::LoadTextFileFromPlatformPackage but without forcing the entire
// file to be loaded at once.
// 
static TUniquePtr<FArchive> CreateReaderFromPlatformPackage(const FString& RelPath)
{
#if PLATFORM_IOS
    // IOS OpenRead assumes it is in cookeddata, using ~ for the base path tells it to use the package base path instead
    const FString AbsPath = FPaths::Combine(TEXT("~"), RelPath);
#else
    const FString AbsPath = FPaths::Combine(FGenericPlatformMisc::RootDir(), RelPath);
#endif
	if (TUniquePtr<IFileHandle> File(IPlatformFile::GetPlatformPhysical().OpenRead(*AbsPath)); File.IsValid())
	{
#if PLATFORM_ANDROID
		// This is a handle to an asset so we need to call Seek(0) to move the internal
		// offset to the start of the asset file.
		File->Seek(0);
#endif //PLATFORM_ANDROID
		const uint32 ReadBufferSize = 256 * 1024;
		const int64 FileSize = File->Size();
		return MakeUnique<FArchiveFileReaderGeneric>(File.Release(), *AbsPath, FileSize, ReadBufferSize);
	}

	return TUniquePtr<FArchive>();
}

////////////////////////////////////////////////////////////////////////////////
FArchive& operator<<(FArchive& Ar, FTocMeta& Meta)
{
	Ar << Meta.EpochTimestamp;
	Ar << Meta.BuildVersion;
	Ar << Meta.TargetPlatform;
	return Ar;
}

FCbWriter& operator<<(FCbWriter& Writer, const FTocMeta& Meta)
{
	Writer.BeginObject();
	Writer.AddInteger(UTF8TEXTVIEW("EpochTimestamp"), Meta.EpochTimestamp);
	Writer.AddString(UTF8TEXTVIEW("BuildVersion"), Meta.BuildVersion);
	Writer.AddString(UTF8TEXTVIEW("TargetPlatform"), Meta.TargetPlatform);
	Writer.EndObject();

	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FTocMeta& OutMeta)
{
	if (FCbObjectView Obj = Field.AsObjectView())
	{
		OutMeta.EpochTimestamp = Obj["EpochTimestamp"].AsInt64();
		OutMeta.BuildVersion = FString(Obj["BuildVersion"].AsString());
		OutMeta.TargetPlatform = FString(Obj["TargetPlatform"].AsString());
		return true;
	}
	
	return false;
}

////////////////////////////////////////////////////////////////////////////////
FArchive& operator<<(FArchive& Ar, FOnDemandTocHeader& Header)
{
	if (Ar.IsLoading() && Ar.TotalSize() < sizeof(FOnDemandTocHeader))
	{
		Ar.SetError();
		return Ar;
	}

	Ar << Header.Magic;
	if (Header.Magic != FOnDemandTocHeader::ExpectedMagic)
	{
		Ar.SetError();
		return Ar;
	}

	Ar << Header.Version;
	if (static_cast<EOnDemandTocVersion>(Header.Version) == EOnDemandTocVersion::Invalid)
	{
		Ar.SetError();
		return Ar;
	}

	if (uint32(Header.Version) > uint32(EOnDemandTocVersion::Latest))
	{
		Ar.SetError();
		return Ar;
	}

	Ar << Header.Flags;
	Ar << Header.BlockSize;
	Ar << Header.CompressionFormat;
	Ar << Header.ChunksDirectory;
	
	if (Ar.IsSaving() || (Header.Version >= static_cast<uint32>(EOnDemandTocVersion::HostGroupName)))
	{
		Ar << Header.HostGroupName;
	}

	if (Ar.IsLoading() && (Header.Version < static_cast<uint32>(EOnDemandTocVersion::TocFlags)))
	{
		Header.Flags = 0; 
	}

	return Ar;
}

FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandTocHeader& Header)
{
	Writer.BeginObject();
	Writer.AddInteger(UTF8TEXTVIEW("Magic"), Header.Magic);
	Writer.AddInteger(UTF8TEXTVIEW("Version"), Header.Version);
	Writer.AddInteger(UTF8TEXTVIEW("Flags"), Header.Flags);
	Writer.AddInteger(UTF8TEXTVIEW("BlockSize"), Header.BlockSize);
	Writer.AddString(UTF8TEXTVIEW("CompressionFormat"), Header.CompressionFormat);
	Writer.AddString(UTF8TEXTVIEW("ChunksDirectory"), Header.ChunksDirectory);
	Writer.AddString(UTF8TEXTVIEW("HostGroupName"), Header.HostGroupName);
	Writer.EndObject();

	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandTocHeader& OutTocHeader)
{
	if (FCbObjectView Obj = Field.AsObjectView())
	{
		OutTocHeader.Magic = Obj["Magic"].AsUInt64();
		OutTocHeader.Version = Obj["Version"].AsUInt32();
		OutTocHeader.Flags = Obj["Flags"].AsUInt32();
		OutTocHeader.BlockSize = Obj["BlockSize"].AsUInt32();
		OutTocHeader.CompressionFormat = FString(Obj["CompressionFormat"].AsString());
		OutTocHeader.ChunksDirectory = FString(Obj["ChunksDirectory"].AsString());
		OutTocHeader.HostGroupName = FString(Obj["HostGroupName"].AsString());

		return OutTocHeader.Magic == FOnDemandTocHeader::ExpectedMagic &&
			static_cast<EOnDemandTocVersion>(OutTocHeader.Version) != EOnDemandTocVersion::Invalid;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
FArchive& operator<<(FArchive& Ar, FOnDemandTocEntry& Entry)
{
	Ar << Entry.Hash;
	Ar << Entry.ChunkId;
	Ar << Entry.RawSize;
	Ar << Entry.EncodedSize;
	Ar << Entry.BlockOffset;
	Ar << Entry.BlockCount;

	return Ar;
}

FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandTocEntry& Entry)
{
	Writer.BeginObject();
	Writer.AddHash(UTF8TEXTVIEW("Hash"), Entry.Hash);
	Writer << UTF8TEXTVIEW("ChunkId") << Entry.ChunkId;
	Writer.AddInteger(UTF8TEXTVIEW("RawSize"), Entry.RawSize);
	Writer.AddInteger(UTF8TEXTVIEW("EncodedSize"), Entry.EncodedSize);
	Writer.AddInteger(UTF8TEXTVIEW("BlockOffset"), Entry.BlockOffset);
	Writer.AddInteger(UTF8TEXTVIEW("BlockCount"), Entry.BlockCount);
	Writer.EndObject();

	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandTocEntry& OutTocEntry)
{
	if (FCbObjectView Obj = Field.AsObjectView())
	{
		if (!LoadFromCompactBinary(Obj["ChunkId"], OutTocEntry.ChunkId))
		{
			return false;
		}

		OutTocEntry.Hash = Obj["Hash"].AsHash();
		OutTocEntry.RawSize = Obj["RawSize"].AsUInt64(~uint64(0));
		OutTocEntry.EncodedSize = Obj["EncodedSize"].AsUInt64(~uint64(0));
		OutTocEntry.BlockOffset = Obj["BlockOffset"].AsUInt32(~uint32(0));
		OutTocEntry.BlockCount = Obj["BlockCount"].AsUInt32();

		return OutTocEntry.Hash != FIoHash::Zero &&
			OutTocEntry.RawSize != ~uint64(0) &&
			OutTocEntry.EncodedSize != ~uint64(0) &&
			OutTocEntry.BlockOffset != ~uint32(0);
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
FArchive& operator<<(FArchive& Ar, FOnDemandTocContainerEntry& ContainerEntry)
{
	EOnDemandTocVersion TocVersion = EOnDemandTocVersion::Latest;

	if (Ar.IsLoading())
	{
		const FCustomVersion* CustomVersion = Ar.GetCustomVersions().GetVersion(FOnDemandToc::VersionGuid);
		check(CustomVersion);
		TocVersion = static_cast<EOnDemandTocVersion>(CustomVersion->Version);

		if (TocVersion >= EOnDemandTocVersion::ContainerId)
		{
			Ar << ContainerEntry.ContainerId;
		}
	}
	else
	{
		Ar << ContainerEntry.ContainerId;
	}

	Ar << ContainerEntry.ContainerName;
	Ar << ContainerEntry.EncryptionKeyGuid;
	Ar << ContainerEntry.Entries;
	Ar << ContainerEntry.BlockSizes;
	Ar << ContainerEntry.BlockHashes;
	Ar << ContainerEntry.UTocHash;

	if (!Ar.IsLoading() || (TocVersion >= EOnDemandTocVersion::ContainerFlags))
	{
		Ar << ContainerEntry.ContainerFlags;
	}

	if (!Ar.IsLoading() || (TocVersion >= EOnDemandTocVersion::ContainerHeader))
	{
		Ar << ContainerEntry.Header;
	}

	return Ar;
}

FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandTocContainerEntry& ContainerEntry)
{
	Writer.BeginObject();
	Writer << UTF8TEXTVIEW("Id") << ContainerEntry.ContainerId;
	Writer.AddString(UTF8TEXTVIEW("Name"), ContainerEntry.ContainerName);
	Writer.AddString(UTF8TEXTVIEW("EncryptionKeyGuid"), ContainerEntry.EncryptionKeyGuid);

	Writer.BeginArray(UTF8TEXTVIEW("Entries"));
	for (const FOnDemandTocEntry& Entry : ContainerEntry.Entries)
	{
		Writer << Entry;
	}
	Writer.EndArray();
	
	Writer.BeginArray(UTF8TEXTVIEW("BlockSizes"));
	for (uint32 BlockSize : ContainerEntry.BlockSizes)
	{
		Writer << BlockSize;
	}
	Writer.EndArray();

	Writer.BeginArray(UTF8TEXTVIEW("BlockHashes"));
	for (uint32 BlockHash : ContainerEntry.BlockHashes)
	{
		Writer << BlockHash;
	}
	Writer.EndArray();

	if (!ContainerEntry.Header.IsEmpty())
	{
		Writer.AddBinary(UTF8TEXTVIEW("Header"), ContainerEntry.Header.GetData(), ContainerEntry.Header.Num());
	}

	Writer.AddHash(UTF8TEXTVIEW("UTocHash"), ContainerEntry.UTocHash);
	Writer.AddInteger(UTF8TEXTVIEW("ContainerFlags"), ContainerEntry.ContainerFlags);

	Writer.EndObject();

	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandTocContainerEntry& OutContainer)
{
	if (FCbObjectView Obj = Field.AsObjectView())
	{
		if (FCbFieldView Id = Obj[UTF8TEXTVIEW("Id")]; Id)
		{
			LoadFromCompactBinary(Id, OutContainer.ContainerId);
		}

		OutContainer.ContainerName = FString(Obj[UTF8TEXTVIEW("Name")].AsString());
		OutContainer.EncryptionKeyGuid = FString(Obj[UTF8TEXTVIEW("EncryptionKeyGuid")].AsString());

		FCbArrayView Entries = Obj[UTF8TEXTVIEW("Entries")].AsArrayView();
		OutContainer.Entries.Reserve(int32(Entries.Num()));
		for (FCbFieldView ArrayField : Entries)
		{
			if (!LoadFromCompactBinary(ArrayField, OutContainer.Entries.AddDefaulted_GetRef()))
			{
				return false;
			}
		}

		FCbArrayView BlockSizes = Obj[UTF8TEXTVIEW("BlockSizes")].AsArrayView();
		OutContainer.BlockSizes.Reserve(int32(BlockSizes.Num()));
		for (FCbFieldView ArrayField : BlockSizes)
		{
			OutContainer.BlockSizes.Add(ArrayField.AsUInt32());
		}

		FCbArrayView BlockHashes = Obj[UTF8TEXTVIEW("BlockHashes")].AsArrayView();
		OutContainer.BlockHashes.Reserve(int32(BlockHashes.Num()));
		for (FCbFieldView ArrayField : BlockHashes)
		{
			if (ArrayField.IsHash())
			{
				const FIoHash BlockHash = ArrayField.AsHash();
				OutContainer.BlockHashes.Add(*reinterpret_cast<const uint32*>(&BlockHash));
			}
			else
			{
				OutContainer.BlockHashes.Add(ArrayField.AsUInt32());
			}
		}

		FMemoryView Header = Obj[UTF8TEXTVIEW("Header")].AsBinaryView();
		if (!Header.IsEmpty())
		{
			OutContainer.Header = MakeArrayView<const uint8>(
				reinterpret_cast<const uint8*>(Header.GetData()),
				IntCastChecked<int32>(Header.GetSize()));
		}

		OutContainer.UTocHash = Obj[UTF8TEXTVIEW("UTocHash")].AsHash();

		if (FCbFieldView Flags = Obj[UTF8TEXTVIEW("ContainerFlags")]; Flags)
		{
			OutContainer.ContainerFlags = Flags.AsInt8();
		}

		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
bool FOnDemandTocSentinel::IsValid()
{
	return FMemory::Memcmp(&Data, FOnDemandTocSentinel::SentinelImg, FOnDemandTocSentinel::SentinelSize) == 0;
}

FArchive& operator<<(FArchive& Ar, FOnDemandTocSentinel& Sentinel)
{
	if (Ar.IsSaving())
	{	
		// We could just cast FOnDemandTocSentinel::SentinelImg to a non-const pointer but we can't be 
		// 100% sure that the FArchive won't change the data, even if it is in Saving mode. Since this 
		// isn't performance critical we will play it safe.
		uint8 Output[FOnDemandTocSentinel::SentinelSize];
		FMemory::Memcpy(Output, FOnDemandTocSentinel::SentinelImg, FOnDemandTocSentinel::SentinelSize);

		Ar.Serialize(&Output, FOnDemandTocSentinel::SentinelSize);
	}
	else
	{
		Ar.Serialize(&Sentinel.Data, FOnDemandTocSentinel::SentinelSize);
	}

	return Ar;
}

////////////////////////////////////////////////////////////////////////////////
FArchive& operator<<(FArchive& Ar, FOnDemandTocAdditionalFile& AdditionalFile)
{
	Ar << AdditionalFile.Hash;
	Ar << AdditionalFile.Filename;
	Ar << AdditionalFile.FileSize;
	return Ar;
}

FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandTocAdditionalFile& AdditionalFile)
{
	Writer.BeginObject();
	Writer.AddHash(UTF8TEXTVIEW("Hash"), AdditionalFile.Hash);
	Writer.AddString(UTF8TEXTVIEW("Filename"), AdditionalFile.Filename);
	Writer.AddInteger(UTF8TEXTVIEW("FileSize"), AdditionalFile.FileSize);
	Writer.EndObject();

	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandTocAdditionalFile& AdditionalFile)
{
	if (FCbObjectView Obj = Field.AsObjectView())
	{
		AdditionalFile.Hash = Obj["Hash"].AsHash();
		AdditionalFile.Filename = FString(Obj["Filename"].AsString());
		AdditionalFile.FileSize = Obj["FileSize"].AsUInt64();
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
FArchive& operator<<(FArchive& Ar, FOnDemandTocTagSetPackageList& TagSet)
{
	Ar << TagSet.ContainerIndex;
	Ar << TagSet.PackageIndicies;
	return Ar;
}

FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandTocTagSetPackageList& TagSet)
{
	Writer.BeginObject();
	Writer.AddInteger(UTF8TEXTVIEW("ContainerIndex"), TagSet.ContainerIndex);
	Writer.BeginArray(UTF8TEXTVIEW("PackageIndicies"));
	for (const uint32 Index : TagSet.PackageIndicies)
	{
		Writer << Index;
	}
	Writer.EndArray();
	Writer.EndObject();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandTocTagSetPackageList& TagSet)
{
	if (FCbObjectView Obj = Field.AsObjectView())
	{
		FCbFieldView ContainerIndex = Obj["ContainerIndex"];
		TagSet.ContainerIndex = ContainerIndex.AsUInt32();
		if (ContainerIndex.HasError())
		{
			return false;
		}

		FCbFieldView PackageIndicies = Obj["PackageIndicies"];
		FCbArrayView PackageIndiciesArray = PackageIndicies.AsArrayView();
		if(PackageIndicies.HasError())
		{
			return false;
		}

		TagSet.PackageIndicies.Reserve(int32(PackageIndiciesArray.Num()));
		for (FCbFieldView ArrayField : PackageIndiciesArray)
		{
			uint32 Index = ArrayField.AsUInt32();
			if (ArrayField.HasError())
			{
				return false;
			}
			TagSet.PackageIndicies.Emplace(Index);
		}

		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
FArchive& operator<<(FArchive& Ar, FOnDemandTocTagSet& TagSet)
{
	Ar << TagSet.Tag;
	Ar << TagSet.Packages;
	return Ar;
}

FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandTocTagSet& TagSet)
{
	Writer.BeginObject();
	Writer.AddString(UTF8TEXTVIEW("Tag"), TagSet.Tag);
	Writer.BeginArray(UTF8TEXTVIEW("Packages"));
	for (const FOnDemandTocTagSetPackageList& PackageList : TagSet.Packages)
	{
		Writer << PackageList;
	}
	Writer.EndArray();
	Writer.EndObject();
	return Writer;
}

bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandTocTagSet& TagSet)
{
	if (FCbObjectView Obj = Field.AsObjectView())
	{
		TagSet.Tag = FString(Obj["Tag"].AsString());
		FCbArrayView Packages = Obj["Packages"].AsArrayView();
		TagSet.Packages.Reserve(int32(Packages.Num()));
		for (FCbFieldView ArrayField : Packages)
		{
			if (!LoadFromCompactBinary(ArrayField, TagSet.Packages.Emplace_GetRef()))
			{
				return false;
			}
		}
		return true;
	}

	return false;
}

////////////////////////////////////////////////////////////////////////////////
FArchive& operator<<(FArchive& Ar, FOnDemandToc& Toc)
{
	Ar << Toc.Header;
	if (Ar.IsError())
	{
		return Ar;
	}

	Ar.SetCustomVersion(Toc.VersionGuid, int32(Toc.Header.Version), TEXT("OnDemandToc"));

	if (uint32(Toc.Header.Version) >= uint32(EOnDemandTocVersion::Meta))
	{
		Ar << Toc.Meta;
	}
	Ar << Toc.Containers;

	if (uint32(Toc.Header.Version) >= uint32(EOnDemandTocVersion::AdditionalFiles))
	{
		Ar << Toc.AdditionalFiles;
	}

	if (uint32(Toc.Header.Version) >= uint32(EOnDemandTocVersion::TagSets))
	{
		Ar << Toc.TagSets;
	}

	return Ar;
}

FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandToc& Toc)
{
	Writer.BeginObject();
	Writer << UTF8TEXTVIEW("Header") << Toc.Header;

	Writer << UTF8TEXTVIEW("Meta") << Toc.Meta;

	Writer.BeginArray(UTF8TEXTVIEW("Containers"));
	for (const FOnDemandTocContainerEntry& Container : Toc.Containers)
	{
		Writer << Container;
	}
	Writer.EndArray();

	if (Toc.AdditionalFiles.Num() > 0)
	{
		Writer.BeginArray(UTF8TEXTVIEW("Files"));
		for (const FOnDemandTocAdditionalFile& File : Toc.AdditionalFiles)
		{
			Writer << File;
		}
		Writer.EndArray();
	}

	if (Toc.TagSets.Num() > 0)
	{
		Writer.BeginArray(UTF8TEXTVIEW("TagSets"));
		for (const FOnDemandTocTagSet& TagSet : Toc.TagSets)
		{
			Writer << TagSet;
		}
		Writer.EndArray();
	}

	Writer.EndObject();
	
	return Writer;
}

const FGuid FOnDemandToc::VersionGuid	= FGuid("C43DD98F353F499D9A0767F6EA0155EB");
const FString FOnDemandToc::FileExt		= FString(".uondemandtoc");

bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandToc& OutToc)
{
	if (FCbObjectView Obj = Field.AsObjectView())
	{
		if (!LoadFromCompactBinary(Obj[UTF8TEXTVIEW("Header")], OutToc.Header))
		{
			return false;
		}

		if (uint32(OutToc.Header.Version) >= uint32(EOnDemandTocVersion::Meta))
		{
			if (!LoadFromCompactBinary(Obj[UTF8TEXTVIEW("Meta")], OutToc.Meta))
			{
				return false;
			}
		}

		FCbArrayView Containers = Obj[UTF8TEXTVIEW("Containers")].AsArrayView();
		OutToc.Containers.Reserve(int32(Containers.Num()));
		for (FCbFieldView ArrayField : Containers)
		{
			if (!LoadFromCompactBinary(ArrayField, OutToc.Containers.AddDefaulted_GetRef()))
			{
				return false;
			}
		}

		if (uint32(OutToc.Header.Version) >= uint32(EOnDemandTocVersion::AdditionalFiles))
		{
			FCbArrayView Files = Obj["Files"].AsArrayView();
			OutToc.AdditionalFiles.Reserve(int32(Files.Num()));
			for (FCbFieldView ArrayField : Files)
			{
				if (!LoadFromCompactBinary(ArrayField, OutToc.AdditionalFiles.AddDefaulted_GetRef()))
				{
					return false;
				}
			}
		}

		if (uint32(OutToc.Header.Version) >= uint32(EOnDemandTocVersion::TagSets))
		{
			FCbArrayView TagSets = Obj[UTF8TEXTVIEW("TagSets")].AsArrayView();
			OutToc.TagSets.Reserve(int32(TagSets.Num()));
			for (FCbFieldView ArrayField : TagSets)
			{
				if (!LoadFromCompactBinary(ArrayField, OutToc.TagSets.AddDefaulted_GetRef()))
				{
					return false;
				}
			}
		}

		return true;
	}

	return false;
}

TIoStatusOr<FOnDemandToc> FOnDemandToc::LoadFromFile(const FString& FilePath, bool bValidate)
{
	TUniquePtr<FArchive> Ar;
	if (FPathViews::IsRelativePath(FilePath) && FPlatformMisc::FileExistsInPlatformPackage(FilePath))
	{
		Ar = CreateReaderFromPlatformPackage(FilePath);
	}
	
	if (Ar.IsValid() == false)
	{
		Ar.Reset(IFileManager::Get().CreateFileReader(*FilePath));
	}

	if (Ar.IsValid() == false)
	{
		FIoStatus Status = FIoStatusBuilder(EIoErrorCode::FileNotOpen) << TEXT("Failed to open '") << FilePath << TEXT("'");
		return Status;
	}

	if (bValidate)
	{
		const int64 SentinelPos = Ar->TotalSize() - FOnDemandTocSentinel::SentinelSize;

		if (SentinelPos < 0)
		{
			FIoStatus Status = FIoStatusBuilder(EIoErrorCode::CorruptToc) << TEXT("Unexpected file size");
			return Status;
		}

		Ar->Seek(SentinelPos);

		FOnDemandTocSentinel Sentinel;
		*Ar << Sentinel;

		if (!Sentinel.IsValid())
		{
			return FIoStatus(EIoErrorCode::CorruptToc);
		}

		Ar->Seek(0);
	}

	FOnDemandToc Toc;
	*Ar << Toc;

	if (Ar->IsError() || Ar->IsCriticalError())
	{
		FIoStatus Status = FIoStatusBuilder(EIoErrorCode::FileNotOpen) << TEXT("Failed to serialize TOC file");
		return Status;
	}

	return Toc; 
}

} // namespace UE::IoStore

////////////////////////////////////////////////////////////////////////////////
FStringBuilderBase& operator<<(FStringBuilderBase& Sb, UE::IoStore::EOnDemandTocFlags TocFlags)
{
	using namespace UE::IoStore;

	static const TCHAR* FlagText[]
	{
		TEXT("None"),
		TEXT("InstallOnDemand"),
		TEXT("StreamOnDemand"),
	};

	if (TocFlags == EOnDemandTocFlags::None)
	{
		Sb << FlagText[0];
		return Sb;
	}

	constexpr uint32 BitCount = 1 + FMath::CountTrailingZeros(
		static_cast<std::underlying_type_t<EOnDemandTocFlags>>(EOnDemandTocFlags::Last));
	static_assert(UE_ARRAY_COUNT(FlagText) == BitCount + 1, "Please update flag text list");

	for (int32 Idx = 0; Idx < BitCount; ++Idx)
	{
		const EOnDemandTocFlags FlagToTest = static_cast<EOnDemandTocFlags>(1 << Idx);
		if (EnumHasAnyFlags(TocFlags, FlagToTest))
		{
			if (Sb.Len())
			{
				Sb << TEXT("|");
			}
			Sb << FlagText[Idx + 1];
		}
	}

	return Sb;
}

FString LexToString(UE::IoStore::EOnDemandTocFlags TocFlags)
{
	TStringBuilder<128> Sb;
	Sb << TocFlags;
	return FString::ConstructFromPtrSize(Sb.ToString(), Sb.Len());
}
