// Copyright Epic Games, Inc. All Rights Reserved.

#include "IO/Serialization/OnDemandContainerToc.h"

#include "Algo/Find.h"
#include "Async/MappedFileHandle.h"
#include "Containers/ArrayView.h"
#include "Containers/ChunkedArray.h"
#include "Containers/Utf8String.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/FileManager.h"
#include "HAL/Platform.h"
#include "HAL/PlatformFileManager.h"
#include "IO/IoStatus.h"
#include "IO/OnDemandToc.h"
#include "Memory/MemoryView.h"
#include "Misc/DateTime.h"
#include "Serialization/Archive.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Templates/UniquePtr.h"
#include "Templates/UnrealTemplate.h"

namespace UE::IoStore::Serialization
{

////////////////////////////////////////////////////////////////////////////////
template <typename T>
FArchive& SerializeArrayData(FArchive& Ar, const TArray<T>& Array)
{
	Ar.Serialize((void*)Array.GetData(), Array.GetTypeSize() * Array.Num());
	return Ar;
}

////////////////////////////////////////////////////////////////////////////////
FOnDemandTocStorage::FOnDemandTocStorage()
{
}

FOnDemandTocStorage::FOnDemandTocStorage(
	FIoBuffer InHeaderData,
	FIoBuffer InContainerData)
		: HeaderData(InHeaderData)
{
	ContainerData.Emplace<FMemoryStorage>(FMemoryStorage{ .Buffer = InContainerData });
}

FOnDemandTocStorage::FOnDemandTocStorage(
	FIoBuffer InHeaderData,
	TUniquePtr<IMappedFileHandle>&& File,
	TUniquePtr<IMappedFileRegion>&& Region)
		: HeaderData(InHeaderData)
{
	ContainerData.Emplace<FMappedFileStorage>(FMappedFileStorage
	{
		.MappedFile		= MoveTemp(File),
		.MappedRegion	= MoveTemp(Region)
	});
}

FOnDemandTocStorage::FOnDemandTocStorage(FOnDemandTocStorage&& Other)
	: HeaderData(MoveTemp(Other.HeaderData))
	, ContainerData(MoveTemp(Other.ContainerData))
{
}

FOnDemandTocStorage::~FOnDemandTocStorage()
{
}

FOnDemandTocStorage& FOnDemandTocStorage::operator=(FOnDemandTocStorage&& Other)
{
	HeaderData = MoveTemp(Other.HeaderData);
	ContainerData = MoveTemp(Other.ContainerData);
	return *this;
}

FMemoryView FOnDemandTocStorage::GetView() const
{
	if (const FMemoryStorage* MemoryStorage = ContainerData.TryGet<FMemoryStorage>())
	{
		return MemoryStorage->Buffer.GetView();
	}
	else if (const FMappedFileStorage* FileStorage = ContainerData.TryGet<FMappedFileStorage>())
	{
		return MakeMemoryView(FileStorage->MappedRegion->GetMappedPtr(), FileStorage->MappedRegion->GetMappedSize());
	}

	return FMemoryView();
}

FOnDemandTocStorageType FOnDemandTocStorage::StorageType() const
{
	if (ContainerData.IsType<FMappedFileStorage>())
	{
		return FOnDemandTocStorageType::MemoryMapped;
	}
	else if (ContainerData.IsType<FMemoryStorage>())
	{
		return FOnDemandTocStorageType::Memory;
	}

	return FOnDemandTocStorageType::Invalid;
}

////////////////////////////////////////////////////////////////////////////////
class FOnDemandTocWriter::FImpl
{
	using FOnDemandStringEntry		= V1::FOnDemandStringEntry;
	using FOnDemandChunkEntry		= V2::FOnDemandChunkEntry;
	using FOnDemandPartitionEntry	= V2::FOnDemandPartitionEntry;
	using FOnDemandTagSetEntry		= V1::FOnDemandTagSetEntry;

	struct FContainer
	{
		FIoHash								UTocHash;
		FGuid								EncryptionKeyGuid;
		TUniquePtr<FLargeMemoryWriter>		PartitionAr;
		TArray<FOnDemandPartitionEntry>		Partitions;
		TArray<FIoChunkId>					ChunkIds;
		TArray<FOnDemandChunkEntry>			ChunkEntries;
		TArray<uint32>						BlockSizes;
		TArray<FIoBlockHash>				BlockHashes;
		TArray<FOnDemandTagSetEntry>		TagSets;
		TArray<uint32>						TagSetIndices;
		FIoBuffer							Header;
		FString								Name;
		FIoContainerId						Id;
		uint32								CompressionBlockSize;
		int32								CurrentPartition = INDEX_NONE;
		FName								CompressionFormat = NAME_None;
		uint32								FileContainerFlags = 0;
		EOnDemandContainerEntryFlags		ContainerFlags = EOnDemandContainerEntryFlags::None;
	};

	struct FMetadata
	{
		FString BuildVersion;
		FString TargetPlatform;
		FString HostGroupName;
		FString ChunksDirectory;
	};

public:
	bool IsEmpty() const
	{
		return Containers.IsEmpty();
	}

	void SetMetadata(const FString& BuildVersion, const FString& TargetPlatform)
	{
		Metadata.BuildVersion	= BuildVersion;
		Metadata.TargetPlatform = TargetPlatform;
	}

	void SetHostGroup(const FString& HostGroupName)
	{
		Metadata.HostGroupName = HostGroupName;
	}

	void SetChunksDirectory(const FString& ChunksDirectory)
	{
		Metadata.ChunksDirectory = ChunksDirectory;
	}

	void BeginContainer(
		const FString& ContainerName,
		const FIoContainerId& ContainerId,
		const FIoHash& UTocHash,
		const FGuid& EncryptionKeyGuid,
		uint32 CompressionBlockSize,
		uint32 FileContainerFlags,
		EOnDemandContainerEntryFlags ContainerFlags,
		FIoBuffer ContainerHeader)
	{
		check(CurrentContainer == nullptr);
		const int32 Idx				= Containers.Add();
		CurrentContainer			= &Containers[Idx];

		CurrentContainer->UTocHash				= UTocHash;
		CurrentContainer->EncryptionKeyGuid		= EncryptionKeyGuid;
		CurrentContainer->Header				= ContainerHeader;
		CurrentContainer->Name					= ContainerName;
		CurrentContainer->Id					= ContainerId;
		CurrentContainer->CompressionBlockSize	= CompressionBlockSize;
		CurrentContainer->FileContainerFlags	= FileContainerFlags;
		CurrentContainer->ContainerFlags		= ContainerFlags;
	}

	int32 BeginPartition()
	{
		check(CurrentContainer != nullptr);
		check(CurrentContainer->PartitionAr.IsValid() == false);
		check(CurrentContainer->CurrentPartition == INDEX_NONE);
		CurrentContainer->CurrentPartition = CurrentContainer->Partitions.Num();
		CurrentContainer->Partitions.AddDefaulted();
		CurrentContainer->PartitionAr = MakeUnique<FLargeMemoryWriter>();

		return CurrentContainer->CurrentPartition;
	}

	uint64 AddChunk(
		int32 Partition,
		const FIoChunkId& ChunkId,
		FIoBuffer EncodedChunk,
		TConstArrayView<uint32> BlockSizes,
		TConstArrayView<FIoBlockHash> BlockHashes,
		uint64 RawChunkSize)
	{
		using namespace V2;
		check(CurrentContainer != nullptr);
		check(CurrentContainer->PartitionAr.IsValid());
		check(BlockSizes.Num() == BlockHashes.Num());
		check(ChunkId.IsValid());

		FArchive& Ar = *CurrentContainer->PartitionAr;

		CurrentContainer->ChunkIds.Add(ChunkId);

		FOnDemandChunkEntry& ChunkEntry = CurrentContainer->ChunkEntries.AddDefaulted_GetRef();
		ChunkEntry.Hash					= FOnDemandChunkHash::HashBuffer(EncodedChunk.GetView());
		ChunkEntry.PartitionIndex		= uint32(CurrentContainer->CurrentPartition);
		ChunkEntry.PartitionOffset		= uint32(Ar.Tell());
		ChunkEntry.RawSize				= uint32(RawChunkSize);
		ChunkEntry.EncodedSize			= uint32(EncodedChunk.GetSize());

		// If the chunk only has a single compression block, the size and block hash are encoded into the chunk entry
		const int32 BlockCount = BlockSizes.Num();
		if (BlockCount == 1)
		{
			ChunkEntry.BlockInfo = FOnDemandChunkBlockInfo::FromSizeAndHash(BlockSizes[0], BlockHashes[0]);
		}
		else
		{
			const int32 BlocksOffset = CurrentContainer->BlockSizes.Num();
			ChunkEntry.BlockInfo = FOnDemandChunkBlockInfo::FromOffsetAndCount(BlocksOffset, BlockCount);
			CurrentContainer->BlockSizes.Append(BlockSizes);
			CurrentContainer->BlockHashes.Append(BlockHashes);
		}

		Ar.Serialize(EncodedChunk.GetData(), EncodedChunk.GetSize());
		return uint64(Ar.TotalSize());
	}

	void AddTagSet(FStringView Tag, TConstArrayView<uint32> PackageIndices)
	{
		check(CurrentContainer != nullptr);
		check(!Tag.IsEmpty());
		check(!PackageIndices.IsEmpty());

		FOnDemandTagSetEntry& Entry = CurrentContainer->TagSets.AddDefaulted_GetRef();
		Entry.Tag					= WriteString(Tag);
		Entry.Count					= uint32(PackageIndices.Num());
		Entry.Offset				= uint32(CurrentContainer->TagSetIndices.Num());

		CurrentContainer->TagSetIndices.Append(PackageIndices);
	}

	FIoBuffer EndPartition(int32 Partition, FIoHash& OutHash)
	{
		check(CurrentContainer != nullptr);
		check(CurrentContainer->PartitionAr.IsValid());

		FLargeMemoryWriter& Ar		= *CurrentContainer->PartitionAr;
		const uint64 PartitionSize	= Ar.TotalSize();

		FOnDemandPartitionEntry& PartitionEntry = CurrentContainer->Partitions[Partition];
		FIoBuffer OutPartition(FIoBuffer::AssumeOwnership, Ar.ReleaseOwnership(), PartitionSize);

		PartitionEntry.Hash	= FIoHash::HashBuffer(OutPartition.GetView());
		PartitionEntry.Size	= uint32(OutPartition.GetSize());
		OutHash				= PartitionEntry.Hash;

		CurrentContainer->PartitionAr.Reset();
		CurrentContainer->CurrentPartition = INDEX_NONE;

		return OutPartition;
	}

	void EndContainer(FName CompressionFormat)
	{
		check(CurrentContainer != nullptr);
		CurrentContainer->CompressionFormat = CompressionFormat;
		CurrentContainer = nullptr;
	}

	TIoStatusOr<uint64> Write(FArchive& Ar)
	{
		using namespace UE::IoStore::Serialization::V2;

		if (Containers.IsEmpty())
		{
			return FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
				<< TEXT("No container(s) to serialize");
		}

		FName							CompressionFormat = NAME_None;
		TArray<FOnDemandContainerEntry> ContainerEntries;
		FLargeMemoryWriter				ContainerDataAr;
		const int64						ArStart = Ar.Tell();

		for (int32 ContainerIndex = 0; const FContainer& Container : Containers)
		{
			check(Container.ChunkIds.Num() == Container.ChunkEntries.Num());

			TArray<int32> EntryIndices;
			EntryIndices.Reserve(Container.ChunkIds.Num());
			for (int32 Idx = 0; Idx < Container.ChunkIds.Num(); ++Idx)
			{
				EntryIndices.Add(Idx);
			}

			EntryIndices.Sort([&Container](int32 LHS, int32 RHS)
			{
				return Container.ChunkIds[LHS] < Container.ChunkIds[RHS];
			});

			TArray<FIoChunkId>			SortedChunkIds;
			TArray<FOnDemandChunkEntry>	SortedChunkEntries;
			const TArray<uint32>&		BlockSizes = Container.BlockSizes;
			const TArray<FIoBlockHash>&	BlockHashes = Container.BlockHashes;

			SortedChunkIds.Reserve(Container.ChunkIds.Num());
			SortedChunkEntries.Reserve(Container.ChunkIds.Num());
			for (int32 Idx : EntryIndices)
			{
				check(Container.ChunkIds[Idx].IsValid());
				SortedChunkIds.Add(Container.ChunkIds[Idx]);
				SortedChunkEntries.Add(Container.ChunkEntries[Idx]);
			}

			FOnDemandContainerEntry& ContainerEntry = ContainerEntries.AddDefaulted_GetRef();
			ContainerEntry.ContainerId			= Container.Id;
			ContainerEntry.ContainerName		= WriteString(Container.Name);
			ContainerEntry.ContainerHeaderSize	= uint32(Container.Header.GetSize());
			ContainerEntry.EncryptionKeyGuid	= Container.EncryptionKeyGuid;
			ContainerEntry.ChunkCount			= Container.ChunkIds.Num();
			ContainerEntry.BlockCount			= Container.BlockSizes.Num();
			ContainerEntry.BlockSize			= Container.CompressionBlockSize; 
			ContainerEntry.TagSetCount			= Container.TagSets.Num();
			ContainerEntry.TagSetIndicesCount	= Container.TagSetIndices.Num();
			ContainerEntry.UTocHash				= Container.UTocHash;
			ContainerEntry.ContainerFlags		= uint32(Container.ContainerFlags);
			ContainerEntry.FileContainerFlags 	= Container.FileContainerFlags;
			ContainerEntry.PartitionCount		= uint32(Container.Partitions.Num());

			ContainerEntry.DataOffset = IntCastChecked<uint32>(ContainerDataAr.Tell());
			{
				SerializeArrayData(ContainerDataAr, Container.Partitions);
				SerializeArrayData(ContainerDataAr, SortedChunkIds);
				SerializeArrayData(ContainerDataAr, SortedChunkEntries);
				SerializeArrayData(ContainerDataAr, BlockSizes);
				SerializeArrayData(ContainerDataAr, BlockHashes);
				SerializeArrayData(ContainerDataAr, Container.TagSets);
				SerializeArrayData(ContainerDataAr, Container.TagSetIndices);
				ContainerDataAr.Serialize((void*)Container.Header.GetData(), Container.Header.GetSize());
			}
			ContainerEntry.DataSize = IntCastChecked<uint32>(ContainerDataAr.Tell()) - ContainerEntry.DataOffset;

			if (Container.CompressionFormat != NAME_None)
			{
				check(CompressionFormat == NAME_None || CompressionFormat == Container.CompressionFormat);
				if (CompressionFormat == NAME_None)
				{
					CompressionFormat = Container.CompressionFormat;
				}
			}
		}

		V2::FOnDemandTocHeader Header;
		Header.EpochTimestamp		= FDateTime::Now().ToUnixTimestamp();
		Header.BuildVersion			= WriteString(Metadata.BuildVersion);
		Header.TargetPlatform		= WriteString(Metadata.TargetPlatform);
		Header.ChunksDirectory		= WriteString(Metadata.ChunksDirectory);
		Header.HostGroupName		= WriteString(Metadata.HostGroupName);
		Header.CompressionFormat	= WriteString(CompressionFormat.ToString());
		Header.ContainerCount		= Containers.Num();
		Header.StringTableLen		= StringTableLen;

		Ar.Serialize(&Header, sizeof(FOnDemandTocHeader));
		{
			const int64 StringTableOffset = Ar.Tell();
			for (const FUtf8String& String : StringTable)
			{
				Ar.Serialize((void*)*String, sizeof(UTF8CHAR) * String.Len());
			}
			const int64 StringTableSize = Ar.Tell() - StringTableOffset;

			if (sizeof(UTF8CHAR) * Header.StringTableLen != StringTableSize)
			{
				return FIoStatus(EIoErrorCode::WriteError, TEXT("Unexpected string table size"));
			}
		}
		Ar.Serialize((void*)ContainerEntries.GetData(), ContainerEntries.GetTypeSize() * ContainerEntries.Num());
		Ar.Serialize(ContainerDataAr.GetData(), ContainerDataAr.TotalSize());

		FOnDemandTocFooter Footer;
		Ar.Serialize(&Footer, sizeof(FOnDemandTocFooter));

		return uint64(Ar.Tell() - ArStart);
	}

private:
	FOnDemandStringEntry WriteString(const FStringView& String)
	{
		if (String.IsEmpty())
		{
			return FOnDemandStringEntry{};
		}

		FUtf8String Utf8String(String);
		if (const FOnDemandStringEntry* ExistingEntry = StringToEntry.Find(Utf8String))
		{
			return *ExistingEntry;
		}
		else
		{
			const FOnDemandStringEntry NewEntry = FOnDemandStringEntry
			{
				.Offset = uint32(StringTableLen),
				.Len	= uint32(Utf8String.Len())
			};

			StringTableLen += Utf8String.Len();
			const int32 Index = StringTable.AddElement(MoveTemp(Utf8String));
			StringToEntry.Add(StringTable[Index], NewEntry);

			return NewEntry;
		}
	}

	using FLookup = TMap<FUtf8StringView, FOnDemandStringEntry>;

	FLookup								StringToEntry;
	TChunkedArray<FUtf8String>			StringTable;
	TChunkedArray<FContainer>			Containers;
	FMetadata							Metadata;
	FContainer*							CurrentContainer = nullptr;
	int32								StringTableLen = 0;
};

////////////////////////////////////////////////////////////////////////////////
FOnDemandTocWriter::FOnDemandTocWriter()
	: Impl(MakeUnique<FImpl>())
{
}

FOnDemandTocWriter::FOnDemandTocWriter(FOnDemandTocWriter&& Other)
	: Impl(MoveTemp(Other.Impl))
{
}

FOnDemandTocWriter::~FOnDemandTocWriter()
{
}

FOnDemandTocWriter& FOnDemandTocWriter::operator=(FOnDemandTocWriter&& Other)
{
	Impl = MoveTemp(Other.Impl);
	return *this;
}

bool FOnDemandTocWriter::IsEmpty() const
{
	return !Impl.IsValid() || Impl->IsEmpty();
}

void FOnDemandTocWriter::SetMetadata(const FString& BuildVersion, const FString& TargetPlatform)
{
	check(Impl.IsValid());
	Impl->SetMetadata(BuildVersion, TargetPlatform);
}

void FOnDemandTocWriter::SetHostGroup(const FString& HostGroupName)
{
	check(Impl.IsValid());
	Impl->SetHostGroup(HostGroupName);
}

void FOnDemandTocWriter::SetChunksDirectory(const FString& ChunksDirectory)
{
	check(Impl.IsValid());
	Impl->SetChunksDirectory(ChunksDirectory);
}

void FOnDemandTocWriter::BeginContainer(
	const FString& ContainerName,
	const FIoContainerId& ContainerId,
	const FIoHash& UTocHash,
	const FGuid& EncryptionKeyGuid,
	uint32 CompressionBlockSize,
	uint32 FileContainerFlags,
	EOnDemandContainerEntryFlags ContainerFlags,
	FIoBuffer ContainerHeader)
{
	check(Impl.IsValid());
	Impl->BeginContainer(
		ContainerName,
		ContainerId,
		UTocHash,
		EncryptionKeyGuid,
		CompressionBlockSize,
		FileContainerFlags,
		ContainerFlags,
		ContainerHeader);
}

int32 FOnDemandTocWriter::BeginPartition()
{
	check(Impl.IsValid());
	return Impl->BeginPartition();
}

uint64 FOnDemandTocWriter::AddChunk(
	int32 Partition,
	const FIoChunkId& ChunkId,
	FIoBuffer EncodedChunk,
	TConstArrayView<uint32> BlockSizes,
	TConstArrayView<FIoBlockHash> BlockHashes,
	uint64 RawChunkSize)
{
	check(Partition >= 0);
	check(Impl.IsValid());
	return Impl->AddChunk(Partition, ChunkId, EncodedChunk, BlockSizes, BlockHashes, RawChunkSize);
}

void FOnDemandTocWriter::AddTagSet(FStringView Tag, TConstArrayView<uint32> PackageIndices)
{
	check(Impl.IsValid());
	Impl->AddTagSet(Tag, PackageIndices);
}

FIoBuffer FOnDemandTocWriter::EndPartition(int32 Partition, FIoHash& OutHash)
{
	check(Partition >= 0);
	check(Impl.IsValid());
	return Impl->EndPartition(Partition, OutHash);
}

void FOnDemandTocWriter::EndContainer(FName CompressionFormat)
{
	check(Impl.IsValid());
	Impl->EndContainer(CompressionFormat);
}

TIoStatusOr<uint64> FOnDemandTocWriter::Write(FArchive& Ar)
{
	return Impl->Write(Ar);
}

TIoStatusOr<uint64> FOnDemandTocWriter::Write(const FString& Filename)
{
	TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*Filename));
	if (Ar.IsValid() == false)
	{
		return FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
			<< TEXT("Failed to open '") << Filename << TEXT("'");
	}

	return Impl->Write(*Ar);
}

////////////////////////////////////////////////////////////////////////////////
namespace V1
{

////////////////////////////////////////////////////////////////////////////////
EIoContainerFlags FOnDemandContainerTocHeaderView::FileContainerFlags() const
{
	return static_cast<EIoContainerFlags>(ContainerEntry->FileContainerFlags);
}

FUtf8StringView	FOnDemandContainerTocHeaderView::GetString(FOnDemandStringEntry StringEntry) const
{
	return FUtf8StringView(StringTable + StringEntry.Offset, StringEntry.Len);
}

////////////////////////////////////////////////////////////////////////////////
FOnDemandTocReader::FOnDemandTocReader(const FString& InFilename, FIoBuffer InBuffer)
	: Filename(InFilename)
	, Buffer(InBuffer)
{
	Header = reinterpret_cast<const FOnDemandTocHeader*>(Buffer.GetData());
	StringTable = reinterpret_cast<const UTF8CHAR*>(Header + 1);
	ContainerEntries = MakeArrayView(
		reinterpret_cast<const FOnDemandContainerEntry*>(StringTable + Header->StringTableLen), Header->ContainerCount);
}

FUtf8StringView FOnDemandTocReader::GetString(FOnDemandStringEntry StringEntry) const
{
	return FUtf8StringView(StringTable + StringEntry.Offset, StringEntry.Len);
}

TIoStatusOr<FOnDemandContainerTocView> FOnDemandTocReader::ReadContainer(
	const FOnDemandContainerEntry& Container,
	FOnDemandTocStorage& Storage,
	EOnDemandTocReaderOptions Options) const
{
	const FOnDemandContainerEntry* Existing = Algo::FindByPredicate(
		ContainerEntries, [&Container](const FOnDemandContainerEntry& Entry) { return &Entry == &Container; });

	if (Existing == nullptr)
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}

	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();

	const int64 ContainerDataOffset = sizeof(FOnDemandTocHeader)
		+ (Header->StringTableLen * sizeof(UTF8CHAR))
		+ (Header->ContainerCount * sizeof(FOnDemandContainerEntry))
		+ Container.DataOffset;

	const int64 ContainerDataSize = Container.DataSize - Container.ContainerHeaderSize;
	if (ContainerDataSize <= 0)
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc)
			<< TEXT("Unexpected container data size in '")
			<< Filename << TEXT("'");
	}

	const bool bMemoryMap = FPlatformProperties::SupportsMemoryMappedFiles()
		&& EnumHasAnyFlags(Options, EOnDemandTocReaderOptions::MemoryMap);

	if (bMemoryMap)
	{
		FOpenMappedResult OpenResult = Ipf.OpenMappedEx(*Filename);
		if (OpenResult.HasError() == false)
		{
			TUniquePtr<IMappedFileHandle> MappedFile = OpenResult.StealValue();
			TUniquePtr<IMappedFileRegion> Region(MappedFile->MapRegion(ContainerDataOffset, ContainerDataSize));

			if (Region.IsValid())
			{
				Storage = FOnDemandTocStorage(Buffer, MoveTemp(MappedFile), MoveTemp(Region));
			}
		}
	}

	FFileOpenResult OpenResult = Ipf.OpenRead(*Filename, IPlatformFile::EOpenReadFlags::None);
	if (OpenResult.HasError())
	{
		FFileSystemError& Error = OpenResult.GetError();
		return FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
			<< TEXT("Failed to open '") << Filename << TEXT("' reason: ")
			<< Error.GetMessage();
	}

	TUniquePtr<IFileHandle> FileHandle = OpenResult.StealValue();
	if (Storage.IsEmpty())
	{
		if (FileHandle->Seek(ContainerDataOffset) == false)
		{
			return FIoStatusBuilder(EIoErrorCode::FileSeekFailed)
				<< TEXT("Failed to seek to container data in '") << Filename << TEXT("'");
		}

		FIoBuffer ContainerData(ContainerDataSize);
		if (FileHandle->Read(ContainerData.GetData(), ContainerData.GetSize()) == false)
		{
			return FIoStatusBuilder(EIoErrorCode::ReadError)
				<< TEXT("Failed to read container data in '") << Filename << TEXT("'");
		}

		Storage = FOnDemandTocStorage(Buffer, ContainerData);
	}

	const int64 ContainerHeaderOffset = ContainerDataOffset + ContainerDataSize;
	if (FileHandle->Seek(ContainerHeaderOffset) == false)
	{
		return FIoStatusBuilder(EIoErrorCode::FileSeekFailed)
			<< TEXT("Failed to seek to container header in '") << Filename << TEXT("'");
	}

	// The container header is currently copied into a separate buffer
	FIoBuffer ContainerHeaderChunk(Container.ContainerHeaderSize);
	if (FileHandle->Read(ContainerHeaderChunk.GetData(), ContainerHeaderChunk.GetSize()) == false)
	{
		return FIoStatusBuilder(EIoErrorCode::ReadError)
			<< TEXT("Failed to read container header in '") << Filename << TEXT("'");
	}

	FMemoryView ContainerData = Storage.GetView();
	FOnDemandContainerTocView View;
	View.Header = FOnDemandContainerTocHeaderView(Header, StringTable, &Container);

	View.ChunkIds = MakeArrayView(reinterpret_cast<const FIoChunkId*>(ContainerData.GetData()), Container.ChunkCount);
	ContainerData += View.ChunkIds.NumBytes();

	View.ChunkEntries = MakeArrayView(reinterpret_cast<const FOnDemandChunkEntry*>(ContainerData.GetData()), Container.ChunkCount);
	ContainerData += View.ChunkEntries.NumBytes();

	View.BlockSizes = MakeArrayView(reinterpret_cast<const uint32*>(ContainerData.GetData()), Container.BlockCount);
	ContainerData += View.BlockSizes.NumBytes();

	View.BlockHashes = MakeArrayView(reinterpret_cast<const FIoBlockHash*>(ContainerData.GetData()), Container.BlockCount);
	ContainerData += View.BlockHashes.NumBytes();

	View.TagSets = MakeArrayView(reinterpret_cast<const FOnDemandTagSetEntry*>(ContainerData.GetData()), Container.TagSetCount);
	ContainerData += View.TagSets.NumBytes();

	View.TagSetIndices = MakeArrayView(reinterpret_cast<const uint32*>(ContainerData.GetData()), Container.TagSetIndicesCount);
	ContainerData += View.TagSetIndices.NumBytes();

	if (ContainerData.IsEmpty() == false)
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc)
			<< TEXT("Unexpected container data size in '") << Filename << TEXT("'");
	}

	View.ContainerHeaderChunk = MoveTemp(ContainerHeaderChunk);

	return View;
}

TIoStatusOr<FOnDemandTocReader> FOnDemandTocReader::Read(const FString& Filename)
{
	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();

	FFileOpenResult OpenResult = Ipf.OpenRead(*Filename, IPlatformFile::EOpenReadFlags::None);
	if (OpenResult.HasError())
	{
		FFileSystemError& Error = OpenResult.GetError();
		return FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
			<< TEXT("Failed to open '") << Filename << TEXT("' reason: ")
			<< Error.GetMessage();
	}

	TUniquePtr<IFileHandle> FileHandle = OpenResult.StealValue();

	FOnDemandTocHeader Header;
	if (FileHandle->Read(reinterpret_cast<uint8*>(&Header), sizeof(FOnDemandTocHeader)) == false)
	{
		return FIoStatusBuilder(EIoErrorCode::ReadError)
			<< TEXT("Failed to read container TOC header from '") << Filename << TEXT("'");
	}

	if (Header.Signature.IsValid() == false)
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc)
			<< TEXT("Invalid TOC signature found in '") << Filename << TEXT("'");
	}

	if (Header.Version.IsValid() == false)
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc)
			<< TEXT("Invalid TOC version found in '") << Filename << TEXT("'");
	}

	const int64 TotalHeaderSize =
		int64(sizeof(FOnDemandTocHeader)) +
		int64((sizeof(UTF8CHAR) * Header.StringTableLen)) +
		int64((sizeof(FOnDemandContainerEntry) * Header.ContainerCount));

	if ((TotalHeaderSize < int64(sizeof(FOnDemandTocHeader))) || (TotalHeaderSize >= FileHandle->Size()))
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc)
			<< TEXT("TOC header size greater than file size in '") << Filename << TEXT("'");
	}

	FIoBuffer Buffer(TotalHeaderSize);
	FMutableMemoryView BufferView = Buffer.GetMutableView();
	FMemory::Memcpy(BufferView.GetData(), &Header, sizeof(FOnDemandTocHeader));
	BufferView += sizeof(FOnDemandTocHeader);

	if (FileHandle->Read(reinterpret_cast<uint8*>(BufferView.GetData()), BufferView.GetSize()) == false)
	{
		return FIoStatusBuilder(EIoErrorCode::ReadError)
			<< TEXT("Failed to read container TOC header from '") << Filename << TEXT("'");
	}

	if (FileHandle->Seek(FileHandle->Size() - int64(sizeof(FOnDemandTocFooter))) == false)
	{
		return FIoStatusBuilder(EIoErrorCode::FileSeekFailed)
			<< TEXT("Failed to seek to container TOC footer in '") << Filename << TEXT("'");
	}

	FOnDemandTocFooter Footer;
	if (FileHandle->Read(reinterpret_cast<uint8*>(&Footer), sizeof(FOnDemandTocFooter)) == false)
	{
		return FIoStatusBuilder(EIoErrorCode::ReadError)
			<< TEXT("Failed to read container TOC footer from '") << Filename << TEXT("'");
	}

	if (Footer.Signature.IsValid() == false)
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc)
			<< TEXT("Invalid TOC footer signature found in '") << Filename << TEXT("'");
	}

	return FOnDemandTocReader(Filename, Buffer);
}

} // V1 

////////////////////////////////////////////////////////////////////////////////
namespace V2
{

FOnDemandTocReader::FOnDemandTocReader(const FString& InFilename, FIoBuffer InBuffer)
	: Filename(InFilename)
	, Buffer(InBuffer)
{
	Header = reinterpret_cast<const FOnDemandTocHeader*>(Buffer.GetData());
	StringTable = reinterpret_cast<const UTF8CHAR*>(Header + 1);
	ContainerEntries = MakeArrayView(
		reinterpret_cast<const FOnDemandContainerEntry*>(StringTable + Header->StringTableLen), Header->ContainerCount);
}

FUtf8StringView FOnDemandTocReader::GetString(FOnDemandStringEntry StringEntry) const
{
	return FUtf8StringView(StringTable + StringEntry.Offset, StringEntry.Len);
}

TIoStatusOr<FOnDemandContainerTocView> FOnDemandTocReader::ReadContainer(
	const FOnDemandContainerEntry& Container,
	FOnDemandTocStorage& Storage,
	EOnDemandTocReaderOptions Options) const
{
	const FOnDemandContainerEntry* Existing = Algo::FindByPredicate(
		ContainerEntries, [&Container](const FOnDemandContainerEntry& Entry) { return &Entry == &Container; });

	if (Existing == nullptr)
	{
		return FIoStatus(EIoErrorCode::NotFound);
	}

	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();

	const int64 ContainerDataOffset = sizeof(FOnDemandTocHeader)
		+ (Header->StringTableLen * sizeof(UTF8CHAR))
		+ (Header->ContainerCount * sizeof(FOnDemandContainerEntry))
		+ Container.DataOffset;

	const int64 ContainerDataSize = int64(Container.DataSize) - int64(Container.ContainerHeaderSize);
	if (ContainerDataSize <= 0)
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc)
			<< TEXT("Unexpected container data size in '")
			<< Filename << TEXT("'");
	}

	const bool bMemoryMap = FPlatformProperties::SupportsMemoryMappedFiles()
		&& EnumHasAnyFlags(Options, EOnDemandTocReaderOptions::MemoryMap);

	if (bMemoryMap)
	{
		FOpenMappedResult OpenResult = Ipf.OpenMappedEx(*Filename);
		if (OpenResult.HasError() == false)
		{
			TUniquePtr<IMappedFileHandle> MappedFile = OpenResult.StealValue();
			TUniquePtr<IMappedFileRegion> Region(MappedFile->MapRegion(ContainerDataOffset, ContainerDataSize));

			if (Region.IsValid())
			{
				Storage = FOnDemandTocStorage(Buffer, MoveTemp(MappedFile), MoveTemp(Region));
			}
		}
	}

	FFileOpenResult OpenResult = Ipf.OpenRead(*Filename, IPlatformFile::EOpenReadFlags::None);
	if (OpenResult.HasError())
	{
		FFileSystemError& Error = OpenResult.GetError();
		return FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
			<< TEXT("Failed to open '") << Filename << TEXT("' reason: ")
			<< Error.GetMessage();
	}

	TUniquePtr<IFileHandle> FileHandle = OpenResult.StealValue();
	if (Storage.IsEmpty())
	{
		if (FileHandle->Seek(ContainerDataOffset) == false)
		{
			return FIoStatusBuilder(EIoErrorCode::FileSeekFailed)
				<< TEXT("Failed to seek to container data in '") << Filename << TEXT("'");
		}

		FIoBuffer ContainerData(ContainerDataSize);
		if (FileHandle->Read(ContainerData.GetData(), ContainerData.GetSize()) == false)
		{
			return FIoStatusBuilder(EIoErrorCode::ReadError)
				<< TEXT("Failed to read container data in '") << Filename << TEXT("'");
		}

		Storage = FOnDemandTocStorage(Buffer, ContainerData);
	}

	const int64 ContainerHeaderOffset = ContainerDataOffset + ContainerDataSize;
	if (FileHandle->Seek(ContainerHeaderOffset) == false)
	{
		return FIoStatusBuilder(EIoErrorCode::FileSeekFailed)
			<< TEXT("Failed to seek to container header in '") << Filename << TEXT("'");
	}

	// The container header is currently copied into a separate buffer
	FIoBuffer ContainerHeaderChunk(Container.ContainerHeaderSize);
	if (FileHandle->Read(ContainerHeaderChunk.GetData(), ContainerHeaderChunk.GetSize()) == false)
	{
		return FIoStatusBuilder(EIoErrorCode::ReadError)
			<< TEXT("Failed to read container header in '") << Filename << TEXT("'");
	}

	FMemoryView ContainerData = Storage.GetView();
	FOnDemandContainerTocView View;
	View.Header = FOnDemandContainerTocHeaderView(Header, StringTable, &Container);

	View.PartitionEntries = MakeArrayView(reinterpret_cast<const FOnDemandPartitionEntry*>(ContainerData.GetData()), Container.PartitionCount);
	ContainerData += View.PartitionEntries.NumBytes();

	View.ChunkIds = MakeArrayView(reinterpret_cast<const FIoChunkId*>(ContainerData.GetData()), Container.ChunkCount);
	ContainerData += View.ChunkIds.NumBytes();

	View.ChunkEntries = MakeArrayView(reinterpret_cast<const FOnDemandChunkEntry*>(ContainerData.GetData()), Container.ChunkCount);
	ContainerData += View.ChunkEntries.NumBytes();

	View.BlockSizes = MakeArrayView(reinterpret_cast<const uint32*>(ContainerData.GetData()), Container.BlockCount);
	ContainerData += View.BlockSizes.NumBytes();

	View.BlockHashes = MakeArrayView(reinterpret_cast<const FIoBlockHash*>(ContainerData.GetData()), Container.BlockCount);
	ContainerData += View.BlockHashes.NumBytes();

	View.TagSets = MakeArrayView(reinterpret_cast<const FOnDemandTagSetEntry*>(ContainerData.GetData()), Container.TagSetCount);
	ContainerData += View.TagSets.NumBytes();

	View.TagSetIndices = MakeArrayView(reinterpret_cast<const uint32*>(ContainerData.GetData()), Container.TagSetIndicesCount);
	ContainerData += View.TagSetIndices.NumBytes();

	if (ContainerData.IsEmpty() == false)
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc)
			<< TEXT("Unexpected container data size in '") << Filename << TEXT("'");
	}

	View.ContainerHeaderChunk = MoveTemp(ContainerHeaderChunk);

	return View;
}

TIoStatusOr<FOnDemandTocReader> FOnDemandTocReader::Read(const FString& Filename)
{
	IPlatformFile& Ipf = FPlatformFileManager::Get().GetPlatformFile();

	FFileOpenResult OpenResult = Ipf.OpenRead(*Filename, IPlatformFile::EOpenReadFlags::None);
	if (OpenResult.HasError())
	{
		FFileSystemError& Error = OpenResult.GetError();
		return FIoStatusBuilder(EIoErrorCode::FileOpenFailed)
			<< TEXT("Failed to open '") << Filename << TEXT("' reason: ")
			<< Error.GetMessage();
	}

	TUniquePtr<IFileHandle> FileHandle = OpenResult.StealValue();

	FOnDemandTocHeader Header;
	if (FileHandle->Read(reinterpret_cast<uint8*>(&Header), sizeof(FOnDemandTocHeader)) == false)
	{
		return FIoStatusBuilder(EIoErrorCode::ReadError)
			<< TEXT("Failed to read container TOC header from '") << Filename << TEXT("'");
	}

	if (Header.Signature.IsValid() == false)
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc)
			<< TEXT("Invalid TOC signature found in '") << Filename << TEXT("'");
	}

	if (Header.Version.IsValid() == false)
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc)
			<< TEXT("Invalid TOC version found in '") << Filename << TEXT("'");
	}

	if (Header.Version.Major != uint16(EOnDemandTocMajorVersion::Latest) || Header.Version.Minor != uint16(EOnDemandTocMinorVersion::Latest))
	{
		return FIoStatusBuilder(EIoErrorCode::InvalidParameter)
			<< TEXT("Unsupported TOC version found in '") << Filename << TEXT("'");
	}

	const int64 TotalHeaderSize =
		int64(sizeof(FOnDemandTocHeader)) +
		int64((sizeof(UTF8CHAR) * Header.StringTableLen)) +
		int64((sizeof(FOnDemandContainerEntry) * Header.ContainerCount));

	if ((TotalHeaderSize < int64(sizeof(FOnDemandTocHeader))) || (TotalHeaderSize >= FileHandle->Size()))
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc)
			<< TEXT("TOC header size greater than file size in '") << Filename << TEXT("'");
	}

	FIoBuffer Buffer(TotalHeaderSize);
	FMutableMemoryView BufferView = Buffer.GetMutableView();
	FMemory::Memcpy(BufferView.GetData(), &Header, sizeof(FOnDemandTocHeader));
	BufferView += sizeof(FOnDemandTocHeader);

	if (FileHandle->Read(reinterpret_cast<uint8*>(BufferView.GetData()), BufferView.GetSize()) == false)
	{
		return FIoStatusBuilder(EIoErrorCode::ReadError)
			<< TEXT("Failed to read container TOC header from '") << Filename << TEXT("'");
	}

	if (FileHandle->Seek(FileHandle->Size() - int64(sizeof(FOnDemandTocFooter))) == false)
	{
		return FIoStatusBuilder(EIoErrorCode::FileSeekFailed)
			<< TEXT("Failed to seek to container TOC footer in '") << Filename << TEXT("'");
	}

	FOnDemandTocFooter Footer;
	if (FileHandle->Read(reinterpret_cast<uint8*>(&Footer), sizeof(FOnDemandTocFooter)) == false)
	{
		return FIoStatusBuilder(EIoErrorCode::ReadError)
			<< TEXT("Failed to read container TOC footer from '") << Filename << TEXT("'");
	}

	if (Footer.Signature.IsValid() == false)
	{
		return FIoStatusBuilder(EIoErrorCode::CorruptToc)
			<< TEXT("Invalid TOC footer signature found in '") << Filename << TEXT("'");
	}

	return FOnDemandTocReader(Filename, Buffer);
}

} // V2

} // UE::IoStore::Serialization

////////////////////////////////////////////////////////////////////////////////
FStringBuilderBase& operator<<(FStringBuilderBase& Sb, UE::IoStore::Serialization::EOnDemandContainerEntryFlags Flags)
{
	using namespace UE::IoStore::Serialization;

	static const TCHAR* FlagText[]
	{
		TEXT("None"),
		TEXT("InstallOnDemand"),
		TEXT("StreamOnDemand"),
	};

	if (Flags == EOnDemandContainerEntryFlags::None)
	{
		Sb << FlagText[0];
		return Sb;
	}

	constexpr uint32 BitCount = 1 + FMath::CountTrailingZeros(
		static_cast<std::underlying_type_t<EOnDemandContainerEntryFlags>>(EOnDemandContainerEntryFlags::Last));
	static_assert(UE_ARRAY_COUNT(FlagText) == BitCount + 1, "Please update flag text list");

	for (int32 Idx = 0; Idx < BitCount; ++Idx)
	{
		const EOnDemandContainerEntryFlags FlagToTest = static_cast<EOnDemandContainerEntryFlags>(1 << Idx);
		if (EnumHasAnyFlags(Flags, FlagToTest))
		{
			if (Sb.Len())
			{
				Sb << TEXT("|");
			}
			Sb << FlagText[Idx + 1];
		}
	}

	return Sb;
}

////////////////////////////////////////////////////////////////////////////////
FString LexToString(UE::IoStore::Serialization::EOnDemandContainerEntryFlags Flags)
{
	TStringBuilder<128> Sb;
	Sb << Flags;
	return FString::ConstructFromPtrSize(Sb.ToString(), Sb.Len());
}

