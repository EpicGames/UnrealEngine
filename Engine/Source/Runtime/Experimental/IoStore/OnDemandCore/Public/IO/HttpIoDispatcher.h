// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/AnsiString.h"
#include "Containers/SharedString.h"
#include "Delegates/Delegate.h"
#include "IO/IoBuffer.h"
#include "IO/IoHash.h"
#include "IO/IoOffsetLength.h"
#include "IO/IoStatus.h"
#include "Misc/EnumClassFlags.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/RefCounting.h"

#define UE_API IOSTOREONDEMANDCORE_API

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogHttpIoDispatcher, Log, All);

////////////////////////////////////////////////////////////////////////////////////////////////////
#if !defined(UE_HTTP_METADATA_ENABLED)
	#if !UE_BUILD_SHIPPING
		#define UE_HTTP_METADATA_ENABLED UE_TRACE_ENABLED
	#else
		#define UE_HTTP_METADATA_ENABLED 0
	#endif
#endif

class FIoChunkId;

namespace UE
{

namespace IoStore { class FOnDemandHttpStats; }
using FOnDemandHttpStats = IoStore::FOnDemandHttpStats;

class IIoHttpRequestHandle;

/** Flags for controlling the behavior of an HTTP request. */
enum class EIoHttpFlags
{
	/** No additional flags. */
	None			= 0,
	/** Whether to read from the HTTP cache. */
	ReadCache		= (1 << 0),
	/** Whether to include response headers. */
	ResponseHeaders	= (1 << 1),
	/** Default flags. */
	Default			= ReadCache
};
ENUM_CLASS_FLAGS(EIoHttpFlags);

/** Represents a range within a resource. */
struct FIoHttpRange
{
										FIoHttpRange() = default;
	explicit inline						FIoHttpRange(uint32 InMin, uint32 InMax);

	[[nodiscard]] bool					IsValid()	const	{ return Min <= Max; }
	[[nodiscard]] uint32				GetMin()	const	{ return Min; }
	[[nodiscard]] uint32				GetMax()	const	{ return Max; }
	[[nodiscard]] uint32				GetSize()	const	{ return (Max - Min); }
	[[nodiscard]] inline FIoHttpRange	Expand(const FIoHttpRange& Other) const;
	inline FIoHttpRange&				ExpandInline(const FIoHttpRange& Other);

	[[nodiscard]] inline FIoOffsetAndLength ToOffsetAndLength() const;
	static inline FIoHttpRange			FromOffsetAndLength(const FIoOffsetAndLength& OffsetAndLength);

	[[nodiscard]] bool					operator==(const FIoHttpRange& Other)	const	{ return Min == Other.Min && Max == Other.Max; }
	[[nodiscard]] bool					operator!=(const FIoHttpRange& Other)	const	{ return Min != Other.Min || Max != Other.Max; }
	[[nodiscard]] FIoHttpRange			operator+(const FIoHttpRange& Other)	const	{ return Expand(Other); }
	FIoHttpRange&						operator+=(const FIoHttpRange& Other)			{ return ExpandInline(Other); }

private:
	uint32 Min = MAX_uint32;
	uint32 Max = MIN_uint32;
};

FIoHttpRange::FIoHttpRange(uint32 InMin, uint32 InMax)
	: Min(InMin)
	, Max(InMax)
{
}

FIoHttpRange& FIoHttpRange::ExpandInline(const FIoHttpRange& Other)
{
	if (IsValid())
	{
		Min = FMath::Min(Min, Other.Min);
		Max = FMath::Max(Max, Other.Max);
	}
	else
	{
		Min = Other.Min;
		Max = Other.Max;
	}

	return *this;
}

FIoHttpRange FIoHttpRange::Expand(const FIoHttpRange& Other) const
{
	FIoHttpRange Expanded(*this);
	Expanded.ExpandInline(Other);
	return Expanded;
}

FIoOffsetAndLength FIoHttpRange::ToOffsetAndLength() const
{
	return IsValid() ? FIoOffsetAndLength(Min, Max - Min) : FIoOffsetAndLength(0, 0);
}

FIoHttpRange FIoHttpRange::FromOffsetAndLength(const FIoOffsetAndLength& OffsetAndLength)
{
	const uint64 End = OffsetAndLength.GetOffset() + OffsetAndLength.GetLength();

	if (OffsetAndLength.GetOffset() > MAX_uint32 || End > MAX_uint32)
	{
		return FIoHttpRange();
	}

	return FIoHttpRange(IntCastChecked<uint32>(OffsetAndLength.GetOffset()), IntCastChecked<uint32>(End));
}

/** Non shipping metadata about an HTTP request. */
class FIoHttpMetadata
{
public:
	UE_API						FIoHttpMetadata();
	UE_API						FIoHttpMetadata(const FIoChunkId& ChunkId);
	UE_API						~FIoHttpMetadata();
	UE_API void					SetChunkId(const FIoChunkId& ChunkId);
	UE_API const FIoChunkId&	GetChunkId() const;

private:
#if UE_HTTP_METADATA_ENABLED
	struct FMetadata;
	TSharedPtr<FMetadata>		Metadata;
#endif
};

/** Options for controlling the behavior of an HTTP request. */
struct FIoHttpOptions
{
public:
	inline						FIoHttpOptions(int32 InPriority, int32 InRetryCount, EIoHttpFlags InFlags, const FIoHttpRange& InRange, const FIoHash& CacheKey, FIoHttpMetadata Metadata);
	inline						FIoHttpOptions(int32 InPriority, int32 InRetryCount, EIoHttpFlags InFlags, const FIoHttpRange& InRange, const FIoHash& CacheKey);
	inline						FIoHttpOptions(int32 InPriority, int32 InRetryCount, EIoHttpFlags InFlags, const FIoHttpRange& InRange);
	inline						FIoHttpOptions(int32 InPriority, int32 InRetryCount, EIoHttpFlags InFlags);
	inline						FIoHttpOptions(int32 InPriority, int32 InRetryCount);
	inline explicit				FIoHttpOptions(int32 InPriority);
								FIoHttpOptions() = default;

	const FIoHash				GetCacheKey()	const { return CacheKey; }
	const FIoHttpRange&			GetRange()		const { return Range; }
	int32						GetPriority()	const { return Priority; }
	int32						GetRetryCount()	const { return RetryCount; }
	EIoHttpFlags				GetFlags()		const { return Flags; }
	uint8						GetCategory()	const { return Category; }
	const FIoHttpMetadata&		GetMetadata()	const { return Metadata; }

	void						SetCacheKey(const FIoHash& InCacheKey)	{ CacheKey = InCacheKey; }
	void						SetRange(const FIoHttpRange InRange)	{ Range = InRange; }
	void						SetPriority(int32 InPriority)			{ Priority = InPriority; }
	void						SetRetryCount(int32 InRetryCount)		{ RetryCount = InRetryCount; }
	void						SetCategory(uint8 InCategory)			{ Category = InCategory; }
	void						SetMetadata(FIoHttpMetadata&& InMeta)	{ Metadata = MoveTemp(InMeta); }

	UE_API static const FIoHttpOptions Default;

private:
	FIoHash			CacheKey = FIoHash::Zero;
	FIoHttpRange	Range;
	int32			Priority = 0;
	int32			RetryCount = 0;
	EIoHttpFlags	Flags = EIoHttpFlags::Default;
	uint8			Category = 0;
	FIoHttpMetadata	Metadata;
};

FIoHttpOptions::FIoHttpOptions(int32 InPriority, int32 InRetryCount, EIoHttpFlags InFlags, const FIoHttpRange& InRange, const FIoHash& InCacheKey, FIoHttpMetadata InMetadata)
	: CacheKey(InCacheKey)
	, Range(InRange)
	, Priority(InPriority)
	, RetryCount(InRetryCount)
	, Flags(InFlags)
	, Metadata(InMetadata)
{
}

FIoHttpOptions::FIoHttpOptions(int32 InPriority, int32 InRetryCount, EIoHttpFlags InFlags, const FIoHttpRange& InRange, const FIoHash& InCacheKey)
	: CacheKey(InCacheKey)
	, Range(InRange)
	, Priority(InPriority)
	, RetryCount(InRetryCount)
	, Flags(InFlags)
{
}

FIoHttpOptions::FIoHttpOptions(int32 InPriority, int32 InRetryCount, EIoHttpFlags InFlags, const FIoHttpRange& InRange)
	: Range(InRange)
	, Priority(InPriority)
	, RetryCount(InRetryCount)
	, Flags(InFlags)
{
}

FIoHttpOptions::FIoHttpOptions(int32 InPriority, int32 InRetryCount, EIoHttpFlags InFlags)
	: Priority(InPriority)
	, RetryCount(InRetryCount)
	, Flags(InFlags)
{
}

FIoHttpOptions::FIoHttpOptions(int32 InPriority, int32 InRetryCount)
	: Priority(InPriority)
	, RetryCount(InRetryCount)
{
}

FIoHttpOptions::FIoHttpOptions(int32 InPriority)
	: Priority(InPriority)
{
}

/** HTTP headers. */
class FIoHttpHeaders
{
public:
									FIoHttpHeaders() = default;
									FIoHttpHeaders(const FIoHttpHeaders&) = default;
									FIoHttpHeaders(FIoHttpHeaders&&) = default;
	UE_API FIoHttpHeaders&			Add(FAnsiString&& HeaderName, FAnsiString&& HeaderValue);
	UE_API FIoHttpHeaders&			Add(FAnsiStringView HeaderName, FAnsiStringView HeaderValue);
	UE_API FAnsiStringView			Get(FAnsiStringView Key) const;
	UE_API FAnsiStringView			Get(const FAnsiString& Key) const;
	UE_API TArray<FAnsiString>		ToArray() &&;
	TConstArrayView<FAnsiString>	ToArrayView() const { return Headers; }

	UE_API static FIoHttpHeaders	Create(FAnsiString&& HeaderName, FAnsiString&& HeaderValue);
	UE_API static FIoHttpHeaders	Create(FAnsiStringView HeaderName, FAnsiStringView HeaderValue);
	UE_API static FIoHttpHeaders	Create(TArray<FAnsiString>&& Headers);
	
	FIoHttpHeaders&					operator=(const FIoHttpHeaders& Other)	{ Headers = Other.Headers; return *this; }
	FIoHttpHeaders&					operator=(FIoHttpHeaders&& Other)		{ Headers = MoveTemp(Other.Headers); return *this; }

private:
									FIoHttpHeaders(TArray<FAnsiString>&& InHeaders)
										: Headers(MoveTemp(InHeaders))
									{ }
	//TODO: Make this better
	TArray<FAnsiString> Headers;
};

/** A relative URL from the host. */
class FIoRelativeUrl
{
public:
									using ElementType = ANSICHAR;

									FIoRelativeUrl() = default;
	bool							IsEmpty() const		{ return Url.IsEmpty(); }
	int32							Len() const			{ return Url.Len(); }
	const ANSICHAR*					ToString() const	{ return *Url; }
	FAnsiStringView					GetView() const		{ return FAnsiStringView(*Url, Url.Len()); }

	const ANSICHAR*					operator*() const	{ return *Url; }
	friend inline bool				operator==(const FIoRelativeUrl& Lhs, const FIoRelativeUrl& Rhs)	{ return Lhs.Url == Rhs.Url; }
	friend inline bool				operator!=(const FIoRelativeUrl& Lhs, const FIoRelativeUrl& Rhs)	{ return Lhs.Url != Rhs.Url; }
	friend inline bool				operator<(const FIoRelativeUrl& Lhs, const FIoRelativeUrl& Rhs)		{ return Lhs.Url < Rhs.Url; }

	friend inline uint32			GetTypeHash(const FIoRelativeUrl& RelativeUrl) { return GetTypeHash(RelativeUrl.Url); }
	UE_API static FIoRelativeUrl	From(FAnsiStringView Url); 

private:
	explicit FIoRelativeUrl(FAnsiStringView InUrl)
		: Url(InUrl) { }

	TSharedString<ElementType> Url;
};

/**
 * An HTTP request handle.
 *
 * The handle needs to be keept alive until the completion callback has been triggered.
 */
class FIoHttpRequest
{
public:
							FIoHttpRequest() = default;
	explicit 				FIoHttpRequest(IIoHttpRequestHandle* InHandle)
								: Handle(InHandle) { }
							FIoHttpRequest(FIoHttpRequest&& Other)
								: Handle(MoveTemp(Other.Handle)) { }

	bool					IsValid() const { return Handle.IsValid(); }
	UE_API void				Cancel();
	UE_API void				UpdatePriorty(int32 NewPriority);
	UE_API EIoErrorCode		Status() const;
	IIoHttpRequestHandle*	GetHandle() { return Handle.GetReference(); }

	FIoHttpRequest&			operator=(const FIoHttpRequest&) = delete;
	UE_API FIoHttpRequest&	operator=(FIoHttpRequest&& Other);

private:
							FIoHttpRequest(const FIoHttpRequest& Other)
								: Handle(Other.Handle) { }

	TRefCountPtr<IIoHttpRequestHandle> Handle;
};

/** Flags describing a HTTP response. */
enum class EIoHttpResponseFlags : uint8
{
	/** No additional flags. */
	None			= 0,
	/** The response was retrieved from the cache. */ 
	Cached			= (1 << 0),
};
ENUM_CLASS_FLAGS(EIoHttpResponseFlags);

/** A HTTP response. */
class FIoHttpResponse
{
public:
							FIoHttpResponse() = delete;
							~FIoHttpResponse() = default;

	inline					FIoHttpResponse(const FIoHash& InCacheKey, const FIoBuffer& InBody, EIoErrorCode InErrorCode, uint32 InStatusCode, EIoHttpResponseFlags InFlags);
	inline					FIoHttpResponse(const FIoHash& InCacheKey, FIoHttpHeaders&& InHeaders, FIoBuffer&& InBody, EIoErrorCode InErrorCode, uint32 InStatusCode, EIoHttpResponseFlags InFlags);
							FIoHttpResponse(EIoErrorCode InErrorCode, uint32 InStatusCode)
								: ErrorCode(InErrorCode)
								, StatusCode(InStatusCode) { }

	const FIoHttpHeaders&	GetHeaders()	const { return Headers; }
	const FIoBuffer&		GetBody()		const { return Body; }
	const FIoHash&			GetCacheKey()	const { return CacheKey; }
	EIoErrorCode			GetErrorCode()	const { return ErrorCode; }
	uint32					GetStatusCode()	const { return StatusCode; }
	EIoHttpResponseFlags	GetFlags()		const { return Flags; }
	bool					IsOk()			const { return ErrorCode == EIoErrorCode::Ok && StatusCode > 199 && StatusCode < 300; }
	bool					IsCancelled()	const { return ErrorCode == EIoErrorCode::Cancelled; }
	bool					IsCached()		const { return EnumHasAnyFlags(Flags, EIoHttpResponseFlags::Cached); } 

private:
	FIoHash					CacheKey;
	FIoHttpHeaders			Headers;
	FIoBuffer				Body;
	EIoErrorCode			ErrorCode;
	uint32					StatusCode = 0;
	EIoHttpResponseFlags	Flags = EIoHttpResponseFlags::None;
};

FIoHttpResponse::FIoHttpResponse(const FIoHash& InCacheKey, const FIoBuffer& InBody, EIoErrorCode InErrorCode, uint32 InStatusCode, EIoHttpResponseFlags InFlags)
	: CacheKey(InCacheKey)
	, Body(InBody)
	, ErrorCode(InErrorCode)
	, StatusCode(InStatusCode)
	, Flags(InFlags)
{
}

FIoHttpResponse::FIoHttpResponse(const FIoHash& InCacheKey, FIoHttpHeaders&& InHeaders, FIoBuffer&& InBody, EIoErrorCode InErrorCode, uint32 InStatusCode, EIoHttpResponseFlags InFlags)
	: CacheKey(InCacheKey)
	, Headers(MoveTemp(InHeaders))
	, Body(MoveTemp(InBody))
	, ErrorCode(InErrorCode)
	, StatusCode(InStatusCode)
	, Flags(InFlags)
{
}

/** HTTP completion callback. */
using FIoHttpRequestCompleted = TUniqueFunction<void(FIoHttpResponse&&)>;

/** Issue one or more HTTP request in a batch. */
class FIoHttpBatch
{
public:
										FIoHttpBatch(const FIoHttpRequest&) = delete;
										FIoHttpBatch(FIoHttpBatch&& Other)
											: First(Other.First), Last(Other.Last)
										{
											Other.First = 0;
											Other.Last = 0;
										}

	[[nodiscard]] UE_API FIoHttpRequest	Get(const FName& HostGroup,
											const FIoRelativeUrl& RelativeUrl,
											FIoHttpHeaders&& Headers,
											const FIoHttpOptions& Options,
											const FIoHash& ChunkHash,
											FIoHttpRequestCompleted&& OnCompleted);
	[[nodiscard]] UE_API FIoHttpRequest	Get(const FName& HostGroup,
											const FIoRelativeUrl& RelativeUrl,
											FIoHttpHeaders&& Headers,
											const FIoHttpOptions& Options,
											FIoHttpRequestCompleted&& OnCompleted);
	[[nodiscard]] UE_API FIoHttpRequest	Get(const FName& HostGroup,
											const FIoRelativeUrl& RelativeUrl,
											FIoHttpHeaders&& Headers,
											FIoHttpRequestCompleted&& OnCompleted);
	[[nodiscard]] UE_API FIoHttpRequest	Get(const FName& HostGroup,
											const FIoRelativeUrl& RelativeUrl,
											FIoHttpRequestCompleted&& OnCompleted);
	UE_API void							Issue();

	FIoHttpBatch&						operator=(const FIoHttpBatch&) = delete;
	FIoHttpBatch&						operator=(FIoHttpBatch&& Other) = delete;

private:
										friend class FHttpIoDispatcher;
										FIoHttpBatch() = default;

	// The batch relies and the caller keeping a reference to each handle
	IIoHttpRequestHandle* First = nullptr;
	IIoHttpRequestHandle* Last = nullptr;
};

/** HTTP I/O dispatcher . */
class FHttpIoDispatcher
{
public:
	[[nodiscard]] UE_API static bool			IsInitialized();
	[[nodiscard]] UE_API static FIoStatus		Initialize(TSharedPtr<class IHttpIoDispatcher> Dispatcher);
	[[nodiscard]] UE_API static FIoStatus		Shutdown();

				  UE_API static void			GetHttpStats(FOnDemandHttpStats& Out);
	[[nodiscard]] UE_API static FIoStatus		RegisterHostGroup(const FName& HostGroup, TConstArrayView<FAnsiString> HostNames, FAnsiStringView TestUrl);
	[[nodiscard]] UE_API static FIoStatus		RegisterHostGroup(const FName& HostGroup, TConstArrayView<FAnsiString> HostNames);
	[[nodiscard]] UE_API static FIoStatus		RegisterHostGroup(const FName& HostGroup, FAnsiStringView HostName, FAnsiStringView TestUrl);
	[[nodiscard]] UE_API static FIoStatus		RegisterHostGroup(const FName& HostGroup, FAnsiStringView HostName);
	[[nodiscard]] UE_API static bool			IsHostGroupRegistered(const FName& HostGroup);
	[[nodiscard]] UE_API static bool			IsHostGroupOk(const FName& HostGroup);

	[[nodiscard]] UE_API static FIoHttpBatch	NewBatch();
	[[nodiscard]] UE_API static FIoHttpRequest	Get(const FName& HostGroup,
													const FIoRelativeUrl& RelativeUrl,
													FIoHttpHeaders&& Headers,
													const FIoHttpOptions& Options,
													FIoHttpRequestCompleted&& OnCompleted);
	[[nodiscard]] UE_API static FIoHttpRequest	Get(const FName& HostGroup,
													const FIoRelativeUrl& RelativeUrl,
													FIoHttpHeaders&& Headers,
													FIoHttpRequestCompleted&& OnCompleted);
	[[nodiscard]] UE_API static FIoHttpRequest	Get(const FName& HostGroup,
													const FIoRelativeUrl& RelativeUrl,
													FIoHttpRequestCompleted&& OnCompleted);

	[[nodiscard]] UE_API static FIoStatus		CacheResponse(const FIoHttpResponse& Response);
	[[nodiscard]] UE_API static FIoStatus		EvictFromCache(const FIoHttpResponse& Response);

	DECLARE_MULTICAST_DELEGATE_OneParam(FHostGroupRegistered, const FName&);
	UE_API static FHostGroupRegistered&			OnHostGroupRegistered();
};

/** A reference counted HTTP I/O dispatcher request handle. */
class IIoHttpRequestHandle
	: public IRefCountedObject
{
public: 
	virtual EIoErrorCode GetStatus() const = 0;
};

/** HTTP I/O dispatcher interface. */
class IHttpIoDispatcher
{
public:
	using FHostGroupRegistered		= FHttpIoDispatcher::FHostGroupRegistered;

	virtual							~IHttpIoDispatcher() = default;
	virtual void					Shutdown() = 0;
	
	virtual void					GetHttpStats(FOnDemandHttpStats& Out) const = 0;
	virtual FIoStatus				RegisterHostGroup(const FName& HostGroup, TConstArrayView<FAnsiString> HostNames, FAnsiStringView TestUrl) = 0;
	virtual bool					IsHostGroupRegistered(const FName& HostGroup) = 0;
	virtual bool					IsHostGroupOk(const FName& HostGroup) = 0;
	virtual FHostGroupRegistered&	OnHostGroupRegistered() = 0;

	virtual FIoHttpRequest			CreateRequest(
										IIoHttpRequestHandle*& First,
										IIoHttpRequestHandle*& Last,
										const FName& HostGroup,
										const FIoRelativeUrl& RelativeUrl,
										const FIoHttpOptions& Options,
										FIoHttpHeaders&& Headers,
										FIoHttpRequestCompleted&& OnCompleted,
										const FIoHash* ChunkHash = nullptr) = 0;
	virtual void					IssueRequest(IIoHttpRequestHandle& Handle) = 0;
	virtual void					CancelRequest(FIoHttpRequest Handle) = 0;
	virtual void					UpdateRequestPriority(FIoHttpRequest Handle, int32 NewPriority) = 0;

	virtual FIoStatus				CacheResponse(const FIoHttpResponse& Response) = 0;
	virtual FIoStatus				EvictFromCache(const FIoHttpResponse& Response) = 0;
};

} // namespace UE

#undef UE_API
