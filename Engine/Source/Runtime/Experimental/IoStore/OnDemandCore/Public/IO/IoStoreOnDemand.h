// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Containers/StringFwd.h"
#include "Containers/SharedString.h"
#include "Containers/StaticArray.h"
#include "Experimental/UnifiedError/UnifiedError.h"
#include "IO/OnDemandHostGroup.h"
#include "IO/OnDemandToc.h"
#include "IO/PackageId.h"
#include "Misc/EnumClassFlags.h"
#include "Features/IModularFeature.h"
#include "Misc/EnumRange.h"
#include "Misc/Optional.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API IOSTOREONDEMANDCORE_API

struct FAnalyticsEventAttribute;

// Custom initialization allows users to control when
// the system should be initialized.
#if !defined(UE_IAS_CUSTOM_INITIALIZATION)
	#define UE_IAS_CUSTOM_INITIALIZATION 0
#endif

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogIoStoreOnDemand, Log, All);
UE_API DECLARE_LOG_CATEGORY_EXTERN(LogIas, Log, All);

namespace UE::IoStore
{

using FSharedOnDemandIoStore				= TSharedPtr<class IOnDemandIoStore, ESPMode::ThreadSafe>;
using FWeakOnDemandIoStore					= TWeakPtr<class IOnDemandIoStore, ESPMode::ThreadSafe>;
using FSharedInternalContentHandle			= TSharedPtr<class FOnDemandInternalContentHandle, ESPMode::ThreadSafe>;
using FSharedInternalInstallRequest			= TSharedPtr<class FOnDemandInternalInstallRequest, ESPMode::ThreadSafe>;
using FSharedOnDemandDevelopmentExtension	= TSharedPtr<class IOnDemandDevelopmentExtension, ESPMode::ThreadSafe>;
;

/** Interface for on-demand request types. */
class FOnDemandRequest
{
public:
	/** Request status. */
	enum EStatus : uint8
	{
		/** The request has not been issued. */
		None,
		/** The request is pending. */
		Pending,
		/** The request is issuing callbacks. */
		PendingCallbacks,
		/** The request completed successfully. */
		Ok,
		/** The request was cancelled. */
		Cancelled,
		/** The request failed. */
		Error
	};
	/** Destructor. */
	virtual ~FOnDemandRequest() = default;
	/** Returns the current status. */
	virtual EStatus GetStatus() const = 0;
	/** Returns true if the request is invalid. */
	bool IsNone() const { return EStatus::None == GetStatus(); }
	/** Returns true if the request is pending. */
	bool IsPending() const { return EStatus::Pending == GetStatus(); }
	/** Returns true if the request is issuing callbacks. */
	bool IsPendingCallbacks() const { return EStatus::PendingCallbacks == GetStatus(); }
	/** Returns true if the request was successful. */
	bool IsOk() const { return EStatus::Ok == GetStatus(); }
	/** Returns true if the request was cancelled. */
	bool IsCancelled() const { return EStatus::Cancelled == GetStatus(); }
	/** Returns true if the request was unsuccessful. */
	bool IsError() const { return EStatus::Error == GetStatus(); }
	/** Returns true if the request is completed. */
	bool IsCompleted() const { return static_cast<uint8>(EStatus::Pending) < static_cast<uint8>(GetStatus()); }

protected:
	FOnDemandRequest() = default;
};

/**
 * Keeps referenced data pinned in the cache until released.
 */
class FOnDemandContentHandle
{
public:
	/** Creates a new invalid content handle. */ 
	FOnDemandContentHandle() = default;
	/** Destroy the handle and release any referenced content. */ 
	~FOnDemandContentHandle() = default;

	FOnDemandContentHandle(const FOnDemandContentHandle& Other) = default;
	FOnDemandContentHandle(FOnDemandContentHandle&& Other) = default;

	FOnDemandContentHandle& operator=(const FOnDemandContentHandle& Other) = default;
	FOnDemandContentHandle& operator=(FOnDemandContentHandle&& Other) = default;

	/** Destroy the handle and release any referenced content. */ 
	void Reset() { Handle.Reset(); }
	/** Returns whether the handle is valid. */
	bool IsValid() const { return Handle.IsValid(); }
	/** Create a new content handle .*/
	UE_API static FOnDemandContentHandle Create();
	/** Create a new content handle with a debug name. */
	UE_API static FOnDemandContentHandle Create(FSharedString DebugName);
	/** Create a new content handle with a debug name. */
	UE_API static FOnDemandContentHandle Create(FStringView DebugName);
	/** Returns a string representing the content handle. */
	UE_API friend FString LexToString(const FOnDemandContentHandle& Handle);

	bool operator==(FOnDemandContentHandle& Other) const
	{
		return Handle == Other.Handle;
	}

private:
	friend class FOnDemandIoStore;
	friend class FOnDemandContentInstaller;
	FSharedInternalContentHandle Handle;
};

/** Arguments for registering a new host group. */ 
struct FOnDemandRegisterHostGroupArgs
{
	/** Name of the host group. */
	FName HostGroupName;
	/** List of host name(s). */ 
	TArray<FString> HostNames;
	/** Test URL. */
	FString TestUrl;
	/** Whether to use secure https. */
	bool bUseSecureHttp = false;
};

/** Holds the result when registering a host group. */
struct FOnDemandRegisterHostGroupResult
{
	/** Returns True if the request succeeded. */
	bool IsOk() const { return Error.IsSet() == false; }
	/** The registered host group. */
	FOnDemandHostGroup HostGroup;
	/** Error information about the request. */ 
	TOptional<UE::UnifiedError::FError> Error;
};

/** Options for controlling the behavior of mounted container(s). */
enum class EOnDemandMountOptions : uint8
{
	/** No additional options. */
	None					= 0,
	/** Trigger callback on game thread. */
	CallbackOnGameThread	= 1 << 2,
	/** Make soft references available */
	WithSoftReferences		= 1 << 3
};
ENUM_CLASS_FLAGS(EOnDemandMountOptions);

/** Arguments for mounting on-demand container TOC(s).
 *
 * On-demand content can be mounted by providing:
 * 1. a serialized TOC
 * 2. a filepath to a TOC on disk
 * 3. a URL from where to fetch the TOC using HTTP
 *
 * The chunk URLs are derived from the provided TOC URL or from the
 * serialized chunks directory property in the TOC, i.e. if the
 * TocRelativeUrl is not specified the TOC ChunksDirectory property
 * needs to form a qualified path from the host.
 * Example:
 * http(s)://<Host>/<TocRelativePath>/chunks/<1-Byte Hex>/<hash>.iochunk
 * http(s)://<Host>/<TOC.ChunksDirectory>/chunks/<1-Byte Hex>/<hash>.iochunk
 *
 * Host group parameters does not need to be specific when mounting on-demand containers.
 * The containers will be deferred until the specified host group is registered.
 */
struct FOnDemandMountArgs
{
	/** Mandatory ID to be used for unmounting all container file(s) included in the TOC. */
	FString MountId;
	/** Relative URL from the primary endpoint from where to download the TOC. */
	FString TocRelativeUrl;
	/** Serialize the TOC from the specified file path. */
	FString FilePath;
	/** Name of a new or existing host group. */
	FName HostGroupName = NAME_None;
	/** List of URLs from where to download the chunks. */
	FOnDemandHostGroup HostGroup;
	/** Mount options. */
	EOnDemandMountOptions Options = EOnDemandMountOptions::None;
};

/** Holds information about a mount request. */
struct FOnDemandMountResult
{
	/** The mount ID used for mounting the container(s). */
	FString MountId;
	/** The status of the mount request. */
	FIoStatus Status;
	/** Duration in seconds. */
	double DurationInSeconds = 0.0;

	/** 
	 * Prints the result to the log.
	 * If the result was a success then nothing will be logged.
	 * If the mount failed on EIoErrorCode::PendingHostGroup then the result will be printed with 'log' verbosity.
	 * All other results will be printed with 'error' verbosity.
	 */
	UE_API void LogResult();
};

/** Mount completion callback. */
using FOnDemandMountCompleted = TUniqueFunction<void(FOnDemandMountResult)>;

#define ON_DEMAND_INSTALL_CAS_TYPE_LIST(XCASTYPE) \
	XCASTYPE(General) \
	XCASTYPE(MMap)

#define ON_DEMAND_INSTALL_DECLARE_CAS_TYPE(CasType) CasType,

/** Specifies a particular CAS Type when relevant */
struct EOnDemandInstallCasTypeNS
{
	enum Enum : uint8
	{
		ON_DEMAND_INSTALL_CAS_TYPE_LIST(ON_DEMAND_INSTALL_DECLARE_CAS_TYPE)
		Count, // Count does not include 'None'
		None = 0xff
	};
};
using EOnDemandInstallCasType = EOnDemandInstallCasTypeNS::Enum;

#undef ON_DEMAND_INSTALL_DECLARE_CAS_TYPE

/** Options for controlling the behavior of the install request. */
enum class EOnDemandInstallOptions : uint16
{
	/** No additional options. */
	None						= 0,
	/** Trigger callback on game thread. */
	CallbackOnGameThread		= 1 << 0,
	/** Follow soft references when gathering packages to install. */
	InstallSoftReferences		= 1 << 1,
	/** Install optional bulk data. */
	InstallOptionalBulkData		= 1 << 2,
	/** Do not install any data, only succeed if all data is already cached. */
	DoNotDownload				= 1 << 3,
	/** Do not return an error if missing iochunk dependencies are missing. */
	AllowMissingDependencies	= 1 << 4
};
ENUM_CLASS_FLAGS(EOnDemandInstallOptions);

/** Arguments for installing/downloading on-demand content. */
struct FOnDemandInstallArgs 
{
	/** Install all content from containers matching this mount ID. */
	FString MountId;
	/** Install content matching a set of tag(s) and optionally the mount ID. */
	TArray<FString> TagSets;
	/** Package ID's to install. */
	TArray<FPackageId> PackageIds;
	/** Content handle. */
	FOnDemandContentHandle ContentHandle;
	/** Install options. */
	EOnDemandInstallOptions Options = EOnDemandInstallOptions::None;
	/** Priority. */
	int32 Priority = 0;
	/** Optional debug name . */
	FSharedString DebugName;
};

/** Holds information about progress for an install request. */
struct FOnDemandInstallProgress
{
	/** The total size of the requested content. */
	uint64 TotalContentSize = 0;
	/** The total size to be installed/downloaded (<= TotalContentSize). */
	uint64 TotalInstallSize = 0;
	/** The size currently installed/downloaded (<= TotalInstallSize). */
	uint64 CurrentInstallSize = 0;

	FOnDemandInstallProgress& Combine(const FOnDemandInstallProgress& Other)
	{
		TotalContentSize += Other.TotalContentSize;
		TotalInstallSize += Other.TotalInstallSize;
		CurrentInstallSize += Other.CurrentInstallSize;
		return *this;
	}

	uint64 GetTotalDownloadSize() const
	{
		return TotalInstallSize;
	}

	uint64 GetAlreadyDownloadedSize() const
	{
		return CurrentInstallSize;
	}

	float GetRelativeProgress() const 
	{
		const double Progress = (TotalInstallSize > 0) ?
			FMath::Clamp(double(CurrentInstallSize) / double(TotalInstallSize), 0.0, 1.0) :
			0;

		return float(Progress);
	}

	float GetAbsoluteProgress() const
	{
		const double Progress = (TotalContentSize > 0) ?
			FMath::Clamp(double(GetCachedSize()) / double(TotalContentSize), 0.0, 1.0) :
			0;

		return float(Progress);
	}

	uint64 GetCachedSize() const 
	{
		return TotalContentSize - TotalInstallSize + CurrentInstallSize;
	}

	uint64 GetTotalSize() const
	{
		return TotalContentSize;
	}
};

/** Install Progress callback. */
using FOnDemandInstallProgressed = TFunction<void(FOnDemandInstallProgress)>;

/** Holds information about an install request. */
struct FOnDemandInstallResult
{
	/** Returns True if the request succeeded. */
	bool IsOk() const { return Error.IsSet() == false; }
	/** Duration in seconds. */
	double DurationInSeconds = 0.0;
	/** Final progress for the install request. */
	FOnDemandInstallProgress Progress;
	/** Error information about the request. */ 
	TOptional<UE::UnifiedError::FError> Error;
};

/** Install completion callback. */
using FOnDemandInstallCompleted = TUniqueFunction<void(FOnDemandInstallResult)>;

/** A single-ownership handle to an install request. */
class FOnDemandInstallRequest final
	: public FOnDemandRequest
{
	friend class FOnDemandIoStore;
public:
	/** Creates an invalid install request. */
	FOnDemandInstallRequest() = default;
	/** Move constructor. */;
	FOnDemandInstallRequest(FOnDemandInstallRequest&&) = default;
	/** Move assignable. */
	FOnDemandInstallRequest& operator=(FOnDemandInstallRequest&&) = default;
	/** Destructor. */
	~FOnDemandInstallRequest() = default;
	/** Returns the current request status. */
	UE_API virtual EStatus GetStatus() const override;
	/** Cancel the install request. */
	UE_API void Cancel();
	/** Update priority of the install request. */
	UE_API void UpdatePriority(int32 NewPriority);

private:
	FOnDemandInstallRequest(FSharedInternalInstallRequest InternalRequest)
		: Request(InternalRequest)
	{}
	FOnDemandInstallRequest(const FOnDemandInstallRequest&) = delete;
	FOnDemandInstallRequest& operator=(const FOnDemandInstallRequest&) = delete;

	FSharedInternalInstallRequest Request;
};

/** Options for controlling the behavior of the purge request. */
enum class EOnDemandPurgeOptions : uint8
{
	/** No additional options. */
	None = 0,
	/** Trigger callback on game thread. */
	CallbackOnGameThread = 1 << 0,
	/** Defrag after purging */
	Defrag = 1 << 1,
};
ENUM_CLASS_FLAGS(EOnDemandPurgeOptions)

/** Arguments for purging on-demand content. */
struct FOnDemandPurgeArgs
{
	/** Cas to purge */
	EOnDemandInstallCasType CasType = EOnDemandInstallCasType::General;
	/** Purge options. */
	EOnDemandPurgeOptions Options = EOnDemandPurgeOptions::None;
	/** Optional Size to purge. If not set will purge all unreferenced blocks. */
	TOptional<uint64> BytesToPurge;
};

/** Holds information about a purge request */
struct FOnDemandPurgeResult
{
	/** Returns True if the request succeeded. */
	bool IsOk() const { return Error.IsSet() == false; }
	/** Duration in seconds. */
	double DurationInSeconds = 0.0;
	/** Error information about the request. */ 
	TOptional<UE::UnifiedError::FError> Error;
};

/** Purge completion callback */
using FOnDemandPurgeCompleted = TUniqueFunction<void(FOnDemandPurgeResult)>;

/** Options for controlling the behavior of the defrag request. */
enum class EOnDemandDefragOptions : uint8
{
	/** No additional options. */
	None = 0,
	/** Trigger callback on game thread. */
	CallbackOnGameThread = 1 << 0,
};
ENUM_CLASS_FLAGS(EOnDemandDefragOptions)

/** Arguments for defragmenting on-demand content. */
struct FOnDemandDefragArgs
{
	/** Cas to defrag */
	EOnDemandInstallCasType CasType = EOnDemandInstallCasType::General;
	/** Defrag options. */
	EOnDemandDefragOptions Options = EOnDemandDefragOptions::None;
	/** Optional Size to free. If not set, will defrag all blocks and free all unreferenced chunks. */
	TOptional<uint64> BytesToFree;
};

/** Holds information about a defrag request */
struct FOnDemandDefragResult
{
	/** Returns True if the request succeeded. */
	bool IsOk() const { return Error.IsSet() == false; }
	/** Duration in seconds. */
	double DurationInSeconds = 0.0;
	/** Error information about the request. */ 
	TOptional<UE::UnifiedError::FError> Error;
};

/** Defrag completion callback */
using FOnDemandDefragCompleted = TUniqueFunction<void(FOnDemandDefragResult)>;

/** Options for controlling the behavior of the install size request. */
enum class EOnDemandGetInstallSizeOptions : uint8
{
	/** No additional options. */
	None					= 0,
	/** Follow soft references when gathering packages to install. */
	IncludeSoftReferences	= 1 << 0,
	/** Include optional bulk data. */ 
	IncludeOptionalBulkData	= 1 << 1
};
ENUM_CLASS_FLAGS(EOnDemandGetInstallSizeOptions);

/** Arguments for getting the size of on-demand content. */
struct FOnDemandGetInstallSizeArgs 
{
	/** Get total install size for containers matching this mount ID. */
	FString MountId;
	/** Get total install size for the specified tag(s) and optionally matching the mount ID. */
	TArray<FString> TagSets;
	/** Get total install size for the specified package IDs. */
	TArray<FPackageId> PackageIds;
	/** Options */
	EOnDemandGetInstallSizeOptions Options = EOnDemandGetInstallSizeOptions::None;
};

/** Holds information about an install size request. */ 
struct FOnDemandInstallSizeResult
{
	/** The total size of the content. */
	uint64	InstallSize = 0;	
};

/** Options for controlling the behavior of the GetIsOnDemand request. */
enum class EOnDemandIsOnDemandOptions : uint8
{
	/** No additional options. */
	None = 0,
	/** Follow soft references when gathering packages. */
	IncludeSoftReferences = 1 << 0,
};
ENUM_CLASS_FLAGS(EOnDemandIsOnDemandOptions);

/** Arguments for checking if any on-demand content exists in the dependency graph. */
struct FOnDemandIsOnDemandArgs
{
	/** Return true if any of the specified package IDs has on-demand dependencies. */
	TArray<FPackageId> PackageIds;
	/** Options */
	EOnDemandIsOnDemandOptions Options = EOnDemandIsOnDemandOptions::None;
};

/** Results for GetIsOnDemand */
struct FOnDemandIsOnDemandResult
{
	bool bIsOnDemand = false;
};

/** Options for controlling the behavior of the install size request. */
enum class EOnDemandGetCacheUsageOptions : uint8
{
	/** No additional options. */
	None = 0,
	/** Dump Handle information to log */
	DumpHandlesToLog = 1 << 0,
	/** Dump Handle information to returned FOnDemandInstallCacheUsage */
	DumpHandlesToResults = 1 << 1,
};
ENUM_CLASS_FLAGS(EOnDemandGetCacheUsageOptions);

/** Arguments for getting the install cache usage. */
struct FOnDemandGetCacheUsageArgs
{
	EOnDemandGetCacheUsageOptions Options = EOnDemandGetCacheUsageOptions::None;
};

struct FOnDemandInstallHandleCacheUsage
{
	UPTRINT			HandleId = 0;
	FSharedString	DebugName;
	uint64			ReferencedBytes = 0;
};

/** Holds information about the install cache usage */
struct FOnDemandInstallCacheUsage
{
	struct FCasUsage
	{
		uint64 MaxSize = 0;
		uint64 TotalSize = 0;
		uint64 ReferencedBlockSize = 0;
		uint64 ReferencedSize = 0;
		uint64 FragmentedChunksSize = 0;
		EOnDemandInstallCasType CasType = EOnDemandInstallCasType::None;

		bool IsValid() const { return CasType != EOnDemandInstallCasType::None && MaxSize != 0; }

		UE_API friend FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Sb, const FCasUsage& CasUsage);
	};

	TStaticArray<FCasUsage, EOnDemandInstallCasType::Count> CasUsage;

	// Only populated if DumpHandlesToResults is specified
	TArray<FOnDemandInstallHandleCacheUsage> ReferencedBytesByHandle;

	UE_API friend FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Sb, const FOnDemandInstallCacheUsage& CacheUsage);
};

/** Holds information about the streaming cache usage */
struct FOnDemandStreamingCacheUsage
{
	uint64 MaxSize = 0;
	uint64 TotalSize = 0;

	UE_API friend FUtf8StringBuilderBase& operator<<(FUtf8StringBuilderBase& Sb, const FOnDemandStreamingCacheUsage& CacheUsage);
};

/** Holds information about install and streaming cache usage */
struct FOnDemandCacheUsage
{
	FOnDemandInstallCacheUsage InstallCache;
	FOnDemandStreamingCacheUsage StreamingCache;
};

/** Result from verifying the install cache. */
struct FOnDemandVerifyCacheResult
{
	/** Returns True if the request succeeded. */
	bool IsOk() const { return Error.IsSet() == false; }
	/** Duration in seconds. */
	double DurationInSeconds = 0.0;
	/** Error information about the request. */ 
	TOptional<UE::UnifiedError::FError> Error;
};

/** Verify cache completion callback. */
using FOnDemandVerifyCacheCompleted = TUniqueFunction<void(FOnDemandVerifyCacheResult)>;

/** Container for http stats */
class FOnDemandHttpStats
{
public:
	enum { SAMPLE_COUNT = 8 };
	UE_API FOnDemandHttpStats();
	UE_API ~FOnDemandHttpStats();
	UE_API uint32 GetRecvKiBps() const;
	UE_API void GetRecvKiBps(uint32 (&Out)[SAMPLE_COUNT]) const;
	UE_API uint32 GetTotalRecvKiB() const;
	UE_API uint32 GetTimeToFirstByteMs() const;

private:
	friend class FMultiEndpointHttpClient;
	TUniquePtr<class IOnDemandInternalHttpStats> Internal;
};

/** Interface for recording analytics over a given time period */
class IAnalyticsRecording
{
public:
	IAnalyticsRecording() = default;
	virtual ~IAnalyticsRecording() = default;

	/** Writes the current value of the analytics to the output array */
	virtual void Report(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const = 0;
	
	/** Calling this will stop recording. Any calls to ::Report after this point will return the same data */
	virtual void StopRecording() = 0;
};

struct FOnDemandImmediateAnalytic
{
	FString EventName;
	TArray<FAnalyticsEventAttribute> AnalyticsArray;
};

struct FOnDemandImmediateAnalyticHandler
{
	TUniqueFunction<void(FOnDemandImmediateAnalytic)> EventHandler;
};

/** Streaming options (IAS). */
enum class EOnDemandStreamingOptions : uint8
{
	/** No special options enabled. */
	Default						= 0,
	/** Optional bulk data disabled. */
	OptionalBulkDataDisabled	= (1 << 0)
};
ENUM_CLASS_FLAGS(EOnDemandStreamingOptions);

/** Interface for installing and streaming content on-demand. */
class IOnDemandIoStore
{
public:
	/** Destroy the I/O store. */
	virtual ~IOnDemandIoStore() = default;
	/** Initialize the I/O store. Called after the module feature has been registered. */
	virtual FIoStatus InitializePostHotfix() = 0;
	/**
	 * Register a new host group.
	 * @param	Args	Request arguments.
	 * @return			Error information about the request.
	 */
	virtual FOnDemandRegisterHostGroupResult RegisterHostGroup(FOnDemandRegisterHostGroupArgs&& Args) = 0;
	/**
	 * Mount an on-demand container.
	 * @param	Args		Mount arguments.
	 * @param	OnCompleted	Completion callback.
	 */
	virtual void Mount(FOnDemandMountArgs&& Args, FOnDemandMountCompleted&& OnCompleted) = 0;
	/**
	 * Unmount all container(s) associated with the specified mount ID.
	 * @param	MountId	The Mount ID passed to the mount call.
	 */
	virtual FIoStatus Unmount(FStringView MountId) = 0;
	/**
	 * Install content.
	 * @param	Args				The install arguments.
	 * @param	OnCompleted			Completion callback.
	 * @param	OnProgress			Optional progress callback.
	 * 
	 * @return A handle representing the install request. @see FOnDemandInstallRequest
	 */
	virtual FOnDemandInstallRequest Install(
			FOnDemandInstallArgs&& Args,
			FOnDemandInstallCompleted&& OnCompleted,
			FOnDemandInstallProgressed&& OnProgress = nullptr) = 0;
	/**
	 * Purge the cache.
	 * @param	Args		The purge arguments.
	 * @param	OnComplete	Completion callback.
	 */
	virtual void Purge(FOnDemandPurgeArgs&& Args, FOnDemandPurgeCompleted&& OnCompleted) = 0;
	/**
	 * Defrag the cache.
	 * @param	Args		The defrag arguments.
	 * @param	OnComplete	Completion callback.
	 */
	virtual void Defrag(FOnDemandDefragArgs&& Args, FOnDemandDefragCompleted&& OnCompleted) = 0;
	/**
	 * Verify the install cache.
	 * @param	OnComplete	Completion callback.
	 */
	virtual void Verify(FOnDemandVerifyCacheCompleted&& OnCompleted) = 0;
	/**
	 * Get the total space in bytes needed to install the specified content.
	 * @param Args	Arguments for computing required install size.
	 */
	virtual TIoStatusOr<FOnDemandInstallSizeResult> GetInstallSize(const FOnDemandGetInstallSizeArgs& Args) const = 0;
	/**
	 * Get the total space in bytes needed to install the specified content.
	 * @param Args	Arguments for computing required install size.
	 * @return		The total size by mount ID.
	 */
	virtual FIoStatus GetInstallSizesByMountId(const FOnDemandGetInstallSizeArgs& Args, TMap<FString, uint64>& OutSizesByMountId) const = 0;
	/**
	 * Query whether any of the specified packages have on-demand dependencies.
	 * @param Args	Array of package sets to query.
	 * @return		One result per package set, in the same order as Args.
	 */
	virtual FIoStatus GetIsOnDemand(TConstArrayView<FOnDemandIsOnDemandArgs> Args, TArrayView<FOnDemandIsOnDemandResult> OutResults) const = 0;
	/**
	 * Query whether any of the specified packages have on-demand dependencies.
	 * @param Args	Package set to query.
	 * @return		Results indicating whether on-demand dependencies exist.
	 */
	UE_API TIoStatusOr<FOnDemandIsOnDemandResult> GetIsOnDemand(const FOnDemandIsOnDemandArgs& Args) const;
	/** Returns the total cache size in bytes. */
	virtual FOnDemandCacheUsage GetCacheUsage(const FOnDemandGetCacheUsageArgs& Args) const = 0;
	/** Dump information about mounted containers to the log */
	virtual void DumpMountedContainersToLog() const = 0;
	/** Return if the OnDemand streaming system is enabled. */
	virtual bool IsOnDemandStreamingEnabled() const = 0;
	/** Set streaming options.*/
	virtual void SetStreamingOptions(EOnDemandStreamingOptions Options) = 0;
	/** Returns statistics about the network */
	virtual void GetHttpStats(FOnDemandHttpStats& Out) const = 0;
	/**
	 * Reports the statistics for the current OnDemandBackend. It will record any events that have occurred from the last time that this was called.
	 * This is a legacy method, prefer using StartAnalyticsRecording if possible.
	 */
	virtual void ReportAnalytics(TArray<FAnalyticsEventAttribute>& OutAnalyticsArray) const = 0;
	/** 
	 * Create a new analytics interface for the current OnDemandBackend, can be used to record statistics for a set period of time.
	 * This can return a nullptr if the analytics system is not enabled. Currently only supports the streaming backend.
	 */
	virtual TUniquePtr<IAnalyticsRecording> StartAnalyticsRecording() const = 0;
	/**
	 * Report immediate analytics events that are not recorded over a timespan.
	 * Useful for analytics that are better reported as single events.
	 */
	virtual void OnImmediateAnalytic(FOnDemandImmediateAnalyticHandler EventHandler) = 0;
	/** Set the development extension. */
	virtual void SetDevelopmentExtension(FSharedOnDemandDevelopmentExtension Ctx) { }
	/** Whether development mode is enabled or not. */
	virtual bool IsDevelopmentModeEnabled() const { return false; }

private:
	/** Initialize the I/O store. Called after the I/O store factory module feature has been registered. */
	virtual FIoStatus Initialize() = 0;
	/** Cancel the install request. */
	virtual void CancelInstallRequest(FSharedInternalInstallRequest InstallRequest) = 0;
	/** Update the priority for the install request. */
	virtual void UpdateInstallRequestPriority(FSharedInternalInstallRequest InstallRequest, int32 NewPriority) = 0;
	/** Release all content referenced by the content handle. */
	virtual void ReleaseContent(class FOnDemandInternalContentHandle& ContentHandle) = 0;

	friend class FIoStoreOnDemandCoreModule;
	friend class FOnDemandInternalContentHandle;
	friend class FOnDemandInstallRequest;
};

/** Module feature for creating a concrete implementation of the on-demand I/O store. */
class IOnDemandIoStoreFactory
	: public IModularFeature
{
public:
	/** Feature name. */
	UE_API static FName FeatureName;
	/** Create a new instance of the I/O store.  Called once the feature has been registered. */
	virtual IOnDemandIoStore* CreateInstance() = 0;
	/** Destroy the instance. */
	virtual void DestroyInstance(IOnDemandIoStore* Instance) = 0;
};

/** Returns the on-demand I/O store if available. */
UE_API IOnDemandIoStore* TryGetOnDemandIoStore();
/** Returns the on-demand I/O store. */
UE_API IOnDemandIoStore& GetOnDemandIoStore();

} // namespace UE::IoStore

UE_API const TCHAR* LexToString(UE::IoStore::EOnDemandInstallCasType CacheType);
UE_API void LexFromString(UE::IoStore::EOnDemandInstallCasType& CacheType, FStringView String);
ENUM_RANGE_BY_COUNT(UE::IoStore::EOnDemandInstallCasType, UE::IoStore::EOnDemandInstallCasType::Count);

#undef UE_API
