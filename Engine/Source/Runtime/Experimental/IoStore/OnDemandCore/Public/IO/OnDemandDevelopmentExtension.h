// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/StringFwd.h"
#include "IO/IoStatus.h"
#include "Serialization/PackageStore.h"

class FIoChunkId;

namespace UE::IoStore
{

/**
 * An interface for exceptional scenarios.
 *
 * Note: When running in IAX development mode (-Iax.DevMode),
 * the on-demand package store and install cache backends
 * are given higher priority than the file-based (installed
 * content) backend. This allows content that has not yet been
 * installed via an install request to be loaded by returning
 * 'Missing' or 'UnknownChunkID'. If the chunk ID exists in the
 * 'global.umeta' file, the corresponding package and/or
 * filename will be provided.
 */
class IOnDemandDevelopmentExtension
{
public:
	virtual ~IOnDemandDevelopmentExtension() = default;

	/** Called when the on-demand package store finds an unreferenced package store entry
	 *
	 * Note: In shipping builds the package store will return 'NotInstalled' for packages
	 * not pinned in the on-demand install cache. This currently halts the
	 * package store traversel and affectively fails the package load request.
	 * Returning 'Missing' will let lower priority package store backends handle
	 * the request. Use 'IsInAsyncLoadingThread() || IsInParallelLoadingThread()' to only
	 * trigger calls from the LoadPackage/Async.
	 */
	virtual EPackageStoreEntryStatus GetUnreferencedPackageStoreEntryStatus(
		FName PackageName,
		const FPackageId& PackageId,
		const FPackageStoreEntry& Entry,
		bool bDevMode) = 0;
	/**
	 * Called when an unreferenced I/O store chunk is loaded via the I/O dispatcher interface.
	 * This happens when the package loader has resolved the package store entry or
	 * when requesting unstructed data via FBulkData.
	 *
	 * Note: In shipping builds the on-demand install cache will return 'NotInstalled'
	 * and fail the request when trying to load an unreferenced I/O store chunk.
	 * Return 'UnknownChunkID' to let lower priority I/O dispacher backends handle
	 * the I/O request. 
	 */
	virtual EIoErrorCode GetUnreferencedIoChunkStatus(
		const FIoChunkId& ChunkId,
		FUtf8StringView Filename,
		FStringView PackageName,
		bool bDevMode) = 0;
};

} // namespace UE::IoStore
