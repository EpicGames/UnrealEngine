// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Experimental/UnifiedError/UnifiedError.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "IO/IoChunkId.h"
#include "IO/IoStatus.h"
#include "IO/IoStoreOnDemand.h"
#include "Templates/ValueOrError.h"

#define UE_API IOSTOREONDEMANDCORE_API

////////////////////////////////////////////////////////////////////////////////

UE_API void SerializeForLog(FCbWriter& Writer, const FIoChunkId& ChunkId);
UE_API void SerializeForLog(FCbWriter& Writer, EIoErrorCode ErrorCode);
UE_API void SerializeForLog(FCbWriter& Writer, const FFileSystemError& Error);

namespace UE::IoStore
{
	UE_API void SerializeForLog(FCbWriter& Writer, EOnDemandInstallCasType CasType);
}

////////////////////////////////////////////////////////////////////////////////
namespace UE::IoStore::OnDemand
{

enum class ECasErrorCode : uint32
{
	None,
	InitializeFailed,
	VerifyFailed,
	ReadBlockFailed,
	WriteBlockFailed,
	DeleteBlockFailed,
	CreateJournalFailed,
	ReplayJournalFailed,
	CommitJournalFailed,
	CreateSnapshotFailed,
	LoadSnapshotFailed,
	SaveSnapshotFailed,
	OpenMappedFailed,
};

inline const TCHAR* ToString(ECasErrorCode Code)
{
	switch (Code)
	{
		case ECasErrorCode::None:
			return TEXT("None");
		case ECasErrorCode::InitializeFailed:
			return TEXT("InitializeFailed");
		case ECasErrorCode::VerifyFailed:
			return TEXT("VerifyFailed");
		case ECasErrorCode::ReadBlockFailed:
			return TEXT("ReadBlockFailed");
		case ECasErrorCode::WriteBlockFailed:
			return TEXT("WriteBlockFailed");
		case ECasErrorCode::DeleteBlockFailed:
			return TEXT("DeleteBlockFailed");
		case ECasErrorCode::CreateJournalFailed:
			return TEXT("CreateJournalFailed");
		case ECasErrorCode::ReplayJournalFailed:
			return TEXT("ReplayJournalFailed");
		case ECasErrorCode::CommitJournalFailed:
			return TEXT("CommitJournalFailed");
		case ECasErrorCode::CreateSnapshotFailed:
			return TEXT("CreateSnapshotFailed");
		case ECasErrorCode::LoadSnapshotFailed:
			return TEXT("LoadSnapshotFailed");
		case ECasErrorCode::SaveSnapshotFailed:
			return TEXT("SaveSnapshotFailed");
		case ECasErrorCode::OpenMappedFailed:
			return TEXT("OpenMappedFailed");
		default:
			return TEXT("<InvalidErrorCode>");
	};
}

inline void SerializeForLog(FCbWriter& Writer, ECasErrorCode Code)
{
	Writer.AddString(ToString(Code));
}

// Cache-state snapshot taken at the moment an install-cache error is produced.
// Used as a plain field on errors that carry cache state directly
// (InstallCachePurgeError, InstallCacheDefragError) and as the sole field of
// the pushable InstallCacheErrorContext, so the same data can be attached to
// errors that originate below the cache layer.
struct FInstallCacheErrorState
{
	FIoStatus		IoError					= FIoStatus(EIoErrorCode::Unknown);
	const TCHAR*	CasType					= nullptr;
	uint64			MaxCacheSize			= 0;
	uint64			CacheSize				= 0;
	uint64			RequestedSize			= 0;
	uint64			DiskTotalBytes			= 0;
	uint64			DiskFreeBytes			= 0;
	uint32			LineNo					= uint32(__builtin_LINE());
	bool			bDiskQuerySucceeded		= false;
};

UE_API void SerializeForLog(FCbWriter& Writer, const FInstallCacheErrorState& State);

} // namespace UE::IoStore::OnDemand

////////////////////////////////////////////////////////////////////////////////
UE_DECLARE_ERROR_MODULE(UE_API, UE::IoStore::OnDemand);

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 1,  HttpError,
	"HTTP error ({StatusCode})",
	(uint32, StatusCode, 0));

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 2,  ChunkMissingError,
	"Chunk missing error",
	(TArray<FIoChunkId>, MissingChunkIds),
	(int32, DiscoveredChunksCount, 0));

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 3,  ChunkHashError,
	"Chunk hash mismatch error",
	(FIoChunkId, ChunkId),
	(FString, ExpectedHash),
	(FString, ActualHash),
	(UE::IoStore::EOnDemandInstallCasType, CasType, UE::IoStore::EOnDemandInstallCasType::None));

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 4,  InstallCacheFlushError,
	"Failed to flush pending data to install cache.");

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 5,  InstallCacheFlushLastAccessError,
	"Failed to flush last access timestamp(s) to journal.");

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 6,  InstallCachePurgeError,
	"Failed to purge unreferenced cache block(s) from the install cache.",
	(UE::IoStore::OnDemand::FInstallCacheErrorState, State));

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 7,  InstallCacheDefragError,
	"Failed to defrag the install cache.",
	(UE::IoStore::OnDemand::FInstallCacheErrorState, State));

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 8,  InstallCacheVerificationError,
	"Verification of installed install cache data failed.",
	(uint32, CorruptChunkCount, 0),
	(uint32, MissingChunkCount, 0),
	(uint32, ReadErrorCount, 0));

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 9,  CasError,
	"Cas error",
	(UE::IoStore::OnDemand::ECasErrorCode, CasErrorCode, UE::IoStore::OnDemand::ECasErrorCode::None),
	(EIoErrorCode, IoErrorCode, EIoErrorCode::Unknown),
	(FString, ErrorMessage),
	(FFileSystemError, FileSystemError, FFileSystemError(FString())),
	(uint32, SystemErrorCode, 0),
	(UE::IoStore::EOnDemandInstallCasType, CasType, UE::IoStore::EOnDemandInstallCasType::None));

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 10, CasJournalError,
	"Cas journal error",
	(UE::IoStore::OnDemand::ECasErrorCode, CasErrorCode, UE::IoStore::OnDemand::ECasErrorCode::None),
	(EIoErrorCode, IoErrorCode, EIoErrorCode::Unknown),
	(FString, ErrorMessage),
	(FFileSystemError, FileSystemError, FFileSystemError(FString())),
	(uint32, SystemErrorCode, 0));

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 11, CasSnapshotError,
	"Cas snapshot error",
	(UE::IoStore::OnDemand::ECasErrorCode, CasErrorCode, UE::IoStore::OnDemand::ECasErrorCode::None),
	(EIoErrorCode, IoErrorCode, EIoErrorCode::Unknown),
	(FString, ErrorMessage),
	(FFileSystemError, FileSystemError, FFileSystemError(FString())),
	(uint32, SystemErrorCode, 0));

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 12, InstallerCacheFullError,
	"{CasType} Cache full error",
	(UE::IoStore::EOnDemandInstallCasType, CasType, UE::IoStore::EOnDemandInstallCasType::None));

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 13, InstallReplayLoadError,
	"Failed to load install replay");

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 14, InstallReplaySaveError,
	"Failed to save install replay");

UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 15, DownloadRequired,
	"DoNotDownload flag specified when download required");

// Previously recorded by pushing an FChunkHashMismatchErrorContext onto an
// InstallCacheDefragError. Now a distinct error identity so callers can
// differentiate hash-consistency failures from the space-related defrag path.
UE_DECLARE_ERROR(UE_API, UE::IoStore::OnDemand, 16, InstallCacheDefragHashMismatchError,
	"Install cache defrag aborted due to chunk hash mismatch.",
	(FIoChunkId, ChunkId),
	(FString, ExpectedHash),
	(FString, ActualHash),
	(UE::IoStore::EOnDemandInstallCasType, CasType, UE::IoStore::EOnDemandInstallCasType::None));

// Pushable form of FInstallCacheErrorState. Use this when the cache layer
// wants to annotate an error that originated at a lower level (e.g. a
// MakeCasError result bubbling up from a defrag read) rather than mint a new
// install-cache error. Errors that own their cache state directly carry
// FInstallCacheErrorState as a field instead.
UE_DECLARE_ERROR_CONTEXT(UE_API, UE::IoStore::OnDemand, InstallCacheErrorContext,
	"({State})",
	(UE::IoStore::OnDemand::FInstallCacheErrorState, State));

////////////////////////////////////////////////////////////////////////////////
namespace UE::IoStore
{

using FResult = TValueOrError<void, UE::UnifiedError::FError>;

template <typename ResultType>
using TResult = TValueOrError<ResultType, UE::UnifiedError::FError>;

} // namespace UE::IoStore

////////////////////////////////////////////////////////////////////////////////
namespace UE::IoStore::OnDemand
{

template <typename ResultType>
inline UE::IoStore::TResult<ResultType> MakeCasError(
	UE::IoStore::EOnDemandInstallCasType CasType,
	ECasErrorCode ErrorCode,
	EIoErrorCode IoErrorCode,
	FString&& ErrorMessage,
	FFileSystemError&& FileSystemError = FFileSystemError(FString()),
	uint32 SystemErrorCode = FPlatformMisc::GetLastError())
{
	return MakeError(CasError(FCasError
	{
		.CasErrorCode		= ErrorCode,
		.IoErrorCode		= IoErrorCode,
		.ErrorMessage		= MoveTemp(ErrorMessage),
		.FileSystemError	= MoveTemp(FileSystemError),
		.SystemErrorCode	= SystemErrorCode,
		.CasType			= CasType
	}));
}

inline UE::IoStore::FResult MakeJournalError(
	ECasErrorCode ErrorCode,
	EIoErrorCode IoErrorCode,
	FString&& ErrorMessage,
	FFileSystemError&& FileSystemError = FFileSystemError(FString()),
	uint32 SystemErrorCode = FPlatformMisc::GetLastError())
{
	return MakeError(CasJournalError(FCasJournalError
	{
		.CasErrorCode		= ErrorCode,
		.IoErrorCode		= IoErrorCode,
		.ErrorMessage		= MoveTemp(ErrorMessage),
		.FileSystemError	= MoveTemp(FileSystemError),
		.SystemErrorCode	= SystemErrorCode
	}));
}

template <typename ResultType>
inline UE::IoStore::TResult<ResultType> MakeSnapshotError(
	ECasErrorCode ErrorCode,
	EIoErrorCode IoErrorCode,
	FString&& ErrorMessage,
	FFileSystemError&& FileSystemError = FFileSystemError(FString()),
	uint32 SystemErrorCode = FPlatformMisc::GetLastError())
{
	return MakeError(CasSnapshotError(FCasSnapshotError
	{
		.CasErrorCode		= ErrorCode,
		.IoErrorCode		= IoErrorCode,
		.ErrorMessage		= MoveTemp(ErrorMessage),
		.FileSystemError	= MoveTemp(FileSystemError),
		.SystemErrorCode	= SystemErrorCode
	}));
}

template <typename ResultType>
inline UE::IoStore::TResult<ResultType> MakeSnapshotError(
	ECasErrorCode ErrorCode,
	EIoErrorCode IoErrorCode,
	FString&& ErrorMessage,
	uint32 SystemErrorCode)
{
	return MakeError(CasSnapshotError(FCasSnapshotError
	{
		.CasErrorCode		= ErrorCode,
		.IoErrorCode		= IoErrorCode,
		.ErrorMessage		= MoveTemp(ErrorMessage),
		.FileSystemError	= FFileSystemError(FString()),
		.SystemErrorCode	= SystemErrorCode
	}));
}

} // namespace UE::IoStore::OnDemand

#undef UE_API
