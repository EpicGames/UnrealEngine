// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/AnsiString.h"
#include "Containers/StringFwd.h"
#include "IO/IoStatus.h"
#include "Templates/SharedPointer.h"

#define UE_API IOSTOREONDEMANDCORE_API

namespace UE::IoStore
{

/** Holds a set of host URLs. */
class FOnDemandHostGroup
{
	struct FImpl;
	using FSharedImpl = TSharedPtr<FImpl, ESPMode::ThreadSafe>;

public:
	/** Creates a new empty host group. */
	UE_API FOnDemandHostGroup();
	/** Destructor. */ 
	UE_API ~FOnDemandHostGroup();
	/** Returns the list of available host URLs. */ 
	UE_API TConstArrayView<FAnsiString> Hosts() const;
	/** Get the URL at the specified index. */
	UE_API FAnsiStringView Host(int32 Index) const;
	/** Get the next available host starting from the specified index. */
	UE_API FAnsiStringView CycleHost(int32& InOutIndex) const;
	/** Set the primary host URL. */
	UE_API void SetPrimaryHost(int32 Index);
	/** Get the primary host URL. */
	UE_API FAnsiStringView PrimaryHost() const;
	/** Get the primary host index. */
	UE_API int32 PrimaryHostIndex() const;
	/** Returns whether the group is empty or not. */
	UE_API bool IsEmpty() const;
	/** Create a new host group with the specified URL. */
	UE_API static TIoStatusOr<FOnDemandHostGroup> Create(FStringView Url);
	/** Create a new host group with the specified URLs. */
	UE_API static TIoStatusOr<FOnDemandHostGroup> Create(TConstArrayView<FAnsiString> Urls);
	/** Create a new host group with the specified URLs. */
	UE_API static TIoStatusOr<FOnDemandHostGroup> Create(TConstArrayView<FString> Urls);
	/** Default host group name. */
	UE_API static FName DefaultName;

	/** Returns whether two host group instances are the same or not. */
	friend inline bool operator==(const FOnDemandHostGroup& Lhs, const FOnDemandHostGroup& Rhs)
	{
		return Lhs.Impl.Get() == Rhs.Impl.Get();
	}
	/** Returns the type hash of the host group. */ 
	friend inline uint32 GetTypeHash(const FOnDemandHostGroup& HostGroup)
	{
		return *reinterpret_cast<const uint32*>(HostGroup.Impl.Get());
	}

private:
	FOnDemandHostGroup(FSharedImpl&& Impl);

	FSharedImpl Impl;
};

} // namespace UE::IoStore

#undef UE_API
