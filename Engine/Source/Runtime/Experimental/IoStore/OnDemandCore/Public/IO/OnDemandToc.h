// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "IO/IoChunkId.h"
#include "IO/IoContainerId.h"
#include "IO/IoHash.h"
#include "IO/IoStatus.h"
#include "Misc/EnumClassFlags.h"

#define UE_API IOSTOREONDEMANDCORE_API

using FIoBlockHash = uint32;

class FArchive;
class FCbWriter;

namespace UE::IoStore
{

///////////////////////////////////////////////////////////////////////////////
enum class EOnDemandTocVersion : uint32
{
	Invalid			= 0,
	Initial			= 1,
	UTocHash		= 2,
	BlockHash32		= 3,
	NoRawHash		= 4,
	Meta			= 5,
	ContainerId		= 6,
	AdditionalFiles	= 7,
	TagSets			= 8,
	ContainerFlags	= 9,
	TocFlags		= 10,
	HostGroupName	= 11,
	ContainerHeader	= 12,

	LatestPlusOne,
	Latest			= (LatestPlusOne - 1)
};

///////////////////////////////////////////////////////////////////////////////
enum class EOnDemandTocFlags : uint32
{
	None			= 0,
	InstallOnDemand	= (1 << 0),
	StreamOnDemand	= (1 << 1),

	Last			= StreamOnDemand
};
ENUM_CLASS_FLAGS(EOnDemandTocFlags);

///////////////////////////////////////////////////////////////////////////////
struct FTocMeta
{
	int64	EpochTimestamp = 0;
	FString BuildVersion;
	FString TargetPlatform;

	UE_API friend FArchive& operator<<(FArchive& Ar, FTocMeta& Meta);
	UE_API friend FCbWriter& operator<<(FCbWriter& Writer, const FTocMeta& Meta);
};

UE_API bool LoadFromCompactBinary(FCbFieldView Field, FTocMeta& OutMeta);

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandTocHeader
{
	static constexpr uint64 ExpectedMagic = 0x6f6e64656d616e64; // ondemand

	uint64	Magic = ExpectedMagic;
	uint32	Version = uint32(EOnDemandTocVersion::Latest);
	uint32	Flags = uint32(EOnDemandTocFlags::None);
	uint32	BlockSize = 0;
	FString CompressionFormat;
	FString ChunksDirectory;
	FString HostGroupName;

	UE_API friend FArchive& operator<<(FArchive& Ar, FOnDemandTocHeader& Header);
	UE_API friend FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandTocHeader& Header);
};

UE_API bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandTocHeader& OutTocHeader);

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandTocEntry
{
	FIoHash		Hash = FIoHash::Zero;
	FIoChunkId	ChunkId = FIoChunkId::InvalidChunkId;
	uint64		RawSize = 0;
	uint64 		EncodedSize = 0;
	uint32 		BlockOffset = ~uint32(0);
	uint32 		BlockCount = 0; 
	
	UE_API friend FArchive& operator<<(FArchive& Ar, FOnDemandTocEntry& Entry);
	UE_API friend FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandTocEntry& Entry);
};

UE_API bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandTocEntry& OutTocEntry);

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandTocContainerEntry
{
	FIoContainerId				ContainerId;
	FString						ContainerName;
	FString						EncryptionKeyGuid;
	TArray<FOnDemandTocEntry>	Entries;
	TArray<uint32>				BlockSizes;
	TArray<FIoBlockHash>		BlockHashes;
	TArray<uint8>				Header;	

	/** Hash of the .utoc file (on disk) used to generate this data */
	FIoHash						UTocHash;
	uint8						ContainerFlags = 0;

	UE_API friend FArchive& operator<<(FArchive& Ar, FOnDemandTocContainerEntry& ContainerEntry);
	UE_API friend FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandTocContainerEntry& ContainerEntry);
};

UE_API bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandTocContainerEntry& OutContainer);

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandTocSentinel
{
public:
	static constexpr inline char SentinelImg[] = "-[]--[]--[]--[]-";
	static constexpr uint32 SentinelSize = 16;

	bool IsValid();

	UE_API friend FArchive& operator<<(FArchive& Ar, FOnDemandTocSentinel& Sentinel);

private:
	uint8 Data[SentinelSize] = { 0 };
};

struct FOnDemandTocAdditionalFile
{
	FIoHash	Hash;
	FString	Filename;
	uint64	FileSize = 0;

	UE_API friend FArchive& operator<<(FArchive& Ar, FOnDemandTocAdditionalFile& AdditionalFile);
	UE_API friend FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandTocAdditionalFile& AdditionalFile);
};

UE_API bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandTocAdditionalFile& AdditionalFile);

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandTocTagSetPackageList
{
	uint32			ContainerIndex = 0;
	TArray<uint32>	PackageIndicies;

	UE_API friend FArchive& operator<<(FArchive& Ar, FOnDemandTocTagSetPackageList& TagSet);
	UE_API friend FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandTocTagSetPackageList& TagSet);
};

UE_API bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandTocTagSetPackageList& TagSet);

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandTocTagSet
{
	using FOnDemandTocTagSetPackageLists = TArray<FOnDemandTocTagSetPackageList>;

	FString							Tag;
	FOnDemandTocTagSetPackageLists	Packages;

	UE_API friend FArchive& operator<<(FArchive& Ar, FOnDemandTocTagSet& TagSet);
	UE_API friend FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandTocTagSet& TagSet);
};

UE_API bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandTocTagSet& TagSet);

///////////////////////////////////////////////////////////////////////////////
struct FOnDemandToc
{
	FOnDemandToc() = default;
	~FOnDemandToc() = default;
	FOnDemandToc(FOnDemandToc&&) = default;
	FOnDemandToc& operator= (FOnDemandToc&&) = default;
	FOnDemandToc(const FOnDemandToc&) = delete;
	FOnDemandToc&  operator= (const FOnDemandToc&) = delete;

	FOnDemandTocHeader					Header;
	FTocMeta							Meta;
	TArray<FOnDemandTocContainerEntry>	Containers;
	TArray<FOnDemandTocAdditionalFile>	AdditionalFiles;
	TArray<FOnDemandTocTagSet>			TagSets;

	UE_API friend FArchive& operator<<(FArchive& Ar, FOnDemandToc& Toc);
	UE_API friend FCbWriter& operator<<(FCbWriter& Writer, const FOnDemandToc& Toc);

	UE_API static const FGuid	VersionGuid;
	UE_API static const FString	FileExt;

	UE_API static TIoStatusOr<FOnDemandToc> LoadFromFile(const FString& FilePath, bool bValidate);
};

UE_API bool LoadFromCompactBinary(FCbFieldView Field, FOnDemandToc& OutToc);

} // namespace UE::IoStore

////////////////////////////////////////////////////////////////////////////////
UE_API FStringBuilderBase& operator<<(FStringBuilderBase& Sb, UE::IoStore::EOnDemandTocFlags TocFlags);
UE_API FString LexToString(UE::IoStore::EOnDemandTocFlags TocFlags);

#undef UE_API
