// Copyright Epic Games, Inc. All Rights Reserved.
using UnrealBuildTool;

public class IoStoreOnDemandTests : TestModuleRules
{
    static IoStoreOnDemandTests()
    {
		TestMetadata = new Metadata();
        TestMetadata.TestName = "IoStoreOnDemand";
        TestMetadata.TestShortName = "IoStoreOnDemand";
    }

    public IoStoreOnDemandTests(ReadOnlyTargetRules Target) : base(Target)
	{
		OptimizeCode = CodeOptimization.Never;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"IoStoreOnDemand",
			});
	}
}
