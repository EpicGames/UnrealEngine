// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedPlatforms("Win64")]
public class IoStoreOnDemandTestsTarget : TestTargetRules
{
	public IoStoreOnDemandTestsTarget(TargetInfo Target) : base(Target)
	{
		bCompileAgainstCoreUObject			= true;
		bCompileAgainstApplicationCore		= true;
		bAdaptiveUnityDisablesOptimizations = true;

		GlobalDefinitions.Add("CATCH_CONFIG_ENABLE_BENCHMARKING=1");
		GlobalDefinitions.Add("WITH_IOSTORE_ONDEMAND_TESTS=1");
	}
}
