// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

#define LOCTEXT_NAMESPACE "JsonOjbectGraphModule"

namespace UE::Private
{

class JsonObjectGraph : public IModuleInterface
{
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

void JsonObjectGraph::StartupModule()
{
}

void JsonObjectGraph::ShutdownModule()
{
}

}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(UE::Private::JsonObjectGraph, JsonObjectGraph)
