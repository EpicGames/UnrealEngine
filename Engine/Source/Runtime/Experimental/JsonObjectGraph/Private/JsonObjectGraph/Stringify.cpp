// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonObjectGraph/Stringify.h"
#include "JsonStringifyImpl.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Stringify)

FUtf8String UE::JsonObjectGraph::Stringify(TConstArrayView<const UObject*> RootObjects, const FJsonStringifyOptions& Options)
{
	UE::Private::FJsonStringifyImpl Impl(RootObjects, Options);
	return Impl.ToJson();
}
