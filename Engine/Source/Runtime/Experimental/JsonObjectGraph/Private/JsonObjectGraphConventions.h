// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Private conventions of the JsonObjectGraph object representation - these 
// would be shared between 'from' and 'to' implementations - if you update
// these make sure to update the tchar version as well if present. TCHAR
// versions exist because the underlying JSON writer/reader I'm using 
// speaks it 'natively', long term I'm hoping to remove all TCHAR.

// 'Package' level conventions, these are used at the root level:
#define UE_JSON_ROOT_OBJECTS_KEY_TCHAR TEXT("__RootObjects")

#define UE_JSON_PACKAGE_SUMMARY_KEY_TCHAR TEXT("__PackageSummary")

#define UE_JSON_CUSTOM_VERSIONS_KEY_TCHAR TEXT("__CustomVersions")
#define UE_JSON_PACKAGE_NAME_KEY_TCHAR TEXT("__PackageName")

// UObject instance conventions:
#define UE_JSON_OBJECT_INSTANCE_KEY_TCHAR TEXT("__UObject")
#define UE_JSON_OBJECT_NAME_KEY_TCHAR TEXT("Name")
#define UE_JSON_OBJECT_CLASS_KEY_TCHAR TEXT("Class")
#define UE_JSON_OBJECT_FLAGS_KEY_TCHAR TEXT("Flags")
#define UE_JSON_OBJECT_STRUCTURED_DATA_KEY_TCHAR TEXT("__StructuredData")
#define UE_JSON_OBJECT_SERIAL_DATA_KEY_TCHAR TEXT("__SerialData")
#define UE_JSON_OBJECT_SPARSE_CLASS_DATA_KEY_TCHAR TEXT("__SparseClassData")
#define UE_JSON_OBJECT_INDIRECTLY_REFERENCED_KEY_TCHAR TEXT("__IndirectlyReferenced")
#define UE_JSON_SCRIPTSTRUCT_TCHAR TEXT("__UScriptStruct")
#define UE_JSON_PROPERTYBAG_TCHAR TEXT("__UPropertyBag")
#define UE_JSON_PROPERTYDESCS_TCHAR TEXT("PropertyDescs")

// Reference encoding related conventions:
#define UE_JSON_OBJECT_REF_PREFIX "uobject:"
#define UE_JSON_FIELD_REF_PREFIX "ffield:"
#define UE_JSON_REF_NONE "None"

// Conventions related to encoding generic intrinsic types (TMap and TOptional):
#define UE_JSON_TMAP_KEY_KEY_TCHAR TEXT("Key")
#define UE_JSON_TMAP_VALUE_KEY_TCHAR TEXT("Value")
#define UE_JSON_OPTIONAL_VALUE_KEY_TCHAR TEXT("OptionalValue")

// Linker tables written under __PackageSummary in UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
// builds, where placeholder stubs hide import class/outer information from the object
// graph.  Encoding the raw tables makes every name and reference auditable from the JSON.
#define UE_JSON_LINKER_NAME_TABLE_KEY_TCHAR   TEXT("PackageNameTable")
#define UE_JSON_LINKER_IMPORT_TABLE_KEY_TCHAR TEXT("PackageImportTable")