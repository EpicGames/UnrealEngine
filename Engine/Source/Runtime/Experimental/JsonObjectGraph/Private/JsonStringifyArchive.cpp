// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonStringifyArchive.h"
#include "Misc/Base64.h"
#include "JsonStringifyImpl.h"
#include "UObject/Object.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "Serialization/CustomVersion.h"

namespace UE::Private
{

FJsonStringifyArchive::FJsonStringifyArchive(
	const UObject* InObject
	, int32 InInitialIndentLevel
	, FJsonStringifyImpl* InRootImpl
	, TArray<FCustomVersion>& InVersionsToHarvest
	, bool bFilterEditorOnly)
	: ObjectBeingStreamSerialized(InObject)
	, RootImpl(InRootImpl)
	, MemoryWriter(Result)
	, Writer(UE::Private::FPrettyJsonWriter::Create(&MemoryWriter, InInitialIndentLevel))
	, InitialIndentLevel(InInitialIndentLevel)
	, VersionsToHarvest(InVersionsToHarvest)
{
	// we don't want to write any properties, only 
	// stuff from user serializes:
	ArUseCustomPropertyList = true;
	ArCustomPropertyList = nullptr;
	SetIsSaving(true);
	SetFilterEditorOnly(bFilterEditorOnly);
	SetWantBinaryPropertySerialization(true); // Setting this to disable SerializeScriptProperties
	MemoryWriter.SetIsPersistent(true);
	MemoryWriter.SetIsTextFormat(true);
}

TArray<uint8> FJsonStringifyArchive::ToJson()
{
	if (ObjectBeingStreamSerialized->HasAnyFlags(RF_ClassDefaultObject))
	{
		// CDOs are never/have never been natively serialized
		return TArray<uint8>();
	}

	Writer->WriteArrayStartInline();
	Writer->WriteLineTerminator();
	const_cast<UObject*>(ObjectBeingStreamSerialized)->Serialize(*this);
	const TArray<uint8> NullStream = GetNullStream();

	const bool bDidWriteAnything = Result != NullStream;
	if (bDidWriteAnything)
	{
		Writer->WriteNewlineAndArrayEnd();
	}

	// record version regardless of whether anything
	// was written, lots of people just use Serialize as
	// a lifecycle function:
	VersionsToHarvest.Append(GetCustomVersions().GetAllVersions());

	return bDidWriteAnything ? MoveTemp(Result) : TArray<uint8>();
}

TArray<uint8> FJsonStringifyArchive::GetNullStream() const
{
	TArray<uint8> NullStream;
	NullStream.Add('[');
	NullStream.Add('\n');
	for (int32 I = 0; I < InitialIndentLevel + 1; ++I)
	{
		NullStream.Add('\t');
	}
	NullStream.Append("false");
	NullStream.Pop(); // should probably just look for the single false token.. but.. this works

	return NullStream;
}

void FJsonStringifyArchive::Serialize(void* V, int64 Length)
{
	// The stream Serializers manage their own branches, and ultimately just write a stream of bytes. 
	// We can encode those bytes any way we choose, but some information is effectively lost (e.g. 
	// did the caller have a float or an int? would have been nice to know!). I've played
	// around with various representations but didn't find any particularly helpful, so i've chosen
	// the simplest representation that can handle all data: Base64 encoding - this is very much
	// a 'get a byte, get a byte, get another byte' kind of implementation, and is not fast. The
	// text representation is also not good. I think if you're having problems with this you
	// should move to a structured archive or, even better, get the UPROPERTY based declarative system 
	// working for you in some way. That may require some ingenuity, but you read and understood 
	// this entire so you're definitely the cream of the crop.
	uint32 ExpectedLength = FBase64::GetEncodedDataSize(Length);
	TArray<uint8> EncodedData;
	EncodedData.SetNum(ExpectedLength + 1);
	FBase64::Encode<ANSICHAR>(static_cast<uint8*>(V), Length, (ANSICHAR*)EncodedData.GetData());
	Writer->WriteValueInline(FAnsiStringView((const ANSICHAR*)EncodedData.GetData(), EncodedData.Num() - 1));
}

#if WITH_EDITOR
void FJsonStringifyArchive::SerializeBool(bool& D)
{
	bool bBool = D;
	Writer->WriteValueInline(bBool);
}
#endif

FArchive& FJsonStringifyArchive::operator<<(UObject*& Value)
{
	RootImpl->WriteObjectAsJsonToWriter(ObjectBeingStreamSerialized, Value, Writer);
	return *this;
}

FArchive& FJsonStringifyArchive::operator<<(FField*& Value)
{
	RootImpl->WriteFieldReferenceTo(ObjectBeingStreamSerialized, Value, Writer);
	return *this;
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
FArchive& FJsonStringifyArchive::operator<<(Verse::VCell*& Value)
{
	// VCell references should only be serialized as reflected properties, which this archive skips.
	check(0);
	return *this;
}
#endif

FArchive& FJsonStringifyArchive::operator<<(struct FLazyObjectPtr& Value)
{
	UObject* Resolved = Value.Get();
	return *this << Resolved;
}

FArchive& FJsonStringifyArchive::operator<<(struct FObjectPtr& Value)
{
	UObject* Resolved = Value.Get();
	return *this << Resolved;
}

FArchive& FJsonStringifyArchive::operator<<(struct FSoftObjectPtr& Value)
{
	FSoftObjectPath Path = Value.ToSoftObjectPath();
	return *this << Path;
}

FArchive& FJsonStringifyArchive::operator<<(struct FSoftObjectPath& Value)
{
	Writer->WriteValueInline(Value.ToString());
	return *this;
}

FArchive& FJsonStringifyArchive::operator<<(struct FWeakObjectPtr& Value)
{
	UObject* Resolved = Value.Get();
	return *this << Resolved;
}

FArchive& FJsonStringifyArchive::operator<<(FName& Value)
{
	Writer->WriteValueInline(Value.ToString());
	return *this;
}

FArchive& FJsonStringifyArchive::operator<<(FText& Value)
{
	Writer->WriteValueInline(Value);
	return *this;
}

}
