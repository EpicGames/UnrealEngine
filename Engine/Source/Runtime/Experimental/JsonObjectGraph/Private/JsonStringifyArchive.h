// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/ArchiveUObject.h"
#include "Serialization/MemoryWriter.h"
#include "PrettyJsonWriter.h"

struct FCustomVersion;

namespace UE::Private
{

struct FJsonStringifyImpl;

// Serial in the name of this type is meant to refer to 'UObject::Serialize(FArchive&)'. 
// This writer writes the resulting byte stream safely to json.
struct FJsonStringifyArchive final : private FArchiveUObject
{
	FJsonStringifyArchive(
		const UObject* InObject, 
		int32 InitialIndentLevel, 
		FJsonStringifyImpl* InRootImpl,
		TArray<FCustomVersion>& InVersionsToHarvest,
		bool bFilterEditorOnly);

	// FMemoryWriter wants to work with TArray<uint8>
	// so that's what we're returning:
	TArray<uint8> ToJson();
private:
	TArray<uint8> GetNullStream() const;

	virtual void Serialize(void* V, int64 Length) override;
	#if WITH_EDITOR
	virtual void SerializeBool(bool& D) override;
	#endif
	virtual FArchive& operator<<(UObject*& Value) override;
	virtual FArchive& operator<<(FField*& Value) override;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual FArchive& operator<<(Verse::VCell*& Value) override;
#endif
	virtual FArchive& operator<<(struct FLazyObjectPtr& Value) override;
	virtual FArchive& operator<<(struct FObjectPtr& Value) override;
	virtual FArchive& operator<<(struct FSoftObjectPtr& Value) override;
	virtual FArchive& operator<<(struct FSoftObjectPath& Value) override;
	virtual FArchive& operator<<(struct FWeakObjectPtr& Value) override;
	// overrides for strings:
	virtual FArchive& operator<<(FName& Value) override;
	virtual FArchive& operator<<(FText& Value) override;

	const UObject* const ObjectBeingStreamSerialized;
	FJsonStringifyImpl* RootImpl;

	TArray<uint8> Result;
	FMemoryWriter MemoryWriter;
	TSharedRef<UE::Private::FPrettyJsonWriter> Writer;
	int32 InitialIndentLevel;
	TArray<FCustomVersion>& VersionsToHarvest;
};

}
