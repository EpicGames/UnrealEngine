// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonStringifyImpl.h"
#include "PrettyJsonWriter.h"
#include "Algo/Copy.h"
#include "Algo/ForEach.h"
#include "Algo/RemoveIf.h"
#include "Misc/App.h"
#include "JsonObjectGraphConventions.h"
#include "JsonStringifyArchive.h"
#include "JsonStringifyStructuredArchive.h"
#include "Serialization/StructuredArchive.h"
#include "Serialization/StructuredArchiveSlots.h"
#include "Serialization/Formatters/JsonArchiveOutputFormatter.h"
#include "Serialization/ArchiveUObjectFromStructuredArchive.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/EnumProperty.h"
#include "UObject/PropertyOptional.h"
#include "UObject/Package.h"
#include "UObject/TextProperty.h"
#include "UObject/VerseValueProperty.h"

#if WITH_EDITORONLY_DATA
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/PropertyBagRepository.h"
#include "UObject/UObjectThreadContext.h"
#endif// WITH_EDITORONLY_DATA

#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
#include "UObject/LinkerLoad.h"
#endif

namespace UE::Private
{

// gathering utils
struct FPackageReferenceFinder : public FArchiveUObject
{
	FPackageReferenceFinder(const UObject* Obj, TArray<const UObject*>& InReferences, bool bFilterEditorOnly)
		: FArchiveUObject()
		, References(InReferences)
	{
		// Copying FPackageHarvester:
		SetIsPersistent(true);
		SetIsSaving(true);
		SetFilterEditorOnly(bFilterEditorOnly);
		ArNoDelta = true;
		ArIsObjectReferenceCollector = true;
		ArShouldSkipBulkData = true;
		if (Obj->HasAnyFlags(RF_ClassDefaultObject))
		{
			Obj->GetClass()->SerializeBin(*this, (UObject*)Obj);
		}
		else
		{
			((UObject*)Obj)->Serialize(*this);
		}
		
#if WITH_EDITORONLY_DATA
		UE::FPropertyBagRepository& Repository = UE::FPropertyBagRepository::Get();
		const UObject* InstanceDataObject = Repository.FindInstanceDataObject(Obj);
		if (InstanceDataObject)
		{
			InstanceDataObject->GetClass()->SerializeBin(*this, (UObject*)InstanceDataObject);
		}
#endif//WITH_EDITORONLY_DATA
	}

private:
	virtual FArchive& operator<<(UObject*& ObjRef) override
	{
		if (ObjRef != nullptr &&
			(!ObjRef->HasAnyFlags(RF_Transient) || ObjRef->IsNative()) &&
			!ObjRef->IsA<UPackage>())
		{
			// Set to null any pointer to an external asset
			References.Add(ObjRef);
		}

		return *this;
	}

	TArray<const UObject*>& References;
};

static const UClass* GetFirstNativeClass(const UClass* Class)
{
	const UClass* Iter = Class;
	while (Iter && !Iter->HasAnyClassFlags(CLASS_Native))
	{
		Iter = Iter->GetSuperClass();
	}
	return Iter;
}

static void FilterEditorOnlyObjects(
	TArray<UObject*>& Objects
)
{
	#if WITH_EDITOR
	Objects.SetNum(Algo::StableRemoveIf(Objects, [](const UObject* Obj) {
		const UClass* NativeClass = GetFirstNativeClass(Obj->GetClass());
		return IsEditorOnlyObject(NativeClass) || !(NativeClass->GetDefaultObject(false)->NeedsLoadForClient() || NativeClass->GetDefaultObject(false)->NeedsLoadForServer());
		}));
	#endif
}

static void GatherExports(
	TConstArrayView<const UObject*> Roots,
	TConstArrayView<const UObject*> DisallowList,
	bool bFilterEditorOnly,
	TArray<const UObject*>& OutRoots,
	TArray<const UObject*>& OutExports)
{
	// find everything roots references that is within roots and put it into
	// OutExports, unless it is in the disallow list:

	TSet<const UObject*> DisallowSet;
	DisallowSet.Append(DisallowList);
	for (const UObject* Obj : DisallowList)
	{
		// const hacks and GetObjects expressiveness junk
		TArray<UObject*> DisallowedSubObjects;
		GetObjectsWithOuter(Obj, DisallowedSubObjects, EGetObjectsFlags::IncludeNestedObjects, RF_Transient);
		for (UObject* DisallowedObj : DisallowedSubObjects)
		{
			DisallowSet.Add(DisallowedObj);
		}
	}
	TSet<const UObject*> AllowedRoots;
	AllowedRoots.Append(Roots);
	for (const UObject* Obj : Roots)
	{
		if (Obj && !Obj->IsA<UPackage>())
		{
			OutRoots.Add(Obj);
		}
	}

	TArray<const UObject*> PendingRefs;
	Algo::CopyIf(Roots, PendingRefs, [](const UObject* Obj) { return IsValid(Obj); });

	TSet<const UObject*> RefsProcessed;
	RefsProcessed.Append(PendingRefs);

	TArray<const UObject*> ScratchRefs; // just keeping this allocation alive across iterations
	while (PendingRefs.Num())
	{
		const UObject* Iter = PendingRefs.Pop();
		if (AllowedRoots.Contains(Iter) && Iter->IsA<UPackage>())
		{
			// add all immediate roots of a package, if filter 'editor only' objects
			// exclude any object that are of an editor only native type:
			TArray<UObject*> PackageInners;
			GetObjectsWithOuter(Iter, PackageInners, EGetObjectsFlags::None, RF_Transient);
			if (bFilterEditorOnly)
			{
				FilterEditorOnlyObjects(PackageInners);
			}
			PackageInners.Sort([](const UObject& A, const UObject& B) { return A.GetFName().LexicalLess(B.GetFName()); });
			OutRoots.Append(PackageInners);
			PendingRefs.Append(PackageInners);
			continue;
		}

		OutExports.Add(Iter);

		// The object graph is incomplete when using UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
		// and so we won't be able to do full graph analysis:
#if !UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
		ScratchRefs.Add(Iter->GetClass());
		FPackageReferenceFinder ReferencedObjects(Iter, ScratchRefs, bFilterEditorOnly);
		for (const UObject* Obj : ScratchRefs)
		{
			if (RefsProcessed.Contains(Obj))
			{
				continue;
			}
			RefsProcessed.Add(Obj);

			if (DisallowSet.Contains(Obj))
			{
				continue;
			}

			const UObject* OuterIter = Obj->GetOuter();
			bool bIsInRoot = false;
			while (OuterIter)
			{
				if (AllowedRoots.Contains(OuterIter))
				{
					bIsInRoot = true;
					break;
				}
				OuterIter = OuterIter->GetOuter();
			}
			if (!bIsInRoot)
			{
				continue;
			}

			PendingRefs.Add(Obj);
		}
		ScratchRefs.Reset();
#endif
	}
	
	// Instead in UE_INTERNAL_UNIVERSAL_PLACEHOLDERS we treat all
	// roots as UPackage and gather all children recursively.
	// In this configuration we cannot reliably categorize objects as
	// editor only vs. not because that determination is made
	// by native code and uproject/uplugin structure - neither of 
	// which are loaded when using placeholders:
#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
	checkf(!bFilterEditorOnly, TEXT("Cannot reliably filter editor only objects in this configuration"));
	for(const UObject* Root : OutRoots)
	{
		TArray<UObject*> ObjectInners;
		GetObjectsWithOuter(Root, ObjectInners, EGetObjectsFlags::IncludeNestedObjects, RF_Transient);
		OutExports.Append(ObjectInners);
	}
#endif
}

static void WriteObjectPath(FUtf8StringBuilderBase& OutPath, const UObject* ForObject, const UObject* OuterLimit)
{
	const auto WriteObjectPathImpl = [&OutPath, OuterLimit](const UObject* ForObject, auto Self)
		{
			check(ForObject);
			// ofpa files still have an outer package, but the GetPackage() terminator
			// will not be reachable via the outer chain, replace any encoutered UPackage
			// w/ the OuterLimit (acquired via GetPackage) when serializing:
			if(Cast<UPackage>(ForObject) && ForObject != OuterLimit)
			{
				ForObject = OuterLimit;
			}

			if (ForObject == OuterLimit)
			{
				return;
			}
			Self(ForObject->GetOuter(), Self);
			OutPath << "/";
			OutPath << ForObject->GetName();
		};
	WriteObjectPathImpl(ForObject, WriteObjectPathImpl);
}

static void WriteFieldPath(FUtf8StringBuilderBase& OutPath, const FField* ForField, const UObject* OuterLimit)
{
	const auto WriteWriteFieldPathImpl = [&OutPath, OuterLimit](const UObject* ForObject, const FField* ForField, auto Self)
		{
			if (ForObject == OuterLimit)
			{
				return;
			}
			if (ForField)
			{
				FFieldVariant Owner = ForField->GetOwnerVariant();
				Self(Owner.ToUObject(), Owner.ToField(), Self);
			}
			else if (ForObject)
			{
				Self(ForObject->GetOuter(), nullptr, Self);
			}
			OutPath << "/";
			if (ForObject)
			{
				OutPath << ForObject->GetName();
			}
			else
			{
				check(ForField);
				OutPath << ForField->GetName();
			}
		};
	WriteWriteFieldPathImpl(nullptr, ForField, WriteWriteFieldPathImpl);
}

FJsonStringifyImpl::FJsonStringifyImpl(TConstArrayView<const UObject*> Roots, const FJsonStringifyOptions& Options)
	: WriteOptions(Options)
	, MemoryWriter(Result)
	, Writer(FJsonWriter::Create(&MemoryWriter))
	, CurrentObject(nullptr)
	, RootPackage(nullptr)
{
	const bool bFilterEditorOnly = (Options.Flags & EJsonStringifyFlags::FilterEditorOnlyData) != EJsonStringifyFlags::Default;
	TArray<const UObject*> Exports;
	// If the requested graph is simple we can store the root package and 
	// include said package's metadata in the result:
	if(Roots.Num() == 1 && Roots[0] && Roots[0]->IsA<UPackage>())
	{
		RootPackage = Cast<UPackage>(Roots[0]);
	}
	GatherExports(Roots, {}, bFilterEditorOnly, RootObjects, Exports);

	ObjectsToExport = TSet<const UObject*>(Exports);

	MemoryWriter.SetIsPersistent(true);
	MemoryWriter.SetFilterEditorOnly(bFilterEditorOnly);
	MemoryWriter.SetIsTextFormat(true);
}

FUtf8String FJsonStringifyImpl::ToJson()
{
	ToJsonBytes();
	// Hacking around TJsonWriter and FMemoryWriter, is an array of bytes by
	// any other name not exceedingly dangerous? Everyone should be working
	// with utf8:
	TArray<ANSICHAR>& ResultTyped = reinterpret_cast<TArray<ANSICHAR>&>(Result);
	return FUtf8String(MoveTemp(ResultTyped));
}

void FJsonStringifyImpl::WriteObjectAsJsonToWriter(const UObject* OwningObject, const UObject* InObject, TSharedRef<FJsonWriter> WriterToUse)
{
	if (CurrentScope)
	{
		CurrentScope->Apply();
	}

	if (InObject && InObject->IsIn(OwningObject) &&
		ObjectsToExport.Contains(InObject) &&
		!ObjectsExported.Contains(InObject))
	{
		ObjectsExported.Add(InObject);
		TSharedRef<FJsonWriter> RootWriter = Writer;
		Writer = WriterToUse;
		WriteObjectToJson(InObject);
		Writer = RootWriter;
	}
	else
	{
		// we just need to write a path
		WriterToUse->WriteValueInline(FUtf8StringView(WriteObjectReference(InObject)));
	}
}

void FJsonStringifyImpl::WriteFieldReferenceTo(const UObject* OwningObject, const FField* Value, TSharedRef<FJsonWriter> WriterToUse)
{
	WriterToUse->WriteValueInline(FUtf8StringView(WriteFieldReference(Value)));
}

void FJsonStringifyImpl::WriteObjectAsJsonToArchive(const UObject* OwningObject, const UObject* InObject, FArchive* ArchiveToUse, int32 InitialIndentLevel)
{
	TSharedRef<FJsonWriter> JsonWriter = FJsonWriter::Create(ArchiveToUse, InitialIndentLevel);
	JsonWriter->HACK_SetPreviousTokenWritten();
	WriteObjectAsJsonToWriter(OwningObject, InObject, JsonWriter);
}

void FJsonStringifyImpl::ToJsonBytes()
{
	if (RootObjects.Num() == 0)
	{
		return;
	}

	// main entry point to our writer state machine:
	Writer->WriteObjectStart();
	Writer->WriteIdentifierPrefix(UE_JSON_ROOT_OBJECTS_KEY_TCHAR);
	Writer->WriteArrayStartInline();
	Writer->WriteLineTerminator();
	for (const UObject* Object : RootObjects)
	{
		ObjectsExported.Add(Object);
		WriteObjectToJson(Object);
	}
	Writer->WriteArrayEnd();
	
	WritePackageSummary();

	Writer->WriteObjectEnd();
	// Result is now populated with Json representation
}

void FJsonStringifyImpl::WriteObjectToJson(const UObject* Object)
{
	CurrentObject = Object;

	// loop properties, writing any that have changed:
	const UObject* const Archetype = Object->GetArchetype();
	const UClass* const ArchetypeClass = Archetype ? Archetype->GetClass() : nullptr; // UObject's CDO itself has no archetype
	Writer->WriteObjectStartInline();
	// Write native UObject data - name, type, flags, native user serialize, etc
	WriteNativeObjectData();

#if WITH_EDITORONLY_DATA
	// attempt to write the IDO instead since it will have more info. if that fails, fallback to the regular object
	if (!TryWriteInstanceDataObjectProperties(Object))
#endif // WITH_EDITORONLY_DATA
	{ 
		for (TFieldIterator<FProperty> FieldIt(Object->GetClass()); FieldIt; ++FieldIt)
		{
			FProperty* Property = *FieldIt;
			const UObject* ValidatedArchtype = ArchetypeClass && ArchetypeClass->IsChildOf(Property->GetOwnerClass()) ?
				Archetype : nullptr;

			WriteIdentifierAndValueToJson(Object, ValidatedArchtype, Property);
		}
	}
	// We may have inner objects that were not referenced directly by this object, 
	// but will be referenced by other objects in the graph. We must write them here, or we
	// would have to encode them at the root level, which would disrupt locality.
	// The draw back of recording them here is that we must order them ourselves,
	// which will degrade the stability of the serialized buffer. It feels like 'the best
	// we can do' is to write these objects in alphabetical order:
	WriteIndirectlyReferencedContainedObjects(Object);
	Writer->WriteObjectEnd();

	// emit stream serializer, omitting tagged properties:
	CurrentObject = CurrentObject->GetOuter();
}

void FJsonStringifyImpl::WriteNativeObjectData()
{
	const UObject* Object = CurrentObject;
	check(Object);

	// Could use a structured archive to write this more declaratively...
	Writer->WriteIdentifierPrefix(UE_JSON_OBJECT_INSTANCE_KEY_TCHAR);
	Writer->WriteObjectStartInline();
	Writer->WriteValue(UE_JSON_OBJECT_NAME_KEY_TCHAR, Object->GetName());
	Writer->WriteUtf8Value(UE_JSON_OBJECT_CLASS_KEY_TCHAR, WriteObjectReference(Object->GetClass()));
	Writer->WriteValue(UE_JSON_OBJECT_FLAGS_KEY_TCHAR, Object->GetFlags() & RF_Load);
	Writer->WriteObjectEnd();

	// Native serializers may produce huge amounts of data that add a size and performance overhead.
	// They can be optionally disabled in scenarios where this data is less important.
	if (!(WriteOptions.Flags & EJsonStringifyFlags::DisableNativeSerializers))
	{
		// This is tricky, we have no good mechanism for detecting whether an object wants a structured
		// serialization or a traditional stream serialization. So I've decided to 'try' the structured
		// serializer and if it writes nothing we fall back to the stream (FArchive) serializer. Hopefully
		// some day we get rid of FStructuredArchive as it does not spark joy.
#if WITH_TEXT_ARCHIVE_SUPPORT
		TArray<uint8> StructuredData = StructuredDataToJson(Object, Writer->GetIndentLevel());
		if (!StructuredData.IsEmpty())
		{
			Writer->WriteIdentifierPrefix(UE_JSON_OBJECT_STRUCTURED_DATA_KEY_TCHAR);
			Writer->WriteJsonRaw(FAnsiStringView((ANSICHAR*)StructuredData.GetData(), StructuredData.Num()));
		}
		else
#endif
		{
			// no useful native structured data, write the native serial data:
			TArray<uint8> SerialData = SerialDataToJson(Object, Writer->GetIndentLevel());
			if (!SerialData.IsEmpty())
			{
				Writer->WriteIdentifierPrefix(UE_JSON_OBJECT_SERIAL_DATA_KEY_TCHAR);
				Writer->WriteJsonRaw(FAnsiStringView((ANSICHAR*)SerialData.GetData(), SerialData.Num()));
			}
		}
	}

	// Sparse class data is ambiguously serialized as part of SerializeDefaultObject,
	// which our text serializer does not use (except for reference gathering). Lets write
	// the special SparseClassData member here:
	if (const UClass* AsClass = Cast<UClass>(Object))
	{
		if (const void* SCD = const_cast<UClass*>(AsClass)->GetSparseClassData(EGetSparseClassDataMethod::ReturnIfNull))
		{
			const UScriptStruct* SCDStruct = AsClass->GetSparseClassDataStruct();
			const void* DefaultSCD = AsClass->GetArchetypeForSparseClassData();
			const UScriptStruct* DefaultSCDStruct = AsClass->GetSparseClassDataArchetypeStruct();
			WriteStructToJsonWithIdentifier(UE_JSON_OBJECT_SPARSE_CLASS_DATA_KEY_TCHAR, SCD, DefaultSCD, SCDStruct, DefaultSCDStruct);
		}
	}
}

#if WITH_EDITORONLY_DATA
bool FJsonStringifyImpl::TryWriteInstanceDataObjectProperties(const UObject* ForObject)
{
#if !UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
	// unless UNIVERSAL_PLACEHOLDERS is enabled, respect the impersonation flag.
	FUObjectSerializeContext* SerializeContext = FUObjectThreadContext::Get().GetSerializeContext();
	if (!SerializeContext || !SerializeContext->bImpersonateProperties)
	{
		return false;
	}
#endif

	UE::FPropertyBagRepository& Repository = UE::FPropertyBagRepository::Get();
	const UObject* InstanceDataObject = Repository.FindInstanceDataObject(ForObject);
	if (!InstanceDataObject)
	{
		return false;
	}

	for (TFieldIterator<FProperty> FieldIt(InstanceDataObject->GetClass()); FieldIt; ++FieldIt)
	{
		WriteIdentifierAndValueToJson(InstanceDataObject, nullptr, *FieldIt);
	}
	return true;
}
#endif// WITH_EDITORONLY_DATA

void FJsonStringifyImpl::WriteIndirectlyReferencedContainedObjects(const UObject* ForObject)
{
	TArray<const UObject*> UnwrittenInners;
	ForEachObjectWithOuter(ForObject, 
		[this, &UnwrittenInners](UObject* ObjectToExport)
		{	
			if(ObjectsToExport.Contains(ObjectToExport) && !ObjectsExported.Contains(ObjectToExport))
			{
				UnwrittenInners.Add(ObjectToExport);
			}
		},
		EGetObjectsFlags::None, RF_Transient);

	if (UnwrittenInners.Num() == 0)
	{
		return;
	}

	UnwrittenInners.Sort([](const UObject& A, const UObject& B) { return A.GetName().Compare(B.GetName()) < 0; });

	// write all UnwrittenInners into the __IndirectlyReferenced member, for simplicity we 
	// will always encode this as an array. We know that these unwritten inners will not be outered
	// to the written objects here, so we don't need to tag them as exported ahead of time:
	Writer->WriteIdentifierPrefix(UE_JSON_OBJECT_INDIRECTLY_REFERENCED_KEY_TCHAR);
	Writer->WriteArrayStartInline();
	Writer->WriteLineTerminator();
	for (const UObject* Object : UnwrittenInners)
	{
		WriteObjectToJson(Object);
	}
	Writer->WriteArrayEnd();
}

FUtf8String FJsonStringifyImpl::WriteObjectReference(const UObject* ForObject) const
{
	// matching FSaveContext::GetSaveableStatusNoOuter, if we integrate into
	// save package we can reuse the logic from there:
	if (!ForObject || 
		!IsValid(ForObject) || 
		(ForObject->HasAnyFlags(RF_Transient) && !ForObject->IsNative()))
	{
		return UE_JSON_OBJECT_REF_PREFIX UE_JSON_REF_NONE;
	}

	// Encode objects with the long standing pathname convention, we can
	// encode a basis in the buffer to give users options when an asset
	// is moved on the filesystem:
	TUtf8StringBuilder<256> Reference;
	Reference << UE_JSON_OBJECT_REF_PREFIX;
	Reference << ForObject->GetPathName();
	return FUtf8String(Reference);
}

FUtf8String FJsonStringifyImpl::WriteFieldReference(const FField* Value) const
{
	if (!Value)
	{
		return UE_JSON_FIELD_REF_PREFIX UE_JSON_REF_NONE;
	}

	// identical convention for FField as above, albeit with a fieldref prefix
	// instead of a uobject:
	TUtf8StringBuilder<256> Reference;
	Reference << UE_JSON_FIELD_REF_PREFIX;
	Reference << Value->GetPathName();
	return FUtf8String(Reference);
}

void FJsonStringifyImpl::WriteIdentifierAndValueToJson(const void* Container, const void* DefaultContainer, const FProperty* Property)
{
	if (!Property->ShouldSerializeValue(MemoryWriter))
	{
		return;
	}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	// FVCellProperty (as opposed to FVRestValueProperty) is only used by UVerseClass/UVerseStruct,
	// where it just points back to the VClass the UStruct was derived from. It can be safely ignored.
	if (const FVCellProperty* VerseProperty = CastField<FVCellProperty>(Property))
	{
		return;
	}
#endif

	// this is inefficient for structs... but it papers over some
	// problematic identical implementations. We will improve it
	bool bMatchesDefault = DefaultContainer != nullptr && IsDeltaEncoding();
	for (int32 Idx = 0; Idx < Property->ArrayDim && bMatchesDefault; ++Idx)
	{
		const void* Value = Property->ContainerPtrToValuePtr<void>(Container, Idx);
		const void* ArchetypeValue = Property->ContainerPtrToValuePtr<void>(DefaultContainer, Idx);

		FString StringValue;
		Property->ExportText_Direct(StringValue, Value, Value, nullptr, PPF_ForDiff);
		if (ArchetypeValue)
		{
			FString DefaultStringValue;
			Property->ExportText_Direct(DefaultStringValue, ArchetypeValue, ArchetypeValue, nullptr, PPF_ForDiff);
			bMatchesDefault = DefaultStringValue.Equals(StringValue);
		}
	}

	if (bMatchesDefault)
	{
		return;
	}

	FPendingScope PropertyIdentifier(this, [Impl = this, Property]()
		{
			Impl->Writer->WriteIdentifierPrefix(Property->GetName());
		});
	
	if (Property->ArrayDim > 1)
	{
		// encode as a static array:
		FPendingScope ArrayIdentifier(this, [Impl = this]()
			{
				Impl->Writer->WriteArrayStartInline();
				Impl->Writer->WriteLineTerminator();
			},
			[Impl = this]()
			{
				Impl->Writer->WriteArrayEnd();
			}
		);
		for (int32 Index = 0; Index < Property->ArrayDim; ++Index)
		{
			WriteValueToJson(
				Property->ContainerPtrToValuePtr<void>(Container, Index),
				DefaultContainer ? Property->ContainerPtrToValuePtr<void>(DefaultContainer, Index) : nullptr,
				Property);
		}
	}
	else
	{
		WriteValueToJson(
			Property->ContainerPtrToValuePtr<void>(Container),
			DefaultContainer ? Property->ContainerPtrToValuePtr<void>(DefaultContainer) : nullptr,
			Property);
	}
}

void FJsonStringifyImpl::WriteValueToJson(const void* Value, const void* DefaultValue, const FProperty* Property)
{
	bool bHandledAsAggregate = true;
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		// write any tagged data for the struct - but be sure to delta serialize
		WriteStructToJson(Value, DefaultValue, StructProperty->Struct, StructProperty->Struct);
	}
	else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
	{
		const TObjectPtr<const UObject>* ObjectPtr = 
			static_cast<const TObjectPtr<const UObject>*>(Value);
		const UObject* Object = ObjectPtr->Get();
		WriteObjectAsJsonToWriter(CurrentObject, Object, Writer);
	}
	// containers:
	else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		WriteArrayToJson(Value, ArrayProperty);
	}
	else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
	{
		WriteSetToJson(Value, SetProperty);
	}
	else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
	{
		WriteMapToJson(Value, MapProperty);
	}
	else if (const FOptionalProperty* OptionalProperty = CastField<FOptionalProperty>(Property))
	{
		WriteOptionalToJson(Value, OptionalProperty);
	}
	else
	{
		bHandledAsAggregate = false; // fall back to value serialization
	}

	if (!bHandledAsAggregate)
	{
		WriteIntrinsicToJson(Value, Property);
	}
}

void FJsonStringifyImpl::WriteIntrinsicToJson(const void* Value, const FProperty* Property)
{
	const FName ReferenceProperty{"ReferenceProperty"}; // this type just crashes if you try to export it
	if(Property->GetClass()->GetFName() == ReferenceProperty)
	{
		return;
	}

	if (CurrentScope)
	{
		CurrentScope->Apply();
	}

	// bools are special, because of GetPropertyValue, which can apply a mask
	if (const FBoolProperty* BoolProp = CastField<FBoolProperty>(Property))
	{
		bool bValue = BoolProp->GetPropertyValue(Value);
		Writer->WriteValueInline(bValue);
	}
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
	{
		// byte property is special because of the enum/byte duality:
		uint8 ValueAsIntrinsic = *(uint8*)Value;
		if (const UEnum* EnumDef = ByteProperty->Enum)
		{
			FString StringValue = EnumDef->GetValueOrBitfieldAsAuthoredNameString(ValueAsIntrinsic);
			Writer->WriteValueInline(StringValue);
		}
		else
		{
			Writer->WriteValueInline(ValueAsIntrinsic);
		}
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		// export enums as strings
		const UEnum* EnumDef = EnumProperty->GetEnum();
		FString StringValue = EnumDef->GetValueOrBitfieldAsAuthoredNameString(
			EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(Value));
		Writer->WriteValueInline(StringValue);
	}
	else if( const FTextProperty* TextProperty = CastField<FTextProperty>(Property))
	{
		FText ValueAsText = *(FText*)Value;
		Writer->WriteValueInline(ValueAsText);
	}
	else if( const FFloatProperty* AsFloatProperty = CastField<FFloatProperty>(Property))
	{
		float ValueAsIntrinsic = *(float*)Value;
		if(FMath::IsFinite(ValueAsIntrinsic))
		{
			Writer->WriteValueInline(ValueAsIntrinsic);
		}
		else
		{
			Writer->WriteValueInline(TEXT("inf"));
		}

	}
	else if( const FDoubleProperty* AsDoubleProperty = CastField<FDoubleProperty>(Property))
	{
		double ValueAsIntrinsic = *(double*)Value;
		if(FMath::IsFinite(ValueAsIntrinsic))
		{
			Writer->WriteValueInline(ValueAsIntrinsic);
		}
		else
		{
			Writer->WriteValueInline(TEXT("inf"));
		}
	}
#define INTRINSIC_TABLE \
INTRINSIC_ENTRY(FInt8Property, int8) \
INTRINSIC_ENTRY(FInt16Property, int16) \
INTRINSIC_ENTRY(FIntProperty, int32) \
INTRINSIC_ENTRY(FInt64Property, int64) \
INTRINSIC_ENTRY(FUInt16Property, uint16) \
INTRINSIC_ENTRY(FUInt32Property, uint32) \
INTRINSIC_ENTRY(FUInt64Property, uint64)

#define INTRINSIC_ENTRY(FPropertyType, intrinsic_type) \
	else if (const FPropertyType* As##intrinsic_type = CastField<FPropertyType>(Property)) \
	{ \
		intrinsic_type ValueAsIntrinsic = *(intrinsic_type*)Value; \
		Writer->WriteValueInline(ValueAsIntrinsic); \
	}
	INTRINSIC_TABLE
#undef INTRINSIC_ENTRY
#undef INTRINSIC_TABLE
	else
	{
		FString StringValue;
		Property->ExportText_Direct(StringValue, Value, Value, nullptr, PPF_None);
		Writer->WriteValueInline(StringValue);
	}
}

void FJsonStringifyImpl::WriteStructToJsonWithIdentifier(const TCHAR* Identifier, const void* StructInstance, const void* DefaultInstance, const UScriptStruct* Struct, const UScriptStruct* DefaultStruct)
{
	check(StructInstance && Struct);
	FPendingScope StructIdentifier(this, [Impl = this, Identifier]()
		{
			Impl->Writer->WriteIdentifierPrefix(Identifier);
		});

	WriteStructToJson(
		StructInstance,
		DefaultInstance,
		Struct,
		DefaultStruct);
}

void FJsonStringifyImpl::WriteStructToJson(const void* StructInstance, const void* DefaultInstance, const UScriptStruct* Struct, const UScriptStruct* DefaultStruct)
{
	// FInstancedStruct is a core level construct, and we can usefully decompose it, so try:
	const FInstancedStruct* StructInstanceTyped = Struct == FInstancedStruct::StaticStruct() ? 
		static_cast<const FInstancedStruct*>(StructInstance) : nullptr;
	const FInstancedStruct* DefaultInstanceTyped = DefaultStruct == FInstancedStruct::StaticStruct() ? 
		static_cast<const FInstancedStruct*>(DefaultInstance) : nullptr;
	const UScriptStruct* InstancedStructType = StructInstanceTyped ? 
		StructInstanceTyped->GetScriptStruct() : nullptr;
	const UScriptStruct* DefaultInstancedStructType = DefaultInstanceTyped ? 
		DefaultInstanceTyped->GetScriptStruct() : nullptr;

	// For structs with an ambiguous type (like instanced structs), we want to disambiguate
	// the type stored in the json, additionally for UPropertyBag, which is a transient 
	// non native struct and is regenerated on load we want to store its current layout.
	// It's possible we could allow structures to customize their representation, but 
	// there are enough serialization customization facilities already and I do not want
	// to introduce more.
	const UPropertyBag* const PropertyBag = Cast<UPropertyBag>(InstancedStructType);
	const bool bWriteStructType = InstancedStructType && 
		!PropertyBag;

	TUniquePtr<FInstancedStruct> FallbackStruct;
	if(InstancedStructType)
	{
		StructInstance = StructInstanceTyped->GetMemory();
		Struct = InstancedStructType;
		if(DefaultInstanceTyped)
		{
			DefaultInstance = DefaultInstanceTyped->GetMemory();
			DefaultStruct = DefaultInstancedStructType; 
		}
		else
		{
			// fall back to default constructed struct
			FallbackStruct = MakeUnique<FInstancedStruct>(InstancedStructType);
			DefaultInstance = FallbackStruct.Get()->GetMemory();
			DefaultStruct = InstancedStructType;
		}
	}

	FPendingScope ObjectStartIdentifier( 
		this, 
		[Impl = this, InstancedStructType, bWriteStructType, PropertyBag]() 
		{ 
			Impl->Writer->WriteObjectStartInline(); 
			if(PropertyBag)
			{
				Impl->WritePropertyBagDescToJson(PropertyBag);
			}
			else if(bWriteStructType)
			{
				Impl->Writer->WriteUtf8Value(UE_JSON_SCRIPTSTRUCT_TCHAR, Impl->WriteObjectReference(InstancedStructType));
			}
		}, 
		[Impl = this]() { Impl->Writer->WriteObjectEnd(); }
		);

	// @todo: can we improve results for structs that have custom serializers? 
	// We do not capture data outside of FProperty because I use this routine
	// for inspecting objects and readability is paramount:
	for (TFieldIterator<FProperty> PropertyIt(Struct); PropertyIt; ++PropertyIt)
	{
		FProperty* Property = *PropertyIt;
		WriteIdentifierAndValueToJson(
			StructInstance,
			Property->IsInContainer(const_cast<UScriptStruct*>(DefaultStruct)) ? DefaultInstance : nullptr,
			Property);
	}
}

void FJsonStringifyImpl::WriteArrayToJson(const void* ArrayInstance, const FArrayProperty* Array)
{
	if (CurrentScope)
	{
		CurrentScope->Apply();
	}

	Writer->WriteArrayStartInline();
	FScriptArrayHelper ArrayHelper(Array, ArrayInstance);
	const int32 ArrayNum = ArrayHelper.Num();
	if (ArrayNum != 0)
	{
		Writer->WriteLineTerminator();
	}
	for (int32 Index = 0; Index < ArrayNum; ++Index)
	{
		const void* Element = ArrayHelper.GetRawPtr(Index);
		const FProperty* Property = Array->Inner;
		WriteValueToJson(Element, nullptr, Property);
	}

	Writer->WriteArrayEnd();
}

void FJsonStringifyImpl::WriteSetToJson(const void* SetInstance, const FSetProperty* SetProperty)
{
	if (CurrentScope)
	{
		CurrentScope->Apply();
	}

	Writer->WriteArrayStartInline();
	FScriptSetHelper SetHelper(SetProperty, SetInstance);
	// Num() is number of elements, max index expensive to calculate, hence the odd loop:
	// @todo: use set iterator
	int32 SetNum = SetHelper.Num();
	if (SetNum != 0)
	{
		Writer->WriteLineTerminator();
	}
	for (int32 Index = 0; SetNum; ++Index)
	{
		if (!SetHelper.IsValidIndex(Index))
		{
			continue;
		}

		--SetNum;
		const void* Element = SetHelper.GetElementPtr(Index);
		const FProperty* Property = SetProperty->ElementProp;
		WriteValueToJson(Element, nullptr, Property);
	}

	Writer->WriteArrayEnd();
}

void FJsonStringifyImpl::WriteMapToJson(const void* MapInstance, const FMapProperty* MapProperty)
{
	if (CurrentScope)
	{
		CurrentScope->Apply();
	}

	Writer->WriteArrayStartInline();
	FScriptMapHelper MapHelper(MapProperty, MapInstance);
	// Num() is number of elements, max index expensive to calculate, hence the odd loop:
	// @todo: use map iterator
	int32 MapNum = MapHelper.Num();
	if (MapNum != 0)
	{
		Writer->WriteLineTerminator();
	}
	for (int32 Index = 0; MapNum; ++Index)
	{
		if (!MapHelper.IsValidIndex(Index))
		{
			continue;
		}

		--MapNum;
		const void* Key = MapHelper.GetKeyPtr(Index);
		const FProperty* KeyProperty = MapProperty->KeyProp;
		const void* Value = MapHelper.GetValuePtr(Index);
		const FProperty* ValueProperty = MapProperty->ValueProp;

		// age old question - how do you want to encode a tuple in json? I've
		// chosen named tuple:
		Writer->WriteObjectStartInline();
		
		{
			FPendingScope PropertyIdentifier(this, [Impl = this]()
				{
					Impl->Writer->WriteIdentifierPrefix(UE_JSON_TMAP_KEY_KEY_TCHAR);
				});
			WriteValueToJson(Key, nullptr, KeyProperty);
		}
		{
			FPendingScope PropertyIdentifier(this, [Impl = this]()
				{
					Impl->Writer->WriteIdentifierPrefix(UE_JSON_TMAP_VALUE_KEY_TCHAR);
				});
			WriteValueToJson(Value, nullptr, ValueProperty);
		}
		Writer->WriteObjectEnd();
	}

	Writer->WriteArrayEnd();
}

void FJsonStringifyImpl::WriteOptionalToJson(const void* OptionalInstnace, const FOptionalProperty* OptionalProperty)
{
	if (CurrentScope)
	{
		CurrentScope->Apply();
	}
	Writer->WriteObjectStartInline();
	if (const void* ValueAddress = static_cast<const void*>(OptionalProperty->GetValuePointerForReadOrReplaceIfSet(OptionalInstnace)))
	{
		FPendingScope PropertyIdentifier(this, [Impl = this]()
			{
				Impl->Writer->WriteIdentifierPrefix(UE_JSON_OPTIONAL_VALUE_KEY_TCHAR);
			});
		WriteValueToJson(ValueAddress, nullptr, OptionalProperty->GetValueProperty());
	}
	Writer->WriteObjectEnd();
}

void FJsonStringifyImpl::WritePropertyBagDescToJson(const UPropertyBag* PropertyBag)
{
	const FArrayProperty* PropertyDescsProperty = CastFieldChecked<FArrayProperty>(
		PropertyBag->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyBag, PropertyDescs)));

	Writer->WriteIdentifierPrefix(UE_JSON_PROPERTYBAG_TCHAR);
	Writer->WriteObjectStartInline();
	Writer->WriteIdentifierPrefix(UE_JSON_PROPERTYDESCS_TCHAR);
	WriteArrayToJson(
		PropertyDescsProperty->ContainerPtrToValuePtr<void>(PropertyBag),
		PropertyDescsProperty
	);
	Writer->WriteObjectEnd();
}

TArray<uint8> FJsonStringifyImpl::SerialDataToJson(const UObject* Object, int32 InitialIndentLevel)
{
	FJsonStringifyArchive SerialData(Object, InitialIndentLevel, this, Versions, MemoryWriter.IsFilterEditorOnly());
	return SerialData.ToJson();
}

#if WITH_TEXT_ARCHIVE_SUPPORT
TArray<uint8> FJsonStringifyImpl::StructuredDataToJson(const UObject* Object, int32 InitialIndentLevel)
{
	FJsonStringifyStructuredArchive StructuredData(Object, InitialIndentLevel, this, Versions, MemoryWriter.IsFilterEditorOnly());
	return StructuredData.ToJson();
}
#endif // WITH_TEXT_ARCHIVE_SUPPORT

void FJsonStringifyImpl::WritePackageSummary()
{
	FPendingScope PackageSummary(this, 
		[this]()
		{
			Writer->WriteIdentifierPrefix(UE_JSON_PACKAGE_SUMMARY_KEY_TCHAR);
			Writer->WriteObjectStartInline();
		},
		[this]()
		{
			Writer->WriteObjectEnd();
		}
	);

	WriteCustomVersions();
	WritePackageMetadata();
#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
	WriteLinkerTables();
#endif
}

void FJsonStringifyImpl::WriteCustomVersions()
{
#if WITH_TEXT_ARCHIVE_SUPPORT
	if (Versions.Num() == 0)
	{
		return;
	}

	if (CurrentScope)
	{
		CurrentScope->Apply();
	}

	// filter dupes, preserving order of first encounter - we need some stable order:
	TSet<FGuid> Encountered;
	Versions.SetNum(Algo::StableRemoveIf(Versions,
		[&Encountered](const FCustomVersion& V)
		{
			if (Encountered.Contains(V.Key))
			{
				return true;
			}
			else
			{
				Encountered.Add(V.Key);
				return false;
			}
		}));

	Writer->WriteIdentifierPrefix(UE_JSON_CUSTOM_VERSIONS_KEY_TCHAR);
	FJsonStringifyStructuredArchive::WriteCustomVersionValueInline(Versions, Writer->GetIndentLevel(), MemoryWriter);
	Writer->HACK_SetPreviousTokenWrittenSquareClose(); // the above will write the custom version as an array of tuples, track that write
#endif // WITH_TEXT_ARCHIVE_SUPPORT
}

void FJsonStringifyImpl::WritePackageMetadata()
{
#if WITH_METADATA
	if(!RootPackage)
	{
		return;
	}
	
	const FPackageFileVersion& Version = RootPackage->GetLinkerPackageVersion();
	if(!Version.IsCompatible( 
		FPackageFileVersion(VER_LATEST_ENGINE_UE4, EUnrealEngineObjectUE5Version::METADATA_SERIALIZATION_OFFSET)) )
	{
		return;
	}
	const FMetaData& MetaData = const_cast<UPackage*>(RootPackage)->GetMetaData();
	const bool bHasObjectMetaData = MetaData.ObjectMetaDataMap.Num() > 0;
	const bool bHasRootMetaData = MetaData.RootMetaDataMap.Num() > 0;
	const bool bHasAnyMetadata = bHasObjectMetaData || bHasRootMetaData;
	if(!bHasAnyMetadata)
	{
		return;
	}
	
	const auto WriteNameStringMap = [this](const TMap<FName, FString>& NameStringMap)
	{
		Writer->WriteArrayStartInline();
		for(const TPair<FName, FString>& Values : NameStringMap)
		{
			Writer->WriteObjectStart();
			Writer->WriteValue(TEXT("Key"), Values.Key.ToString());
			Writer->WriteValue(TEXT("Value"), Values.Value);
			Writer->WriteObjectEnd();
		}
		Writer->WriteArrayEnd();
	};

	if (CurrentScope)
	{
		CurrentScope->Apply();
	}

	Writer->WriteIdentifierPrefix(TEXT("PackageMetadata"));
	Writer->WriteObjectStartInline();

	if(bHasObjectMetaData)
	{
		Writer->WriteIdentifierPrefix(TEXT("ObjectMetadata"));
		Writer->WriteArrayStartInline();
		for(const TPair<FSoftObjectPath, TMap<FName, FString>>& ObjectMetaData : MetaData.ObjectMetaDataMap)
		{
			Writer->WriteObjectStart();

			Writer->WriteValue(TEXT("Key"), ObjectMetaData.Key.ToString());
			
			Writer->WriteIdentifierPrefix(TEXT("Values"));
			WriteNameStringMap(ObjectMetaData.Value);

			Writer->WriteObjectEnd();
		}
		Writer->WriteArrayEnd();
	}
	if(bHasRootMetaData)
	{
		Writer->WriteIdentifierPrefix(TEXT("RootMetadata"));
		WriteNameStringMap(MetaData.RootMetaDataMap);
	}

	Writer->WriteObjectEnd();
#endif // WITH_METADATA
}

#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
void FJsonStringifyImpl::WriteLinkerTables()
{
	if(!RootPackage)
	{
		return;
	}

	FLinkerLoad* Linker = FLinkerLoad::FindExistingLinkerForPackage(RootPackage);
	if(!Linker)
	{
		return;
	}

	if(CurrentScope)
	{
		CurrentScope->Apply();
	}

	// The ordered list of FName strings baked into the package binary. In
	// placeholder mode, names that only appear in stream-serialised data are invisible 
	// in the exported object graph; the raw table makes them auditable.
	if(Linker->NameMap.Num() > 0)
	{
		Writer->WriteIdentifierPrefix(UE_JSON_LINKER_NAME_TABLE_KEY_TCHAR);
		Writer->WriteArrayStartInline();
		Writer->WriteLineTerminator();
		for(FNameEntryId EntryId : Linker->NameMap)
		{
			// FName::CreateFromDisplayId is the same conversion FLinkerLoad::operator<<(FName&)
			// uses when deserialising a name reference: pair the entry id with number 0 (the
			// per-reference number is stored separately in the binary, not in the table itself).
			Writer->WriteValueInline(FName::CreateFromDisplayId(EntryId, NAME_NO_NUMBER_INTERNAL).ToString());
		}
		Writer->WriteArrayEnd();
	}

	// Every external object this package depends on, encoded as "uobject:" path
	// strings using the same convention as live object references in the rest of
	// the JSON. In placeholder mode the object graph is incomplete, so we encode
	// all imports in the import table here.
	//
	// Path resolution: top-level imports (OuterIndex == null) store the full
	// package path as ObjectName (e.g. "/Script/Engine").  Sub-imports walk the
	// outer chain and join with '.' between a package and its direct children,
	// then ':' (SUBOBJECT_DELIMITER_CHAR) between a direct child and its immediate
	// subobjects, then '.' for all deeper levels — matching the convention of
	// UObjectBaseUtility::GetPathName
	// (e.g. "/Script/Engine.StaticMesh:SubObject.DeepSubObject").
	const TArray<FObjectImport>& ImportMap = Linker->ImportMap;
	if(ImportMap.Num() > 0)
	{
		// Returns the plain path (no prefix) for the import at Idx.
		// Mirrors UObjectBaseUtility::GetPathName: recurse into the outer first,
		// then append the separator and this object's name.
		const auto BuildPath = [&ImportMap](int32 Idx, auto& Self, FUtf8StringBuilderBase& Result) -> void
		{
			const FObjectImport& Import = ImportMap[Idx];
			if (!Import.OuterIndex.IsImport())
			{
				Import.ObjectName.AppendString(Result);
				return;
			}
			const int32 OuterIdx = Import.OuterIndex.ToImport();
			const FObjectImport& Outer = ImportMap[OuterIdx];
			Self(OuterIdx, Self, Result);
			// Use ':' when the outer is not a package but the outer's outer is,
			// matching the SUBOBJECT_DELIMITER_CHAR convention in GetPathName.
			const UTF8CHAR Sep = (Outer.OuterIndex.IsImport() && ImportMap[Outer.OuterIndex.ToImport()].OuterIndex.IsNull()) ? UTF8CHAR(':') : UTF8CHAR('.');
			Result << Sep;
			Import.ObjectName.AppendString(Result);
		};

		Writer->WriteIdentifierPrefix(UE_JSON_LINKER_IMPORT_TABLE_KEY_TCHAR);
		Writer->WriteArrayStartInline();
		Writer->WriteLineTerminator();
		for(int32 Idx = 0; Idx < ImportMap.Num(); ++Idx)
		{
			TUtf8StringBuilder<256> Ref;
			Ref << UE_JSON_OBJECT_REF_PREFIX;
			BuildPath(Idx, BuildPath, Ref);
			Writer->WriteValueInline(Ref.ToView());
		}
		Writer->WriteArrayEnd();
	}
}
#endif // UE_INTERNAL_UNIVERSAL_PLACEHOLDERS

bool FJsonStringifyImpl::IsDeltaEncoding() const
{
	return !EnumHasAnyFlags(WriteOptions.Flags, EJsonStringifyFlags::DisableDeltaEncoding);
}

FJsonStringifyImpl::FPendingScope::FPendingScope(FJsonStringifyImpl* To, const TFunction<void()>& Prefix)
	: Owner(To)
	, Outer(To->CurrentScope)
	, PendingPrefix(Prefix)
{
	To->CurrentScope = this;
}

FJsonStringifyImpl::FPendingScope::FPendingScope(FJsonStringifyImpl* To, const TFunction<void()>& Prefix, const TFunction<void()>& Postfix)
	: Owner(To)
	, Outer(To->CurrentScope)
	, PendingPrefix(Prefix)
	, PendingPostfix({Postfix})
{
	To->CurrentScope = this;
}

FJsonStringifyImpl::FPendingScope::~FPendingScope()
{
	if (bHasBeenApplied && PendingPostfix)
	{
		(*PendingPostfix)();
	}

	// scope left, if we've applied the prefix then we need to apply the postfix:
	Owner->CurrentScope = Outer;
}

void FJsonStringifyImpl::FPendingScope::Apply()
{
	// writer has decided to apply the scope, write all pending prefixes:
	if (bHasBeenApplied)
	{
		return;
	}

	bHasBeenApplied = true;
	if (Outer)
	{
		Outer->Apply();
	}
	PendingPrefix();
}

}
