// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Utf8String.h"
#include "Containers/Array.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/MemoryWriter.h"
#include "JsonObjectGraph/Stringify.h"

class FArrayProperty;
class FSetProperty;
class FMapProperty;
class FOptionalProperty;
class UPropertyBag;

namespace UE::Private 
{

struct FPrettyJsonWriter; 
using FJsonWriter = FPrettyJsonWriter;

struct FJsonStringifyImpl
{
	FJsonStringifyImpl(TConstArrayView<const UObject*> Roots, const FJsonStringifyOptions& Options);

	// Writes and returns the provided root objects as a UTF8 encode Json string, honoring
	// options specified in our constructor:
	FUtf8String ToJson();

	// Helpers for writing to an object to a specific FJsonWriter, or specific FArchive. These
	// are used by the native stream serializers and native structured serializers respectively -
	// WriteObjectAsJsonToWriter is also used as objects are encountered in reflected properties.
	// The owning object must be provided so that we can determine whether to write InObject as a 
	// peer reference, or inline. For root level serialization use ToJson():
	void WriteObjectAsJsonToWriter(const UObject* OwningObject, const UObject* InObject, TSharedRef<FJsonWriter> WriterToUse);
	void WriteObjectAsJsonToArchive(const UObject* OwningObject, const UObject* InObject, FArchive* ArchiveToUse, int32 InitialIndentLevel);

	// Helper to improve support for FField references, which are not uncommon:
	void WriteFieldReferenceTo(const UObject* OwningObject, const FField* Value, TSharedRef<FJsonWriter> WriterToUse);

private:
	void ToJsonBytes();
	void WriteObjectToJson(const UObject* Object);

	// Utility for unreflected UObject data - name, class, and flags - this 
	// data is placed into a special key for a loader that needs to order
	// UObject construction (find archetypes, UClasses, UStructs, etc):
	void WriteNativeObjectData();
#if WITH_EDITORONLY_DATA
	bool TryWriteInstanceDataObjectProperties(const UObject* ForObject);
#endif // WITH_EDITORONLY_DATA
	void WriteIndirectlyReferencedContainedObjects(const UObject* ForObject);
	FUtf8String WriteObjectReference(const UObject* Object) const;
	FUtf8String WriteFieldReference(const FField* Value) const;

	void WriteIdentifierAndValueToJson(const void* Container, const void* DefaultContainer, const FProperty* Property);
	void WriteValueToJson(const void* Value, const void* DefaultValue, const FProperty* Property);
	void WriteIntrinsicToJson(const void* Value, const FProperty* Property);

	// May want to expose WriteStructToJsonWithIdentifier for users to manually write USTRUCT instances into json:
	void WriteStructToJsonWithIdentifier(const TCHAR* Identifier, const void* StructInstance, const void* DefaultInstance, const UScriptStruct* Struct, const UScriptStruct* DefaultStruct);
	void WriteStructToJson(const void* StructInstance, const void* DefaultInstance, const UScriptStruct* Struct, const UScriptStruct* DefaultStruct);
	void WriteArrayToJson(const void* ArrayInstance, const FArrayProperty* Array);
	void WriteSetToJson(const void* SetInstance, const FSetProperty* SetProperty);
	void WriteMapToJson(const void* MapInstance, const FMapProperty* MapProperty);
	void WriteOptionalToJson(const void* OptionalInstnace, const FOptionalProperty* OptionalProperty);
	void WritePropertyBagDescToJson(const UPropertyBag* PropertyBag);

	TArray<uint8> SerialDataToJson(const UObject* Object, int32 InitialIndentLevel);
#if WITH_TEXT_ARCHIVE_SUPPORT
	TArray<uint8> StructuredDataToJson(const UObject* Object, int32 InitialIndentLevel);
#endif// WITH_TEXT_ARCHIVE_SUPPORT

	void WritePackageSummary();
	void WriteCustomVersions();
	void WritePackageMetadata();
#if UE_INTERNAL_UNIVERSAL_PLACEHOLDERS
	// Writes __LinkerNameTable and __LinkerImportTable into the package summary.
	// In this configuration the live object graph contains placeholder stubs for
	// imports, so the linker tables are the only reliable source for the full set
	// of names and external references stored in the binary.
	void WriteLinkerTables();
#endif

	bool IsDeltaEncoding() const;

	const FJsonStringifyOptions& WriteOptions;

	TArray<uint8> Result;
	FMemoryWriter MemoryWriter;

	// helper for enqueuing writes that may not need to be applied, e.g. if there
	// are values that match the default value we won't write out the enclosing scope:
	struct FPendingScope
	{
		FPendingScope(FJsonStringifyImpl* To, const TFunction<void()>& Prefix);
		FPendingScope(FJsonStringifyImpl* To, const TFunction<void()>& Prefix, const TFunction<void()>& Postfix);
		~FPendingScope();
		void Apply();
	private:
		FJsonStringifyImpl* Owner = nullptr;
		FPendingScope* Outer = nullptr;
		TFunction<void()> PendingPrefix;
		TOptional<TFunction<void()>> PendingPostfix;

		bool bHasBeenApplied = false;

		FPendingScope(const FPendingScope& RHS) = delete;
		FPendingScope(FPendingScope&& RHS) = delete;
		FPendingScope& operator=(const FPendingScope& RHS) = delete;
		FPendingScope& operator=(FPendingScope&& RHS) = delete;
	};
	FPendingScope* CurrentScope = nullptr;

	TSharedRef<FJsonWriter> Writer;

	const UObject* CurrentObject;
	TArray<const UObject*> RootObjects;
	TSet<const UObject*> ObjectsToExport;
	TSet<const UObject*> ObjectsExported;
	const UPackage* RootPackage;

	TArray<FCustomVersion> Versions;
};

}
