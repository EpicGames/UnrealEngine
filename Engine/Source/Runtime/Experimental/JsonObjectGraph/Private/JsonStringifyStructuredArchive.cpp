// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonStringifyStructuredArchive.h"
#include "Misc/Base64.h"
#include "Misc/SecureHash.h"
#include "JsonObjectGraphConventions.h"
#include "JsonStringifyImpl.h"
#include "UObject/Object.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "Serialization/Formatters/JsonArchiveOutputFormatter.h"

#include <cinttypes>

#if WITH_TEXT_ARCHIVE_SUPPORT

namespace UE::Private
{

FJsonStringifyStructuredArchive::FJsonStringifyStructuredArchive(
	const UObject* JsonObjectGraph
	, int32 InitialIndentLevel
	, FJsonStringifyImpl* InRootImpl
	, TArray<FCustomVersion>& InVersionsToHarvest
	, bool bFilterEditorOnly)
	: VersionsToHarvest(&InVersionsToHarvest)
	, Object(JsonObjectGraph)
	, RootImpl(InRootImpl)
	, Inner(ResultBuff)
{
	Newline.Add('\n');
	IndentLevel = InitialIndentLevel;
	for (int32 I = 0; I < InitialIndentLevel; ++I)
	{
		Newline.Add('\t');
	}

	Inner.SetIsPersistent(true);
	Inner.SetFilterEditorOnly(bFilterEditorOnly);
	Inner.SetIsTextFormat(true);
}

FJsonStringifyStructuredArchive::FJsonStringifyStructuredArchive(FArchive& ToWriter, int32 InitialIndentLevel)
	: VersionsToHarvest(nullptr)
	, Object(nullptr)
	, Inner(ResultBuff)
	, Override(&ToWriter)
{
	Newline.Add('\n');
	IndentLevel = InitialIndentLevel;
	for (int32 I = 0; I < InitialIndentLevel; ++I)
	{
		Newline.Add('\t');
	}
}

TArray<uint8> FJsonStringifyStructuredArchive::ToJson()
{
	if(!Object->HasAnyFlags(RF_ClassDefaultObject))
	{
		FStructuredArchive StructuredArchive(*this);
		FStructuredArchive::FRecord ExportRecord = StructuredArchive.Open().EnterRecord();
		const_cast<UObject*>(Object)->Serialize(ExportRecord);
		// RAII will close the json block.. so if you remove these containing {} you'll lose your trailing '}'
		// in the Json - I do not think this is a good design for FStructuredArchive.. alternatively 
		// we can make this a long one liner, but that would also require an explanatory comment
	}
	if (ResultBuff.Num() > 2)// Num() > 2 here eliminates the default object {}, empty string, default container etc
	{
		if (VersionsToHarvest)
		{
			VersionsToHarvest->Append(GetUnderlyingArchive().GetCustomVersions().GetAllVersions());
		}
		return ResultBuff;
	}
	return TArray<uint8>(); 
}

void FJsonStringifyStructuredArchive::WriteTextValueInline(FText Value, int32 IndentLevel, FArchive& ToWriter)
{
	FJsonStringifyStructuredArchive Formatter(ToWriter, IndentLevel);
	Formatter.Serialize(Value);
}

void FJsonStringifyStructuredArchive::WriteCustomVersionValueInline(const TArray<FCustomVersion>& Version, int32 IndentLevel, FArchive& ToWriter)
{
	FJsonStringifyStructuredArchive Formatter(ToWriter, IndentLevel);
	FStructuredArchive ChildArchive(Formatter);
	ChildArchive.Open() << (TArray<FCustomVersion>&)Version;
	ChildArchive.Close();
}

FArchive& FJsonStringifyStructuredArchive::GetUnderlyingArchive()
{
	return Override ? *Override : Inner;
}

bool FJsonStringifyStructuredArchive::HasDocumentTree() const
{
	return true;
}

void FJsonStringifyStructuredArchive::EnterRecord()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	WriteOptionalComma();
	WriteOptionalNewline();
	Write("{");
	Newline.Add('\t');
	++IndentLevel;
	bNeedsNewline = true;

	TextStartPosStack.Push(GetUnderlyingArchive().Tell());
}

void FJsonStringifyStructuredArchive::LeaveRecord()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	--IndentLevel;
	Newline.Pop(EAllowShrinking::No);
	if (TextStartPosStack.Pop() == GetUnderlyingArchive().Tell())
	{
		bNeedsNewline = false;
	}
	WriteOptionalNewline();
	Write("}");
	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonStringifyStructuredArchive::EnterField(FArchiveFieldName Name)
{
	// The base UObject serializer for structured archives is badly flawed,
	// so I have disabled it. The reflected properties are handled by FJsonStringifyImpl
	// so we filter them here. The macro generated BaseClassAutoGen is also
	// useless, and is handled by the FSerialDataJsonWriter which calls the 
	// natively provided stream serializer.
	if (FCString::Strcmp(Name.Name, TEXT("Properties")) == 0 ||
		FCString::Strcmp(Name.Name, TEXT("BaseClassAutoGen")) == 0 ||
		ScopeSkipCount != 0)
	{
		++ScopeSkipCount;
	}

	check(ScopeSkipCount >= 0); // scope overflow, halt

	WriteOptionalComma();
	WriteOptionalNewline();
	WriteFieldName(Name.Name);
}

void FJsonStringifyStructuredArchive::LeaveField()
{
	const bool bWasSkipping = ScopeSkipCount > 0;
	if (ScopeSkipCount)
	{
		--ScopeSkipCount;
	}

	if (bWasSkipping)
	{
		return;
	}

	bNeedsComma = true;
	bNeedsNewline = true;
}

bool FJsonStringifyStructuredArchive::TryEnterField(FArchiveFieldName Name, bool bEnterWhenSaving)
{
	if (bEnterWhenSaving)
	{
		EnterField(Name);
	}
	return bEnterWhenSaving;
}

void FJsonStringifyStructuredArchive::EnterArray(int32& NumElements)
{
	EnterStream();
}

void FJsonStringifyStructuredArchive::LeaveArray()
{
	LeaveStream();
}

void FJsonStringifyStructuredArchive::EnterArrayElement()
{
	EnterStreamElement();
}

void FJsonStringifyStructuredArchive::LeaveArrayElement()
{
	LeaveStreamElement();
}

void FJsonStringifyStructuredArchive::EnterStream()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	WriteOptionalComma();
	WriteOptionalNewline();
	Write("[");
	Newline.Add('\t');
	++IndentLevel;
	bNeedsNewline = true;
	TextStartPosStack.Push(GetUnderlyingArchive().Tell());
}

void FJsonStringifyStructuredArchive::LeaveStream()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	--IndentLevel;
	Newline.Pop(EAllowShrinking::No);
	if (TextStartPosStack.Pop() == GetUnderlyingArchive().Tell())
	{
		bNeedsNewline = false;
	}
	WriteOptionalNewline();
	Write("]");
	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonStringifyStructuredArchive::EnterStreamElement()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	WriteOptionalComma();
	WriteOptionalNewline();
}

void FJsonStringifyStructuredArchive::LeaveStreamElement()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonStringifyStructuredArchive::EnterMap(int32& NumElements)
{
	EnterRecord();
}

void FJsonStringifyStructuredArchive::LeaveMap()
{
	LeaveRecord();
}

void FJsonStringifyStructuredArchive::EnterMapElement(FString& Name)
{
	EnterField(FArchiveFieldName(*Name));
}

void FJsonStringifyStructuredArchive::LeaveMapElement()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	LeaveField();
}

void FJsonStringifyStructuredArchive::EnterAttributedValue()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	NumAttributesStack.Push(0);
}

void FJsonStringifyStructuredArchive::EnterAttribute(FArchiveFieldName AttributeName)
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	WriteOptionalComma();
	WriteOptionalNewline();
	WriteOptionalAttributedBlockOpening();
	WriteOptionalComma();
	WriteOptionalNewline();
	checkf(FCString::Strcmp(AttributeName.Name, TEXT("Value")) != 0, TEXT("Attributes called 'Value' are reserved by the implementation"));
	WriteFieldName(*FString::Printf(TEXT("_%s"), AttributeName.Name));
	++NumAttributesStack.Top();
}

void FJsonStringifyStructuredArchive::LeaveAttribute()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonStringifyStructuredArchive::LeaveAttributedValue()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	WriteOptionalAttributedBlockClosing();
	NumAttributesStack.Pop();
	bNeedsComma = true;
	bNeedsNewline = true;
}

void FJsonStringifyStructuredArchive::EnterAttributedValueValue()
{
	WriteOptionalComma();
	WriteOptionalNewline();
	WriteOptionalAttributedBlockValue();
}

bool FJsonStringifyStructuredArchive::TryEnterAttributedValueValue()
{
	return false;
}

bool FJsonStringifyStructuredArchive::TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenSaving)
{
	if (bEnterWhenSaving)
	{
		EnterAttribute(AttributeName);
	}
	return bEnterWhenSaving;
}

void FJsonStringifyStructuredArchive::Serialize(uint8& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonStringifyStructuredArchive::Serialize(uint16& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonStringifyStructuredArchive::Serialize(uint32& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonStringifyStructuredArchive::Serialize(uint64& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonStringifyStructuredArchive::Serialize(int8& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonStringifyStructuredArchive::Serialize(int16& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonStringifyStructuredArchive::Serialize(int32& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonStringifyStructuredArchive::Serialize(int64& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonStringifyStructuredArchive::Serialize(float& Value)
{
	if (FPlatformMath::IsFinite(Value))
	{
		FString String = FString::Printf(TEXT("%.17g"), Value);
#if DO_GUARD_SLOW
		float RoundTripped;
		LexFromString(RoundTripped, *String);
		check(RoundTripped == Value);
#endif
		WriteValue(String);
	}
	else if (FPlatformMath::IsNaN(Value))
	{
		const uint32 ValueAsInt = BitCast<uint32>(Value);
		const bool bIsNegative = !!(ValueAsInt & 0x80000000);
		const uint32 Significand = ValueAsInt & 0x007fffff;
		WriteValue(FString::Printf(TEXT("\"Number:%snan:0x%" PRIx32 "\""), bIsNegative ? TEXT("-") : TEXT("+"), Significand));
	}
	else
	{
		WriteValue(Value < 0.0f ? TEXT("\"Number:-inf\"") : TEXT("\"Number:+inf\""));
	}
}

void FJsonStringifyStructuredArchive::Serialize(double& Value)
{
	if (FPlatformMath::IsFinite(Value))
	{
		FString String = FString::Printf(TEXT("%.17g"), Value);
#if DO_GUARD_SLOW
		double RoundTripped;
		LexFromString(RoundTripped, *String);
		check(RoundTripped == Value);
#endif
		WriteValue(String);
	}
	else if (FPlatformMath::IsNaN(Value))
	{
		const uint64 ValueAsInt = BitCast<uint64>(Value);
		const bool bIsNegative = !!(ValueAsInt & 0x8000000000000000);
		const uint64 Significand = ValueAsInt & 0x000fffffffffffff;
		WriteValue(FString::Printf(TEXT("\"Number:%snan:0x%" PRIx64 "\""), bIsNegative ? TEXT("-") : TEXT("+"), Significand));
	}
	else
	{
		WriteValue(Value < 0.0 ? TEXT("\"Number:-inf\"") : TEXT("\"Number:+inf\""));
	}
}

void FJsonStringifyStructuredArchive::Serialize(bool& Value)
{
	WriteValue(LexToString(Value));
}

void FJsonStringifyStructuredArchive::Serialize(UTF32CHAR& Value)
{
	WriteValue(LexToString(*(uint32*)&Value));
}

void FJsonStringifyStructuredArchive::Serialize(FString& Value)
{
	// Insert a "String:" prefix to prevent incorrect interpretation as another explicit type
	if (Value.StartsWith(TEXT("Object:")) || Value.StartsWith(TEXT("String:")) || Value.StartsWith(TEXT("Base64:")))
	{
		SerializeStringInternal(FString::Printf(TEXT("String:%s"), *Value));
	}
	else
	{
		SerializeStringInternal(Value);
	}
}

void FJsonStringifyStructuredArchive::Serialize(FName& Value)
{
	SerializeStringInternal(*Value.ToString());
}

void FJsonStringifyStructuredArchive::Serialize(UObject*& Value)
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	RootImpl->WriteObjectAsJsonToArchive(Object, Value, &Inner, IndentLevel);
}

#if WITH_VERSE_VM || defined(__INTELLISENSE__)
void FJsonStringifyStructuredArchive::Serialize(Verse::VCell*& Value)
{
	WriteValue(TEXT("null"));
}
#endif

void FJsonStringifyStructuredArchive::Serialize(FText& Value)
{
	/* 
		We could write using the structured serializer defined by FText but
		it is not as widely used as FTextStringHelper::WriteToBuffer and 
		is currently private:
	FStructuredArchive ChildArchive(*this);
	FText::SerializeText(ChildArchive.Open(), Value);
	ChildArchive.Close();
	*/

	FString AsString;
	FTextStringHelper::WriteToBuffer(AsString, Value);
	Serialize(AsString);
}

void FJsonStringifyStructuredArchive::Serialize(FWeakObjectPtr& Value)
{
	UObject* Ptr = Value.IsValid() ? Value.Get() : nullptr;
	Serialize(Ptr);
}

void FJsonStringifyStructuredArchive::Serialize(FSoftObjectPtr& Value)
{
	FSoftObjectPath Path = Value.ToSoftObjectPath();
	Serialize(Path);
}

void FJsonStringifyStructuredArchive::Serialize(FSoftObjectPath& Value)
{
	FString ValueStr;
	Value.ExportTextItem(ValueStr, FSoftObjectPath(), nullptr, 0, nullptr);
	Serialize(ValueStr);
}

void FJsonStringifyStructuredArchive::Serialize(FLazyObjectPtr& Value)
{
	UObject* ObjectResolved = Value.Get();
	Serialize(ObjectResolved);
}

void FJsonStringifyStructuredArchive::Serialize(FObjectPtr& Value)
{
	UObject* ResolvedObject = Value.Get();
	Serialize(ResolvedObject);
}

void FJsonStringifyStructuredArchive::Serialize(TArray<uint8>& Data)
{
	Serialize(Data.GetData(), Data.Num());
}

void FJsonStringifyStructuredArchive::Serialize(void* Data, uint64 DataSize)
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	static const int32 MaxLineChars = 120;
	static const int32 MaxLineBytes = FBase64::GetMaxDecodedDataSize(MaxLineChars);

	if (DataSize < MaxLineBytes)
	{
		// Encode the data on a single line. No need for hashing; intra-line merge conflicts are rare.
		WriteValue(FString::Printf(TEXT("\"Base64:%s\""), *FBase64::Encode((const uint8*)Data, static_cast<uint32>(DataSize))));
	}
	else
	{
		// Encode the data as a record containing a digest and array of base-64 encoded lines
		EnterRecord();
		GetUnderlyingArchive().Serialize(Newline.GetData(), Newline.Num());

		// Compute a SHA digest for the raw data, so we can check if it's corrupted
		uint8 Digest[FSHA1::DigestSize];
		FSHA1::HashBuffer(Data, DataSize, Digest);

		// Convert the hash to a string
		ANSICHAR DigestString[(FSHA1::DigestSize * 2) + 1];
		for (int32 Idx = 0; Idx < UE_ARRAY_COUNT(Digest); Idx++)
		{
			static const ANSICHAR HexDigits[] = "0123456789abcdef";
			DigestString[(Idx * 2) + 0] = HexDigits[Digest[Idx] >> 4];
			DigestString[(Idx * 2) + 1] = HexDigits[Digest[Idx] & 15];
		}
		DigestString[UE_ARRAY_COUNT(DigestString) - 1] = 0;

		FArchive& Writer = GetUnderlyingArchive();
		// Write the digest
		Write("\"Digest\": \"");
		Write(DigestString);
		Write("\",");
		Writer.Serialize(Newline.GetData(), Newline.Num());

		// Write the base64 data
		Write("\"Base64\": ");
		for (uint64 DataPos = 0; DataPos < DataSize; DataPos += MaxLineBytes)
		{
			Write((DataPos > 0) ? ',' : '[');
			Writer.Serialize(Newline.GetData(), Newline.Num());
			Write("\t\"");

			ANSICHAR LineData[MaxLineChars + 1];
			uint64 NumLineChars = FBase64::Encode((const uint8*)Data + DataPos, FMath::Min<uint32>(IntCastChecked<uint32>(DataSize - DataPos), MaxLineBytes), LineData);
			Writer.Serialize(LineData, NumLineChars);

			Write("\"");
		}

		// Close the array
		Writer.Serialize(Newline.GetData(), Newline.Num());
		Write(']');
		bNeedsNewline = true;

		// Close the record
		LeaveRecord();
	}
}

void FJsonStringifyStructuredArchive::Write(ANSICHAR Character)
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	GetUnderlyingArchive().Serialize((void*)&Character, 1);
}

void FJsonStringifyStructuredArchive::Write(const ANSICHAR* Text)
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	GetUnderlyingArchive().Serialize((void*)Text, TCString<ANSICHAR>::Strlen(Text));
}

void FJsonStringifyStructuredArchive::Write(const FString& Text)
{
	Write(TCHAR_TO_UTF8(*Text));
}

void FJsonStringifyStructuredArchive::WriteFieldName(const TCHAR* Name)
{
	if (FCString::Stricmp(Name, TEXT("Base64")) == 0 || FCString::Stricmp(Name, TEXT("Digest")) == 0)
	{
		Write(FString::Printf(TEXT("\"_%s\": "), Name));
	}
	else if (Name[0] == '_')
	{
		Write(FString::Printf(TEXT("\"_%s\": "), Name));
	}
	else
	{
		Write(FString::Printf(TEXT("\"%s\": "), Name));
	}
}

void FJsonStringifyStructuredArchive::WriteValue(const FString& Text)
{
	Write(Text);
}

void FJsonStringifyStructuredArchive::WriteOptionalComma()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	if (bNeedsComma)
	{
		Write(',');
		bNeedsComma = false;
	}
}

void FJsonStringifyStructuredArchive::WriteOptionalNewline()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	if (bNeedsNewline)
	{
		GetUnderlyingArchive().Serialize(Newline.GetData(), Newline.Num());
		bNeedsNewline = false;
	}
}

void FJsonStringifyStructuredArchive::WriteOptionalAttributedBlockOpening()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	if (NumAttributesStack.Top() == 0)
	{
		Write('{');
		Newline.Add('\t');
		++IndentLevel;
		bNeedsNewline = true;
	}
}

void FJsonStringifyStructuredArchive::WriteOptionalAttributedBlockValue()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	if (NumAttributesStack.Top() != 0)
	{
		WriteFieldName(TEXT("_Value"));
	}
}

void FJsonStringifyStructuredArchive::WriteOptionalAttributedBlockClosing()
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	if (NumAttributesStack.Top() != 0)
	{
		--IndentLevel;
		Newline.Pop(EAllowShrinking::No);
		WriteOptionalNewline();
		Write("}");
		bNeedsComma = true;
		bNeedsNewline = true;
	}
}

void FJsonStringifyStructuredArchive::SerializeStringInternal(const FString& String)
{
	if (ScopeSkipCount > 0)
	{
		return;
	}

	FString Result = "\"";

	// Escape the string characters
	for (int32 Idx = 0; Idx < String.Len(); Idx++)
	{
		switch (String[Idx])
		{
		case '\"':
			Result += "\\\"";
			break;
		case '\\':
			Result += "\\\\";
			break;
		case '\b':
			Result += "\\b";
			break;
		case '\f':
			Result += "\\f";
			break;
		case '\n':
			Result += "\\n";
			break;
		case '\r':
			Result += "\\r";
			break;
		case '\t':
			Result += "\\t";
			break;
		default:
			if (String[Idx] <= 0x1f || String[Idx] >= 0x7f)
			{
				Result += FString::Printf(TEXT("\\u%04x"), String[Idx]);
			}
			else
			{
				Result.AppendChar(String[Idx]);
			}
			break;
		}
	}
	Result += TEXT("\"");

	WriteValue(Result);
}

}

#endif // WITH_TEXT_ARCHIVE_SUPPORT
