// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/StructuredArchiveFormatter.h"
#include "Serialization/CustomVersion.h"
#include "Serialization/MemoryWriter.h"

#if WITH_TEXT_ARCHIVE_SUPPORT

namespace UE::Private
{

struct FJsonStringifyImpl;

/*
	FJsonStringifyStructuredArchive is an implementation detail of JsonObjectGraph. It provides support
	for saving the relatively rare data that provides a native Serialize(FStructuredArchive::FRecord) 
	but explicitly disables serialization of reflected properties and versioning data, which are handled
	by the root implementation
*/
struct FJsonStringifyStructuredArchive : private FStructuredArchiveFormatter
{
	FJsonStringifyStructuredArchive(
		const UObject* InObject, 
		int32 InitialIndentLevel, 
		FJsonStringifyImpl* InRootImpl,
		TArray<FCustomVersion>& InVersionsToHarvest,
		bool bFilterEditorOnly);
	
	// FMemoryWriter wants a byte array, so that's what we're working with at this level:
	TArray<uint8> ToJson();

	// Unsure if we want to use the structured archive for FText or the string encoding
	// long term, this provides us the option:
	static void WriteTextValueInline(FText Value, int32 IndentLevel, FArchive& ToWriter);
	// FCustomVersion is best encoded via operator<<(FStructuredArchive::FSlot Slot, FCustomVersion& Version):
	static void WriteCustomVersionValueInline(const TArray<FCustomVersion>& Version, int32 IndentLevel, FArchive& ToWriter);
private:
	FJsonStringifyStructuredArchive(FArchive& ToWriter, int32 InitialIndentLevel);

	virtual FArchive& GetUnderlyingArchive() override;

	virtual bool HasDocumentTree() const override;

	virtual void EnterRecord() override;
	virtual void LeaveRecord() override;
	virtual void EnterField(FArchiveFieldName Name) override;
	virtual void LeaveField() override;
	virtual bool TryEnterField(FArchiveFieldName Name, bool bEnterWhenSaving) override;

	virtual void EnterArray(int32& NumElements) override;
	virtual void LeaveArray() override;
	virtual void EnterArrayElement() override;
	virtual void LeaveArrayElement() override;

	virtual void EnterStream() override;
	virtual void LeaveStream() override;
	virtual void EnterStreamElement() override;
	virtual void LeaveStreamElement() override;

	virtual void EnterMap(int32& NumElements) override;
	virtual void LeaveMap() override;
	virtual void EnterMapElement(FString& Name) override;
	virtual void LeaveMapElement() override;

	virtual void EnterAttributedValue() override;
	virtual void EnterAttribute(FArchiveFieldName AttributeName) override;
	virtual void EnterAttributedValueValue() override;
	virtual void LeaveAttribute() override;
	virtual void LeaveAttributedValue() override;
	virtual bool TryEnterAttribute(FArchiveFieldName AttributeName, bool bEnterWhenSaving) override;

	virtual bool TryEnterAttributedValueValue() override;

	virtual void Serialize(uint8& Value) override;
	virtual void Serialize(uint16& Value) override;
	virtual void Serialize(uint32& Value) override;
	virtual void Serialize(uint64& Value) override;
	virtual void Serialize(int8& Value) override;
	virtual void Serialize(int16& Value) override;
	virtual void Serialize(int32& Value) override;
	virtual void Serialize(int64& Value) override;
	virtual void Serialize(float& Value) override;
	virtual void Serialize(double& Value) override;
	virtual void Serialize(bool& Value) override;
	virtual void Serialize(UTF32CHAR& Value) override;
	virtual void Serialize(FString& Value) override;
	virtual void Serialize(FName& Value) override;
	virtual void Serialize(UObject*& Value) override;
#if WITH_VERSE_VM || defined(__INTELLISENSE__)
	virtual void Serialize(Verse::VCell*& Value) override;
#endif
	virtual void Serialize(FText& Value) override;
	virtual void Serialize(FWeakObjectPtr& Value) override;
	virtual void Serialize(FSoftObjectPtr& Value) override;
	virtual void Serialize(FSoftObjectPath& Value) override;
	virtual void Serialize(FLazyObjectPtr& Value) override;
	virtual void Serialize(FObjectPtr& Value) override;
	virtual void Serialize(TArray<uint8>& Value) override;
	virtual void Serialize(void* Data, uint64 DataSize) override;

	void Write(ANSICHAR Character);

	void Write(const ANSICHAR* Text);
	void Write(const FString& Text);

	void WriteFieldName(const TCHAR* Name);
	void WriteValue(const FString& Value);

	void WriteOptionalComma();
	void WriteOptionalNewline();
	void WriteOptionalAttributedBlockOpening();
	void WriteOptionalAttributedBlockValue();
	void WriteOptionalAttributedBlockClosing();

	void SerializeStringInternal(const FString& String);

	TArray<FCustomVersion>* VersionsToHarvest;
	TArray<ANSICHAR> Newline;

	TArray<int32> NumAttributesStack;
	TArray<int64> TextStartPosStack;

	const UObject* const Object;
	FJsonStringifyImpl* RootImpl;

	TArray<uint8> ResultBuff;
	FMemoryWriter Inner;
	FArchive* Override = nullptr;

	int64_t ScopeSkipCount = 0;
	int32_t IndentLevel = 0;
	bool bNeedsComma = false;
	bool bNeedsNewline = false;
};

}

#endif
