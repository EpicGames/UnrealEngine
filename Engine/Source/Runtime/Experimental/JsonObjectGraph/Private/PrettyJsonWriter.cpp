// Copyright Epic Games, Inc. All Rights Reserved.

#include "PrettyJsonWriter.h"
#include "JsonStringifyStructuredArchive.h" // for inline structured data, e.g. FText

namespace UE::Private
{

FPrettyJsonWriter::FPrettyJsonWriter(FArchive* const InStream, int32 InitialIndentLevel)
	: TJsonWriter<UTF8CHAR, TPrettyJsonPrintPolicySingleNewLine>(InStream, InitialIndentLevel)
{
}

TSharedRef<FPrettyJsonWriter> FPrettyJsonWriter::Create(FArchive* const InStream, int32 InitialIndent)
{
	return MakeShareable(new FPrettyJsonWriter(InStream, InitialIndent));
}

void FPrettyJsonWriter::WriteJsonRaw(FAnsiStringView Value)
{
	check(CanWriteValueWithoutIdentifier());
	WriteCommaAndNewlineIfNeeded();
	PrintPolicy::WriteString(Stream, Value);
	PreviousTokenWritten = EJsonToken::String;
}

void FPrettyJsonWriter::WriteValueInline(FText Value)
{
	// use a structured archive to write FText nicely:
	#if WITH_TEXT_ARCHIVE_SUPPORT
	WriteCommaAndNewlineIfNeeded();
	FJsonStringifyStructuredArchive::WriteTextValueInline(Value, GetIndentLevel(), *Stream);
	PreviousTokenWritten = EJsonToken::String;
	#else
	FString AsString;
	FTextStringHelper::WriteToBuffer(AsString, Value);
	WriteValueInline(AsString);
	#endif
}

void FPrettyJsonWriter::WriteValueInline(const FString& Value)
{
	check(CanWriteValueWithoutIdentifier());
	WriteCommaAndNewlineIfNeeded();
	PreviousTokenWritten = Super::WriteValueOnly(Value);
}

void FPrettyJsonWriter::WriteValueInline(FAnsiStringView UTF8Value)
{
	check(CanWriteValueWithoutIdentifier());
	WriteCommaAndNewlineIfNeeded();
	Super::WriteStringValue(UTF8Value);
	PreviousTokenWritten = EJsonToken::String; 
}

void FPrettyJsonWriter::WriteValueInline(FUtf8StringView UTF8Value)
{
	check(CanWriteValueWithoutIdentifier());
	WriteCommaAndNewlineIfNeeded();
	// @todo: I think super should be utf8 aware?
	FAnsiStringView Hack((char*)UTF8Value.GetData(), UTF8Value.Len());
	Super::WriteStringValue(Hack);
	PreviousTokenWritten = EJsonToken::String;
}

void FPrettyJsonWriter::WriteValueInline(int16 Value)
{
	WriteCommaAndNewlineIfNeeded();
	PreviousTokenWritten = Super::WriteValueOnly((int64)Value);
}

void FPrettyJsonWriter::WriteValueInline(uint16 Value)
{
	WriteCommaAndNewlineIfNeeded();
	PreviousTokenWritten = Super::WriteValueOnly((uint64)Value);
}

void FPrettyJsonWriter::WriteValueInline(uint32 Value)
{
	WriteCommaAndNewlineIfNeeded();
	PreviousTokenWritten = Super::WriteValueOnly((uint64)Value);
}

void FPrettyJsonWriter::WriteUtf8Value(FStringView Identifier, FUtf8StringView UTF8Value)
{
	check(Stack.Top() == EJson::Object);
	WriteIdentifier(Forward<FStringView>(Identifier));

	PrintPolicy::WriteSpace(Stream);

	// @todo: I think super should be utf8 aware?
	FAnsiStringView Hack((char*)UTF8Value.GetData(), UTF8Value.Len());
	Super::WriteStringValue(Hack);
	PreviousTokenWritten = EJsonToken::String;
}

void FPrettyJsonWriter::WriteObjectStartInline()
{
	check(CanWriteObjectWithoutIdentifier());
	if (PreviousTokenWritten != EJsonToken::None)
	{
		WriteCommaAndNewlineIfNeeded();
	}

	PrintPolicy::WriteChar(Stream, CharType('{'));
	++IndentLevel;
	Stack.Push(EJson::Object);
	PreviousTokenWritten = EJsonToken::CurlyOpen;
}

void FPrettyJsonWriter::WriteArrayStartInline()
{
	check(CanWriteObjectWithoutIdentifier());
	if (PreviousTokenWritten != EJsonToken::None)
	{
		WriteCommaAndNewlineIfNeeded();
	}

	PrintPolicy::WriteChar(Stream, CharType('['));
	++IndentLevel;
	Stack.Push(EJson::Array);
	PreviousTokenWritten = EJsonToken::SquareOpen;
}

void FPrettyJsonWriter::WriteNewlineAndArrayEnd()
{
	check(Stack.Top() == EJson::Array);

	--IndentLevel;

	PrintPolicy::WriteLineTerminator(Stream);
	PrintPolicy::WriteTabs(Stream, IndentLevel);
	PrintPolicy::WriteChar(Stream, CharType(']'));
	Stack.Pop();
	PreviousTokenWritten = EJsonToken::SquareClose;
}

void FPrettyJsonWriter::WriteLineTerminator()
{
	PrintPolicy::WriteLineTerminator(Stream);
	PrintPolicy::WriteTabs(Stream, IndentLevel);
}

void FPrettyJsonWriter::HACK_SetPreviousTokenWritten()
{
	// so that we can use this to write inline object references in structured archives..
	// we don't want any commas written... currently relied upon by FMetaData, 
	// which will write object references out using structured archive
	PreviousTokenWritten = EJsonToken::CurlyOpen; 
}

void FPrettyJsonWriter::HACK_SetPreviousTokenWrittenSquareClose()
{
	PreviousTokenWritten = EJsonToken::SquareClose;
}

void FPrettyJsonWriter::WriteCommaAndNewlineIfNeeded()
{
	if (PreviousTokenWritten != EJsonToken::CurlyOpen && 
		PreviousTokenWritten != EJsonToken::SquareOpen && 
		PreviousTokenWritten != EJsonToken::Identifier)
	{
		PrintPolicy::WriteChar(Stream, CharType(','));
		PrintPolicy::WriteLineTerminator(Stream);
		PrintPolicy::WriteTabs(Stream, IndentLevel);
	}
}

}
