// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/JsonWriter.h"

namespace UE::Private
{

struct TPrettyJsonPrintPolicySingleNewLine
	: public TPrettyJsonPrintPolicy<UTF8CHAR>
{
	static inline void WriteLineTerminator(FArchive* Stream)
	{
		TJsonPrintPolicy<UTF8CHAR>::WriteString(Stream, FStringView(TEXT("\n")));
	}
};

// Extending TJsonWriter to get some control over whitespace and tabbing:
struct FPrettyJsonWriter : public TJsonWriter<UTF8CHAR, TPrettyJsonPrintPolicySingleNewLine>
{
	using Super = TJsonWriter<UTF8CHAR, TPrettyJsonPrintPolicySingleNewLine>;
	using PrintPolicy = TPrettyJsonPrintPolicySingleNewLine;
	using CharType = UTF8CHAR;

	FPrettyJsonWriter(FArchive* const InStream, int32 InitialIndentLevel);

	static TSharedRef<FPrettyJsonWriter> Create(FArchive* const InStream, int32 InitialIndent = 0);

	// useful for composing blocks of json
	void WriteJsonRaw(FAnsiStringView Value);

	void WriteValueInline(FText Value);
	void WriteValueInline(const FString& Value);
	void WriteValueInline(FAnsiStringView UTF8Value);
	void WriteValueInline(FUtf8StringView UTF8Value);
	void WriteValueInline(int16 Value);
	void WriteValueInline(uint16 Value);
	void WriteValueInline(uint32 Value);
	template<typename T>
	void WriteValueInline(T Value)
	{
		check(CanWriteValueWithoutIdentifier());
		WriteCommaAndNewlineIfNeeded();
		PreviousTokenWritten = WriteValueOnly(Value);
	}
	void WriteUtf8Value(FStringView Identifier, FUtf8StringView UTF8Value);

	void WriteObjectStartInline();
	void WriteArrayStartInline();
	void WriteNewlineAndArrayEnd();
	void WriteLineTerminator();
	void HACK_SetPreviousTokenWritten();
	void HACK_SetPreviousTokenWrittenSquareClose();

	void WriteCommaAndNewlineIfNeeded();
};

}
