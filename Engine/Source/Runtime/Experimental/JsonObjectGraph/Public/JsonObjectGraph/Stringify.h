// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Utf8String.h"
#include "UObject/ObjectMacros.h"

#include "Stringify.generated.h"

class UObject;

UENUM(BlueprintType, Flags, meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EJsonStringifyFlags : uint8
{
	/** The default representation of an object attempts to be complete and stable across time */
	Default = 0,
	/** Filters editor only data such that it is not written to the Json */
	FilterEditorOnlyData = 1 << 0,
	/** Disables delta encoding such that all properties are encoded in the Json, rather than only changes from the objects' archetypes */
	DisableDeltaEncoding = 1 << 1,
	/** Disables running the native serializers, reduces size and improves performance when the native data is less important */
	DisableNativeSerializers = 1 << 2,
};
ENUM_CLASS_FLAGS(EJsonStringifyFlags);

USTRUCT(BlueprintType)
struct FJsonStringifyOptions
{
	GENERATED_BODY()
	
	FJsonStringifyOptions(EJsonStringifyFlags InFlags)
		: Flags(InFlags)
	{
	}

	FJsonStringifyOptions() = default;
	~FJsonStringifyOptions() = default;
	FJsonStringifyOptions(const FJsonStringifyOptions&) = default;
	FJsonStringifyOptions(FJsonStringifyOptions&&) = default;
	FJsonStringifyOptions& operator=(const FJsonStringifyOptions&) = default;
	FJsonStringifyOptions& operator=(FJsonStringifyOptions&&) = default;

	UPROPERTY(BlueprintReadWrite, Category = "Options")
	EJsonStringifyFlags Flags = EJsonStringifyFlags::Default;
};

namespace UE::JsonObjectGraph
{
	/** 
	 * ! EXPERIMENTAL ! contents of return string will change. Currently this is used
	 * as a debugging facility.
	 * 
	 * @return: A string containing single JSON object with serialized representations of the 
	 * provided Objects in that single object's __RootObjects field.
	 */
	JSONOBJECTGRAPH_API FUtf8String Stringify(TConstArrayView<const UObject*> RootObjects, const FJsonStringifyOptions& Options = FJsonStringifyOptions());
}
