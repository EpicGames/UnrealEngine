// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/Geometry/AnyGeometry.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	template <typename GeometryType>
	FString GeometryToString(const GeometryType& Geometry)
	{
		return Geometry.ToString();
	}

	template <typename GeometryType>
	FString GeometryToString(const TRefCountPtr<GeometryType>& Geometry)
	{
		return Geometry.IsValid() ? Geometry->ToString() : FString("Null");
	}

	bool FAnyGeometry::operator==(const FAnyGeometry& Rhs) const
	{
		if (Geometry.GetIndex() != Rhs.Geometry.GetIndex())
		{
			return false;
		}
		return Visit([&Rhs]<typename GeometryType>(const GeometryType& SelfGeometry)
		{
			const GeometryType& RhsGeometry = Rhs.Geometry.template Get<GeometryType>();
			return SelfGeometry == RhsGeometry;
		});
	}

	FString FAnyGeometry::ToString() const
	{
		return Visit([]<typename GeometryType>(const GeometryType& Geometry)
		{
			return GeometryToString(Geometry);
		});
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
