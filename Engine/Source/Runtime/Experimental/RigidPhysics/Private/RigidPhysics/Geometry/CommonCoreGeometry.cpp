// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/Geometry/CommonCoreGeometry.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	FSphereGeometry::FSphereGeometry(const float InRadius)
		: Radius(InRadius)
	{
	}

	float FSphereGeometry::GetRadius() const
	{
		return Radius;
	}

	void FSphereGeometry::SetRadius(float InRadius)
	{
		Radius = InRadius;
	}

	FString FSphereGeometry::ToString() const
	{
		return FString::Printf(TEXT("Sphere: Radius: %f"), GetRadius());
	}

	FBoxGeometry::FBoxGeometry(const FVector3f& InExtents)
		: Extents(InExtents)
	{
	}

	FVector3f FBoxGeometry::GetExtents() const
	{
		return Extents;
	}

	void FBoxGeometry::SetExtents(const FVector3f& InExtents)
	{
		Extents = InExtents;
	}

	FString FBoxGeometry::ToString() const
	{
		return FString::Printf(TEXT("Box: Extents: [%s]"), *Extents.ToString());
	}

	FCapsuleGeometry::FCapsuleGeometry(float InRadius, float InSegmentHalfLength)
		: Radius(InRadius)
		, SegmentHalfLength(InSegmentHalfLength)
	{
	}

	float FCapsuleGeometry::GetRadius() const
	{
		return Radius;
	}

	void FCapsuleGeometry::SetRadius(const float InRadius)
	{
		Radius = InRadius;
	}

	float FCapsuleGeometry::GetSegmentHalfLength() const
	{
		return SegmentHalfLength;
	}

	void FCapsuleGeometry::SetSegmentHalfLength(const float InHalfLength)
	{
		SegmentHalfLength = InHalfLength;
	}

	FString FCapsuleGeometry::ToString() const
	{
		return FString::Printf(TEXT("Capsule: Radius: %f, SegmentHalfLength: %f"), Radius, SegmentHalfLength);
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
