// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/Geometry/ConvexGeometry.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Chaos/CollisionConvexMesh.h"
#include "Chaos/Convex.h"

namespace UE::Physics
{
	static Chaos::FConvexBuilder::EBuildMethod Convert(const FConvexGeometrySetup::EBuildMethod InMethod)
	{
		switch (InMethod)
		{
			case FConvexGeometrySetup::EBuildMethod::Default:             return Chaos::FConvexBuilder::EBuildMethod::Default;
			case FConvexGeometrySetup::EBuildMethod::Original:            return Chaos::FConvexBuilder::EBuildMethod::Original;
			case FConvexGeometrySetup::EBuildMethod::ConvexHull3:         return Chaos::FConvexBuilder::EBuildMethod::ConvexHull3;
			case FConvexGeometrySetup::EBuildMethod::ConvexHull3Simplified: return Chaos::FConvexBuilder::EBuildMethod::ConvexHull3Simplified;
			default:
			{
				ensureMsgf(false, TEXT("Unhandled conversion for FConvexGeometrySetup::EBuildMethod"));
				return Chaos::FConvexBuilder::EBuildMethod::Default;
			}
		}
	}

	FConvexGeometry::~FConvexGeometry() = default;
	FConvexGeometry::FConvexGeometry(const FConvexGeometry&) = default;
	FConvexGeometry::FConvexGeometry(FConvexGeometry&&) = default;
	FConvexGeometry& FConvexGeometry::operator=(const FConvexGeometry&) = default;
	FConvexGeometry& FConvexGeometry::operator=(FConvexGeometry&&) = default;

	FConvexGeometry::FConvexGeometry(FConvexGeometrySetup&& InSetup)
	{
		TArray<Chaos::FVec3f> Vertices;
		Vertices.SetNum(InSetup.Vertices.Num());
		for (int32 I = 0; I < InSetup.Vertices.Num(); ++I)
		{
			Vertices[I] = InSetup.Vertices[I];
		}
		Implicit = new Chaos::FConvex(Vertices, InSetup.Margin, Convert(InSetup.BuildMethod));
	}

	FConvexGeometry::FConvexGeometry(Chaos::FConvex* InImplicit)
		: Implicit(InImplicit)
	{
	}

	int32 FConvexGeometry::GetNumVertices() const
	{
		return Implicit->NumVertices();
	}

	FVector3f FConvexGeometry::GetVertex(const int32 InIndex) const
	{
		return FVector3f(Implicit->GetVertex(InIndex));
	}

	float FConvexGeometry::GetMargin() const
	{
		return (float)Implicit->GetMargin();
	}

	FString FConvexGeometry::ToString() const
	{
		return TEXT("Convex");
	}

	TRefCountPtr<Chaos::FConvex> FConvexGeometry::GetImplicit() const
	{
		return Implicit;
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
