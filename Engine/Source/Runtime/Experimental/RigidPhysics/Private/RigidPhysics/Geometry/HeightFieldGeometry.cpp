// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/Geometry/HeightFieldGeometry.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Chaos/HeightField.h"

namespace UE::Physics
{
	FHeightFieldGeometry::~FHeightFieldGeometry() = default;
	FHeightFieldGeometry::FHeightFieldGeometry(const FHeightFieldGeometry&) = default;
	FHeightFieldGeometry::FHeightFieldGeometry(FHeightFieldGeometry&&) = default;
	FHeightFieldGeometry& FHeightFieldGeometry::operator=(const FHeightFieldGeometry&) = default;
	FHeightFieldGeometry& FHeightFieldGeometry::operator=(FHeightFieldGeometry&&) = default;

	FHeightFieldGeometry::FHeightFieldGeometry(FHeightFieldGeometrySetup&& InSetup)
	{
		if (!ensure(InSetup.NumCols * InSetup.NumRows == InSetup.Heights.Num()))
		{
			InSetup.Heights.SetNumZeroed(InSetup.NumCols * InSetup.NumRows);
		}
		const int32 NumCellCols = FMath::Max(0, InSetup.NumCols - 1);
		const int32 NumCellRows = FMath::Max(0, InSetup.NumRows - 1);
		const int32 NumQuads = NumCellCols * NumCellRows;
		if (!ensure(NumQuads == InSetup.MaterialIndices.Num()))
		{
			InSetup.MaterialIndices.SetNumZeroed(NumQuads);
		}

		TArray<Chaos::FReal> Heights;
		Heights.SetNum(InSetup.Heights.Num());
		for (int32 I = 0; I < InSetup.Heights.Num(); ++I)
		{
			Heights[I] = (Chaos::FReal)InSetup.Heights[I];
		}
		const Chaos::FVec3 Scale(InSetup.Scale);

		Implicit = new Chaos::FHeightField(
			MoveTemp(Heights),
			MoveTemp(InSetup.MaterialIndices),
			InSetup.NumRows,
			InSetup.NumCols,
			Scale
		);
	}

	FHeightFieldGeometry::FHeightFieldGeometry(Chaos::FHeightField* InImplicit)
		: Implicit(InImplicit)
	{
	}

	int32 FHeightFieldGeometry::GetNumRows() const
	{
		return Implicit->GetNumRows();
	}

	int32 FHeightFieldGeometry::GetNumCols() const
	{
		return Implicit->GetNumCols();
	}

	FVector3f FHeightFieldGeometry::GetScale() const
	{
		return FVector3f(Implicit->GeomData.Scale);
	}

	float FHeightFieldGeometry::GetHeight(const int32 InIndex) const
	{
		return (float)Implicit->GetHeight(InIndex);
	}

	uint8 FHeightFieldGeometry::GetMaterialIndex(const int32 InIndex) const
	{
		return Implicit->GetMaterialIndex(InIndex);
	}

	FString FHeightFieldGeometry::ToString() const
	{
		return TEXT("HeightField");
	}

	TRefCountPtr<Chaos::FHeightField> FHeightFieldGeometry::GetImplicit() const
	{
		return Implicit;
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
