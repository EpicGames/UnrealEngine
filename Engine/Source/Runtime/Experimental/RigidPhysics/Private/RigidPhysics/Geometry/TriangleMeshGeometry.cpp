// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/Geometry/TriangleMeshGeometry.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Chaos/TriangleMeshImplicitObject.h"

namespace UE::Physics
{
	FTriangleMeshGeometry::~FTriangleMeshGeometry() = default;
	FTriangleMeshGeometry::FTriangleMeshGeometry(const FTriangleMeshGeometry&) = default;
	FTriangleMeshGeometry::FTriangleMeshGeometry(FTriangleMeshGeometry&&) = default;
	FTriangleMeshGeometry& FTriangleMeshGeometry::operator=(const FTriangleMeshGeometry&) = default;
	FTriangleMeshGeometry& FTriangleMeshGeometry::operator=(FTriangleMeshGeometry&&) = default;

	Chaos::FTrimeshBVH::EPerFaceBoundsRepresentation Convert(FTriangleMeshGeometrySetup::EPerFaceBoundsRepresentation InValue)
	{
		switch (InValue)
		{
			case FTriangleMeshGeometrySetup::EPerFaceBoundsRepresentation::VectorizedFloat:
			{
				return Chaos::FTrimeshBVH::EPerFaceBoundsRepresentation::VectorizedFloat;
			}
			case FTriangleMeshGeometrySetup::EPerFaceBoundsRepresentation::Byte:
			{
				return Chaos::FTrimeshBVH::EPerFaceBoundsRepresentation::Byte;
			}
			default:
			{
				ensureMsgf(false, TEXT("Unhandled conversion for FTriangleMeshGeometrySetup::EPerFaceBoundsRepresentation"));
				return Chaos::FTrimeshBVH::EPerFaceBoundsRepresentation::VectorizedFloat;
			}
		}
	}

	FTriangleMeshGeometry::FTriangleMeshGeometry(TRefCountPtr<Chaos::FTriangleMeshImplicitObject> InImplicit)
		: Implicit(InImplicit)
	{
	}

	FTriangleMeshGeometry::FTriangleMeshGeometry(FTriangleMeshGeometrySetup&& InSetup)
	{
		if (InSetup.Vertices.Num() < TNumericLimits<uint16>::Max())
		{
			BuildInternal<uint16>(MoveTemp(InSetup));
		}
		else
		{
			BuildInternal<int32>(MoveTemp(InSetup));
		}
	}

	int32 FTriangleMeshGeometry::GetNumVertices() const
	{
		return (int32)Implicit->Particles().Size();
	}

	FVector3f FTriangleMeshGeometry::GetVertex(const int32 InIndex) const
	{
		return FVector3f(Implicit->Particles().GetX(InIndex));
	}

	int32 FTriangleMeshGeometry::GetNumTriangleIndices() const
	{
		return Implicit->Elements().GetNumTriangles();
	}

	FInt32Vector3 FTriangleMeshGeometry::GetTriangleIndices(const int32 InIndex) const
	{
		const Chaos::FTrimeshIndexBuffer& Elements = Implicit->Elements();
		if (Elements.RequiresLargeIndices())
		{
			const Chaos::TVec3<int32>& TriIndices = Elements.GetLargeIndexBuffer()[InIndex];
			return FInt32Vector3(TriIndices.X, TriIndices.Y, TriIndices.Z);
		}
		else
		{
			const Chaos::TVec3<uint16>& TriIndices = Elements.GetSmallIndexBuffer()[InIndex];
			return FInt32Vector3(TriIndices.X, TriIndices.Y, TriIndices.Z);
		}
	}

	int32 FTriangleMeshGeometry::GetNumMaterialIndices() const
	{
		return Implicit->MaterialIndices.Num();
	}

	FMaterialIndex FTriangleMeshGeometry::GetMaterialIndex(const int32 InIndex) const
	{
		return Implicit->GetMaterialIndex(InIndex);
	}

	bool FTriangleMeshGeometry::HasExternalFaceIndices() const
	{
		return Implicit->ExternalFaceIndexMap.IsValid();
	}

	int32 FTriangleMeshGeometry::GetExternalFaceIndex(const int32 InIndex) const
	{
		if (InIndex > INDEX_NONE && HasExternalFaceIndices())
		{
			if (ensure(InIndex < Implicit->ExternalFaceIndexMap->Num()))
			{
				return (*Implicit->ExternalFaceIndexMap)[InIndex];
			}
		}
		return INDEX_NONE;
	}

	bool FTriangleMeshGeometry::HasExternalVertexIndices() const
	{
		return Implicit->ExternalVertexIndexMap.IsValid();
	}

	int32 FTriangleMeshGeometry::GetExternalVertexIndex(const int32 InIndex) const
	{
		if (InIndex > INDEX_NONE && HasExternalVertexIndices())
		{
			if (ensure(InIndex < Implicit->ExternalVertexIndexMap->Num()))
			{
				return (*Implicit->ExternalVertexIndexMap)[InIndex];
			}
		}
		return INDEX_NONE;
	}

	bool FTriangleMeshGeometry::GetCullsBackFaceRaycast() const
	{
		return Implicit->GetCullsBackFaceRaycast();
	}

	FTriangleMeshGeometrySetup::EPerFaceBoundsRepresentation FTriangleMeshGeometry::GetPerFaceBoundsRepresentation() const
	{
		const bool bQuantizedFaceBounds = EnumHasAnyFlags(Implicit->MeshSettingsFlags, Chaos::FTriangleMeshImplicitObject::ETriangleMeshSettingsFlags::QuantizePerFaceBounds);
		return bQuantizedFaceBounds ? EPerFaceBoundsRepresentation::Byte : EPerFaceBoundsRepresentation::VectorizedFloat;
	}

	FString FTriangleMeshGeometry::ToString() const
	{
		return TEXT("TriangleMesh");
	}

	TRefCountPtr<Chaos::FTriangleMeshImplicitObject> FTriangleMeshGeometry::GetImplicit() const
	{
		return Implicit;
	}

	template <typename IndexType>
	void FTriangleMeshGeometry::BuildInternal(FTriangleMeshGeometrySetup&& InSetup)
	{
		const int32 VerticesNum = InSetup.Vertices.Num();
		const int32 TriIndicesNum = InSetup.TriangleIndices.Num();

		// Material indices must match triangle indices in count.
		if (!ensure(InSetup.MaterialIndices.Num() == TriIndicesNum))
		{
			InSetup.MaterialIndices.SetNumZeroed(TriIndicesNum);
		}
		// External index maps must either match the size of their corresponding array or be empty
		if (!ensure(InSetup.ExternalFaceIndexMap.IsEmpty() || InSetup.ExternalFaceIndexMap.Num() == TriIndicesNum))
		{
			InSetup.ExternalFaceIndexMap.SetNumZeroed(TriIndicesNum);
		}
		if (!ensure(InSetup.ExternalVertexIndexMap.IsEmpty() || InSetup.ExternalVertexIndexMap.Num() == VerticesNum))
		{
			InSetup.ExternalVertexIndexMap.SetNumZeroed(VerticesNum);
		}

		TArray<Chaos::FVec3f> Vertices;
		Vertices.SetNum(VerticesNum);
		for (int32 I = 0; I < VerticesNum; ++I)
		{
			Vertices[I] = InSetup.Vertices[I];
		}

		TArray<Chaos::TVec3<IndexType>> TriangleIndices;
		TriangleIndices.SetNum(TriIndicesNum);
		for (int32 I = 0; I < TriIndicesNum; ++I)
		{
			TriangleIndices[I][0] = IndexType(InSetup.TriangleIndices[I][0]);
			TriangleIndices[I][1] = IndexType(InSetup.TriangleIndices[I][1]);
			TriangleIndices[I][2] = IndexType(InSetup.TriangleIndices[I][2]);
		}

		TArray<uint16> MaterialIndices;
		MaterialIndices.SetNum(InSetup.MaterialIndices.Num());
		for (int32 I = 0; I < InSetup.MaterialIndices.Num(); ++I)
		{
			MaterialIndices[I] = InSetup.MaterialIndices[I];
		}

		TUniquePtr<TArray<int32>> ExternalFaceIndexMap;
		TUniquePtr<TArray<int32>> ExternalVertexIndexMap;
		if (!InSetup.ExternalFaceIndexMap.IsEmpty())
		{
			ExternalFaceIndexMap = MakeUnique<TArray<int32>>(MoveTemp(InSetup.ExternalFaceIndexMap));
		}
		if (!InSetup.ExternalVertexIndexMap.IsEmpty())
		{
			ExternalVertexIndexMap = MakeUnique<TArray<int32>>(MoveTemp(InSetup.ExternalVertexIndexMap));
		}

		Chaos::FTriangleMeshImplicitObject::FBuildSettings BuildSettings;
		BuildSettings.PerFaceBounds = Convert(InSetup.PerFaceBoundsRepresentation);

		Implicit = new Chaos::FTriangleMeshImplicitObject(
			MoveTemp(Vertices),
			MoveTemp(TriangleIndices),
			MoveTemp(MaterialIndices),
			MoveTemp(ExternalFaceIndexMap),
			MoveTemp(ExternalVertexIndexMap),
			InSetup.bCullsBackFaceRaycast,
			BuildSettings);
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
