// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/Internal/IJointConstraint.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(IJointConstraint, IRigidTyped);
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
