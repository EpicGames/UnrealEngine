// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/Internal/IJointConstraint6DOF.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidLog.h"

namespace UE::Physics
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(IJointConstraint6DOF, IJointConstraint);

	FJointConstraint6DOFHandle IJointConstraint6DOF::GetTypedHandle() const
	{
		return FJointConstraint6DOFHandle(GetHandle());
	}

	void IJointConstraint6DOF::Activate()
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetBodies(IRigidBody* Body0, IRigidBody* Body1)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetJointTransforms(const FTransform3d& InTransform0, const FTransform3d& InTransform1)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	double IJointConstraint6DOF::GetLinearLimit() const
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return 0;
	}

	void IJointConstraint6DOF::SetLinearLimit(const double InLinearLimit)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	EJointMotionType IJointConstraint6DOF::GetLinearMotionTypesX() const
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return EJointMotionType::Free;
	}

	void IJointConstraint6DOF::SetLinearMotionTypesX(EJointMotionType InMotionType)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	EJointMotionType IJointConstraint6DOF::GetLinearMotionTypesY() const
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return EJointMotionType::Free;
	}

	void IJointConstraint6DOF::SetLinearMotionTypesY(EJointMotionType InMotionType)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	EJointMotionType IJointConstraint6DOF::GetLinearMotionTypesZ() const
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return EJointMotionType::Free;
	}

	void IJointConstraint6DOF::SetLinearMotionTypesZ(EJointMotionType InMotionType)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetAngularMotionTypesX(const EJointMotionType InMotionType)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetAngularMotionTypesY(const EJointMotionType InMotionType)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetAngularMotionTypesZ(const EJointMotionType InMotionType)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetAngularLimits(const FVector3d& InAngularLimits)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetCollisionEnabled(const bool bCollisionEnabled)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetMassConditioningEnabled(const bool bMassConditioningEnabled)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetUseLinearSolver(const bool bUseLinearSolver)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetProjectionEnabled(const bool bProjectionEnabled)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetSoftLinearLimitsEnabled(const bool bSoftLinearLimitsEnabled)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetSoftTwistLimitsEnabled(const bool bSoftTwistLimitsEnabled)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetSoftSwingLimitsEnabled(const bool bSoftSwingLimitsEnabled)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetSoftLinearStiffness(const double InSoftLinearStiffness)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetSoftLinearDamping(const double InSoftLinearDamping)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetSoftTwistStiffness(const double InSoftTwistStiffness)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetSoftTwistDamping(const double InSoftTwistDamping)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetSoftSwingStiffness(const double InSoftSwingStiffness)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetSoftSwingDamping(const double InSoftSwingDamping)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetProjectionLinearAlpha(const double InProjectionLinearAlpha)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetProjectionAngularAlpha(const double InProjectionAngularAlpha)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetProjectionLinearTolerance(const double InProjectionLinearTolerance)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetProjectionAngularTolerance(const double InProjectionAngularTolerance)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetLinearContactDistance(const double InLinearContactDistance)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetTwistContactDistance(const double InTwistContactDistance)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetSwingContactDistance(const double InSwingContactDistance)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetAngularDriveStiffness(const FVector3d& InAngularDriveStiffness)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetAngularDriveDamping(const FVector3d& InAngularDriveDamping)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetLinearDriveStiffness(const FVector3d& InLinearDriveStiffness)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IJointConstraint6DOF::SetLinearDriveDamping(const FVector3d& InLinearDriveDamping)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
