// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/Internal/IRigidBody.h"

#include "RigidPhysics/RigidLog.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(IRigidBody, IRigidTyped);

	bool IRigidBody::IsActive() const
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return false;
	}

	void IRigidBody::Activate()
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::Deactivate()
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetId(const FRigidObjectId& InId)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetIsDynamic(bool bIsDynamic) const
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	bool IRigidBody::IsSleeping() const
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return false;
	}

	IRigidShapeInstance* IRigidBody::CreateShape(const FRigidShapeInstanceSetup& InSetup)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return nullptr;
	}

	IRigidShapeInstance* IRigidBody::CreateShape(TUniquePtr<Chaos::FPerShapeData>&& InShapeData)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return nullptr;
	}

	void IRigidBody::DestroyShape(IRigidShapeInstance* InShape, bool bWakeTouching)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::InitTransform(const FTransform3d& InTransform)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::UpdateTransform(const FTransform3d& InTransform)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetKinematicTarget(const FTransform3d& InTransform)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	FTransform3d IRigidBody::GetTransform() const
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return FTransform3d::Identity;
	}

	FBounds3d IRigidBody::GetBounds() const
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return FBounds3d(FVector3d::Zero(), FVector3d::Zero());
	}

	double IRigidBody::GetMass() const
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return 0;
	}

	void IRigidBody::SetMass(double InM)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	FVector3d IRigidBody::GetInertia() const
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return FVector3d::Zero();
	}

	void IRigidBody::SetInertia(const FVector3d& InInertia)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetCenterOfMass(const FVector3d& InCenterOfMass)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetRotationOfMass(const FQuat& InRotationOfMass)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetCCDEnabled(bool bInEnabled)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetMACDEnabled(bool bInEnabled)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetSmoothEdgeCollisionsEnabled(bool bInEnabled)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetMaxLinearVelocity(float InMax)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetMaxAngularVelocity(float InMax)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetInertiaConditioningEnabled(bool bInEnabled)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetGyroscopicTorqueEnabled(bool bInEnabled)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetGravityGroup(int32 InId)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetGravityScale(float InScale)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetWakeEventsEnabled(bool bInEnabled)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetNumPositionIterations(int32 InNum)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetNumVelocityIterations(int32 InNum)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetNumProjectionIterations(int32 InNum)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidBody::SetIsSleeping(bool bInSleeping)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
