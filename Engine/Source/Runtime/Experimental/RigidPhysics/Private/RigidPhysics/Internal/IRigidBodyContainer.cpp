// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/Internal/IRigidBodyContainer.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(IRigidBodyContainer, IRigidTyped);

	void IRigidBodyContainer::SetId(const FRigidObjectId& InBodyId)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	IRigidBody* IRigidBodyContainer::CreateBody(const FRigidDebugName& InName, ERigidMovementType InMovementType)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return nullptr;
	}

	void IRigidBodyContainer::DestroyBody(IRigidBody* InBody)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	FRigidBodyContainer::FRigidBodyContainer()
	{
		// TODO_CHAOSAPI: Make slack a parameter
		BodyRegistry = MakeUnique<FRigidBodyRegistry>(10);
	}

	IRigidBody* FRigidBodyContainer::GetBody(const FRigidObjectId& InBodyId) const
	{
		return BodyRegistry->FindObject(InBodyId);
	}

	FRigidBodyRegistry* FRigidBodyContainer::GetBodyRegistry() const
	{
		return BodyRegistry.Get();
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
