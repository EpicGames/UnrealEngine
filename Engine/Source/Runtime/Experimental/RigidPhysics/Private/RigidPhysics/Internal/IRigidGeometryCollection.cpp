// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/Internal/IRigidGeometryCollection.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(IRigidGeometryCollection, IRigidTyped);
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
