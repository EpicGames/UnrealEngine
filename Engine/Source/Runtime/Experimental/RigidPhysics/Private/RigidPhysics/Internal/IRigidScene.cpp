// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/Internal/IRigidScene.h"

#include "RigidPhysics/RigidBodyContainer.h"
#include "RigidPhysics/RigidGeometryCollection.h"
#include "RigidPhysics/RigidLog.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(IRigidScene, IRigidLockable);

	bool IRigidScene::Query(const FRigidQueryDesc& InQueryDesc, TArray<FRigidQueryHit>& OutHits)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return false;
	}

	void IRigidScene::StartTick(float InDt)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidScene::WaitOnTick()
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidScene::EndTick()
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidScene::RegisterModifier(IRigidSceneModifier* InModifier)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidScene::UnregisterModifier(IRigidSceneModifier* InModifier)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	IRigidBody* IRigidScene::CreateBody(const FRigidDebugName& InName, ERigidMovementType InMovementType)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return nullptr;
	}

	void IRigidScene::DestroyBody(IRigidBody* InBody)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	IRigidGeometryCollection* IRigidScene::CreateGeometryCollection(const FRigidDebugName& InName, const FSimulationParameters& InParameters)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return nullptr;
	}

	void IRigidScene::DestroyGeometryCollection(IRigidGeometryCollection* InGC)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	void IRigidScene::AddDisabledCollisions(const TMap<FRigidBodyHandle, TArray<FRigidBodyHandle>>& InMap)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	IJointConstraint6DOF* IRigidScene::CreateJointConstraint6DOF()
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return nullptr;
	}

	void IRigidScene::DestroyJointConstraint(IJointConstraint* InConstraint)
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
	}

	IJointConstraint* IRigidScene::GetJointConstraint(const FRigidObjectId& InConstraintId) const
	{
		RIGIDPHYSICS_API_UNSUPPORTED();
		return nullptr;
	}

	FRigidScene::FRigidScene()
	{
		// TODO_CHAOSAPI: registry slack should be a parameter
		BodyContainerRegistry = MakeUnique<FRigidBodyContainerRegistry>(10);
		BodyRegistry = MakeUnique<FRigidBodyRegistry>(10);
		JointConstraintRegistry = MakeUnique<FJointConstraintRegistry>(10);
	}

	IRigidBody* FRigidScene::GetBody(const FRigidObjectId& InBodyId) const
	{
		return BodyRegistry->FindObject(InBodyId);
	}

	IRigidBodyContainer* FRigidScene::GetBodyContainer(const FRigidObjectId& InBodyContainerId) const
	{
		return BodyContainerRegistry->FindObject(InBodyContainerId);
	}

	IRigidGeometryCollection* FRigidScene::GetGeometryCollection(const FRigidObjectId& InGeometryCollectionId) const
	{
		if (IRigidBodyContainer* BodyContainer = BodyContainerRegistry->FindObject(InGeometryCollectionId))
		{
			return BodyContainer->AsA<IRigidGeometryCollection>();
		}
		return nullptr;
	}

	void FRigidScene::DestructScene()
	{
		BodyContainerRegistry.Reset();
		BodyRegistry.Reset();
		JointConstraintRegistry.Reset();
	}

	FRigidBodyContainerRegistry* FRigidScene::GetBodyContainerRegistry() const
	{
		return BodyContainerRegistry.Get();
	}

	FRigidBodyRegistry* FRigidScene::GetBodyRegistry() const
	{
		return BodyRegistry.Get();
	}

	FJointConstraintRegistry* FRigidScene::GetJointConstraintRegistry() const
	{
		return JointConstraintRegistry.Get();
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
