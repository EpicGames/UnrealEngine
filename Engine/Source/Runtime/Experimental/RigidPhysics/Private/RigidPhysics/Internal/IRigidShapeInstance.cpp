// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/Internal/IRigidShapeInstance.h"

#include "RigidPhysics/RigidLog.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(IRigidShapeInstance, IRigidTyped);
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
