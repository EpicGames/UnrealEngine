// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/JointConstraint6DOFHandle.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidLog.h"

namespace UE::Physics
{
	FJointConstraint6DOFHandle::FJointConstraint6DOFHandle(const FRigidSceneId& InSceneId, const FRigidObjectId& InConstraintId)
		: FJointConstraintHandle(InSceneId, InConstraintId)
	{
	}

	FJointConstraint6DOFHandle::FJointConstraint6DOFHandle(const FJointConstraintHandle& InHandle)
		: FJointConstraintHandle(InHandle)
	{
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
