// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/JointConstraintHandle.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidLog.h"
#include "RigidPhysics/RigidScene.h"

namespace UE::Physics
{
	FJointConstraintHandle::FJointConstraintHandle(const FRigidSceneId& InSceneId, const FRigidObjectId& InConstraintId)
		: SceneId(InSceneId)
		, ConstraintId(InConstraintId)
	{
	}

	FRigidSceneHandle FJointConstraintHandle::GetSceneHandle() const
	{
		return FRigidSceneHandle(SceneId);
	}

	const FRigidObjectId& FJointConstraintHandle::GetId() const
	{
		return ConstraintId;
	}

	const FRigidSceneId& FJointConstraintHandle::GetSceneId() const
	{
		return SceneId;
	}

	void FJointConstraintHandle::Reset()
	{
		SceneId.Reset();
		ConstraintId.Reset();
	}

	IJointConstraint* FJointConstraintHandle::PinInternal(const IRigidScene* Scene) const
	{
		return Scene->GetJointConstraint(ConstraintId);
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
