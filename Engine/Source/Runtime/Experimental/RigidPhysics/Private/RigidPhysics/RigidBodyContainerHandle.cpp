// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidBodyContainerHandle.h"

#include "RigidPhysics/RigidScene.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	FRigidBodyContainerHandle::FRigidBodyContainerHandle(const FRigidSceneId& InSceneId, const FRigidObjectId& InBodyContainerId)
		: BodyContainerId(InBodyContainerId)
		, SceneId(InSceneId)
	{
	}

	void FRigidBodyContainerHandle::Reset()
	{
		BodyContainerId.Reset();
		SceneId.Reset();
	}

	const FRigidObjectId& FRigidBodyContainerHandle::GetId() const
	{
		return BodyContainerId;
	}

	const FRigidSceneId& FRigidBodyContainerHandle::GetSceneId() const
	{
		return SceneId;
	}

	IRigidBodyContainer* FRigidBodyContainerHandle::PinInternal(const IRigidScene* InScene) const
	{
		return InScene->GetBodyContainer(GetId());
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
