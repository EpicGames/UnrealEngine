// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidBodyHandle.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidScene.h"

namespace UE::Physics
{
	FRigidBodyHandle::FRigidBodyHandle(const FRigidSceneId& InSceneId, const FRigidObjectId& InBodyId)
		: BodyId(InBodyId)
		, SceneId(InSceneId)
	{
	}

	const FRigidObjectId& FRigidBodyHandle::GetId() const
	{
		return BodyId;
	}

	const FRigidSceneId& FRigidBodyHandle::GetSceneId() const
	{
		return SceneId;
	}

	void FRigidBodyHandle::Reset()
	{
		SceneId.Reset();
		BodyId.Reset();
	}

	uint32 GetTypeHash(const FRigidBodyHandle& InHandle)
	{
		return HashCombine(GetTypeHash(InHandle.BodyId), GetTypeHash(InHandle.GetSceneId()));
	}

	IRigidBody* FRigidBodyHandle::PinInternal(const IRigidScene* Scene) const
	{
		return Scene->GetBody(GetId());
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
