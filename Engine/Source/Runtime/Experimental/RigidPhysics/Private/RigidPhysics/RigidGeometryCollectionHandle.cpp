// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidGeometryCollectionHandle.h"

#include "RigidPhysics/RigidBodyContainer.h"
#include "RigidPhysics/RigidGeometryCollection.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	FRigidGeometryCollectionHandle::FRigidGeometryCollectionHandle(const FRigidSceneId& InSceneId, const FRigidObjectId& InBodyContainerId)
		: BodyContainerHandle(InSceneId, InBodyContainerId)
	{
	}

	void FRigidGeometryCollectionHandle::Reset()
	{
		BodyContainerHandle.Reset();
	}

	const FRigidObjectId& FRigidGeometryCollectionHandle::GetId() const
	{
		return BodyContainerHandle.GetId();
	}

	const FRigidSceneId& FRigidGeometryCollectionHandle::GetSceneId() const
	{
		return BodyContainerHandle.GetSceneId();
	}

	FRigidBodyContainerHandle FRigidGeometryCollectionHandle::AsBodyContainerHandle() const
	{
		return BodyContainerHandle;
	}

	FRigidGeometryCollectionHandle::operator FRigidBodyContainerHandle() const
	{
		return AsBodyContainerHandle();
	}

	IRigidGeometryCollection* FRigidGeometryCollectionHandle::PinInternal(IRigidBodyContainer* Container) const
	{
		// TODO_CHAOSAPI: Could use checked cast here because a FRigidGeometryCollectionHandle will 
		// always reference a GC or nothing. Currently it will perform an RTTI cast.
		if (Container != nullptr)
		{
			return Container->AsA<IRigidGeometryCollection>();
		}
		return nullptr;
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
