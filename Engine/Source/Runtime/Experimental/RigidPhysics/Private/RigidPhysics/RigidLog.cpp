// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidLog.h"

#if UE_RIGIDPHYSICS_API_ENABLED

DEFINE_LOG_CATEGORY(LogRigidPhysics);

namespace UE::Physics::Debugger
{
	// Used in natvis for FRigidObjectId.
	// If true, will show <Index.Epoch> for an object Id. E.g.: Body<1.0>
	// If false, will show <Index> for an object Id. E.g.: Body<1>
	static bool GShowIdEpoch = false;

	// Used in natvis for FRigidBodyContainerHandle.
	// Is true when a BodyContainerHandle contains a SceneHandle, which is
	// required for natvis to be able to reach the object referenced by the handle
	// (in the runtime, the scene is provided by the Context). 
	static bool GHaveSceneHandle = UE_RIGIDPHYSICS_CONTAINERSCENEHANDLE_ENABLED;
}

#endif // UE_RIGIDPHYSICS_API_ENABLED