// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidMaterialIndex.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	FMaterialIndex::FMaterialIndex(const uint16 InIndex)
		: Index(InIndex)
	{
	}

	FMaterialIndex::operator uint16() const
	{
		return Index;
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
