// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidMaterials.h"

#include "Chaos/PhysicalMaterials.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	FMaterialHandle::FMaterialHandle(const Chaos::FMaterialHandle& InMaterialHandle)
		: HandleData(InMaterialHandle.InnerHandle.AsUint())
	{
	}

	Chaos::FMaterialHandle FMaterialHandle::GetMaterialHandle() const
	{
		Chaos::FMaterialHandle Result;
		Result.InnerHandle.FromUint(HandleData);
		return Result;
	}

	Chaos::FChaosPhysicsMaterial* FMaterialHandle::GetMaterial() const
	{
		return GetMaterialHandle().Get();
	}

	FMaterialMaskHandle::FMaterialMaskHandle(const Chaos::FMaterialMaskHandle& InMaterialMaskHandle)
		: HandleData(InMaterialMaskHandle.InnerHandle.AsUint())
	{
	}

	Chaos::FMaterialMaskHandle FMaterialMaskHandle::GetMaterialMaskHandle() const
	{
		Chaos::FMaterialMaskHandle Result;
		Result.InnerHandle.FromUint(HandleData);
		return Result;
	}

	Chaos::FChaosPhysicsMaterialMask* FMaterialMaskHandle::GetMaterialMask() const
	{
		return GetMaterialMaskHandle().Get();
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
