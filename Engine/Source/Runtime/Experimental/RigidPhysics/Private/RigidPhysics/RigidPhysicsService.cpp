// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidPhysicsService.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	FRigidPhysicsService& FRigidPhysicsService::GetInstance()
	{
		static FRigidPhysicsService Instance;
		return Instance;
	}

	void FRigidPhysicsService::RegisterFactory(TUniquePtr<IRigidFactory>&& InFactory, TUniquePtr<IRigidSceneSettings>&& InDefaultSettings)
	{
		Factory = MoveTemp(InFactory);
		SceneSettings = MoveTemp(InDefaultSettings);
	}

	FRigidSceneHandle FRigidPhysicsService::CreateScene(const FRigidDebugName& InName) const
	{
		if (ensure(Factory.IsValid() && SceneSettings.IsValid()))
		{
			return Factory->CreateScene(InName, SceneSettings.Get());
		}
		return FRigidSceneHandle();
	}

	void FRigidPhysicsService::DestroyScene(const FRigidSceneHandle& InHandle) const
	{
		if (ensure(Factory.IsValid()))
		{
			return Factory->DestroyScene(InHandle);
		}
	}

	void FRigidPhysicsService::Reset()
	{
		Factory = nullptr;
		SceneSettings = nullptr;
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
