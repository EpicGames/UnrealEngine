// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidSceneHandle.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidLog.h"
#include "RigidPhysics/RigidScene.h"

namespace UE::Physics
{
	FRigidSceneHandle::FRigidSceneHandle(FRigidScene* InScene)
		: SceneId(InScene->GetId())
		, SceneRaw(InScene)
#if UE_RIGIDPHYSICS_SCENEHANDLE_SAFTEY_ENABLED
		, SceneWeak(InScene)
#endif
	{
	}

	FRigidSceneHandle::FRigidSceneHandle(const FRigidSceneId& InSceneId)
		: SceneId(InSceneId)
	{
		IRigidScene* BaseScene = InSceneId.Get();
		UE_RIGIDPHYSICS_CHECK(BaseScene);
		if (BaseScene)
		{
			SceneRaw = BaseScene->AsAChecked<FRigidScene>();
#if UE_RIGIDPHYSICS_SCENEHANDLE_SAFTEY_ENABLED
			SceneWeak = SceneRaw;
#endif // UE_RIGIDPHYSICS_SCENEHANDLE_SAFTEY_ENABLED
		}
	}

	FRigidContextGameRO FRigidSceneHandle::LockRO() const
	{
		DoSafetyCheck();
		return FRigidContextGameRO(GetScene());
	}

	FRigidContextGameRW FRigidSceneHandle::LockRW() const
	{
		DoSafetyCheck();
		return FRigidContextGameRW(GetScene());
	}

	void FRigidSceneHandle::Reset()
	{
		SceneId.Reset();
		SceneRaw = nullptr;
#if UE_RIGIDPHYSICS_SCENEHANDLE_SAFTEY_ENABLED
		SceneWeak.Reset();
#endif
	}

	FRigidSceneId FRigidSceneHandle::GetId() const
	{
		return SceneId;
	}

	IRigidScene* FRigidSceneHandle::GetScene() const
	{
		return SceneId.Get();
	}

	void FRigidSceneHandle::DoSafetyCheck() const
	{
#if UE_RIGIDPHYSICS_SCENEHANDLE_SAFTEY_ENABLED
		// NOTE: In order to support the UE_RIGIDPHYSICS_SCENEHANDLE_SAFTEY_ENABLED=0 optimization,
		// SceneHandles cannot be used after the scene has expired. The onus is on the 
		// user to ensure that this is the case, but when we have handle safety
		// enabled we can assert on this. Without the weak pointer, there's no way 
		// to know that the raw pointer held in SceneWrapper points to a deleted object.
		check(SceneRaw == SceneWeak.Pin().Get());
#endif
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
