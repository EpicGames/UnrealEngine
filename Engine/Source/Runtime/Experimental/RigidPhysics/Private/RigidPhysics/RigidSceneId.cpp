// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidSceneId.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidScene.h"
#include "RigidPhysics/RigidSceneRegistry.h"

namespace UE::Physics
{
	IRigidScene* FRigidSceneId::Get() const
	{
		FRigidSceneRegistry& Registry = FRigidSceneRegistry::GetInstance();
		return Registry.Get(*this);
	}

	void FRigidSceneId::Reset()
	{
		Id = INDEX_NONE;
	}

	uint32 GetTypeHash(const FRigidSceneId& InId)
	{
		return ::GetTypeHash(InId.Id);
	}

} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
