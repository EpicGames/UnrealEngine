// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidSceneRegistry.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidLog.h"

namespace UE::Physics
{
	FRigidSceneRegistry FRigidSceneRegistry::Instance;

	FRigidSceneRegistry& FRigidSceneRegistry::GetInstance()
	{
		return Instance;
	}

	FRigidSceneId FRigidSceneRegistry::Register(IRigidScene* Scene)
	{
		checkf(SceneCount < MaxScenes, TEXT("FRigidSceneRegistry: Ran out of scenes. Trying to create scene %d when %d is the max"), SceneCount, MaxScenes);
		if (SceneCount >= MaxScenes)
		{
			return FRigidSceneId();
		}

		uint8 Id = SceneCount;
		++SceneCount;

		FRigidSceneId Result;
		Result.Id = Id;
		Scenes[Id] = Scene;
		return Result;
	}

	void FRigidSceneRegistry::Unregister(const FRigidSceneId Id)
	{
		const int32 ArrayIndex = Id.Id;
		check(ArrayIndex < SceneCount);

		if (ArrayIndex < SceneCount)
		{
			// TODO: This is technically not thread safe as someone could be reading this pointer at the same time.
			// Consider deferring the null for later with a queue.
			Scenes[ArrayIndex] = nullptr;
		}
	}

	IRigidScene* FRigidSceneRegistry::Get(const FRigidSceneId Id) const
	{
		const int32 ArrayIndex = Id.Id;
		if (ArrayIndex < SceneCount)
		{
			return Scenes[ArrayIndex];
		}
		return nullptr;
	}

	void FRigidSceneRegistry::Reset()
	{
		for (int32 I = 0; I < MaxScenes; ++I)
		{
			Scenes[I] = nullptr;
		}
		SceneCount = 0;
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
