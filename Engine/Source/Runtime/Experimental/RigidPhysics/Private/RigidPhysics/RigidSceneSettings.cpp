// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidSceneSettings.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(IRigidSceneSettings, IRigidTyped);
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
