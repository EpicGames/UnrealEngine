// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidShapeInstanceHandle.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidScene.h"

namespace UE::Physics
{
	FRigidShapeInstanceHandle::FRigidShapeInstanceHandle(const FRigidSceneId& InSceneId, const FRigidObjectId& InId)
		: Id(InId)
		, SceneId(InSceneId)
	{
	}

	const FRigidObjectId& FRigidShapeInstanceHandle::GetId() const
	{
		return Id;
	}
	const FRigidSceneId& FRigidShapeInstanceHandle::GetSceneId() const
	{
		return SceneId;
	}

	IRigidShapeInstance* FRigidShapeInstanceHandle::PinInternal(const IRigidScene* Scene) const
	{
		return Scene->GetShape(Id);
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
