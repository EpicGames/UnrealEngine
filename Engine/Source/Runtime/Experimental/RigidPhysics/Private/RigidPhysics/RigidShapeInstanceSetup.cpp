// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidShapeInstanceSetup.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	static constexpr Chaos::EFilterFlags DefaultFilterFlags = Chaos::EFilterFlags::SimpleCollision | Chaos::EFilterFlags::ContactNotify | Chaos::EFilterFlags::ModifyContacts;

	FRigidShapeInstanceSetup::FRigidShapeInstanceSetup(const FAnyGeometry& InGeometry)
		: Geometry(InGeometry)
		, ShapeFilterData(Chaos::Filter::FShapeFilterBuilder::BuildBlockAll(DefaultFilterFlags))
	{
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
