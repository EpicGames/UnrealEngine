// Copyright Epic Games, Inc. All Rights Reserved.
#include "RigidPhysics/RigidTyped.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace UE::Physics
{
	const FRigidTypeId& IRigidTyped::GetStaticTypeId()
	{
		static const TCHAR* StaticTypeName = TEXT("IRigidTyped");
		static FRigidTypeId StaticTypeId = FRigidTypeId(StaticTypeName, nullptr);
		return StaticTypeId;
	}

	const FRigidTypeId& IRigidTyped::GetTypeId() const
	{
		return IRigidTyped::GetStaticTypeId();
	}
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
