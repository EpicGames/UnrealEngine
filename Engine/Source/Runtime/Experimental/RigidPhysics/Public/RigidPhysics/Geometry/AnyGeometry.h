// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Misc/TVariant.h"
#include "RigidPhysics/Geometry/CommonCoreGeometry.h"
#include "RigidPhysics/Geometry/ConvexGeometry.h"
#include "RigidPhysics/Geometry/HeightFieldGeometry.h"
#include "RigidPhysics/Geometry/TriangleMeshGeometry.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class FAnyGeometry
	{
	public:
		FAnyGeometry() = default;
		FAnyGeometry(const FAnyGeometry&) = default;
		FAnyGeometry(FAnyGeometry&&) = default;

		template <typename GeometryType>
		FAnyGeometry(const GeometryType& InGeometry)
		{
			Geometry.Set<GeometryType>(InGeometry);
		}

		FAnyGeometry& operator=(const FAnyGeometry& InGeometry) = default;

		template <typename GeometryType>
		FAnyGeometry& operator=(const GeometryType& InGeometry)
		{
			Geometry.Set<GeometryType>(InGeometry);
			return *this;
		}

		bool operator==(const FAnyGeometry& Rhs) const;
		bool operator!=(const FAnyGeometry& Rhs) const = default;

		template <typename GeometryType>
		bool IsA() const
		{
			return Geometry.IsType<GeometryType>();
		}

		template <typename GeometryType>
		GeometryType& Get()
		{
			return Geometry.Get<GeometryType>();
		}

		template <typename GeometryType>
		const GeometryType& Get() const
		{
			return Geometry.Get<GeometryType>();
		}

		template <typename GeometryType>
		GeometryType* TryGet()
		{
			return Geometry.TryGet<GeometryType>();
		}

		template <typename GeometryType>
		const GeometryType* TryGet() const
		{
			return Geometry.TryGet<GeometryType>();
		}

		template <typename TLambda>
		decltype(auto) Visit(const TLambda& Lambda)
		{
			return ::Visit(Lambda, Geometry);
		}

		template <typename TLambda>
		decltype(auto) Visit(const TLambda& Lambda) const
		{
			return ::Visit(Lambda, Geometry);
		}

		RIGIDPHYSICS_API FString ToString() const;
	private:
		TVariant<FSphereGeometry, FBoxGeometry, FCapsuleGeometry, FConvexGeometry, FTriangleMeshGeometry, FHeightFieldGeometry> Geometry;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
