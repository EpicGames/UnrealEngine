// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "CoreMinimal.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	struct FSphereGeometry
	{
		FSphereGeometry() = default;
		RIGIDPHYSICS_API FSphereGeometry(const float InRadius);

		bool operator==(const FSphereGeometry&) const = default;
		bool operator!=(const FSphereGeometry&) const = default;

		RIGIDPHYSICS_API float GetRadius() const;
		RIGIDPHYSICS_API void SetRadius(float InRadius);

		RIGIDPHYSICS_API FString ToString() const;

	private:
		float Radius = 0;
	};

	struct FBoxGeometry
	{
		FBoxGeometry() = default;
		RIGIDPHYSICS_API FBoxGeometry(const FVector3f& InExtents);

		bool operator==(const FBoxGeometry&) const = default;
		bool operator!=(const FBoxGeometry&) const = default;

		RIGIDPHYSICS_API FVector3f GetExtents() const;
		RIGIDPHYSICS_API void SetExtents(const FVector3f& InExtents);

		RIGIDPHYSICS_API FString ToString() const;

	private:
		FVector3f Extents = FVector3f::Zero();
	};

	struct FCapsuleGeometry
	{
		FCapsuleGeometry() = default;
		RIGIDPHYSICS_API FCapsuleGeometry(float InRadius, float InSegmentHalfLength);

		bool operator==(const FCapsuleGeometry&) const = default;
		bool operator!=(const FCapsuleGeometry&) const = default;

		RIGIDPHYSICS_API float GetRadius() const;
		RIGIDPHYSICS_API void SetRadius(const float InRadius);

		RIGIDPHYSICS_API float GetSegmentHalfLength() const;
		RIGIDPHYSICS_API void SetSegmentHalfLength(const float InHalfLength);

		RIGIDPHYSICS_API FString ToString() const;

	private:
		float Radius = 0;
		float SegmentHalfLength = 0;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
