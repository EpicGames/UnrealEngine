// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos
{
	class FConvex;
} // namespace Chaos

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class FConvexGeometrySetup
	{
	public:
		enum class EBuildMethod : uint8
		{
			Default = 0,
			Original = 1,
			ConvexHull3 = 2,
			ConvexHull3Simplified = 3,
		};

		TArray<FVector3f> Vertices;
		float Margin = 0.f;
		EBuildMethod BuildMethod = EBuildMethod::Default;
	};

	class FConvexGeometry
	{
	public:
		FConvexGeometry() = delete;
		RIGIDPHYSICS_API ~FConvexGeometry();
		RIGIDPHYSICS_API FConvexGeometry(const FConvexGeometry&);
		RIGIDPHYSICS_API FConvexGeometry(FConvexGeometry&&);
		RIGIDPHYSICS_API FConvexGeometry& operator=(const FConvexGeometry&);
		RIGIDPHYSICS_API FConvexGeometry& operator=(FConvexGeometry&&);
		RIGIDPHYSICS_API FConvexGeometry(FConvexGeometrySetup&& InSetup);
		UE_INTERNAL RIGIDPHYSICS_API FConvexGeometry(Chaos::FConvex* InImplicit);

		bool operator==(const FConvexGeometry&) const = default;
		bool operator!=(const FConvexGeometry&) const = default;

		RIGIDPHYSICS_API int32 GetNumVertices() const;
		RIGIDPHYSICS_API FVector3f GetVertex(const int32 InIndex) const;
		RIGIDPHYSICS_API float GetMargin() const;

		RIGIDPHYSICS_API FString ToString() const;

		UE_INTERNAL RIGIDPHYSICS_API TRefCountPtr<Chaos::FConvex> GetImplicit() const;

	private:
		TRefCountPtr<Chaos::FConvex> Implicit;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
