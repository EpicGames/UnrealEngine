// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos
{
	class FHeightField;
} // namespace Chaos

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class FHeightFieldGeometrySetup
	{
	public:
		TArray<float> Heights;
		TArray<uint8> MaterialIndices;
		int32 NumRows = 0;
		int32 NumCols = 0;
		FVector3f Scale = FVector3f::OneVector;
	};

	class FHeightFieldGeometry
	{
	public:
		FHeightFieldGeometry() = delete;
		RIGIDPHYSICS_API ~FHeightFieldGeometry();
		RIGIDPHYSICS_API FHeightFieldGeometry(const FHeightFieldGeometry&);
		RIGIDPHYSICS_API FHeightFieldGeometry(FHeightFieldGeometry&&);
		RIGIDPHYSICS_API FHeightFieldGeometry& operator=(const FHeightFieldGeometry&);
		RIGIDPHYSICS_API FHeightFieldGeometry& operator=(FHeightFieldGeometry&&);
		RIGIDPHYSICS_API FHeightFieldGeometry(FHeightFieldGeometrySetup&& InSetup);
		UE_INTERNAL RIGIDPHYSICS_API FHeightFieldGeometry(Chaos::FHeightField* InImplicit);

		bool operator==(const FHeightFieldGeometry&) const = default;
		bool operator!=(const FHeightFieldGeometry&) const = default;

		RIGIDPHYSICS_API int32 GetNumRows() const;
		RIGIDPHYSICS_API int32 GetNumCols() const;
		RIGIDPHYSICS_API FVector3f GetScale() const;

		RIGIDPHYSICS_API float GetHeight(const int32 InIndex) const;
		RIGIDPHYSICS_API uint8 GetMaterialIndex(const int32 InIndex) const;

		RIGIDPHYSICS_API FString ToString() const;

		UE_INTERNAL RIGIDPHYSICS_API TRefCountPtr<Chaos::FHeightField> GetImplicit() const;

	private:
		TRefCountPtr<Chaos::FHeightField> Implicit;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
