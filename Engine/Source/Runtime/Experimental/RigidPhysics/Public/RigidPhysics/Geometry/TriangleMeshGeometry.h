// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidMaterialIndex.h"

namespace Chaos
{
	class FTriangleMeshImplicitObject;
} // namespace Chaos

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class FTriangleMeshGeometrySetup
	{
	public:
		enum class EPerFaceBoundsRepresentation : uint8
		{
			VectorizedFloat = 0,
			Byte,
		};

		TArray<FVector3f> Vertices;
		TArray<FInt32Vector3> TriangleIndices;
		TArray<FMaterialIndex> MaterialIndices;
		TArray<int32> ExternalFaceIndexMap;
		TArray<int32> ExternalVertexIndexMap;
		bool bCullsBackFaceRaycast = false;
		EPerFaceBoundsRepresentation PerFaceBoundsRepresentation = EPerFaceBoundsRepresentation::VectorizedFloat;
	};

	class FTriangleMeshGeometry
	{
	public:
		using EPerFaceBoundsRepresentation = FTriangleMeshGeometrySetup::EPerFaceBoundsRepresentation;

		RIGIDPHYSICS_API ~FTriangleMeshGeometry();
		RIGIDPHYSICS_API FTriangleMeshGeometry(const FTriangleMeshGeometry&);
		RIGIDPHYSICS_API FTriangleMeshGeometry(FTriangleMeshGeometry&&);
		RIGIDPHYSICS_API FTriangleMeshGeometry& operator=(const FTriangleMeshGeometry&);
		RIGIDPHYSICS_API FTriangleMeshGeometry& operator=(FTriangleMeshGeometry&&);
		UE_INTERNAL RIGIDPHYSICS_API FTriangleMeshGeometry(TRefCountPtr<Chaos::FTriangleMeshImplicitObject> InImplicit);
		RIGIDPHYSICS_API FTriangleMeshGeometry(FTriangleMeshGeometrySetup&& InSetup);

		bool operator==(const FTriangleMeshGeometry&) const = default;
		bool operator!=(const FTriangleMeshGeometry&) const = default;

		RIGIDPHYSICS_API int32 GetNumVertices() const;
		RIGIDPHYSICS_API FVector3f GetVertex(const int32 InIndex) const;

		RIGIDPHYSICS_API int32 GetNumTriangleIndices() const;
		RIGIDPHYSICS_API FInt32Vector3 GetTriangleIndices(const int32 InIndex) const;

		RIGIDPHYSICS_API int32 GetNumMaterialIndices() const;
		RIGIDPHYSICS_API FMaterialIndex GetMaterialIndex(const int32 InIndex) const;

		RIGIDPHYSICS_API bool HasExternalFaceIndices() const;
		RIGIDPHYSICS_API int32 GetExternalFaceIndex(const int32 InIndex) const;

		RIGIDPHYSICS_API bool HasExternalVertexIndices() const;
		RIGIDPHYSICS_API int32 GetExternalVertexIndex(const int32 InIndex) const;

		RIGIDPHYSICS_API bool GetCullsBackFaceRaycast() const;
		RIGIDPHYSICS_API EPerFaceBoundsRepresentation GetPerFaceBoundsRepresentation() const;

		RIGIDPHYSICS_API FString ToString() const;

		UE_INTERNAL RIGIDPHYSICS_API TRefCountPtr<Chaos::FTriangleMeshImplicitObject> GetImplicit() const;

	private:
		template <typename IndexType>
		void BuildInternal(FTriangleMeshGeometrySetup&& InSetup);

		TRefCountPtr<Chaos::FTriangleMeshImplicitObject> Implicit;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
