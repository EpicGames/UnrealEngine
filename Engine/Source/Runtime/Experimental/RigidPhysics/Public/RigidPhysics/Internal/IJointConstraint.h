// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/JointConstraintHandle.h"
#include "RigidPhysics/RigidTyped.h"

namespace UE::Physics
{
	class UE_INTERNAL IJointConstraint : public IRigidTyped
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(RIGIDPHYSICS_API, IJointConstraint);

		IJointConstraint() = default;
		virtual ~IJointConstraint() = default;

		UE_INTERNAL virtual FJointConstraintHandle GetHandle() const = 0;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
