// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Math/Transform.h"
#include "RigidPhysics/JointConstraint.h"
#include "RigidPhysics/JointConstraint6DOFHandle.h"

namespace UE::Physics
{
	class UE_INTERNAL IJointConstraint6DOF : public IJointConstraint
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(RIGIDPHYSICS_API, IJointConstraint6DOF);

		IJointConstraint6DOF() = default;
		virtual ~IJointConstraint6DOF() = default;

		UE_INTERNAL virtual FJointConstraintHandle GetHandle() const = 0;
		UE_INTERNAL RIGIDPHYSICS_API FJointConstraint6DOFHandle GetTypedHandle() const;

		UE_INTERNAL RIGIDPHYSICS_API virtual void Activate();

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetBodies(IRigidBody* Body0, IRigidBody* Body1);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetJointTransforms(const FTransform3d& InTransform0, const FTransform3d& InTransform1);

		UE_INTERNAL RIGIDPHYSICS_API virtual double GetLinearLimit() const;
		UE_INTERNAL RIGIDPHYSICS_API virtual void SetLinearLimit(const double InLinearLimit);

		UE_INTERNAL RIGIDPHYSICS_API virtual EJointMotionType GetLinearMotionTypesX() const;
		UE_INTERNAL RIGIDPHYSICS_API virtual void SetLinearMotionTypesX(EJointMotionType InMotionType);

		UE_INTERNAL RIGIDPHYSICS_API virtual EJointMotionType GetLinearMotionTypesY() const;
		UE_INTERNAL RIGIDPHYSICS_API virtual void SetLinearMotionTypesY(EJointMotionType InMotionType);

		UE_INTERNAL RIGIDPHYSICS_API virtual EJointMotionType GetLinearMotionTypesZ() const;
		UE_INTERNAL RIGIDPHYSICS_API virtual void SetLinearMotionTypesZ(EJointMotionType InMotionType);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetAngularMotionTypesX(const EJointMotionType InMotionType);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetAngularMotionTypesY(const EJointMotionType InMotionType);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetAngularMotionTypesZ(const EJointMotionType InMotionType);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetAngularLimits(const FVector3d& InAngularLimits);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetCollisionEnabled(const bool bCollisionEnabled);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetProjectionEnabled(const bool bProjectionEnabled);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetMassConditioningEnabled(const bool bMassConditioningEnabled);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetUseLinearSolver(const bool bUseLinearSolver);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetSoftLinearLimitsEnabled(const bool bSoftLinearLimitsEnabled);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetSoftTwistLimitsEnabled(const bool bSoftTwistLimitsEnabled);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetSoftSwingLimitsEnabled(const bool bSoftSwingLimitsEnabled);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetSoftLinearStiffness(const double InSoftLinearStiffness);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetSoftLinearDamping(const double InSoftLinearDamping);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetSoftTwistStiffness(const double InSoftTwistStiffness);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetSoftTwistDamping(const double InSoftTwistDamping);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetSoftSwingStiffness(const double InSoftSwingStiffness);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetSoftSwingDamping(const double InSoftSwingDamping);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetProjectionLinearAlpha(const double InProjectionLinearAlpha);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetProjectionAngularAlpha(const double InProjectionAngularAlpha);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetProjectionLinearTolerance(const double InProjectionLinearTolerance);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetProjectionAngularTolerance(const double InProjectionAngularTolerance);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetLinearContactDistance(const double InLinearContactDistance);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetTwistContactDistance(const double InTwistContactDistance);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetSwingContactDistance(const double InSwingContactDistance);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetAngularDriveStiffness(const FVector3d& InAngularDriveStiffness);
		UE_INTERNAL RIGIDPHYSICS_API virtual void SetAngularDriveDamping(const FVector3d& InAngularDriveDamping);
		UE_INTERNAL RIGIDPHYSICS_API virtual void SetLinearDriveStiffness(const FVector3d& InLinearDriveStiffness);
		UE_INTERNAL RIGIDPHYSICS_API virtual void SetLinearDriveDamping(const FVector3d& InLinearDriveDamping);
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
