// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Math/Bounds.h"
#include "Math/Transform.h"
#include "RigidPhysics/RigidBodyHandle.h"
#include "RigidPhysics/RigidLog.h"
#include "RigidPhysics/RigidShapeInstance.h"
#include "RigidPhysics/RigidTyped.h"

namespace UE::Physics
{
	// The full rigid body API. There will be several implementations of this API. E.g., synchronous
	// physics, asynchronous physics, immediate physics, remote physics, etc.
	// NOTE: for some implementations like async physics, there are both Game- and Physics-Context
	// implementations within the async implementation. Pinning a handle will return the object 
	// appropriate for the context.
	class UE_INTERNAL IRigidBody : public IRigidTyped
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(RIGIDPHYSICS_API, IRigidBody);

		IRigidBody() = default;
		virtual ~IRigidBody() = default;

		// Is the body active in its scene?
		// Inactive bodies do not participate in physics in any way.
		UE_INTERNAL RIGIDPHYSICS_API virtual bool IsActive() const;
		UE_INTERNAL RIGIDPHYSICS_API virtual void Activate();
		UE_INTERNAL RIGIDPHYSICS_API virtual void Deactivate();


		// The handle to this body
		UE_INTERNAL virtual FRigidBodyHandle GetHandle() const = 0;

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetId(const FRigidObjectId& InId);

		// Is this body Static? Static bodies can be moved but have no velocity.
		UE_INTERNAL virtual bool IsStatic() const = 0;

		// Is this body kinematic? Kinematic bodies can be explicitly moved and have velocity, but do
		// not react to forces, collisions, etc.
		UE_INTERNAL virtual bool IsKinematic() const = 0;

		// Is this a dynamic body? Dynamic bodies have momentum and react to forces and collisions.
		UE_INTERNAL virtual bool IsDynamic() const = 0;

		// Make the body dynamic (will only affect Kinematic bodies. Static bodies cannot be made dynamic)
		UE_INTERNAL RIGIDPHYSICS_API virtual void SetIsDynamic(bool bIsDynamic) const;

		// Is this a sleeping dynamic body? Sleeping bodies are dynamic bodies that are ignored by
		// the simulation (as an optimization) until the scene changes in a way that might affect them, 
		// or they are explicitly woken.
		UE_INTERNAL RIGIDPHYSICS_API virtual bool IsSleeping() const;

		UE_INTERNAL virtual int32 GetNumShapes() const = 0;
		UE_INTERNAL virtual IRigidShapeInstance* GetShape(int32 ShapeIndex) const = 0;
		UE_INTERNAL RIGIDPHYSICS_API virtual IRigidShapeInstance* CreateShape(const FRigidShapeInstanceSetup& InSetup);
		UE_INTERNAL UE_RIGIDPHYSICS_LEGACY_API RIGIDPHYSICS_API virtual IRigidShapeInstance* CreateShape(TUniquePtr<Chaos::FPerShapeData>&& InShapeData);
		UE_INTERNAL RIGIDPHYSICS_API virtual void DestroyShape(IRigidShapeInstance* InShape, bool bWakeTouching);

		// Set the transform and initialize body state (velocity etc.) at the new location.
		UE_INTERNAL RIGIDPHYSICS_API virtual void InitTransform(const FTransform3d& InTransform);

		// Set the transform without changing any other rigid body state.
		UE_INTERNAL RIGIDPHYSICS_API virtual void UpdateTransform(const FTransform3d& InTransform);

		// Set the kinematic target. The body will attempt to reach the target at the next
		// tick. Kinematic bodies will always reach the target bscause they do not respond
		// to forces or collisions. Dynamic bodies will only reach their target if their
		// velocity in the absence of interactions (forces, collisions, or other constraints).
		UE_INTERNAL RIGIDPHYSICS_API virtual void SetKinematicTarget(const FTransform3d& InTransform);

		// Get the current transform. NOTE: This will immediately reflect any recent calls to
		// to InitTransform or UpdateTransform, but will not reflect the latest KinematicTarget
		// until after the next physics tick.
		UE_INTERNAL RIGIDPHYSICS_API virtual FTransform3d GetTransform() const;

		// Get the world-space bounds at the current transform
		UE_INTERNAL RIGIDPHYSICS_API virtual FBounds3d GetBounds() const;

		// TODO: Maybe just Set/GetMassProperties
		UE_INTERNAL RIGIDPHYSICS_API virtual double GetMass() const;
		UE_INTERNAL RIGIDPHYSICS_API virtual void SetMass(double InM);

		UE_INTERNAL RIGIDPHYSICS_API virtual FVector3d GetInertia() const;
		UE_INTERNAL RIGIDPHYSICS_API virtual void SetInertia(const FVector3d& InInertia);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetCenterOfMass(const FVector3d& InCenterOfMass);
		UE_INTERNAL RIGIDPHYSICS_API virtual void SetRotationOfMass(const FQuat& InRotationOfMass);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetCCDEnabled(bool bInEnabled);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetMACDEnabled(bool bInEnabled);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetSmoothEdgeCollisionsEnabled(bool bInEnabled);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetMaxLinearVelocity(float InMax);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetMaxAngularVelocity(float InMax);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetInertiaConditioningEnabled(bool bInEnabled);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetGyroscopicTorqueEnabled(bool bInEnabled);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetGravityGroup(int32 InId);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetGravityScale(float InScale);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetWakeEventsEnabled(bool bInEnabled);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetNumPositionIterations(int32 InNum);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetNumVelocityIterations(int32 InNum);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetNumProjectionIterations(int32 InNum);

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetIsSleeping(bool bInSleeping);
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
