// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidBodyContainerHandle.h"
#include "RigidPhysics/RigidLog.h"
#include "RigidPhysics/RigidObjectId.h"
#include "RigidPhysics/RigidObjectRegistry.h"
#include "RigidPhysics/RigidTyped.h"

namespace UE::Physics
{
	// The interface for rigid body containers. There will be several implementations of this API. E.g., synchronous
	// physics, asynchronous physics, immediate physics, remote physics, etc. 
	class UE_INTERNAL IRigidBodyContainer : public IRigidTyped
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(RIGIDPHYSICS_API, IRigidBodyContainer);

		IRigidBodyContainer() = default;
		virtual ~IRigidBodyContainer() = default;

		UE_INTERNAL virtual FRigidBodyContainerHandle GetHandle() const = 0;

		UE_INTERNAL virtual int32 GetNumBodies() const = 0;

		UE_INTERNAL virtual IRigidBody* GetBody(const FRigidObjectId& InBodyId) const = 0;

		//
		// Below here is GT only. See if we can separate the Game and SimCallback APIs
		//

		UE_INTERNAL RIGIDPHYSICS_API virtual void SetId(const FRigidObjectId& InBodyId);

		UE_INTERNAL RIGIDPHYSICS_API virtual IRigidBody* CreateBody(const FRigidDebugName& InName, ERigidMovementType InMovementType);
		UE_INTERNAL RIGIDPHYSICS_API virtual void DestroyBody(IRigidBody* InBody);
	};

	class UE_INTERNAL FRigidBodyContainer : public IRigidBodyContainer
	{
	public:
		UE_INTERNAL RIGIDPHYSICS_API FRigidBodyContainer();
		virtual ~FRigidBodyContainer() = default;

		UE_INTERNAL RIGIDPHYSICS_API virtual IRigidBody* GetBody(const FRigidObjectId& InBodyId) const override final;

	protected:
		UE_INTERNAL RIGIDPHYSICS_API FRigidBodyRegistry* GetBodyRegistry() const;

	private:
		TUniquePtr<FRigidBodyRegistry> BodyRegistry;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
