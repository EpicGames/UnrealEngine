// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/Internal/IRigidBodyContainer.h"
#include "RigidPhysics/RigidGeometryCollectionHandle.h"

namespace UE::Physics
{
	// The interface for rigid body containers. There will be several implementations of this API. E.g., synchronous
	// physics, asynchronous physics, immediate physics, remote physics, etc. 
	class UE_INTERNAL IRigidGeometryCollection : public IRigidBodyContainer
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(RIGIDPHYSICS_API, IRigidGeometryCollection);

		IRigidGeometryCollection() = default;
		virtual ~IRigidGeometryCollection() = default;

		// TODO_CHAOSAPI: Figure out the public GeometryCollection API!!
		UE_INTERNAL virtual IRigidBody* GetBodyAt(int32 BodyIndex) const = 0;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
