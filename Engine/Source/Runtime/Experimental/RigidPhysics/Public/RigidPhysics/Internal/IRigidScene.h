// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidLockable.h"
#include "RigidPhysics/RigidLog.h"
#include "RigidPhysics/RigidObjectId.h"
#include "RigidPhysics/RigidSceneHandle.h"
#include "RigidPhysics/RigidTyped.h"

namespace UE::Physics
{
	// The interface to a rigid body scene. The rigid body scene owns the physics simulations and
	// manages a set of rigid body containers which in turn manage sets of rigid bodies.
	class UE_INTERNAL IRigidScene : public IRigidLockable
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(RIGIDPHYSICS_API, IRigidScene);

		IRigidScene() = default;
		virtual ~IRigidScene() = default;

		UE_INTERNAL virtual FRigidSceneId GetId() const = 0;
		UE_INTERNAL virtual FRigidSceneHandle GetHandle() const = 0;

		UE_INTERNAL virtual IRigidBody* GetBody(const FRigidObjectId& InBodyId) const = 0;

		UE_INTERNAL virtual IRigidBodyContainer* GetBodyContainer(const FRigidObjectId& InBodyContainerId) const = 0;

		UE_INTERNAL virtual IRigidGeometryCollection* GetGeometryCollection(const FRigidObjectId& InGeometryCollectionId) const = 0;

		UE_INTERNAL RIGIDPHYSICS_API virtual bool Query(const FRigidQueryDesc& InQueryDesc, TArray<FRigidQueryHit>& OutHits);

		// Advance the simulation by Dt. If the implementation is asynchronous, this will return immediately
		UE_INTERNAL RIGIDPHYSICS_API virtual void StartTick(const float InDt);
		UE_INTERNAL RIGIDPHYSICS_API virtual void WaitOnTick();
		UE_INTERNAL RIGIDPHYSICS_API virtual void EndTick();

		// GT Only API - Move this somehow
		UE_INTERNAL RIGIDPHYSICS_API virtual void RegisterModifier(IRigidSceneModifier* InModifier);
		UE_INTERNAL RIGIDPHYSICS_API virtual void UnregisterModifier(IRigidSceneModifier* InModifier);

		UE_INTERNAL virtual IRigidShapeInstance* GetShape(const FRigidObjectId& InId) const = 0;

		UE_INTERNAL RIGIDPHYSICS_API virtual IRigidBody* CreateBody(const FRigidDebugName& InName, ERigidMovementType InMovementType);
		UE_INTERNAL RIGIDPHYSICS_API virtual void DestroyBody(IRigidBody* InBody);

		UE_INTERNAL RIGIDPHYSICS_API virtual IRigidGeometryCollection* CreateGeometryCollection(const FRigidDebugName& InName, const FSimulationParameters& InParameters);
		UE_INTERNAL RIGIDPHYSICS_API virtual void DestroyGeometryCollection(IRigidGeometryCollection* InGC);

		UE_INTERNAL RIGIDPHYSICS_API virtual void AddDisabledCollisions(const TMap<FRigidBodyHandle, TArray<FRigidBodyHandle>>& InMap);

		UE_INTERNAL RIGIDPHYSICS_API virtual IJointConstraint6DOF* CreateJointConstraint6DOF();
		UE_INTERNAL RIGIDPHYSICS_API virtual void DestroyJointConstraint(IJointConstraint* InConstraint);
		UE_INTERNAL RIGIDPHYSICS_API virtual IJointConstraint* GetJointConstraint(const FRigidObjectId& InConstraintId) const;
	};

	//
	// Base class for the Game Context Scene. This provides the BodyContainerRegistry, partly for convenience
	// and partly so that we can convert Handles to Objects in the debugger with natvis
	//
	class UE_INTERNAL FRigidScene : public IRigidScene
	{
	public:
		RIGIDPHYSICS_API FRigidScene();
		virtual ~FRigidScene() = default;

		UE_INTERNAL RIGIDPHYSICS_API virtual IRigidBody* GetBody(const FRigidObjectId& InBodyId) const override final;
		UE_INTERNAL RIGIDPHYSICS_API virtual IRigidBodyContainer* GetBodyContainer(const FRigidObjectId& InBodyContainerId) const override final;
		UE_INTERNAL RIGIDPHYSICS_API virtual IRigidGeometryCollection* GetGeometryCollection(const FRigidObjectId& InGeometryCollectionId) const override final;

	protected:
		UE_INTERNAL RIGIDPHYSICS_API void DestructScene();

		UE_INTERNAL RIGIDPHYSICS_API FRigidBodyContainerRegistry* GetBodyContainerRegistry() const;
		UE_INTERNAL RIGIDPHYSICS_API FRigidBodyRegistry* GetBodyRegistry() const;
		UE_INTERNAL RIGIDPHYSICS_API FJointConstraintRegistry* GetJointConstraintRegistry() const;

	private:
		TUniquePtr<FRigidBodyContainerRegistry> BodyContainerRegistry;
		TUniquePtr<FRigidBodyRegistry> BodyRegistry;
		TUniquePtr<FJointConstraintRegistry> JointConstraintRegistry;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
