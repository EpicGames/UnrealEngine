// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "BodySetupEnums.h"
#include "Chaos/CollisionFilterData.h"
#include "Math/Transform.h"
#include "RigidPhysics/Geometry/AnyGeometry.h"
#include "RigidPhysics/RigidLog.h"
#include "RigidPhysics/RigidMaterials.h"
#include "RigidPhysics/RigidShapeInstanceHandle.h"
#include "RigidPhysics/RigidTyped.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class UE_INTERNAL IRigidShapeInstance : public IRigidTyped
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(RIGIDPHYSICS_API, IRigidShapeInstance);

		IRigidShapeInstance() = default;
		virtual ~IRigidShapeInstance() = default;

		UE_INTERNAL virtual FRigidShapeInstanceHandle GetHandle() const = 0;

		UE_INTERNAL virtual FAnyGeometry GetGeometry() const = 0;
		// TODO: SetGeometry

		UE_INTERNAL virtual FTransform3f GetLocalTransform() const = 0;
		// TODO: SetLocalTransform

		UE_INTERNAL virtual bool GetQueryEnabled() const = 0;
		UE_INTERNAL virtual void SetQueryEnabled(const bool bInEnabled) = 0;

		UE_INTERNAL virtual bool GetSimEnabled() const = 0;
		UE_INTERNAL virtual void SetSimEnabled(const bool bInEnabled) = 0;

		UE_INTERNAL virtual bool GetIsProbe() const = 0;
		UE_INTERNAL virtual void SetIsProbe(const bool bInIsProbe) = 0;

		UE_INTERNAL virtual ECollisionTraceFlag GetCollisionTraceType() const = 0;
		UE_INTERNAL virtual void SetCollisionTraceType(const ECollisionTraceFlag InTraceFlag) = 0;

		UE_INTERNAL virtual Chaos::Filter::FShapeFilterData GetShapeFilter() const = 0;
		UE_INTERNAL virtual void SetShapeFilter(const Chaos::Filter::FShapeFilterData& InShapeFilter) = 0;

		UE_INTERNAL virtual Chaos::Filter::FInstanceData GetFilterInstanceData() const = 0;
		UE_INTERNAL virtual void SetFilterInstanceData(const Chaos::Filter::FInstanceData& InInstanceData) = 0;
		
		UE_INTERNAL virtual int32 GetNumMaterials() const = 0;
		UE_INTERNAL virtual FMaterialHandle GetMaterial(const int32 InIndex) const = 0;
		UE_INTERNAL virtual void SetMaterials(TArray<FMaterialHandle>&& InMaterials) = 0;

		UE_INTERNAL virtual int32 GetNumMaterialMasks() const = 0;
		UE_INTERNAL virtual FMaterialMaskHandle GetMaterialMask(const int32 InIndex) const = 0;
		UE_INTERNAL virtual void SetMaterialMasks(TArray<FMaterialMaskHandle>&& InMaterialMasks) = 0;

		UE_INTERNAL virtual int32 GetNumMaterialMaskMaps() const = 0;
		UE_INTERNAL virtual uint32 GetMaterialMaskMap(const int32 InIndex) const = 0;
		UE_INTERNAL virtual void SetMaterialMaskMaps(TArray<uint32>&& InMaterialMaskMaps) = 0;

		UE_INTERNAL virtual int32 GetNumMaterialMaskMapMaterials() const = 0;
		UE_INTERNAL virtual FMaterialHandle GetMaterialMaskMapMaterial(const int32 InIndex) const = 0;
		UE_INTERNAL virtual void SetMaterialMaskMapMaterials(TArray<FMaterialHandle>&& InMaterialMaskMapMaterials) = 0;

		UE_INTERNAL virtual void* GetUserData() const = 0;
		UE_INTERNAL virtual void SetUserData(void* InUserData) = 0;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
