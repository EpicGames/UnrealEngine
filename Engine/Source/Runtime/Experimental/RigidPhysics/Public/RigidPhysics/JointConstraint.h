// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/Internal/IJointConstraint.h"
#include "RigidPhysics/JointConstraintHandle.h"
#include "RigidPhysics/RigidLog.h"
#include "RigidPhysics/RigidObjectPtr.h"
#include "RigidPhysics/RigidTyped.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	template <typename ContextType>
	class TJointConstraintPtrImpl
	{
	public:
		using FContext = ContextType;
		using FInterface = IJointConstraint;
		using FHandle = FJointConstraintHandle;

		UE_INTERNAL TJointConstraintPtrImpl() = default;
		UE_INTERNAL TJointConstraintPtrImpl(IJointConstraint* InConstraint)
			: ConstraintRaw(InConstraint)
		{
		}

		TJointConstraintPtrImpl(const TJointConstraintPtrImpl&) = delete;

		TJointConstraintPtrImpl(TJointConstraintPtrImpl&& InPtr)
			: ConstraintRaw(InPtr.ConstraintRaw)
		{
			InPtr.ConstraintRaw = nullptr;
		}

		~TJointConstraintPtrImpl()
		{
			Reset();
		}

		friend bool operator==(const TJointConstraintPtrImpl&, const TJointConstraintPtrImpl&) = default;
		friend bool operator!=(const TJointConstraintPtrImpl&, const TJointConstraintPtrImpl&) = default;

		bool IsValid() const
		{
			return (ConstraintRaw != nullptr);
		}

		FJointConstraintHandle GetHandle() const
		{
			if (ConstraintRaw != nullptr)
			{
				return ConstraintRaw->GetHandle();
			}
			return FJointConstraintHandle();
		}

		UE_INTERNAL void Reset()
		{
			ConstraintRaw = nullptr;
		}

		UE_INTERNAL const FRigidTypeId& GetTypeId() const
		{
			return ConstraintRaw->GetTypeId();
		}

		UE_INTERNAL IJointConstraint* Get() const
		{
			return ConstraintRaw;
		}

	private:
		IJointConstraint* ConstraintRaw = nullptr;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
