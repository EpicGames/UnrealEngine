// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/Internal/IJointConstraint6DOF.h"
#include "RigidPhysics/JointConstraint.h"
#include "RigidPhysics/JointConstraint6DOFHandle.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	template <typename ContextType>
	class TJointConstraint6DOFPtrImpl
	{
	public:
		using FContext = ContextType;
		using FInterface = IJointConstraint6DOF;
		using FHandle = FJointConstraint6DOFHandle;

		UE_INTERNAL TJointConstraint6DOFPtrImpl() = default;
		UE_INTERNAL TJointConstraint6DOFPtrImpl(IJointConstraint6DOF* InConstraint)
			: ConstraintRaw(InConstraint)
		{
		}

		// Construct from base class (does a safe downcast)
		UE_INTERNAL TJointConstraint6DOFPtrImpl(IJointConstraint* InConstraint)
			: ConstraintRaw(InConstraint ? InConstraint->AsA<IJointConstraint6DOF>() : nullptr)
		{
		}

		TJointConstraint6DOFPtrImpl(const TJointConstraint6DOFPtrImpl&) = delete;

		TJointConstraint6DOFPtrImpl(TJointConstraint6DOFPtrImpl&& InPtr)
			: ConstraintRaw(InPtr.ConstraintRaw)
		{
			InPtr.ConstraintRaw = nullptr;
		}

		~TJointConstraint6DOFPtrImpl()
		{
			Reset();
		}

		friend bool operator==(const TJointConstraint6DOFPtrImpl&, const TJointConstraint6DOFPtrImpl&) = default;
		friend bool operator!=(const TJointConstraint6DOFPtrImpl&, const TJointConstraint6DOFPtrImpl&) = default;

		bool IsValid() const
		{
			return (ConstraintRaw != nullptr);
		}

		FJointConstraint6DOFHandle GetHandle() const
		{
			if (ConstraintRaw != nullptr)
			{
				return ConstraintRaw->GetTypedHandle();
			}
			return FJointConstraint6DOFHandle();
		}

		void Activate()
		{
			ConstraintRaw->Activate();
		}

		void SetBodies(const TRigidBodyPtr<ContextType>& Body0, const TRigidBodyPtr<ContextType>& Body1) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetBodies(Body0.Get(), Body1.Get());
		}

		void SetJointTransforms(const FTransform3d& InTransform0, const FTransform3d& InTransform1) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetJointTransforms(InTransform0, InTransform1);
		}

		double GetLinearLimit() const
		{
			return ConstraintRaw->GetLinearLimit();
		}

		void SetLinearLimit(const double InLinearLimit)
		{
			ConstraintRaw->SetLinearLimit(InLinearLimit);
		}

		EJointMotionType GetLinearMotionTypesX()
		{
			return ConstraintRaw->GetLinearMotionTypesX();
		}

		void SetLinearMotionTypesX(const EJointMotionType InMotionType)
		{
			ConstraintRaw->SetLinearMotionTypesX(InMotionType);
		}

		EJointMotionType GetLinearMotionTypesY()
		{
			return ConstraintRaw->GetLinearMotionTypesY();
		}

		void SetLinearMotionTypesY(const EJointMotionType InMotionType)
		{
			ConstraintRaw->SetLinearMotionTypesY(InMotionType);
		}

		EJointMotionType GetLinearMotionTypesZ()
		{
			return ConstraintRaw->GetLinearMotionTypesZ();
		}

		void SetLinearMotionTypesZ(const EJointMotionType InMotionType)
		{
			ConstraintRaw->SetLinearMotionTypesZ(InMotionType);
		}

		void SetAngularMotionTypesX(const EJointMotionType InMotionType) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetAngularMotionTypesX(InMotionType);
		}

		void SetAngularMotionTypesY(const EJointMotionType InMotionType) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetAngularMotionTypesY(InMotionType);
		}

		void SetAngularMotionTypesZ(const EJointMotionType InMotionType) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetAngularMotionTypesZ(InMotionType);
		}

		void SetAngularLimits(const FVector3d& InAngularLimits) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetAngularLimits(InAngularLimits);
		}

		void SetCollisionEnabled(const bool bCollisionEnabled) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetCollisionEnabled(bCollisionEnabled);
		}

		void SetProjectionEnabled(const bool bProjectionEnabled) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetProjectionEnabled(bProjectionEnabled);
		}

		void SetMassConditioningEnabled(const bool bMassConditioningEnabled) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetMassConditioningEnabled(bMassConditioningEnabled);
		}

		void SetUseLinearSolver(const bool bUseLinearSolver) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetUseLinearSolver(bUseLinearSolver);
		}

		void SetSoftLinearLimitsEnabled(const bool bSoftLinearLimitsEnabled) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetSoftLinearLimitsEnabled(bSoftLinearLimitsEnabled);
		}

		void SetSoftTwistLimitsEnabled(const bool bSoftTwistLimitsEnabled) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetSoftTwistLimitsEnabled(bSoftTwistLimitsEnabled);
		}

		void SetSoftSwingLimitsEnabled(const bool bSoftSwingLimitsEnabled) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetSoftSwingLimitsEnabled(bSoftSwingLimitsEnabled);
		}

		void SetSoftLinearStiffness(const double InSoftLinearStiffness) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetSoftLinearStiffness(InSoftLinearStiffness);
		}

		void SetSoftLinearDamping(const double InSoftLinearDamping) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetSoftLinearDamping(InSoftLinearDamping);
		}

		void SetSoftTwistStiffness(const double InSoftTwistStiffness) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetSoftTwistStiffness(InSoftTwistStiffness);
		}

		void SetSoftTwistDamping(const double InSoftTwistDamping) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetSoftTwistDamping(InSoftTwistDamping);
		}	
		
		void SetSoftSwingStiffness(const double InSoftSwingStiffness) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetSoftSwingStiffness(InSoftSwingStiffness);
		}

		void SetSoftSwingDamping(const double InSoftSwingDamping) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetSoftSwingDamping(InSoftSwingDamping);
		}

		void SetProjectionLinearAlpha(const double InProjectionLinearAlpha) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetProjectionLinearAlpha(InProjectionLinearAlpha);
		}

		void SetProjectionAngularAlpha(const double InProjectionAngularAlpha) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetProjectionAngularAlpha(InProjectionAngularAlpha);
		}

		void SetProjectionLinearTolerance(const double InProjectionLinearTolerance) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetProjectionLinearTolerance(InProjectionLinearTolerance);
		}

		void SetProjectionAngularTolerance(const double InProjectionAngularTolerance) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetProjectionAngularTolerance(InProjectionAngularTolerance);
		}

		void SetLinearContactDistance(const double InLinearContactDistance) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetLinearContactDistance(InLinearContactDistance);
		}

		void SetTwistContactDistance(const double InTwistContactDistance) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetTwistContactDistance(InTwistContactDistance);
		}

		void SetSwingContactDistance(const double InSwingContactDistance) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetSwingContactDistance(InSwingContactDistance);
		}

		void SetAngularDriveStiffness(const FVector3d& InAngularDriveStiffness) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetAngularDriveStiffness(InAngularDriveStiffness);
		}

		void SetAngularDriveDamping(const FVector3d& InAngularDriveDamping) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetAngularDriveDamping(InAngularDriveDamping);
		}

		void SetLinearDriveStiffness(const FVector3d& InLinearDriveStiffness) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetLinearDriveStiffness(InLinearDriveStiffness);
		}

		void SetLinearDriveDamping(const FVector3d& InLinearDriveDamping) requires CIsGameContext<FContext>
		{
			ConstraintRaw->SetLinearDriveDamping(InLinearDriveDamping);
		}

		UE_INTERNAL void Reset()
		{
			ConstraintRaw = nullptr;
		}

		UE_INTERNAL const FRigidTypeId& GetTypeId() const
		{
			return ConstraintRaw->GetTypeId();
		}

		UE_INTERNAL IJointConstraint6DOF* Get() const
		{
			return ConstraintRaw;
		}
	private:
		IJointConstraint6DOF* ConstraintRaw = nullptr;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
