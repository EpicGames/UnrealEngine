// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/JointConstraintHandle.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class IJointConstraint6DOF;
	template <typename ContextType>
	class UE_INTERNAL TJointConstraint6DOFPtrImpl;

	class FJointConstraint6DOFHandle : public FJointConstraintHandle
	{
	public:
		FJointConstraint6DOFHandle() = default;
		FJointConstraint6DOFHandle(const FJointConstraint6DOFHandle&) = default;
		UE_INTERNAL RIGIDPHYSICS_API FJointConstraint6DOFHandle(const FRigidSceneId& InSceneId, const FRigidObjectId& InConstraintId);

		friend bool operator==(const FJointConstraint6DOFHandle&, const FJointConstraint6DOFHandle&) = default;
		friend bool operator!=(const FJointConstraint6DOFHandle&, const FJointConstraint6DOFHandle&) = default;

		template <typename ContextType>
		TJointConstraint6DOFPtr<ContextType> Pin(const ContextType& InContext) const
		{
			return FJointConstraintHandle::Pin(InContext);
		}

	private:
		friend class IJointConstraint6DOF;

		// Private so we only allow downcast in the context where we explicitly know this constraint handle is the correct type.
		UE_INTERNAL RIGIDPHYSICS_API FJointConstraint6DOFHandle(const FJointConstraintHandle& InHandle);
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
