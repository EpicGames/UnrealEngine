// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidObjectId.h"
#include "RigidPhysics/RigidObjectPtr.h"
#include "RigidPhysics/RigidSceneHandle.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class IJointConstraint;
	template <typename ContextType>
	class UE_INTERNAL TJointConstraintPtrImpl;

	class FJointConstraintHandle
	{
	public:
		FJointConstraintHandle() = default;
		FJointConstraintHandle(const FJointConstraintHandle&) = default;
		UE_INTERNAL RIGIDPHYSICS_API FJointConstraintHandle(const FRigidSceneId& InSceneId, const FRigidObjectId& InConstraintId);

		friend bool operator==(const FJointConstraintHandle&, const FJointConstraintHandle&) = default;
		friend bool operator!=(const FJointConstraintHandle&, const FJointConstraintHandle&) = default;

		RIGIDPHYSICS_API FRigidSceneHandle GetSceneHandle() const;

		template <typename ContextType>
		TJointConstraintPtr<ContextType> Pin(const ContextType& InContext) const
		{
			if (const IRigidScene* Scene = InContext.GetRawSceneChecked(SceneId))
			{
				return TJointConstraintPtr<ContextType>(PinInternal(Scene));
			}
			return TJointConstraintPtr<ContextType>();
		}

		UE_INTERNAL RIGIDPHYSICS_API const FRigidObjectId& GetId() const;
		UE_INTERNAL RIGIDPHYSICS_API const FRigidSceneId& GetSceneId() const;
		UE_INTERNAL RIGIDPHYSICS_API void Reset();

	private:
		RIGIDPHYSICS_API IJointConstraint* PinInternal(const IRigidScene* Scene) const;

		FRigidSceneId SceneId;
		FRigidObjectId ConstraintId;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
