// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Math/Bounds.h"
#include "Math/Transform.h"
#include "RigidPhysics/Internal/IRigidBody.h"
#include "RigidPhysics/RigidBodyHandle.h"
#include "RigidPhysics/RigidObjectId.h"
#include "RigidPhysics/RigidObjectPtr.h"
#include "RigidPhysics/RigidShapeInstance.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	// This class only exists so that TRigidBodyPtr can have pointer semantics rather than object semantics,
	// and so that we can disable functions based on ContextType
	// See TRigidBodyPtr for comments.
	template <typename ContextType>
	class UE_INTERNAL TRigidBodyPtrImpl
	{
	public:
		using FContext = ContextType;
		using FInterface = IRigidBody;
		using FHandle = FRigidBodyHandle;

		UE_INTERNAL TRigidBodyPtrImpl() = default;
		UE_INTERNAL TRigidBodyPtrImpl(IRigidBody* InBody)
			: BodyRaw(InBody)
		{
		}

		TRigidBodyPtrImpl(const TRigidBodyPtrImpl& InBodyPtr) = delete;

		TRigidBodyPtrImpl(TRigidBodyPtrImpl&& InBodyPtr)
			: BodyRaw(InBodyPtr.BodyRaw)
		{
			InBodyPtr.BodyRaw = nullptr;
		}

		~TRigidBodyPtrImpl()
		{
			Reset();
		}

		friend bool operator==(const TRigidBodyPtrImpl&, const TRigidBodyPtrImpl&) = default;
		friend bool operator!=(const TRigidBodyPtrImpl&, const TRigidBodyPtrImpl&) = default;

		bool IsValid() const
		{
			return (BodyRaw != nullptr);
		}

		FRigidBodyHandle GetHandle() const
		{
			if (BodyRaw != nullptr)
			{
				return BodyRaw->GetHandle();
			}
			return FRigidBodyHandle();
		}

		bool IsActive() const
		{
			return BodyRaw->IsActive();
		}

		void Activate() requires CIsGameContext<FContext>
		{
			BodyRaw->Activate();
		}

		void Deactivate() requires CIsGameContext<FContext>
		{
			BodyRaw->Deactivate();
		}

		bool IsStatic() const
		{
			return BodyRaw->IsStatic();
		}

		bool IsKinematic() const
		{
			return BodyRaw->IsKinematic();
		}

		bool IsDynamic() const
		{
			return BodyRaw->IsDynamic();
		}

		int32 GetNumShapes() const
		{
			return BodyRaw->GetNumShapes();
		}

		TRigidShapeInstancePtr<FContext> GetShape(int32 InShapeIndex) const
		{
			return BodyRaw->GetShape(InShapeIndex);
		}

		TRigidShapeInstancePtr<FContext> CreateShape(const FRigidShapeInstanceSetup& InSetup) requires CIsGameContext<FContext>
		{
			return BodyRaw->CreateShape(InSetup);
		}

		UE_RIGIDPHYSICS_LEGACY_API TRigidShapeInstancePtr<FContext> CreateShape(TUniquePtr<Chaos::FPerShapeData>&& InShapeData) requires CIsGameContext<FContext>
		{
			return BodyRaw->CreateShape(MoveTemp(InShapeData));
		}

		void DestroyShape(const TRigidShapeInstancePtr<FContext>& InShape, bool bWakeTouching = true) requires CIsGameContext<FContext>
		{
			BodyRaw->DestroyShape(InShape.Get(), bWakeTouching);
		}

		void InitTransform(const FTransform3d& InTransform)
		{
			BodyRaw->InitTransform(InTransform);
		}

		void UpdateTransform(const FTransform3d& InTransform)
		{
			BodyRaw->UpdateTransform(InTransform);
		}

		void SetKinematicTarget(const FTransform3d& InTransform)
		{
			BodyRaw->SetKinematicTarget(InTransform);
		}

		FTransform3d GetTransform() const
		{
			return BodyRaw->GetTransform();
		}

		FBounds3d GetBounds() const
		{
			return BodyRaw->GetBounds();
		}

		double GetMass() const
		{
			return BodyRaw->GetMass();
		}

		void SetMass(double InMass)
		{
			BodyRaw->SetMass(InMass);
		}

		FVector3d GetInertia() const
		{
			return BodyRaw->GetInertia();
		}

		void SetInertia(const FVector3d& InInertia)
		{
			BodyRaw->SetInertia(InInertia);
		}

		void SetCenterOfMass(const FVector3d& InCenterOfMass) requires CIsGameContext<FContext>
		{
			BodyRaw->SetCenterOfMass(InCenterOfMass);
		}

		void SetRotationOfMass(const FQuat& InRotationOfMass) requires CIsGameContext<FContext>
		{
			BodyRaw->SetRotationOfMass(InRotationOfMass);
		}

		void SetCCDEnabled(bool bInEnabled)
		{
			BodyRaw->SetCCDEnabled(bInEnabled);
		}

		void SetMACDEnabled(bool bInEnabled)
		{
			BodyRaw->SetMACDEnabled(bInEnabled);
		}

		void SetSmoothEdgeCollisionsEnabled(bool bInEnabled)
		{
			BodyRaw->SetSmoothEdgeCollisionsEnabled(bInEnabled);
		}

		void SetMaxLinearVelocity(float InMax)
		{
			BodyRaw->SetMaxLinearVelocity(InMax);
		}

		void SetMaxAngularVelocity(float InMax)
		{
			BodyRaw->SetMaxAngularVelocity(InMax);
		}

		void SetInertiaConditioningEnabled(bool bInEnabled)
		{
			BodyRaw->SetInertiaConditioningEnabled(bInEnabled);
		}

		void SetGyroscopicTorqueEnabled(bool bInEnabled)
		{
			BodyRaw->SetGyroscopicTorqueEnabled(bInEnabled);
		}

		void SetGravityGroup(int32 InId)
		{
			BodyRaw->SetGravityGroup(InId);
		}

		void SetGravityScale(float InScale)
		{
			BodyRaw->SetGravityScale(InScale);
		}

		void SetWakeEventsEnabled(bool bInEnabled)
		{
			BodyRaw->SetWakeEventsEnabled(bInEnabled);
		}

		void SetNumPositionIterations(int32 InNum)
		{
			BodyRaw->SetNumPositionIterations(InNum);
		}

		void SetNumVelocityIterations(int32 InNum)
		{
			BodyRaw->SetNumVelocityIterations(InNum);
		}

		void SetNumProjectionIterations(int32 InNum)
		{
			BodyRaw->SetNumProjectionIterations(InNum);
		}
		
		void SetIsSleeping(bool bInSleeping)
		{
			BodyRaw->SetIsSleeping(bInSleeping);
		}

		UE_INTERNAL void Reset()
		{
			BodyRaw = nullptr;
		}

		UE_INTERNAL IRigidBody* Get() const
		{
			return BodyRaw;
		}

		UE_INTERNAL const FRigidTypeId& GetTypeId() const
		{
			return BodyRaw->GetTypeId();
		}

	private:
		// IRigidBody implementation depends on Context type
		IRigidBody* BodyRaw = nullptr;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
