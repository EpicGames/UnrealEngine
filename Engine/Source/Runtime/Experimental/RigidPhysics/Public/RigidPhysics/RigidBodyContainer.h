// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/Internal/IRigidBodyContainer.h"
#include "RigidPhysics/RigidBodyContainerHandle.h"
#include "RigidPhysics/RigidLog.h"
#include "RigidPhysics/RigidObjectId.h"
#include "RigidPhysics/RigidObjectPtr.h"
#include "RigidPhysics/RigidObjectRegistry.h"
#include "RigidPhysics/RigidTyped.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	// Utility wrapper around IRigidBodyContainer that adds Context and allows
	// TRigidBodyContainerPtr to have pointer semantics without exposing IRigidBodyContainer
	template <typename ContextType>
	class UE_INTERNAL TRigidBodyContainerPtrImpl
	{
	public:
		using FContext = ContextType;
		using FInterface = IRigidBodyContainer;
		using FHandle = FRigidBodyContainerHandle;
		using FRigidScenePtr = TRigidScenePtr<FContext>;

		UE_INTERNAL TRigidBodyContainerPtrImpl(IRigidBodyContainer* InBodyContainer)
			: BodyContainerRaw(InBodyContainer)
		{
		}

		~TRigidBodyContainerPtrImpl() = default;


		bool IsValid() const
		{
			return (BodyContainerRaw != nullptr);
		}

		FRigidBodyContainerHandle GetHandle() const
		{
			return BodyContainerRaw->GetHandle();
		}

		int32 GetNumBodies() const
		{
			return BodyContainerRaw->GetNumBodies();
		}

		TRigidBodyPtr<FContext> GetBody(const FRigidObjectId& InBodyId) const
		{
			return BodyContainerRaw->GetBody(InBodyId);
		}

		TRigidBodyPtr<FContext> CreateBody(const FRigidDebugName& InName, ERigidMovementType InMovementType) const
		{
			return BodyContainerRaw->CreateBody(InName, InMovementType);
		}

		void DestroyBody(TRigidBodyPtr<FContext>&& InBodyPtr)
		{
			BodyContainerRaw->DestroyBody(InBodyPtr.Get());
		}

		UE_INTERNAL void Reset()
		{
			BodyContainerRaw = nullptr;
		}

		UE_INTERNAL const FRigidTypeId& GetTypeId() const
		{
			return BodyContainerRaw->GetTypeId();
		}

		UE_INTERNAL IRigidBodyContainer* Get() const
		{
			return BodyContainerRaw;
		}

	private:
		// Pointer to implementation, object depends on Context type
		IRigidBodyContainer* BodyContainerRaw = nullptr;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
