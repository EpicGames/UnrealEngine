// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidObjectId.h"
#include "RigidPhysics/RigidObjectPtr.h"
#include "RigidPhysics/RigidSceneHandle.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	template <typename ContextType>
	class UE_INTERNAL TRigidBodyContainerPtrImpl;

	class FRigidBodyContainerHandle
	{
	public:
		FRigidBodyContainerHandle() = default;
		UE_INTERNAL RIGIDPHYSICS_API FRigidBodyContainerHandle(const FRigidSceneId& InSceneId, const FRigidObjectId& InBodyContainerId);

		friend bool operator==(const FRigidBodyContainerHandle&, const FRigidBodyContainerHandle&) = default;
		friend bool operator!=(const FRigidBodyContainerHandle&, const FRigidBodyContainerHandle&) = default;

		// Convert the handle to a pointer in the specified context.
		template <typename ContextType>
		TRigidBodyContainerPtr<ContextType> Pin(const ContextType& InContext) const
		{
			if (const IRigidScene* Scene = InContext.GetRawSceneChecked(SceneId))
			{
				return PinInternal(Scene);
			}
			return nullptr;
		}

		UE_INTERNAL RIGIDPHYSICS_API void Reset();
		UE_INTERNAL RIGIDPHYSICS_API const FRigidObjectId& GetId() const;
		UE_INTERNAL RIGIDPHYSICS_API const FRigidSceneId& GetSceneId() const;

	private:
		RIGIDPHYSICS_API IRigidBodyContainer* PinInternal(const IRigidScene* InScene) const;

		FRigidObjectId BodyContainerId;
		FRigidSceneId SceneId;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
