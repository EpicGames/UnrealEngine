// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidObjectId.h"
#include "RigidPhysics/RigidObjectPtr.h"
#include "RigidPhysics/RigidSceneHandle.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class IRigidBody;
	template <typename ContextType>
	class UE_INTERNAL TRigidBodyPtrImpl;

	// A handle to a rigid body that may be copied and stored.
	// A body reference can be retrieved by pinning the handle with a context.
	class FRigidBodyHandle
	{
	public:
		FRigidBodyHandle() = default;
		FRigidBodyHandle(const FRigidBodyHandle&) = default;

		UE_INTERNAL RIGIDPHYSICS_API FRigidBodyHandle(const FRigidSceneId& InSceneId, const FRigidObjectId& InBodyId);

		friend bool operator==(const FRigidBodyHandle&, const FRigidBodyHandle&) = default;
		friend bool operator!=(const FRigidBodyHandle&, const FRigidBodyHandle&) = default;

		// Convert the handle to a pointer in the specified context.
		template <typename ContextType>
		TRigidBodyPtr<ContextType> Pin(const ContextType& InContext) const
		{
			if (const IRigidScene* Scene = InContext.GetRawSceneChecked(SceneId))
			{
				return PinInternal(Scene);
			}
			return TRigidBodyPtr<ContextType>();
		}

		UE_INTERNAL RIGIDPHYSICS_API const FRigidObjectId& GetId() const;
		UE_INTERNAL RIGIDPHYSICS_API const FRigidSceneId& GetSceneId() const;
		UE_INTERNAL RIGIDPHYSICS_API void Reset();

		RIGIDPHYSICS_API friend uint32 GetTypeHash(const FRigidBodyHandle& InHandle);

	private:
		RIGIDPHYSICS_API IRigidBody* PinInternal(const IRigidScene* Scene) const;

		FRigidObjectId BodyId;
		FRigidSceneId SceneId;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
