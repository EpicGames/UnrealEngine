// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidLockable.h"
#include "RigidPhysics/RigidLog.h"
#include "RigidPhysics/RigidScene.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	// A Context holds a lock (RW or RO) and converts object handles
	// to context-specific object pointers.
	template <ERigidContextType ContextTypeParam, ERigidLockType LockTypeParam>
	class TRigidContext
	{
	public:
		static constexpr ERigidContextType ContextType = ContextTypeParam;
		static constexpr ERigidLockType LockType = LockTypeParam;
		static constexpr bool bWriteEnabled = (LockType == ERigidLockType::ReadWrite);

		using FContext = TRigidContext<ContextType, LockType>;

		UE_INTERNAL TRigidContext(IRigidScene* InScene)
		{
			if (InScene != nullptr)
			{
				SceneId = InScene->GetHandle().GetId();
				InScene->Lock(LockType);
				ScenePtr.Init(InScene);
			}
		}

		~TRigidContext()
		{
			if (ScenePtr.IsValid())
			{
				ScenePtr.Get()->Unlock(LockType);
			}
		}

		bool IsValid() const
		{
			return ScenePtr.IsValid();
		}

		operator bool() const
		{
			return IsValid();
		}

		// Expose Scene API on Context
		const TRigidScenePtrImpl<FContext>* operator->() const requires (!FContext::bWriteEnabled)
		{
			return ScenePtr.operator->();
		}

		TRigidScenePtrImpl<FContext>* operator->() const requires (FContext::bWriteEnabled)
		{
			return ScenePtr.operator->();
		}

		UE_INTERNAL const TRigidScenePtr<FContext>& GetScene() const
		{
			return ScenePtr;
		}

		UE_INTERNAL const IRigidScene* GetRawSceneChecked(const FRigidSceneId InSceneId) const
		{
			return (IsValid() && InSceneId == SceneId) ? ScenePtr->Get() : nullptr;
		}

	private:
		TRigidScenePtr<FContext> ScenePtr;
		FRigidSceneId SceneId;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
