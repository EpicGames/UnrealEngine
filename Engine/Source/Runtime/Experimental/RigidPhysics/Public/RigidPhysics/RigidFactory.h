// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidTyped.h"

namespace UE::Physics
{
	class IRigidFactory
	{
	public:
		IRigidFactory() = default;
		virtual ~IRigidFactory() = default;

		virtual const FRigidTypeId& GetSceneTypeId() const = 0;
		virtual FRigidSceneHandle CreateScene(const FRigidDebugName& InName, const IRigidSceneSettings* InSettings) = 0;
		virtual void DestroyScene(const FRigidSceneHandle& InSceneHandle) = 0;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
