// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// A define to allow us to toggle between old and new APIs in BodyInstance etc
// Also can be searched to identify all engine changes required to change to the new API
// For now it is set only in Test Target.cs files like this:
//   GlobalDefinitions.Add("UE_RIGIDPHYSICS_API_ENABLED=1");
#ifndef UE_RIGIDPHYSICS_API_ENABLED
// NOTE: Must be 0 (for now)
#define UE_RIGIDPHYSICS_API_ENABLED 0
#endif

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Chaos/ChaosDebugName.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"

#define UE_RIGIDPHYSICS_LEGACY_API UE_INTERNAL /*UE_DEPRECATED(5.8, "TODO")*/

#if DO_CHECK

// Enable checks in the Rigid accessors. This can add significant overhead.
#define UE_RIGIDPHYSICS_CHECK_ENABLED 1

// Required to show the Objects associated with a Handle in the debugger
#define UE_RIGIDPHYSICS_CONTAINERSCENEHANDLE_ENABLED 1

// Enable to assert when Handles are Pinned after their scene is destroyed.
// This adds a Weak Pointer to SceneHandle which must be checked every time we pin so is
// not recommended if performance is a concern
#define UE_RIGIDPHYSICS_SCENEHANDLE_SAFTEY_ENABLED 0

#else
#define UE_RIGIDPHYSICS_CHECK_ENABLED 0
#define UE_RIGIDPHYSICS_CONTAINERSCENEHANDLE_ENABLED 0
#define UE_RIGIDPHYSICS_SCENEHANDLE_SAFTEY_ENABLED 0
#endif

#if UE_RIGIDPHYSICS_CHECK_ENABLED
#define UE_RIGIDPHYSICS_CHECK(expr) check(expr)
#define UE_RIGIDPHYSICS_CHECKF(expr, format, ...) checkf(expr, format, ##__VA_ARGS__)
#else
#define UE_RIGIDPHYSICS_CHECK(X)
#define UE_RIGIDPHYSICS_CHECKF(expr, format, ...)
#endif


// Math/Bounds.h
template<typename T> struct TBounds;
using FBounds3f = TBounds<float>;
using FBounds3d = TBounds<double>;

class FGeometryCollection;
struct FSimulationParameters;	// GC setup

// User-facing classes
namespace UE::Physics
{
	class FJointConstraintHandle;
	class FJointConstraint6DOFHandle;
	class FRigidBodyContainerHandle;
	class FRigidBodyHandle;
	class FRigidGeometryCollectionHandle;
	class FRigidObjectId;
	class FRigidQueryDesc;
	class FRigidQueryHit;
	class FRigidSceneHandle;
	class FRigidSceneId;
	class FRigidShapeInstance;
	class FRigidShapeInstanceHandle;
	class FRigidShapeInstanceSetup;

	class IJointConstraint;
	class IJointConstraint6DOF;
	class IRigidBody;
	class IRigidBodyContainer;
	class IRigidFactory;
	class IRigidGeometryCollection;
	class IRigidLockable;
	class IRigidQuery;
	class IRigidScene;
	class IRigidSceneModifier;
	class IRigidSceneSettings;
	class IRigidShapeInstance;
	class IRigidTyped;

	template<typename ContextType> class TJointConstraintPtrImpl;
	template<typename ContextType> class TJointConstraint6DOFPtrImpl;
	template<typename ContextType> class TRigidBodyContainerPtrImpl;
	template<typename ContextType> class TRigidBodyPtrImpl;
	template<typename ContextType> class TRigidGeometryCollectionPtrImpl;
	template<typename PtrImplType> class TRigidObjectPtr;
	template<typename ContextType> class TRigidScenePtrImpl;
	template<typename ContextType> class TRigidShapeInstancePtrImpl;

	template<typename ContextType> using TJointConstraintPtr = TRigidObjectPtr<TJointConstraintPtrImpl<ContextType>>;
	template<typename ContextType> using TJointConstraint6DOFPtr = TRigidObjectPtr<TJointConstraint6DOFPtrImpl<ContextType>>;
	template<typename ContextType> using TRigidBodyContainerPtr = TRigidObjectPtr<TRigidBodyContainerPtrImpl<ContextType>>;
	template<typename ContextType> using TRigidBodyPtr = TRigidObjectPtr<TRigidBodyPtrImpl<ContextType>>;
	template<typename ContextType> using TRigidGeometryCollectionPtr = TRigidObjectPtr<TRigidGeometryCollectionPtrImpl<ContextType>>;
	template<typename ContextType> using TRigidScenePtr = TRigidObjectPtr<TRigidScenePtrImpl<ContextType>>;
	template<typename ContextType> using TRigidShapeInstancePtr = TRigidObjectPtr<TRigidShapeInstancePtrImpl<ContextType>>;

	using FRigidDebugName = ::Chaos::FSharedDebugName;
	using FRigidSceneWeakPtr = TWeakPtr<IRigidScene, ESPMode::ThreadSafe>;
} // namespace UE::Physics

namespace UE::Physics
{
	enum class EJointMotionType : int32
	{
		Free,
		Limited,
		Locked,
	};

	enum class ERigidVisitorResponse
	{
		Continue,
		Break,
	};

	enum class ERigidMovementType
	{
		Static,
		Kinematic,
		Dynamic
	};

	// Physics execution is split into Game and Simulation contexts. In async
	// modes, these are on different threads.
	enum class ERigidContextType
	{
		Game,
		Sim,
	};

	// The context must be locked before reading or writing to any objects in physics.
	// The lock may be read-only or read-write.
	enum class ERigidLockType
	{
		None,
		ReadOnly,
		ReadWrite,
	};

	template<ERigidContextType ContextTypeParam, ERigidLockType LockTypeParam> class TRigidContext;
	using FRigidContextGameRO = TRigidContext<ERigidContextType::Game, ERigidLockType::ReadOnly>;
	using FRigidContextGameRW = TRigidContext<ERigidContextType::Game, ERigidLockType::ReadWrite>;
	using FRigidContextSimRW = TRigidContext<ERigidContextType::Sim, ERigidLockType::ReadWrite>;

	template <typename ContextType>
	concept CIsGameContext = ContextType::ContextType == ERigidContextType::Game;
} // namespace UE::Physics

//
// Internal Forward Decls
//

namespace UE::Physics
{
	template<typename ObjectType> class UE_INTERNAL TRigidObjectRegistry;
	using UE_INTERNAL FJointConstraintRegistry = TRigidObjectRegistry<IJointConstraint>;
	using UE_INTERNAL FRigidBodyContainerRegistry = TRigidObjectRegistry<IRigidBodyContainer>;
	using UE_INTERNAL FRigidGeometryCollectionRegistry = TRigidObjectRegistry<IRigidGeometryCollection>;
	using UE_INTERNAL FRigidBodyRegistry = TRigidObjectRegistry<IRigidBody>;
} // namespace UE::Physics

extern uint32 GetTypeHash(UE::Physics::FRigidObjectId& RigidBodyId);

#endif // UE_RIGIDPHYSICS_API_ENABLED
