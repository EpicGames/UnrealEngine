// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/Internal/IRigidGeometryCollection.h"
#include "RigidPhysics/RigidBodyContainer.h"
#include "RigidPhysics/RigidGeometryCollectionHandle.h"
#include "RigidPhysics/RigidObjectPtr.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	// Utility wrapper around IRigidGeometryCollection that adds Context and allows
	// TRigidGeometryCollectionPtr to have pointer semantics without exposing IRigidGeometryCollection
	template <typename ContextType>
	class UE_INTERNAL TRigidGeometryCollectionPtrImpl
	{
	public:
		using FContext = ContextType;
		using FInterface = IRigidGeometryCollection;
		using FHandle = FRigidGeometryCollectionHandle;
		using FRigidScenePtr = TRigidScenePtr<FContext>;

		TRigidGeometryCollectionPtrImpl() = default;
		UE_INTERNAL TRigidGeometryCollectionPtrImpl(IRigidGeometryCollection* InGeometryCollection)
			: GeometryCollectionRaw(InGeometryCollection)
		{
		}

		UE_INTERNAL TRigidGeometryCollectionPtrImpl(IRigidBodyContainer* InBodyContainer)
			: GeometryCollectionRaw(nullptr)
		{
			if (InBodyContainer != nullptr)
			{
				GeometryCollectionRaw = InBodyContainer->AsA<IRigidGeometryCollection>();
			}
		}

		~TRigidGeometryCollectionPtrImpl() = default;

		friend bool operator==(const TRigidGeometryCollectionPtrImpl&, const TRigidGeometryCollectionPtrImpl&) = default;
		friend bool operator!=(const TRigidGeometryCollectionPtrImpl&, const TRigidGeometryCollectionPtrImpl&) = default;

		bool IsValid() const
		{
			return (GeometryCollectionRaw != nullptr);
		}

		FRigidGeometryCollectionHandle GetHandle() const
		{
			if (GeometryCollectionRaw != nullptr)
			{
				FRigidBodyContainerHandle BodyContainerHandle = GeometryCollectionRaw->GetHandle();
				return FRigidGeometryCollectionHandle(BodyContainerHandle.GetSceneId(), BodyContainerHandle.GetId());
			}
			return FRigidGeometryCollectionHandle();
		}

		int32 GetNumBodies() const
		{
			return GeometryCollectionRaw->GetNumBodies();
		}

		TRigidBodyPtr<FContext> GetBodyAt(int32 InBodyIndex) const
		{
			return GeometryCollectionRaw->GetBodyAt(InBodyIndex);
		}

		TRigidBodyPtr<FContext> GetBody(const FRigidObjectId& InBodyId) const
		{
			return GeometryCollectionRaw->GetBody(InBodyId);
		}

		UE_INTERNAL void Reset()
		{
			GeometryCollectionRaw = nullptr;
		}

		UE_INTERNAL const FRigidTypeId& GetTypeId() const
		{
			return GeometryCollectionRaw->GetTypeId();
		}

		UE_INTERNAL IRigidGeometryCollection* Get() const
		{
			return GeometryCollectionRaw;
		}

		UE_INTERNAL TRigidBodyPtr<FContext> CreateBody(const FRigidDebugName& InName, ERigidMovementType InMovementType)
		{
			return GeometryCollectionRaw->CreateBody(InName, InMovementType);
		}

	private:
		// Pointer to implementation, object depends on Context type
		IRigidGeometryCollection* GeometryCollectionRaw = nullptr;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
