// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidBodyContainerHandle.h"
#include "RigidPhysics/RigidObjectPtr.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class UE_INTERNAL IRigidGeometryCollection;
	template <typename ContextType>
	class UE_INTERNAL TRigidGeometryCollectionPtrImpl;

	// A handle to a GeometryCollection. A GeometryCollection is a type of BodyContainer and the handle
	// is auto-castable to FRigidBodyContainerHandle, but the reverse is more complex.
	//
	// NOTE: If the GeometryCollectionHandle has a valid BodyContainerHandle, it will always reference a GC. 
	// There should be no way to get a GCHandle that contains a non-GC body container handle. This is enforced
	// because the only way to create a FRigidGeometryCollectionHandle is from a TRigidGeometryCollectionPtr.
	// 
	// To safely convert a FRigidBodyContainerHandle to FRigidGeometryCollectionHandle we must go through
	// pointer conversion:
	// 
	//		FRigidBodyContainerHandle BodyContainerHandle = ...;
	//		TRigidGeometryCollectionPtrImpl<FRigidContextGameRW> GCPtr = BodyContainerHandle.Pin(Context);
	//		FRigidGeometryCollectionHandle GCHandle = GCPtr;
	// 
	// This ensures that we get an empty handle if BodyContainerHandle is not a GC, because the cast from
	// TRigidBodyContainerPtr to TRigidGeometryCollectionPtr will fail and return nullptr.
	//
	class FRigidGeometryCollectionHandle
	{
	public:
		FRigidGeometryCollectionHandle() = default;
		UE_INTERNAL RIGIDPHYSICS_API FRigidGeometryCollectionHandle(const FRigidSceneId& InSceneId, const FRigidObjectId& InBodyContainerId);

		friend bool operator==(const FRigidGeometryCollectionHandle&, const FRigidGeometryCollectionHandle&) = default;
		friend bool operator!=(const FRigidGeometryCollectionHandle&, const FRigidGeometryCollectionHandle&) = default;

		// Convert the handle to a pointer in the specified context
		template <typename ContextType>
		TRigidGeometryCollectionPtr<ContextType> Pin(const ContextType& InContext) const
		{
			if (InContext.IsValid())
			{
				IRigidBodyContainer* Container = BodyContainerHandle.Pin(InContext).Get();
				return PinInternal(Container);
			}
			return nullptr;
		}

		UE_INTERNAL RIGIDPHYSICS_API void Reset();
		UE_INTERNAL RIGIDPHYSICS_API const FRigidObjectId& GetId() const;
		UE_INTERNAL RIGIDPHYSICS_API const FRigidSceneId& GetSceneId() const;

		UE_INTERNAL RIGIDPHYSICS_API FRigidBodyContainerHandle AsBodyContainerHandle() const;
		UE_INTERNAL RIGIDPHYSICS_API operator FRigidBodyContainerHandle() const;

	private:
		RIGIDPHYSICS_API IRigidGeometryCollection* PinInternal(IRigidBodyContainer* Container) const;

		FRigidBodyContainerHandle BodyContainerHandle;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
