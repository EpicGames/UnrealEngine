// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"
#include "RigidPhysics/RigidTyped.h"

#if UE_RIGIDPHYSICS_API_ENABLED

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class UE_INTERNAL IRigidLockable : public IRigidTyped
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(RIGIDPHYSICS_API, IRigidLockable);

		UE_INTERNAL IRigidLockable() = default;
		UE_INTERNAL virtual ~IRigidLockable() = default;

		UE_INTERNAL virtual void Lock(ERigidLockType InLockType) const = 0;
		UE_INTERNAL virtual void Unlock(ERigidLockType InLockType) const = 0;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
