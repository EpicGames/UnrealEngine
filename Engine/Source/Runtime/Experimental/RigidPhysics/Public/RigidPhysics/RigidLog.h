// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Logging/StructuredLog.h"

RIGIDPHYSICS_API DECLARE_LOG_CATEGORY_EXTERN(LogRigidPhysics, Log, All);

namespace UE::Physics::Private
{
	template <typename FuncNameType, typename TypeNameType>
	UE_INTERNAL inline void LogUnsupportedAPI(const FuncNameType& FuncName, const TypeNameType& TypeName)
	{
		// This warning means a function is being called in a context where it is not supported.
		// E.g., Calling SetMass from the physics thread.
		UE_LOGFMT(LogRigidPhysics, Warning, "{0} not supported on {1}", FuncName, TypeName);
	}
} // namespace UE::Physics::Private

#define RIGIDPHYSICS_API_UNSUPPORTED() UE::Physics::Private::LogUnsupportedAPI(__FUNCTION__, GetTypeName());

#endif // UE_RIGIDPHYSICS_API_ENABLED
