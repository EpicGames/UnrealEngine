// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class FMaterialIndex
	{
	public:
		FMaterialIndex(const uint16 InIndex);
		operator uint16() const;

	private:
		uint16 Index = INDEX_NONE;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
