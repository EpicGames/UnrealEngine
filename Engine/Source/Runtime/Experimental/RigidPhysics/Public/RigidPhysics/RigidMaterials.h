// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos
{
	struct FMaterialHandle;
	class FChaosPhysicsMaterial;
	struct FMaterialMaskHandle;
	class FChaosPhysicsMaterialMask;
} // namespace Chaos

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class FMaterialHandle
	{
	public:
		FMaterialHandle() = default;
		UE_INTERNAL RIGIDPHYSICS_API FMaterialHandle(const Chaos::FMaterialHandle& InMaterialHandle);

		bool operator==(const FMaterialHandle&) const = default;
		bool operator!=(const FMaterialHandle&) const = default;

		UE_INTERNAL RIGIDPHYSICS_API Chaos::FMaterialHandle GetMaterialHandle() const;
		UE_INTERNAL RIGIDPHYSICS_API Chaos::FChaosPhysicsMaterial* GetMaterial() const;

	private:
		uint32 HandleData = 0;
	};

	class FMaterialMaskHandle
	{
	public:
		FMaterialMaskHandle() = default;
		UE_INTERNAL RIGIDPHYSICS_API FMaterialMaskHandle(const Chaos::FMaterialMaskHandle& InMaterialMaskHandle);

		bool operator==(const FMaterialMaskHandle&) const = default;
		bool operator!=(const FMaterialMaskHandle&) const = default;

		UE_INTERNAL RIGIDPHYSICS_API Chaos::FMaterialMaskHandle GetMaterialMaskHandle() const;
		UE_INTERNAL RIGIDPHYSICS_API Chaos::FChaosPhysicsMaterialMask* GetMaterialMask() const;

	private:
		uint32 HandleData = 0;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
