// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Crc.h"
#include "RigidPhysics/RigidFwd.h"
#include "Templates/SharedPointer.h"

#if UE_RIGIDPHYSICS_API_ENABLED

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class IRigidSceneModifier
	{
	public:
		IRigidSceneModifier() = default;
		virtual ~IRigidSceneModifier() = default;

		// Called from the game context before physics is ticked for this frame. Can be used to
		// update inputs to the objects used in the sim context.
		virtual void PreSimulate(const FRigidContextGameRW& Context)
		{
		}

		// Call from the game context after physics is ticked for this frame, but before the
		// sim results have been sent out to the game framework objects (Actors, Components, etc.)
		virtual void PostSimulate(const FRigidContextGameRW& Context)
		{
		}

		// Called from the simulation context before each simulation tick. At this point you can add forces, move bodies,
		// or do pretty much anything to the scene or objects in it, and the results will impact the next tick.
		virtual void PreTick(const FRigidContextSimRW& Context)
		{
		}

		// Called from the simulation context at the end of each simulation tick. At this point you can modify physics
		// object state and the altered results will be reported back to the game, and be used for the next tick.
		virtual void PostTick(const FRigidContextSimRW& Context)
		{
		}
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
