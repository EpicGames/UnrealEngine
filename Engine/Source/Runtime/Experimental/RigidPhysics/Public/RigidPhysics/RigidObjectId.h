// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	// An ID for an object held in an array that is guaranteed to be unique and 
	// never reused within its container (as long as less than 1<<31 objects are allocated). 
	// There is no guarantee of uniqueness between containers, which means you need a 
	// Container + Id to access an object. (E.g., see FRigidBodyHandle, and FRigidBodyContainerHandle).
	class FRigidObjectId
	{
	public:
		FRigidObjectId()
			: ArrayIndex(INDEX_NONE)
			, Epoch(INDEX_NONE)
		{
		}

		FRigidObjectId(int32 InIndex, int32 InEpoch)
			: ArrayIndex(InIndex)
			, Epoch(InEpoch)
		{
		}

		bool IsValid() const
		{
			return (ArrayIndex >= 0);
		}

		friend bool operator==(const FRigidObjectId& L, const FRigidObjectId& R) = default;

		friend bool operator<(const FRigidObjectId& L, const FRigidObjectId& R)
		{
			return L.ToUInt64() < R.ToUInt64();
		}

		friend uint32 GetTypeHash(const FRigidObjectId& InId)
		{
			return GetTypeHash(InId.ToUInt64());
		}

		UE_INTERNAL int32 GetArrayIndex() const
		{
			return ArrayIndex;
		}

		UE_INTERNAL int32 GetEpoch() const
		{
			return Epoch;
		}

		UE_INTERNAL void Reset()
		{
			ArrayIndex = INDEX_NONE;
			Epoch = INDEX_NONE;
		}

	protected:
		uint64 ToUInt64() const
		{
			return (((uint64)Epoch << 32) & 0xFFFFFFFF00000000LL) | (((uint64)ArrayIndex) & 0x00000000FFFFFFFFLL);
		}

		void FromInt64(uint64 InInt)
		{
			Epoch = (int32)((InInt >> 32) & 0xFFFFFFFF);
			ArrayIndex = (int32)(InInt & 0xFFFFFFFF);
		}

		int32 ArrayIndex = INDEX_NONE;
		int32 Epoch = INDEX_NONE;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
