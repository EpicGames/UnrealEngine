// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Templates/Requires.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{

	// A Context-Specific Smart Pointer to a physics object. 
	//
	// This is a wrapper around a smart pointer implementation object like TRigidBodyPtrImpl, 
	// TRigidBodyContainerPtrImpl, etc. It provides operator-> to give it const and non-const pointer 
	// semantics and behaviour.
	//
	// May be "null". Check IsValid() (or cast to bool) before dereferencing.
	//
	// 	Cannot be copied or stored - use Handles for that.
	//
	// NOTE: Much like a regular pointer, a "valid" pointer may still reference deleted memory if the 
	// object is destroyed after the Handle is Pinned. Do not use pointers, even to check for validity, 
	// after the underlying object has been destroyed. If the underlying object may have been destroyed, 
	// you should re-Pin the Handle to obtain a new pointer.
	//
	template <typename PointerImplType>
	class TRigidObjectPtr
	{
	public:
		using FPointerImpl = PointerImplType;
		using FContext = FPointerImpl::FContext;
		using FInterface = FPointerImpl::FInterface;
		using FHandle = FPointerImpl::FHandle;

		UE_NONCOPYABLE(TRigidObjectPtr);

		TRigidObjectPtr() = default;
		UE_INTERNAL TRigidObjectPtr(FInterface* InObject)
			: Impl(InObject)
		{
		}

		// Downcasting. Will fail to compile for attempts to cast to non-derived types.
		// Will generate "invalid" pointer in the runtime when InPtr is not an instance of right type.
		// Note: Also make sure we don't allow conversion to different context types.
		template<
			typename OtherPointerImplType
			UE_REQUIRES(std::is_base_of_v<typename OtherPointerImplType::FInterface, FInterface>&& std::is_same_v<typename OtherPointerImplType::FContext, FContext>)
		>
		UE_INTERNAL TRigidObjectPtr(const TRigidObjectPtr<OtherPointerImplType>& InPtr)
			: Impl(InPtr.Get())
		{
		}

		~TRigidObjectPtr() = default;

		// Reset the pointer (it will now be Invalid)
		void Reset()
		{
			Impl.Reset();
		}

		// Has this pointer been initialized with a valid body pointer?
		// NOTE: This will still return true if the underlying object is destroyed after the Handle is Pinned.
		// If this is a possibility, the Handle should be re-Pinned.
		bool IsValid() const
		{
			return Impl.IsValid();
		}

		// Auto cast to bool to support using in if statements
		//	e.g., if (TRigidBodyPtr<FRigidContextGameRW> Body = BodyHandle.Pin(Context)) { ... }
		operator bool() const
		{
			return IsValid();
		}

		// Extract the handle
		FHandle ToHandle() const
		{
			return Impl.GetHandle();
		}

		// Cast to a handle
		operator FHandle() const
		{
			return ToHandle();
		}

		// Const pointer semantics
		const FPointerImpl* operator->() const
		{
			CheckIsValid();
			return &Impl;
		}

		// Non-const pointer semantics
		FPointerImpl* operator->() const requires (FContext::bWriteEnabled)
		{
			CheckIsValid();
			return &Impl;
		}

		// Extract the object interface (internal use only)
		UE_INTERNAL FInterface* Get() const
		{
			return Impl.Get();
		}

		UE_INTERNAL void Init(FInterface* InObject)
		{
			Impl = FPointerImpl(InObject);
		}

	private:
		void CheckIsValid() const
		{
			UE_RIGIDPHYSICS_CHECKF(IsValid(), TEXT("Ptr is null. Check IsValid() before dereferencing"));
		}

		// Mutable to support const TRigidObjectPtr to an non-const object
		// i.e., this wrapper cannot change, but the pointee is mutabale
		// if the wrapper was created from a mutable context. This avoids
		// an undefined behaviour compiler warning.
		mutable FPointerImpl Impl = nullptr;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
