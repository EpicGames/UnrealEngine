// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Chaos/ObjectPool.h"
#include "RigidPhysics/RigidLog.h"
#include "RigidPhysics/RigidObjectId.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	// Info held in the TRigidObjectRegistry indexed by ObjectId
	template <typename ObjectPtrType>
	class UE_INTERNAL TRigidObjectRegistryItem
	{
	public:
		using FObjectPtrType = ObjectPtrType;

		UE_INTERNAL TRigidObjectRegistryItem()
			: Object(nullptr)
			, Epoch(0)
			, AddedObjectsIndex(INDEX_NONE)
		{
		}

		// The object that was added
		FObjectPtrType Object;

		// How many times the Object's slot in TRigidObjectRegistry has been used.
		// This will be unique for every reuse of the slot and is used by Object
		// Handles to determine if they have expired or not.
		int32 Epoch;

		// Index into the AddedObjects array in TRigidObjectRegistry
		int32 AddedObjectsIndex;
	};


	// A Registry for Rigid Objects that allows us to convert from a Handle to an Object Pointer.
	// Each registered object is assigned an Id, which is a combination of an array index and an epoch counter.
	// The array index and epoch counter combined are unique for every allocated object, even when array indices
	// get reused after an object is unregistered and a new one takes its place.
	// 
	// ObjectType must have a GetHandle() function.
	//
	// TODO_CHAOSAPI: ideally would not require a GetHandle() function on the object - it is only used the repack
	// the AddedObjects array. Should be a template policy instead to get to the Id? See RemoveObject().
	// 
	// TODO_CHAOSAPI: this container will fail to work if we reuse any ObjectsById array index enough times to
	// wrap its Epoch. We can mitigate this by removing those indices from the free list. This will leave unusable
	// gaps in the array but would allow for infinite growth at the cost of some wasted memory.
	//
	template <typename ObjectType>
	class UE_INTERNAL TRigidObjectRegistry
	{
	public:
		using FObjectType = ObjectType;
		using FObjectPtrType = ObjectType*;
		using FObjectRegistryItem = TRigidObjectRegistryItem<FObjectPtrType>;

	public:
		UE_INTERNAL TRigidObjectRegistry(int32 InSlack)
			: ArraySlack(InSlack)
			, TotalNumAdded(0)
			, bIsVisiting(false)
		{
		}

		UE_INTERNAL ~TRigidObjectRegistry() = default;

		// The number of objects that have been added (and not yet removed)
		UE_INTERNAL int32 GetNumObjects() const
		{
			return ObjectsById.Num() - ObjectsByIdFreeIndices.Num();
		}

		// True if there are no objects in the registry
		UE_INTERNAL bool IsEmpty() const
		{
			return GetNumObjects() == 0;
		}

		// Add an object to the registry and assign it an Id. The Id is persistent
		// and will never be reused as long as less that 1<<31 objects are added.
		// After that, Ids may start to repeat.
		UE_INTERNAL FRigidObjectId AddObject(FObjectPtrType InObject)
		{
			UE_RIGIDPHYSICS_CHECK(InObject != nullptr);

			// Cannot be called while visiting elements
			UE_RIGIDPHYSICS_CHECK(!bIsVisiting);

			// Assert if we add enough objects such that we might wrap the Epoch counter
			// (this is a lower bound, but is consistent and doesn't depend on add/remove order)
			// See class comments for future solution to this issue.
			UE_RIGIDPHYSICS_CHECK(TotalNumAdded < std::numeric_limits<int32>::max());
			++TotalNumAdded;

			// Make sure we have free array entries
			if (ObjectsByIdFreeIndices.IsEmpty())
			{
				ReserveArray(ObjectsById.Num() + ArraySlack);
			}

			// Assign an array index
			int32 ObjectsByIdIndex = ObjectsByIdFreeIndices.Pop(EAllowShrinking::No);

			// Put the item in the array
			FObjectRegistryItem& ObjectsByIdItem = ObjectsById[ObjectsByIdIndex];
			UE_RIGIDPHYSICS_CHECK(ObjectsByIdItem.Object == nullptr);
			ObjectsByIdItem.Object = InObject;
			ObjectsByIdItem.AddedObjectsIndex = AddedObjects.Add(InObject);

			// Build the Id from the Index and Epoch
			return FRigidObjectId(ObjectsByIdIndex, ObjectsByIdItem.Epoch);
		}

		UE_INTERNAL void RemoveObject(const FRigidObjectId& InObjectId)
		{
			// Cannot be called while visiting elements
			UE_RIGIDPHYSICS_CHECK(!bIsVisiting);

			// Call FindObject() first if presence is unknown
			UE_RIGIDPHYSICS_CHECK(ObjectsById.IsValidIndex(InObjectId.GetArrayIndex()));

			// Get the entry
			FObjectRegistryItem& ObjectsByIdItem = ObjectsById[InObjectId.GetArrayIndex()];

			// NOTE: We already know that this object was in the map before calling
			// so the epoch should always match
			UE_RIGIDPHYSICS_CHECK(ObjectsByIdItem.Epoch == InObjectId.GetEpoch());

			// Remove from the Allocated array (and fix up the AddedObjectsIndex of the moved item)
			int32 AddedObjectsIndex = ObjectsByIdItem.AddedObjectsIndex;
			if (AddedObjects.IsValidIndex(AddedObjectsIndex))
			{
				// If this fires, we have a ObjectsByIdItem.AddedObjectsIndex mismatch
				UE_RIGIDPHYSICS_CHECK(AddedObjects[AddedObjectsIndex] == ObjectsByIdItem.Object);

				AddedObjects.RemoveAtSwap(AddedObjectsIndex, EAllowShrinking::No);

				if (AddedObjects.IsValidIndex(AddedObjectsIndex))
				{
					// TODO_CHAOSAPI: GetId() is only required to maintain the packed allocated object array. Try to remove it.
					FRigidObjectId RelocatedObjectId = AddedObjects[AddedObjectsIndex]->GetHandle().GetId();
					FObjectRegistryItem& ReloctedArrayItem = ObjectsById[RelocatedObjectId.GetArrayIndex()];
					ReloctedArrayItem.AddedObjectsIndex = AddedObjectsIndex;
				}
			}

			// Reset the entry and increment the epoch to invalidate outstanding Ids using the same index
			ObjectsByIdItem.Object = nullptr;
			ObjectsByIdItem.Epoch = ObjectsByIdItem.Epoch + 1;
			ObjectsByIdItem.AddedObjectsIndex = INDEX_NONE;

			// Add the array item back into the free pool
			ObjectsByIdFreeIndices.Push(InObjectId.GetArrayIndex());

			// TODO_CHAOSAPI: we could shrink the ObjectArray here if we have too much slack, but
			// we'd need to know what the max allocated index was, and would have to prune
			// the free indices.
		}

		UE_INTERNAL void RemoveObjectChecked(const FRigidObjectId& InObjectId, const ObjectType* Object)
		{
			UE_RIGIDPHYSICS_CHECK(FindObject(InObjectId) == Object);
			RemoveObject(InObjectId);
		}

		UE_INTERNAL FObjectType* FindObject(const FRigidObjectId& InId) const
		{
			if (ObjectsById.IsValidIndex(InId.GetArrayIndex()))
			{
				const FObjectRegistryItem& ArrayItem = ObjectsById[InId.GetArrayIndex()];
				if (ArrayItem.Epoch == InId.GetEpoch())
				{
					return ArrayItem.Object;
				}
			}
			return nullptr;
		}

		// Visit all of the added objects in somewhat arbitrary order that may change whenever
		// objects are added or removed.
		UE_INTERNAL void VisitObjects(const TFunctionRef<ERigidVisitorResponse(FObjectType*)>& Visitor)
		{
			bIsVisiting = true;
			for (FObjectType* Object : AddedObjects)
			{
				ERigidVisitorResponse VisitorResponse = Visitor(Object);
				if (VisitorResponse == ERigidVisitorResponse::Break)
				{
					break;
				}
			}
			bIsVisiting = false;
		}

	private:
		void ReserveArray(int32 MaxItems)
		{
			if (MaxItems > ObjectsById.Num())
			{
				// Resize the arrays
				int32 PrevNumItems = ObjectsById.Num();
				ObjectsById.SetNum(MaxItems);
				ObjectsByIdFreeIndices.Reserve(MaxItems);
				AddedObjects.Reserve(MaxItems);

				// Add to the free indices
				// Loop backawards so we allocate lowest indices first in AddObject
				for (int32 ItemIndex = MaxItems - 1; ItemIndex >= PrevNumItems; --ItemIndex)
				{
					ObjectsByIdFreeIndices.Push(ItemIndex);
				}
			}
		}

		// An array with no holes containing all currently allocated objects indexed by ObjectId.AddedObjectsIndex.
		// This exists to support iteration over items and easier debugging - we may be able remove it later.
		// NOTE: Object position in this array is not persistent (RemoveObject uses RemoveAtSwap)
		// TODO_CHAOSAPI: see if we really need the AddedObjects array.
		TArray<FObjectType*> AddedObjects;

		// An Id->Object map, implemented as an array indexed by ObjectId.ArrayIndex.
		// Indices into this array are persistent as long as the object exists, but the array may contain holes.
		// Array Entries will be reused after objects have been deleted, and the Epoch counter is used to 
		// validate handles against the current object at the array index.
		// This array is presized and will often contain many empty (unused) elements.
		TArray<FObjectRegistryItem> ObjectsById;

		// A pool of unused entries in ObjectsById.
		TArray<int32> ObjectsByIdFreeIndices;

		// How much extra to allocate in ObjectsById when we run out of space.
		int32 ArraySlack;

		// How many objects have been added. This is only used to assert if we may reuse Ids.
		int32 TotalNumAdded;

		// Use to trap modifications while in the Visitor loop
		bool bIsVisiting;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
