// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidFactory.h"
#include "RigidPhysics/RigidSceneHandle.h"
#include "RigidPhysics/RigidSceneSettings.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class FRigidPhysicsService
	{
	public:
		RIGIDPHYSICS_API static FRigidPhysicsService& GetInstance();

		RIGIDPHYSICS_API void RegisterFactory(TUniquePtr<IRigidFactory>&& InFactory, TUniquePtr<IRigidSceneSettings>&& InDefaultSettings);

		RIGIDPHYSICS_API FRigidSceneHandle CreateScene(const FRigidDebugName& InName) const;
		RIGIDPHYSICS_API void DestroyScene(const FRigidSceneHandle& InHandle) const;

		RIGIDPHYSICS_API void Reset();
	private:
		TUniquePtr<IRigidFactory> Factory;
		TUniquePtr<IRigidSceneSettings> SceneSettings;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
