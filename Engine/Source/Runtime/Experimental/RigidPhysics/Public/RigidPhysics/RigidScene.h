// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/Internal/IRigidScene.h"
#include "RigidPhysics/RigidLockable.h"
#include "RigidPhysics/RigidLog.h"
#include "RigidPhysics/RigidObjectId.h"
#include "RigidPhysics/RigidObjectPtr.h"
#include "RigidPhysics/RigidSceneHandle.h"
#include "RigidPhysics/RigidTyped.h"
#include "Templates/SharedPointer.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	// A wrapper around IRigidScene.
	// Exists only to give pointer semantics to TRigidScenePtr.
	template <typename ContextType>
	class UE_INTERNAL TRigidScenePtrImpl
	{
	public:
		using FContext = ContextType;
		using FInterface = IRigidScene;
		using FHandle = FRigidSceneHandle;

		UE_INTERNAL TRigidScenePtrImpl() = default;
		UE_INTERNAL TRigidScenePtrImpl(IRigidScene* InScene)
			: SceneRaw(InScene)
		{
		}
		~TRigidScenePtrImpl() = default;

		bool IsValid() const
		{
			return (SceneRaw != nullptr);
		}

		FRigidSceneHandle GetHandle() const
		{
			return SceneRaw->GetHandle();
		}

		IRigidBodyContainer* GetBodyContainer(const FRigidObjectId& InBodyContainerId) const
		{
			return SceneRaw->GetBodyContainer(InBodyContainerId);
		}

		IRigidGeometryCollection* GetGeometryCollection(const FRigidObjectId& InGeometryCollectionId) const
		{
			return SceneRaw->GetGeometryCollection(InGeometryCollectionId);
		}

		void RegisterModifier(IRigidSceneModifier* Modifier) requires CIsGameContext<FContext>
		{
			SceneRaw->RegisterModifier(Modifier);
		}

		void UnregisterModifier(IRigidSceneModifier* Modifier) requires CIsGameContext<FContext>
		{
			SceneRaw->UnregisterModifier(Modifier);
		}

		TRigidBodyPtr<FContext> CreateBody(const FRigidDebugName& InName, ERigidMovementType InMovementType) requires CIsGameContext<FContext>
		{
			return SceneRaw->CreateBody(InName, InMovementType);
		}

		void DestroyBody(TRigidBodyPtr<FContext>&& InBody) requires CIsGameContext<FContext>
		{
			if (InBody.IsValid())
			{
				SceneRaw->DestroyBody(InBody.Get());
				InBody.Reset();
			}
		}

		TRigidGeometryCollectionPtr<FContext> CreateGeometryCollection(const FRigidDebugName& InName, const FSimulationParameters& InParameters) requires CIsGameContext<FContext>
		{
			return SceneRaw->CreateGeometryCollection(InName, InParameters);
		}

		void DestroyGeometryCollection(TRigidGeometryCollectionPtr<ContextType>&& InGC) requires CIsGameContext<FContext>
		{
			SceneRaw->DestroyGeometryCollection(InGC.Get());
			InGC.Reset();
		}

		void AddDisabledCollisions(const TMap<FRigidBodyHandle, TArray<FRigidBodyHandle>>& InMap) requires CIsGameContext<FContext>
		{
			SceneRaw->AddDisabledCollisions(InMap);
		}

		TJointConstraint6DOFPtr<FContext> CreateJointConstraint6DOF() requires CIsGameContext<FContext>
		{
			return SceneRaw->CreateJointConstraint6DOF();
		}

		void DestroyJointConstraint(TJointConstraint6DOFPtr<FContext>&& InConstraint) requires CIsGameContext<FContext>
		{
			if (InConstraint.IsValid())
			{
				SceneRaw->DestroyJointConstraint(InConstraint.Get());
				InConstraint.Reset();
			}
		}

		UE_INTERNAL IJointConstraint* GetJointConstraint(const FRigidObjectId& InConstraintId) const
		{
			return SceneRaw->GetJointConstraint(InConstraintId);
		}

		bool Query(const FRigidQueryDesc& QueryDesc, TArray<FRigidQueryHit>& OutHits)  const
		{
			return SceneRaw->Query(QueryDesc, OutHits);
		}

		void StartTick(const float InDt) requires CIsGameContext<FContext>
		{
			SceneRaw->StartTick(InDt);
		}

		void WaitOnTick() requires CIsGameContext<FContext>
		{
			SceneRaw->WaitOnTick();
		}

		void EndTick() requires CIsGameContext<FContext>
		{
			SceneRaw->EndTick();
		}

		UE_INTERNAL void Reset()
		{
			SceneRaw = nullptr;
		}

		UE_INTERNAL IRigidScene* Get() const
		{
			return SceneRaw;
		}

	private:
		IRigidScene* SceneRaw = nullptr;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
