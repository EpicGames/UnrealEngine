// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidSceneId.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class FRigidScene;

	// A handle to a rigid body scene. FRigidSceneHandle is only used in the GameContext
	// because all PhysicsContext use cases (callbacks, modifiers, etc.) will be passed their
	// scene as a TRigidScenePtr.
	class FRigidSceneHandle
	{
	public:
		FRigidSceneHandle() = default;
		UE_INTERNAL RIGIDPHYSICS_API FRigidSceneHandle(FRigidScene* InScene);
		UE_INTERNAL RIGIDPHYSICS_API FRigidSceneHandle(const FRigidSceneId& InSceneId);

		friend bool operator==(const FRigidSceneHandle&, const FRigidSceneHandle&) = default;
		friend bool operator!=(const FRigidSceneHandle&, const FRigidSceneHandle&) = default;

		// Lock the scene on the game thread and return a locked context for use in pinning object handles
		RIGIDPHYSICS_API FRigidContextGameRO LockRO() const;
		RIGIDPHYSICS_API FRigidContextGameRW LockRW() const;

		UE_INTERNAL RIGIDPHYSICS_API void Reset();
		UE_INTERNAL RIGIDPHYSICS_API FRigidSceneId GetId() const;
		UE_INTERNAL RIGIDPHYSICS_API IRigidScene* GetScene() const;

	private:
		void DoSafetyCheck() const;

		FRigidSceneId SceneId;
		// NOTE: Always the GT Scene, but never accessed on PT
		IRigidScene* SceneRaw = nullptr;
#if UE_RIGIDPHYSICS_SCENEHANDLE_SAFTEY_ENABLED
		FRigidSceneWeakPtr SceneWeak;
#endif
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
