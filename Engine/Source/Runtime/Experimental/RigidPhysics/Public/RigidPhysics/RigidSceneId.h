// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class FRigidSceneId
	{
	public:
		bool operator==(const FRigidSceneId&) const = default;
		bool operator!=(const FRigidSceneId&) const = default;

		RIGIDPHYSICS_API friend uint32 GetTypeHash(const FRigidSceneId& InId);

		UE_INTERNAL RIGIDPHYSICS_API IRigidScene* Get() const;
		UE_INTERNAL RIGIDPHYSICS_API void Reset();

	private:
		friend class FRigidSceneRegistry;

		uint8 Id = static_cast<uint8>(INDEX_NONE);
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
