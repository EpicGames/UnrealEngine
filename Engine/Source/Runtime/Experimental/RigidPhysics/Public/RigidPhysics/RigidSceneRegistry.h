// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Containers/StaticArray.h"
#include "RigidPhysics/RigidSceneId.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class FRigidSceneRegistry
	{
	public:
		RIGIDPHYSICS_API static FRigidSceneRegistry& GetInstance();

		RIGIDPHYSICS_API FRigidSceneId Register(IRigidScene* Scene);
		RIGIDPHYSICS_API void Unregister(const FRigidSceneId Id);

		RIGIDPHYSICS_API IRigidScene* Get(const FRigidSceneId Id) const;

		// Resets the singleton back to empty. Primarily used for tests.
		UE_INTERNAL RIGIDPHYSICS_API void Reset();

	private:
		static constexpr uint32 MaxScenes = 255;
		static FRigidSceneRegistry Instance;

		FRigidSceneRegistry() = default;

		uint8 SceneCount = 0;
		TStaticArray<IRigidScene*, MaxScenes> Scenes;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
