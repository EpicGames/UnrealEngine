// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidTyped.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class IRigidSceneSettings : public IRigidTyped
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(RIGIDPHYSICS_API, IRigidSceneSettings);

		IRigidSceneSettings() = default;
		virtual ~IRigidSceneSettings() = default;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
