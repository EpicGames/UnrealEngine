// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED
#include "Math/Bounds.h"
#include "Math/Transform.h"
#include "RigidPhysics/Internal/IRigidShapeInstance.h"
#include "RigidPhysics/RigidObjectId.h"
#include "RigidPhysics/RigidObjectPtr.h"
#include "RigidPhysics/RigidShapeInstanceHandle.h"
#include "RigidPhysics/RigidShapeInstanceSetup.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	template <typename ContextType>
	class TRigidShapeInstancePtrImpl
	{
	public:
		using FContext = ContextType;
		using FInterface = IRigidShapeInstance;
		using FHandle = FRigidShapeInstanceHandle;

		UE_INTERNAL TRigidShapeInstancePtrImpl() = default;
		UE_INTERNAL TRigidShapeInstancePtrImpl(IRigidShapeInstance* InShapeInstance)
			: ShapeInstanceRaw(InShapeInstance)
		{
		}

		TRigidShapeInstancePtrImpl(const TRigidShapeInstancePtrImpl&) = delete;

		TRigidShapeInstancePtrImpl(TRigidShapeInstancePtrImpl&& InPtr)
			: ShapeInstanceRaw(InPtr.ShapeInstanceRaw)
		{
			InPtr.ShapeInstanceRaw = nullptr;
		}

		~TRigidShapeInstancePtrImpl()
		{
			Reset();
		}

		friend bool operator==(const TRigidShapeInstancePtrImpl&, const TRigidShapeInstancePtrImpl&) = default;
		friend bool operator!=(const TRigidShapeInstancePtrImpl&, const TRigidShapeInstancePtrImpl&) = default;

		bool IsValid() const
		{
			return (ShapeInstanceRaw != nullptr);
		}

		FRigidShapeInstanceHandle GetHandle() const
		{
			if (ShapeInstanceRaw != nullptr)
			{
				return ShapeInstanceRaw->GetHandle();
			}
			return FRigidShapeInstanceHandle();
		}

		FAnyGeometry GetGeometry() const
		{
			return ShapeInstanceRaw->GetGeometry();
		}
		// TODO: SetGeometry

		FTransform3f GetLocalTransform() const
		{
			return ShapeInstanceRaw->GetLocalTransform();
		}
		// TODO: SetLocalTransform

		bool GetQueryEnabled() const
		{
			return ShapeInstanceRaw->GetQueryEnabled();
		}

		void SetQueryEnabled(const bool bInEnabled) requires CIsGameContext<FContext>
		{
			ShapeInstanceRaw->SetQueryEnabled(bInEnabled);
		}

		bool GetSimEnabled() const
		{
			return ShapeInstanceRaw->GetSimEnabled();
		}

		void SetSimEnabled(const bool bInEnabled) requires CIsGameContext<FContext>
		{
			ShapeInstanceRaw->SetSimEnabled(bInEnabled);
		}

		bool GetIsProbe() const
		{
			return ShapeInstanceRaw->GetIsProbe();
		}

		void SetIsProbe(const bool bInIsProbe) requires CIsGameContext<FContext>
		{
			ShapeInstanceRaw->SetIsProbe(bInIsProbe);
		}

		ECollisionTraceFlag GetCollisionTraceType() const
		{
			return ShapeInstanceRaw->GetCollisionTraceType();
		}

		void SetCollisionTraceType(const ECollisionTraceFlag InTraceFlag) requires CIsGameContext<FContext>
		{
			ShapeInstanceRaw->SetCollisionTraceType(InTraceFlag);
		}

		Chaos::Filter::FShapeFilterData GetShapeFilter() const
		{
			return ShapeInstanceRaw->GetShapeFilter();
		}

		void SetShapeFilter(const Chaos::Filter::FShapeFilterData& InShapeFilter) requires CIsGameContext<FContext>
		{
			ShapeInstanceRaw->SetShapeFilter(InShapeFilter);
		}

		Chaos::Filter::FInstanceData GetFilterInstanceData() const
		{
			return ShapeInstanceRaw->GetFilterInstanceData();
		}

		void SetFilterInstanceData(const Chaos::Filter::FInstanceData& InInstanceData) requires CIsGameContext<FContext>
		{
			ShapeInstanceRaw->SetFilterInstanceData(InInstanceData);
		}
		
		int32 GetNumMaterials() const
		{
			return ShapeInstanceRaw->GetNumMaterials();
		}

		FMaterialHandle GetMaterial(const int32 InIndex) const
		{
			return ShapeInstanceRaw->GetMaterial(InIndex);
		}

		void SetMaterials(TArray<FMaterialHandle>&& InMaterials) requires CIsGameContext<FContext>
		{
			ShapeInstanceRaw->SetMaterials(MoveTemp(InMaterials));
		}

		int32 GetNumMaterialMasks() const
		{
			return ShapeInstanceRaw->GetNumMaterialMasks();
		}

		FMaterialMaskHandle GetMaterialMask(const int32 InIndex) const
		{
			return ShapeInstanceRaw->GetMaterialMask(InIndex);
		}

		void SetMaterialMasks(TArray<FMaterialMaskHandle>&& InMaterialMasks) requires CIsGameContext<FContext>
		{
			ShapeInstanceRaw->SetMaterialMasks(MoveTemp(InMaterialMasks));
		}

		int32 GetNumMaterialMaskMaps() const
		{
			return ShapeInstanceRaw->GetNumMaterialMaskMaps();
		}

		uint32 GetMaterialMaskMap(const int32 InIndex) const
		{
			return ShapeInstanceRaw->GetMaterialMaskMap(InIndex);
		}

		void SetMaterialMaskMaps(TArray<uint32>&& InMaterialMaskMaps) requires CIsGameContext<FContext>
		{
			ShapeInstanceRaw->SetMaterialMaskMaps(MoveTemp(InMaterialMaskMaps));
		}

		int32 GetNumMaterialMaskMapMaterials() const
		{
			return ShapeInstanceRaw->GetNumMaterialMaskMapMaterials();
		}

		FMaterialHandle GetMaterialMaskMapMaterial(const int32 InIndex) const
		{
			return ShapeInstanceRaw->GetMaterialMaskMapMaterial(InIndex);
		}

		void SetMaterialMaskMapMaterials(TArray<FMaterialHandle>&& InMaterialMaskMapMaterials) requires CIsGameContext<FContext>
		{
			ShapeInstanceRaw->SetMaterialMaskMapMaterials(MoveTemp(InMaterialMaskMapMaterials));
		}

		void* GetUserData() const
		{
			return ShapeInstanceRaw->GetUserData();
		}

		void SetUserData(void* InUserData) requires CIsGameContext<FContext>
		{
			ShapeInstanceRaw->SetUserData(InUserData);
		}

		UE_INTERNAL void Reset()
		{
			ShapeInstanceRaw = nullptr;
		}

		UE_INTERNAL const FRigidTypeId& GetTypeId() const
		{
			return ShapeInstanceRaw->GetTypeId();
		}

		UE_INTERNAL IRigidShapeInstance* Get() const
		{
			return ShapeInstanceRaw;
		}

	private:
		IRigidShapeInstance* ShapeInstanceRaw = nullptr;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
