// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidObjectId.h"
#include "RigidPhysics/RigidObjectPtr.h"
#include "RigidPhysics/RigidSceneHandle.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class FRigidShapeInstanceHandle
	{
	public:
		FRigidShapeInstanceHandle() = default;
		FRigidShapeInstanceHandle(const FRigidShapeInstanceHandle&) = default;
		UE_INTERNAL RIGIDPHYSICS_API FRigidShapeInstanceHandle(const FRigidSceneId& InSceneId, const FRigidObjectId& InId);

		friend bool operator==(const FRigidShapeInstanceHandle&, const FRigidShapeInstanceHandle&) = default;
		friend bool operator!=(const FRigidShapeInstanceHandle&, const FRigidShapeInstanceHandle&) = default;

		template <typename ContextType>
		TRigidShapeInstancePtr<ContextType> Pin(const ContextType& InContext) const
		{
			if (const IRigidScene* Scene = InContext.GetRawSceneChecked(SceneId))
			{
				return PinInternal(Scene);
			}
			return TRigidShapeInstancePtr<ContextType>();
		}

		UE_INTERNAL RIGIDPHYSICS_API const FRigidObjectId& GetId() const;
		UE_INTERNAL RIGIDPHYSICS_API const FRigidSceneId& GetSceneId() const;
	private:
		RIGIDPHYSICS_API IRigidShapeInstance* PinInternal(const IRigidScene* Scene) const;

		FRigidObjectId Id;
		FRigidSceneId SceneId;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
