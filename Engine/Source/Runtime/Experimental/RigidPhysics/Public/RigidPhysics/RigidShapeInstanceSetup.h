// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED
#include "Math/Transform.h"
#include "RigidPhysics/Geometry/AnyGeometry.h"
#include "RigidPhysics/RigidMaterials.h"
#include "BodySetupEnums.h"
#include "Chaos/CollisionFilterData.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	class FRigidShapeInstanceSetup
	{
	public:
		RIGIDPHYSICS_API FRigidShapeInstanceSetup(const FAnyGeometry& InGeometry);

		FAnyGeometry Geometry;
		FTransform3f LocalTransform;
		Chaos::Filter::FShapeFilterData ShapeFilterData;
		Chaos::Filter::FInstanceData FilterInstanceData;
		ECollisionTraceFlag CollisionTraceType = ECollisionTraceFlag::CTF_UseDefault;
		TArray<FMaterialHandle> Materials;
		TArray<FMaterialMaskHandle> MaterialMaskHandles;
		TArray<uint32> MaterialMaskMaps;
		TArray<FMaterialHandle> MaterialMaskMapMaterials;
		bool bEnableSim = true;
		bool bEnableQuery = true;
		bool bEnableProbe = false;
		void* UserData = nullptr;
	};
} // namespace UE::Physics

#endif // UE_RIGIDPHYSICS_API_ENABLED
