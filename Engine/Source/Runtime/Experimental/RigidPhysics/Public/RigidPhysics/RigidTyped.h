// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Misc/Crc.h"

UE_EXPERIMENTAL(5.8, "The new Chaos API is experimental")
namespace UE::Physics
{
	// A type ID for a rigid object to support downcasting and type checking.
	// Each type-checkable class will contain a static FRigidTypeId.
	// NOTE: Only support single inheritance because that's all we need for now
	// and hopefully we can keep it that way.
	// TODO_CHAOSAPI: Consider using a more robust Id generator (to avoid CRC collisions).
	// TODO_CHAOSAPI: Consider using USTRUCT rather than this custom RTTI system.
	class FRigidTypeId
	{
	public:
		FRigidTypeId(const TCHAR* InTypeName, const FRigidTypeId* InBaseTypeId)
			: TypeIdValue(FCrc::StrCrc32(InTypeName))
			, BaseTypeId(InBaseTypeId)
			, TypeName(InTypeName)
		{
		}

		const uint64 GetValue() const
		{
			return TypeIdValue;
		}

		const FString& GetTypeName() const
		{
			return TypeName;
		}

		const FRigidTypeId* GetBase() const
		{
			return BaseTypeId;
		}

		inline bool IsCastableTo(const FRigidTypeId& Other) const
		{
			const FRigidTypeId* Base = this;
			while (Base != nullptr)
			{
				if (*Base == Other)
				{
					return true;
				}
				Base = Base->BaseTypeId;
			}
			return false;
		}

		friend inline bool operator==(const FRigidTypeId& L, const FRigidTypeId& R)
		{
			return L.TypeIdValue == R.TypeIdValue;
		}

		friend inline bool operator<(const FRigidTypeId& L, const FRigidTypeId& R)
		{
			return L.TypeIdValue < R.TypeIdValue;
		}

	private:
		uint64 TypeIdValue = 0;
		const FRigidTypeId* BaseTypeId = nullptr;
		const FString TypeName;
	};

	class IRigidTyped
	{
	public:
		IRigidTyped() = default;
		virtual ~IRigidTyped() = default;

		// Get the TypeId of this class
		RIGIDPHYSICS_API static const FRigidTypeId& GetStaticTypeId();

		// Get the leaf TypeId of this object
		RIGIDPHYSICS_API virtual const FRigidTypeId& GetTypeId() const;

		const FString& GetTypeName() const
		{
			return GetTypeId().GetTypeName();
		}

		// Is T the same class or a base class of this object?
		template <typename T>
		bool IsA() const
		{
			return GetTypeId().IsCastableTo(T::GetStaticTypeId());
		}

		// Try to cast this object to type T. Returns null if T is not a base class of this object.
		template <typename T>
		T* AsA()
		{
			if (IsA<T>())
			{
				return static_cast<T*>(this);
			}
			return nullptr;
		}

		// Try to cast this object to type T. Returns null if T is not a base class of this object.
		template <typename T>
		const T* AsA() const
		{
			if (IsA<T>())
			{
				return static_cast<const T*>(this);
			}
			return nullptr;
		}

		// Try to cast this object to type T. Assert T is a base class of this object.
		template <typename T>
		T* AsAChecked()
		{
			check(IsA<T>());
			return static_cast<T*>(this);
		}

		// Try to cast this object to type T. Assert T is a base class of this object.
		template <typename T>
		const T* AsAChecked() const
		{
			check(IsA<T>());
			return static_cast<const T*>(this);
		}
	};

} // namespace UE::Physics

#define UE_RIGIDPHYSICS_RIGIDTYPED_DECL(API, T) \
	API static const UE::Physics::FRigidTypeId& GetStaticTypeId(); \
	API virtual const UE::Physics::FRigidTypeId& GetTypeId() const override

#define UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(T, TBASE) \
	const UE::Physics::FRigidTypeId& T::GetTypeId() const \
	{ \
		return T::GetStaticTypeId(); \
	} \
	const UE::Physics::FRigidTypeId& T::GetStaticTypeId() \
	{ \
		using namespace UE::Physics; \
		static const TCHAR* StaticTypeName = TEXT(#T); \
		static FRigidTypeId StaticTypeId = FRigidTypeId(StaticTypeName, &TBASE::GetStaticTypeId()); \
		return StaticTypeId; \
	}

#endif // UE_RIGIDPHYSICS_API_ENABLED
