// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using System.Collections.Generic;

namespace UnrealBuildTool.Rules
{
    public class RigidPhysics : ModuleRules
    {
        public RigidPhysics(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
                new string[] {
                "Core",
                "CoreUObject",
				}
			);

			// Ideally RigidPhysics would not depend on Chaos Physics because this makes it harder
			// to implement completely separate physics engines, but for now we need some Chaos types.
			// TODO_CHAOSAPI: See if we can move everything we need into ChaosCore, or split Chaos
			// into more modules to reduce this dependency.
			SetupModulePhysicsSupport(Target);

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
    }
}
