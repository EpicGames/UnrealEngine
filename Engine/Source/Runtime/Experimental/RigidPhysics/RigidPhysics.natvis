<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">
  
  <Type Name="UE::Physics::TRigidObjectRegistry&lt;*&gt;">
    <Intrinsic Name="_IsValidIndex" Expression="0 &lt;= ArrayIndex &amp;&amp; ArrayIndex &lt; ObjectsById.ArrayNum">
      <Parameter Name="ArrayIndex" Type="int32"/>
    </Intrinsic>
    <Intrinsic Name="_IsValidEpoch" Expression="Epoch == ObjectsById._GetElementPtr(ArrayIndex)->Epoch">
      <Parameter Name="ArrayIndex" Type="int32"/>
      <Parameter Name="Epoch" Type="int32"/>
    </Intrinsic>
    <Intrinsic Name="_IsValidId" Expression="_IsValidIndex(ObjectId.ArrayIndex) &amp;&amp; _IsValidEpoch(ObjectId.ArrayIndex, ObjectId.Epoch)">
      <Parameter Name="ObjectId" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
    <Intrinsic Name="_FindObjectInternal" Expression="ObjectsById._GetElementPtr(ArrayIndex)->Object">
      <Parameter Name="ArrayIndex" Type="int32"/>
      <Parameter Name="Epoch" Type="int32"/>
    </Intrinsic>
    <Intrinsic Name="_FindObject" Expression="_FindObjectInternal(ObjectId.ArrayIndex, ObjectId.Epoch)">
      <Parameter Name="ObjectId" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
    <DisplayString>Num={AddedObjects.ArrayNum} Free={ObjectsByIdFreeIndices.ArrayNum}/{ObjectsById.ArrayNum}</DisplayString>
  </Type>

  <Type Name="UE::Physics::FRigidObjectId">
    <DisplayString Condition="ArrayIndex &lt; 0">&lt;Invalid&gt;</DisplayString>
    <DisplayString Condition="UE::Physics::Debugger::GShowIdEpoch">&lt;{ArrayIndex}.{Epoch}&gt;</DisplayString>
    <DisplayString>&lt;{ArrayIndex}&gt;</DisplayString>
  </Type>

  <Type Name="UE::Physics::IRigidScene">
    <!--Currently this is always an async scene. This needs to be handled better later. 
    Unfortunately, we return different return types from intrinsics based on any conditionals so this has to be hardcoded for now...-->
    <Intrinsic Name="_Downcast" Expression="(Chaos::Rigids::Async::FRigidSceneAsyncGT*)this"/>
  </Type>
  	
  <Type Name="UE::Physics::FRigidSceneId">
    <!--TODO: Figure out how to handle different scene types better-->
    <Intrinsic Name="_GetScene" Expression="UE::Physics::FRigidSceneRegistry::Instance.Scenes[Id]->_Downcast()"/>
    <Intrinsic Name="_IsValid" Expression="_GetScene() != nullptr"/>
    <Intrinsic Name="_IsValidJointConstraint" Expression="_IsValid() &amp;&amp; _GetScene()->JointConstraintRegistry->_IsValidId(Id)">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
    <Intrinsic Name="_IsValidRigidBody" Expression="_IsValid() &amp;&amp; _GetScene()->BodyRegistry->_IsValidId(Id)">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
    <Intrinsic Name="_IsValidRigidBodyContainer" Expression="_IsValid() &amp;&amp; _GetScene()->BodyContainerRegistry->_IsValidId(Id)">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
    <Intrinsic Name="_IsValidShapeInstance" Expression="_IsValid() &amp;&amp; _GetScene()->_IsValidShapeInstance(Id)">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
    <DisplayString Condition="Id == 255">&lt;Invalid&gt;</DisplayString>
    <!--Note: Cast to int so it does not display as a character-->
    <DisplayString>&lt;{(int)Id}&gt;</DisplayString>
    <Expand>
      <Item Name="Scene" Condition="_IsValid()">_GetScene()</Item>
    </Expand>
  </Type>

  <Type Name="UE::Physics::FRigidScene">
    <Intrinsic Name="_GetJointConstraint" Expression="JointConstraintRegistry.Ptr->_FindObject(ConstraintId)">
      <Parameter Name="ConstraintId" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
    <Intrinsic Name="_GetRigidBody" Expression="BodyRegistry.Ptr->_FindObject(Id)">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
    <Intrinsic Name="_GetRigidBodyContainer" Expression="BodyContainerRegistry.Ptr->_FindObject(Id)">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
  </Type>

  <Type Name="UE::Physics::FRigidSceneHandle">
    <Intrinsic Name="_GetScene" Expression="SceneId._GetScene()"/>
    <Intrinsic Name="_IsValid" Expression="_GetScene() != nullptr"/>

    <DisplayString Condition="_GetScene() == nullptr">&lt;Invalid&gt;</DisplayString>
    <DisplayString>{*_GetScene()}</DisplayString>
    <Expand>
      <Item Name="Scene">_GetScene()</Item>
      <Item Name="SceneId">SceneId</Item>
    </Expand>
  </Type>

  <Type Name="UE::Physics::FRigidBodyContainerHandle">
    <Intrinsic Name="_IsCurrent" Expression="SceneId._IsValidRigidBodyContainer(BodyContainerId)"/>
    <Intrinsic Name="_GetBodyContainer" Expression="SceneId._GetScene()->_GetRigidBodyContainer(BodyContainerId)"/>
    <DisplayString Condition="BodyContainerId.ArrayIndex &lt; 0">&lt;Invalid&gt;</DisplayString>
    <DisplayString Condition="!_IsCurrent()">&lt;Expired&gt;</DisplayString>
    <DisplayString Condition="UE::Physics::Debugger::GHaveSceneHandle">{*_GetBodyContainer()}</DisplayString>
    <DisplayString>BodyContainer&lt;{BodyContainerId}&gt;</DisplayString>
    <Expand>
      <Item Name="BodyContainer" Condition="_IsCurrent()">_GetBodyContainer()</Item>
      <Item Name="Scene" Condition="SceneId._IsValid()">SceneId._GetScene()</Item>
      <Item Name="BodyContainerId">BodyContainerId</Item>
      <Item Name="SceneId">SceneId</Item>
    </Expand>
  </Type>

  <Type Name="UE::Physics::FRigidGeometryCollectionHandle">
    <Intrinsic Name="_IsCurrent" Expression="BodyContainerHandle.SceneId._IsValidRigidBodyContainer(BodyContainerHandle.BodyContainerId)"/>
    <Intrinsic Name="_GetGeometryCollection" Expression="BodyContainerHandle.SceneId._GetScene()->_GetRigidBodyContainer(BodyContainerHandle.BodyContainerId)"/>
    <DisplayString>{BodyContainerHandle}</DisplayString>
    <Expand>
      <Item Name="GeometryCollection" Condition="_IsCurrent()">_GetGeometryCollection()</Item>
      <Item Name="Scene" Condition="BodyContainerHandle.SceneId._IsValid()">BodyContainerHandle.SceneId._GetScene()</Item>
      <Item Name="BodyContainerId">BodyContainerHandle.BodyContainerId</Item>
      <Item Name="SceneId">BodyContainerHandle.SceneId</Item>
    </Expand>
  </Type>
    

  <Type Name="UE::Physics::FRigidBodyHandle">
    <Intrinsic Name="_IsCurrent" Expression="SceneId._IsValidRigidBody(BodyId)"/>
    <Intrinsic Name="_GetBody" Expression="SceneId._GetScene()->_GetRigidBody(BodyId)"/>
    <DisplayString Condition="BodyId.ArrayIndex &lt; 0">&lt;Invalid&gt;</DisplayString>
    <DisplayString Condition="!_IsCurrent()">&lt;Expired&gt;</DisplayString>
    <DisplayString>{*_GetBody()}</DisplayString>
    <Expand>
      <Item Name="Body" Condition="_IsCurrent()">_GetBody()</Item>
      <Item Name="Scene" Condition="SceneId._IsValid()">SceneId._GetScene()</Item>
      <Item Name="BodyId">BodyId</Item>
      <Item Name="SceneId">SceneId</Item>
    </Expand>
  </Type>

  <Type Name="UE::Physics::FJointConstraintHandle">
    <Intrinsic Name="_GetJointConstraint" Expression="SceneId._GetScene()->_GetJointConstraint(ConstraintId)"/>
    <Intrinsic Name="_IsValid" Expression="SceneId._IsValidJointConstraint(ConstraintId)"/>
    <DisplayString Condition="ConstraintId.ArrayIndex &lt; 0">&lt;Invalid&gt;</DisplayString>
    <DisplayString Condition="!_IsValid()">&lt;Expired&gt;</DisplayString>
    <DisplayString>{*_GetJointConstraint()}</DisplayString>
    <Expand>
      <Item Name="Constraint" Condition="_IsValid()">_GetJointConstraint()</Item>
      <Item Name="Scene" Condition="_IsValid()">SceneId._GetScene()</Item>
      <Item Name="ConstraintId">ConstraintId</Item>
      <Item Name="SceneId">SceneId</Item>
    </Expand>
  </Type>
  
  <Type Name="UE::Physics::FRigidShapeInstanceHandle">
    <Intrinsic Name="_IsValid" Expression="SceneId._IsValidShapeInstance(Id)"/>
    <Intrinsic Name="_GetShapeInstance" Expression="SceneId._GetScene()->_GetShapeInstance(Id)"/>
    <DisplayString Condition="Id.ArrayIndex &lt; 0">&lt;Invalid&gt;</DisplayString>
    <DisplayString Condition="!_IsValid()">&lt;Expired&gt;</DisplayString>
    <DisplayString>{*_GetShapeInstance()}</DisplayString>
    <Expand>
      <Item Name="ShapeInstance" Condition="_IsValid()">_GetShapeInstance()</Item>
      <Item Name="Scene" Condition="SceneId._IsValid()">SceneId._GetScene()</Item>
      <Item Name="Id">Id</Item>
      <Item Name="SceneId">SceneId</Item>
    </Expand>
  </Type>

  <Type Name="UE::Physics::TRigidContext&lt;*,*&gt;">
    <DisplayString>{ScenePtr}</DisplayString>
  </Type>
  
  <Type Name="UE::Physics::TRigidObjectPtr&lt;*&gt;">
    <DisplayString>{Impl}</DisplayString>
    <Expand>
      <ExpandedItem>Impl</ExpandedItem>
    </Expand>
  </Type>
  
  <Type Name="UE::Physics::TRigidScenePtrImpl&lt;*&gt;">
    <DisplayString Condition="SceneRaw == 0">&lt;NULL&gt;</DisplayString>
    <DisplayString>{*SceneRaw}</DisplayString>
    <Expand>
      <ExpandedItem Condition="SceneRaw != nullptr">*SceneRaw</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="UE::Physics::TRigidBodyContainerPtrImpl&lt;*&gt;">
    <DisplayString Condition="BodyContainerRaw == 0">&lt;NULL&gt;</DisplayString>
    <DisplayString>{*BodyContainerRaw}</DisplayString>
    <Expand>
      <ExpandedItem Condition="BodyContainerRaw != nullptr">*BodyContainerRaw</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="UE::Physics::TRigidGeometryCollectionPtrImpl&lt;*&gt;">
    <DisplayString Condition="GeometryCollectionRaw == 0">&lt;NULL&gt;</DisplayString>
    <DisplayString>{*GeometryCollectionRaw}</DisplayString>
    <Expand>
      <ExpandedItem Condition="GeometryCollectionRaw != nullptr">*GeometryCollectionRaw</ExpandedItem>
    </Expand>
  </Type>

  <Type Name="UE::Physics::TRigidBodyPtrImpl&lt;*&gt;">
    <DisplayString Condition="BodyRaw == 0">&lt;NULL&gt;</DisplayString>
    <DisplayString>{*BodyRaw}</DisplayString>
    <Expand>
      <ExpandedItem Condition="BodyRaw != 0">*BodyRaw</ExpandedItem>
    </Expand>
  </Type>
  
</AutoVisualizer>