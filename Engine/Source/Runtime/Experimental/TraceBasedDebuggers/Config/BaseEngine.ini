[CoreRedirects]
+EnumRedirects=(OldName="/Script/ChaosVDRuntime.EChaosVDRecordingMode", NewName="/Script/TraceBasedDebuggers.ERecordingMode")
+EnumRedirects=(OldName="/Script/ChaosVDRuntime.EChaosVDTransportMode", NewName="/Script/TraceBasedDebuggers.ETraceTransportMode")
+EnumRedirects=(OldName="/Script/ChaosVDRuntime.EChaosVDRemoteSessionAttributes", NewName="/Script/TraceBasedDebuggers.ERemoteSessionAttributes")
+EnumRedirects=(OldName="/Script/ChaosVDRuntime.EChaosVDRemoteSessionReadyState", NewName="/Script/TraceBasedDebuggers.ERemoteSessionReadyState")

+StructRedirects=(OldName="/Script/ChaosVDRuntime.ChaosVDTraceDetails", NewName="/Script/TraceBasedDebuggers.TraceConnectionDetails")
+StructRedirects=(OldName="/Script/ChaosSolverEngine.RelayConnectionInfo", NewName="/Script/TraceBasedDebuggers.RelayConnectionInfo")
+StructRedirects=(OldName="/Script/ChaosSolverEngine.ChaosVDSessionPing", NewName="/Script/TraceBasedDebuggers.SessionPing")
+StructRedirects=(OldName="/Script/ChaosSolverEngine.ChaosVDSessionPong", NewName="/Script/TraceBasedDebuggers.SessionPong")
+StructRedirects=(OldName="/Script/ChaosSolverEngine.ChaosVDTraceConnectionDetailsMessage", NewName="/Script/TraceBasedDebuggers.TraceConnectionDetailsMessage")
+StructRedirects=(OldName="/Script/ChaosSolverEngine.ChaosVDRecordingStatusMessage", NewName="/Script/TraceBasedDebuggers.RecordingStatusMessage")
+StructRedirects=(OldName="/Script/ChaosSolverEngine.ChaosVDFullSessionInfoRequestMessage", NewName="/Script/TraceBasedDebuggers.FullSessionInfoRequestMessage")
+StructRedirects=(OldName="/Script/ChaosSolverEngine.ChaosVFFullSessionInfoResponseMessage", NewName="/Script/TraceBasedDebuggers.FullSessionInfoResponseMessage")
