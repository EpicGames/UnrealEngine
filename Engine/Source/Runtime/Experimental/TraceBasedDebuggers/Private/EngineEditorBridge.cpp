// Copyright Epic Games, Inc. All Rights Reserved.

#include "EngineEditorBridge.h"

#if WITH_TRACE_BASED_DEBUGGERS

#include "Misc/App.h"

namespace UE::TraceBasedDebuggers
{

TWeakPtr<FRemoteSessionsManager> FEngineEditorBridge::SharedRemoteSessionsManager;

TSharedPtr<FRemoteSessionsManager> FEngineEditorBridge::GetSessionsManager() const
{
	return RemoteSessionsManager;
}

void FEngineEditorBridge::HandleRecordingStarted()
{
	BroadcastSessionStatus();

	// Create periodic status update (every 0.5 seconds)
	if (!RecordingStatusUpdateHandle.IsValid())
	{
		constexpr float UpdateInterval = 0.5f;
		RecordingStatusUpdateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float)
			{
				BroadcastSessionStatus();
				return true;
			}), UpdateInterval);
	}

	// Allow derived classes to perform additional actions
	OnRecordingStartedInternal();
}

void FEngineEditorBridge::HandleRecordingStopped()
{
	// Remove periodic status update
	FTSTicker::RemoveTicker(RecordingStatusUpdateHandle);
	RecordingStatusUpdateHandle.Reset();

	BroadcastSessionStatus();

	// Allow derived classes to perform additional actions
	OnRecordingStoppedInternal();
}

void FEngineEditorBridge::BroadcastSessionStatus() const
{
	FRecordingStatusMessage StatusMessage;

	// Allow derived classes to populate the message
	BuildRecordingStatusInternal(StatusMessage);
	if (ensureMsgf(StatusMessage.DebuggerId.IsValid(), TEXT("It is mandatory to provide the debugger type identifier.")))
	{
		StatusMessage.InstanceId = FApp::GetInstanceId();
		RemoteSessionsManager->PublishMessage(StatusMessage);
	}
}

void FEngineEditorBridge::SetDataRelayTransportInstance(const TSharedPtr<IDataRelayTransport>& InExternalTraceRelayInstance)
{
	if (!ensure(!RelayTraceDataTransportInstance))
	{
		UE_LOG_REF(LogCategory, Error, TEXT("An external relay instance was already set. This cannot be changed afterwards. Ignoring set request..."));
		return;
	}

	RelayTraceDataTransportInstance = InExternalTraceRelayInstance;
}

FEngineEditorBridge::FEngineEditorBridge(const FLogCategoryAlias& LogCategory)
	: RemoteSessionsManager(SharedRemoteSessionsManager.IsValid() ? SharedRemoteSessionsManager.Pin().ToSharedRef() : MakeShared<FRemoteSessionsManager>())
	, LogCategory(LogCategory)
{
	if (!SharedRemoteSessionsManager.IsValid())
	{
		SharedRemoteSessionsManager = RemoteSessionsManager;
	}
}

void FEngineEditorBridge::Initialize()
{
	// Allow derived classes to perform their initialization
	OnInitializeInternal();

	bIsInitialized = true;

	InitializedDelegate.Broadcast();
}

void FEngineEditorBridge::RegisterRemoteSessionsHandler(TSharedRef<IRemoteSessionsHandler> Handler)
{
	ensureMsgf(!RemoteSessionsHandler.IsValid(), TEXT("Handler already registered."));
	RemoteSessionsHandler = Handler;
	GetSessionsManager()->RegisterExternalHandler(Handler);
}

void FEngineEditorBridge::TearDown()
{
	if (RelayTraceDataTransportInstance)
	{
		RelayTraceDataTransportInstance->Shutdown();
	}

	FTSTicker::RemoveTicker(RecordingStatusUpdateHandle);
	RecordingStatusUpdateHandle.Reset();

	// Allow derived classes to perform their cleanup
	OnTearDownInternal();

	if (RemoteSessionsHandler)
	{
		GetSessionsManager()->UnregisterExternalHandler(RemoteSessionsHandler.ToSharedRef());
	}

	bIsInitialized = false;
}

} // namespace UE::TraceBasedDebuggers
#endif // WITH_TRACE_BASED_DEBUGGERS
