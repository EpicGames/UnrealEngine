// Copyright Epic Games, Inc. All Rights Reserved.

#include "RelayTraceDataWriter.h"

#if WITH_TRACE_BASED_DEBUGGERS
#include "Math/GuardedInt.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"

namespace UE::TraceBasedDebuggers
{
	FRelayTraceDataWriter::FRelayTraceDataWriter(const FLogCategoryAlias& LogCategory)
		: LogCategory(LogCategory)
	{
	}

	FRelayTraceDataWriter::~FRelayTraceDataWriter()
	{
		Close();
	}

	void FRelayTraceDataWriter::SetMaxBytesPerBunch(uint32 MaxBytes)
	{
		if (MaxBytes == 0)
		{
			UE_LOG_REF(LogCategory, Error, TEXT("[%hs] Only values greater than 0 are supported"), __func__);
			return;
		}

		if (MaxBytes != MaxPendingBytesPerBunch)
		{
			if (!ensure(PendingBunchesToRelay.IsEmpty()))
			{
				return;
			}
		}

		MaxPendingBytesPerBunch = MaxBytes;
	}

	void FRelayTraceDataWriter::SetMaxAllowedPendingBytes(uint64 MaxBytes)
	{
		if (MaxBytes == 0)
		{
			UE_LOG_REF(LogCategory, Error, TEXT("[%hs] Only values greater than 0 are supported"), __func__);
			return;
		}

		MaxAllowedPendingBytes = MaxBytes;
	}

	bool FRelayTraceDataWriter::WriteHelper(UPTRINT WriterHandle, const void* Data, uint32 Size)
	{
		if (!WriterHandle)
		{
			return false;
		}

		return reinterpret_cast<FRelayTraceDataWriter*>(WriterHandle)->Write_Internal(Data, Size);
	}

	TOptional<TUniquePtr<FBufferArchive>> FRelayTraceDataWriter::DequeuePendingBunch()
	{
		if (PendingBunchesCount == 0)
		{
			return TOptional<TUniquePtr<FBufferArchive>>();
		}

		QueuedBytesNum -= NextPendingBunchSize();

		--PendingBunchesCount;
		return PendingBunchesToRelay.Dequeue();
	}

	int32 FRelayTraceDataWriter::NextPendingBunchSize() const
	{
		if (PendingBunchesCount == 0)
		{
			return 0;
		}

		TUniquePtr<FBufferArchive>* PendingBunch = PendingBunchesToRelay.Peek();

		if (PendingBunch && PendingBunch->IsValid())
		{
			return (*PendingBunch)->Num();
		}

		return 0;
	}

	TUniquePtr<FBufferArchive> FRelayTraceDataWriter::CreateBunch()
	{
		TUniquePtr<FBufferArchive> NewBunch = MakeUnique<FBufferArchive>();
		return NewBunch;
	}

	bool FRelayTraceDataWriter::Write_Internal(const void* Data, uint32 Size)
	{
		if (bClosed)
		{
			return false;
		}

		const bool bHasValidLimits = MaxPendingBytesPerBunch != 0;
		if (!ensure(bHasValidLimits))
		{
			return false;
		}

		if (!CanWriteData(Size))
		{
			DataOverflowDelegate.ExecuteIfBound();
			Close();
			return false;
		}

		if (!CurrentPendingBunch)
		{
			CurrentPendingBunch = CreateBunch();
		}

		++TotalWriteCalls;
		TotalBytesWritten += Size;

		uint32 RemainingSizeToCopy = Size;

		while (RemainingSizeToCopy > 0)
		{
			int32 AvailableSpaceInCurrentBunch = MaxPendingBytesPerBunch - CurrentPendingBunch->Num();

			if (AvailableSpaceInCurrentBunch <= 0)
			{
				QueuedBytesNum += CurrentPendingBunch->Num();
				PendingBunchesToRelay.Enqueue(MoveTemp(CurrentPendingBunch));
				++PendingBunchesCount;
				++TotalBunchesCompleted;
				CurrentPendingBunch = CreateBunch();
				AvailableSpaceInCurrentBunch = MaxPendingBytesPerBunch;

				NewDataAvailableDelegate.ExecuteIfBound();
			}

			const uint32 BytesToCopyNum = static_cast<uint32>(AvailableSpaceInCurrentBunch) > RemainingSizeToCopy ? RemainingSizeToCopy : static_cast<uint32>(AvailableSpaceInCurrentBunch);

			const uint8* DestBufferPtr = static_cast<const uint8*>(Data);
			const int32 Offset = Size - RemainingSizeToCopy;

			const bool bIsValidOffset = Offset >= 0 && static_cast<uint32>(Offset) < Size;
			if (!ensureMsgf(bIsValidOffset, TEXT("Unexpected calculated offset [%d] | Size [%u]"), Offset, Size))
			{
				return false;
			}

			CurrentPendingBunch->Serialize(const_cast<uint8*>(DestBufferPtr + Offset), BytesToCopyNum);

			if (CurrentPendingBunch->IsError())
			{
				return false;
			}

			RemainingSizeToCopy -= BytesToCopyNum;
		}

		return !CurrentPendingBunch->IsError();
	}

	bool FRelayTraceDataWriter::CanWriteData(uint32 NewDataSize) const
	{
		TGuardedInt<uint64> ExpectedQueuedBytes(0);

		ExpectedQueuedBytes += QueuedBytesNum;

		ExpectedQueuedBytes += NewDataSize;

		if (CurrentPendingBunch)
		{
			ExpectedQueuedBytes += CurrentPendingBunch->Num();
		}

		constexpr uint64 DefaultValue = std::numeric_limits<uint64>::max();
		return ExpectedQueuedBytes.Get(DefaultValue) <= MaxAllowedPendingBytes;
	}

	int64 FRelayTraceDataWriter::GetQueuedBytesNum() const
	{
		return QueuedBytesNum;
	}

	void FRelayTraceDataWriter::Close()
	{
		// Flush the current incomplete bunch so it is available for the relay transport to send
		const int32 CurrentBunchBytes = CurrentPendingBunch ? CurrentPendingBunch->Num() : 0;
		if (CurrentPendingBunch && CurrentBunchBytes > 0)
		{
			QueuedBytesNum += CurrentBunchBytes;
			PendingBunchesToRelay.Enqueue(MoveTemp(CurrentPendingBunch));
			++PendingBunchesCount;
			++TotalBunchesCompleted;

			NewDataAvailableDelegate.ExecuteIfBound();
		}
		CurrentPendingBunch.Reset();

		UE_LOG_REF(LogCategory, Log, TEXT("[%hs] Closing relay writer | WriteCalls [%d] | TotalBytesWritten [%lld] | BunchesCompleted [%d] | PendingQueueBytes [%lld] | FlushedIncompleteBunch [%d bytes]"),
			__func__, TotalWriteCalls, TotalBytesWritten, TotalBunchesCompleted, static_cast<int64>(QueuedBytesNum), CurrentBunchBytes);

		bClosed = true;
	}
}
#endif // WITH_TRACE_BASED_DEBUGGERS