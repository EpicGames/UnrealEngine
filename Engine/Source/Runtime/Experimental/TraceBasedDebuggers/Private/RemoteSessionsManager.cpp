// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteSessionsManager.h"

#include "IMessageBridge.h"
#include "Containers/ArrayBuilder.h"
#include "Features/IModularFeatures.h"
#include "IMessageBus.h"
#include "IMessagingModule.h"
#include "INetworkMessagingExtension.h"
#include "MessageEndpoint.h"
#include "MessageEndpointBuilder.h"
#include "Misc/App.h"
#include "SessionInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RemoteSessionsManager)

#ifndef TRACE_BASED_DEBUGGERS_WITHOUT_TRACE
#define TRACE_BASED_DEBUGGERS_WITHOUT_TRACE 0
#endif

namespace UE::TraceBasedDebuggers
{

bool bIgnoreExpiredSessionsForDebugging = false;
static FAutoConsoleVariableRef CVarIgnoreExpiredSessionsForDebugging(
	TEXT("RemoteSessionsManager.IgnoreExpiredSessionsForDebugging"),
	bIgnoreExpiredSessionsForDebugging,
	TEXT("Prevent sessions to expire due to breakpoints."),
	ECVF_Cheat);

const FGuid FRemoteSessionsManager::AllRemoteSessionsWrapperGUID = FGuid::NewGuid();
const FGuid FRemoteSessionsManager::AllRemoteServersWrapperGUID = FGuid::NewGuid();
const FGuid FRemoteSessionsManager::AllRemoteClientsWrapperGUID = FGuid::NewGuid();
const FGuid FRemoteSessionsManager::AllSessionsWrapperGUID = FGuid::NewGuid();
const FGuid FRemoteSessionsManager::CustomSessionsWrapperGUID = FGuid::NewGuid();
const FGuid FRemoteSessionsManager::InvalidSessionGUID = FGuid();

const FString FRemoteSessionsManager::LocalEditorSessionName = TEXT("Local Editor");
const FGuid FRemoteSessionsManager::LocalEditorSessionID = IsController() ? FApp::GetInstanceId() : InvalidSessionGUID;
const FName FRemoteSessionsManager::MessageBusEndpointName = FName("FDebuggerSessionManager");
const FString FRemoteSessionsManager::AllRemoteSessionsTargetName = TEXT("All Remote");
const FString FRemoteSessionsManager::AllRemoteServersTargetName = TEXT("All Remote Servers");
const FString FRemoteSessionsManager::AllRemoteClientsTargetName = TEXT("All Remote Clients");
const FString FRemoteSessionsManager::AllSessionsTargetName = TEXT("All Sessions");
const FString FRemoteSessionsManager::CustomSessionsTargetName = TEXT("Custom Selection");

const FTraceConnectionDetails& FSessionInfo::GetTraceConnectionDetails()
{
	return LastKnownConnectionDetails;
}

bool FSessionInfo::IsAnyDebuggerRecording() const
{
	return LastKnownRecordingState.RequesterId.IsValid();
}

bool FSessionInfo::IsRecording(const FGuid& DebuggerTypeId) const
{
	return LastKnownRecordingState.DebuggerId == DebuggerTypeId
		&& LastKnownRecordingState.RequesterId == FRemoteSessionsManager::LocalEditorSessionID;
}

uint64 FSessionInfo::GetBufferedBytesNum() const
{
	return LastKnownRecordingState.BufferedDataBytesSize;
}

void FSessionInfo::SetReceivedBytesPerSecond(uint64 InNewBytesPerSecond)
{
	ReceivedBytesPerSecond = InNewBytesPerSecond;
}

uint64 FSessionInfo::GetReceivedBytesPerSecond() const
{
	return ReceivedBytesPerSecond;
}

bool FMultiSessionInfo::IsAnyDebuggerRecording() const
{
	bool bIsRecording = false;
	EnumerateInnerSessions([&bIsRecording](const TSharedRef<FSessionInfo>& InSessionRef)
	{
		if (InSessionRef->IsAnyDebuggerRecording())
		{
			bIsRecording = true;
			return false;
		}
		return true;
	});

	return bIsRecording;
}

bool FMultiSessionInfo::IsRecording(const FGuid& DebuggerTypeId) const
{
	bool bIsRecording = false;
	EnumerateInnerSessions([&bIsRecording, DebuggerTypeId](const TSharedRef<FSessionInfo>& InSessionRef)
	{
		if (InSessionRef->IsRecording(DebuggerTypeId))
		{
			bIsRecording = true;
			return false;
		}
		return true;
	});

	return bIsRecording;
}

uint64 FMultiSessionInfo::GetBufferedBytesNum() const
{
	uint64 BufferedBytesNum = 0;
	EnumerateInnerSessions([&BufferedBytesNum](const TSharedRef<FSessionInfo>& InSessionRef)
	{
		BufferedBytesNum += InSessionRef->GetBufferedBytesNum();
		return true;
	});

	return BufferedBytesNum;
}

void FRemoteSessionsManager::RegisterBuiltInMessageTypes()
{
	SupportedMessageTypes.Emplace(FSessionPong::StaticStruct());
	SupportedMessageTypes.Emplace(FRecordingStatusMessage::StaticStruct());
	SupportedMessageTypes.Emplace(FSessionPing::StaticStruct());
	SupportedMessageTypes.Emplace(FFullSessionInfoRequestMessage::StaticStruct());
	SupportedMessageTypes.Emplace(FFullSessionInfoResponseMessage::StaticStruct());
	SupportedMessageTypes.Emplace(FTraceConnectionDetailsMessage::StaticStruct());
}

FRemoteSessionsManager::FRemoteSessionsManager()
{
	RegisterBuiltInMessageTypes();
	Initialize();
}

FRemoteSessionsManager::~FRemoteSessionsManager()
{
	Shutdown();
}

TWeakPtr<FSessionInfo> FRemoteSessionsManager::GetSessionInfo(const FGuid Id)
{
	if (TSharedPtr<FSessionInfo>* FoundSessionPtrPtr = ActiveSessionsById.Find(Id))
	{
		return *FoundSessionPtrPtr;
	}

	return nullptr;
}

TWeakPtr<FSessionInfo> FRemoteSessionsManager::GetFirstAvailableSingleSessionInfo()
{
	for (const TPair<FGuid, TSharedPtr<FSessionInfo>>& SessionWithID : ActiveSessionsById)
	{
		if (SessionWithID.Value && !EnumHasAnyFlags(SessionWithID.Value->GetSessionTypeAttributes(), ERemoteSessionAttributes::IsMultiSessionWrapper))
		{
			return SessionWithID.Value;
		}
	}

	return nullptr;
}

TSharedPtr<FSessionInfo> FRemoteSessionsManager::CreatedWrapperSessionInfo(FGuid InstanceId, const FStringView SessionName)
{
	TSharedPtr<FMultiSessionInfo> NewSessionInfo = MakeShared<FMultiSessionInfo>();
	NewSessionInfo->InstanceId = InstanceId;
	NewSessionInfo->SessionName = SessionName;

	return NewSessionInfo;
}

TSharedPtr<FMessageEndpoint> FRemoteSessionsManager::CreateEndPoint(const TSharedRef<IMessageBus>& InMessageBus)
{
	// Create process specific name to make it easier to debug messages
	const FName EndPointName(FString::Format(TEXT("{0} ({1} {2})"),
		{ MessageBusEndpointName.ToString()
		, FApp::GetProjectName()
		, FString::FromInt(FPlatformProcess::GetCurrentProcessId()) }));

	FMessageEndpointBuilder EndpointBuilder = FMessageEndpoint::Builder(EndPointName, InMessageBus)
		.Handling<FSessionPing>(this, &FRemoteSessionsManager::HandleSessionPingMessage)
		.Handling<FFullSessionInfoRequestMessage>(this, &FRemoteSessionsManager::HandleFullSessionStateRequestMessage);

	if (IsController())
	{
		EndpointBuilder.Handling<FSessionPong>(this, &FRemoteSessionsManager::HandleSessionPongMessage)
			.Handling<FRecordingStatusMessage>(this, &FRemoteSessionsManager::HandleRecordingStatusUpdateMessage)
			.Handling<FFullSessionInfoResponseMessage>(this, &FRemoteSessionsManager::HandleFullSessionStateResponseMessage)
			.Handling<FTraceConnectionDetailsMessage>(this, &FRemoteSessionsManager::HandleConnectionDetailsUpdateMessage);
	}

	for (const TSharedRef<IRemoteSessionsHandler>& Handler : Handlers)
	{
		Handler->OnCreatingMessageEndpoint(AsShared(), EndpointBuilder);
	}

	return EndpointBuilder.Build();
}

void FRemoteSessionsManager::ReInitializeMessagingSystem(const TSharedPtr<IMessageBus>& InMessageBus)
{
	ShutdownMessagingSystem();
	InitializeMessagingSystem(InMessageBus);
}

bool FRemoteSessionsManager::CanPublishToNetwork(const FTopLevelAssetPath& MessageType) const
{
	// A standard bridge (e.g., UdpMessaging enabled via -Messaging flag) subscribes to
	// all message types and forwards every Network-scope Published message.
	if (IModularFeatures::Get().IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
	{
		return true;
	}

	// When an external message bridge is set we let it determine
	// if a specific message type can reach remote endpoints.
	if (CustomMessageBridge)
	{
		return CustomMessageBridge->IsSubscribed(MessageType);
	}

	return false;
}

void FRemoteSessionsManager::RegisterExternalSupportedMessageType(const UScriptStruct* ScriptStruct)
{
	if (ensure(ScriptStruct))
	{
		SupportedMessageTypes.Emplace(ScriptStruct);
		BroadcastSupportedMessageTypes();
	}
}

void FRemoteSessionsManager::BroadcastSupportedMessageTypes()
{
	SupportedMessageTypesChangedDelegate.Broadcast(SupportedMessageTypes);
}

void FRemoteSessionsManager::InitializeMessagingSystem(const TSharedPtr<IMessageBus>& InMessageBus)
{
	if (!InMessageBus)
	{
		return;
	}

	MessageBusPtr = InMessageBus;
	MessageEndpoint = CreateEndPoint(InMessageBus.ToSharedRef());

	if (!MessageEndpoint)
	{
		return;
	}

	// Subscribe built-in message types
	if (IsController())
	{
		MessageEndpoint->Subscribe<FSessionPong>();
		MessageEndpoint->Subscribe<FRecordingStatusMessage>();
		MessageEndpoint->Subscribe<FFullSessionInfoResponseMessage>();
		MessageEndpoint->Subscribe<FTraceConnectionDetailsMessage>();
	}

	MessageEndpoint->Subscribe<FSessionPing>();
	MessageEndpoint->Subscribe<FFullSessionInfoRequestMessage>();

	// Allow external owner to subscribe
	for (const TSharedRef<IRemoteSessionsHandler>& Handler : Handlers)
	{
		Handler->OnSubscribingMessageTypes(AsShared(), *MessageEndpoint);
	}

	if (bInitialized)
	{
		MessageEndpoint->Enable();
	}
	else
	{
		MessageEndpoint->Disable();
	}

	// Notify external bridge subscribers about the current message types
	BroadcastSupportedMessageTypes();

	// Notify that messaging initialization is done
	MessagingInitializedDelegate.Broadcast(InMessageBus, MessageEndpoint);
}

void FRemoteSessionsManager::ShutdownMessagingSystem()
{
	if (MessageEndpoint)
	{
		// Unsubscribe from all message types (i.e. no delegate required)
		MessageEndpoint->Unsubscribe();

		if (const TSharedPtr MessageBus = MessageBusPtr.Pin())
		{
			MessageBus->Unregister(MessageEndpoint->GetAddress());
		}
	}

	MessageBusPtr.Reset();
	MessageEndpoint = nullptr;
}

void FRemoteSessionsManager::Initialize()
{
#if !defined(WITH_TRACE_BASED_DEBUGGERS_EXTERNAL_MESSAGING) || !WITH_TRACE_BASED_DEBUGGERS_EXTERNAL_MESSAGING
	InitializeMessagingSystem(IMessagingModule::Get().GetDefaultBus());
#endif

	ActiveSessionsById.Add(AllRemoteSessionsWrapperGUID, CreatedWrapperSessionInfo(AllRemoteSessionsWrapperGUID, AllRemoteSessionsTargetName));
	ActiveSessionsById.Add(AllRemoteServersWrapperGUID, CreatedWrapperSessionInfo(AllRemoteServersWrapperGUID, AllRemoteServersTargetName));
	ActiveSessionsById.Add(AllRemoteClientsWrapperGUID, CreatedWrapperSessionInfo(AllRemoteClientsWrapperGUID, AllRemoteClientsTargetName));
	ActiveSessionsById.Add(AllSessionsWrapperGUID, CreatedWrapperSessionInfo(AllSessionsWrapperGUID, AllSessionsTargetName));
	ActiveSessionsById.Add(CustomSessionsWrapperGUID, CreatedWrapperSessionInfo(CustomSessionsWrapperGUID, CustomSessionsTargetName));

	if (MessageEndpoint)
	{
		MessageEndpoint->Enable();
	}

	bInitialized = true;
}

void FRemoteSessionsManager::Shutdown()
{
	bInitialized = false;
	ShutdownMessagingSystem();

	if (TickHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickHandle);
		TickHandle.Reset();
	}
}

void FRemoteSessionsManager::RegisterExternalHandler(TSharedRef<IRemoteSessionsHandler> Handler)
{
	Handlers.Emplace(Handler);
	if (const TSharedPtr<IMessageBus> MessageBus = MessageBusPtr.Pin())
	{
		ReInitializeMessagingSystem(MessageBus);
	}
}

void FRemoteSessionsManager::UnregisterExternalHandler(TSharedRef<IRemoteSessionsHandler> Handler)
{
	Handlers.Remove(Handler);
	if (const TSharedPtr<IMessageBus> MessageBus = MessageBusPtr.Pin())
	{
		ReInitializeMessagingSystem(MessageBus);
	}
}

void FRemoteSessionsManager::EnumerateMessageTypes(const FVisitorFunction& InVisitor)
{
	for (const UScriptStruct* MessageType : SupportedMessageTypes)
	{
		if (MessageType)
		{
			InVisitor(MessageType);
		}
	}
}

void FRemoteSessionsManager::StartSessionDiscovery(const FGuid& InDebuggerGuid)
{
	if (ActiveDebuggersUsingSessionDiscovery.IsEmpty())
	{
		ensureMsgf(!TickHandle.IsValid(), TEXT("Tick handle should never be valid when a debugger starts sessions discovery"));
		constexpr float TickInterval = 1.0f;
		TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FRemoteSessionsManager::Tick), TickInterval);
	}

	ActiveDebuggersUsingSessionDiscovery.Add(InDebuggerGuid);
}

void FRemoteSessionsManager::StopSessionDiscovery(const FGuid& InDebuggerGuid)
{
	const int32 NumRemoved = ActiveDebuggersUsingSessionDiscovery.RemoveSingle(InDebuggerGuid);

	UE_CLOGF(NumRemoved == 0, LogTraceBasedDebuggers, Warning, "[%s] Session discovery was not started by debugger %ls (or stopped more often than it was started", __func__, *InDebuggerGuid.ToString());

	if (ActiveDebuggersUsingSessionDiscovery.IsEmpty())
	{
		// Tick handle should be valid when removing the last debugger requesting sessions discovery,
		// but during shutdown it is possible that the CoreTicker gets reset before StopSessionDiscovery gets called.
		// In that case (i.e., engine shutdown), we don't have anything to do.
		if (!IsEngineExitRequested()
			&& ensureMsgf(TickHandle.IsValid(), TEXT("Tick handle should be valid when removing the last debugger requesting sessions discovery")))
		{
			RemoveExpiredSessions(ERemoveSessionOptions::ForceRemoveAll);
			FTSTicker::RemoveTicker(TickHandle);
			TickHandle.Reset();
		}
	}
}

void FRemoteSessionsManager::SendFullSessionStateRequestCommand(const FMessageAddress& InDestinationAddress)
{
	if (!MessageEndpoint)
	{
		UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed to send command | Invalid endpoint.", __FUNCTION__);
		return;
	}

	MessageEndpoint->Send(
	FMessageEndpoint::MakeMessage<FFullSessionInfoRequestMessage>(),
	FFullSessionInfoRequestMessage::StaticStruct(),
	EMessageFlags::Reliable,
	nullptr,
	TArrayBuilder<FMessageAddress>().Add(InDestinationAddress),
	FTimespan::Zero(),
	FDateTime::MaxValue());
}

bool FRemoteSessionsManager::Tick(float DeltaTime)
{
	if (IsController())
	{
		SendPing();
		RemoveExpiredSessions();
	}

	return true;
}

void FRemoteSessionsManager::SendPing()
{
	if (ensure(MessageEndpoint))
	{
		if (FSessionPing* SessionPingData = FMessageEndpoint::MakeMessage<FSessionPing>())
		{
			SessionPingData->ControllerInstanceId = FApp::GetInstanceId();
			MessageEndpoint->Publish(SessionPingData, EMessageScope::Network);
		}
	}
}

void FRemoteSessionsManager::SendPong(const FSessionPing& InMessage, const TSharedRef<IMessageContext>& InContext)
{
	if (!ensure(MessageEndpoint))
	{
		return;
	}

	if (FSessionPong* PongMessage = FMessageEndpoint::MakeMessage<FSessionPong>())
	{
		PongMessage->InstanceId = FApp::GetInstanceId();
		PongMessage->SessionId = FApp::GetSessionId();

		const EBuildTargetType TargetType = GetBuildTargetType();
		PongMessage->BuildTargetType = static_cast<uint8>(TargetType);

		FString AppSessionName = FApp::GetSessionName();
		PongMessage->SessionName = AppSessionName == TEXT("None") || AppSessionName.IsEmpty()
			? FString::Format(TEXT("{0} {1} {2}"),
					{FApp::GetProjectName()
					, FString(LexToString(TargetType))
					, FString::FromInt(FPlatformProcess::GetCurrentProcessId())})
			: AppSessionName;

		// Send answer back directly to the sender of the Ping message (i.e., no broadcast)
		MessageEndpoint->Send(PongMessage, InContext.Get().GetSender());
	}
}

void FRemoteSessionsManager::RegisterSessionInMultiSessionWrapper(const TSharedRef<FSessionInfo>& InSessionInfoRef)
{
	if (InSessionInfoRef->SessionName != LocalEditorSessionName)
	{
		StaticCastSharedPtr<FMultiSessionInfo>(ActiveSessionsById.FindChecked(AllRemoteSessionsWrapperGUID))->InnerSessionsByInstanceID.Emplace(InSessionInfoRef->InstanceId, InSessionInfoRef);

		if (InSessionInfoRef->BuildTargetType == EBuildTargetType::Server)
		{
			StaticCastSharedPtr<FMultiSessionInfo>(ActiveSessionsById.FindChecked(AllRemoteServersWrapperGUID))->InnerSessionsByInstanceID.Emplace(InSessionInfoRef->InstanceId, InSessionInfoRef);
		}
		else
		{
			StaticCastSharedPtr<FMultiSessionInfo>(ActiveSessionsById.FindChecked(AllRemoteClientsWrapperGUID))->InnerSessionsByInstanceID.Emplace(InSessionInfoRef->InstanceId, InSessionInfoRef);
		}
	}

	StaticCastSharedPtr<FMultiSessionInfo>(ActiveSessionsById.FindChecked(AllSessionsWrapperGUID))->InnerSessionsByInstanceID.Emplace(InSessionInfoRef->InstanceId, InSessionInfoRef);
}

void FRemoteSessionsManager::DeRegisterSessionInMultiSessionWrapper(const TSharedRef<FSessionInfo>& InSessionInfoRef)
{
	StaticCastSharedPtr<FMultiSessionInfo>(ActiveSessionsById.FindChecked(AllRemoteSessionsWrapperGUID))->InnerSessionsByInstanceID.Remove(InSessionInfoRef->InstanceId);
	StaticCastSharedPtr<FMultiSessionInfo>(ActiveSessionsById.FindChecked(AllSessionsWrapperGUID))->InnerSessionsByInstanceID.Remove(InSessionInfoRef->InstanceId);
	StaticCastSharedPtr<FMultiSessionInfo>(ActiveSessionsById.FindChecked(AllRemoteServersWrapperGUID))->InnerSessionsByInstanceID.Remove(InSessionInfoRef->InstanceId);
	StaticCastSharedPtr<FMultiSessionInfo>(ActiveSessionsById.FindChecked(AllRemoteClientsWrapperGUID))->InnerSessionsByInstanceID.Remove(InSessionInfoRef->InstanceId);
}

void FRemoteSessionsManager::ProcessPendingMessagesForSession(const FSessionPong& InMessage, const TSharedRef<FSessionInfo>& InSessionInfoPtr)
{
	PendingRecordingStatusMessages.RemoveAndCopyValue(InMessage.InstanceId, InSessionInfoPtr->LastKnownRecordingState);
	PendingRecordingConnectionDetailsMessages.RemoveAndCopyValue(InMessage.InstanceId, InSessionInfoPtr->LastKnownConnectionDetails);
}

void FRemoteSessionsManager::HandleSessionPongMessage(const FSessionPong& InMessage, const TSharedRef<IMessageContext>& InContext)
{
	TSharedPtr<FSessionInfo>& SessionInfoPtr = ActiveSessionsById.FindOrAdd(InMessage.InstanceId);

	if (!SessionInfoPtr)
	{
		SessionInfoPtr = MakeShared<FSessionInfo>();
		SessionInfoPtr->Address = InContext->GetSender();
		SessionInfoPtr->InstanceId = InMessage.InstanceId;
		SessionInfoPtr->SessionName = InMessage.SessionName;

		// We received the message from our own process, so let's use the user 
		// friendly name for that session (i.e., Local Editor)
		if (InMessage.InstanceId == LocalEditorSessionID)
		{
			SessionInfoPtr->SessionName = LocalEditorSessionName;
		}
		SessionInfoPtr->BuildTargetType = static_cast<EBuildTargetType>(InMessage.BuildTargetType);

		RegisterSessionInMultiSessionWrapper(SessionInfoPtr.ToSharedRef());

		SessionDiscoveredDelegate.Broadcast(SessionInfoPtr);

		// This is the first time we see this session, so we need to request the rest of its state so we can properly populate the UI
		SendFullSessionStateRequestCommand(SessionInfoPtr->Address);
	}

	SessionInfoPtr->LastPingTime = FDateTime::UtcNow();

	ProcessPendingMessagesForSession(InMessage, SessionInfoPtr.ToSharedRef());

	SessionsUpdatedDelegate.Broadcast();
}

void FRemoteSessionsManager::HandleSessionPingMessage(const FSessionPing& InMessage, const TSharedRef<IMessageContext>& InContext)
{
	// If this instance is not running trace, don't answer to the ping as the debugger instance will not be able to do anything useful with it
#if !TRACE_BASED_DEBUGGERS_WITHOUT_TRACE
	// Only respond if we can actually publish messages back to the controller.
	// Without a network bridge, status messages (FRecordingStatusMessage, etc.) won't
	// reach the controller, resulting in a non-functional ghost session.
	// or in same-process scenarios without a network bridge (e.g., PIE) where all messages are delivered locally on
	// the same bus and no bridge is needed
	if (CanPublishToNetwork<FSessionPong>()
		|| InMessage.ControllerInstanceId == FApp::GetInstanceId())
	{
		SendPong(InMessage, InContext);
	}
#endif
}

void FRemoteSessionsManager::HandleRecordingStatusUpdateMessage(const FRecordingStatusMessage& Message, const TSharedRef<IMessageContext>& InContext)
{
	if (TSharedPtr<FSessionInfo>* SessionInfoPtrPtr = ActiveSessionsById.Find(Message.InstanceId))
	{
		TSharedPtr<FSessionInfo>& SessionInfoPtr = *SessionInfoPtrPtr;
		check(SessionInfoPtr);

		const FGuid& LastKnownDebuggerTypeId = SessionInfoPtr->LastKnownRecordingState.DebuggerId;
		const FGuid LastKnownRecordingRequestedId = SessionInfoPtr->LastKnownRecordingState.RequesterId;
		if (LastKnownDebuggerTypeId.IsValid()
			&& LastKnownDebuggerTypeId != Message.DebuggerId
			&& LastKnownRecordingRequestedId.IsValid())
		{
			ensureMsgf(false, TEXT("A trace-based debugger changed is recording status while another debugger is already recording."
				" This is not supported and indicates a problem on the runtime modules side."));
			return;
		}

		// Update recording state before broadcasting so listeners can use the information to only
		// process request from their debugger Id.
		SessionInfoPtr->LastKnownRecordingState = Message;

		// Only process message if we (current application instance) instigated the recording
		if (LastKnownRecordingRequestedId == LocalEditorSessionID
			|| Message.RequesterId == LocalEditorSessionID)
		{
			if (LastKnownRecordingRequestedId != Message.RequesterId)
			{
				if (Message.RequesterId.IsValid())
				{
					RecordingStartedDelegate.Broadcast(SessionInfoPtr);
				}
				else
				{
					RecordingStoppedDelegate.Broadcast(SessionInfoPtr);
				}
			}
		}
	}
	else
	{
		PendingRecordingStatusMessages.FindOrAdd(Message.InstanceId) = Message;
	}
}

void FRemoteSessionsManager::HandleConnectionDetailsUpdateMessage(const FTraceConnectionDetailsMessage& InMessage, const TSharedRef<IMessageContext>& InContext)
{
	if (TSharedPtr<FSessionInfo>* SessionInfoPtrPtr = ActiveSessionsById.Find(InMessage.InstanceId))
	{
		const TSharedPtr<FSessionInfo>& SessionInfoPtr = *SessionInfoPtrPtr;
		check(SessionInfoPtr);

		SessionInfoPtr->LastKnownConnectionDetails = InMessage.TraceDetails;
	}
	else
	{
		PendingRecordingConnectionDetailsMessages.FindOrAdd(InMessage.InstanceId) = InMessage.TraceDetails;
	}
}

void FRemoteSessionsManager::HandleFullSessionStateRequestMessage(const FFullSessionInfoRequestMessage& InMessage, const TSharedRef<IMessageContext>& InContext)
{
	if (!ensure(MessageEndpoint))
	{
		return;
	}

	if (FFullSessionInfoResponseMessage* FullSessionStateResponse = FMessageEndpoint::MakeMessage<FFullSessionInfoResponseMessage>())
	{
		FullSessionStateResponse->InstanceId = FApp::GetInstanceId();

		// Allow external handlers to append information
		for (const TSharedRef<IRemoteSessionsHandler>& Handler : Handlers)
		{
			Handler->OnBuildingFullSessionInfoResponseMessage(*FullSessionStateResponse);
		}

		MessageEndpoint->Send(
			FullSessionStateResponse,
			FFullSessionInfoResponseMessage::StaticStruct(),
			EMessageFlags::Reliable,
			nullptr,
			TArrayBuilder<FMessageAddress>().Add(InContext->GetSender()),
			FTimespan::Zero(),
			FDateTime::MaxValue());
	}
}

void FRemoteSessionsManager::HandleFullSessionStateResponseMessage(const FFullSessionInfoResponseMessage& InMessage, const TSharedRef<IMessageContext>& InContext)
{
	if (const TSharedPtr<FSessionInfo>* SessionInfoPtr = ActiveSessionsById.Find(InMessage.InstanceId))
	{
		if (const TSharedPtr<FSessionInfo>& SessionInfo = *SessionInfoPtr)
		{
			SessionInfo->LastKnownRecordingState.RequesterId = InMessage.RecordingRequesterId;
			SessionInfo->LastKnownRecordingState.DebuggerId = InMessage.DebuggerId;

			// Allow external handlers to process the message
			for (const TSharedRef<IRemoteSessionsHandler>& Handler : Handlers)
			{
				Handler->OnHandlingFullSessionInfoResponseMessage(InMessage, SessionInfo);
			}
		}
	}
}

void FRemoteSessionsManager::RemoveExpiredSessions(const ERemoveSessionOptions Options)
{
	if (bIgnoreExpiredSessionsForDebugging
		&& !EnumHasAnyFlags(Options, ERemoveSessionOptions::ForceRemoveAll))
	{
		return;
	}

	bool bAnySessionRemoved = false;
	const FDateTime CurrentTime = FDateTime::UtcNow();
	for (TMap<FGuid, TSharedPtr<FSessionInfo>>::TIterator RemoveIterator = ActiveSessionsById.CreateIterator(); RemoveIterator; ++RemoveIterator)
	{
		TSharedPtr<FSessionInfo>& SessionInfoPtr = RemoveIterator.Value();
		if (!SessionInfoPtr)
		{
			RemoveIterator.RemoveCurrent();
			bAnySessionRemoved = true;
			continue;
		}

		if (!EnumHasAnyFlags(SessionInfoPtr->GetSessionTypeAttributes(), ERemoteSessionAttributes::CanExpire))
		{
			continue;
		}

		const FTimespan ElapsedTime = CurrentTime - SessionInfoPtr->LastPingTime;

		// A session goes into busy state if we are attempting to issue a command that might stall the target, currently that only happens on recording start commands of complex maps
		// In these cases, we need to allow more time between pings. If a recording command failed, it is expected the state to be changed to Ready again
		const float MaxAllowedTimeBetweenPings = SessionInfoPtr->ReadyState == ERemoteSessionReadyState::Busy || SessionInfoPtr->IsAnyDebuggerRecording() ? 60.0f : 3.0f;
		if (EnumHasAnyFlags(Options, ERemoveSessionOptions::ForceRemoveAll) || ElapsedTime > FTimespan::FromSeconds(MaxAllowedTimeBetweenPings))
		{
			SessionExpiredDelegate.Broadcast(SessionInfoPtr);

			DeRegisterSessionInMultiSessionWrapper(SessionInfoPtr.ToSharedRef());
			RemoveIterator.RemoveCurrent();
			bAnySessionRemoved = true;
		}
	}

	if (bAnySessionRemoved)
	{
		SessionsUpdatedDelegate.Broadcast();
	}
}
} // namespace UE::TraceBasedDebuggers