// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceBasedDebuggerRuntime.h"

#if WITH_TRACE_BASED_DEBUGGERS

#include "Misc/CommandLine.h"
#include "RelayTraceDataWriter.h"
#include "RemoteSessionsManager.h"
#include "Math/GuardedInt.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"

#if WITH_EDITOR
#include "Misc/ScopedSlowTask.h"
#include "Misc/MessageDialog.h"
#endif

#define LOCTEXT_NAMESPACE "TraceBasedDebuggerRuntime"

namespace UE::TraceBasedDebuggers
{
static FAutoConsoleVariable CVarMaxTimeToWaitForDisconnect(
	TEXT("tracebaseddebuggers.MaxTimeToWaitForDisconnectSeconds"),
	5.0f,
	TEXT("Max time to wait after attempting to stop an active trace session. After that time has passed if we are still connected, debuggers will continue and eventually error out."));

static FAutoConsoleVariable CVarMaxBytesPerRelayChunk(
	TEXT("tracebaseddebuggers.MaxBytesPerRelayChunk"),
	65536,
	TEXT("How many bytes per relay data message we should send"));

static FAutoConsoleVariable CVarMaxPendingBytesInRelayWriter(
	TEXT("tracebaseddebuggers.MaxPendingBytesInRelayWriter"),
	536870912,
	TEXT("How many bytes the relay writer can hold before overflowing, triggering a close. Defaults to ~512MB"));

FRuntimeModule::FRuntimeModule(const FLogCategoryAlias& InLogCategory, const FGuid& InDebuggerTypeID)
	: LogCategory(InLogCategory)
	, DebuggerGuid(InDebuggerTypeID)
{
}

FRuntimeModule::~FRuntimeModule()
{
}

bool FRuntimeModule::TryRecordOnStartup()
{
	if (AutoRecordCommand.IsEmpty()
		|| !FParse::Param(FCommandLine::Get(), *AutoRecordCommand))
	{
		return false;
	}

	TArray<FString, TInlineAllocator<1>> Options;
	FString HostAddress;
	if (FParse::Value(FCommandLine::Get(), *HostAddressCommand, HostAddress))
	{
		Options.Emplace(TEXT("Server"));
		Options.Emplace(MoveTemp(HostAddress));
	}

	StartRecording(Options);
	return true;
}

void FRuntimeModule::StartRecording(const TConstArrayView<FString> Args)
{
	using namespace UE::TraceBasedDebuggers;
	FStartRecordingCommandMessage StartRecordingCommand;
	StartRecordingCommand.InstanceId = FApp::GetInstanceId();

	// Default (i.e., no arguments) or "File" trace to file using GenerateRecordingFileName
	if (Args.Num() == 0 || Args[0] == TEXT("File"))
	{
		StartRecordingCommand.RecordingMode = ERecordingMode::File;
		StartRecordingCommand.TransportMode = ETraceTransportMode::FileSystem;
	}
	// "Server [ADDRESS]": trace server using either the optional [ADDRESS] or local host (i.e., "127.0.0.1")
	else if (Args[0] == TEXT("Server"))
	{
		StartRecordingCommand.RecordingMode = ERecordingMode::Live;
		StartRecordingCommand.TransportMode = ETraceTransportMode::TraceServer;
		StartRecordingCommand.Target = Args.IsValidIndex(1) ? Args[1] : TEXT("127.0.0.1");
	}
	// "-tracefile [FILENAME]": trace to file using either the optional [FILENAME] or GenerateRecordingFileName
	// "-tracehost [ADDRESS]": trace server using either the optional [ADDRESS] or local host (i.e., "127.0.0.1")
	else
	{
		FTraceAuxiliary::EConnectionType TraceType = FTraceAuxiliary::EConnectionType::None;
		FString TraceDestination;

		for (const FString& Arg : Args)
		{
			if (Arg.StartsWith(TEXT("-tracefile"), ESearchCase::IgnoreCase))
			{
				ensureMsgf(TraceType == FTraceAuxiliary::EConnectionType::None
					, TEXT("StartRecording: Specifying more than 1 trace destination is not supported. Received: %s")
					, *FString::Join(Args, TEXT(" ")));

				TraceType = FTraceAuxiliary::EConnectionType::File;

				// Try to extract filename.
				if (FParse::Value(*Arg, TEXT("-tracefile="), TraceDestination))
				{
					// Make sure it's a valid filename
					FText FilenameError;
					if (!FFileHelper::IsFilenameValidForSaving(TraceDestination, FilenameError))
					{
						ensureMsgf(false, TEXT("StartRecording: Specified filename is not supported: %s"), *FilenameError.ToString());
						TraceDestination = "";
					}
				}
			}
			else if (Arg.StartsWith(TEXT("-tracehost"), ESearchCase::IgnoreCase))
			{
				ensureMsgf(TraceType == FTraceAuxiliary::EConnectionType::None
					, TEXT("StartRecording: Specifying more than 1 trace destination is not supported. Received: %s")
					, *FString::Join(Args, TEXT(" ")));

				TraceType = FTraceAuxiliary::EConnectionType::Network;

				if (FParse::Value(*Arg, TEXT("-tracehost="), TraceDestination))
				{
					// Should we validate that TraceDestination is valid ip address?
				}
			}
			else
			{
				ensureMsgf(false, TEXT("StartRecording: Received unknown argument: %s"), *Arg);
			}
		}

		switch (TraceType)
		{
		case FTraceAuxiliary::EConnectionType::Network:
			StartRecordingCommand.RecordingMode = ERecordingMode::Live;
			StartRecordingCommand.TransportMode = ETraceTransportMode::TraceServer;
			StartRecordingCommand.Target = TraceDestination;
			break;
		case FTraceAuxiliary::EConnectionType::File:
			StartRecordingCommand.RecordingMode = ERecordingMode::File;
			StartRecordingCommand.TransportMode = ETraceTransportMode::FileSystem;
			StartRecordingCommand.Target = TraceDestination;
			break;
		}
	}

	StartRecording(StartRecordingCommand);
}

void FRuntimeModule::StartRecording(const FStartRecordingCommandMessage& InRecordingStartCommand)
{
	using namespace UE::TraceBasedDebuggers;

	// Reset the channels-enabled event so WaitForTraceChannelsEnabled will block
	// until HandleTraceConnectionEstablished completes
	TraceChannelsEnabledEvent->Reset();

	if (!ensureAlwaysMsgf(!IsRecording(), TEXT("Received a recording start command while a recording is already active")))
	{
		UE_LOG_REF(LogCategory, Log, TEXT("[%hs] There is an active recording. Attempting to stop it..."), __func__)
		StopRecording();
	}

	// Start with a generic Failure reason
	FText FailureReason = LOCTEXT("SeeLogsForErrorDetailsText", "Please see the logs for more details...");

	bool bIsRecording = false;

#if UE_TRACE_ENABLED || UE_TRACE_MINIMAL_ENABLED

	// Other tools could be using trace
	// This is aggressive but until Trace supports multi-sessions, just take over.
	if (FTraceAuxiliary::IsConnected())
	{
		UE_LOG_REF(LogCategory, Log, TEXT("[%hs] There is an active trace session. attempting to disconnect..."), __func__);

		//TODO: We should make the wait async like we do whe we attempt to connect to a live session
		if (FTraceAuxiliary::Stop() && WaitForTraceSessionDisconnect())
		{
			UE_LOG_REF(LogCategory, Log, TEXT("[%hs] Successful disconnect attempt!."), __func__);
		}
		else
		{
			FailureReason = LOCTEXT("FailedToStopActiveRecordingErrorMessage", "Failed to Stop active Trace Session.");
		}
	}

	// Disable current channels when beginning the "start recording" action, then wait
	// for the connection to be established to enable the required channels (i.e., HandleTraceConnectionEstablished)
	SaveAndDisabledCurrentEnabledTraceChannels();

	// Allow derived classes to perform additional actions before we start the traces
	OnRecordingStartingInternal();

	FTraceAuxiliary::FOptions TracingOptions;
	TracingOptions.bExcludeTail = true;

	if (InRecordingStartCommand.RecordingMode == ERecordingMode::File)
	{
		if (InRecordingStartCommand.TransportMode == ETraceTransportMode::FileSystem)
		{
			FString FileTarget = InRecordingStartCommand.Target;
			if (FileTarget.IsEmpty())
			{
				GenerateRecordingFileName(FileTarget);
			}

			UE_LOG_REF(LogCategory, Log, TEXT("[%hs] Generated trace file name [%s]"), __func__, *FileTarget);

			bIsRecording = FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::File, *FileTarget, nullptr, &TracingOptions, LogCategory);
		}
		else
		{
			UE_LOG_REF(LogCategory, Error, TEXT("[%hs] Unsupported Transport mode [%s]"), __func__, *UEnum::GetValueAsString(InRecordingStartCommand.TransportMode))
		}
	}
	else if (InRecordingStartCommand.RecordingMode == ERecordingMode::Live)
	{
		switch (InRecordingStartCommand.TransportMode)
		{
		case ETraceTransportMode::Direct:
		case ETraceTransportMode::TraceServer:
			bIsRecording = FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::Network, *InRecordingStartCommand.Target, nullptr, &TracingOptions);
			break;
		case ETraceTransportMode::Relay:
		{
			if (!ensureMsgf(ExternalRelayExecutorDelegate.IsBound(), TEXT("Cannot start a trace in relay mode without a Relay Executor")))
			{
				UE_LOG_REF(LogCategory, Error, TEXT("[%hs] Cannot start a trace in relay mode without a Relay Executor"), __func__);
				break;
			}

			if (RelayWriter && !RelayWriter->IsClosed())
			{
				UE_LOG_REF(LogCategory, Warning, TEXT("[%hs] Attempting to relay trace, but there is a relay writer still open"), __func__);
				RelayWriter->Close();
			}

			RelayWriter = MakeUnique<FRelayTraceDataWriter>(LogCategory);

			constexpr int32 MinBunchSize = 4096;
			RelayWriter->SetMaxBytesPerBunch(FMath::Clamp(CVarMaxBytesPerRelayChunk->GetInt(), MinBunchSize, std::numeric_limits<int32>::max()));

			constexpr int32 MinBufferSizeLimit = 2097152;
			RelayWriter->SetMaxAllowedPendingBytes(FMath::Clamp(CVarMaxPendingBytesInRelayWriter->GetInt(), MinBufferSizeLimit, std::numeric_limits<int32>::max()));

			bIsRecording = FTraceAuxiliary::Relay(reinterpret_cast<UPTRINT>(RelayWriter.Get()), FRelayTraceDataWriter::WriteHelper, FRelayTraceDataWriter::CloseHelper, nullptr, &TracingOptions);

			if (bIsRecording)
			{
				SetupRelayDataPumpDelegates();
			}

			break;
		}
		case ETraceTransportMode::Invalid:
		case ETraceTransportMode::FileSystem:
			UE_LOG_REF(LogCategory, Error, TEXT("[%hs] Unsupported Transport mode [%s]"), __func__, *UEnum::GetValueAsString(InRecordingStartCommand.TransportMode));
			break;
		}
	}
	else
	{
		FailureReason = LOCTEXT("WrongCommandArgumentsError", "The start recording command was called with invalid arguments");
	}
#endif

	AccumulatedRecordingTime = 0.0f;

	if (ensureMsgf(bIsRecording, TEXT("Failed to start recording | Reason [%s]"), *FailureReason.ToString()))
	{
		CurrentTransportMode = InRecordingStartCommand.TransportMode;

		if (ensureMsgf(!OnConnectionDelegateHandle.IsValid(), TEXT("Starting a trace while we are waiting for a pending connection")))
		{
			OnConnectionDelegateHandle = FTraceAuxiliary::OnConnection.AddRaw(this, &FRuntimeModule::HandleTraceConnectionEstablished);
		}

		// Start Listening for Trace Stopped events, in case Trace is stopped outside our control so we can gracefully stop recording and log a warning
		if (ensureMsgf(!OnTraceStoppedDelegateHandle.IsValid(), TEXT("Starting a trace while we are waiting for a trace session to stop.")))
		{
			OnTraceStoppedDelegateHandle = FTraceAuxiliary::OnTraceStopped.AddRaw(this, &FRuntimeModule::HandleTraceStopRequest);
		}
	}
	else
	{
		UE_LOG_REF(LogCategory, Error, TEXT("[%hs] Failed to start recording | Reason: [%s]"), __func__, *FailureReason.ToString());

#if WITH_EDITOR
		FMessageDialog::Open(EAppMsgType::Ok, FText::FormatOrdered(LOCTEXT("StartRecordingFailedMessage", "Failed to start recording. \n\n{0}"), FailureReason));
#endif

		RelayWriter = nullptr;

		// Allow derived classes to perform additional actions on failure
		OnRecordingStartFailedInternal(FailureReason);
	}

	if (bIsRecording)
	{
		if (ensureMsgf(InRecordingStartCommand.InstanceId.IsValid()
			, TEXT("FStartRecordingCommandMessage sent without a valid InstanceId, using local instance Id as a fallback."
			" Make sure all code paths properly set 'InstanceId'.")))
		{
			ActiveRecordingRequesterId = InRecordingStartCommand.InstanceId;
		}
		else
		{
			ActiveRecordingRequesterId = FApp::GetInstanceId();
		}
	}
	else
	{
		ActiveRecordingRequesterId = FGuid{};
	}
}

void FRuntimeModule::StopRecording()
{
	FTraceAuxiliary::OnConnection.RemoveAll(this);
	OnConnectionDelegateHandle = FDelegateHandle();

	FTraceAuxiliary::OnTraceStopped.RemoveAll(this);
	OnTraceStoppedDelegateHandle = FDelegateHandle();

	// Reset the channels-enabled event since we are stopping
	TraceChannelsEnabledEvent->Reset();

	CurrentTransportMode = ETraceTransportMode::Invalid;

	if (!IsRecording())
	{
		UE_LOG_REF(LogCategory, Warning, TEXT("[%hs] Attempted to stop recording when there is no recording active."), __func__);
		return;
	}

#if UE_TRACE_ENABLED || UE_TRACE_MINIMAL_ENABLED
	RestoreTraceChannelsToPreRecordingState();

	StopTrace();
#endif

	if (RecordingTimerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(RecordingTimerHandle);
		RecordingTimerHandle.Reset();
	}

	// Close the relay writer before clearing the data pump delegates so the final
	// incomplete bunch flush in Close() can still trigger the relay transport via
	// NewDataAvailableDelegate. Without this ordering, data accumulated in the
	// current bunch that never reached the bunch-size threshold is silently lost.
	if (RelayWriter)
	{
		RelayWriter->Close();
	}

	ClearRelayDataPumpDelegates();

	ActiveRecordingRequesterId = FGuid{};
	AccumulatedRecordingTime = 0.0f;

	// Allow derived classes to perform additional actions
	OnRecordingStoppedInternal();
}


int64 FRuntimeModule::GetBufferedDataBytesSize() const
{
	FGuardedInt64 RelaySendBufferSize(0);

	if (RelayWriter)
	{
		RelaySendBufferSize += RelayWriter->GetQueuedBytesNum();
	}

	if (ExternalRelayBufferSizeProviderDelegate.IsBound())
	{
		RelaySendBufferSize += ExternalRelayBufferSizeProviderDelegate.Execute();
	}

	constexpr int64 DefaultValue = std::numeric_limits<int64>::max();
	return RelaySendBufferSize.Get(DefaultValue);
}

FTraceConnectionDetails FRuntimeModule::GetCurrentTraceConnectionDetails() const
{
	FTraceConnectionDetails Details;
	if (ExternalTraceStatusDelegate.IsBound())
	{
		return ExternalTraceStatusDelegate.Execute();
	}

	if (FTraceAuxiliary::IsConnected(Details.SessionGuid, Details.TraceGuid))
	{
		Details.TraceTarget = FTraceAuxiliary::GetTraceDestinationString();
		Details.TransportMode = CurrentTransportMode;
		Details.MarkAsValid();
		return Details;
	}

	return Details;
}

void FRuntimeModule::StopTrace()
{
	bRequestedStop = true;
	FTraceAuxiliary::Stop();
}

void FRuntimeModule::GenerateRecordingFileName(FString& OutFileName) const
{
	const FStringFormatOrderedArguments NameArgs
	{
		DebuggerName,
		FString(FApp::GetProjectName())
		, LexToString(GetBuildTargetType())
		, FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S%s"))
	};

	OutFileName = FString::Format(TEXT("{0}-{1}-{2}-{3}.utrace"), NameArgs);
}

void FRuntimeModule::RegisterExternalRelayExecutor(const FRelayTraceDataDelegate& InExternalRelayExecutor)
{
	if (!ensureMsgf(!ExternalRelayExecutorDelegate.IsBound(), TEXT("An external relay executor is already registered!")))
	{
		return;
	}

	ExternalRelayExecutorDelegate = InExternalRelayExecutor;
}

void FRuntimeModule::RegisterExternalRelayOverflowHandler(const FSimpleDelegate& InExternalRelayOverflowHandler)
{
	if (!ensureMsgf(!ExternalRelayOverflowHandlerDelegate.IsBound(), TEXT("An external relay overflow handler is already registered!")))
	{
		return;
	}

	ExternalRelayOverflowHandlerDelegate = InExternalRelayOverflowHandler;
}

void FRuntimeModule::UnregisterExternalRelayOverflowHandler()
{
	ExternalRelayOverflowHandlerDelegate.Unbind();
}

void FRuntimeModule::UnregisterCurrentExternalRelayExecutor()
{
	ExternalRelayExecutorDelegate.Unbind();
}

void FRuntimeModule::RegisterExternalTraceStatusProvider(const FExternalTraceConnectionStatusDelegate& InExternalTraceStatusCallback)
{
	ExternalTraceStatusDelegate = InExternalTraceStatusCallback;
}

void FRuntimeModule::UnregisterExternalTraceStatusProvider()
{
	ExternalTraceStatusDelegate.Unbind();
}

void FRuntimeModule::RegisterExternalRelayBufferSizeProvider(const FExternalRelayBufferQueueSizeDelegate& InExternalRelayBufferSizeCallback)
{
	if (!ensureMsgf(!ExternalRelayBufferSizeProviderDelegate.IsBound(), TEXT("An external relay InExternalRelayBufferSizeCallback is already registered!")))
	{
		return;
	}

	ExternalRelayBufferSizeProviderDelegate = InExternalRelayBufferSizeCallback;
}

void FRuntimeModule::UnregisterExternalRelayBufferSizeProvider()
{
	ExternalRelayBufferSizeProviderDelegate.Unbind();
}

void FRuntimeModule::HandleRelayOverflow()
{
	if (ExternalRelayOverflowHandlerDelegate.IsBound())
	{
		UE_LOG_REF(LogCategory, Error, TEXT("[%hs] Relay writer has overflowed. Executing external overflow handler... "), __func__);
		ExternalRelayOverflowHandlerDelegate.Execute();
		return;
	}

	UE_LOG_REF(LogCategory, Error, TEXT("[%hs] Relay writer has overflowed. Stopping Recording... "), __func__);
	StopRecording();
}

void FRuntimeModule::RelayTraceData() const
{
	if (ensure(ExternalRelayExecutorDelegate.IsBound() && RelayWriter.IsValid()))
	{
		ExternalRelayExecutorDelegate.Execute(*RelayWriter);
	}
}

void FRuntimeModule::SetupRelayDataPumpDelegates()
{
	if (!ensure(RelayWriter))
	{
		return;
	}

	RelayWriter->OnNewDataAvailable().BindRaw(this, &FRuntimeModule::RelayTraceData);
	RelayWriter->OnDataOverflow().BindRaw(this, &FRuntimeModule::HandleRelayOverflow);
}

void FRuntimeModule::ClearRelayDataPumpDelegates()
{
	if (RelayWriter)
	{
		RelayWriter->OnNewDataAvailable().Unbind();
		RelayWriter->OnDataOverflow().Unbind();
	}
}

bool FRuntimeModule::RecordingTimerTick(float DeltaTime)
{
	if (IsRecording())
	{
		AccumulatedRecordingTime += DeltaTime;
	}

	return true;
}

void FRuntimeModule::HandleTraceConnectionEstablished()
{
	UE_LOG_REF(LogCategory, Log, TEXT("Trace connection established."));

	if (OnConnectionDelegateHandle.IsValid())
	{
		FTraceAuxiliary::OnConnection.RemoveAll(this);
		OnConnectionDelegateHandle.Reset();
	}

	if (!ensureAlwaysMsgf(IsRecording(), TEXT("Received a trace connection established callback but no trace is active. This should not happen.")))
	{
		return;
	}

	EnableRequiredTraceChannels();

	// Allow derived classes to perform additional actions
	OnRecordingStartedInternal();

	TraceConnectionDetailsUpdatedDelegate.Broadcast();

	// Signal that trace channels are now enabled so any thread waiting
	// in WaitForTraceChannelsEnabled can proceed
	TraceChannelsEnabledEvent->Trigger();

	// Add ticker to keep track of accumulated recording time
	RecordingTimerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FRuntimeModule::RecordingTimerTick));
}

void FRuntimeModule::HandleTraceStopRequest(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination)
{
	if (IsRecording())
	{
		if (!ensure(bRequestedStop))
		{
			UE_LOG_REF(LogCategory, Warning, TEXT("Trace Recording has been stopped unexpectedly"));

#if WITH_EDITOR
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("UnexpectedStopMessage", "Trace recording has been stopped unexpectedly. Debugger cannot continue with the recording session... "));
#endif
		}

		StopRecording();
	}

	bRequestedStop = false;
}

bool FRuntimeModule::WaitForTraceSessionDisconnect()
{
	float MaxWaitTime = CVarMaxTimeToWaitForDisconnect->GetFloat();
	float CurrentWaitTime = 0.0f;

#if WITH_EDITOR
	FScopedSlowTask DisconnectAttemptSlowTask(MaxWaitTime, LOCTEXT("DisconnectAttemptMessage", " Active Trace Session detected, attempting to disconnect ..."));

	constexpr bool bShowCancelButton = false;
	constexpr bool bAllowInPIE = true;
	DisconnectAttemptSlowTask.MakeDialog(bShowCancelButton, bAllowInPIE);
#endif

	while (CurrentWaitTime < MaxWaitTime)
	{
		constexpr float WaitInterval = 0.1f;
		FPlatformProcess::Sleep(0.1f);

		if (!FTraceAuxiliary::IsConnected())
		{
			return true;
		}

		// We don't need to be precise for this, we can just accumulate the wait
		CurrentWaitTime += WaitInterval;

#if WITH_EDITOR
		DisconnectAttemptSlowTask.EnterProgressFrame(CurrentWaitTime);
#endif
	}

	return !FTraceAuxiliary::IsConnected();
}

bool FRuntimeModule::WaitForTraceChannelsEnabled(float MaxWaitTimeSeconds)
{
	// Fast path: channels already enabled
	if (TraceChannelsEnabledEvent->Wait(0))
	{
		return true;
	}

	UE_LOG_REF(LogCategory, Log, TEXT("[%hs] Waiting for trace channels to be enabled..."), __func__);

#if WITH_EDITOR
	FScopedSlowTask WaitForChannelsSlowTask(MaxWaitTimeSeconds,
		LOCTEXT("WaitForTraceChannelsMessage", "Waiting for trace channels to be enabled..."));
	WaitForChannelsSlowTask.MakeDialog(/*bShowCancelButton*/true, /*bAllowInPIE*/true);
#endif

	float CurrentWaitTime = 0.0f;
	while (CurrentWaitTime < MaxWaitTimeSeconds)
	{
		constexpr float WaitIntervalSeconds = 0.1f;
		constexpr uint32 WaitIntervalMs = 100;

		if (TraceChannelsEnabledEvent->Wait(WaitIntervalMs))
		{
			UE_LOG_REF(LogCategory, Log, TEXT("[%hs] Trace channels enabled after %.1f seconds"), __func__, CurrentWaitTime);
			return true;
		}

		CurrentWaitTime += WaitIntervalSeconds;

#if WITH_EDITOR
		WaitForChannelsSlowTask.EnterProgressFrame(WaitIntervalSeconds);
		if (WaitForChannelsSlowTask.ShouldCancel())
		{
			UE_LOG_REF(LogCategory, Log, TEXT("[%hs] Cancelled waiting for trace channels to be enabled per user request after %.1f seconds"), __func__, CurrentWaitTime);
			return false;
		}
#endif
	}

	// There's a small race window where the event could have been triggered between the last Wait(WaitIntervalMs) timing out and the loop condition re-evaluating.
	// Let's do one last instantaneous check in case the event fired just after our last poll.
	// Without it, we could get a false negative where the channels ARE enabled but the function returns false because the timing of the last poll just barely missed the trigger.
	const bool bResult = TraceChannelsEnabledEvent->Wait(0);
	if (!bResult)
	{
		UE_LOG_REF(LogCategory, Warning, TEXT("[%hs] Timed out waiting for trace channels to be enabled after %.1f seconds"), __func__, MaxWaitTimeSeconds);
	}

	return bResult;
}

void FRuntimeModule::SaveAndDisabledCurrentEnabledTraceChannels()
{
	// Until we support allowing other channels, indicate in the logs that we are disabling everything else
	UE_LOG_REF(LogCategory, Log, TEXT("[%hs] Disabling additional trace channels..."), __func__);

#if UE_TRACE_ENABLED || UE_TRACE_MINIMAL_ENABLED
	OriginalTraceChannelsState.Reset();

	// Disable any enabled additional channel
	Trace::EnumerateChannels([](const ANSICHAR* ChannelName, bool bEnabled, void* SavedTraceChannelsPtr)
		{
			TMap<FString, bool>* SavedTraceChannels = static_cast<TMap<FString, bool>*>(SavedTraceChannelsPtr);
			FString ChannelNameFString(ChannelName);
			SavedTraceChannels->Add(ChannelNameFString, bEnabled);
			if (bEnabled)
			{
				Trace::ToggleChannel(ChannelNameFString.GetCharArray().GetData(), false);
			}
		}, &OriginalTraceChannelsState);
#endif
}

void FRuntimeModule::RestoreTraceChannelsToPreRecordingState()
{
#if UE_TRACE_ENABLED || UE_TRACE_MINIMAL_ENABLED
	UE_LOG_REF(LogCategory, Log, TEXT("[%hs] Restoring trace channels state..."), __func__);

	for (const TPair<FString, bool>& ChannelWithState : OriginalTraceChannelsState)
	{
		Trace::ToggleChannel(GetData(ChannelWithState.Key), ChannelWithState.Value);
	}

	OriginalTraceChannelsState.Reset();
#endif
}

void FRuntimeModule::EnableRequiredTraceChannels()
{
#if UE_TRACE_ENABLED || UE_TRACE_MINIMAL_ENABLED
	Trace::ToggleChannel(TEXT("Frame"), true);
#endif

	// Allow derived classes to enable their specific channels
	OnEnableRequiredTraceChannelsInternal();
}

void FRuntimeModule::OnRecordingStartingInternal()
{
}

void FRuntimeModule::OnRecordingStartedInternal()
{
}

void FRuntimeModule::OnRecordingStartFailedInternal(const FText& FailureReason)
{
}

void FRuntimeModule::OnRecordingStoppedInternal()
{
}

void FRuntimeModule::OnEnableRequiredTraceChannelsInternal()
{
}

} // UE::TraceBasedDebuggers

#undef LOCTEXT_NAMESPACE
#endif // WITH_TRACE_BASED_DEBUGGERS