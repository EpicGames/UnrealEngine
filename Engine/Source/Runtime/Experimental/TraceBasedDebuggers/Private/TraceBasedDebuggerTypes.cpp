// Copyright Epic Games, Inc. All Rights Reserved.

#include "TraceBasedDebuggerTypes.h"

#include "Misc/App.h"

DEFINE_LOG_CATEGORY(LogTraceBasedDebuggers)

EBuildTargetType UE::TraceBasedDebuggers::GetBuildTargetType()
{
	// Using IsRunning* helpers to handle cases where the editor is launched with -server, -client, -game
	EBuildTargetType TargetType;
	if (IsRunningDedicatedServer())
	{
		TargetType = EBuildTargetType::Server;
	}
	else if (IsRunningClientOnly())
	{
		TargetType = EBuildTargetType::Client;
	}
	else if (IsRunningGame())
	{
		TargetType = EBuildTargetType::Game;
	}
	else
	{
		TargetType = FApp::GetBuildTargetType();
	}
	return TargetType;
}
