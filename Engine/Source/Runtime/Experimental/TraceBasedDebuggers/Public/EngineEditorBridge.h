// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TRACE_BASED_DEBUGGERS

#include "RemoteSessionsManager.h"
#include "Templates/SharedPointer.h"

class UGameInstance;

#define UE_API TRACEBASEDDEBUGGERS_API

#ifndef WITH_TRACE_BASED_DEBUGGERS_EXTERNAL_MESSAGING
#define WITH_TRACE_BASED_DEBUGGERS_EXTERNAL_MESSAGING 0
#endif

namespace UE::TraceBasedDebuggers
{

/**
 * Object that bridges the gap between a given debugger runtime module and the Engine & associated editor.
 * As the debugger runtime module often does not have access to the engine module, this object reacts to events
 * and performs necessary operations the runtime module cannot do directly.
 */
class FEngineEditorBridge
{
public:

#if NO_LOGGING
	typedef FNoLoggingCategory FLogCategoryAlias;
#else
	typedef FLogCategoryBase FLogCategoryAlias;
#endif

	FEngineEditorBridge() = delete;
	UE_API explicit FEngineEditorBridge(const FLogCategoryAlias& LogCategory);

	virtual ~FEngineEditorBridge() = default;

	UE_API void Initialize();
	UE_API void TearDown();

	UE_API TSharedPtr<FRemoteSessionsManager> GetSessionsManager() const;

	/** 
	 * Derived classes can use this method to register a remote sessions handler that adds support
	 * for more message types. 
	 * Method will ensure if a handler is already registered.
	 * The handler will be automatically unregistered from the session manager when the Engine-Editor bridge 
	 * gets teared down.
	 */
	UE_API void RegisterRemoteSessionsHandler(TSharedRef<IRemoteSessionsHandler> Handler);

	TSharedPtr<IDataRelayTransport> GetTraceRelayTransportInstance()
	{
		return RelayTraceDataTransportInstance;
	}

	bool IsInitialized() const
	{
		return bIsInitialized;
	}

	FSimpleMulticastDelegate& OnEngineEditorBridgeInitialized()
	{
		return InitializedDelegate;
	}

	UE_API void SetDataRelayTransportInstance(const TSharedPtr<IDataRelayTransport>& InExternalTraceRelayInstance);

protected:
	UE_API void HandleRecordingStarted();
	UE_API void HandleRecordingStopped();
	UE_API void BroadcastSessionStatus() const;

	virtual void OnInitializeInternal()
	{
	}

	virtual void OnTearDownInternal()
	{
	}

	virtual void OnRecordingStartedInternal()
	{
	}

	virtual void OnRecordingStoppedInternal()
	{
	}

	virtual void BuildRecordingStatusInternal(FRecordingStatusMessage& OutStatusMessage) const
	{
	}

private:
	TSharedPtr<IDataRelayTransport> RelayTraceDataTransportInstance;

	static TWeakPtr<FRemoteSessionsManager> SharedRemoteSessionsManager;
	TSharedRef<FRemoteSessionsManager> RemoteSessionsManager;
	/** Handler created by derived classes to handle more message types */
	TSharedPtr<IRemoteSessionsHandler> RemoteSessionsHandler;

	const FLogCategoryAlias& LogCategory;

	FTSTicker::FDelegateHandle RecordingStatusUpdateHandle;

	FSimpleMulticastDelegate InitializedDelegate;

	bool bIsInitialized = false;
};

} // UE::TraceBasedDebuggers

#undef UE_API

#endif // WITH_TRACE_BASED_DEBUGGERS
