// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TRACE_BASED_DEBUGGERS
#include <atomic>
#include "Containers/Array.h"
#include "Containers/SpscQueue.h"
#include "Delegates/Delegate.h"
#include "Serialization/BitWriter.h"
#include "Templates/UniquePtr.h"

#define UE_API TRACEBASEDDEBUGGERS_API

class FBufferArchive;

namespace UE::TraceBasedDebuggers
{
/**
 * Writer object that chunks trace data into smaller bunches and queues them so a relay instance
 * can dequeue and send them over the network later
 */
struct FRelayTraceDataWriter
{
#if NO_LOGGING
	typedef FNoLoggingCategory FLogCategoryAlias;
#else
	typedef FLogCategoryBase FLogCategoryAlias;
#endif

	FRelayTraceDataWriter() = delete;
	UE_API explicit FRelayTraceDataWriter(const FLogCategoryAlias& LogCategory);

	UE_API ~FRelayTraceDataWriter();

	/**
	 * Sets max number of bytes a queued bunch can have
	 * @param MaxBytes Greater than zero number of bytes to set as limit
	 */
	UE_API void SetMaxBytesPerBunch(uint32 MaxBytes);

	/**
	 * Sets max number of bytes this writer can buffer before overflowing, triggering a close
	 * @param MaxBytes number of bytes to set as limit
	 */
	UE_API void SetMaxAllowedPendingBytes(uint64 MaxBytes);

	UE_API static bool WriteHelper(UPTRINT WriterHandle, const void* Data, uint32 Size);

	static void CloseHelper(UPTRINT WriterHandle)
	{
		if (!WriterHandle)
		{
			return;
		}

		reinterpret_cast<FRelayTraceDataWriter*>(WriterHandle)->Close();
	}

	/**
	 * Dequeues a bunch ready to send over the network, if any
	 */
	UE_API TOptional<TUniquePtr<FBufferArchive>> DequeuePendingBunch();

	/**
	 * Returns the size of the next bunch in the queue
	 */
	UE_API int32 NextPendingBunchSize() const;

	/**
	 * Returns true if we have any data pending to be sent over the network
	 */
	bool HasPendingBunches() const
	{
		return PendingBunchesCount > 0;
	}

	/**
	 * returns the number of bytes overall that are in the queue waiting to be sent
	 */
	UE_API int64 GetQueuedBytesNum() const;

	/**
	 * Closes this Writer. After this is called, no new data will be processed, but any pending data can still be dequeued and sent over
	 */
	UE_API void Close();

	/**
	 * Returns true if this writer is closed and no new data is being processed
	 */
	bool IsClosed() const
	{
		return bClosed;
	}

	DECLARE_DELEGATE(FNewDataAvailableDelegate)
	FNewDataAvailableDelegate& OnNewDataAvailable()
	{
		return NewDataAvailableDelegate;
	}

	FSimpleDelegate& OnDataOverflow()
	{
		return DataOverflowDelegate;
	}

private:

	FNewDataAvailableDelegate NewDataAvailableDelegate;
	FNewDataAvailableDelegate DataOverflowDelegate;

	TUniquePtr<FBufferArchive> CreateBunch();

	bool Write_Internal(const void* Data, uint32 Size);

	bool CanWriteData(uint32 NewDataSize) const;

	const FLogCategoryAlias& LogCategory;
	TSpscQueue<TUniquePtr<FBufferArchive>> PendingBunchesToRelay;
	TUniquePtr<FBufferArchive> CurrentPendingBunch;
	std::atomic<int32> PendingBunchesCount = 0;
	std::atomic<bool> bClosed = false;
	uint32 MaxPendingBytesPerBunch = 0;
	std::atomic<int64> QueuedBytesNum = 0;
	uint64 MaxAllowedPendingBytes = 1073741824;

	// Diagnostic counters — logged at Close() to diagnose relay data pipeline issues
	int64 TotalBytesWritten = 0;
	int32 TotalWriteCalls = 0;
	int32 TotalBunchesCompleted = 0;
};

} // UE::TraceBasedDebuggers

#undef UE_API
#endif // WITH_TRACE_BASED_DEBUGGERS
