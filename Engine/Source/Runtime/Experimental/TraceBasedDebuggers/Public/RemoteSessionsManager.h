// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "IMessageContext.h"
#include "MessageEndpoint.h"
#include "TraceBasedDebuggerTypes.h"
#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "TraceDataRelayTransport.h"

#include "RemoteSessionsManager.generated.h"

#define UE_API TRACEBASEDDEBUGGERS_API

class IMessageBridge;
class IMessageBus;
class IMessageContext;

namespace UE::TraceBasedDebuggers
{
struct FFullSessionInfoRequestMessage;
struct FFullSessionInfoResponseMessage;
struct FRemoteSessionsManager;
struct FSessionInfo;
struct FTraceConnectionDetails;


USTRUCT(MinimalAPI)
struct FSessionPing
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid ControllerInstanceId;
};

USTRUCT(MinimalAPI)
struct FSessionPong
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid InstanceId;

	UPROPERTY()
	FGuid SessionId;

	UPROPERTY()
	FString SessionName;

	UPROPERTY()
	uint8 BuildTargetType = static_cast<uint8>(EBuildTargetType::Unknown);
};

USTRUCT(MinimalAPI)
struct FStopRecordingCommandMessage
{
	GENERATED_BODY()
};

USTRUCT(MinimalAPI)
struct FRecordingStatusMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid InstanceId = FGuid();

	UPROPERTY()
	FGuid DebuggerId = FGuid();

	UPROPERTY()
	FGuid RequesterId = FGuid();

	UPROPERTY()
	float ElapsedTime = false;

	UPROPERTY()
	int64 BufferedDataBytesSize = 0;
};

USTRUCT(MinimalAPI)
struct FTraceConnectionDetailsMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid InstanceId = FGuid();

	UPROPERTY()
	FTraceConnectionDetails TraceDetails;
};

USTRUCT(MinimalAPI)
struct FStartRecordingCommandMessage
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid InstanceId = FGuid();

	UPROPERTY()
	FString Target;

	UPROPERTY()
	ERecordingMode RecordingMode = ERecordingMode::Invalid;

	UPROPERTY()
	ETraceTransportMode TransportMode = ETraceTransportMode::Invalid;
};

UENUM()
enum class ERemoteSessionAttributes
{
	None = 0,
	SupportsDataChannelChange = 1 << 0,
	CanExpire = 1 << 1,
	IsMultiSessionWrapper = 1 << 2,
};
ENUM_CLASS_FLAGS(ERemoteSessionAttributes)

DECLARE_MULTICAST_DELEGATE_OneParam(FRecordingStateChangeDelegate, const TSharedPtr<FSessionInfo>&);
DECLARE_MULTICAST_DELEGATE_OneParam(FSessionLifetimeDelegate, const TSharedPtr<FSessionInfo>&);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnSupportedMessageTypesChanged, TConstArrayView<const UScriptStruct*>);

struct IRemoteSessionsHandler : TSharedFromThis<IRemoteSessionsHandler>
{
	virtual ~IRemoteSessionsHandler() = default;

	/** Callback to allow registered sessions handlers to add additional message handlers to the message endpoint builder */
	virtual void OnCreatingMessageEndpoint(const TSharedPtr<FRemoteSessionsManager>&, FMessageEndpointBuilder&) = 0;

	/** Callback to allow registered sessions handlers to subscribe to specific message types */
	virtual void OnSubscribingMessageTypes(const TSharedPtr<FRemoteSessionsManager>&, FMessageEndpoint&) = 0;

	/** Callback to allow registered sessions handlers to build the response for a session information request */
	virtual void OnBuildingFullSessionInfoResponseMessage(FFullSessionInfoResponseMessage&) = 0;

	/** Callback to allow registered sessions handlers to process the response for a session information request */
	virtual void OnHandlingFullSessionInfoResponseMessage(const FFullSessionInfoResponseMessage&, const TSharedPtr<FSessionInfo>&) = 0;
};

/** Object that is able to discover, issue and execute commands back and forth between the trace-based debuggers and client/server/editor instances */
struct FRemoteSessionsManager : TSharedFromThis<FRemoteSessionsManager>
{
	UE_API FRemoteSessionsManager();
	UE_API virtual ~FRemoteSessionsManager();

	UE_API void ReInitializeMessagingSystem(const TSharedPtr<IMessageBus>& InMessageBus);

	FSimpleMulticastDelegate& OnSessionsUpdated()
	{
		return SessionsUpdatedDelegate;
	}

	FSessionLifetimeDelegate& OnSessionDiscovered()
	{
		return SessionDiscoveredDelegate;
	}

	FSessionLifetimeDelegate& OnSessionExpired()
	{
		return SessionExpiredDelegate;
	}
	UE_API void RegisterExternalHandler(TSharedRef<IRemoteSessionsHandler> Handler);
	UE_API void UnregisterExternalHandler(TSharedRef<IRemoteSessionsHandler> Handler);

	typedef TFunction<void(const UScriptStruct*)> FVisitorFunction;

	/**
	 * Enumerate messages types allowed by this module.
	 * @param InVisitor
	 */
	UE_API void EnumerateMessageTypes(const FVisitorFunction& InVisitor);

	/**
	 * Starts sending ping requests that potential targets (client, servers or other editors) can answer to and report themselves as available
	 */
	UE_API void StartSessionDiscovery(const FGuid& InDebuggerGuid);
	UE_DEPRECATED(5.8, "Use overload taking a debugger guid")
	void StartSessionDiscovery()
	{
		StartSessionDiscovery(FGuid{});
	}

	/**
	 * Stop sending ping requests to potential targets (client, servers or other editors).
	 */
	UE_API void StopSessionDiscovery(const FGuid& InDebuggerGuid);
	UE_DEPRECATED(5.8, "Use overload taking a debugger guid")
	void StopSessionDiscovery()
	{
		StopSessionDiscovery(FGuid{});
	}

	/**
	 * Issues a command to the provided address to change the state of a data channel
	 * 	@param InDestinationAddress Message bus address that will execute the command
	 * 	@param InMessage New data channel state to apply in the receiving instance
	 */
	template <typename T>
	void SendCommand(const FMessageAddress& InDestinationAddress, const T& InMessage)
	{
		if (!MessageEndpoint)
		{
			UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed to send command | Invalid endpoint.", __FUNCTION__);
			return;
		}

		MessageEndpoint->Send(
			FMessageEndpoint::MakeMessage<T>(InMessage),
			T::StaticStruct(),
			EMessageFlags::Reliable,
			nullptr,
			TArrayBuilder<FMessageAddress>().Add(InDestinationAddress),
			FTimespan::Zero(),
			FDateTime::MaxValue());
	}

	/**
	 * Returns the session info object for the provided ID
	 * @param Id SessionID
	 */
	UE_API TWeakPtr<FSessionInfo> GetSessionInfo(FGuid Id);

	/**
	 * Returns the first available session info
	 */
	UE_API TWeakPtr<FSessionInfo> GetFirstAvailableSingleSessionInfo();

	/**
	 * Iterates through all active and valid sessions, and executes the provided callback to it.
	 * if the callbacks returns false, the iteration will stop
	* */
	template <typename CallbackType>
	void EnumerateActiveSessions(const CallbackType& Callback)
	{
		for (const TPair<FGuid, TSharedPtr<FSessionInfo>>& ActiveSession : ActiveSessionsById)
		{
			if (ActiveSession.Value)
			{
				bool bContinue = Callback(ActiveSession.Value.ToSharedRef());

				if (!bContinue)
				{
					return;
				}
			}
		}
	}

	/**
	 * Broadcast to the network a given message
	 * @param InMessage message to broadcast
	 */
	template <typename T>
	void PublishMessage(const T& InMessage)
	{
		if (ensure(MessageEndpoint))
		{
			MessageEndpoint->Publish(FMessageEndpoint::MakeMessage<T>(InMessage), EMessageScope::Network);
		}
	}

	/**
	 * Delegate that broadcast when a recording was started in a session (either local or remote)
	 */
	FRecordingStateChangeDelegate& OnSessionRecordingStarted()
	{
		return RecordingStartedDelegate;
	}

	/**
	 * Delegate that broadcast when a recording stops in a session (either local or remote)
	 */
	FRecordingStateChangeDelegate& OnSessionRecordingStopped()
	{
		return RecordingStoppedDelegate;
	}

	UE_API static const FString AllRemoteSessionsTargetName;
	UE_API static const FString AllRemoteServersTargetName;
	UE_API static const FString AllRemoteClientsTargetName;
	UE_API static const FString AllSessionsTargetName;
	UE_API static const FString CustomSessionsTargetName;
	UE_API static const FString LocalEditorSessionName;
	UE_API static const FName MessageBusEndpointName;
	UE_API static const FGuid AllRemoteSessionsWrapperGUID;
	UE_API static const FGuid AllRemoteServersWrapperGUID;
	UE_API static const FGuid AllRemoteClientsWrapperGUID;
	UE_API static const FGuid AllSessionsWrapperGUID;
	UE_API static const FGuid CustomSessionsWrapperGUID;
	UE_API static const FGuid InvalidSessionGUID;
	UE_API static const FGuid LocalEditorSessionID;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FMessagingInitialized, TSharedPtr<IMessageBus> NewMessageBus, TSharedPtr<FMessageEndpoint> NewEndpoint);
	FMessagingInitialized& OnMessagingInitialized()
	{
		return MessagingInitializedDelegate;
	}

	TWeakPtr<IMessageBus> GetMessageBusInstance()
	{
		return MessageBusPtr;
	}

	TWeakPtr<FMessageEndpoint> GetMessageBusEndPoint()
	{
		return MessageEndpoint;
	}

	/**
	 * Returns true if this instance has controller capabilities (is either an editor or standalone application, which is also an editor)
	 */
	static constexpr bool IsController()
	{
#if WITH_EDITOR
		return true;
#else
		return false;
#endif
	}

	/**
	 * Returns true if this instance can publish messages to remote endpoints via a network bridge.
	 * Checks for a standard bridge (e.g., UdpMessaging via -messaging flag) or a per-type bridge
	 * set using SetMessageBridge.
	 * @see SetMessageBridge
	 */
	UE_API bool CanPublishToNetwork(const FTopLevelAssetPath& MessageType) const;

	/** Templated convenience wrapper for CanPublishToNetwork. */
	template <typename TMessageType>
	bool CanPublishToNetwork() const
	{
		return CanPublishToNetwork(FTopLevelAssetPath(TBaseStructure<TMessageType>::Get()));
	}

	/**
	 * Adds a message type to the list of supported message types for this session system.
	 * @param ScriptStruct Message type info
	 */
	UE_API void RegisterExternalSupportedMessageType(const UScriptStruct* ScriptStruct);

	/**
	 * Adds a message type to the list of supported message types for this session system.
	 */
	template <typename TDataType>
	void RegisterExternalSupportedMessageType()
	{
		RegisterExternalSupportedMessageType(TBaseStructure<TDataType>::Get());
	}

	/**
	 * Delegate broadcast whenever the supported message types list changes.
	 * Callers that maintain external message bridges with per-type subscriptions
	 * should bind to this delegate to keep bridge subscriptions in sync.
	 */
	FOnSupportedMessageTypesChanged& OnSupportedMessageTypesChanged()
	{
		return SupportedMessageTypesChangedDelegate;
	}

	/**
	 * Sets an external message bridge for per-type reachability checks.
	 * When set, CanPublishToNetwork queries the bridge's IsSubscribed to determine
	 * if a specific message type can reach remote endpoints.
	 * @param InMessageBridge The bridge to query, or nullptr to clear.
	 */
	void SetMessageBridge(const TSharedPtr<IMessageBridge>& InMessageBridge)
	{
		CustomMessageBridge = InMessageBridge;
	}

private:
	void Initialize();
	void Shutdown();

	void RegisterBuiltInMessageTypes();

	void InitializeMessagingSystem(const TSharedPtr<IMessageBus>& InMessageBus);
	void ShutdownMessagingSystem();
	void BroadcastSupportedMessageTypes();

	/**
	 * Sends a request to obtain the full session information to the provided message bus address
	 * @param InDestinationAddress
	 */
	void SendFullSessionStateRequestCommand(const FMessageAddress& InDestinationAddress);

	/**
	 * Registers a session object that is able to control multiple session instances
	 * @param InSessionInfoRef Multi Session object
	 */
	void RegisterSessionInMultiSessionWrapper(const TSharedRef<FSessionInfo>& InSessionInfoRef);

	/**
	 * Deregisters a session object that is able to control multiple session instances
	 * @param InSessionInfoRef Multi Session object
	 */
	void DeRegisterSessionInMultiSessionWrapper(const TSharedRef<FSessionInfo>& InSessionInfoRef);

	/**
	 * Creates a session object that is able to control multiple other session objects.
	 * @param InstanceId Instance ID for this object
	 * @param SessionName Session name that will be used in the UI
	 * @return
	 */
	TSharedPtr<FSessionInfo> CreatedWrapperSessionInfo(FGuid InstanceId, FStringView SessionName);

	/**
	 * Creates the message bus endpoint this session manager will use
	 */
	TSharedPtr<FMessageEndpoint> CreateEndPoint(const TSharedRef<IMessageBus>& InMessageBus);

	bool Tick(float DeltaTime);

	/**
	 * Usually used by the controller. Broadcast to the network this controller exists.
	 */
	void SendPing();

	/**
	 * Replies directly to the ping sender with a small subset of this instance information.
	 * @param InMessage Ping message that was received
	 * @param InContext Message context for the received Ping
	 */
	void SendPong(const FSessionPing& InMessage, const TSharedRef<IMessageContext>& InContext);

	void HandleSessionPongMessage(const FSessionPong& InMessage, const TSharedRef<IMessageContext>& InContext);
	void HandleSessionPingMessage(const FSessionPing& InMessage, const TSharedRef<IMessageContext>& InContext);
	void HandleRecordingStatusUpdateMessage(const FRecordingStatusMessage& InMessage, const TSharedRef<IMessageContext>& InContext);
	void HandleConnectionDetailsUpdateMessage(const FTraceConnectionDetailsMessage& InMessage, const TSharedRef<IMessageContext>& InContext);
	void HandleFullSessionStateRequestMessage(const FFullSessionInfoRequestMessage& InMessage, const TSharedRef<IMessageContext>& InContext);
	void HandleFullSessionStateResponseMessage(const FFullSessionInfoResponseMessage& InMessage, const TSharedRef<IMessageContext>& InContext);

	enum class ERemoveSessionOptions : uint8
	{
		None = 0,
		ForceRemoveAll = 1 << 0
	};
	FRIEND_ENUM_CLASS_FLAGS(ERemoveSessionOptions)

	void RemoveExpiredSessions(ERemoveSessionOptions Options = ERemoveSessionOptions::None);

	void ProcessPendingMessagesForSession(const FSessionPong& InMessage, const TSharedRef<FSessionInfo>& InSessionInfoPtr);

	TArray<TSharedRef<IRemoteSessionsHandler>> Handlers;

	/** Holds the time at which the last ping was sent. */
	FDateTime LastPingTime;

	/** Holds a pointer to the message bus. */
	TWeakPtr<IMessageBus> MessageBusPtr;

	/** Holds the messaging endpoint. */
	TSharedPtr<FMessageEndpoint> MessageEndpoint;

	TMap<FGuid, TSharedPtr<FSessionInfo>> ActiveSessionsById;
	TMap<FGuid, FRecordingStatusMessage> PendingRecordingStatusMessages;
	TMap<FGuid, FTraceConnectionDetails> PendingRecordingConnectionDetailsMessages;

	FSimpleMulticastDelegate SessionsUpdatedDelegate;
	FSessionLifetimeDelegate SessionDiscoveredDelegate;
	FSessionLifetimeDelegate SessionExpiredDelegate;

	FRecordingStateChangeDelegate RecordingStartedDelegate;
	FRecordingStateChangeDelegate RecordingStoppedDelegate;

	FTSTicker::FDelegateHandle TickHandle;

	FMessagingInitialized MessagingInitializedDelegate;

	TArray<FGuid> ActiveDebuggersUsingSessionDiscovery;
	TArray<const UScriptStruct*> SupportedMessageTypes;
	TSharedPtr<IMessageBridge> CustomMessageBridge;
	FOnSupportedMessageTypesChanged SupportedMessageTypesChangedDelegate;

	bool bInitialized = false;
};

} // namespace UE::TraceBasedDebuggers
#undef UE_API