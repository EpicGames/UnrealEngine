// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteSessionsManager.h"
#include "StructUtils/InstancedStruct.h"

#include "SessionInfo.generated.h"

#define UE_API TRACEBASEDDEBUGGERS_API

namespace UE::TraceBasedDebuggers
{

USTRUCT(MinimalAPI)
struct FFullSessionInfoRequestMessage
{
	GENERATED_BODY()
};

USTRUCT(MinimalAPI)
struct FFullSessionInfoResponseMessage
{
	GENERATED_BODY()

	template <typename DataType>
	DataType* GetDebuggerData()
	{
		for (FInstancedStruct& InstancedDebuggerData : DebuggerSpecificData)
		{
			if (DataType* DebuggerData = InstancedDebuggerData.GetMutablePtr<DataType>())
			{
				return DebuggerData;
			}
		}
		return nullptr;
	}

	template <typename DataType>
	const DataType* GetDebuggerData() const
	{
		for (const FInstancedStruct& InstancedDebuggerData : DebuggerSpecificData)
		{
			if (const DataType* DebuggerData = InstancedDebuggerData.GetPtr<DataType>())
			{
				return DebuggerData;
			}
		}
		return nullptr;
	}

	UPROPERTY()
	FGuid InstanceId;

	UPROPERTY()
	FGuid DebuggerId = FGuid();

	UPROPERTY()
	TArray<FInstancedStruct> DebuggerSpecificData;

	UPROPERTY()
	FGuid RecordingRequesterId = FGuid();
};

UENUM()
enum class ERemoteSessionReadyState : uint8
{
	/** The session is ready to execute commands */
	Ready,
	/** We are executing a command in the session we expect to take a while without hearing anything from the target. */
	Busy
};

/**
 * Session object that contains all the information needed to communicate with a remote instance, and the state of that instance
 */
struct FSessionInfo
{
	explicit FSessionInfo() : InstanceId(FGuid()),
	SessionTypeAttributes(ERemoteSessionAttributes::CanExpire | ERemoteSessionAttributes::SupportsDataChannelChange)
	{
	}

	virtual ~FSessionInfo() = default;

protected:

	explicit FSessionInfo(ERemoteSessionAttributes InSessionTypeAttributes) : SessionTypeAttributes(InSessionTypeAttributes)
	{
	}

public:

	template <typename DataType>
	DataType* GetDebuggerData()
	{
		for (FInstancedStruct& InstancedDebuggerData : DebuggerSpecificSessionData)
		{
			if (DataType* DebuggerData = InstancedDebuggerData.GetMutablePtr<DataType>())
			{
				return DebuggerData;
			}
		}
		return nullptr;
	}

	/** Append or update data associated to a given debugger type */
	template <typename DataType>
	void SetDebuggerData(DataType&& NewSessionData)
	{
		if (DataType* DebuggerData = GetDebuggerData<DataType>())
		{
			*DebuggerData = MoveTemp(NewSessionData);
		}
		else
		{
			DebuggerSpecificSessionData.Emplace(FInstancedStruct::Make(MoveTemp(NewSessionData)));
		}
	}

	FGuid InstanceId;
	FString SessionName;
	FMessageAddress Address;
	FDateTime LastPingTime;
	EBuildTargetType BuildTargetType = EBuildTargetType::Unknown;
	ERemoteSessionReadyState ReadyState = ERemoteSessionReadyState::Ready;

	FRecordingStatusMessage LastKnownRecordingState;

	UE_API const FTraceConnectionDetails& GetTraceConnectionDetails();

	/** @return Whether any debugger is currently recording, regardless of its type, or by which application instance it was requested */
	UE_API virtual bool IsAnyDebuggerRecording() const;

	/** 
	 * Indicates whether a given debugger type has an active recording requested by the current application instance.
	 * @param DebuggerTypeId The identifier of the debugger type
	 * @return Whether the provided debugger type id is currently recording
	 */
	UE_API virtual bool IsRecording(const FGuid& DebuggerTypeId) const;

	UE_API virtual uint64 GetBufferedBytesNum() const;

	ERemoteSessionAttributes GetSessionTypeAttributes() const
	{
		return SessionTypeAttributes;
	}

	UE_API void SetReceivedBytesPerSecond(uint64 InNewBytesPerSecond);
	UE_API uint64 GetReceivedBytesPerSecond() const;

protected:

	TArray<FInstancedStruct> DebuggerSpecificSessionData;

	FTraceConnectionDetails LastKnownConnectionDetails;

	uint64 ReceivedBytesPerSecond = 0;

	const ERemoteSessionAttributes SessionTypeAttributes;

	friend struct FRemoteSessionsManager;
};

/**
 * Session object that is able to control and provide information about multiple session objects.
 * Used so the UI can use the same API to control multiple sessions like it does for single sessions.
 */
struct FMultiSessionInfo : FSessionInfo
{
	explicit FMultiSessionInfo() : FSessionInfo(ERemoteSessionAttributes::IsMultiSessionWrapper)
	{
	}

	virtual ~FMultiSessionInfo() override = default;

	UE_API virtual bool IsRecording(const FGuid& DebuggerTypeId) const override;
	UE_API virtual bool IsAnyDebuggerRecording() const override;

	UE_API virtual uint64 GetBufferedBytesNum() const override;

	template<typename TCallback>
	void EnumerateInnerSessions(const TCallback& Callback) const
	{
		for (const TPair<FGuid, TWeakPtr<FSessionInfo>>& InnerSessionWithID : InnerSessionsByInstanceID)
		{
			if (const TSharedPtr<FSessionInfo> SessionPtr = InnerSessionWithID.Value.Pin())
			{
				if (!Callback(SessionPtr.ToSharedRef()))
				{
					return;
				}
			}
		}
	}

	TMap<FGuid, TWeakPtr<FSessionInfo>> InnerSessionsByInstanceID;
};

} // namespace UE::TraceBasedDebuggers
#undef UE_API
