// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_TRACE_BASED_DEBUGGERS
#include "Containers/Ticker.h"
#include "HAL/Event.h"
#include "Misc/Guid.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "TraceDataRelayTransport.h"

#define UE_API TRACEBASEDDEBUGGERS_API

namespace UE::TraceBasedDebuggers
{
struct FRelayTraceDataWriter;
struct FStartRecordingCommandMessage;

DECLARE_MULTICAST_DELEGATE(FRecordingStateChangedDelegate)
DECLARE_MULTICAST_DELEGATE_OneParam(FRecordingStartFailedDelegate, const FText&)
DECLARE_DELEGATE_OneParam(FRelayTraceDataDelegate, FRelayTraceDataWriter&)
DECLARE_DELEGATE_RetVal(FTraceConnectionDetails, FExternalTraceConnectionStatusDelegate)
DECLARE_DELEGATE_RetVal(uint64, FExternalRelayBufferQueueSizeDelegate)

class FRuntimeModule
{
#if NO_LOGGING
	typedef FNoLoggingCategory FLogCategoryAlias;
#else
	typedef FLogCategoryBase FLogCategoryAlias;
#endif

protected:
	FRuntimeModule() = delete;
	UE_API explicit FRuntimeModule(const FLogCategoryAlias& InLogCategory, const FGuid& InDebuggerTypeID);

	UE_API virtual ~FRuntimeModule();

	/**
	 * Parses the command line arguments for <AutoRecordCommand>, if set, then start recording if the command is found.
	 * @return Whether the recording was started.
	 */
	UE_API bool TryRecordOnStartup();

public:
	/**
	 * Starts a recording by starting a Trace session. It will stop any existing trace session.
	 * @param Args : Arguments array usually provided by the commandline. Used to determine if we want to record to file or a local trace server.
	 * Available options are:
	 * - "" (default): trace to file using GenerateRecordingFileName
	 * - "File": trace to file using GenerateRecordingFileName
	 * - "-tracefile [FILENAME]": trace to file using either the optional [FILENAME] or GenerateRecordingFileName
	 * - "Server [ADDRESS]": trace server using either the optional [ADDRESS] or local host (i.e., "127.0.0.1")
	 * - "-tracehost [ADDRESS]": trace server using either the optional [ADDRESS] or local host (i.e., "127.0.0.1")
	 * @see GenerateRecordingFileName
	 */
	UE_API void StartRecording(TConstArrayView<FString> Args);

	/**
	 * Starts a recording by starting a Trace session. It will stop any existing trace session
	 * @param InRecordingStartCommand : Used to determine if we want to record to file or a local trace server among other settings
	 */

	UE_API void StartRecording(const FStartRecordingCommandMessage& InRecordingStartCommand);

	/* Stops an active recording */
	UE_API void StopRecording();

	/** Returns true if we are currently recording trace data for the current debugger */
	bool IsRecording() const
	{
		return ActiveRecordingRequesterId.IsValid();
	}

	/** @return Unique identifier of the application instance that requested the current recording, if active. */
	const FGuid& GetRecordingRequesterId() const
	{
		return ActiveRecordingRequesterId;
	}

	/** @return Unique identifier for the current debugger type */
	const FGuid& GetDebuggerId() const
	{
		return DebuggerGuid;
	}

	/** 
	 * Finds a valid file name for a new file for Trace recording.
	 * The file name is built using the following format:
	 * <DebuggerName>-<ProjectName>-<BuildTargetType>-<DateTime>
	 * where <DebuggerName> is provided by the derived classes.
	 */
	UE_API void GenerateRecordingFileName(FString& OutFileName) const;

	/** Return the current connection details for the underlying trace session */
	UE_API FTraceConnectionDetails GetCurrentTraceConnectionDetails() const;

	/** Returns the accumulated recording time in seconds since the recording started */
	float GetAccumulatedRecordingTime() const
	{
		return AccumulatedRecordingTime;
	}

	/** Returns the accumulated current send buffer size used by the relay transport system - This should be used for logging or UI feedback only */
	UE_API int64 GetBufferedDataBytesSize() const;

	/**
	 * Blocks the calling thread until trace channels have been enabled after a recording has started, or until the timeout expires.
	 * Intended for use on the game thread to synchronize PIE startup with asynchronous trace channel enablement.
	 * @param MaxWaitTimeSeconds Maximum time to wait in seconds before giving up.
	 * @return true if channels were enabled within the timeout, false otherwise.
	 */
	UE_API bool WaitForTraceChannelsEnabled(float MaxWaitTimeSeconds);

	FSimpleMulticastDelegate& OnTraceConnectionDetailsUpdated()
	{
		return TraceConnectionDetailsUpdatedDelegate;
	}

	/**
	 * Registers an external handler that will be in charge of relaying the traced data to a remote instance if the relay transport mode is selected
	 * @param InExternalRelayExecutor Delegate that will be executed to relay data to a remote debugger instance, periodically
	 */
	UE_API void RegisterExternalRelayExecutor(const FRelayTraceDataDelegate& InExternalRelayExecutor);

	/**
	 * Registers an external handler that will be in charge handling what happens when the Relay writer overflows
	 * @param InExternalRelayOverflowHandler Delegate that will be executed when the Relay writer overflows
	 */
	UE_API void RegisterExternalRelayOverflowHandler(const FSimpleDelegate& InExternalRelayOverflowHandler);

	/**
	 * Unbinds the delegate to an external relay overflow handler
	 */
	UE_API void UnregisterExternalRelayOverflowHandler();

	/**
	 * Unbinds the delegate to an external relay executor
	 */
	UE_API void UnregisterCurrentExternalRelayExecutor();

	/**
	 * Registers an external handler that will be in charge of providing the trace status if the recording mode is set to Relay
	 * @param InExternalTraceStatusCallback Delegate that will be executed to get the trace status, periodically
	 */
	UE_API void RegisterExternalTraceStatusProvider(const FExternalTraceConnectionStatusDelegate& InExternalTraceStatusCallback);

	/**
	 * Unbinds the delegate to the current external trace status provider
	 */
	UE_API void UnregisterExternalTraceStatusProvider();

	/**
	 *  Registers an external handler that will provide information about any external send buffer the current relay transport implementation might be using, if any.
	 *  This is used for UI metrics. To handle buffer overflows, use RegisterExternalRelayOverflowHandler
	 * @param InExternalRelayBufferSizeCallback
	 */
	UE_API void RegisterExternalRelayBufferSizeProvider(const FExternalRelayBufferQueueSizeDelegate& InExternalRelayBufferSizeCallback);

	/**
	 * Unbinds the delegate to the current external Relay buffer size provider
	 */
	UE_API void UnregisterExternalRelayBufferSizeProvider();

	FRelayTraceDataWriter* GetRelayTraceWriterInstance() const
	{
		return RelayWriter.Get();
	}

protected:
	UE_API virtual void OnRecordingStartingInternal();
	UE_API virtual void OnRecordingStartedInternal();
	UE_API virtual void OnRecordingStartFailedInternal(const FText& FailureReason);
	UE_API virtual void OnRecordingStoppedInternal();

	UE_API virtual void OnEnableRequiredTraceChannelsInternal();

	/**
	 * Prefix provided by the derived classes used by GenerateRecordingFileName 
	 * using the following format "<DebuggerName>-<ProjectName>-<BuildTargetName>-<DateTime>.utrace"
	 */
	FString DebuggerName;

	/**
	 * Command line provided by the derived classes used during initialization to start recording or not.
	 */
	FString AutoRecordCommand;

	/**
	 * Command line that derived classes can override and used during initialization to specify the address of the host.
	 * e.g., -<HostAddressCommand>=x.x.x.x where 'HostAddressCommand' is set to 'DebuggerHost' by default.
	 */
	FString HostAddressCommand = TEXT("DebuggerHost");

	const FLogCategoryAlias& LogCategory;

private:

	void HandleRelayOverflow();

	void RelayTraceData() const;

	void SetupRelayDataPumpDelegates();
	void ClearRelayDataPumpDelegates();

	/** Stops the current Trace session */
	void StopTrace();

	/** Queues a full Capture of the simulation on the next frame */
	bool RecordingTimerTick(float DeltaTime);

	void HandleTraceConnectionEstablished();

	/**
	 * Used to handle stop requests to the active trace session that were not done by us
	 * That is a possible scenario because Trace is shared by other In-Editor tools
	 */
	void HandleTraceStopRequest(FTraceAuxiliary::EConnectionType TraceType, const FString& TraceDestination);

	bool WaitForTraceSessionDisconnect();

	void EnableRequiredTraceChannels();
	void SaveAndDisabledCurrentEnabledTraceChannels();
	void RestoreTraceChannelsToPreRecordingState();

	FTSTicker::FDelegateHandle RecordingTimerHandle;
	FTSTicker::FDelegateHandle RelayExecutorTickerHandle;

	FDelegateHandle OnConnectionDelegateHandle;
	FDelegateHandle OnTraceStoppedDelegateHandle;

	TUniquePtr<FRelayTraceDataWriter> RelayWriter;

	FSimpleMulticastDelegate TraceConnectionDetailsUpdatedDelegate;
	FSimpleDelegate ExternalRelayOverflowHandlerDelegate;
	FRelayTraceDataDelegate ExternalRelayExecutorDelegate;
	FExternalRelayBufferQueueSizeDelegate ExternalRelayBufferSizeProviderDelegate;
	FExternalTraceConnectionStatusDelegate ExternalTraceStatusDelegate;

	TMap<FString, bool> OriginalTraceChannelsState;

	/** Unique identifier of the debugger type associated to this instance */
	FGuid DebuggerGuid;

	/** Id of the application instance that requested the current active recording; invalid if not currently recording */
	FGuid ActiveRecordingRequesterId = FGuid{};

	float AccumulatedRecordingTime = 0.0f;

	ETraceTransportMode CurrentTransportMode = ETraceTransportMode::Invalid;
	bool bRequestedStop = false;

	/** Event signaled when HandleTraceConnectionEstablished completes and trace channels are enabled */
	FEventRef TraceChannelsEnabledEvent{EEventMode::ManualReset};
};

} // UE::TraceBasedDebuggers

#undef UE_API
#endif //WITH_TRACE_BASED_DEBUGGERS