// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogCategory.h"

#define UE_API TRACEBASEDDEBUGGERS_API

enum class EBuildTargetType : uint8;

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogTraceBasedDebuggers, Log, All);

namespace UE::TraceBasedDebuggers
{
extern UE_API EBuildTargetType GetBuildTargetType();
}

#undef UE_API