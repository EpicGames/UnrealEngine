// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TraceDataRelayTransport.generated.h"

struct FGuid;

namespace UE::TraceBasedDebuggers
{

UENUM()
enum class ERecordingMode : uint8
{
	Invalid,
	Live,
	File
};

/** Available transport modes for the Trace Data produced by the trace-based debuggers */
UENUM()
enum class ETraceTransportMode : uint8
{
	Invalid UMETA(Hidden),
	/** The data will be traced directly to a file in the local file system */
	FileSystem UMETA(Hidden),
	/** Data will be traced to the selected trace store server */
	TraceServer UMETA(Hidden),
	/** Data will be traced directly to the editor, accessible via sockets */
	Direct,
	/** Data will be traced via the trace relay system using a custom transport, like normal UE Networking or a direct socket connection */
	Relay
};

/** Structure containing the info necessary to connect to a trace session and start receiving data */
USTRUCT()
struct FTraceConnectionDetails
{
	GENERATED_BODY()

	FTraceConnectionDetails() = default;

	void MarkAsValid()
	{
		bIsValid = true;
	}

	bool IsValid() const
	{
		return bIsValid;
	}

	/** ID used to find the active trace session in the trace system */
	UPROPERTY()
	FGuid TraceGuid;

	/** ID from the trace-based debugger remote session from which this connection details come from */
	UPROPERTY()
	FGuid SessionGuid;

	/** Address of the trace server or editor in direct trace mode that will receive the trace data */
	UPROPERTY()
	FString TraceTarget;

	/*** Port number used for the trace connection, if any*/
	UPROPERTY()
	uint16 Port = 0;

	/** How the data is being transported to the editor (Trace Server, Custom Relay, Direct, etc.)*/
	UPROPERTY()
	ETraceTransportMode TransportMode = ETraceTransportMode::Invalid;

	/** Encoded SSL certificate for the trace connection, if any */
	UPROPERTY()
	TArray<uint8> CertAuth;

	UPROPERTY()
	bool bIsValid = false;
};


/** Structure containing the details necessary to connect to a relay instance */
USTRUCT()
struct FRelayConnectionInfo
{
	GENERATED_BODY()

	/** Relay IP address */
	UPROPERTY()
	FString Address;

	/** Relay port */
	UPROPERTY()
	uint16 Port = 0;

	/** Encoded SSL certificate */
	UPROPERTY()
	TArray<uint8> CertificateAuthority;
};

#if WITH_TRACE_BASED_DEBUGGERS

struct FRelayTraceDataWriter;

/** Possible connection attempt results */
enum class EConnectionAttemptResult : uint8
{
	NotStarted,
	InProgress,
	Success,
	Failed
};

/**
 * Interface for any trace-based debugger relay transport implementation
 */
class IDataRelayTransport
{
public:
	virtual ~IDataRelayTransport() = default;

	virtual void Initialize() = 0;
	virtual void Shutdown() = 0;

	DECLARE_DELEGATE_OneParam(FProcessReceivedRelayDataDelegate, const TConstArrayView<uint8> DataPacket);
	virtual void RegisterRelayDataReceiverForSessionID(FGuid RemoteSessionID, const FProcessReceivedRelayDataDelegate& InProcessDataDelegate) = 0;
	virtual void UnregisterRelayDataReceiverForSessionID(FGuid RemoteSessionID) = 0;

	virtual void RelayTraceDataFromWriter(FRelayTraceDataWriter& InRelayTraceDataWriter) = 0;

	virtual FRelayConnectionInfo GetConnectionInfo() = 0;
	virtual EConnectionAttemptResult ConnectToRelay(FGuid RemoteSessionID, const FRelayConnectionInfo ConnectionInfo) = 0;
	virtual EConnectionAttemptResult GetConnectionAttemptResult(FGuid RemoteSessionID) = 0;
};

#endif // WITH_TRACE_BASED_DEBUGGERS
} // namespace UE::TraceBasedDebuggers