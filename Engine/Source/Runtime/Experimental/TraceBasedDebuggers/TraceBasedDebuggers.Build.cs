// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class TraceBasedDebuggers : ModuleRules
	{
		public TraceBasedDebuggers(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Messaging",
					"MessagingCommon",
					"TraceLog"
				}
				);

			bAllowUETypesInNamespaces = true;
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

			if (AreTraceBasedDebuggersSupported(Target))
			{
				PublicDefinitions.Add("WITH_TRACE_BASED_DEBUGGERS=1");
			}
			else
			{
				PublicDefinitions.Add("WITH_TRACE_BASED_DEBUGGERS=0");
			}
		}
	}
}
