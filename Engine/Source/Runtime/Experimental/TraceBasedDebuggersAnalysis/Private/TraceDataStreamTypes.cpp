// Copyright Epic Games, Inc. All Rights Reserved.

#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS

#include "TraceDataStreamTypes.h"

#include "HAL/IConsoleManager.h"
#include "RemoteSessionsManager.h"
#include "SessionInfo.h"

namespace UE::TraceBasedDebuggers
{

float IntervalBetweenUpdatesSeconds = 1.0f;

static FAutoConsoleVariableRef CVarMaxSendQueueBytes(
	TEXT("tracebaseddebuggers.StreamStatsUpdateInterval"),
	IntervalBetweenUpdatesSeconds,
	TEXT("Delay, in seconds before fetching statistics from the data stream instance for the associated remote session."));

//----------------------------------------------------------------------//
// FSessionStreamStatsUpdater
//----------------------------------------------------------------------//
FSessionStreamStatsUpdater::~FSessionStreamStatsUpdater()
{
}

bool FSessionStreamStatsUpdater::Tick(const float DeltaTime)
{
	ElapsedTimeSinceLastUpdate += DeltaTime;

	if (ElapsedTimeSinceLastUpdate < IntervalBetweenUpdatesSeconds)
	{
		return true;
	}

	if (!ensure(DataStreamInstance))
	{
		return false;
	}

	ElapsedTimeSinceLastUpdate = 0.0;

	using namespace UE::TraceBasedDebuggers;
	const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = DataStreamInstance->GetSessionsManager();
	const TSharedPtr<FSessionInfo> SessionInfo = RemoteSessionManager ? RemoteSessionManager->GetSessionInfo(DataStreamInstance->GetOwningRemoteSessionID()).Pin() : nullptr;
	if (!SessionInfo)
	{
		return false;
	}

	SessionInfo->SetReceivedBytesPerSecond(DataStreamInstance->UpdateAndGetBytesReadSinceLastMeasurement());

	return true;
}

//----------------------------------------------------------------------//
// FDirectSocketStream
//----------------------------------------------------------------------//
int32 FDirectSocketStream::Read(void* Data, const uint32 Size)
{
	const int32 BytesRead = Trace::FDirectSocketStream::Read(Data, Size);

	if (BytesRead > 0)
	{
		BytesReadSinceLastMeasurement += BytesRead;
	}

	return BytesRead;
}

FGuid FDirectSocketStream::GetOwningRemoteSessionID() const
{
	return RemoteSessionID;
}

uint64 FDirectSocketStream::UpdateAndGetBytesReadSinceLastMeasurement()
{
	const uint64 BytesRead = BytesReadSinceLastMeasurement;
	BytesReadSinceLastMeasurement = 0;
	return BytesRead;
}

TSharedPtr<FRemoteSessionsManager> FDirectSocketStream::GetSessionsManager() const
{
	ensureAlways(WeakSessionManager.IsValid());
	return WeakSessionManager.Pin();
}

//----------------------------------------------------------------------//
// FRelayDataStream
//----------------------------------------------------------------------//
FRelayDataStream::FRelayDataStream(
	const TSharedRef<FRemoteSessionsManager>& InSessionsManager
	, const TSharedRef<IDataRelayTransport>& InDataRelay
	, const FGuid& InRemoteSessionID
	, TUniquePtr<FArchiveFileWriterGeneric>&& InFileWriter)
	: WeakSessionManager(InSessionsManager.ToWeakPtr()),
	CancelEvent(FPlatformProcess::GetSynchEventFromPool(false)),
	WaitForDataEvent(FPlatformProcess::GetSynchEventFromPool(false)),
	RemoteSessionID(InRemoteSessionID),
	FileWriter(MoveTemp(InFileWriter)),
#if WITH_TRACE_BASED_DEBUGGERS
	RelayTransportInstance(InDataRelay),
#endif
	BytesReadSinceLastMeasurement(0),
	StatsUpdater(this)
{
#if WITH_TRACE_BASED_DEBUGGERS
	RelayTransportInstance->RegisterRelayDataReceiverForSessionID(RemoteSessionID
		, IDataRelayTransport::FProcessReceivedRelayDataDelegate::CreateRaw(this, &FRelayDataStream::EnqueueRelayedData));
#endif
}

FRelayDataStream::~FRelayDataStream()
{
	FPlatformProcess::ReturnSynchEventToPool(CancelEvent);
	FPlatformProcess::ReturnSynchEventToPool(WaitForDataEvent);

#if WITH_TRACE_BASED_DEBUGGERS
	RelayTransportInstance->UnregisterRelayDataReceiverForSessionID(RemoteSessionID);
#endif
}

void FRelayDataStream::EnqueueRelayedData(const TConstArrayView<uint8> InTraceDataBuffer)
{
	// An empty data array signals an end of file event from the relay instance
	if (InTraceDataBuffer.IsEmpty())
	{
		Close();
		return;
	}

	WaitForDataEvent->Trigger();

	FSerializedDataBuffer DataBuffer(InTraceDataBuffer);
	DataQueue.Enqueue(MoveTemp(DataBuffer));
}

void FRelayDataStream::EnqueueRelayedData(TArray<uint8>&& InTraceDataBuffer)
{
	// An empty data array signals an end of file event from the relay instance
	if (InTraceDataBuffer.IsEmpty())
	{
		Close();
		return;
	}

	WaitForDataEvent->Trigger();

	FSerializedDataBuffer DataBuffer(MoveTemp(InTraceDataBuffer));
	DataQueue.Enqueue(MoveTemp(DataBuffer));
}

FGuid FRelayDataStream::GetOwningRemoteSessionID() const
{
	return RemoteSessionID;
}

uint64 FRelayDataStream::UpdateAndGetBytesReadSinceLastMeasurement()
{
	const uint64 BytesRead = BytesReadSinceLastMeasurement;
	BytesReadSinceLastMeasurement = 0;
	return BytesRead;
}

TSharedPtr<FRemoteSessionsManager> FRelayDataStream::GetSessionsManager() const
{
	ensureAlways(WeakSessionManager.IsValid());
	return WeakSessionManager.Pin();
}

int32 FRelayDataStream::Read(void* Dest, const uint32 DestSize)
{
	if (DestSize == 0)
	{
		return 0;
	}

	if (Dest == nullptr)
	{
		return 0;
	}

	if (!ensure(CancelEvent))
	{
		return 0;
	}

	uint8* DestBuffer = static_cast<uint8*>(Dest);
	uint32 BytesCopied = 0;

	// We need to wait for some data to be queued. Returning 
	// zero bytes read is interpreted as eof.
	while (!CancelEvent->Wait(0, true))
	{
		if (!CurrentPacket)
		{
			CurrentPacket = DataQueue.Dequeue();
			CurrentPacketOffset = 0;
		}

		// Consume as many queued packets as possible and copy into
		// the destination buffer
		while (CurrentPacket && BytesCopied < DestSize)
		{
			const TArray<uint8>& SrcBuffer = CurrentPacket->GetDataAsByteArrayRef();
			if (SrcBuffer.IsEmpty())
			{
				// We should never get an empty packet. From this point on we can't trust the integrity of the data
				// so we need to close the stream
				Close();
				break;
			}

			const uint32 ReceivedBufferSize = static_cast<uint32>(SrcBuffer.Num());

			// Copy as much of the current buffer as possible into the remaining space 
			// in the destination buffer
			const uint32 BytesToCopy = FMath::Min(ReceivedBufferSize - CurrentPacketOffset, DestSize - BytesCopied);
			FMemory::Memcpy(&DestBuffer[BytesCopied], SrcBuffer.GetData() + CurrentPacketOffset, BytesToCopy);
			BytesCopied += BytesToCopy;
			CurrentPacketOffset += BytesToCopy;

			if (CurrentPacketOffset == ReceivedBufferSize)
			{
				// Current packet buffer has been fully copied, goto next
				CurrentPacket = DataQueue.Dequeue();
				CurrentPacketOffset = 0;
			}
		}

		// If at least some bytes have been written return those. Otherwise, 
		// we'll wait for some data to arrive.
		if (BytesCopied > 0)
		{
			BytesReadSinceLastMeasurement += BytesCopied;

			if (FileWriter.IsValid() && !FileWriter->IsError())
			{
				FileWriter->Serialize(Dest, BytesCopied);
			}
			break;
		}
		else
		{
			WaitForDataEvent->Reset();
			WaitForDataEvent->Wait();
		}
	}

	return BytesCopied;
}

void FRelayDataStream::Close()
{
	if (CancelEvent)
	{
		CancelEvent->Trigger();
	}

	if (WaitForDataEvent)
	{
		WaitForDataEvent->Trigger();
	}

	if (FileWriter.IsValid())
	{
		FileWriter->Close();
		FileWriter.Reset();
	}

	bClosed = true;
}

bool FRelayDataStream::WaitUntilReady()
{
	return DataQueue.Peek() != nullptr;
}

} // UE::TraceBasedDebuggers

#endif // UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS