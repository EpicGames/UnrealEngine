// Copyright Epic Games, Inc. All Rights Reserved.

#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS

#include "TraceSessionsManager.h"

#include "Containers/Ticker.h"
#include "EngineEditorBridge.h"
#include "Features/IModularFeatures.h"
#if WITH_EDITOR
#include "Framework/Notifications/NotificationManager.h"
#endif
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Trace/DataStream.h"
#include "Trace/StoreClient.h"
#include "TraceBasedDebuggerRuntime.h"
#include "TraceBasedDebuggerTypes.h"
#include "TraceDataStreamTypes.h"
#include "TraceServices/AnalysisService.h"
#include "TraceServices/ITraceServicesModule.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/ModuleService.h"

namespace UE::TraceBasedDebuggers
{

struct FFileDataStream : public Trace::IInDataStream
{
	explicit FFileDataStream(TUniquePtr<IFileHandle>&& FileHandle)
		: AsyncProgressNotification(FAsyncProgressNotification(NSLOCTEXT("TraceBasedDebuggers", "FileLoadingProgressNotification", "Loading File")))
		, Handle(MoveTemp(FileHandle))
	{
		check(Handle);
		FileSize = Handle->Size();
		RemainingDataSize = FileSize;
		constexpr int32 FileFullyLoadedPercentage = 100;

		check(FileSize > 0);
		ProgressPercentagePerByte = static_cast<double>(FileFullyLoadedPercentage) / static_cast<double>(FileSize);
	}

	virtual int32 Read(void* Data, uint32 Size) override
	{
		if (RemainingDataSize <= 0)
		{
			return 0;
		}

		if (Size > RemainingDataSize)
		{
			Size = static_cast<uint32>(RemainingDataSize);
		}

		RemainingDataSize -= Size;

		const double ProgressPercentage = static_cast<double>(FileSize - RemainingDataSize) * ProgressPercentagePerByte;
		AsyncProgressNotification.EnterProgress(FMath::CeilToInt32(ProgressPercentage));

		check(Handle);
		if (!Handle->Read(static_cast<uint8*>(Data), Size))
		{
			return 0;
		}
		return Size;
	}

	virtual void Close() override
	{
		Handle.Reset();
	}

private:

	/**
	 * Object that allows showing a progress bar in the status bar of the editor,
	 * where the progress can be updated from any thread
	 */
	struct FAsyncProgressNotification : FTSTickerObjectBase
	{
		FAsyncProgressNotification(const FText& InMessage) : NotificationTitle(InMessage)
		{
		}

		virtual ~FAsyncProgressNotification() override;

		/** Updates the current progress
		 * @param InCurrentProgress Value between 0 and 100, representing the progress we want to display
		 */
		void EnterProgress(int32 InCurrentProgress);

	private:
		virtual bool Tick(float DeltaTime) override;

		float ElapsedTimeSinceLastUpdate = 0.0f;

#if WITH_EDITOR
		FProgressNotificationHandle NotificationHandle;
#endif // WITH_EDITOR

		std::atomic<int32> CurrentProgress = 0;
		FText NotificationTitle;
	};
	FAsyncProgressNotification AsyncProgressNotification;

	TUniquePtr<IFileHandle> Handle;
	uint64 RemainingDataSize = 0;
	uint64 FileSize = 0;
	double ProgressPercentagePerByte = 0.0;
};

FFileDataStream::FAsyncProgressNotification::~FAsyncProgressNotification()
{
#if WITH_EDITOR
	if (NotificationHandle.IsValid())
	{
		// FSlateNotificationManager is not thread safe, so we need to make sure this runs in the game thread
		if (IsInGameThread())
		{
			FSlateNotificationManager::Get().CancelProgressNotification(NotificationHandle);
		}
		else
		{
			constexpr float Delay = 0.0f;
			FTSTicker::GetCoreTicker().AddTicker(TEXT("CancelCVDAsyncNotification"), Delay, [HandleCopy = NotificationHandle](float DeltaTime)
				{
					FSlateNotificationManager::Get().CancelProgressNotification(HandleCopy);
					return false;
				});
		}
	}
#endif // WITH_EDITOR
}

void FFileDataStream::FAsyncProgressNotification::EnterProgress(const int32 InCurrentProgress)
{
	CurrentProgress = InCurrentProgress;
}

bool FFileDataStream::FAsyncProgressNotification::Tick(const float DeltaTime)
{
#if WITH_EDITOR
	constexpr int32 MaxProgress = 100;
	constexpr float TimeBetweenUpdatesSeconds = 0.5f;

	ElapsedTimeSinceLastUpdate += DeltaTime;
	if (ElapsedTimeSinceLastUpdate < TimeBetweenUpdatesSeconds)
	{
		return true;
	}

	ElapsedTimeSinceLastUpdate = 0.0f;

	if (CurrentProgress >= MaxProgress)
	{
		if (NotificationHandle.IsValid())
		{
			FSlateNotificationManager::Get().UpdateProgressNotification(NotificationHandle, MaxProgress, MaxProgress, NotificationTitle);
		}
		NotificationHandle = FProgressNotificationHandle();

		// We are done and we can stop ticking
		return false;
	}

	if (!NotificationHandle.IsValid())
	{
		NotificationHandle = FSlateNotificationManager::Get().StartProgressNotification(NotificationTitle, MaxProgress);
	}
	else
	{
		FSlateNotificationManager::Get().UpdateProgressNotification(NotificationHandle, CurrentProgress, MaxProgress, NotificationTitle);
	}

	return true;
#else
	return false;
#endif // WITH_EDITOR
}

FTraceSessionsManager::FTraceSessionsManager(
	FRuntimeModule& RuntimeModule
	, FEngineEditorBridge& Bridge
	, const TSharedPtr<TraceServices::IModule>& TraceModule
	, FStringView SaveDirectorySubPathInUserDir
)
	: RuntimeModule(RuntimeModule)
	, EngineEditorBridge(Bridge)
	, TraceModule(TraceModule)
	, SaveDirectorySubPathInUserDir(SaveDirectorySubPathInUserDir)
{
	if (TraceModule)
	{
		IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, TraceModule.Get());
	}
}

FTraceSessionsManager::~FTraceSessionsManager()
{
	if (TraceModule)
	{
		IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, TraceModule.Get());
	}
}

FString FTraceSessionsManager::LoadTraceFile(const FString& InTraceFilename, const TFunction<void()>& PreStartAnalysisFunc)
{
	CloseSession(InTraceFilename);

	IPlatformFile& FileSystem = IPlatformFile::GetPlatformPhysical();

	constexpr bool bAllowWrite = false;
	IFileHandle* Handle = FileSystem.OpenRead(*InTraceFilename, bAllowWrite);
	if (!Handle)
	{
		return FString();
	}

	return LoadTraceFile(TUniquePtr<IFileHandle>(Handle), InTraceFilename, PreStartAnalysisFunc);
}

FString FTraceSessionsManager::LoadTraceFile(TUniquePtr<IFileHandle>&& InFileHandle, const FString& InTraceSessionName, const TFunction<void()>& PreStartAnalysisFunc)
{
	if (InFileHandle->Size() == 0)
	{
		return FString();
	}

	ITraceServicesModule& TraceServicesModule = FModuleManager::LoadModuleChecked<ITraceServicesModule>("TraceServices");
	if (const TSharedPtr<TraceServices::IAnalysisService> TraceAnalysisService = TraceServicesModule.GetAnalysisService())
	{
		if (PreStartAnalysisFunc)
		{
			PreStartAnalysisFunc();
		}

		if (const TSharedPtr<const TraceServices::IAnalysisSession> NewSession = TraceAnalysisService->StartAnalysis(~0, *InTraceSessionName, CreateFileDataStream(MoveTemp(InFileHandle))))
		{
			AnalysisSessionByName.Add(InTraceSessionName, NewSession.ToSharedRef());

			return NewSession->GetName();
		}
	}

	return FString();
}

FString FTraceSessionsManager::ConnectToLiveSession(const FStringView InSessionHost, const uint32 SessionID, TFunction<void()> PreStartAnalysisFunc)
{
	if (InSessionHost.IsEmpty())
	{
		return FString();
	}

	using namespace UE::Trace;
	FString LiveSessionName = FString::Printf(TEXT("LiveSession[%.*s - %u]"), InSessionHost.Len(), InSessionHost.GetData(), SessionID);

	FStoreClient::FTraceData TraceData = GetTraceDataStreamFromStore(InSessionHost, SessionID);
	if (!TraceData.IsValid())
	{
		UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed to get trace data stream from the trace store | Host [%ls] | Session ID [%u]", __func__, InSessionHost.GetData(), SessionID);
		return FString();
	}

	if (ConnectToLiveSession_Internal(SessionID, PreStartAnalysisFunc, LiveSessionName, MoveTemp(TraceData)))
	{
		return LiveSessionName;
	}

	UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed connect to session | Host [%ls] | Session ID [%u]", __func__, InSessionHost.GetData(), SessionID);

	return FString();
}

FString FTraceSessionsManager::ConnectToLiveSession_Direct(FGuid RemoteSessionID, uint16& OutSessionPort, TFunction<void()> PreStartAnalysisFunc)
{
#if WITH_TRACE_BASED_DEBUGGERS
	TUniquePtr<FArchiveFileWriterGeneric> TraceFileWriter = nullptr;

	if (bSaveRecordingsToDisk.IsSet() && bSaveRecordingsToDisk.Get())
	{
		TraceFileWriter = OpenTraceFileForWrite(GetSaveDirPath());
		if (!TraceFileWriter)
		{
			UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed to open open file for direct trace stream", __func__);
			return FString();
		}
	}

	TUniquePtr<FDirectSocketStream> DirectSocketStream = MakeUnique<FDirectSocketStream>(
		EngineEditorBridge.GetSessionsManager().ToSharedRef()
		, RemoteSessionID
		, MoveTemp(TraceFileWriter));

	OutSessionPort = DirectSocketStream->StartListening(CurrentDirectModeStartingPort);

	if (OutSessionPort == 0)
	{
		UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed to open direct trace stream socket", __func__);
		return FString();
	}

	// If the port was in use, FDirectSocketStream will try multiple ports in sequence, therefore the next port we should try while we have recording active (common case in Multi-session mode)
	// is the next one from the last success
	CurrentDirectModeStartingPort = OutSessionPort + 1;

	FString DirectSessionName = FString::Printf(TEXT("DirectSession[127.0.0.1:%u]"), OutSessionPort);

	if (ConnectToLiveSession_Internal(~0u, PreStartAnalysisFunc, DirectSessionName, MoveTemp(DirectSocketStream)))
	{
		return DirectSessionName;
	}

	UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed start trace analysis with direct trace stream socket | [%ls]", __func__, *DirectSessionName);
#endif // WITH_TRACE_BASED_DEBUGGERS

	return FString();
}

FString FTraceSessionsManager::ConnectToLiveSession_Relay(const FGuid RemoteSessionID, TFunction<void()> PreStartAnalysisFunc)
{
	FString RelaySessionName = FString::Printf(TEXT("RelaySession[%s]"), *RemoteSessionID.ToString());

	TUniquePtr<FRelayDataStream> RelayDataStream = CreateRelayDataStream(RemoteSessionID);

	if (ConnectToLiveSession_Internal(~0u, PreStartAnalysisFunc, RelaySessionName, MoveTemp(RelayDataStream)))
	{
		return RelaySessionName;
	}
	else
	{
		UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed start trace analysis with relay trace stream | [%ls]", __func__, *RelaySessionName);
		return FString();
	}
}


FString FTraceSessionsManager::GetLocalTraceStoreDirPath()
{
	const TUniquePtr<Trace::FStoreClient> StoreClient = Trace::FStoreClient::Create(TEXT("localhost"));

	if (!StoreClient)
	{
		UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed to connect to local Trace Store client", __func__);
		return TEXT("");
	}

	const Trace::FStoreClient::FStatus* Status = StoreClient->GetStatus();
	if (!Status)
	{
		UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed to to get Trace Store status", __func__);
		return TEXT("");
	}

	return FString(Status->GetStoreDir());
}

TSharedPtr<const TraceServices::IAnalysisSession> FTraceSessionsManager::GetSession(const FString& InSessionName)
{
	if (TSharedPtr<const TraceServices::IAnalysisSession>* FoundSession = AnalysisSessionByName.Find(InSessionName))
	{
		return *FoundSession;
	}

	return nullptr;
}

void FTraceSessionsManager::CloseSession(const FString& InSessionName)
{
	if (const TSharedPtr<const TraceServices::IAnalysisSession>* Session = AnalysisSessionByName.Find(InSessionName))
	{
		if (Session->IsValid())
		{
			(*Session)->Stop(true);
		}

		AnalysisSessionByName.Remove(InSessionName);

		if (AnalysisSessionByName.IsEmpty())
		{
			// If we don't have any more session left, we can reset back to the original starting port
			// As all sockets we had in use are closed at this point
			CurrentDirectModeStartingPort = DefaultStartingPort;
		}
	}
}

void FTraceSessionsManager::StopSession(const FString& InSessionName)
{
	if (const TSharedPtr<const TraceServices::IAnalysisSession>* Session = AnalysisSessionByName.Find(InSessionName))
	{
		if (Session->IsValid())
		{
			(*Session)->Stop(true);
		}
	}
}

const Trace::FStoreClient::FSessionInfo* FTraceSessionsManager::GetTraceSessionInfo(const FStringView InSessionHost, const FGuid TraceGuid)
{
	if (InSessionHost.IsEmpty())
	{
		UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed to connect to trace store. Provided session host is empty", __func__);
		return nullptr;
	}

	using namespace UE::Trace;
	const TUniquePtr<FStoreClient> StoreClient = FStoreClient::Create(InSessionHost.GetData());

	if (!StoreClient)
	{
		UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed to connect to trace store at [%ls]", __func__, InSessionHost.GetData())
		return nullptr;
	}

	return StoreClient->GetSessionInfoByGuid(TraceGuid);
}

FString FTraceSessionsManager::GetTraceFileNameFromStoreForSession(const FStringView InSessionHost, const uint32 SessionID)
{
	using namespace UE::Trace;

	if (InSessionHost.IsEmpty())
	{
		return FString();
	}

	const TUniquePtr<FStoreClient> StoreClient = FStoreClient::Create(InSessionHost.GetData());
	if (!StoreClient)
	{
		return FString();
	}

	// Note: The following calls to the store client are scoped because FStoreClient::FTraceInfo and FStoreClient::FStatus share the same buffer
	// under the hood, therefor although the ptr will still be valid, the data will not after we obtain one after the other
	FString FileName;
	{
		const FStoreClient::FTraceInfo* TraceInfo = StoreClient->GetTraceInfoById(SessionID);
		if (!TraceInfo)
		{
			return FString();
		}

		const FUtf8StringView Utf8NameView = TraceInfo->GetName();
		FileName = FString(Utf8NameView);
		if (!FileName.EndsWith(TEXT(".utrace")))
		{
			FileName += TEXT(".utrace");
		}
	}

	FString TraceStorePath;
	{
		const FStoreClient::FStatus* StoreStatus = StoreClient->GetStatus();
		if (!StoreStatus)
		{
			return FString();
		}

		TraceStorePath = FString(StoreStatus->GetStoreDir());
	}

	FString FullFileName = FPaths::Combine(TraceStorePath, FileName);
	FPaths::NormalizeFilename(FullFileName);

	return FullFileName;
}


Trace::FStoreClient::FTraceData FTraceSessionsManager::GetTraceDataStreamFromStore(const FStringView InSessionHost, const uint32 SessionID)
{
	using namespace UE::Trace;
	const TUniquePtr<FStoreClient> StoreClient = FStoreClient::Create(InSessionHost.GetData());
	if (!StoreClient)
	{
		return FStoreClient::FTraceData();
	}

	return StoreClient->ReadTrace(SessionID);
}

FString FTraceSessionsManager::GetSaveDirPath() const
{
	if (SavePathOverride.IsSet()
		&& SavePathOverride.Get().FilePath.Len() > 0)
	{
		return SavePathOverride.Get().FilePath;
	}

	return GetDefaultSaveDirPath();
}

FString FTraceSessionsManager::GetDefaultSaveDirPath() const
{
	return FPaths::Combine(FPlatformProcess::UserDir(), SaveDirectorySubPathInUserDir);
}

TUniquePtr<FArchiveFileWriterGeneric> FTraceSessionsManager::OpenTraceFileForWrite(const FString& InDirectoryToSavePath) const
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	if (InDirectoryToSavePath.IsEmpty())
	{
		UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed open file for write | No save directory was provided", __func__);
		return nullptr;
	}

	PlatformFile.CreateDirectory(*InDirectoryToSavePath);
	if (!PlatformFile.DirectoryExists(*InDirectoryToSavePath))
	{
		UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed open file for write | Failed to create or access provided save directory [%ls]", __func__, *InDirectoryToSavePath);
		return nullptr;
	}

	TUniquePtr<FArchiveFileWriterGeneric> FileWriter = nullptr;

#if WITH_TRACE_BASED_DEBUGGERS

	constexpr int32 MaxAttempts = 10;
	int32 CurrentAttempts = 0;
	while (CurrentAttempts < MaxAttempts)
	{
		FString OutFileName;
		RuntimeModule.GenerateRecordingFileName(OutFileName);
		OutFileName = FPaths::Combine(InDirectoryToSavePath, OutFileName);

		if (IFileHandle* FileHandle = PlatformFile.OpenWrite(*OutFileName))
		{
			FileWriter = MakeUnique<FArchiveFileWriterGeneric>(FileHandle, *OutFileName, FileHandle->Tell());
			break;
		}

		CurrentAttempts++;

		FPlatformProcess::Sleep(0.1f);
	}

#endif

	return MoveTemp(FileWriter);
}

bool FTraceSessionsManager::ConnectToLiveSession_Internal(const uint32 SessionID, const TFunction<void()>& PreStartAnalysisFunc, const FString& InRequestedSessionName, Trace::FStoreClient::FTraceData&& InTraceDataStream)
{
	if (!InTraceDataStream.IsValid())
	{
		return false;
	}

	ITraceServicesModule& TraceServicesModule = FModuleManager::LoadModuleChecked<ITraceServicesModule>("TraceServices");
	if (const TSharedPtr<TraceServices::IAnalysisService> TraceAnalysisService = TraceServicesModule.GetAnalysisService())
	{
		// Close this session in case we were already analysing it
		CloseSession(InRequestedSessionName);

		if (PreStartAnalysisFunc)
		{
			PreStartAnalysisFunc();
		}

		if (const TSharedPtr<const TraceServices::IAnalysisSession> NewSession = TraceAnalysisService->StartAnalysis(SessionID, *InRequestedSessionName, MoveTemp(InTraceDataStream)))
		{
			AnalysisSessionByName.Add(InRequestedSessionName, NewSession.ToSharedRef());
			return true;
		}
	}

	return false;
}

TUniquePtr<Trace::IInDataStream> FTraceSessionsManager::CreateFileDataStream(TUniquePtr<IFileHandle>&& InFileHandle)
{
	check(InFileHandle);
	return TUniquePtr<Trace::IInDataStream>(new FFileDataStream(MoveTemp(InFileHandle)));
}

TUniquePtr<FRelayDataStream> FTraceSessionsManager::CreateRelayDataStream(FGuid RemoteSessionID) const
{
	TUniquePtr<FArchiveFileWriterGeneric> TraceFileWriter = nullptr;

	if (bSaveRecordingsToDisk.IsSet() && bSaveRecordingsToDisk.Get())
	{
		TraceFileWriter = OpenTraceFileForWrite(GetSaveDirPath());
		if (!TraceFileWriter)
		{
			UE_LOGF(LogTraceBasedDebuggers, Error, "[%s] Failed to open file for relay trace stream", __func__);
			return nullptr;
		}
	}

#if WITH_TRACE_BASED_DEBUGGERS
	const TSharedPtr<IDataRelayTransport> DataRelay = EngineEditorBridge.GetTraceRelayTransportInstance();
	const TSharedPtr<FRemoteSessionsManager> SessionsManager = EngineEditorBridge.GetSessionsManager();
	if (ensure(DataRelay && SessionsManager))
	{
		TUniquePtr<FRelayDataStream> NewDataStream = MakeUnique<FRelayDataStream>(
			SessionsManager.ToSharedRef()
			, DataRelay.ToSharedRef()
			, RemoteSessionID
			, MoveTemp(TraceFileWriter));

		return MoveTemp(NewDataStream);
	}
#endif // WITH_TRACE_BASED_DEBUGGERS

	return nullptr;
}

uint16 FTraceSessionsManager::CurrentDirectModeStartingPort = DefaultStartingPort;

} // UE::TraceBasedDebuggers

#endif // UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS