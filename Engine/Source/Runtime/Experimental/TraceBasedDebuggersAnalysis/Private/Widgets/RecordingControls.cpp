// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "Widgets/RecordingControls.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/SlateDelegates.h"
#include "RemoteSessionsManager.h"
#include "SEnumCombo.h"
#include "SessionInfo.h"
#include "SocketSubsystem.h"
#include "ToolMenuEntry.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "TraceSessionsManager.h"
#include "Styling/ToolBarStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RecordingControls)

#define LOCTEXT_NAMESPACE "RecordingControls"
namespace UE::TraceBasedDebuggers
{

const FName FRecordingControls::BaseRecordingOptionsMenuName = "TraceBasedDebuggers.RecordingOptions";

FRecordingControls::FRecordingControls(const FName InStatusBarId
	, const TSharedRef<FRemoteSessionsManager>& InRemoteSessionManager
	, const FGuid& InDebuggerId
	, const FLogCategoryBase& InLogCategory)
	: RemoteSessionManagerWeakPtr(InRemoteSessionManager)
	, LogCategory(InLogCategory)
	, DebuggerGuid(InDebuggerId)
	, StatusBarID(InStatusBarId)
{
	RecordingAnimation = FCurveSequence();
	RecordingAnimation.AddCurve(0.f, 1.5f, ECurveEaseFunction::Linear);

	CurrentSelectedSessionId = FRemoteSessionsManager::LocalEditorSessionID;
}

FRecordingControls::~FRecordingControls()
{
	Deinitialize();
}

void FRecordingControls::Initialize()
{
	if (bInitialized)
	{
		return;
	}

	if (const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = RemoteSessionManagerWeakPtr.Pin())
	{
		RemoteSessionManager->OnSessionRecordingStarted().AddSP(this, &FRecordingControls::HandleRecordingStart);
		RemoteSessionManager->OnSessionRecordingStopped().AddSP(this, &FRecordingControls::HandleRecordingStop);
		RemoteSessionManager->OnSessionExpired().AddSP(this, &FRecordingControls::CancelInFlightConnectionAttempt);
		RemoteSessionManager->OnSessionsUpdated().AddSP(this, &FRecordingControls::HandleRemoteSessionsUpdated);
	}

	bInitialized = true;
}

void FRecordingControls::Deinitialize()
{
	if (!bInitialized)
	{
		return;
	}

	if (const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = RemoteSessionManagerWeakPtr.Pin())
	{
		RemoteSessionManager->OnSessionRecordingStarted().RemoveAll(this);
		RemoteSessionManager->OnSessionRecordingStopped().RemoveAll(this);
		RemoteSessionManager->OnSessionExpired().RemoveAll(this);
		RemoteSessionManager->OnSessionsUpdated().RemoveAll(this);
	}

	bInitialized = false;
}

TSharedRef<SWidget> FRecordingControls::GenerateToggleRecordingStateButton(const FText& StartRecordingTooltip)
{
	TSharedRef<SImage> Animation = SNew(SImage)
		.Image_Raw(this, &FRecordingControls::GetRecordOrStopButton);

	Animation->SetColorAndOpacity(MakeAttributeSPLambda(Animation, [this, Animation]
		{
			if (IsRecording())
			{
				if (!RecordingAnimation.IsPlaying())
				{
					RecordingAnimation.Play(Animation, true);
				}

				const FLinearColor Color = bRecordingButtonHovered ? FLinearColor::Red : FLinearColor::White;
				return FSlateColor(bRecordingButtonHovered ? Color : Color.CopyWithNewOpacity(0.2f + 0.8f * RecordingAnimation.GetLerp()));
			}

			RecordingAnimation.Pause();
			return FSlateColor::UseSubduedForeground();
		}));

	TSharedPtr<SWidget> ButtonContent = Animation;
	
	if (GetControlsConfig().bHasRecordingLabel)
	{
		ButtonContent = SNew(SHorizontalBox)
			
		+ SHorizontalBox::Slot()
		.Padding(FMargin(0, 0, 0, 0))
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			Animation
		]
			
		+ SHorizontalBox::Slot()
		.Padding(FMargin(4, 0, 0, 0))
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Visibility_Lambda([this]()
				{
					return IsRecording() ? EVisibility::Collapsed : EVisibility::Visible;
				})
			.TextStyle(FAppStyle::Get(), "SmallButtonText")
			.Text(LOCTEXT("StartRecordingButtonLabel", "Start Recording"))
		];
	}
	
	return SNew(SButton)
		.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar").ButtonStyle)
		.OnClicked(FOnClicked::CreateRaw(this, &FRecordingControls::ToggleRecordingState))
		.ForegroundColor(FSlateColor::UseForeground())
		.IsFocusable(false)
		.IsEnabled_Raw(this, &FRecordingControls::IsRecordingToggleButtonEnabled)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.OnHovered_Lambda([this]() { bRecordingButtonHovered = true; })
		.OnUnhovered_Lambda([this]() { bRecordingButtonHovered = false; })
		.ToolTipText_Lambda([this, StartRecordingTooltip]()
			{
				return IsRecording() ? LOCTEXT("StopRecordButtonInlineDesc", "Stop the current recording") : StartRecordingTooltip;
			})
		[
			ButtonContent.ToSharedRef()
		];
}

FText FRecordingControls::GetCurrentSelectedSessionName() const
{
	if (const TSharedPtr<FSessionInfo> CurrentSessionPtr = GetCurrentSessionInfo())
	{
		return FText::AsCultureInvariant(CurrentSessionPtr->SessionName);
	}

	static FText InvalidSessionName = LOCTEXT("InvalidSessionLabel", "Select a Session");

	return InvalidSessionName;
}

TSharedRef<SWidget> FRecordingControls::GenerateTargetSessionSelector()
{
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SComboButton)
			.MenuPlacement(MenuPlacement_AboveAnchor).ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar").ComboButtonStyle)
			.OnGetMenuContent(this, &FRecordingControls::GenerateTargetSessionDropdown)
			.HasDownArrow(true)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text_Raw(this, &FRecordingControls::GetCurrentSelectedSessionName)
				.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText"))
			]];
}

TSharedRef<SWidget> FRecordingControls::GenerateTargetSessionDropdown()
{
	using namespace UE::TraceBasedDebuggers;

	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("RecordingWidgetTargets", LOCTEXT("RecordingTargetsMenu", "Available Targets"));
	{
		if (const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = RemoteSessionManagerWeakPtr.Pin())
		{
			RemoteSessionManager->EnumerateActiveSessions([this, &MenuBuilder, bIsRecording = IsRecording()](const TSharedRef<FSessionInfo>& InSessionInfoRef)
				{
					if (EnumHasAnyFlags(InSessionInfoRef->GetSessionTypeAttributes(), ERemoteSessionAttributes::IsMultiSessionWrapper))
					{
						return true;
					}

					// Allow derived classes to ignore some sessions
					if (ShouldFilterOutSessionInternal(InSessionInfoRef))
					{
						return true;
					}

					const FText SessionNameAsText = FText::AsCultureInvariant(InSessionInfoRef->SessionName);
					FUIAction Action(
						FExecuteAction::CreateSP(this, &FRecordingControls::SelectTargetSession, InSessionInfoRef->InstanceId),
						FCanExecuteAction(),
						FGetActionCheckState::CreateSPLambda(this, [CurrentSelectedSessionId = CurrentSelectedSessionId, InstanceId = InSessionInfoRef->InstanceId]()
						{
							return InstanceId == CurrentSelectedSessionId ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
						}),
						EUIActionRepeatMode::RepeatDisabled
					);
					
					MenuBuilder.AddMenuEntry(
						SessionNameAsText,
						FText::Format(LOCTEXT("SingleTargetItemTooltip", "Select {0} session as current target"), SessionNameAsText),
						GetIconForSession(InSessionInfoRef->InstanceId),
						Action,
						NAME_None,
						bIsRecording ? EUserInterfaceActionType::Button : EUserInterfaceActionType::RadioButton
					);
					
					return true;
				});
		}
	}

	MenuBuilder.EndSection();

	OnGenerateTargetSessionSelectorInternal(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FRecordingControls::GenerateRecordingTimeTextBlock()
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		.Visibility_Raw(this, &FRecordingControls::GetRecordingTimeTextBlockVisibility)
		.Padding(12.0f, 0.0f, 0.0f, 0.0f)
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "SmallButtonText")
			.Text_Raw(this, &FRecordingControls::GetRecordingTimeText)
			.ColorAndOpacity(FColor::White)
		];
}

EVisibility FRecordingControls::GetRecordingTimeTextBlockVisibility() const
{
	using namespace UE::TraceBasedDebuggers;
	const TSharedPtr<FSessionInfo> SessionInfo = GetCurrentSessionInfo();
	const bool bIsVisible = SessionInfo && !EnumHasAnyFlags(SessionInfo->GetSessionTypeAttributes(), ERemoteSessionAttributes::IsMultiSessionWrapper) && IsRecording();

	return bIsVisible ? EVisibility::Visible : EVisibility::Collapsed;
}

void FRecordingControls::SelectTargetSession(const FGuid SessionId)
{
	CurrentSelectedSessionId = SessionId;

	if (const TSharedPtr<FSessionInfo> SessionInfo = GetSessionInfo(SessionId))
	{
		OnSessionSelected.ExecuteIfBound(SessionInfo);
	}
}

FSlateIcon FRecordingControls::GetIconForSession(const FGuid SessionId) const
{
	if (const TSharedPtr<FSessionInfo> SessionInfo = GetSessionInfo(SessionId))
	{
		return SessionInfo->IsRecording(DebuggerGuid)
			? RecordingSessionIcon
			: FSlateIcon();
	}

	return FSlateIcon();
}

TSharedPtr<FSessionInfo> FRecordingControls::GetCurrentSessionInfo() const
{
	return GetSessionInfo(CurrentSelectedSessionId);
}

TSharedPtr<FSessionInfo> FRecordingControls::GetSessionInfo(const FGuid Id) const
{
	using namespace UE::TraceBasedDebuggers;
	const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = RemoteSessionManagerWeakPtr.Pin();
	TSharedPtr<FSessionInfo> SessionInfo = RemoteSessionManager ? RemoteSessionManager->GetSessionInfo(Id).Pin() : nullptr;

	return SessionInfo;
}

const FSlateBrush* FRecordingControls::GetRecordOrStopButton() const
{
	if (bRecordingButtonHovered && IsRecording())
	{
		return StopRecordingBrush;
	}

	return StartRecordingBrush;
}

void FRecordingControls::HandleRecordingStop(const TSharedPtr<FSessionInfo>& InSessionInfo)
{
	if (!InSessionInfo.IsValid()
		|| DebuggerGuid != InSessionInfo->LastKnownRecordingState.DebuggerId)
	{
		// Recording was stopped by another debugger, ignore it
		return;
	}

	if (UStatusBarSubsystem* StatusBarSubsystem = GEditor ? GEditor->GetEditorSubsystem<UStatusBarSubsystem>() : nullptr)
	{
		StatusBarSubsystem->PopStatusBarMessage(StatusBarID, RecordingMessageHandle);

		const FText LiveSessionEnded = LOCTEXT("RecordingSessionEndedMessage", "Recording session has ended");
		LiveSessionEndedMessageHandle = StatusBarSubsystem->PushStatusBarMessage(StatusBarID, LiveSessionEnded);
	}

	OnRecordingStopped.ExecuteIfBound(InSessionInfo);
}

void FRecordingControls::HandleRecordingStart(const TSharedPtr<FSessionInfo>& InSessionInfo)
{
	if (!InSessionInfo.IsValid()
		|| DebuggerGuid != InSessionInfo->LastKnownRecordingState.DebuggerId)
	{
		// Recording was started by another debugger, ignore it
		return;
	}

	UStatusBarSubsystem* StatusBarSubsystem = GEditor ? GEditor->GetEditorSubsystem<UStatusBarSubsystem>() : nullptr;
	if (!StatusBarSubsystem)
	{
		return;
	}

	if (RecordingPathMessageHandle.IsValid())
	{
		StatusBarSubsystem->PopStatusBarMessage(StatusBarID, RecordingPathMessageHandle);
		RecordingPathMessageHandle = FStatusBarMessageHandle();
	}

	if (LiveSessionEndedMessageHandle.IsValid())
	{
		StatusBarSubsystem->PopStatusBarMessage(StatusBarID, LiveSessionEndedMessageHandle);
		LiveSessionEndedMessageHandle = FStatusBarMessageHandle();
	}
	
	// Show user that we are recording (and for how long)
	{
		const TAttribute<FText> RecordingText = TAttribute<FText>::CreateLambda([WeakSessionInfo = InSessionInfo.ToWeakPtr()]()
		{
			const TSharedPtr<FSessionInfo> SessionInfo = WeakSessionInfo.Pin();
			if (!SessionInfo)
			{
				return FText::GetEmpty();
			}

			static const FNumberFormattingOptions SecondsFormat = FNumberFormattingOptions()
				.SetMinimumIntegralDigits(2)
				.SetMinimumFractionalDigits(1)
				.SetMaximumFractionalDigits(1)
				.SetUseGrouping(false);

			const float ElapsedTime = SessionInfo->LastKnownRecordingState.ElapsedTime;
			const int32 Minutes = FMath::FloorToInt(ElapsedTime) / 60;
			const float RemainingSecs = ElapsedTime - static_cast<float>(Minutes) * 60.f;
			return FText::Format(LOCTEXT("RecordingMessageWithDuration", "Recording {0}:{1}"), FText::AsNumber(Minutes), FText::AsNumber(RemainingSecs, &SecondsFormat));
		});

		const FSlateBrush* IconBrush = StatusBarRecordingBrush != nullptr ? StatusBarRecordingBrush : StartRecordingBrush;
		if (IconBrush != nullptr)
		{
			RecordingMessageHandle = StatusBarSubsystem->PushStatusBarMessage(
				StatusBarID, 
				RecordingText, 
				TAttribute<FText>(),
				TAttribute<const FSlateBrush*>(IconBrush),
				FSlateColor(FLinearColor(1.f, 0.f, 0.f, 1.f))
			);
		}
		else
		{
			RecordingMessageHandle = StatusBarSubsystem->PushStatusBarMessage(StatusBarID, RecordingText);
		}
	}
	
	OnRecordingStarted.ExecuteIfBound(InSessionInfo);
}

void FRecordingControls::ExecuteAsyncConnectionAttemptTaskWithRetry(
	FGuid SessionID
	, const int32 RemainingRetries
	, const TFunction<bool()>& InRecordingStartAttemptCallback
	, const TFunction<void()>& InRecordingFailedCallback)
{
	const TSharedPtr<FAsyncConnectionAttemptTask> ConnectionAttemptRequest = MakeShared<FAsyncConnectionAttemptTask>(
		SessionID
		, LogCategory
		, InRecordingStartAttemptCallback
		, InRecordingFailedCallback
		, StaticCastWeakPtr<FRecordingControls>(AsWeak()));

	ConnectionAttemptRequest->Start(RemainingRetries, IntervalBetweenAutoplayConnectionAttemptsSeconds);
}

void FRecordingControls::FAsyncConnectionAttemptTask::HandleSuccess()
{
	State = EState::Success;

	if (const TSharedPtr<FRecordingControls> OwnerPtr = Owner.Pin())
	{
		OwnerPtr->HandleConnectionAttemptResult(SessionID, ELiveConnectionAttemptResult::Success, ProgressNotification);
	}
}

void FRecordingControls::FAsyncConnectionAttemptTask::HandleFailure()
{
	State = EState::Failed;

	if (RecordingFailedCallback)
	{
		RecordingFailedCallback();
	}

	if (const TSharedPtr<FRecordingControls> OwnerPtr = Owner.Pin())
	{
		OwnerPtr->HandleConnectionAttemptResult(SessionID, ELiveConnectionAttemptResult::Failed, ProgressNotification);
	}
}

bool FRecordingControls::FAsyncConnectionAttemptTask::CanExecute() const
{
	// The only two states where Execute() can be run are Not Started and InProgress
	// In Progress is allowed because we might need to execute again as part of the retry process
	return State == EState::NotStarted || State == EState::InProgress;
}

void FRecordingControls::CancelInFlightConnectionAttempt(const TSharedPtr<FSessionInfo>& SessionInfo)
{
	const FGuid SessionID = SessionInfo ? SessionInfo->InstanceId : FGuid{};
	if (const TSharedPtr<FAsyncConnectionAttemptTask>* ExistingRequestInFlightPtrPtr = InFlightAsyncConnectionRequests.Find(SessionID))
	{
		if (const TSharedPtr<FAsyncConnectionAttemptTask> ExistingRequestInFlightPtr = *ExistingRequestInFlightPtrPtr)
		{
			ExistingRequestInFlightPtr->Cancel();
		}
		else
		{
			UE_LOG_REF(LogCategory, Verbose, TEXT("[%hs] Attempted to cancel a connection of a Session that is no linger valid | Session ID [%s]."), __func__, *SessionID.ToString());
		}

		return;
	}

	UE_LOG_REF(LogCategory, Verbose, TEXT("[%hs] Session ID [%s] not found."), __func__, *SessionID.ToString());
}

void FRecordingControls::HandleRemoteSessionsUpdated()
{
	using namespace UE::TraceBasedDebuggers;
	const TSharedPtr<FSessionInfo> CurrentSessionPtr = GetCurrentSessionInfo();
	if (!CurrentSessionPtr)
	{
		bool bValidSessionSelected = false;

		// if the session that is currently selected is not valid, try to find a valid session and select that as default target session
		if (const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = RemoteSessionManagerWeakPtr.Pin())
		{
			RemoteSessionManager->EnumerateActiveSessions([this, &bValidSessionSelected](const TSharedRef<FSessionInfo>& InSessionInfoRef)
				{
					if (EnumHasAnyFlags(InSessionInfoRef->GetSessionTypeAttributes(), ERemoteSessionAttributes::IsMultiSessionWrapper))
					{
						return true;
					}

					// Allow derived classes to ignore some sessions
					if (ShouldFilterOutSessionInternal(InSessionInfoRef))
					{
						return true;
					}

					UE_LOG_REF(LogCategory, Verbose, TEXT("[%hs] Selecting first valid session [%s: %s]."), __func__, *InSessionInfoRef->SessionName, LexToString(InSessionInfoRef->BuildTargetType));
					SelectTargetSession(InSessionInfoRef->InstanceId);
					bValidSessionSelected = true;

					// Stop the iteration
					return false;
				});
		}

		if (!bValidSessionSelected)
		{
			SelectTargetSession(FRemoteSessionsManager::InvalidSessionGUID);
		}
	}
}

FRecordingControls::FAsyncConnectionAttemptTask::~FAsyncConnectionAttemptTask()
{
	if (RetryDelegateHandle.IsValid())
	{
		FTSTicker::RemoveTicker(RetryDelegateHandle);
		RetryDelegateHandle = FTSTicker::FDelegateHandle();
	}
}

void FRecordingControls::FAsyncConnectionAttemptTask::Cancel()
{
	State = EState::Canceled;

	if (RecordingFailedCallback)
	{
		RecordingFailedCallback();
	}

	if (const TSharedPtr<FRecordingControls> OwnerPtr = Owner.Pin())
	{
		OwnerPtr->HandleConnectionAttemptResult(SessionID, ELiveConnectionAttemptResult::Canceled, ProgressNotification);
	}
}

void FRecordingControls::FAsyncConnectionAttemptTask::ScheduleRetry()
{
	if (!CanExecute())
	{
		UE_LOG_REF(LogCategory, Error, TEXT("[%hs] This connection attempt is in an invalid state | Current State Value [%u]"), __func__, EnumToUnderlyingType(State));
		return;
	}

	if (RetryDelegateHandle.IsValid())
	{
		UE_LOG_REF(LogCategory, Warning, TEXT("[%hs] Attempted to schedule a retry when there is one already scheduled"), __func__);
		return;
	}

	if (RemainingRetries <= 0)
	{
		UE_LOG_REF(LogCategory, Warning, TEXT("[%hs] Failed to connect to live session | attempts exhausted..."), __func__);
		HandleFailure();
		return;
	}

	RemainingRetries--;

	RetryDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSPLambda(AsShared(), [this](float DeltaTime)
		{
			Execute();
			return false;
		}), IntervalBetweenAttemptsSeconds);
}

void FRecordingControls::FAsyncConnectionAttemptTask::Start(const int32 InMaxRetriesAttempts, const float InIntervalBetweenAttemptsSeconds)
{
	if (State != EState::NotStarted)
	{
		UE_LOG_REF(LogCategory, Error, TEXT("[%hs] This connection attempt is already started or in an invalid state | Current State Value [%u]"), __func__, EnumToUnderlyingType(State));
		return;
	}

	const TSharedPtr<FRecordingControls> OwnerPtr = Owner.Pin();
	if (!OwnerPtr)
	{
		UE_LOG_REF(LogCategory, Error, TEXT("[%hs] This connection attempt is does not have a valid owner | Cannot start attempt... "), __func__);
		return;
	}

	if (const TSharedPtr<FAsyncConnectionAttemptTask>* ExistingRequestInFlight = OwnerPtr->InFlightAsyncConnectionRequests.Find(SessionID))
	{
		UE_LOG_REF(LogCategory, Warning, TEXT("[%hs] There is a connection attempt in fight for this session. attempting to cancel it | Session ID [%s]"), __func__, *SessionID.ToString());
		(*ExistingRequestInFlight)->Cancel();
	}

	OwnerPtr->InFlightAsyncConnectionRequests.Add(SessionID, AsShared());

	State = EState::InProgress;

	RemainingRetries = InMaxRetriesAttempts;
	IntervalBetweenAttemptsSeconds = InIntervalBetweenAttemptsSeconds;

	if (!ProgressNotification)
	{
		FNotificationInfo Info(LOCTEXT("ConnectingToSessionMessage", "Connecting Session ..."));
		Info.bFireAndForget = false;
		Info.FadeOutDuration = 3.0f;
		Info.ExpireDuration = 0.0f;
		Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("CancelConnectionAttemptButton", "Cancel"), LOCTEXT("CancelConnectionAttemptButtonTip", "Cancels the active connection attempt."),
			FSimpleDelegate::CreateSP(AsShared(), &FAsyncConnectionAttemptTask::Cancel)));

		ProgressNotification = OwnerPtr->PushConnectionAttemptNotification(Info);
	}

	// We need to wait at least one tick before attempting to connect to give it time to the trace to be initialized, write to disk, and for the
	// session manager to hear back from a remote instance
	ScheduleRetry();
}

void FRecordingControls::FAsyncConnectionAttemptTask::Execute()
{
	if (RetryDelegateHandle.IsValid())
	{
		FTSTicker::RemoveTicker(RetryDelegateHandle);
		RetryDelegateHandle.Reset();
	}

	if (!CanExecute())
	{
		UE_LOG_REF(LogCategory, Error, TEXT("[%hs] This connection attempt is in an invalid state | Current State Value [%u]"), __func__, EnumToUnderlyingType(State));
		return;
	}

	if (!RecordingStartAttemptCallback)
	{
		UE_LOG_REF(LogCategory, Error, TEXT("[%hs] Failed to connect to session | Invalid callback provided..."), __func__);
		HandleFailure();
		return;
	}

	if (const TSharedPtr<FRecordingControls> OwnerPtr = Owner.Pin())
	{
		OwnerPtr->UpdateConnectionAttemptNotification(ProgressNotification, RemainingRetries);
	}

	// Debugger needs the trace session name to be able to load a live session. Although the session exist, the session name might not be written right away
	// Trace files don't really have metadata, it is all part of the same stream, so we need to wait until it is written which might take a few ticks.
	// Therefore, if it is not ready, try again a few times.
	if (!RecordingStartAttemptCallback())
	{
		UE_LOG_REF(LogCategory, Verbose, TEXT("[%hs] Failed to connect to live session | Attempting again in [%f]..."), __func__, IntervalBetweenAttemptsSeconds);
		ScheduleRetry();
	}
	else
	{
		HandleSuccess();
	}
}

void FRecordingControls::ToggleSingleSessionRecordingState(const TSharedRef<FSessionInfo>& InSessionInfo)
{
	SetSessionRecordingState(!InSessionInfo->IsRecording(DebuggerGuid), InSessionInfo);
}

void FRecordingControls::SetSessionRecordingState(const bool bIsRecording, const TSharedRef<FSessionInfo>& SessionInfoRef)
{
#if WITH_TRACE_BASED_DEBUGGERS
	using namespace UE::TraceBasedDebuggers;
	const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = RemoteSessionManagerWeakPtr.Pin();
	if (!RemoteSessionManager)
	{
		UE_LOG_REF(LogCategory, Error, TEXT("[%hs] Session Manager is not available"), __func__);
		return;
	}

	if (bIsRecording)
	{
		const int32 RemainingRetries = GetMaxConnectionRetriesInternal();
		const ETraceTransportMode DataTransportOverrideMode = GetTransportMode(SessionInfoRef->BuildTargetType);

		// Set the session in a busy state so the auto-expire policy for it is relaxed. Starting a recording might cause the remote instance to stall for a few seconds
		// if the map loaded is too complex
		SessionInfoRef->ReadyState = ERemoteSessionReadyState::Busy;

		const TSharedPtr<FRemoteSessionsManager> RemoteSessionManagerPtr = RemoteSessionManagerWeakPtr.Pin();
		const TSharedPtr<FSessionInfo> SessionInfoPtr = RemoteSessionManagerPtr ? RemoteSessionManagerPtr->GetSessionInfo(SessionInfoRef->InstanceId).Pin() : nullptr;

		auto RecordingAttemptFailedCallback =
			[SessionGUID = SessionInfoRef->InstanceId, SessionInfoPtr, WeakThis = AsWeak()]()
			{
				if (!SessionInfoPtr)
				{
					return;
				}

				SessionInfoPtr->ReadyState = ERemoteSessionReadyState::Ready;

				if (const TSharedPtr<FRecordingControls> Controls = WeakThis.Pin())
				{
					Controls->CloseSessionByRemoteSessionIDInternal(SessionGUID);
				}
			};

		TNotNull<TUniquePtr<FStartRecordingCommandMessage>> RecordingParams = BuildStartRecordingParamsInternal();
		RecordingParams->InstanceId = FApp::GetInstanceId();
		RecordingParams->RecordingMode = ERecordingMode::Live;
		RecordingParams->TransportMode = DataTransportOverrideMode != ETraceTransportMode::Invalid ? DataTransportOverrideMode : ETraceTransportMode::Direct;

		check(GLog);
		bool bOutCanBindAll = false;
		//TODO: Add a way to specify a local address in case we have multiple adapters?
		const TSharedRef<FInternetAddr> LocalIP = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, bOutCanBindAll);

		constexpr bool bAppendPort = false;
		RecordingParams->Target = LocalIP->ToString(bAppendPort);

		if (RecordingParams->TransportMode == ETraceTransportMode::Direct)
		{
			// Direct Tracing mode requires opening a trace session in the editor side first before starting the recording
			// so targets have something to connect to (Usually targets connect to a running trace server)
			if (!ConnectToLiveSession_DirectInternal(SessionInfoRef->InstanceId))
			{
				UE_LOG_REF(LogCategory, Error, TEXT("[%hs] Failed to connect to live session using direct trace mode"), __func__);
				RecordingAttemptFailedCallback();
				return;
			}

			// If we get here it means we have at least one valid session
			const FTraceSessionDescriptor& SessionDesc = GetCurrentSessionDescriptorInternal();
			RecordingParams->Target.Appendf(TEXT(":%u"), SessionDesc.SessionPort);

			SendStartRecordingCommandInternal(SessionInfoRef->Address, MoveTemp(RecordingParams));

			ExecuteAsyncConnectionAttemptTaskWithRetry(SessionInfoRef->InstanceId, RemainingRetries, [SessionInfoPtr]()
				{
					if (!SessionInfoPtr)
					{
						return false;
					}

					// In Direct trace mode we if we get connection details from the remote instance, we can assume the connection succeeded.
					return SessionInfoPtr->GetTraceConnectionDetails().IsValid();

				}, RecordingAttemptFailedCallback);
		}
		else if (RecordingParams->TransportMode == ETraceTransportMode::Relay)
		{
			// Relay Tracing mode requires opening a trace session in the editor side first before starting the recording
			// so targets have something to connect to (Usually targets connect to a running trace server)
			if (!ConnectToLiveSession_RelayInternal(SessionInfoRef->InstanceId))
			{
				UE_LOG_REF(LogCategory, Error, TEXT("[%hs] Failed to connect to live session using relay trace mode"), __func__);
				RecordingAttemptFailedCallback();
				return;
			}

			SendStartRecordingCommandInternal(SessionInfoRef->Address, MoveTemp(RecordingParams));

			// Once the start recording command is issued, we can try to connect to the created session (if everything went well).
			// If it didn't go well, this will take care of update the UI to notify the user
			ExecuteAsyncConnectionAttemptTaskWithRetry(SessionInfoRef->InstanceId, RemainingRetries, 
			[SessionInfoPtr, SessionInstanceId = SessionInfoRef->InstanceId, RelayTraceDataTransportInstance = TraceRelayTransportInstanceWeakPtr.Pin()]
				{
					if (!SessionInfoPtr)
					{
						return false;
					}

					const FTraceConnectionDetails& RecordingSessionDetails = SessionInfoPtr->GetTraceConnectionDetails();

					// We didn't receive the connection details yet. Trying again later...
					if (!RecordingSessionDetails.IsValid())
					{
						return false;
					}

					// Check if there is a connection attempt in progress and its results
					const EConnectionAttemptResult ConnectionAttemptStatus = RelayTraceDataTransportInstance.IsValid()
						? RelayTraceDataTransportInstance->GetConnectionAttemptResult(SessionInstanceId)
						: EConnectionAttemptResult::Failed;

					switch (ConnectionAttemptStatus)
					{
					case EConnectionAttemptResult::InProgress:
					{
						// A connection attempt started, but we don't have the results yet. Trying again later...
						return false;
					}
					case EConnectionAttemptResult::NotStarted:
					{
						FRelayConnectionInfo ConnectionInfo;
						ConnectionInfo.Port = RecordingSessionDetails.Port;
						ConnectionInfo.Address = RecordingSessionDetails.TraceTarget;
						ConnectionInfo.CertificateAuthority = RecordingSessionDetails.CertAuth;
						EConnectionAttemptResult Result = RelayTraceDataTransportInstance.IsValid()
							? RelayTraceDataTransportInstance->ConnectToRelay(SessionInstanceId, ConnectionInfo)
							: EConnectionAttemptResult::Failed;

						return Result == EConnectionAttemptResult::Success;
					}
					case EConnectionAttemptResult::Success:
						// The current connection attempt succeeded, we can stop the async retry loop
						return true;
					case EConnectionAttemptResult::Failed:
					default:
						// There was a connection attempt, but it failed. Abandoning the retry attempt as they will not succeed
						return false;
					}
				}, RecordingAttemptFailedCallback);
		}
	}
	else
	{
		CancelInFlightConnectionAttempt(SessionInfoRef.ToSharedPtr());
		SendStopRecordingCommandInternal(SessionInfoRef->Address);
	}
#endif // WITH_TRACE_BASED_DEBUGGERS
}

void FRecordingControls::ToggleRecordingState(const TSharedPtr<FSessionInfo>& InSessionInfo)
{
	using namespace UE::TraceBasedDebuggers;
	if (!InSessionInfo)
	{
		return;
	}

	OnToggleRecordingStateInternal(InSessionInfo.ToSharedRef());
}

bool FRecordingControls::IsRecordingPossibleForSession(const TSharedPtr<FSessionInfo>& InSessionInfo) const
{
	using namespace UE::TraceBasedDebuggers;
	if (CurrentSelectedSessionId == FRemoteSessionsManager::InvalidSessionGUID)
	{
		return false;
	}

	if (!InSessionInfo)
	{
		return false;
	}

	const FRecordingStatusMessage& LastState = InSessionInfo->LastKnownRecordingState;
	const bool bWasRecording = LastState.DebuggerId.IsValid() && LastState.RequesterId.IsValid();

	// Do not allow start/stop recording if a trace-based debuggers is already recording in the current session
	// AND it is not associated to the current debugger type (i.e., DebuggerId)
	// OR it was not requested by the current application instance (e.g., multiple editors trying to debug the same remote process)
	if (bWasRecording && (LastState.DebuggerId != DebuggerGuid || LastState.RequesterId != FApp::GetInstanceId()))
	{
		return false;
	}

	// Allow derived classes to deny
	if (!IsRecordingAvailableForSessionInternal(InSessionInfo.ToSharedRef()))
	{
		return false;
	}

	return true;
}

EVisibility FRecordingControls::IsRecordingToggleButtonVisible()
{
	return EVisibility::Visible;
}

void FRecordingControls::AddToMenu(const FName ExistingMenu)
{
	UToolMenu* ToolBar = UToolMenus::Get()->ExtendMenu(ExistingMenu);
	if (!ensure(ToolBar))
	{
		return;
	}

	// Register the shared parent options menu that plugins can extend to hook into all debuggers
	if (!UToolMenus::Get()->IsMenuRegistered(BaseRecordingOptionsMenuName))
	{
		UToolMenus::Get()->RegisterMenu(BaseRecordingOptionsMenuName);
	}

	const FRecordingControlsConfig Config = GetControlsConfig();
	const FText RecordTooltipFormat = LOCTEXT("RecordButtonDesc", "Starts a recording and analysis for the current session, saving it directly to '{0}'");
	FToolMenuSection& Section = ToolBar->FindOrAddSection("RecordingControls", FText::GetEmpty(), Config.SectionInsertPosition);

	if (!Config.OptionsMenuName.IsNone())
	{
		Section.AddDynamicEntry("RecordButton", FNewToolMenuSectionDelegate::CreateSPLambda(this, [RecordingControls = this, RecordTooltipFormat](FToolMenuSection& InSection)
			{
				const FText StartTooltip = FText::Format(RecordTooltipFormat, FText::FromString(RecordingControls->GetSaveDirPath()));

				FToolMenuEntry RecordEntry = FToolMenuEntry::InitToolBarButton(
					"StartRecording",
					FUIAction(
						FExecuteAction::CreateLambda([RecordingControls]()
						{
							RecordingControls->ToggleRecordingState(RecordingControls->GetCurrentSessionInfo());
						}),
						FCanExecuteAction::CreateLambda([RecordingControls]()
						{
							return RecordingControls->IsRecordingToggleButtonEnabled();
						})
					),
					FText::GetEmpty(),
					TAttribute<FText>::CreateLambda([RecordingControls, StartTooltip]()
					{
						return RecordingControls->IsRecording()
							? LOCTEXT("StopRecordButtonDesc", "Stop the current recording")
							: StartTooltip;
					})
				);

				// Custom ButtonContent so we still get the "stop" icon on hover.
				RecordEntry.MakeCustomWidget = FNewToolMenuCustomWidget::CreateLambda([RecordingControls](const FToolMenuContext& MenuContext, const FToolMenuCustomWidgetContext& WidgetContext) -> TSharedRef<SWidget>
					{
						TSharedRef<SImage> Animation = SNew(SImage);
						Animation->SetVisibility(EVisibility::Visible);

						// Switch to stop-brush on hover while recording
						Animation->SetImage(TAttribute<const FSlateBrush*>::CreateLambda([RecordingControls, Animation]() -> const FSlateBrush*
							{
								if (Animation->IsHovered() && RecordingControls->IsRecording())
								{
									return RecordingControls->StopRecordingBrush;
								}
								
								return RecordingControls->StartRecordingBrush;
							})
						);

						Animation->SetColorAndOpacity(MakeAttributeSPLambda(Animation, [RecordingControls, Animation]
							{
								// Pulse record button (and adjust style when showing stop icon on hover)
								if (RecordingControls->IsRecording())
								{
									if (!RecordingControls->RecordingAnimation.IsPlaying())
									{
										RecordingControls->RecordingAnimation.Play(Animation, true);
									}
							
									const bool bHovered = Animation->IsHovered();
									const FLinearColor Color = bHovered ? FLinearColor::Red : FLinearColor::White;
									
									return FSlateColor(bHovered ? Color : Color.CopyWithNewOpacity(0.2f + 0.8f * RecordingControls->RecordingAnimation.GetLerp()));
								}
								
								// Stop playing pulsing when not recording.
								RecordingControls->RecordingAnimation.Pause();
								
								return FSlateColor::UseSubduedForeground();
							})
						);

						return Animation;
					});

				InSection.AddEntry(RecordEntry);
			})
		);
		
		Section.AddDynamicEntry("RecordingOptionsEntry", FNewToolMenuSectionDelegate::CreateSPLambda(this, [RecordingControls = this, Config](FToolMenuSection& InSection)
			{
				FToolMenuEntry OptionsEntry = FToolMenuEntry::InitComboButton(
					"RecordingOptions",
					FUIAction(),
					FOnGetContent::CreateLambda([Config]() -> TSharedRef<SWidget>
					{
						FToolMenuContext Context;
						return UToolMenus::Get()->GenerateWidget(Config.OptionsMenuName, Context);
					}),
					FText::GetEmpty(),
					LOCTEXT("RecordingOptionsTooltip", "Recording options"),
					FSlateIcon(),
					true
				);
				
				InSection.AddEntry(OptionsEntry);
			})
		);
	}
	else
	{
		Section.AddDynamicEntry("RecordButton", FNewToolMenuSectionDelegate::CreateSPLambda(this, [RecordingControls = this, RecordTooltipFormat](FToolMenuSection& InSection)
			{
				const TSharedRef<SWidget> StartRecordingButton = SNew(SBox).Padding(4.0f, 0.0f)
					[
						RecordingControls->GenerateToggleRecordingStateButton(
							FText::Format(RecordTooltipFormat, FText::FromString(RecordingControls->GetSaveDirPath()))
						)
					];

				InSection.AddEntry(FToolMenuEntry::InitWidget("StartRecording", StartRecordingButton, FText::GetEmpty(), true, false));
			})
		);
	}

	// Session selector + remaining controls (always the same regardless of path)
	Section.AddDynamicEntry("Controls", FNewToolMenuSectionDelegate::CreateSPLambda(this, [RecordingControls = this](FToolMenuSection& InSection)
		{
			const TSharedRef<SWidget> SessionsDropdown = SNew(SBox).Padding(4.0f, 0.0f)
				[
					RecordingControls->GenerateTargetSessionSelector()
				];

			InSection.AddEntry(FToolMenuEntry::InitWidget(
				"AvailableSessions",
				SessionsDropdown,
				FText::GetEmpty(),
				false,
				false
			));

			RecordingControls->AddToMenuInternal(InSection);
		}));
}

int32 FRecordingControls::GetMaxConnectionRetriesInternal() const
{
	return DefaultNumberOfConnectionRetries;
}

ETraceTransportMode FRecordingControls::GetTransportMode(const EBuildTargetType InBuildTarget) const
{
	const ETraceTransportMode Override = GetTransportModeOverrideForTargetTypeInternal(InBuildTarget);
	return Override != ETraceTransportMode::Invalid ? Override : ETraceTransportMode::Direct;
}

ETraceTransportMode FRecordingControls::GetTransportModeOverrideForTargetTypeInternal(EBuildTargetType InBuildTarget) const
{
	return ETraceTransportMode::Invalid;
}

bool FRecordingControls::IsRecordingAvailableForSessionInternal(const TSharedRef<FSessionInfo>& InSessionInfo) const
{
	return true;
}

bool FRecordingControls::ShouldFilterOutSessionInternal(const TSharedRef<FSessionInfo>& InSessionInfo) const
{
	return false;
}

bool FRecordingControls::IsRecording() const
{
	using namespace UE::TraceBasedDebuggers;
	const TSharedPtr<FSessionInfo> SessionInfo = GetCurrentSessionInfo();
	return SessionInfo ? SessionInfo->IsRecording(DebuggerGuid) : false;
}

FText FRecordingControls::GetRecordingTimeText() const
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FSessionInfo> SessionInfo = GetCurrentSessionInfo())
	{
		FNumberFormattingOptions FormatOptions;
		FormatOptions.MinimumFractionalDigits = 2;
		FormatOptions.MaximumFractionalDigits = 2;

		const FText SecondsText = FText::AsNumber(SessionInfo->LastKnownRecordingState.ElapsedTime, &FormatOptions);
		return FText::Format(LOCTEXT("RecordingTimer", "{0} s"), SecondsText);
	}

	return LOCTEXT("RecordingTimerError", "Failed to get time information");
}

TSharedPtr<SNotificationItem> FRecordingControls::PushConnectionAttemptNotification(const FNotificationInfo& InNotificationInfo)
{
	TSharedPtr<SNotificationItem> ConnectionAttemptNotification = FSlateNotificationManager::Get().AddNotification(InNotificationInfo);

	if (ConnectionAttemptNotification.IsValid())
	{
		ConnectionAttemptNotification->SetCompletionState(SNotificationItem::CS_Pending);
		return ConnectionAttemptNotification;
	}

	return nullptr;
}

void FRecordingControls::UpdateConnectionAttemptNotification(const TSharedPtr<SNotificationItem>& InNotification, int32 AttemptsRemaining)
{
	if (InNotification)
	{
		InNotification->SetSubText(FText::FormatOrdered(LOCTEXT("SessionConnectionAttemptSubText", "Attempts Remaining {0}"), AttemptsRemaining));
	}
}

void FRecordingControls::HandleConnectionAttemptResult(const FGuid SessionGUID, const ELiveConnectionAttemptResult Result, const TSharedPtr<SNotificationItem>& InNotification)
{
	InFlightAsyncConnectionRequests.Remove(SessionGUID);

	if (InNotification)
	{
		switch (Result)
		{
		case ELiveConnectionAttemptResult::Success:
		{
			InNotification->SetText(LOCTEXT("SessionConnectionSuccess", "Connected!"));
			InNotification->SetSubText(FText::GetEmpty());
			InNotification->SetCompletionState(SNotificationItem::ECompletionState::CS_Success);
			break;
		}
		case ELiveConnectionAttemptResult::Canceled:
		{
			InNotification->SetText(LOCTEXT("SessionConnectionCanceledText", "Connection Attempt Canceled"));
			InNotification->SetCompletionState(SNotificationItem::ECompletionState::CS_None);
			break;
		}
		case ELiveConnectionAttemptResult::Failed:
		{
			InNotification->SetText(LOCTEXT("SessionConnectionFailedText", "Failed to connect"));
			InNotification->SetSubText(LOCTEXT("SessionConnectionFailedSubText", "See the logs for more details..."));
			InNotification->SetCompletionState(SNotificationItem::ECompletionState::CS_Fail);
			break;
		}
		default:
		{
			InNotification->SetText(LOCTEXT("SessionConnectionUnknownText", "Something went wrong..."));
			InNotification->SetSubText(LOCTEXT("SessionConnectionUnknownTextSubText", "See the logs for more details..."));
			InNotification->SetCompletionState(SNotificationItem::ECompletionState::CS_Fail);
			break;
		}
		}

		InNotification->ExpireAndFadeout();
	}
}

} // UE::TraceBasedDebuggers
#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR