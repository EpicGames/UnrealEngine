// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SConnectionStatusBar.h"

#if WITH_EDITOR

#include "Components/HorizontalBox.h"
#include "RemoteSessionsManager.h"
#include "SessionInfo.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ConnectionStatusBar"

namespace UE::TraceBasedDebuggers
{

SLATE_IMPLEMENT_WIDGET(SConnectionStatusBar)
void SConnectionStatusBar::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

void SConnectionStatusBar::Construct(const FArguments& InArgs, const FGuid& InDebuggerGuid, const TSharedRef<FRemoteSessionsManager>& InRemoteSessionManager)
{
	DebuggerGuid = InDebuggerGuid;
	RemoteSessionManagerWeakPtr = InRemoteSessionManager;

	ChildSlot
	[
		SNew(SHorizontalBox)
	
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 3, 0)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image_Raw(this, &SConnectionStatusBar::GetStateBackgroundIcon)
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("DerivedData.RemoteCache.Busy"))
					.ColorAndOpacity_Raw(this, &SConnectionStatusBar::GetWaitingIconColor)
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("DerivedData.RemoteCache.Uploading"))
					.ColorAndOpacity_Raw(this, &SConnectionStatusBar::GetUploadingIconColor)
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("DerivedData.RemoteCache.Downloading"))
					.ColorAndOpacity_Raw(this, &SConnectionStatusBar::GetDownloadIconColor)
				]
				+ SOverlay::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Top)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Zen.Status.Idle"))
					.ColorAndOpacity_Raw(this, &SConnectionStatusBar::GetReadyIconColor)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 10, 0)
			[
				SNew(STextBlock)
				.Text_Raw(this, &SConnectionStatusBar::GetTitleText)
				.ToolTipText_Raw(this, &SConnectionStatusBar::GetTitleToolTipText)
			]
	];
}

const FSlateBrush* SConnectionStatusBar::GetStateBackgroundIcon() const
{
	return State == EState::WaitingForData ? FAppStyle::Get().GetBrush("DerivedData.RemoteCache.BusyBG") : FAppStyle::Get().GetBrush("DerivedData.RemoteCache.IdleBG");
}

FText SConnectionStatusBar::GetTitleText() const
{
	static const FNumberFormattingOptions SizeFormattingOptions = FNumberFormattingOptions().SetMinimumFractionalDigits(2).SetMaximumFractionalDigits(2);

	switch (State)
	{
	case EState::Ready:
		return LOCTEXT("ConnectionReadyStatusText", "Ready");
	case EState::ReceivingData:
		{
			FText BytesPerSecondText = FText::AsMemory(CurrentReceivedDataBytesPerSecond, &SizeFormattingOptions, nullptr, IEC);
			FText PendingBytesText = FText::AsMemory(CurrentPendingBytes, &SizeFormattingOptions, nullptr, IEC);
			FText TotalBytesText = FText::AsMemory(CurrentTotalReceivedData, &SizeFormattingOptions, nullptr, IEC);
			return FText::FormatOrdered(LOCTEXT("ConnectionDownloadedStatusText", "Receiving {0}/s | {1} Pending | Total {2}"), BytesPerSecondText, PendingBytesText, TotalBytesText);
		}
	case EState::WaitingForData:
	default:
		{
			FText PendingBytesText = FText::AsMemory(CurrentPendingBytes, &SizeFormattingOptions, nullptr, IEC);
			FText TotalBytesText = FText::AsMemory(CurrentTotalReceivedData, &SizeFormattingOptions, nullptr, IEC);
			return FText::FormatOrdered(LOCTEXT("ConnectionWaitingStatusText", "Waiting for Data | {0} Pending | Total {1}"), PendingBytesText, TotalBytesText);
		}
	}
}

FText SConnectionStatusBar::GetTitleToolTipText() const
{
	return LOCTEXT("DownloadingStatusToolTipText", "Current state of the connection to the recording target.");
}

FSlateColor SConnectionStatusBar::GetDownloadIconColor() const
{
	return State == EState::ReceivingData ? FLinearColor::White.CopyWithNewOpacity(FMath::MakePulsatingValue(ElapsedTime, 2)) : FLinearColor(0.0,0.0,0.0,0.0);
}

FSlateColor SConnectionStatusBar::GetUploadingIconColor() const
{
	// The debugger does not upload data, but if we are downloading data we want to slightly see the upload arrow
	return State == EState::ReceivingData ? FLinearColor::White.CopyWithNewOpacity(0.2f) : FLinearColor(0.0,0.0,0.0,0.0);
}

FSlateColor SConnectionStatusBar::GetWaitingIconColor() const
{
	return State == EState::WaitingForData ? FSlateColor::UseForeground() : FLinearColor(0.0,0.0,0.0,0.0);
}

FSlateColor SConnectionStatusBar::GetReadyIconColor() const
{
	return State == EState::Ready ? FSlateColor::UseForeground() : FLinearColor(0.0,0.0,0.0,0.0);
}

void SConnectionStatusBar::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	ElapsedTime += InDeltaTime;

	UpdateConnectionStatus(InDeltaTime);
}

void SConnectionStatusBar::UpdateConnectionStatus(double DeltaTime)
{
	using namespace UE::TraceBasedDebuggers;

	ElapsedTimeSinceLastUpdate += DeltaTime;

	constexpr double UpdateIntervalSeconds = 1.0;
	if (ElapsedTimeSinceLastUpdate < UpdateIntervalSeconds)
	{
		return;
	}

	ElapsedTimeSinceLastUpdate = 0;

	const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = RemoteSessionManagerWeakPtr.Pin();
	if (!RemoteSessionManager)
	{
		State = EState::Ready;
		return;
	}

	bool bIsAnyRecording = false;
	int64 PendingBytesNum = 0;
	uint64 BytesReadSinceLastMeasurement = 0;
	RemoteSessionManager->EnumerateActiveSessions([this, &BytesReadSinceLastMeasurement, &bIsAnyRecording, &PendingBytesNum](const TSharedRef<FSessionInfo>& InSessionInfoRef)
	{
		if (!EnumHasAnyFlags(InSessionInfoRef->GetSessionTypeAttributes(), ERemoteSessionAttributes::IsMultiSessionWrapper))
		{
			if (InSessionInfoRef->IsRecording(DebuggerGuid))
			{
				bIsAnyRecording = true;
				PendingBytesNum += InSessionInfoRef->LastKnownRecordingState.BufferedDataBytesSize;
				BytesReadSinceLastMeasurement += InSessionInfoRef->GetReceivedBytesPerSecond();
			}
		}

		return true;
	});

	CurrentPendingBytes = PendingBytesNum;

	if (bIsAnyRecording)
	{
		if (BytesReadSinceLastMeasurement > 0)
		{
			CurrentReceivedDataBytesPerSecond = BytesReadSinceLastMeasurement;
			CurrentTotalReceivedData += BytesReadSinceLastMeasurement;
			State = EState::ReceivingData;
		}
		else
		{
			State = EState::WaitingForData;
		}
	}
	else
	{
		State = EState::Ready;
		CurrentPendingBytes = 0;
		CurrentReceivedDataBytesPerSecond = 0;
		CurrentTotalReceivedData = 0;
	}
}

}

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR