// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS

#include "Containers/SpscQueue.h"
#include "Containers/Ticker.h"
#include "Misc/Guid.h"
#include "Trace/DataStream.h"

#define UE_API TRACEBASEDDEBUGGERSANALYSIS_API

namespace TraceServices
{
class IAnalysisSession;
}

namespace UE::TraceBasedDebuggers
{
struct FRemoteSessionsManager;
class IDataRelayTransport;

/** Wrapper around a data buffer used for serialization of trace-based debuggers data */
struct FSerializedDataBuffer
{
	FSerializedDataBuffer(const TConstArrayView<uint8> ConstView)
		: DataBuffer(ConstView.GetData(), ConstView.Num())
	{
	}

	FSerializedDataBuffer(TArray<uint8>&& InByteArray)
		: DataBuffer(MoveTemp(InByteArray))
	{
	}

	FSerializedDataBuffer(const TArray<uint8>& InByteArray)
		: DataBuffer(InByteArray)
	{
	}

	FSerializedDataBuffer()
	{
	}

	TArray<uint8>& GetDataAsByteArrayRef()
	{
		return DataBuffer;
	}

	int32 GetSize() const
	{
		return DataBuffer.Num();
	}

	void Serialize(FArchive& Ar)
	{
		Ar << DataBuffer;
	}

private:
	TArray<uint8> DataBuffer;
};

inline FArchive& operator<<(FArchive& Ar, FSerializedDataBuffer& Data)
{
	Data.Serialize(Ar);
	return Ar;
}

class IRemoteSessionDataStream
{
public:
	virtual ~IRemoteSessionDataStream() = default;
	virtual FGuid GetOwningRemoteSessionID() const = 0;
	virtual uint64 UpdateAndGetBytesReadSinceLastMeasurement() = 0;
	virtual TSharedPtr<FRemoteSessionsManager> GetSessionsManager() const = 0;
};

class FSessionStreamStatsUpdater : FTSTickerObjectBase
{
public:
	virtual ~FSessionStreamStatsUpdater() override;

	explicit FSessionStreamStatsUpdater(IRemoteSessionDataStream* const DataStreamInstance)
		: DataStreamInstance(DataStreamInstance)
	{
	}

private:
	virtual bool Tick(float DeltaTime) override;

	IRemoteSessionDataStream* DataStreamInstance;

	double ElapsedTimeSinceLastUpdate = 0.0;
};

class FDirectSocketStream : public Trace::FDirectSocketStream, public IRemoteSessionDataStream
{
public:

	FDirectSocketStream() = delete;

	explicit FDirectSocketStream(const TSharedRef<FRemoteSessionsManager>& InSessionsManager
		, const FGuid& InRemoteSessionID
		, TUniquePtr<FArchiveFileWriterGeneric>&& InFileWriter
		)
		: Trace::FDirectSocketStream(MoveTemp(InFileWriter))
		, WeakSessionManager(InSessionsManager.ToWeakPtr())
		, BytesReadSinceLastMeasurement(0)
		, RemoteSessionID(InRemoteSessionID)
		, StatsUpdater(this)
	{
	}

	virtual int32 Read(void* Data, uint32 Size) override;

	virtual FGuid GetOwningRemoteSessionID() const override;
	virtual uint64 UpdateAndGetBytesReadSinceLastMeasurement() override;
	virtual TSharedPtr<FRemoteSessionsManager> GetSessionsManager() const override;

private:
	TWeakPtr<FRemoteSessionsManager> WeakSessionManager;
	std::atomic<uint64> BytesReadSinceLastMeasurement;
	FGuid RemoteSessionID;

	FSessionStreamStatsUpdater StatsUpdater;
};

/** A trace data stream object that listens for data coming from a relay stream via provided DataRelayTransport instance */
class FRelayDataStream : public Trace::IInDataStream, public IRemoteSessionDataStream
{
public:
	explicit FRelayDataStream(const TSharedRef<FRemoteSessionsManager>& InSessionsManager
		, const TSharedRef<IDataRelayTransport>& InDataRelay
		, const FGuid& InRemoteSessionID
		, TUniquePtr<FArchiveFileWriterGeneric>&& InFileWriter);

	virtual ~FRelayDataStream() override;

	void EnqueueRelayedData(const TConstArrayView<uint8> InTraceDataBuffer);
	void EnqueueRelayedData(TArray<uint8>&& InTraceDataBuffer);

	virtual FGuid GetOwningRemoteSessionID() const override;
	virtual uint64 UpdateAndGetBytesReadSinceLastMeasurement() override;
	virtual TSharedPtr<FRemoteSessionsManager> GetSessionsManager() const override;

private:
	virtual int32 Read(void* Dest, uint32 DestSize) override;
	virtual void Close() override;
	virtual bool WaitUntilReady() override;

	TWeakPtr<FRemoteSessionsManager> WeakSessionManager;
	TSpscQueue<FSerializedDataBuffer> DataQueue;
	FEvent* CancelEvent = nullptr;
	FEvent* WaitForDataEvent = nullptr;
	TOptional<FSerializedDataBuffer> CurrentPacket;

	FGuid RemoteSessionID = FGuid();

	TUniquePtr<FArchiveFileWriterGeneric> FileWriter;

#if WITH_TRACE_BASED_DEBUGGERS
	TSharedPtr<IDataRelayTransport> RelayTransportInstance;
#endif

	std::atomic<uint64> BytesReadSinceLastMeasurement;
	FSessionStreamStatsUpdater StatsUpdater;
	uint32 CurrentPacketOffset = 0;
	bool bClosed = false;
};

} // UE::TraceBasedDebuggers

#undef UE_API

#endif // UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS