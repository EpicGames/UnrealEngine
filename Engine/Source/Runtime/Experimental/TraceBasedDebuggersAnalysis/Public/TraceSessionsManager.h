// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS

#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Trace/StoreClient.h"
#include "UObject/SoftObjectPath.h"

namespace UE::Trace
{
class IInDataStream;
}

class FArchiveFileWriterGeneric;

namespace TraceServices
{
class IModule;
class IAnalysisSession;
}

namespace UE::TraceBasedDebuggers
{
class FEngineEditorBridge;
class FRuntimeModule;
class FRelayDataStream;

#define UE_API TRACEBASEDDEBUGGERSANALYSIS_API

/**
 * Structure containing info about a trace session used by trace-based debuggers
 */
struct FTraceSessionDescriptor
{
	/** The trace session name. Used to get the trace analysis instance for this session*/
	FString SessionName;

	/** Session ID of this session in the Remote Session management system*/
	FGuid RemoteSessionID;

	/** Port number used to establish a connection to this session. The value is 0 if no specific port was used*/
	uint16 SessionPort = 0;

	/** True if this is a session recorded live (not loaded from disk)*/
	bool bIsLiveSession = false;

	/** Returns true if valid. For a session to be valued needs at minimum have a Trace Session name */
	bool IsValid() const
	{
		return !SessionName.IsEmpty();
	}

	friend bool operator==(const FTraceSessionDescriptor& Lhs, const FTraceSessionDescriptor& RHS)
	{
		return Lhs.SessionName == RHS.SessionName
			&& Lhs.RemoteSessionID == RHS.RemoteSessionID;
	}

	friend bool operator!=(const FTraceSessionDescriptor& Lhs, const FTraceSessionDescriptor& RHS)
	{
		return !(Lhs == RHS);
	}
};

/** Manager class used by trace-based debuggers to interact/control UE Trace systems */
class FTraceSessionsManager
{
#if NO_LOGGING
	typedef FNoLoggingCategory FLogCategoryAlias;
#else
	typedef FLogCategoryBase FLogCategoryAlias;
#endif


public:
	/**
	 * @param RuntimeModule Debugger runtime module used to provide names for the trace files
	 * @param Bridge The engine-editor bridge instance used to provide access to the remote sessions manager
	 * @param TraceModule Optional trace module that will be registered/deregistered as a ModularFeature for the TraceServices
	 * @param SaveDirectorySubPathInUserDir Subdirectory path that will be used by GetDefaultSaveDirPath
	 */
	UE_API FTraceSessionsManager(FRuntimeModule& RuntimeModule
		, FEngineEditorBridge& Bridge
		, const TSharedPtr<TraceServices::IModule>& TraceModule
		, FStringView SaveDirectorySubPathInUserDir);
	UE_API ~FTraceSessionsManager();

	void SetSavePathOverride(const TAttribute<FFilePath>& InSavePathOverride)
	{
		SavePathOverride = InSavePathOverride;
	}

	void SetSaveRecordingsToDisk(const TAttribute<bool>& bInSaveRecordingsToDisk)
	{
		bSaveRecordingsToDisk = bInSaveRecordingsToDisk;
	}

	/** 
	 * Loads a trace file and starts analyzing it
	 * @param InTraceFilename File Name including (Path Included) of the Trace file to load
	 * @param PreStartAnalysisFunc Optional function called right before starting the trace analysis (i.e., TraceServices::IAnalysisService::StartAnalysis)
	 */
	UE_API FString LoadTraceFile(const FString& InTraceFilename, const TFunction<void()>& PreStartAnalysisFunc = nullptr);

	UE_API FString LoadTraceFile(TUniquePtr<IFileHandle>&& InFileHandle, const FString& InTraceSessionName, const TFunction<void()>& PreStartAnalysisFunc = nullptr);

	/**
	 * Connects to a live Trace Session and starts analyzing it.
	 * @param InSessionHost Trace Store Address for this session
	 * @param SessionID Trace ID in the Trace Store provided as host
	 * @param PreStartAnalysisFunc Optional function called right before starting the trace analysis (i.e., TraceServices::IAnalysisService::StartAnalysis)
	 * @return Session Name
	 */
	UE_API FString ConnectToLiveSession(FStringView InSessionHost, uint32 SessionID, TFunction<void()> PreStartAnalysisFunc = nullptr);

	/**
	 * Opens up a direct trace stream and waits for data
	 * @param RemoteSessionID Session ID to which we want to connect to
	 * @param OutSessionPort Number of the port we open for this session
	 * @param PreStartAnalysisFunc Optional function called right before starting the trace analysis (i.e., TraceServices::IAnalysisService::StartAnalysis)
	 * @return Session Name
	 */
	UE_API FString ConnectToLiveSession_Direct(FGuid RemoteSessionID, uint16& OutSessionPort, TFunction<void()> PreStartAnalysisFunc = nullptr);

	/**
	 * Opens up a relay trace stream and waits for data
	 * @param RemoteSessionID Session ID to which we want to connect to
	 * @param PreStartAnalysisFunc Optional function called right before starting the trace analysis (i.e., TraceServices::IAnalysisService::StartAnalysis)
	 * @return Session Name
	 */
	UE_API FString ConnectToLiveSession_Relay(FGuid RemoteSessionID, TFunction<void()> PreStartAnalysisFunc = nullptr);

	/** Returns the path to the local trace store */
	UE_API FString GetLocalTraceStoreDirPath();

	/** Returns a ptr to the session registered with the provided session name. Null if no session is found */
	UE_API TSharedPtr<const TraceServices::IAnalysisSession> GetSession(const FString& InSessionName);

	/** Stops and de-registers a trace session registered with the provided session name */
	UE_API void CloseSession(const FString& InSessionName);

	/** Stops a trace session registered with the provided session name */
	UE_API void StopSession(const FString& InSessionName);

	UE_API static const Trace::FStoreClient::FSessionInfo* GetTraceSessionInfo(FStringView InSessionHost, FGuid TraceGuid);

	template <typename TVisitor>
	void EnumerateActiveSessions(TVisitor Callback);

	/** Access the trace store at the provided host address, and returns the file name for the trace file
	 * @param InSessionHost Address of the Trace Store
	 * @param SessionID Session id of the trace data from which get the file name from
	 * @return Full Filename with path to the trace file if it exists. Returns an empty string if a trace file cannot be found
	 */
	UE_API FString GetTraceFileNameFromStoreForSession(FStringView InSessionHost, uint32 SessionID);

	/** Access the trace store at the provided host address, and returns a trace data stream ready to use
	 * @param InSessionHost Address of the Trace Store
	 * @param SessionID Session id of the trace data
	 */
	UE_API Trace::FStoreClient::FTraceData GetTraceDataStreamFromStore(FStringView InSessionHost, uint32 SessionID);

	/**
	 * Returns the actual path where recordings should be saved.
	 * That path is either the override provided by the path override or
	 * the default one.
	 * @see SetSavePathOverride
	 * @see GetDefaultSaveDirPath
	 * */
	UE_API FString GetSaveDirPath() const;

	/** @return The default path inside the user directory where recordings should be saved. */
	UE_API FString GetDefaultSaveDirPath() const;

private:
	TUniquePtr<FArchiveFileWriterGeneric> OpenTraceFileForWrite(const FString& InDirectoryToSavePath) const;

	bool ConnectToLiveSession_Internal(uint32 SessionID, const TFunction<void()>& PreStartAnalysisFunc, const FString& InRequestedSessionName, Trace::FStoreClient::FTraceData&& InTraceDataStream);

	static TUniquePtr<Trace::IInDataStream> CreateFileDataStream(TUniquePtr<IFileHandle>&& InFileHandle);
	TUniquePtr<FRelayDataStream> CreateRelayDataStream(FGuid RemoteSessionID) const;

	FRuntimeModule& RuntimeModule;
	FEngineEditorBridge& EngineEditorBridge;

	/** The trace analysis sessions. */
	TMap<FString, TSharedPtr<const TraceServices::IAnalysisSession>> AnalysisSessionByName;

	TSharedPtr<TraceServices::IModule> TraceModule;

	const FString SaveDirectorySubPathInUserDir;
	TAttribute<FFilePath> SavePathOverride;
	TAttribute<bool> bSaveRecordingsToDisk = true;

	static uint16 CurrentDirectModeStartingPort;
	static constexpr uint16 DefaultStartingPort = 1993;
};

template <typename TVisitor>
void FTraceSessionsManager::EnumerateActiveSessions(TVisitor Callback)
{
	for (const TPair<FString, TSharedPtr<const TraceServices::IAnalysisSession>>& SessionWithName : AnalysisSessionByName)
	{
		if (SessionWithName.Value)
		{
			if (!Callback(SessionWithName.Value.ToSharedRef()))
			{
				return;
			}
		}
	}
}

} // UE::TraceBasedDebuggers
#undef UE_API
#endif // UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS