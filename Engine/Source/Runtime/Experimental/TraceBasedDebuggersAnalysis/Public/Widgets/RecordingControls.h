// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Containers/Ticker.h"
#include "StatusBarSubsystem.h"
#include "Textures/SlateIcon.h"
#include "ToolMenuSection.h"
#include "TraceDataRelayTransport.h"
#include "TraceSessionsManager.h"
#include "RecordingControls.generated.h"

#define UE_API TRACEBASEDDEBUGGERSANALYSIS_API

struct FMessageAddress;

namespace UE::TraceBasedDebuggers
{
struct FStartRecordingCommandMessage;
struct FRemoteSessionsManager;
struct FSessionInfo;

DECLARE_DELEGATE_OneParam(FOnSessionEvent, const TSharedPtr<FSessionInfo>&)

UENUM()
enum class ELiveConnectionAttemptResult
{
	Success,
	Failed,
	Canceled
};

/** Toolbar integration options for FRecordingControls. */
struct FRecordingControlsConfig
{
	/** Options menu shown by the combo button. NAME_None hides the button. */
	FName OptionsMenuName = NAME_None;

	/** Show a "Start Recording" text label next to the record icon. */
	bool bHasRecordingLabel = true;

	/** Insert position of the "RecordingControls" section within the host toolbar. */
	FToolMenuInsert SectionInsertPosition = FToolMenuInsert();
};

struct FRecordingControls : TSharedFromThis<FRecordingControls>
{
	/** Parent menu plugins can extend to add entries to all debugger options menus. */
	UE_API static const FName BaseRecordingOptionsMenuName;

	UE_API FRecordingControls(const FName InStatusBarId
		, const TSharedRef<FRemoteSessionsManager>& InRemoteSessionManager
		, const FGuid& InDebuggerId
		, const FLogCategoryBase& InLogCategory);

	UE_API virtual ~FRecordingControls();

	UE_API void Initialize();
	UE_API void Deinitialize();
	UE_API void AddToMenu(FName ExistingMenu);

	/** Return the details about the currently selected session */
	UE_API TSharedPtr<FSessionInfo> GetCurrentSessionInfo() const;

	/** Delegate executed when recording got started in a given session */
	FOnSessionEvent OnRecordingStarted;

	/** Delegate executed when recording got stopped in a given session */
	FOnSessionEvent OnRecordingStopped;

	/** Delegate executed when a given session got selected */
	FOnSessionEvent OnSessionSelected;

protected:

	virtual void AddToMenuInternal(FToolMenuSection& InSection)
	{
	}

	virtual void OnGenerateTargetSessionSelectorInternal(FMenuBuilder& InMenuBuilder)
	{
	}

	/** Override to opt in to the options combo button and toolbar-style entries. Defaults keep the widget path. */
	virtual FRecordingControlsConfig GetControlsConfig() const
	{
		return {};
	}

	UE_API virtual TSharedRef<SWidget> GenerateTargetSessionSelector();
	UE_API TSharedRef<SWidget> GenerateTargetSessionDropdown();

	virtual FString GetSaveDirPath() const = 0;

	UE_API virtual int32 GetMaxConnectionRetriesInternal() const;
	UE_API ETraceTransportMode GetTransportMode(EBuildTargetType InBuildTarget) const;
	UE_API virtual ETraceTransportMode GetTransportModeOverrideForTargetTypeInternal(EBuildTargetType InBuildTarget) const;

	virtual bool ConnectToLiveSession_RelayInternal(const FGuid& InSessionId) = 0;
	virtual bool ConnectToLiveSession_DirectInternal(const FGuid& InSessionId) = 0;

	/**
	 * Derived classes must override this method to provide the specific message type required to start recording
	 * That type must inherit from FStartRecordingCommandMessage.
	 * Once message is ready to be sent the RecordingControls base class will call SendStartRecordingCommandInternal
	 * so the type can be de-referenced to call the templated function SendCommand of the sessions manager.
	 * e.g.,
	 *    TNotNull<TUniquePtr<FStartRecordingCommandMessage>> FMyRecordingControls::BuildStartRecordingParamsInternal()
	 *    {
	 *        return MakeUnique<FMyStartRecordingCommandMessage>();
	 *    }
	 *    
	 *    void FMyRecordingControls::SendStartRecordingCommandInternal(const FMessageAddress& InAddress, TUniquePtr<FStartRecordingCommandMessage> InStartRecordingCommand)
	 *    {
	 *        if (const TSharedPtr<FRemoteSessionsManager> SessionsManager = RemoteSessionManagerWeakPtr.Pin())
	 *        {
	 *            const FMyStartRecordingCommandMessage* StartRecordingCommand = static_cast<FMyStartRecordingCommandMessage*>(InStartRecordingCommand.Get());
	 *            SessionsManager->SendCommand(InAddress, *StartRecordingCommand);
	 *        }
	 *    }
	 * @see SendStartRecordingCommandInternal
	 */
	[[nodiscard]] virtual TNotNull<TUniquePtr<FStartRecordingCommandMessage>> BuildStartRecordingParamsInternal() = 0;

	virtual void SendStartRecordingCommandInternal(const FMessageAddress& InAddress, TUniquePtr<FStartRecordingCommandMessage> InStartRecordingCommand) = 0;

	virtual void SendStopRecordingCommandInternal(const FMessageAddress& InAddress) = 0;

	UE_API virtual bool IsRecordingAvailableForSessionInternal(const TSharedRef<FSessionInfo>& InSessionInfo) const;

	/**
	 * @return Whether a given session info should be excluded from the list exposed to the user
	 */
	UE_API virtual bool ShouldFilterOutSessionInternal(const TSharedRef<FSessionInfo>& InSessionInfo) const;

	virtual void OnToggleRecordingStateInternal(const TSharedRef<FSessionInfo>& InSessionInfo) = 0;
	virtual void CloseSessionByRemoteSessionIDInternal(const FGuid& InSessionId) = 0;

	virtual const FTraceSessionDescriptor& GetCurrentSessionDescriptorInternal() = 0;

	UE_API void SelectTargetSession(FGuid SessionId);

	UE_API void ToggleRecordingState(const TSharedPtr<FSessionInfo>& InSessionInfo);
	FReply ToggleRecordingState()
	{
		ToggleRecordingState(GetCurrentSessionInfo());
		return FReply::Handled();
	}

	UE_API void ToggleSingleSessionRecordingState(const TSharedRef<FSessionInfo>& InSessionInfo);

	UE_API void SetSessionRecordingState(bool bIsRecording, const TSharedRef<FSessionInfo>& SessionInfoRef);
	UE_API bool IsRecording() const;
	UE_API bool IsRecordingPossibleForSession(const TSharedPtr<FSessionInfo>& InSessionInfo) const;
	bool IsRecordingToggleButtonEnabled() const
	{
		return IsRecordingPossibleForSession(GetCurrentSessionInfo());
	}

	UE_API FSlateIcon GetIconForSession(FGuid SessionId) const;

	const TWeakPtr<FRemoteSessionsManager> RemoteSessionManagerWeakPtr;
#if WITH_TRACE_BASED_DEBUGGERS
	TWeakPtr<IDataRelayTransport> TraceRelayTransportInstanceWeakPtr;
#endif
	const FSlateBrush* StartRecordingBrush = nullptr;
	const FSlateBrush* StopRecordingBrush = nullptr;
	const FSlateBrush* StatusBarRecordingBrush = nullptr;
	FSlateIcon RecordingSessionIcon;

	const FLogCategoryBase& LogCategory;
	FGuid DebuggerGuid;

private:

	struct FAsyncConnectionAttemptTask : public TSharedFromThis<FAsyncConnectionAttemptTask>
	{
		explicit FAsyncConnectionAttemptTask(const FGuid InSessionID
			, const FLogCategoryBase& LogCategory
			, const TFunction<bool()>& InRecordingStartAttemptCallback
			, const TFunction<void()>& InRecordingFailedCallback
			, const TWeakPtr<FRecordingControls>& RequestOwner
		)
			: RecordingStartAttemptCallback(InRecordingStartAttemptCallback),
			RecordingFailedCallback(InRecordingFailedCallback),
			Owner(RequestOwner),
			SessionID(InSessionID),
			LogCategory(LogCategory)
		{
		}

		~FAsyncConnectionAttemptTask();

		enum class EState : uint8
		{
			NotStarted,
			InProgress,
			Success,
			Canceled,
			Failed
		};

		void Cancel();

		void Start(int32 InMaxRetriesAttempts, float InIntervalBetweenAttemptsSeconds);
		void ScheduleRetry();
		void HandleSuccess();
		void HandleFailure();

	private:

		bool CanExecute() const;

		void Execute();

		TFunction<bool()> RecordingStartAttemptCallback;
		TSharedPtr<SNotificationItem> ProgressNotification;
		const TFunction<void()> RecordingFailedCallback;
		FTSTicker::FDelegateHandle RetryDelegateHandle;
		TWeakPtr<FRecordingControls> Owner;
		FGuid SessionID;
		const FLogCategoryBase& LogCategory;
		int32 RemainingRetries = 10;
		float IntervalBetweenAttemptsSeconds = 1.0f;
		EState State = EState::NotStarted;
	};


	TSharedRef<SWidget> GenerateToggleRecordingStateButton(const FText& StartRecordingTooltip);
	TSharedRef<SWidget> GenerateRecordingTimeTextBlock();

	EVisibility GetRecordingTimeTextBlockVisibility() const;

	TSharedPtr<FSessionInfo> GetSessionInfo(FGuid Id) const;

	void HandleRemoteSessionsUpdated();

	const FSlateBrush* GetRecordOrStopButton() const;

	void HandleRecordingStop(const TSharedPtr<FSessionInfo>& InSessionInfo);
	void HandleRecordingStart(const TSharedPtr<FSessionInfo>& InSessionInfo);

	void ExecuteAsyncConnectionAttemptTaskWithRetry(FGuid SessionID, int32 RemainingRetries, const TFunction<bool()>& InRecordingStartAttemptCallback, const TFunction<void()>& InRecordingFailedCallback);

	static EVisibility IsRecordingToggleButtonVisible();

	FText GetRecordingTimeText() const;

	static TSharedPtr<SNotificationItem> PushConnectionAttemptNotification(const FNotificationInfo& InNotificationInfo);
	static void UpdateConnectionAttemptNotification(const TSharedPtr<SNotificationItem>& InNotification, int32 AttemptsRemaining);
	void HandleConnectionAttemptResult(FGuid SessionGUID, ELiveConnectionAttemptResult Result, const TSharedPtr<SNotificationItem>& InNotification);
	void CancelInFlightConnectionAttempt(const TSharedPtr<FSessionInfo>& SessionInfo);

	FText GetCurrentSelectedSessionName() const;

	FGuid CurrentSelectedSessionId = FGuid();

	FDelegateHandle RecordingStartedHandle;
	FDelegateHandle RecordingStoppedHandle;

	FCurveSequence RecordingAnimation;

	TMap<FGuid, TSharedPtr<FAsyncConnectionAttemptTask>> InFlightAsyncConnectionRequests;

	const FName StatusBarID;

	FStatusBarMessageHandle RecordingMessageHandle;
	FStatusBarMessageHandle RecordingPathMessageHandle;
	FStatusBarMessageHandle LiveSessionEndedMessageHandle;

	float IntervalBetweenAutoplayConnectionAttemptsSeconds = 1.0f;

	bool bInitialized = false;

	bool bRecordingButtonHovered = false;

	static constexpr int32 DefaultNumberOfConnectionRetries = 3;
};

} // UE::TraceBasedDebuggers

#undef UE_API

#endif // WITH_EDITOR