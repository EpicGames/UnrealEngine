// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Widgets/SCompoundWidget.h"

#define UE_API TRACEBASEDDEBUGGERSANALYSIS_API

namespace UE::TraceBasedDebuggers
{
struct FRemoteSessionsManager;

/**
 * Status Bar widget that provides information about active trace-based debugger connections
 */
class SConnectionStatusBar : public SCompoundWidget
{
public:
	SLATE_DECLARE_WIDGET_API(SConnectionStatusBar, SCompoundWidget, UE_API)

	SLATE_BEGIN_ARGS(SConnectionStatusBar)
	{
	}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const FGuid& InDebuggerGuid, const TSharedRef<FRemoteSessionsManager>& InRemoteSessionManager);

private:

	const FSlateBrush* GetStateBackgroundIcon() const;
	FText GetTitleText() const;
	FText GetTitleToolTipText() const;
	FSlateColor GetDownloadIconColor() const;
	FSlateColor GetUploadingIconColor() const;
	FSlateColor GetWaitingIconColor() const;
	FSlateColor GetReadyIconColor() const;

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void UpdateConnectionStatus(double DeltaTime);

	double ElapsedTime = 0.0;
	double ElapsedTimeSinceLastUpdate = 0.0;

	enum class EState
	{
		Ready,
		ReceivingData,
		WaitingForData
	};

	TWeakPtr<FRemoteSessionsManager> RemoteSessionManagerWeakPtr;
	FGuid DebuggerGuid;
	EState State = EState::Ready;

	int64 CurrentPendingBytes = 0;
	uint64 CurrentReceivedDataBytesPerSecond = 0;
	uint64 CurrentTotalReceivedData = 0;
};

}

#undef UE_API

#endif // WITH_EDITOR