// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	/** 
	 * This module is extending the sharable functionalities of the TraceBasedDebuggers module
	 * for targets supporting trace analysis.
	 */
	public class TraceBasedDebuggersAnalysis : ModuleRules
	{
		public TraceBasedDebuggersAnalysis(ReadOnlyTargetRules Target) : base(Target)
		{
			bAllowUETypesInNamespaces = true;
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"TraceBasedDebuggers"
				}
				);

			if (Target.bBuildEditor || Target.bCompileAgainstEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[]
					{
						"EditorSubsystem",
						"EditorWidgets",
						"Slate",
						"SlateCore",
						"StatusBar",
						"Sockets",
						"ToolMenus",
						"UnrealEd"
					}
					);
			}

			// Allow debugger trace analysis on desktop platforms
			if (Target.Platform.IsInGroup(UnrealPlatformGroup.Desktop) && Target.bBuildDeveloperTools)
			{
				PublicDefinitions.Add("UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS=1");
				PublicDependencyModuleNames.AddRange(
					new[]
					{
						"TraceServices",
						"TraceAnalysis"
					}
				);
			}
			else
			{
				PublicDefinitions.Add("UE_WITH_TRACE_BASED_DEBUGGERS_ANALYSIS=0");
			}

			if (AreTraceBasedDebuggersSupported(Target))
			{
				PublicDefinitions.Add("WITH_TRACE_BASED_DEBUGGERS=1");
			}
			else
			{
				PublicDefinitions.Add("WITH_TRACE_BASED_DEBUGGERS=0");
			}
		}
	}
}
