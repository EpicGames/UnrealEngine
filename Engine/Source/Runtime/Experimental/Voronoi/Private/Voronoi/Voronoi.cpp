// Copyright Epic Games, Inc. All Rights Reserved.
#include "Voronoi/Voronoi.h"

#include "Async/ParallelFor.h"

THIRD_PARTY_INCLUDES_START
#include "voro++/voro++.hh"
THIRD_PARTY_INCLUDES_END

namespace {

	constexpr double VoronoiDefaultBoundingBoxSlack = .01f;

	// initialize AABB, ignoring NaNs
	FBox SafeInitBounds(const TArrayView<const FVector>& Points)
	{
		FBox BoundingBox;
		BoundingBox.Init();
		for (const FVector &V : Points)
		{
			if (!V.ContainsNaN())
			{
				BoundingBox += V;
			}
		}
		return BoundingBox;
	}

	// Add sites to Voronoi container, with contiguous ids, ignoring NaNs
	void PutSites(voro::container *Container, const TArrayView<const FVector>& Sites, int32 Offset)
	{
		for (int i = 0; i < Sites.Num(); i++)
		{
			const FVector &V = Sites[i];
			if (V.ContainsNaN())
			{
				ensureMsgf(false, TEXT("Cannot construct voronoi neighbor for site w/ NaNs in position vector"));
			}
			else
			{
				Container->put(Offset + i, V.X, V.Y, V.Z);
			}
		}
	}

	// Add sites to Voronoi container, with contiguous ids, ignoring NaNs, ignoring Sites that are on top of existing sites
	int32 PutSitesWithDistanceCheck(voro::container *Container, const TArrayView<const FVector>& Sites, int32 Offset, double SquaredDistThreshold = 1e-4)
	{
		int32 SkippedPts = 0;
		voro::voro_compute<voro::container> VoroCompute = Container->make_compute();
		for (int i = 0; i < Sites.Num(); i++)
		{
			const FVector &V = Sites[i];
			if (V.ContainsNaN())
			{
				SkippedPts++;
				ensureMsgf(false, TEXT("Cannot construct voronoi neighbor for site w/ NaNs in position vector"));
			}
			else
			{
				double EX, EY, EZ;
				int ExistingPtID;
				if (Container->find_voronoi_cell(V.X, V.Y, V.Z, EX, EY, EZ, ExistingPtID, VoroCompute))
				{
					double dx = V.X - EX;
					double dy = V.Y - EY;
					double dz = V.Z - EZ;
					if (dx*dx + dy * dy + dz * dz < SquaredDistThreshold)
					{
						SkippedPts++;
						continue;
					}
				}
				Container->put(Offset + i, V.X, V.Y, V.Z);
			}
		}
		return SkippedPts;
	}

	TPimplPtr<voro::container> EmptyVoroContainerInit(int32 ExpectedNumSites, FBox& BoundingBox,
		double BoundingBoxSlack = VoronoiDefaultBoundingBoxSlack)
	{
		BoundingBox = BoundingBox.ExpandBy(BoundingBoxSlack);
		FVector BoundingBoxSize = BoundingBox.GetSize();

		// If points are too far apart, voro++ will ask for unbounded memory to build its grid over space
		// TODO: Figure out reasonable bounds / behavior for this case
		ensure(BoundingBoxSize.GetMax() < HUGE_VALF);

		int GridCellsX, GridCellsY, GridCellsZ;
		voro::guess_optimal(ExpectedNumSites, BoundingBoxSize.X, BoundingBoxSize.Y, BoundingBoxSize.Z, GridCellsX, GridCellsY, GridCellsZ);

		TPimplPtr<voro::container> Container = MakePimpl<voro::container>(
			BoundingBox.Min.X, BoundingBox.Max.X, BoundingBox.Min.Y,
			BoundingBox.Max.Y, BoundingBox.Min.Z, BoundingBox.Max.Z,
			GridCellsX, GridCellsY, GridCellsZ, false, false, false, 10
			);

		return Container;
	}

	TPimplPtr<voro::container> StandardVoroContainerInit(const TArrayView<const FVector>& Sites, FBox& BoundingBox, 
		double BoundingBoxSlack = VoronoiDefaultBoundingBoxSlack, double SquaredDistSkipPtThreshold = 0.0f)
	{
		TPimplPtr<voro::container> Container = EmptyVoroContainerInit(Sites.Num(), BoundingBox, BoundingBoxSlack);
		
		if (SquaredDistSkipPtThreshold > 0)
		{
			PutSitesWithDistanceCheck(Container.Get(), Sites, 0, SquaredDistSkipPtThreshold);
		}
		else
		{
			PutSites(Container.Get(), Sites, 0);
		}

		return Container;
	}


};

FVoronoiDiagram::~FVoronoiDiagram() = default;

FVoronoiDiagram::FVoronoiDiagram(const TArrayView<const FVector>& Sites, double ExtraBoundingSpace, double SquaredDistSkipPtThreshold) : NumSites(0), Container(nullptr)
{
	FBox BoundingBox = SafeInitBounds(Sites).ExpandBy(FMath::Max(0, ExtraBoundingSpace));
	Container = StandardVoroContainerInit(Sites, BoundingBox, 0, SquaredDistSkipPtThreshold);
	this->Bounds = BoundingBox;
	NumSites = Sites.Num();
}

FVoronoiDiagram::FVoronoiDiagram(const TArrayView<const FVector>& Sites, const FBox &Bounds, double ExtraBoundingSpace, double SquaredDistSkipPtThreshold) : NumSites(0), Container(nullptr)
{
	FBox BoundingBox = Bounds.ExpandBy(FMath::Max(0, ExtraBoundingSpace));
	Container = StandardVoroContainerInit(Sites, BoundingBox, 0, SquaredDistSkipPtThreshold);
	this->Bounds = BoundingBox;
	NumSites = Sites.Num();
}

FVoronoiDiagram::FVoronoiDiagram(int32 ExpectedNumSites, const FBox& Bounds, double ExtraBoundingSpace) : NumSites(0), Container(nullptr)
{
	FBox BoundingBox = Bounds.ExpandBy(FMath::Max(0, ExtraBoundingSpace));
	Container = EmptyVoroContainerInit(ExpectedNumSites, BoundingBox, 0);
	this->Bounds = BoundingBox;
}


FBox FVoronoiDiagram::GetBounds(const TArrayView<const FVector>& Sites, double ExtraBoundingSpace)
{
	FBox Bounds = SafeInitBounds(Sites);
	return Bounds.ExpandBy(ExtraBoundingSpace);
}

void FVoronoiDiagram::Initialize(const TArrayView<const FVector>& Sites, const FBox& BoundsIn, double ExtraBoundingSpace, double SquaredDistSkipPtThreshold)
{
	FBox BoundingBox = BoundsIn.ExpandBy(FMath::Max(0, ExtraBoundingSpace));
	Container = StandardVoroContainerInit(Sites, BoundingBox, 0, SquaredDistSkipPtThreshold);
	Bounds = BoundingBox;
	NumSites = Sites.Num();
}


bool VoronoiNeighbors(const TArrayView<const FVector> &Sites, TArray<TArray<int>> &Neighbors, bool bExcludeBounds, double SquaredDistSkipPtThreshold) 
{
	FBox BoundingBox = SafeInitBounds(Sites);
	auto Container = StandardVoroContainerInit(Sites, BoundingBox, VoronoiDefaultBoundingBoxSlack, SquaredDistSkipPtThreshold);
	int32 NumSites = Sites.Num();

	Neighbors.Empty(NumSites);
	Neighbors.SetNum(NumSites, EAllowShrinking::Yes);

	voro::voro_compute<voro::container> VoroCompute = Container->make_compute();
	voro::c_loop_all CellIterator(*Container);
	voro::voronoicell_neighbor cell;
	if (CellIterator.start())
	{
		do
		{
			bool bCouldComputeCell = Container->compute_cell(cell, CellIterator, VoroCompute);
			ensureMsgf(bCouldComputeCell, TEXT("Failed to compute a Voronoi cell -- this may indicate sites positioned directly on top of other sites, which is not valid for a Voronoi diagram"));
			if (bCouldComputeCell)
			{
				int id = CellIterator.pid();

				cell.neighborsTArray(Neighbors[id], bExcludeBounds);
			}
		} while (CellIterator.inc());
	}

	return true;
}

bool GetVoronoiEdges(const TArrayView<const FVector>& Sites, const FBox& Bounds, TArray<TTuple<FVector, FVector>>& Edges, TArray<int32>& CellMember, double SquaredDistSkipPtThreshold)
{
	int32 NumSites = Sites.Num();
	FBox BoundingBox(Bounds);
	FVoronoiDiagram Diagram(Sites, Bounds, VoronoiDefaultBoundingBoxSlack, SquaredDistSkipPtThreshold);
	Diagram.ComputeCellEdges(Edges, CellMember, Diagram.ApproxSitesPerThreadWithDefault(-1));
	return true;
}


// TODO: maybe make this a "SetSites" instead of an Add?
void FVoronoiDiagram::AddSites(const TArrayView<const FVector>& AddSites, double SquaredDistSkipPtThreshold)
{
	int32 OrigSitesNum = NumSites;
	if (SquaredDistSkipPtThreshold > 0)
	{
		PutSitesWithDistanceCheck(Container.Get(), AddSites, OrigSitesNum, SquaredDistSkipPtThreshold);
	}
	else
	{
		PutSites(Container.Get(), AddSites, OrigSitesNum);
	}
	NumSites += AddSites.Num();
}

void FVoronoiDiagram::ComputeAllCellsSerial(TArray<FVoronoiCellInfo>& AllCells)
{
	check(Container);
	voro::voro_compute<voro::container> VoroCompute = Container->make_compute();

	AllCells.SetNum(NumSites);

	voro::c_loop_all CellIterator(*Container);
	voro::voronoicell_neighbor cell;

	if (CellIterator.start())
	{
		do
		{
			bool bCouldComputeCell = Container->compute_cell(cell, CellIterator, VoroCompute);
			ensureMsgf(bCouldComputeCell, TEXT("Failed to compute a Voronoi cell -- this may indicate sites positioned directly on top of other sites, which is not valid for a Voronoi diagram"));
			if (bCouldComputeCell)
			{
				int32 id = CellIterator.pid();
				double x, y, z;
				CellIterator.pos(x, y, z);

				FVoronoiCellInfo& Cell = AllCells[id];
				cell.extractCellInfo(FVector(x,y,z), Cell.Vertices, Cell.Faces, Cell.Neighbors, Cell.Normals);
			}
		} while (CellIterator.inc());
	}
}

TArray<int32> FVoronoiDiagram::GetParallelBlockRanges(int32 ApproxSitesPerThread)
{
	TArray<int32> BlockBounds;

	ApproxSitesPerThread = ApproxSitesPerThreadWithDefault(ApproxSitesPerThread);
	if (ApproxSitesPerThread >= NumSites * 1.5)
	{
		BlockBounds.Add(0); // just single-thread; put everything in a single chunk
		BlockBounds.Add(Container->nxyz);
		return BlockBounds;
	}

	BlockBounds.Reserve(2 + NumSites / ApproxSitesPerThread);
	BlockBounds.Add(0);
	int32 AccumSites = 0;
	for (int32 ijk = 0; ijk < Container->nxyz; ijk++)
	{
		if (AccumSites >= ApproxSitesPerThread)
		{
			BlockBounds.Add(ijk);
			AccumSites = 0;
		}
		AccumSites += Container->co[ijk];
	}
	// final bound should cover the full container, either by extending the existing final bound to cover it or creating a new final bound
	if (AccumSites < ApproxSitesPerThread * .5)
	{
		BlockBounds.Last() = Container->nxyz;
	}
	else // enough sites to merit a separate 'thread'
	{
		BlockBounds.Add(Container->nxyz);
	}
	return BlockBounds;
}

void FVoronoiDiagram::ComputeAllCells(TArray<FVoronoiCellInfo>& AllCells, int32 ApproxSitesPerThread)
{
	check(Container);

	TArray<int32> ChunkBounds = GetParallelBlockRanges(ApproxSitesPerThread);
	if (ChunkBounds.Num() < 3)
	{
		ComputeAllCellsSerial(AllCells);
		return;
	}

	AllCells.SetNum(NumSites);

	ParallelFor(ChunkBounds.Num() - 1, [&](int ChunkIdx)
	{
		voro::voro_compute<voro::container> VoroCompute = Container->make_compute();
		voro::c_loop_block_range CellIterator(*Container);
		CellIterator.setup_range(ChunkBounds[ChunkIdx], ChunkBounds[ChunkIdx + 1]);
		voro::voronoicell_neighbor cell;
		if (!CellIterator.start())
		{
			return;
		}
		do
		{
			bool bCouldComputeCell = Container->compute_cell(cell, CellIterator, VoroCompute);
			ensureMsgf(bCouldComputeCell, TEXT("Failed to compute a Voronoi cell -- this may indicate sites positioned directly on top of other sites, which is not valid for a Voronoi diagram"));
			if (bCouldComputeCell)
			{
				int32 id = CellIterator.pid();
				double x, y, z;
				CellIterator.pos(x, y, z);

				FVoronoiCellInfo& Cell = AllCells[id];
				check(Cell.Faces.Num() == 0); // TODO RM
				cell.extractCellInfo(FVector(x,y,z), Cell.Vertices, Cell.Faces, Cell.Neighbors, Cell.Normals);
			}
		} while (CellIterator.inc());
	});
}

void FVoronoiDiagram::ComputeAllNeighbors(TArray<TArray<int32>>& AllNeighbors, bool bExcludeBounds, int32 ApproxSitesPerThread)
{
	check(Container);

	TArray<int32> ChunkBounds = GetParallelBlockRanges(ApproxSitesPerThread);

	AllNeighbors.SetNum(NumSites);

	ParallelFor(ChunkBounds.Num() - 1, [&](int ChunkIdx)
	{
		voro::voro_compute<voro::container> VoroCompute = Container->make_compute();
		voro::c_loop_block_range CellIterator(*Container);
		CellIterator.setup_range(ChunkBounds[ChunkIdx], ChunkBounds[ChunkIdx + 1]);
		voro::voronoicell_neighbor cell;
		if (!CellIterator.start())
		{
			return;
		}
		do
		{
			bool bCouldComputeCell = Container->compute_cell(cell, CellIterator, VoroCompute);
			ensureMsgf(bCouldComputeCell, TEXT("Failed to compute a Voronoi cell -- this may indicate sites positioned directly on top of other sites, which is not valid for a Voronoi diagram"));
			if (bCouldComputeCell)
			{
				int32 id = CellIterator.pid();

				cell.neighborsTArray(AllNeighbors[id], bExcludeBounds);
			}
		} while (CellIterator.inc());
	});
}

void FVoronoiDiagram::ComputeCellEdgesSerial(TArray<TTuple<FVector, FVector>>& Edges, TArray<int32>& CellMember)
{
	check(Container);

	TArray<FVector> Vertices;
	TArray<int32> FacesVertices;

	voro::c_loop_all CellIterator(*Container);
	voro::voronoicell cell;
	voro::voro_compute<voro::container> VoroCompute = Container->make_compute();

	if (CellIterator.start())
	{
		if (!CellIterator.start())
		{
			return;
		}
		do
		{
			bool bCouldComputeCell = Container->compute_cell(cell, CellIterator, VoroCompute);
			ensureMsgf(bCouldComputeCell, TEXT("Failed to compute a Voronoi cell -- this may indicate sites positioned directly on top of other sites, which is not valid for a Voronoi diagram"));
			if (bCouldComputeCell)
			{
				int32 ID = CellIterator.pid();
				double X, Y, Z;
				CellIterator.pos(X, Y, Z);
				cell.extractCellInfo(FVector(X, Y, Z), Vertices, FacesVertices);

				int32 FaceOffset = 0;
				for (size_t ii = 0, ni = FacesVertices.Num(); ii < ni; ii += FacesVertices[ii] + 1)
				{
					int32 VertCount = FacesVertices[ii];
					int32 PreviousVertexIndex = FacesVertices[FaceOffset + VertCount];
					for (int32 kk = 0; kk < VertCount; ++kk)
					{
						CellMember.Emplace(ID);
						int32 VertexIndex = FacesVertices[1 + FaceOffset + kk]; // Index of vertex X coordinate in raw coordinates array

						Edges.Emplace(Vertices[PreviousVertexIndex], Vertices[VertexIndex]);
						PreviousVertexIndex = VertexIndex;
					}
					FaceOffset += VertCount + 1;
				}
			}
		} while (CellIterator.inc());
	}
}

void FVoronoiDiagram::ComputeCellEdges(TArray<TTuple<FVector, FVector>>& Edges, TArray<int32>& CellMember, int32 ApproxSitesPerThread)
{
	check(Container);

	TArray<int32> ChunkBounds = GetParallelBlockRanges(ApproxSitesPerThread);

	if (ChunkBounds.Num() < 3)
	{
		ComputeCellEdgesSerial(Edges, CellMember);
		return;
	}

	struct FChunkOutput
	{
		TArray<TTuple<FVector, FVector>> Edges;
		TArray<int32> CellMember;
	};
	TArray<FChunkOutput> ChunkOutputs;
	ChunkOutputs.SetNum(ChunkBounds.Num() - 1);

	ParallelFor(ChunkBounds.Num() - 1, [this, &ChunkOutputs, &ChunkBounds](int ChunkIdx)
	{
		FChunkOutput& Chunk = ChunkOutputs[ChunkIdx];
		TArray<FVector> Vertices;
		TArray<int32> FacesVertices;

		voro::voro_compute<voro::container> VoroCompute = Container->make_compute();
		voro::c_loop_block_range CellIterator(*Container);
		CellIterator.setup_range(ChunkBounds[ChunkIdx], ChunkBounds[ChunkIdx + 1]);
		voro::voronoicell cell;
		if (!CellIterator.start())
		{
			return;
		}
		do
		{
			bool bCouldComputeCell = Container->compute_cell(cell, CellIterator, VoroCompute);
			ensureMsgf(bCouldComputeCell, TEXT("Failed to compute a Voronoi cell -- this may indicate sites positioned directly on top of other sites, which is not valid for a Voronoi diagram"));
			if (bCouldComputeCell)
			{
				int32 ID = CellIterator.pid();
				double X, Y, Z;
				CellIterator.pos(X, Y, Z);
				cell.extractCellInfo(FVector(X, Y, Z), Vertices, FacesVertices);

				int32 FaceOffset = 0;
				for (size_t ii = 0, ni = FacesVertices.Num(); ii < ni; ii += FacesVertices[ii] + 1)
				{
					int32 VertCount = FacesVertices[ii];
					int32 PreviousVertexIndex = FacesVertices[FaceOffset + VertCount];
					for (int32 kk = 0; kk < VertCount; ++kk)
					{
						Chunk.CellMember.Emplace(ID);
						int32 VertexIndex = FacesVertices[1 + FaceOffset + kk]; // Index of vertex X coordinate in raw coordinates array

						Chunk.Edges.Emplace(Vertices[PreviousVertexIndex], Vertices[VertexIndex]);
						PreviousVertexIndex = VertexIndex;
					}
					FaceOffset += VertCount + 1;
				}
			}
		} while (CellIterator.inc());
	});

	int ToReserve = 0;
	for (const FChunkOutput& Chunk : ChunkOutputs)
	{
		ToReserve += Chunk.Edges.Num();
	}
	Edges.Reset(ToReserve);
	CellMember.Reset(ToReserve);

	for (const FChunkOutput& Chunk : ChunkOutputs)
	{
		Edges.Append(Chunk.Edges);
		CellMember.Append(Chunk.CellMember);
	}
}

int32 FVoronoiDiagram::FindCell(const FVector& Pos, FVoronoiComputeHelper& ComputeHelper, FVector& OutFoundSite) const
{
	check(Container && ComputeHelper.Compute && &ComputeHelper.Compute->con == Container.Get());
	
	int pid;
	bool found = Container->find_voronoi_cell(Pos.X, Pos.Y, Pos.Z, OutFoundSite.X, OutFoundSite.Y, OutFoundSite.Z, pid, *ComputeHelper.Compute.Get());
	
	return found ? pid : -1;
}

FVoronoiComputeHelper FVoronoiDiagram::GetComputeHelper() const
{
	FVoronoiComputeHelper Helper;
	Helper.Compute = MakePimpl<voro::voro_compute<voro::container>>(Container->make_compute());
	return Helper;
}

const int FVoronoiDiagram::MinDefaultSitesPerThread = 150;