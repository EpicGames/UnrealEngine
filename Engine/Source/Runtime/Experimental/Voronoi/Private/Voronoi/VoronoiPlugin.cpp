// Copyright Epic Games, Inc. All Rights Reserved.

#include "Voronoi/VoronoiPlugin.h"

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"




class FVoronoiPlugin : public IVoronoiPlugin
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE( FVoronoiPlugin, Voronoi )

void FVoronoiPlugin::StartupModule()
{
	
}


void FVoronoiPlugin::ShutdownModule()
{
	
}



