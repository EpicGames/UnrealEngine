Version 0.4.6 (October 17th 2013)
=================================
* Fixed an issue with template instantiation in wall.cc that was causing
  compilation problems with new versions of Apple Xcode. Thanks to Barend
  Thijsse (Delft University of Technology) and Juan Pablo Cruz Pérez (Instituto
  de Física y Matemáticas) for pointing this out.
* Added routines to the container classes to compute a ghost Voronoi cell at
  any given location, as though an extra particle was inserted at that
  location
* Removed unused private variable in c_loops.hh
* Updated Doxyfile to 1.8.5

Version 0.4.5 (July 27th 2012)
==============================
* Removed "using namespace std" from the header files and added extra "std::"
  qualifiers on STL vectors and queues. This is to prevent naming conflicts
  with other libraries. Thanks to Florian Hoffmann (University of Luxembourg)
  for pointing this out.
* Fixed an error in the check_facets diagnostic routine. Thanks to Tristan
  Carrier-Baudouin (Université catholique de Louvain) for pointing this out.
* Fixed a memory leak in the new container_periodic class. Thanks to Massimo
  Marchi (Commissariat á l'Energie Atomique) for pointing this out.
* Removed some duplicate or unnecessary #include statements of the standard C++
  header files
* Fix small naming mismatch between cell.cc and cell.hh in plane_intersects
  and nplane routines, which would give a warning with Doxygen
* More changes for older MS Visual C++ compatability
* Updated Doxyfile to 1.8.1.1

Version 0.4.4 (January 17th 2012)
=================================
* Fixed an error in pointer arithmetic that was introduced in version 0.4,
  which would occur in cases where the delete stack was extended when a large
  number of vertices were being removed. Thanks to Jeff Johnson (Lawrence
  Berkeley Laboratory) and Matt Freeman (Texas A&M University) for pointing
  this out.

Version 0.4.3 (November 12th 2011)
==================================
* Removed spurious Gnuplot output line from torus.cc example
* More changes for better Visual C++ compatibility
* Fixed indexing error in worklist generation routine. This had no effect
  on the generated worklists, but would become apparent in other situations.
* Added rad_option.hh to the list of files included by voro++.hh for
  completeness
* Fix errors in Gnuplot commands in the basic examples README file. Thanks to
  William-Fernando Oquendo (National University of Colombia) for pointing this
  out.
* Fixed memory allocation issue in face_freq_table routine - thanks to Lingti
  Kong for pointing this out
* Minor cleanups and optimizations to wall classes

Version 0.4.2 (September 23rd 2011)
===================================
* Fixed a bug in the put_remap routine where a z periodic boolean was
  switched to an x periodic boolean. This would only be noticed when using
  mixed periodic boundary conditions and importing particles that need to be
  remapped.
* The pid function in the c_loop_base class was mistakenly labeled as double
  instead of int - this is now fixed. Thanks to William-Fernando Oquendo
  (National University of Colombia) for pointing this out.
* Fixed an extremely subtle bounds checking issue that would only materialize
  for the radical Voronoi tessellation with large particle radii. Thanks to
  Richard Martin and Maciej Haranczyk (Lawrence Berkeley Laboratory) for
  pointing out the original problem.
* Fixed other problems with the radical tessellation. Originally, the
  algorithms were concieved for with hard spheres in mind, where the particles
  would not overlap, and an assumption was used in the cutoff procedure that
  made use of this, but would not be true in general.
* Removed more bracket initializations like "double a(3);" for better
  consistency, and more compatibility with older versions of MS Visual C++.
  Thanks to Martin Brehm for help with this.
* Factored out the regular/radical inline routines into separate classes, so
  that they don't have to be duplicated in the container_periodic classes
* Fixed problems with the container_periodic classes referencing the wrong
  loop classes. Thanks to Wang Chuncheng (Nanyang Technological University) for
  pointing this out.
* Added c_loop_order_periodic class for doing ordered loops with the periodic
  container classes
* Updated the README file in the src directory. The detailed description of
  each file has been removed since this is difficult to maintain and this
  information is already present in the class reference manual.
* Fixed an error in the pre_container selection of the optimal computational
  grid size
* Code comment cleanups, particularly in wall.hh
* Minor man page reformatting
* Updated Doxyfile to version 1.7.5.1

Version 0.4.1 (September 7th 2011)
==================================
* Fixed bug in command-line utility that would give incorrect results when
  using the "-c" option for custom output. Thanks to Prof. Aldo Romero
  (CINVESTAV-Unidad Queretaro) for pointing this out.
* Added some additional documentation about neighbor information

Version 0.4 (August 30th 2011)
==============================
* New classes called container_periodic and container_periodic_poly that carry
  out Voronoi computations in 3D periodic parallelepiped unit cells have been
  added. This adds significant complexity to the computation, and the memory
  organization of these classes is different, requiring allocation for ghost
  regions. Currently these classes cannot be accessed by the command-line
  utility, but will be integrated further in future releases. These classes are
  currently mainly exploited by a sister software project Zeo++ for chemical
  informatics (Maciej Haranczyk, Chris Rycroft, Thomas Willems, Richard Martin)
  to be released in late 2011.
* As part of the new container_periodic classes, a new class called unitcell
  has been added, which will compute the Voronoi cell for a single particle in
  a 3D periodic parallelepiped class. The Voronoi cell is formed by the
  influences of the periodic images of itself only. This turns out to be an
  important component of the calculations done by the container_periodic
  classes but can also be used independently.
* The container classes now have a routine called find_voronoi_cell that will
  take a given position vector and return the Voronoi cell which is contained
  within. For the regular Voronoi tessellation, this is just the Voronoi cell
  corresponding to the closest particle (by definition). For the radical
  Voronoi tessellation, this is weighted by the particle radii.
* The library is now enclosed within the "voro" namespace. This requires
  specifying "using namespace voro;" in the preamble of codes or prepending
  "voro::" to access functions. This greatly minimizes the possiblity of naming
  conflicts when using multiple libraries, particularly since many of the
  classes iin the library have fairly generic names (eg. "container"). Thanks to
  Allan Johns (Dr. D Studios) for suggesting this.
* Voro++ can now be built as a static library that can be installed and linked
  to. As part of this, the .cc and .hh files have been significantly revised
  with some small inline functions and template code being written into the
  header files.
* Previously, all of the voronoicell output routines sent data to an output
  stream. However, input and output are now carried out using the STL vector
  class, allowing for programs to directly link to and use Voro++ output. The
  STL vector class has been chosen since it the simplest and most standard
  mechanism for returning a variable amount of data. Several new examples are
  provided to demonstrate this functionality, for both the voronoicell class
  and the container class.
* A common issue has been that the order of particles in the output file does
  not match the input file, since Voro++ internally sorts particles according
  to their position. A new class called voro_order has been provided, which can
  link to particles as they are stored, allowing for the Voronoi computation to
  be carried out in the same order as the input file. The core computation
  routines can be run with a variety of different loops, and a new example
  "loops.cc" is provided to demonstrate this. 
* The order of parameters to certain functions have been changed to take
  advantage of the C++ default parameter mechanism wherever possible
* The container and container_poly classes are now derived from a base class,
  rather than being different instances of a template. This simplifies the code
  and stops common routines from being duplicated. The core Voronoi computation
  routine is now held within a voro_compute class, which can be instantiated on
  any container with the correct access functions.
* The voronoicell and voronoicell_neighbor classes are now derived from a
  common base class, rather than being different instances of a template. This
  simplifies the code and stops common routines from being duplicated. The core
  nplane routine is now a function template that can be instantiated on the
  parent class itself. 
* All output routines have been switched from using C++ iostreams to using the C
  cstdio library. This has been done for increased performance. In general the
  cstdio routines run significantly faster than the comparable iostream
  routines: for one file import test, using fscanf from cstdio was five times
  as fast as the comparable << operator from iostream. As another example,
  the program "cylinder.cc", which uses both file import and export, runs
  approximately twice as fast before. Since many people use Voro++ to batch
  process large numbers of input files, this increase in input/output speed
  seemed desirable.
* The Gnuplot output routines now merge individual lines into contiguous paths
  wherever possible, resulting in a ~15% reduction in their file size and a
  noticeable improvement in rendering speed
* Previously, the POV-Ray output routines could create cylinders where the start
  and end points matched. This was a benign problem, due to the fact the vertex
  positions within Voro++ were held to higher precision than the POV-Ray output
  file, meaning that distinct vertices could appear to be at the same position;
  a perl script was provided to post-process and remove these. However, Voro++
  now scans each cylinder before saving to file to automatically remove these
  cases from consideration.
* Pointer arithmetic is now used within a number of critical cell computation
  routines, and gives a speedup of 10%-20% in many cases. On some typical
  problems on typical machines, Voro++ currently computes around 50,000
  cells/second. New machines can achieve upwards of 70,000 cells/second.
* A new pre-container mechanism is provided, which can read in a file of
  unknown length, and then make a guess at the correct computational grid size
  to use. This removes the need for specifying a length scale with the
  command-line utility. However, a length scale or grid decomposition can still
  be manually specified.
* Removed the option to compile in single-precision. This was a highly
  specialized feature, and it appears better to allow users who wish to do this
  to make their own modifications. This option also conflicted with building
  Voro++ as a standard static library.
* Fixed a benign bug in the block computation list that would cause it to get
  extended prematurely
* Numerous small improvements to memory allocation routines
* Added routines to copy a Voronoi cell
* Created a new wall_list class to handle a list of walls. This simplifies the
  command-line utility code.
* Labels in the worklist that are used in the transition from the radius search
  to the block search have been optimized
* The command-line utility can now produce POV-Ray output. Also, the main loop
  in the utility has been combined so that the Voronoi tessellation is only
  carried out once, even if POV-Ray and Gnuplot output is also required.
* The command-line utility does range checking on the filename to prevent
  buffer overruns
* The routines that put particles into a container now do so more reliably for
  periodic cases, remapping them into the primary domain if necessary
* A number of the POV-Ray header files have been altered to directly make use
  of a right-handed coordinate system in which the z axis points upwards. This
  matches the Gnuplot style, and the style used in many scientific simulations.
  It avoids the need for a rotation or matrix transformation when rendering
  many of the particle packings. Some of the test particle packings have been
  transformed to ensure consistency with previous renderings.
* Some POV-Ray files have been cleaned up and there are additional comments
* Fixed a bug in the print_all_custom() routine, affecting "%P" output. Thanks
  to David Waroquiers (Université Catholique de Louvain, Belgium) for pointing
  this out.
* Fixed a bug that caused the neighbor list not to be outputted in the same
  order as the other plane diagnostic routines. Thanks to Olufemi Olorode
  (Texas A&M University) for pointing this out.
* Removed the add_vertex() and init_test() routines, which were mainly used for
  debugging
* Updated Doxyfile to version 1.7.4

Version 0.3.1 (September 29, 2009)
==================================
* Carried out a major overhaul of the source code comments that are used by
  Doxygen. Several errors in the documentation were fixed.
* Additional comments in the Makefiles, and added the "phony target" keyword
* Many code cleanups and standardizations. Some double and triple loops
  were concatenated into one to reduce the amount of indentation.
* Fixed the import.cc example program
* Updated Doxyfile to version 1.6.1

Version 0.3 (August 17, 2009)
=============================
* In response to feedback from several people, the routines for computing
  statistics about computed Voronoi cells have been significantly revised,
  extended, and put within a common framework. The voronoicell class now has a
  large number of routines of the form output_...() that will print cell
  information to an output stream. There are also a number of new, simple
  routines for computing basic statistics such as the number of edges and faces
  of each cell. All of these routines can now be used during the container
  analysis, by making use of the new print_custom() routine, which can output
  according to a given format string, that uses control sequences similar to
  the standard C printf() routine.
* Renamed the "radical" example directory to "custom", and added two
  new programs called "cell_statistics.cc" and "custom_output.cc" for
  demonstrating the new output routines for the voronoicell and container
  classes respectively
* Added a routine for computing the centroid of a Voronoi cell
* Added new routines for computing neighbor normals, in response to a
  request from Dave Greenwood
* Removed unnecessary trailing tabs and spaces in the source code
* Fixed a few text justification problems, and altered some of the comments
  in the example programs
* Improved the command-line utility, allowing it to specify custom information
  in the output. Implemented better error checking on command-line arguments.
* Removed the facets() routine, since this has been superseded by the new
  output_...() routines. This also rendered some functions in the neighbor_none
  and neighbor_track classes obsolete, and they have also been removed.
* Some reorganization of the cell.hh and container.hh header files to group
  similar functions together
* Added torus.cc example that shows how to write custom walls as derived
  classes
* Updated Doxyfile to version 1.5.9

Version 0.2.7 (March 25, 2009)
==============================
* Added some missing cstdlib and cstring header files that cause compilation
  errors on some systems
* Some text cleanups in the main README file and cmd_line.cc
* The worklist_gen.pl script was in DOS format. It has been switched to Unix
  format to match the rest of the code.
* Fixed some incorrect inequalities in degenerate2.cc
* Improved some error messages in container.cc

Version 0.2.6 (March 23, 2009)
==============================
* Significantly improved the error messages generated by the code. It now
  makes use of the exit() routine rather than throwing errors, and it returns
  various status codes (defined in config.hh) for different types of abnormal
  behavior.
* Changed two occurences of "cin" to "cout" in the draw_particles routines
* Corrected error in comment in the tetrahedron example
* Minor comment cleanups of the worklist_gen.pl script
* Updated Doxyfile to version 1.5.8
* Added a routine print_facet_information() in response to a problem
  from Hengxing Lan
* Many of the cell based statistics routines (like number_of_faces() and
  facets()) internally track their progress by temporarily flipping the edge
  table entries to negative values, so that they know where they have already
  been. The common code that was used at the end of each of these functions for
  resetting the edges back to positive has now been made into a private inline
  function called reset_edges().
* Fixed a missing initialization of max_radius to zero in the radius_poly
  class. This should not have ever caused any errors, as max_radius would
  always have been big enough. However if max_radius was initially set to a
  large value, it could have potentially made the code run slower.
* Fixed the code layout in the suretest class header
* Added some additional checks in the command-line utility to screen out
  invalid command-line arguments. Switched the test on the number of
  computational blocks to use floating point arithmetic, because integers
  could potentially overflow and become negative.
* Included several more of the POV-Ray header files that were used to create
  the example images on the website
* Removed a "cout" statement in the wall_cone class
* Cleanup of the README files for the basic examples and the wall examples

Version 0.2.5 (January 1, 2009)
===============================
* Added the DOE acknowledgements to the code overview document

Version 0.2.4 (December 14, 2008)
=================================
* Added the code overview PDF document to the package, in the docs directory
* Fixed some spelling errors in the comments
* Many more documentation updates

Version 0.2.3 (December 9, 2008)
================================
* Removed an unused variable that was reported by the Portland C compiler
* Documentation updates
* Added the import.pov script
* Added some simple functions to solve the problem from Stefan Eibl of counting
  faces
* Renamed the facets_loop to voropp_loop

Version 0.2.2 (November 19, 2008)
=================================
* Main README file updated

Version 0.2.1 (November 3, 2008)
================================
* Much more documentation
* Brief Doxygen class comments
* Updated Doxyfile to version 1.5.7

Version 0.2 (October 31, 2008)
==============================
* Updated some occurrences of char* with const char* for compatability with
  version 4.2 of the GNU C++ compiler
* Making use of default parameters in the wall classes
* The command-line utility can now add walls

Version 0.1 (August 22, 2008)
=============================
* Initial version uploaded to the web
