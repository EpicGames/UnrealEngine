Voro++, a 3D cell-based Voronoi library (http://math.lbl.gov/voro++/)
By Chris H. Rycroft (UC Berkeley / Lawrence Berkeley Laboratory)
================================================================
Voro++ is a software library for carrying out three-dimensional computations
of the Voronoi tessellation. A distinguishing feature of the Voro++ library
is that it carries out cell-based calculations, computing the Voronoi cell
for each particle individually, rather than computing the Voronoi
tessellation as a global network of vertices and edges. It is particularly
well-suited for applications that rely on cell-based statistics, where
features of Voronoi cells (eg. volume, centroid, number of faces) can be
used to analyze a system of particles

Voro++ comprises of several C++ classes that can be built as a static library
and linked to. A command-line utility is also provided that can analyze text
files of particle configurations and use most of the features of the code.
Numerous examples are provided to demonstrate the library's features and all of
these are discussed in detail on the library website.


Compilation - Linux / Mac OS / Windows with Cygwin
==================================================
The code is written in ANSI C++, and compiles on many system architectures. The
package contains the C++ source code, example files, miscellaneous utilities
and documentation. On Linux, Mac OS, and Windows (using Cygwin), the
compilation and installed can be carried out using GNU Make.

To begin, the user should review the file "config.mk" in the top level
directory, to make sure that the compilation and installation settings are
appropriate for their system. Typing "make" will then compile the static
library, command-line utility, and examples. The command-line utility and
library will appear within the "src" directory.

Following successful compilation, the library, command-line utility, and
documentation can be installed by typing "sudo make install". By default, the
program files are installed into /usr/local, and it may be necessary to modify
your environment variables in order to access the installed files:

- to use the command-line utility, the variable PATH should contain
  /usr/local/bin.
- to access the Voro++ man page, the variable MANPATH should contain
  /usr/local/man.
- to access the Voro++ header files, code compilation should include
  the flag '-I/usr/local/include/voro++'.
- to link to the static library, code compilation should include the
  flags '-L/usr/local/lib' to tell the linker where to look, and then
  '-lvoro++' to link to the library.

The library website contains additional notes on setting environment variables,
and many guides are available on the Internet.

The code can later be uninstalled with "sudo make uninstall". It is also
possible to use the library and command-line utility without installation by
calling the files directly once they have been compiled. On systems where the
user does not have root privileges to install into /usr/local, the "config.mk"
file can be modified to install into the user's home directory by setting
PREFIX=$(HOME). Voro++ supports parallel compilation by using the "make -j <n>"
command where n is the number of threads.


Compilation - Windows without Cygwin
====================================
On a Windows machine without a terminal environment like Cygwin, it is possible
to import and compile the library in many standard C++ development
environments. Users have reported success in building the library with
Microsoft Visual C++ Express and Code::Blocks.


Related programs
================
No external dependencies are required to compile and run the code, but several
programs may be useful for analyzing the output:

- The freeware plotting program Gnuplot (available at www.gnuplot.info) can be
  used for rapid 2D and 3D visualization of the program output.

- The freeware raytracer POV-Ray (available at www.povray.org) can be used for
  high-quality renderings of the program output.

- The reference manual is generated from comments in the source code using
  Doxygen (available at www.doxygen.org). This package is only required if the
  library files are being developed and the reference manuals need to be
  regenerated. The complete reference manual to the current code is available
  online at http://math.lbl.gov/voro++/doc/refman/


Contents
========
examples - many documented examples making use of the library
html - an HTML-based reference manual (generated by Doxygen)
man - contains the man page that is installed with the program
scripts - miscellaneous helper scripts
src - source code files


Usage
=====
Voro++ is released as free software through the Lawrence Berkeley National
Laboratory - a detailed copyright notice is provided below, and the complete
terms of the license can be found in the LICENSE file.

I am very interested to hear from users of the software, so if you find this
useful, please email me at chr@alum.mit.edu. Also, if you plan to publish an
academic paper using this software, please consider citing one of the following
publications:

- Chris H. Rycroft, "Voro++: A three-dimensional Voronoi cell library in C++",
  Chaos 19, 041111 (2009).

- Chris H. Rycroft, Gary S. Grest, James W. Landry, and Martin Z. Bazant,
  "Analysis of Granular Flow in a Pebble-Bed Nuclear Reactor",
  Phys. Rev. E 74, 021306 (2006).

- Chris H. Rycroft, "Multiscale Modeling in Granular Flow", PhD thesis
  submitted to the Massachusetts Institute of Technology, September 2007.
  (http://math.berkeley.edu/~chr/publish/phd.html)

The first reference contains a one-page overview of the library. The second
reference contains some of the initial images that were made using a very early
version of this code, to track small changes in packing fraction in a large
particle simulation. The third reference discusses the use of 3D Voronoi cells,
and describes the algorithms that were employed in the early version of this
code. Since the publication of the above references, the algorithms in Voro++
have been significantly improved, and a paper specifically devoted to the
current code architecture will be published during 2012.


Copyright Notice
================
Voro++ Copyright (c) 2008, The Regents of the University of California, through
Lawrence Berkeley National Laboratory (subject to receipt of any required
approvals from the U.S. Dept. of Energy). All rights reserved.

If you have questions about your rights to use or distribute this software,
please contact Berkeley Lab's Technology Transfer Department at TTD@lbl.gov.

NOTICE. This software was developed under partial funding from the U.S.
Department of Energy. As such, the U.S. Government has been granted for itself
and others acting on its behalf a paid-up, nonexclusive, irrevocable, worldwide
license in the Software to reproduce, prepare derivative works, and perform
publicly and display publicly. Beginning five (5) years after the date
permission to assert copyright is obtained from the U.S. Department of Energy,
and subject to any subsequent five (5) year renewals, the U.S. Government is
granted for itself and others acting on its behalf a paid-up, nonexclusive,
irrevocable, worldwide license in the Software to reproduce, prepare derivative
works, distribute copies to the public, perform publicly and display publicly,
and to permit others to do so.


Acknowledgments
===============
This work was supported by the Director, Office of Science, Computational and
Technology Research, U.S. Department of Energy under Contract No.
DE-AC02-05CH11231.
