Voro++ library source files
===========================
This directory contains the source code for the library. For full details of
each file, see the files section of the online reference manual at
http://math.lbl.gov/voro++/doc/refman/

Several other files are present:

worklist_gen.pl - perl script for automatically generating the worklist.hh and
v_base_wl.cc files.
