// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file c_loops.cc
 * \brief Function implementations for the loop classes. */

#include "c_loops.hh"

namespace voro {

/** Initializes a c_loop_subset object to scan over all particles within a
 * given sphere.
 * \param[in] (vx,vy,vz) the position vector of the center of the sphere.
 * \param[in] r the radius of the sphere.
 * \param[in] bounds_test whether to do detailed bounds checking. If this is
 *                        false then the class will loop over all particles in
 *                        blocks that overlap the given sphere. If it is true,
 *                        the particle will only loop over the particles which
 *                        actually lie within the sphere.
 * \return True if there is any valid point to loop over, false otherwise. */
void c_loop_subset::setup_sphere(double vx,double vy,double vz,double r,bool bounds_test) {
	if(bounds_test) {mode=sphere;v0=vx;v1=vy;v2=vz;v3=r*r;} else mode=no_check;
	ai=step_int((vx-ax-r)*xsp);
	bi=step_int((vx-ax+r)*xsp);
	aj=step_int((vy-ay-r)*ysp);
	bj=step_int((vy-ay+r)*ysp);
	ak=step_int((vz-az-r)*zsp);
	bk=step_int((vz-az+r)*zsp);
	setup_common();
}

/** Initializes the class to loop over all particles in a rectangular subgrid
 * of blocks.
 * \param[in] (ai_,bi_) the subgrid range in the x-direction, inclusive of both
 *                      ends.
 * \param[in] (aj_,bj_) the subgrid range in the y-direction, inclusive of both
 *                      ends.
 * \param[in] (ak_,bk_) the subgrid range in the z-direction, inclusive of both
 *                      ends.
 * \return True if there is any valid point to loop over, false otherwise. */
void c_loop_subset::setup_intbox(int ai_,int bi_,int aj_,int bj_,int ak_,int bk_) {
	ai=ai_;bi=bi_;aj=aj_;bj=bj_;ak=ak_;bk=bk_;
	mode=no_check;
	setup_common();
}

/** Sets up all of the common constants used for the loop.
 * \return True if there is any valid point to loop over, false otherwise. */
void c_loop_subset::setup_common() {
	if(!xperiodic) {
		if(ai<0) {ai=0;if(bi<0) bi=0;}
		if(bi>=nx) {bi=nx-1;if(ai>=nx) ai=nx-1;}
	}
	if(!yperiodic) {
		if(aj<0) {aj=0;if(bj<0) bj=0;}
		if(bj>=ny) {bj=ny-1;if(aj>=ny) aj=ny-1;}
	}
	if(!zperiodic) {
		if(ak<0) {ak=0;if(bk<0) bk=0;}
		if(bk>=nz) {bk=nz-1;if(ak>=nz) ak=nz-1;}
	}
	ci=ai;cj=aj;ck=ak;
	di=i=step_mod(ci,nx);apx=px=step_div(ci,nx)*sx;
	dj=j=step_mod(cj,ny);apy=py=step_div(cj,ny)*sy;
	dk=k=step_mod(ck,nz);apz=pz=step_div(ck,nz)*sz;
	inc1=di-step_mod(bi,nx);
	inc2=nx*(ny+dj-step_mod(bj,ny))+inc1;
	inc1+=nx;
	ijk=di+nx*(dj+ny*dk);
	q=0;
}

/** Starts the loop by finding the first particle within the container to
 * consider.
 * \return True if there is any particle to consider, false otherwise. */
bool c_loop_subset::start() {
	while(co[ijk]==0) {if(!next_block()) return false;}
	while(mode!=no_check&&out_of_bounds()) {
		q++;
		while(q>=co[ijk]) {q=0;if(!next_block()) return false;}
	}
	return true;
}

/** Initializes the class to loop over all particles in a rectangular box.
 * \param[in] (xmin,xmax) the minimum and maximum x coordinates of the box.
 * \param[in] (ymin,ymax) the minimum and maximum y coordinates of the box.
 * \param[in] (zmin,zmax) the minimum and maximum z coordinates of the box.
 * \param[in] bounds_test whether to do detailed bounds checking. If this is
 *                        false then the class will loop over all particles in
 *                        blocks that overlap the given box. If it is true, the
 *                        particle will only loop over the particles which
 *                        actually lie within the box.
 * \return True if there is any valid point to loop over, false otherwise. */
void c_loop_subset::setup_box(double xmin,double xmax,double ymin,double ymax,double zmin,double zmax,bool bounds_test) {
	if(bounds_test) {mode=box;v0=xmin;v1=xmax;v2=ymin;v3=ymax;v4=zmin;v5=zmax;} else mode=no_check;
	ai=step_int((xmin-ax)*xsp);
	bi=step_int((xmax-ax)*xsp);
	aj=step_int((ymin-ay)*ysp);
	bj=step_int((ymax-ay)*ysp);
	ak=step_int((zmin-az)*zsp);
	bk=step_int((zmax-az)*zsp);
	setup_common();
}

/** Computes whether the current point is out of bounds, relative to the
 * current loop setup.
 * \return True if the point is out of bounds, false otherwise. */
bool c_loop_subset::out_of_bounds() {
	double *pp=p[ijk]+ps*q;
	if(mode==sphere) {
		double fx(*pp+px-v0),fy(pp[1]+py-v1),fz(pp[2]+pz-v2);
		return fx*fx+fy*fy+fz*fz>v3;
	} else {
		double f(*pp+px);if(f<v0||f>v1) return true;
		f=pp[1]+py;if(f<v2||f>v3) return true;
		f=pp[2]+pz;return f<v4||f>v5;
	}
}

/** Returns the next block to be tested in a loop, and updates the periodicity
 * vector if necessary. */
bool c_loop_subset::next_block() {
	if(i<bi) {
		i++;
		if(ci<nx-1) {ci++;ijk++;} else {ci=0;ijk+=1-nx;px+=sx;}
		return true;
	} else if(j<bj) {
		i=ai;ci=di;px=apx;j++;
		if(cj<ny-1) {cj++;ijk+=inc1;} else {cj=0;ijk+=inc1-nxy;py+=sy;}
		return true;
	} else if(k<bk) {
		i=ai;ci=di;j=aj;cj=dj;px=apx;py=apy;k++;
		if(ck<nz-1) {ck++;ijk+=inc2;} else {ck=0;ijk+=inc2-nxyz;pz+=sz;}
		return true;
	} else return false;
}

/** Extends the memory available for storing the ordering. */
void particle_order::add_ordering_memory() {
	int *no=new int[size_t(size)<<2],*nop=no,*opp=o;
	while(opp<op) *(nop++)=*(opp++);
	delete [] o;
	size<<=1;o=no;op=nop;
}

}
