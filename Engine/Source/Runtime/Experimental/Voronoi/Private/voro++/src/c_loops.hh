// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file c_loops.hh
 * \brief Header file for the loop classes. */

#ifndef VOROPP_C_LOOPS_HH
#define VOROPP_C_LOOPS_HH

#include "Misc/AssertionMacros.h"

#include <cstddef>

#include "config.hh"

namespace voro {

/** A type associated with a c_loop_subset class, determining what type of
 * geometrical region to loop over. */
enum c_loop_subset_mode {
	sphere,
	box,
	no_check
};

/** \brief A class for storing ordering information when particles are added to
 * a container.
 *
 * When particles are added to a container class, they are sorted into an
 * internal computational grid of blocks. The particle_order class provides a
 * mechanism for remembering which block particles were sorted into. The import
 * and put routines in the container class have variants that also take a
 * particle_order class. Each time they are called, they will store the block
 * that the particle was sorted into, plus the position of the particle within
 * the block. The particle_order class can used by the c_loop_order class to
 * specifically loop over the particles that have their information stored
 * within it. */
class particle_order {
	public:
		/** A pointer to the array holding the ordering. */
		int *o;
		/** A pointer to the next position in the ordering array in
		 * which to store an entry. */
		int *op;
		/** The current memory allocation for the class, set to the
		 * number of entries which can be stored. */
		int size;
		/** The particle_order constructor allocates memory to store the
		 * ordering information.
		 * \param[in] init_size the initial amount of memory to
		 *                      allocate. */
		particle_order(int init_size=init_ordering_size)
			: o(new int[size_t(init_size)<<1]),op(o),size(init_size) {}
		/** The particle_order destructor frees the dynamically allocated
		 * memory used to store the ordering information. */
		~particle_order() {
			delete [] o;
		}
		/** Adds a record to the order, corresponding to the memory
		 * address of where a particle was placed into the container.
		 * \param[in] ijk the block into which the particle was placed.
		 * \param[in] q the position within the block where the
		 * 		particle was placed. */
		inline void add(int ijk,int q) {
			if(op==o+size) add_ordering_memory();
			*(op++)=ijk;*(op++)=q;
		}
	private:
		void add_ordering_memory();
};

/** \brief Base class for looping over particles in a container.
 *
 * This class forms the base of all classes that can loop over a subset of
 * particles in a contaner in some order. When initialized, it stores constants
 * about the corresponding container geometry. It also contains a number of
 * routines for interrogating which particle currently being considered by the
 * loop, which are common between all of the derived classes. */
class c_loop_base {
	public:
		/** The number of blocks in the x direction. */
		const int nx;
		/** The number of blocks in the y direction. */
		const int ny;
		/** The number of blocks in the z direction. */
		const int nz;
		/** A constant, set to the value of nx multiplied by ny, which
		 * is used in the routines that step through blocks in
		 * sequence. */
		const int nxy;
		/** A constant, set to the value of nx*ny*nz, which is used in
		 * the routines that step through blocks in sequence. */
		const int nxyz;
		/** The number of floating point numbers per particle in the
		 * associated container data structure. */
		const int ps;
		/** A pointer to the particle position information in the
		 * associated container data structure. */
		double **p;
		/** A pointer to the particle ID information in the associated
		 * container data structure. */
		int **id;
		/** A pointer to the particle counts in the associated
		 * container data structure. */
		int *co;
		/** The current x-index of the block under consideration by the
		 * loop. */
		int i;
		/** The current y-index of the block under consideration by the
		 * loop. */
		int j;
		/** The current z-index of the block under consideration by the
		 * loop. */
		int k;
		/** The current index of the block under consideration by the
		 * loop. */
		int ijk;
		/** The index of the particle under consideration within the current
		 * block. */
		int q;
		/** The constructor copies several necessary constants from the
		 * base container class.
		 * \param[in] con the container class to use. */
		template<class c_class>
		c_loop_base(c_class &con) : nx(con.nx), ny(con.ny), nz(con.nz),
					    nxy(con.nxy), nxyz(con.nxyz), ps(con.ps),
					    p(con.p), id(con.id), co(con.co) {}
		/** Returns the position vector of the particle currently being
		 * considered by the loop.
		 * \param[out] (x,y,z) the position vector of the particle. */
		inline void pos(double &x,double &y,double &z) {
			double *pp=p[ijk]+ps*q;
			x=*(pp++);y=*(pp++);z=*pp;
		}
		/** Returns the ID, position vector, and radius of the particle
		 * currently being considered by the loop.
		 * \param[out] pid the particle ID.
		 * \param[out] (x,y,z) the position vector of the particle.
		 * \param[out] r the radius of the particle. If no radius
		 * 		 information is available the default radius
		 * 		 value is returned. */
		inline void pos(int &pid,double &x,double &y,double &z,double &r) {
			pid=id[ijk][q];
			double *pp=p[ijk]+ps*q;
			x=*(pp++);y=*(pp++);z=*pp;
			r=ps==3?default_radius:*(++pp);
		}
		/** Returns the x position of the particle currently being
		 * considered by the loop. */
		inline double x() {return p[ijk][ps*q];}
		/** Returns the y position of the particle currently being
		 * considered by the loop. */
		inline double y() {return p[ijk][ps*q+1];}
		/** Returns the z position of the particle currently being
		 * considered by the loop. */
		inline double z() {return p[ijk][ps*q+2];}
		/** Returns the ID of the particle currently being considered
		 * by the loop. */
		inline int pid() {return id[ijk][q];}
};

/** \brief Class for looping over all of the particles in a container.
 *
 * This is one of the simplest loop classes, that scans the computational
 * blocks in order, and scans all the particles within each block in order. */
class c_loop_all : public c_loop_base {
	public:
		/** The constructor copies several necessary constants from the
		 * base container class.
		 * \param[in] con the container class to use. */
		template<class c_class>
		c_loop_all(c_class &con) : c_loop_base(con) {}
		/** Sets the class to consider the first particle.
		 * \return True if there is any particle to consider, false
		 * otherwise. */
		inline bool start() {
			i=j=k=ijk=q=0;
			while(co[ijk]==0) if(!next_block()) return false;
			return true;
		}
		/** Finds the next particle to test.
		 * \return True if there is another particle, false if no more
		 * particles are available. */
		inline bool inc() {
			q++;
			if(q>=co[ijk]) {
				q=0;
				do {
					if(!next_block()) return false;
				} while(co[ijk]==0);
			}
			return true;
		}
	private:
		/** Updates the internal variables to find the next
		 * computational block with any particles.
		 * \return True if another block is found, false if there are
		 * no more blocks. */
		inline bool next_block() {
			ijk++;
			i++;
			if(i==nx) {
				i=0;j++;
				if(j==ny) {
					j=0;k++;
					if(ijk==nxyz) return false;
				}
			}
			return true;
		}
};

/** \brief Class for looping over the particles in a range of blocks within a container.
 *
 * This scans the computational blocks in order, and scans all the particles within each block in order. */
class c_loop_block_range : public c_loop_base {
	public:
		/** The constructor copies several necessary constants from the
		 * base container class.  Defaults to full range.
		 * \param[in] con the container class to use. */
		template<class c_class>
		c_loop_block_range(c_class &con) : c_loop_base(con) {}
		/** Sets the class to consider the first particle.
		 * \return True if there is any particle to consider, false
		 * otherwise. */
		inline bool start() {
			ijk = ijk_start;
			checkSlow(ijk >= 0 && ijk < nxyz);
			k = ijk / nxy;
			int ijkt = ijk - nxy * k;
			j = ijkt / nx;
			i = ijkt - j * nx;
			q=0;
			while(co[ijk]==0) if(!next_block()) return false;
			return true;
		}
		void setup_range(int ijk_start_in, int ijk_stop_in)
		{
			ijk_start = ijk_start_in;
			ijk_stop = ijk_stop_in;

			check(ijk_start >= 0 && ijk_start < nxyz && ijk_stop >= 0 && ijk_stop <= nxyz);
			checkSlow(ijk_start < ijk_stop);
		}
		/** Finds the next particle to test.
		 * \return True if there is another particle, false if no more
		 * particles are available. */
		inline bool inc() {
			q++;
			if(q>=co[ijk]) {
				q=0;
				do {
					if(!next_block()) return false;
				} while(co[ijk]==0);
			}
			return true;
		}
	private:
		int ijk_start = 0, ijk_stop = 0;

		/** Updates the internal variables to find the next
		 * computational block with any particles.
		 * \return True if another block is found, false if there are
		 * no more blocks. */
		inline bool next_block() {
			ijk++;
			if (ijk >= ijk_stop) return false;
			i++;
			if(i==nx) {
				i=0;j++;
				if(j==ny) {
					j=0;k++;
				}
			}
			return true;
		}
};

/** \brief Class for looping over a subset of particles in a container.
 *
 * This class can loop over a subset of particles in a certain geometrical
 * region within the container. The class can be set up to loop over a
 * rectangular box or sphere. It can also rectangular group of internal
 * computational blocks. */
class c_loop_subset : public c_loop_base {
	public:
		/** The current mode of operation, determining whether tests
		 * should be applied to particles to ensure they are within a
		 * certain geometrical object. */
		c_loop_subset_mode mode;
		/** The constructor copies several necessary constants from the
		 * base container class.
		 * \param[in] con the container class to use. */
		template<class c_class>
		c_loop_subset(c_class &con) : c_loop_base(con), ax(con.ax), ay(con.ay), az(con.az),
			sx(con.bx-ax), sy(con.by-ay), sz(con.bz-az), xsp(con.xsp), ysp(con.ysp), zsp(con.zsp),
			xperiodic(con.xperiodic), yperiodic(con.yperiodic), zperiodic(con.zperiodic) {}
		void setup_sphere(double vx,double vy,double vz,double r,bool bounds_test=true);
		void setup_box(double xmin,double xmax,double ymin,double ymax,double zmin,double zmax,bool bounds_test=true);
		void setup_intbox(int ai_,int bi_,int aj_,int bj_,int ak_,int bk_);
		bool start();
		/** Finds the next particle to test.
		 * \return True if there is another particle, false if no more
		 * particles are available. */
		inline bool inc() {
			do {
				q++;
				while(q>=co[ijk]) {q=0;if(!next_block()) return false;}
			} while(mode!=no_check&&out_of_bounds());
			return true;
		}
	private:
		const double ax,ay,az,sx,sy,sz,xsp,ysp,zsp;
		double px,py,pz,apx,apy,apz;
		double v0,v1,v2,v3,v4,v5;
		int ai,bi,aj,bj,ak,bk;
		int ci,cj,ck,di,dj,dk,inc1,inc2;
		const bool xperiodic,yperiodic,zperiodic;
		inline int step_mod(int a,int b) const {return a>=0?a%b:b-1-(b-1-a)%b;}
		inline int step_div(int a,int b) const {return a>=0?a/b:-1+(a+1)/b;}
		inline int step_int(double a) const {return a<0?int(a)-1:int(a);}
		void setup_common();
		bool next_block();
		bool out_of_bounds();
};

/** \brief Class for looping over all of the particles specified in a
 * pre-assembled particle_order class.
 *
 * The particle_order class can be used to create a specific order of particles
 * within the container. This class can then loop over these particles in this
 * order. The class is particularly useful in cases where the ordering of the
 * output must match the ordering of particles as they were inserted into the
 * container. */
class c_loop_order : public c_loop_base {
	public:
		/** A reference to the ordering class to use. */
		particle_order &vo;
		/** A pointer to the current position in the ordering class. */
		int *cp;
		/** A pointer to the end position in the ordering class. */
		int *op;
		/** The constructor copies several necessary constants from the
		 * base class, and sets up a reference to the ordering class to
		 * use.
		 * \param[in] con the container class to use.
		 * \param[in] vo_ the ordering class to use. */
		template<class c_class>
		c_loop_order(c_class &con,particle_order &vo_)
		: c_loop_base(con), vo(vo_), nx(con.nx), nxy(con.nxy) {}
		/** Sets the class to consider the first particle.
		 * \return True if there is any particle to consider, false
		 * otherwise. */
		inline bool start() {
			cp=vo.o;op=vo.op;
			if(cp!=op) {
				ijk=*(cp++);decode();
				q=*(cp++);
				return true;
			} else return false;
		}
		/** Finds the next particle to test.
		 * \return True if there is another particle, false if no more
		 * particles are available. */
		inline bool inc() {
			if(cp==op) return false;
			ijk=*(cp++);decode();
			q=*(cp++);
			return true;
		}
	private:
		/** The number of computational blocks in the x direction. */
		const int nx;
		/** The number of computational blocks in a z-slice. */
		const int nxy;
		/** Takes the current block index and computes indices in the
		 * x, y, and z directions. */
		inline void decode() {
			k=ijk/nxy;
			int ijkt=ijk-nxy*k;
			j=ijkt/nx;
			i=ijkt-j*nx;
		}
};

/** \brief A class for looping over all particles in a container_periodic or
 * container_periodic_poly class.
 *
 * Since the container_periodic and container_periodic_poly classes have a
 * fundamentally different memory organization, the regular loop classes cannot
 * be used with them. */
class c_loop_all_periodic : public c_loop_base {
	public:
		/** The constructor copies several necessary constants from the
		 * base periodic container class.
		 * \param[in] con the periodic container class to use. */
		template<class c_class>
		c_loop_all_periodic(c_class &con) : c_loop_base(con), ey(con.ey), ez(con.ez), wy(con.wy), wz(con.wz),
			ijk0(nx*(ey+con.oy*ez)), inc2(2*nx*con.ey+1) {}
		/** Sets the class to consider the first particle.
		 * \return True if there is any particle to consider, false
		 * otherwise. */
		inline bool start() {
			i=0;
			j=ey;
			k=ez;
			ijk=ijk0;
			q=0;
			while(co[ijk]==0) if(!next_block()) return false;
			return true;
		}
		/** Finds the next particle to test.
		 * \return True if there is another particle, false if no more
		 * particles are available. */
		inline bool inc() {
			q++;
			if(q>=co[ijk]) {
				q=0;
				do {
					if(!next_block()) return false;
				} while(co[ijk]==0);
			}
			return true;
		}
	private:
		/** The lower y index (inclusive) of the primary domain within
		 * the block structure. */
		int ey;
		/** The lower y index (inclusive) of the primary domain within
		 * the block structure. */
		int ez;
		/** The upper y index (exclusive) of the primary domain within
		 * the block structure. */
		int wy;
		/** The upper z index (exclusive) of the primary domain within
		 * the block structure. */
		int wz;
		/** The index of the (0,0,0) block within the block structure.
		 */
		int ijk0;
		/** A value to increase ijk by when the z index is increased.
		 */
		int inc2;
		/** Updates the internal variables to find the next
		 * computational block with any particles.
		 * \return True if another block is found, false if there are
		 * no more blocks. */
		inline bool next_block() {
			i++;
			if(i==nx) {
				i=0;j++;
				if(j==wy) {
					j=ey;k++;
					if(k==wz) return false;
					ijk+=inc2;
				} else ijk++;
			} else ijk++;
			return true;
		}
};

/** \brief Class for looping over all of the particles specified in a
 * pre-assembled particle_order class, for use with container_periodic classes.
 *
 * The particle_order class can be used to create a specific order of particles
 * within the container. This class can then loop over these particles in this
 * order. The class is particularly useful in cases where the ordering of the
 * output must match the ordering of particles as they were inserted into the
 * container. */
class c_loop_order_periodic : public c_loop_base {
	public:
		/** A reference to the ordering class to use. */
		particle_order &vo;
		/** A pointer to the current position in the ordering class. */
		int *cp;
		/** A pointer to the end position in the ordering class. */
		int *op;
		/** The constructor copies several necessary constants from the
		 * base class, and sets up a reference to the ordering class to
		 * use.
		 * \param[in] con the container class to use.
		 * \param[in] vo_ the ordering class to use. */
		template<class c_class>
		c_loop_order_periodic(c_class &con,particle_order &vo_)
		: c_loop_base(con), vo(vo_), nx(con.nx), oxy(con.nx*con.oy) {}
		/** Sets the class to consider the first particle.
		 * \return True if there is any particle to consider, false
		 * otherwise. */
		inline bool start() {
			cp=vo.o;op=vo.op;
			if(cp!=op) {
				ijk=*(cp++);decode();
				q=*(cp++);
				return true;
			} else return false;
		}
		/** Finds the next particle to test.
		 * \return True if there is another particle, false if no more
		 * particles are available. */
		inline bool inc() {
			if(cp==op) return false;
			ijk=*(cp++);decode();
			q=*(cp++);
			return true;
		}
	private:
		/** The number of computational blocks in the x direction. */
		const int nx;
		/** The number of computational blocks in a z-slice. */
		const int oxy;
		/** Takes the current block index and computes indices in the
		 * x, y, and z directions. */
		inline void decode() {
			k=ijk/oxy;
			int ijkt=ijk-oxy*k;
			j=ijkt/nx;
			i=ijkt-j*nx;
		}
};

}

#endif
