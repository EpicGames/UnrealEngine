// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file cell.cc
 * \brief Function implementations for the voronoicell and related classes. */

#include <cmath>
#include <cstring>

#include "config.hh"
#include "common.hh"
#include "cell.hh"
#include "HAL/PlatformMisc.h"


PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS

namespace voro {

/** Constructs a Voronoi cell and sets up the initial memory. */
voronoicell_base::voronoicell_base() :
	current_vertices(init_vertices), current_vertex_order(init_vertex_order),
	current_delete_size(init_delete_size), current_delete2_size(init_delete2_size),
	ed(new int*[current_vertices]), nu(new int[current_vertices]),
	pts(new double[3*current_vertices]), mem(new int[current_vertex_order]),
	mec(new int[current_vertex_order]), mep(new int*[current_vertex_order]),
	ds(new int[current_delete_size]), stacke(ds+current_delete_size),
	ds2(new int[current_delete2_size]), stacke2(ds2+current_delete_size),
	current_marginal(init_marginal), marg(new int[current_marginal]) {
	int i;
	for(i=0;i<3;i++) {
		mem[i]=init_n_vertices;mec[i]=0;
		mep[i]=new int[init_n_vertices*((i<<1)+1)];
	}
	mem[3]=init_3_vertices;mec[3]=0;
	mep[3]=new int[init_3_vertices*7];
	for(i=4;i<current_vertex_order;i++) {
		mem[i]=init_n_vertices;mec[i]=0;
		mep[i]=new int[init_n_vertices*((i<<1)+1)];
	}
}

/** The voronoicell destructor deallocates all the dynamic memory. */
voronoicell_base::~voronoicell_base() {
	for(int i=current_vertex_order-1;i>=0;i--) if(mem[i]>0) delete [] mep[i];
	delete [] marg;
	delete [] ds2;delete [] ds;
	delete [] mep;delete [] mec;
	delete [] mem;delete [] pts;
	delete [] nu;delete [] ed;
}

/** Ensures that enough memory is allocated prior to carrying out a copy.
 * \param[in] vc a reference to the specialized version of the calling class.
 * \param[in] vb a pointered to the class to be copied. */
template<class vc_class>
void voronoicell_base::check_memory_for_copy(vc_class &vc,voronoicell_base* vb) {
	while(current_vertex_order<vb->current_vertex_order) add_memory_vorder(vc);
	for(int i=0;i<current_vertex_order;i++) while(mem[i]<vb->mec[i]) add_memory(vc,i,ds2);
	while(current_vertices<vb->p) add_memory_vertices(vc);
}

/** Copies the vertex and edge information from another class. The routine
 * assumes that enough memory is available for the copy.
 * \param[in] vb a pointer to the class to copy. */
void voronoicell_base::copy(voronoicell_base* vb) {
	int i,j;
	p=vb->p;up=0;
	for(i=0;i<current_vertex_order;i++) {
		mec[i]=vb->mec[i];
		for(j=0;j<mec[i]*(2*i+1);j++) mep[i][j]=vb->mep[i][j];
		for(j=0;j<mec[i]*(2*i+1);j+=2*i+1) ed[mep[i][j+2*i]]=mep[i]+j;
	}
	for(i=0;i<p;i++) nu[i]=vb->nu[i];
	for(i=0;i<3*p;i++) pts[i]=vb->pts[i];
}

/** Copies the information from another voronoicell class into this
 * class, extending memory allocation if necessary.
 * \param[in] c the class to copy. */
void voronoicell_neighbor::operator=(voronoicell &c) {
	voronoicell_base *vb=((voronoicell_base*) &c);
	check_memory_for_copy(*this,vb);copy(vb);
	int i,j;
	for(i=0;i<c.current_vertex_order;i++) {
		for(j=0;j<c.mec[i]*i;j++) mne[i][j]=0;
		for(j=0;j<c.mec[i];j++) ne[c.mep[i][(2*i+1)*j+2*i]]=mne[i]+(j*i);
	}
}

/** Copies the information from another voronoicell_neighbor class into this
 * class, extending memory allocation if necessary.
 * \param[in] c the class to copy. */
void voronoicell_neighbor::operator=(voronoicell_neighbor &c) {
	voronoicell_base *vb=((voronoicell_base*) &c);
	check_memory_for_copy(*this,vb);copy(vb);
	int i,j;
	for(i=0;i<c.current_vertex_order;i++) {
		for(j=0;j<c.mec[i]*i;j++) mne[i][j]=c.mne[i][j];
		for(j=0;j<c.mec[i];j++) ne[c.mep[i][(2*i+1)*j+2*i]]=mne[i]+(j*i);
	}
}

/** Translates the vertices of the Voronoi cell by a given vector.
 * \param[in] (x,y,z) the coordinates of the vector. */
void voronoicell_base::translate(double x,double y,double z) {
	x*=2;y*=2;z*=2;
	double *ptsp=pts;
	while(ptsp<pts+3*p) {
		*(ptsp++)=x;*(ptsp++)=y;*(ptsp++)=z;
	}
}

/** Increases the memory storage for a particular vertex order, by increasing
 * the size of the of the corresponding mep array. If the arrays already exist,
 * their size is doubled; if they don't exist, then new ones of size
 * init_n_vertices are allocated. The routine also ensures that the pointers in
 * the ed array are updated, by making use of the back pointers. For the cases
 * where the back pointer has been temporarily overwritten in the marginal
 * vertex code, the auxiliary delete stack is scanned to find out how to update
 * the ed value. If the template has been instantiated with the neighbor
 * tracking turned on, then the routine also reallocates the corresponding mne
 * array.
 * \param[in] i the order of the vertex memory to be increased. */
template<class vc_class>
void voronoicell_base::add_memory(vc_class &vc,int i,int *stackp2) {
	int s=(i<<1)+1;
	if(mem[i]==0) {
		vc.n_allocate(i,init_n_vertices);
		mep[i]=new int[init_n_vertices*s];
		mem[i]=init_n_vertices;
#if VOROPP_VERBOSE >=2
		fprintf(stderr,"Order %d vertex memory created\n",i);
#endif
	} else {
		int j=0,k,*l;
		mem[i]<<=1;
		if(mem[i]>max_n_vertices) voro_fatal_error("Point memory allocation exceeded absolute maximum",VOROPP_MEMORY_ERROR);
#if VOROPP_VERBOSE >=2
		fprintf(stderr,"Order %d vertex memory scaled up to %d\n",i,mem[i]);
#endif
		check(mem[i] > 1 && s > 1);
		l=new int[s*mem[i]];
		int m=0;
		vc.n_allocate_aux1(i);
		while(j<s*mec[i]) {
			k=mep[i][j+(i<<1)];
			if(k>=0) {
				ed[k]=l+j;
				vc.n_set_to_aux1_offset(k,m);
			} else {
				int *dsp;
				for(dsp=ds2;dsp<stackp2;dsp++) {
					if(ed[*dsp]==mep[i]+j) {
						ed[*dsp]=l+j;
						vc.n_set_to_aux1_offset(*dsp,m);
						break;
					}
				}
				if(dsp==stackp2) voro_fatal_error("Couldn't relocate dangling pointer",VOROPP_INTERNAL_ERROR);
#if VOROPP_VERBOSE >=3
				fputs("Relocated dangling pointer",stderr);
#endif
			}
			for(k=0;k<s;k++,j++) l[j]=mep[i][j];
			for(k=0;k<i;k++,m++) vc.n_copy_to_aux1(i,m);
		}
		delete [] mep[i];
		mep[i]=l;
		vc.n_switch_to_aux1(i);
	}
}

/** Doubles the maximum number of vertices allowed, by reallocating the ed, nu,
 * and pts arrays. If the allocation exceeds the absolute maximum set in
 * max_vertices, then the routine exits with a fatal error. If the template has
 * been instantiated with the neighbor tracking turned on, then the routine
 * also reallocates the ne array. */
template<class vc_class>
void voronoicell_base::add_memory_vertices(vc_class &vc) {
	int i=(current_vertices<<1),j,**pp,*pnu;
	if(i>max_vertices) voro_fatal_error("Vertex memory allocation exceeded absolute maximum",VOROPP_MEMORY_ERROR);
#if VOROPP_VERBOSE >=2
	fprintf(stderr,"Vertex memory scaled up to %d\n",i);
#endif
	double *ppts;
	pp=new int*[i];
	for(j=0;j<current_vertices;j++) pp[j]=ed[j];
	delete [] ed;ed=pp;
	vc.n_add_memory_vertices(i);
	pnu=new int[i];
	for(j=0;j<current_vertices;j++) pnu[j]=nu[j];
	delete [] nu;nu=pnu;
	ppts=new double[3*i];
	for(j=0;j<3*current_vertices;j++) ppts[j]=pts[j];
	delete [] pts;pts=ppts;
	current_vertices=i;
}

/** Doubles the maximum allowed vertex order, by reallocating mem, mep, and mec
 * arrays. If the allocation exceeds the absolute maximum set in
 * max_vertex_order, then the routine causes a fatal error. If the template has
 * been instantiated with the neighbor tracking turned on, then the routine
 * also reallocates the mne array. */
template<class vc_class>
void voronoicell_base::add_memory_vorder(vc_class &vc) {
	int i=(current_vertex_order<<1),j,*p1,**p2;
	if(i>max_vertex_order) voro_fatal_error("Vertex order memory allocation exceeded absolute maximum",VOROPP_MEMORY_ERROR);
#if VOROPP_VERBOSE >=2
	fprintf(stderr,"Vertex order memory scaled up to %d\n",i);
#endif
	p1=new int[i];
	for(j=0;j<current_vertex_order;j++) p1[j]=mem[j];while(j<i) p1[j++]=0;
	delete [] mem;mem=p1;
	p2=new int*[i];
	for(j=0;j<current_vertex_order;j++) p2[j]=mep[j];
	delete [] mep;mep=p2;
	p1=new int[i];
	for(j=0;j<current_vertex_order;j++) p1[j]=mec[j];while(j<i) p1[j++]=0;
	delete [] mec;mec=p1;
	vc.n_add_memory_vorder(i);
	current_vertex_order=i;
}

/** Doubles the size allocation of the main delete stack. If the allocation
 * exceeds the absolute maximum set in max_delete_size, then routine causes a
 * fatal error. */
void voronoicell_base::add_memory_ds(int *&stackp) {
	current_delete_size<<=1;
	if(current_delete_size>max_delete_size) voro_fatal_error("Delete stack 1 memory allocation exceeded absolute maximum",VOROPP_MEMORY_ERROR);
#if VOROPP_VERBOSE >=2
	fprintf(stderr,"Delete stack 1 memory scaled up to %d\n",current_delete_size);
#endif
	int *dsn=new int[current_delete_size],*dsnp=dsn,*dsp=ds;
	while(dsp<stackp) *(dsnp++)=*(dsp++);
	delete [] ds;ds=dsn;stackp=dsnp;
	stacke=ds+current_delete_size;
}

/** Doubles the size allocation of the auxiliary delete stack. If the
 * allocation exceeds the absolute maximum set in max_delete2_size, then the
 * routine causes a fatal error. */
void voronoicell_base::add_memory_ds2(int *&stackp2) {
	current_delete2_size<<=1;
	if(current_delete2_size>max_delete2_size) voro_fatal_error("Delete stack 2 memory allocation exceeded absolute maximum",VOROPP_MEMORY_ERROR);
#if VOROPP_VERBOSE >=2
	fprintf(stderr,"Delete stack 2 memory scaled up to %d\n",current_delete2_size);
#endif
	int *dsn=new int[current_delete2_size],*dsnp=dsn,*dsp=ds2;
	while(dsp<stackp2) *(dsnp++)=*(dsp++);
	delete [] ds2;ds2=dsn;stackp2=dsnp;
	stacke2=ds2+current_delete2_size;
}

/** Initializes a Voronoi cell as a rectangular box with the given dimensions.
 * \param[in] (xmin,xmax) the minimum and maximum x coordinates.
 * \param[in] (ymin,ymax) the minimum and maximum y coordinates.
 * \param[in] (zmin,zmax) the minimum and maximum z coordinates. */
void voronoicell_base::init_base(double xmin,double xmax,double ymin,double ymax,double zmin,double zmax) {
	for(int i=0;i<current_vertex_order;i++) mec[i]=0;up=0;
	mec[3]=p=8;xmin*=2;xmax*=2;ymin*=2;ymax*=2;zmin*=2;zmax*=2;
	*pts=xmin;pts[1]=ymin;pts[2]=zmin;
	pts[3]=xmax;pts[4]=ymin;pts[5]=zmin;
	pts[6]=xmin;pts[7]=ymax;pts[8]=zmin;
	pts[9]=xmax;pts[10]=ymax;pts[11]=zmin;
	pts[12]=xmin;pts[13]=ymin;pts[14]=zmax;
	pts[15]=xmax;pts[16]=ymin;pts[17]=zmax;
	pts[18]=xmin;pts[19]=ymax;pts[20]=zmax;
	pts[21]=xmax;pts[22]=ymax;pts[23]=zmax;
	int *q=mep[3];
	*q=1;q[1]=4;q[2]=2;q[3]=2;q[4]=1;q[5]=0;q[6]=0;
	q[7]=3;q[8]=5;q[9]=0;q[10]=2;q[11]=1;q[12]=0;q[13]=1;
	q[14]=0;q[15]=6;q[16]=3;q[17]=2;q[18]=1;q[19]=0;q[20]=2;
	q[21]=2;q[22]=7;q[23]=1;q[24]=2;q[25]=1;q[26]=0;q[27]=3;
	q[28]=6;q[29]=0;q[30]=5;q[31]=2;q[32]=1;q[33]=0;q[34]=4;
	q[35]=4;q[36]=1;q[37]=7;q[38]=2;q[39]=1;q[40]=0;q[41]=5;
	q[42]=7;q[43]=2;q[44]=4;q[45]=2;q[46]=1;q[47]=0;q[48]=6;
	q[49]=5;q[50]=3;q[51]=6;q[52]=2;q[53]=1;q[54]=0;q[55]=7;
	*ed=q;ed[1]=q+7;ed[2]=q+14;ed[3]=q+21;
	ed[4]=q+28;ed[5]=q+35;ed[6]=q+42;ed[7]=q+49;
	*nu=nu[1]=nu[2]=nu[3]=nu[4]=nu[5]=nu[6]=nu[7]=3;
}

/** Initializes a Voronoi cell as a regular octahedron.
 * \param[in] l The distance from the octahedron center to a vertex. Six
 *              vertices are initialized at (-l,0,0), (l,0,0), (0,-l,0),
 *              (0,l,0), (0,0,-l), and (0,0,l). */
void voronoicell_base::init_octahedron_base(double l) {
	for(int i=0;i<current_vertex_order;i++) mec[i]=0;up=0;
	mec[4]=p=6;l*=2;
	*pts=-l;pts[1]=0;pts[2]=0;
	pts[3]=l;pts[4]=0;pts[5]=0;
	pts[6]=0;pts[7]=-l;pts[8]=0;
	pts[9]=0;pts[10]=l;pts[11]=0;
	pts[12]=0;pts[13]=0;pts[14]=-l;
	pts[15]=0;pts[16]=0;pts[17]=l;
	int *q=mep[4];
	*q=2;q[1]=5;q[2]=3;q[3]=4;q[4]=0;q[5]=0;q[6]=0;q[7]=0;q[8]=0;
	q[9]=2;q[10]=4;q[11]=3;q[12]=5;q[13]=2;q[14]=2;q[15]=2;q[16]=2;q[17]=1;
	q[18]=0;q[19]=4;q[20]=1;q[21]=5;q[22]=0;q[23]=3;q[24]=0;q[25]=1;q[26]=2;
	q[27]=0;q[28]=5;q[29]=1;q[30]=4;q[31]=2;q[32]=3;q[33]=2;q[34]=1;q[35]=3;
	q[36]=0;q[37]=3;q[38]=1;q[39]=2;q[40]=3;q[41]=3;q[42]=1;q[43]=1;q[44]=4;
	q[45]=0;q[46]=2;q[47]=1;q[48]=3;q[49]=1;q[50]=3;q[51]=3;q[52]=1;q[53]=5;
	*ed=q;ed[1]=q+9;ed[2]=q+18;ed[3]=q+27;ed[4]=q+36;ed[5]=q+45;
	*nu=nu[1]=nu[2]=nu[3]=nu[4]=nu[5]=4;
}

/** Initializes a Voronoi cell as a tetrahedron. It assumes that the normal to
 * the face for the first three vertices points inside.
 * \param (x0,y0,z0) a position vector for the first vertex.
 * \param (x1,y1,z1) a position vector for the second vertex.
 * \param (x2,y2,z2) a position vector for the third vertex.
 * \param (x3,y3,z3) a position vector for the fourth vertex. */
void voronoicell_base::init_tetrahedron_base(double x0,double y0,double z0,double x1,double y1,double z1,double x2,double y2,double z2,double x3,double y3,double z3) {
	for(int i=0;i<current_vertex_order;i++) mec[i]=0;up=0;
	mec[3]=p=4;
	*pts=x0*2;pts[1]=y0*2;pts[2]=z0*2;
	pts[3]=x1*2;pts[4]=y1*2;pts[5]=z1*2;
	pts[6]=x2*2;pts[7]=y2*2;pts[8]=z2*2;
	pts[9]=x3*2;pts[10]=y3*2;pts[11]=z3*2;
	int *q=mep[3];
	*q=1;q[1]=3;q[2]=2;q[3]=0;q[4]=0;q[5]=0;q[6]=0;
	q[7]=0;q[8]=2;q[9]=3;q[10]=0;q[11]=2;q[12]=1;q[13]=1;
	q[14]=0;q[15]=3;q[16]=1;q[17]=2;q[18]=2;q[19]=1;q[20]=2;
	q[21]=0;q[22]=1;q[23]=2;q[24]=1;q[25]=2;q[26]=1;q[27]=3;
	*ed=q;ed[1]=q+7;ed[2]=q+14;ed[3]=q+21;
	*nu=nu[1]=nu[2]=nu[3]=3;
}

/** Checks that the relational table of the Voronoi cell is accurate, and
 * prints out any errors. This algorithm is O(p), so running it every time the
 * plane routine is called will result in a significant slowdown. */
void voronoicell_base::check_relations() {
	int i,j;
	for(i=0;i<p;i++) for(j=0;j<nu[i];j++) if(ed[ed[i][j]][ed[i][nu[i]+j]]!=i)
		printf("Relational error at point %d, edge %d.\n",i,j);
}

/** This routine checks for any two vertices that are connected by more than
 * one edge. The plane algorithm is designed so that this should not happen, so
 * any occurrences are most likely errors. Note that the routine is O(p), so
 * running it every time the plane routine is called will result in a
 * significant slowdown. */
void voronoicell_base::check_duplicates() {
	int i,j,k;
	for(i=0;i<p;i++) for(j=1;j<nu[i];j++) for(k=0;k<j;k++) if(ed[i][j]==ed[i][k])
		printf("Duplicate edges: (%d,%d) and (%d,%d) [%d]\n",i,j,i,k,ed[i][j]);
}

/** Constructs the relational table if the edges have been specified. */
void voronoicell_base::construct_relations() {
	int i,j,k,l;
	for(i=0;i<p;i++) for(j=0;j<nu[i];j++) {
		k=ed[i][j];
		l=0;
		while(ed[k][l]!=i) {
			l++;
			if(l==nu[k]) voro_fatal_error("Relation table construction failed",VOROPP_INTERNAL_ERROR);
		}
		ed[i][nu[i]+j]=l;
	}
}

/** Starting from a point within the current cutting plane, this routine attempts
 * to find an edge to a point outside the cutting plane. This prevents the plane
 * routine from .
 * \param[in] vc a reference to the specialized version of the calling class.
 * \param[in,out] upRef reference to the up member used to initialize search, 
				  just to make it extra clear that this function changes this value */
template<class vc_class>
inline bool voronoicell_base::search_for_outside_edge(vc_class &vc,int &upRef) {
	int i,lp,lw,*j(ds2),*stackp2(ds2);
	double l;
	*(stackp2++)=upRef;
	while(j<stackp2) {
		upRef=*(j++);
		for(i=0;i<nu[upRef];i++) {
			lp=ed[upRef][i];
			lw=m_test(lp,l);
			if(lw==-1) return true;
			else if(lw==0) add_to_stack(vc,lp,stackp2);
		}
	}
	return false;
}

/** Adds a point to the auxiliary delete stack if it is not already there.
 * \param[in] vc a reference to the specialized version of the calling class.
 * \param[in] lp the index of the point to add.
 * \param[in,out] stackp2 a pointer to the end of the stack entries. */
template<class vc_class>
inline void voronoicell_base::add_to_stack(vc_class &vc,int lp,int *&stackp2) {
	for(int *k(ds2);k<stackp2;k++) if(*k==lp) return;
	if(stackp2==stacke2) add_memory_ds2(stackp2);
	*(stackp2++)=lp;
}

/** Cuts the Voronoi cell by a particle whose center is at a separation of
 * (x,y,z) from the cell center. The value of rsq should be initially set to
 * \f$x^2+y^2+z^2\f$.
 * \param[in] vc a reference to the specialized version of the calling class.
 * \param[in] (x,y,z) the normal vector to the plane.
 * \param[in] rsq the distance along this vector of the plane.
 * \param[in] p_id the plane ID (for neighbor tracking only).
 * \return False if the plane cut deleted the cell entirely, true otherwise. */
template<class vc_class>
bool voronoicell_base::nplane(vc_class &vc,double x,double y,double z,double rsq,int p_id) {
	int count=0,i,j,k,lp=up,cp,qp,rp,*stackp(ds),*stackp2(ds2),*dsp;
	int us=0,ls=0,qs,iqs,cs,uw,qw,lw;
	int *edp,*edd;
	double u,l=0,r,q;bool complicated_setup=false,new_double_edge=false,double_edge=false;

	// Initialize the safe testing routine
	n_marg=0;px=x;py=y;pz=z;prsq=rsq;

	// Test approximately sqrt(n)/4 points for their proximity to the plane
	// and keep the one which is closest
	uw=m_test(up,u);

	// Starting from an initial guess, we now move from vertex to vertex,
	// to try and find an edge which intersects the cutting plane,
	// or a vertex which is on the plane
	bool failed = false;
	if(uw==1) {

		// The test point is inside the cutting plane.
		us=0;
		do {
			lp=ed[up][us];
			lw=m_test(lp,l);
			if(l<u) break;
			us++;
		} while (us<nu[up]);

		if(us==nu[up]) {
			return false;
		}

		ls=ed[up][nu[up]+us];
		while(lw==1) {
			if (++count >= p) {
				failed = true;
				break;
			}
			u=l;up=lp;
			for(us=0;us<ls;us++) {
				lp=ed[up][us];
				lw=m_test(lp,l);
				if(l<u) break;
			}
			if(us==ls) {
				us++;
				while(us<nu[up]) {
					lp=ed[up][us];
					lw=m_test(lp,l);
					if(l<u) break;
					us++;
				}
				if(us==nu[up]) {
					return false;
				}
			}
			ls=ed[up][nu[up]+us];
		}

		// If the last point in the iteration is within the
		// plane, we need to do the complicated setup
		// routine. Otherwise, we use the regular iteration.
		if (!failed) {
			if (lw == 0) {
				up = lp;
				complicated_setup = true;
			}
			else complicated_setup = false;
		}
	} else if(uw==-1) {
		us=0;
		do {
			qp=ed[up][us];
			qw=m_test(qp,q);
			if(u<q) break;
			us++;
		} while (us<nu[up]);
		if(us==nu[up]) return true;

		while(qw==-1) {
			qs=ed[up][nu[up]+us];
			if (++count >= p) {
				failed = true;
				break;
			}
			u=q;up=qp;
			for(us=0;us<qs;us++) {
				qp=ed[up][us];
				qw=m_test(qp,q);
				if(u<q) break;
			}
			if(us==qs) {
				us++;
				while(us<nu[up]) {
					qp=ed[up][us];
					qw=m_test(qp,q);
					if(u<q) break;
					us++;
				}
				if(us==nu[up]) return true;
			}
		}
		if (!failed) {
			if (qw == 1) {
				lp = up; ls = us; l = u;
				up = qp; us = ed[lp][nu[lp] + ls]; u = q;
				complicated_setup = false;
			}
			else {
				up = qp;
				complicated_setup = true;
			}
		}
	} else {

		// Our original test point was on the plane, so we
		// automatically head for the complicated setup
		// routine
		complicated_setup=true;
	}
	
	if (failed) {
		// This routine is a fall-back, in case floating point errors
		// cause the usual search routine to fail. In the fall-back
		// routine, we just test every edge to find one straddling
		// the plane.
#if VOROPP_VERBOSE >=1
		fputs("Bailed out of convex calculation\n",stderr);
#endif
		qw=1;lw=0;
		for(qp=0;qp<p;qp++) {
			qw=m_test(qp,q);
			if(qw==1) {

				// The point is inside the cutting space. Now
				// see if we can find a neighbor which isn't.
				for(us=0;us<nu[qp];us++) {
					lp=ed[qp][us];
					if(lp<qp) {
						lw=m_test(lp,l);
						if(lw!=1) break;
					}
				}
				if(us<nu[qp]) {
					up=qp;
					if(lw==0) {
						complicated_setup=true;
					} else {
						complicated_setup=false;
						u=q;
						ls=ed[up][nu[up]+us];
					}
					break;
				}
			} else if(qw==-1) {

				// The point is outside the cutting space. See
				// if we can find a neighbor which isn't.
				for(ls=0;ls<nu[qp];ls++) {
					up=ed[qp][ls];
					if(up<qp) {
						uw=m_test(up,u);
						if(uw!=-1) break;
					}
				}
				if(ls<nu[qp]) {
					if(uw==0) {
						up=qp;
						complicated_setup=true;
					} else {
						complicated_setup=false;
						lp=qp;l=q;
						us=ed[lp][nu[lp]+ls];
					}
					break;
				}
			} else {

				// The point is in the plane, so we just
				// proceed with the complicated setup routine
				up=qp;
				complicated_setup=true;
				break;
			}
		}
		if(qp==p) return qw==-1?true:false;
	}

	// We're about to add the first point of the new facet. In either
	// routine, we have to add a point, so first check there's space for
	// it.
	if(p==current_vertices) add_memory_vertices(vc);

	if(complicated_setup) {

		// We want to be strict about reaching the conclusion that the
		// cell is entirely within the cutting plane. It's not enough
		// to find a vertex that has edges which are all inside or on
		// the plane. If the vertex has neighbors that are also on the
		// plane, we should check those too.
		if(!search_for_outside_edge(vc,up)) return false;

		// The search algorithm found a point which is on the cutting
		// plane. We leave that point in place, and create a new one at
		// the same location.
		pts[3*p]=pts[3*up];
		pts[3*p+1]=pts[3*up+1];
		pts[3*p+2]=pts[3*up+2];

		// Search for a collection of edges of the test vertex which
		// are outside of the cutting space. Begin by testing the
		// zeroth edge.
		i=0;
		lp=*ed[up];
		lw=m_test(lp,l);
		if(lw!=-1) {

			// The first edge is either inside the cutting space,
			// or lies within the cutting plane. Test the edges
			// sequentially until we find one that is outside.
			rp=lw;
			do {
				i++;

				// If we reached the last edge with no luck
				// then all of the vertices are inside
				// or on the plane, so the cell is completely
				// deleted
				if(i==nu[up]) return false;
				lp=ed[up][i];
				lw=m_test(lp,l);
			} while (lw!=-1);
			j=i+1;

			// We found an edge outside the cutting space. Keep
			// moving through these edges until we find one that's
			// inside or on the plane.
			while(j<nu[up]) {
				lp=ed[up][j];
				lw=m_test(lp,l);
				if(lw!=-1) break;
				j++;
			}

			// Compute the number of edges for the new vertex. In
			// general it will be the number of outside edges
			// found, plus two. But we need to recognize the
			// special case when all but one edge is outside, and
			// the remaining one is on the plane. For that case we
			// have to reduce the edge count by one to prevent
			// doubling up.
			if(j==nu[up]&&i==1&&rp==0) {
				nu[p]=nu[up];
				double_edge=true;
			} else nu[p]=j-i+2;
			k=1;

			// Add memory for the new vertex if needed, and
			// initialize
			while (nu[p]>=current_vertex_order) add_memory_vorder(vc);
			if(mec[nu[p]]==mem[nu[p]]) add_memory(vc,nu[p],stackp2);
			vc.n_set_pointer(p,nu[p]);
			ed[p]=mep[nu[p]]+((nu[p]<<1)+1)*mec[nu[p]]++;
			ed[p][nu[p]<<1]=p;

			// Copy the edges of the original vertex into the new
			// one. Delete the edges of the original vertex, and
			// update the relational table.
			us=cycle_down(i,up);
			while(i<j) {
				qp=ed[up][i];
				qs=ed[up][nu[up]+i];
				vc.n_copy(p,k,up,i);
				ed[p][k]=qp;
				ed[p][nu[p]+k]=qs;
				ed[qp][qs]=p;
				ed[qp][nu[qp]+qs]=k;
				ed[up][i]=-1;
				i++;k++;
			}
			qs=i==nu[up]?0:i;
		} else {

			// In this case, the zeroth edge is outside the cutting
			// plane. Begin by searching backwards from the last
			// edge until we find an edge which isn't outside.
			i=nu[up]-1;
			lp=ed[up][i];
			lw=m_test(lp,l);
			while(lw==-1) {
				i--;

				// If i reaches zero, then we have a point in
				// the plane all of whose edges are outside
				// the cutting space, so we just exit
				if(i==0) return true;
				lp=ed[up][i];
				lw=m_test(lp,l);
			}

			// Now search forwards from zero
			j=1;
			qp=ed[up][j];
			qw=m_test(qp,q);
			while(qw==-1) {
				j++;
				qp=ed[up][j];
				qw=m_test(qp,l);
			}

			// Compute the number of edges for the new vertex. In
			// general it will be the number of outside edges
			// found, plus two. But we need to recognize the
			// special case when all but one edge is outside, and
			// the remaining one is on the plane. For that case we
			// have to reduce the edge count by one to prevent
			// doubling up.
			if(i==j&&qw==0) {
				double_edge=true;
				nu[p]=nu[up];
			} else {
				nu[p]=nu[up]-i+j+1;
			}

			// Add memory to store the vertex if it doesn't exist
			// already
			k=1;
			while(nu[p]>=current_vertex_order) add_memory_vorder(vc);
			if(mec[nu[p]]==mem[nu[p]]) add_memory(vc,nu[p],stackp2);

			// Copy the edges of the original vertex into the new
			// one. Delete the edges of the original vertex, and
			// update the relational table.
			vc.n_set_pointer(p,nu[p]);
			ed[p]=mep[nu[p]]+((nu[p]<<1)+1)*mec[nu[p]]++;
			ed[p][nu[p]<<1]=p;
			us=i++;
			while(i<nu[up]) {
				qp=ed[up][i];
				qs=ed[up][nu[up]+i];
				vc.n_copy(p,k,up,i);
				ed[p][k]=qp;
				ed[p][nu[p]+k]=qs;
				ed[qp][qs]=p;
				ed[qp][nu[qp]+qs]=k;
				ed[up][i]=-1;
				i++;k++;
			}
			i=0;
			while(i<j) {
				qp=ed[up][i];
				qs=ed[up][nu[up]+i];
				vc.n_copy(p,k,up,i);
				ed[p][k]=qp;
				ed[p][nu[p]+k]=qs;
				ed[qp][qs]=p;
				ed[qp][nu[qp]+qs]=k;
				ed[up][i]=-1;
				i++;k++;
			}
			qs=j;
		}
		if(!double_edge) {
			vc.n_copy(p,k,up,qs);
			vc.n_set(p,0,p_id);
		} else vc.n_copy(p,0,up,qs);

		// Add this point to the auxiliary delete stack
		if(stackp2==stacke2) add_memory_ds2(stackp2);
		*(stackp2++)=up;

		// Look at the edges on either side of the group that was
		// detected. We're going to commence facet computation by
		// moving along one of them. We are going to end up coming back
		// along the other one.
		cs=k;
		qp=up;q=u;
		i=ed[up][us];
		us=ed[up][nu[up]+us];
		up=i;
		ed[qp][nu[qp]<<1]=-p;

	} else {

		// The search algorithm found an intersected edge between the
		// points lp and up. Create a new vertex between them which
		// lies on the cutting plane. Since u and l differ by at least
		// the tolerance, this division should never screw up.
		if(stackp==stacke) add_memory_ds(stackp);
		*(stackp++)=up;
		r=u/(u-l);l=1-r;
		pts[3*p]=pts[3*lp]*r+pts[3*up]*l;
		pts[3*p+1]=pts[3*lp+1]*r+pts[3*up+1]*l;
		pts[3*p+2]=pts[3*lp+2]*r+pts[3*up+2]*l;

		// This point will always have three edges. Connect one of them
		// to lp.
		nu[p]=3;
		if(mec[3]==mem[3]) add_memory(vc,3,stackp2);
		vc.n_set_pointer(p,3);
		vc.n_set(p,0,p_id);
		vc.n_copy(p,1,up,us);
		vc.n_copy(p,2,lp,ls);
		ed[p]=mep[3]+7*mec[3]++;
		ed[p][6]=p;
		ed[up][us]=-1;
		ed[lp][ls]=p;
		ed[lp][nu[lp]+ls]=1;
		ed[p][1]=lp;
		ed[p][nu[p]+1]=ls;
		cs=2;

		// Set the direction to move in
		qs=cycle_up(us,up);
		qp=up;q=u;
	}

	// When the code reaches here, we have initialized the first point, and
	// we have a direction for moving it to construct the rest of the facet
	cp=p;rp=p;p++;
	while(qp!=up||qs!=us) {

		// We're currently tracing round an intersected facet. Keep
		// moving around it until we find a point or edge which
		// intersects the plane.
		lp=ed[qp][qs];
		lw=m_test(lp,l);

		if(lw==1) {

			// The point is still in the cutting space. Just add it
			// to the delete stack and keep moving.
			qs=cycle_up(ed[qp][nu[qp]+qs],lp);
			qp=lp;
			q=l;
			if(stackp==stacke) add_memory_ds(stackp);
			*(stackp++)=qp;

		} else if(lw==-1) {

			// The point is outside of the cutting space, so we've
			// found an intersected edge. Introduce a regular point
			// at the point of intersection. Connect it to the
			// point we just tested. Also connect it to the previous
			// new point in the facet we're constructing.
			if(p==current_vertices) add_memory_vertices(vc);
			r=q/(q-l);l=1-r;
			pts[3*p]=pts[3*lp]*r+pts[3*qp]*l;
			pts[3*p+1]=pts[3*lp+1]*r+pts[3*qp+1]*l;
			pts[3*p+2]=pts[3*lp+2]*r+pts[3*qp+2]*l;
			nu[p]=3;
			if(mec[3]==mem[3]) add_memory(vc,3,stackp2);
			ls=ed[qp][qs+nu[qp]];
			vc.n_set_pointer(p,3);
			vc.n_set(p,0,p_id);
			vc.n_copy(p,1,qp,qs);
			vc.n_copy(p,2,lp,ls);
			ed[p]=mep[3]+7*mec[3]++;
			*ed[p]=cp;
			ed[p][1]=lp;
			ed[p][3]=cs;
			ed[p][4]=ls;
			ed[p][6]=p;
			ed[lp][ls]=p;
			ed[lp][nu[lp]+ls]=1;
			ed[cp][cs]=p;
			ed[cp][nu[cp]+cs]=0;
			ed[qp][qs]=-1;
			qs=cycle_up(qs,qp);
			cp=p++;
			cs=2;
		} else {

			// We've found a point which is on the cutting plane.
			// We're going to introduce a new point right here, but
			// first we need to figure out the number of edges it
			// has.
			if(p==current_vertices) add_memory_vertices(vc);

			// If the previous vertex detected a double edge, our
			// new vertex will have one less edge.
			k=double_edge?0:1;
			qs=ed[qp][nu[qp]+qs];
			qp=lp;
			iqs=qs;

			// Start testing the edges of the current point until
			// we find one which isn't outside the cutting space
			do {
				k++;
				qs=cycle_up(qs,qp);
				lp=ed[qp][qs];
				lw=m_test(lp,l);
			} while (lw==-1);

			// Now we need to find out whether this marginal vertex
			// we are on has been visited before, because if that's
			// the case, we need to add vertices to the existing
			// new vertex, rather than creating a fresh one. We also
			// need to figure out whether we're in a case where we
			// might be creating a duplicate edge.
			j=-ed[qp][nu[qp]<<1];
	 		if(qp==up&&qs==us) {

				// If we're heading into the final part of the
				// new facet, then we never worry about the
				// duplicate edge calculation.
				new_double_edge=false;
				if(j>0) k+=nu[j];
			} else {
				if(j>0) {

					// This vertex was visited before, so
					// count those vertices to the ones we
					// already have.
					k+=nu[j];

					// The only time when we might make a
					// duplicate edge is if the point we're
					// going to move to next is also a
					// marginal point, so test for that
					// first.
					if(lw==0) {

						// Now see whether this marginal point
						// has been visited before.
						i=-ed[lp][nu[lp]<<1];
						if(i>0) {

							// Now see if the last edge of that other
							// marginal point actually ends up here.
							if(ed[i][nu[i]-1]==j) {
								new_double_edge=true;
								k-=1;
							} else new_double_edge=false;
						} else {

							// That marginal point hasn't been visited
							// before, so we probably don't have to worry
							// about duplicate edges, except in the
							// case when that's the way into the end
							// of the facet, because that way always creates
							// an edge.
							if(j==rp&&lp==up&&ed[qp][nu[qp]+qs]==us) {
								new_double_edge=true;
								k-=1;
							} else new_double_edge=false;
						}
					} else new_double_edge=false;
				} else {

					// The vertex hasn't been visited
					// before, but let's see if it's
					// marginal
					if(lw==0) {

						// If it is, we need to check
						// for the case that it's a
						// small branch, and that we're
						// heading right back to where
						// we came from
						i=-ed[lp][nu[lp]<<1];
						if(i==cp) {
							new_double_edge=true;
							k-=1;
						} else new_double_edge=false;
					} else new_double_edge=false;
				}
			}

			// k now holds the number of edges of the new vertex
			// we are forming. Add memory for it if it doesn't exist
			// already.
			while(k>=current_vertex_order) add_memory_vorder(vc);
			if(mec[k]==mem[k]) add_memory(vc,k,stackp2);

			// Now create a new vertex with order k, or augment
			// the existing one
			if(j>0) {

				// If we're augmenting a vertex but we don't
				// actually need any more edges, just skip this
				// routine to avoid memory confusion
				if(nu[j]!=k) {
					// Allocate memory and copy the edges
					// of the previous instance into it
					vc.n_set_aux1(k);
					edp=mep[k]+((k<<1)+1)*mec[k]++;
					i=0;
					while(i<nu[j]) {
						vc.n_copy_aux1(j,i);
						edp[i]=ed[j][i];
						edp[k+i]=ed[j][nu[j]+i];
						i++;
					}
					edp[k<<1]=j;

					// Remove the previous instance with
					// fewer vertices from the memory
					// structure
					edd=mep[nu[j]]+((nu[j]<<1)+1)*--mec[nu[j]];
					if(edd!=ed[j]) {
						for(lw=0;lw<=(nu[j]<<1);lw++) ed[j][lw]=edd[lw];
						vc.n_set_aux2_copy(j,nu[j]);
						vc.n_copy_pointer(edd[nu[j]<<1],j);
						ed[edd[nu[j]<<1]]=ed[j];
					}
					vc.n_set_to_aux1(j);
					ed[j]=edp;
				} else i=nu[j];
			} else {

				// Allocate a new vertex of order k
				vc.n_set_pointer(p,k);
				ed[p]=mep[k]+((k<<1)+1)*mec[k]++;
				ed[p][k<<1]=p;
				if(stackp2==stacke2) add_memory_ds2(stackp2);
				*(stackp2++)=qp;
				pts[3*p]=pts[3*qp];
				pts[3*p+1]=pts[3*qp+1];
				pts[3*p+2]=pts[3*qp+2];
				ed[qp][nu[qp]<<1]=-p;
				j=p++;
				i=0;
			}
			nu[j]=k;

			// Unless the previous case was a double edge, connect
			// the first available edge of the new vertex to the
			// last one in the facet
			if(!double_edge) {
				ed[j][i]=cp;
				ed[j][nu[j]+i]=cs;
				vc.n_set(j,i,p_id);
				ed[cp][cs]=j;
				ed[cp][nu[cp]+cs]=i;
				i++;
			}

			// Copy in the edges of the underlying vertex,
			// and do one less if this was a double edge
			qs=iqs;
			while(i<(new_double_edge?k:k-1)) {
				qs=cycle_up(qs,qp);
				lp=ed[qp][qs];ls=ed[qp][nu[qp]+qs];
				vc.n_copy(j,i,qp,qs);
				ed[j][i]=lp;
				ed[j][nu[j]+i]=ls;
				ed[lp][ls]=j;
				ed[lp][nu[lp]+ls]=i;
				ed[qp][qs]=-1;
				i++;
			}
			qs=cycle_up(qs,qp);
			cs=i;
			cp=j;
			vc.n_copy(j,new_double_edge?0:cs,qp,qs);

			// Update the double_edge flag, to pass it
			// to the next instance of this routine
			double_edge=new_double_edge;
		}
	}

	// Connect the final created vertex to the initial one
	ed[cp][cs]=rp;
	*ed[rp]=cp;
	ed[cp][nu[cp]+cs]=0;
	ed[rp][nu[rp]]=cs;

	// Delete points: first, remove any duplicates
	dsp=ds;
	while(dsp<stackp) {
		j=*dsp;
		if(ed[j][nu[j]]!=-1) {
			ed[j][nu[j]]=-1;
			dsp++;
		} else *dsp=*(--stackp);
	}

	// Add the points in the auxiliary delete stack,
	// and reset their back pointers
	for(dsp=ds2;dsp<stackp2;dsp++) {
		j=*dsp;
		ed[j][nu[j]<<1]=j;
		if(ed[j][nu[j]]!=-1) {
			ed[j][nu[j]]=-1;
			if(stackp==stacke) add_memory_ds(stackp);
			*(stackp++)=j;
		}
	}

	// Scan connections and add in extras
	for(dsp=ds;dsp<stackp;dsp++) {
		cp=*dsp;
		for(edp=ed[cp];edp<ed[cp]+nu[cp];edp++) {
			qp=*edp;
			if(qp!=-1&&ed[qp][nu[qp]]!=-1) {
				if(stackp==stacke) {
					int dis=stackp-dsp;
					add_memory_ds(stackp);
					dsp=ds+dis;
				}
				*(stackp++)=qp;
				ed[qp][nu[qp]]=-1;
			}
		}
	}
	up=0;

	// Delete them from the array structure
	while(stackp>ds) {
		--p;
		while(ed[p][nu[p]]==-1) {
			j=nu[p];
			edp=ed[p];edd=(mep[j]+((j<<1)+1)*--mec[j]);
			while(edp<ed[p]+(size_t(j)<<1)+1) *(edp++)=*(edd++);
			vc.n_set_aux2_copy(p,j);
			vc.n_copy_pointer(ed[p][(j<<1)],p);
			ed[ed[p][(j<<1)]]=ed[p];
			--p;
		}
		up=*(--stackp);
		if(up<p) {

			// Vertex management
			pts[3*up]=pts[3*p];
			pts[3*up+1]=pts[3*p+1];
			pts[3*up+2]=pts[3*p+2];

			// Memory management
			j=nu[up];
			edp=ed[up];edd=(mep[j]+((j<<1)+1)*--mec[j]);
			while(edp<ed[up]+(size_t(j)<<1)+1) *(edp++)=*(edd++);
			vc.n_set_aux2_copy(up,j);
			vc.n_copy_pointer(ed[up][j<<1],up);
			vc.n_copy_pointer(up,p);
			ed[ed[up][j<<1]]=ed[up];

			// Edge management
			ed[up]=ed[p];
			nu[up]=nu[p];
			for(i=0;i<nu[up];i++) ed[ed[up][i]][ed[up][nu[up]+i]]=up;
			ed[up][nu[up]<<1]=up;
		} else up=p++;
	}

	// Check for any vertices of zero order
	if(*mec>0) voro_fatal_error("Zero order vertex formed",VOROPP_INTERNAL_ERROR);

	// Collapse any order 2 vertices and exit
	return collapse_order2(vc);
}

/** During the creation of a new facet in the plane routine, it is possible
 * that some order two vertices may arise. This routine removes them.
 * Suppose an order two vertex joins c and d. If there's a edge between
 * c and d already, then the order two vertex is just removed; otherwise,
 * the order two vertex is removed and c and d are joined together directly.
 * It is possible this process will create order two or order one vertices,
 * and the routine is continually run until all of them are removed.
 * \return False if the vertex removal was unsuccessful, indicative of the cell
 *         reducing to zero volume and disappearing; true if the vertex removal
 *         was successful. */
template<class vc_class>
inline bool voronoicell_base::collapse_order2(vc_class &vc) {
	if(!collapse_order1(vc)) return false;
	int a,b,i,j,k,l;
	while(mec[2]>0) {

		// Pick a order 2 vertex and read in its edges
		i=--mec[2];
		j=mep[2][5*i];k=mep[2][5*i+1];
		if(j==k) {
#if VOROPP_VERBOSE >=1
			fputs("Order two vertex joins itself",stderr);
#endif
			return false;
		}

		// Scan the edges of j to see if joins k
		for(l=0;l<nu[j];l++) {
			if(ed[j][l]==k) break;
		}

		// If j doesn't already join k, join them together.
		// Otherwise delete the connection to the current
		// vertex from j and k.
		a=mep[2][5*i+2];b=mep[2][5*i+3];i=mep[2][5*i+4];
		if(l==nu[j]) {
			ed[j][a]=k;
			ed[k][b]=j;
			ed[j][nu[j]+a]=b;
			ed[k][nu[k]+b]=a;
		} else {
			if(!delete_connection(vc,j,a,false)) return false;
			if(!delete_connection(vc,k,b,true)) return false;
		}

		// Compact the memory
		--p;
		if(up==i) up=0;
		if(p!=i) {
			if(up==p) up=i;
			pts[3*i]=pts[3*p];
			pts[3*i+1]=pts[3*p+1];
			pts[3*i+2]=pts[3*p+2];
			for(k=0;k<nu[p];k++) ed[ed[p][k]][ed[p][nu[p]+k]]=i;
			vc.n_copy_pointer(i,p);
			ed[i]=ed[p];
			nu[i]=nu[p];
			ed[i][nu[i]<<1]=i;
		}

		// Collapse any order 1 vertices if they were created
		if(!collapse_order1(vc)) return false;
	}
	return true;
}

/** Order one vertices can potentially be created during the order two collapse
 * routine. This routine keeps removing them until there are none left.
 * \return False if the vertex removal was unsuccessful, indicative of the cell
 *         having zero volume and disappearing; true if the vertex removal was
 *         successful. */
template<class vc_class>
inline bool voronoicell_base::collapse_order1(vc_class &vc) {
	int i,j,k;
	while(mec[1]>0) {
		up=0;
#if VOROPP_VERBOSE >=1
		fputs("Order one collapse\n",stderr);
#endif
		i=--mec[1];
		j=mep[1][3*i];k=mep[1][3*i+1];
		i=mep[1][3*i+2];
		if(!delete_connection(vc,j,k,false)) return false;
		--p;
		if(up==i) up=0;
		if(p!=i) {
			if(up==p) up=i;
			pts[3*i]=pts[3*p];
			pts[3*i+1]=pts[3*p+1];
			pts[3*i+2]=pts[3*p+2];
			for(k=0;k<nu[p];k++) ed[ed[p][k]][ed[p][nu[p]+k]]=i;
			vc.n_copy_pointer(i,p);
			ed[i]=ed[p];
			nu[i]=nu[p];
			ed[i][nu[i]<<1]=i;
		}
	}
	return true;
}

/** This routine deletes the kth edge of vertex j and reorganizes the memory.
 * If the neighbor computation is enabled, we also have to supply an handedness
 * flag to decide whether to preserve the plane on the left or right of the
 * connection.
 * \return False if a zero order vertex was formed, indicative of the cell
 *         disappearing; true if the vertex removal was successful. */
template<class vc_class>
inline bool voronoicell_base::delete_connection(vc_class &vc,int j,int k,bool hand) {
	int q=hand?k:cycle_up(k,j);
	int i=nu[j]-1,l,*edp,*edd,m;
#if VOROPP_VERBOSE >=1
	if(i<1) {
		fputs("Zero order vertex formed\n",stderr);
		return false;
	}
#endif
	if(mec[i]==mem[i]) add_memory(vc,i,ds2);
	vc.n_set_aux1(i);
	for(l=0;l<q;l++) vc.n_copy_aux1(j,l);
	while(l<i) {
		vc.n_copy_aux1_shift(j,l);
		l++;
	}
	edp=mep[i]+((i<<1)+1)*mec[i]++;
	edp[i<<1]=j;
	for(l=0;l<k;l++) {
		edp[l]=ed[j][l];
		edp[l+i]=ed[j][l+nu[j]];
	}
	while(l<i) {
		m=ed[j][l+1];
		edp[l]=m;
		k=ed[j][l+nu[j]+1];
		edp[l+i]=k;
		ed[m][nu[m]+k]--;
		l++;
	}

	edd=mep[nu[j]]+((nu[j]<<1)+1)*--mec[nu[j]];
	for(l=0;l<=(nu[j]<<1);l++) ed[j][l]=edd[l];
	vc.n_set_aux2_copy(j,nu[j]);
	vc.n_set_to_aux2(edd[nu[j]<<1]);
	vc.n_set_to_aux1(j);
	ed[edd[nu[j]<<1]]=edd;
	ed[j]=edp;
	nu[j]=i;
	return true;
}

/** Calculates the volume of the Voronoi cell, by decomposing the cell into
 * tetrahedra extending outward from the zeroth vertex, whose volumes are
 * evaluated using a scalar triple product.
 * \return A floating point number holding the calculated volume. */
double voronoicell_base::volume() {
	const double fe=1/48.0;
	double vol=0;
	int i,j,k,l,m,n;
	double ux,uy,uz,vx,vy,vz,wx,wy,wz;
	for(i=1;i<p;i++) {
		ux=*pts-pts[3*i];
		uy=pts[1]-pts[3*i+1];
		uz=pts[2]-pts[3*i+2];
		for(j=0;j<nu[i];j++) {
			k=ed[i][j];
			if(k>=0) {
				ed[i][j]=-1-k;
				l=cycle_up(ed[i][nu[i]+j],k);
				vx=pts[3*k]-*pts;
				vy=pts[3*k+1]-pts[1];
				vz=pts[3*k+2]-pts[2];
				m=ed[k][l];ed[k][l]=-1-m;
				while(m!=i) {
					n=cycle_up(ed[k][nu[k]+l],m);
					wx=pts[3*m]-*pts;
					wy=pts[3*m+1]-pts[1];
					wz=pts[3*m+2]-pts[2];
					vol+=ux*vy*wz+uy*vz*wx+uz*vx*wy-uz*vy*wx-uy*vx*wz-ux*vz*wy;
					k=m;l=n;vx=wx;vy=wy;vz=wz;
					m=ed[k][l];ed[k][l]=-1-m;
				}
			}
		}
	}
	reset_edges();
	return vol*fe;
}

/** Calculates the areas of each face of the Voronoi cell and prints the
 * results to an output stream.
 * \param[out] v the vector to store the results in. */
void voronoicell_base::face_areas(std::vector<double> &v) {
	double area;
	v.clear();
	int i,j,k,l,m,n;
	double ux,uy,uz,vx,vy,vz,wx,wy,wz;
	for(i=1;i<p;i++) for(j=0;j<nu[i];j++) {
		k=ed[i][j];
		if(k>=0) {
			area=0;
			ed[i][j]=-1-k;
			l=cycle_up(ed[i][nu[i]+j],k);
			m=ed[k][l];ed[k][l]=-1-m;
			while(m!=i) {
				n=cycle_up(ed[k][nu[k]+l],m);
				ux=pts[3*k]-pts[3*i];
				uy=pts[3*k+1]-pts[3*i+1];
				uz=pts[3*k+2]-pts[3*i+2];
				vx=pts[3*m]-pts[3*i];
				vy=pts[3*m+1]-pts[3*i+1];
				vz=pts[3*m+2]-pts[3*i+2];
				wx=uy*vz-uz*vy;
				wy=uz*vx-ux*vz;
				wz=ux*vy-uy*vx;
				area+=sqrt(wx*wx+wy*wy+wz*wz);
				k=m;l=n;
				m=ed[k][l];ed[k][l]=-1-m;
			}
			v.push_back(0.125*area);
		}
	}
	reset_edges();
}


/** Calculates the total surface area of the Voronoi cell.
 * \return The computed area. */
double voronoicell_base::surface_area() {
	double area=0;
	int i,j,k,l,m,n;
	double ux,uy,uz,vx,vy,vz,wx,wy,wz;
	for(i=1;i<p;i++) for(j=0;j<nu[i];j++) {
		k=ed[i][j];
		if(k>=0) {
			ed[i][j]=-1-k;
			l=cycle_up(ed[i][nu[i]+j],k);
			m=ed[k][l];ed[k][l]=-1-m;
			while(m!=i) {
				n=cycle_up(ed[k][nu[k]+l],m);
				ux=pts[3*k]-pts[3*i];
				uy=pts[3*k+1]-pts[3*i+1];
				uz=pts[3*k+2]-pts[3*i+2];
				vx=pts[3*m]-pts[3*i];
				vy=pts[3*m+1]-pts[3*i+1];
				vz=pts[3*m+2]-pts[3*i+2];
				wx=uy*vz-uz*vy;
				wy=uz*vx-ux*vz;
				wz=ux*vy-uy*vx;
				area+=sqrt(wx*wx+wy*wy+wz*wz);
				k=m;l=n;
				m=ed[k][l];ed[k][l]=-1-m;
			}
		}
	}
	reset_edges();
	return 0.125*area;
}


/** Calculates the centroid of the Voronoi cell, by decomposing the cell into
 * tetrahedra extending outward from the zeroth vertex.
 * \param[out] (cx,cy,cz) references to floating point numbers in which to
 *                        pass back the centroid vector. */
void voronoicell_base::centroid(double &cx,double &cy,double &cz) {
	double tvol,vol=0;cx=cy=cz=0;
	int i,j,k,l,m,n;
	double ux,uy,uz,vx,vy,vz,wx,wy,wz;
	for(i=1;i<p;i++) {
		ux=*pts-pts[3*i];
		uy=pts[1]-pts[3*i+1];
		uz=pts[2]-pts[3*i+2];
		for(j=0;j<nu[i];j++) {
			k=ed[i][j];
			if(k>=0) {
				ed[i][j]=-1-k;
				l=cycle_up(ed[i][nu[i]+j],k);
				vx=pts[3*k]-*pts;
				vy=pts[3*k+1]-pts[1];
				vz=pts[3*k+2]-pts[2];
				m=ed[k][l];ed[k][l]=-1-m;
				while(m!=i) {
					n=cycle_up(ed[k][nu[k]+l],m);
					wx=pts[3*m]-*pts;
					wy=pts[3*m+1]-pts[1];
					wz=pts[3*m+2]-pts[2];
					tvol=ux*vy*wz+uy*vz*wx+uz*vx*wy-uz*vy*wx-uy*vx*wz-ux*vz*wy;
					vol+=tvol;
					cx+=(wx+vx-ux)*tvol;
					cy+=(wy+vy-uy)*tvol;
					cz+=(wz+vz-uz)*tvol;
					k=m;l=n;vx=wx;vy=wy;vz=wz;
					m=ed[k][l];ed[k][l]=-1-m;
				}
			}
		}
	}
	reset_edges();
	if(vol>tolerance_sq) {
		vol=0.125/vol;
		cx=cx*vol+0.5*(*pts);
		cy=cy*vol+0.5*pts[1];
		cz=cz*vol+0.5*pts[2];
	} else cx=cy=cz=0;
}

/** Computes the maximum radius squared of a vertex from the center of the
 * cell. It can be used to determine when enough particles have been testing an
 * all planes that could cut the cell have been considered.
 * \return The maximum radius squared of a vertex.*/
double voronoicell_base::max_radius_squared() {
	double r,s,*ptsp=pts+3,*ptse=pts+3*p;
	r=*pts*(*pts)+pts[1]*pts[1]+pts[2]*pts[2];
	while(ptsp<ptse) {
		s=*ptsp*(*ptsp);ptsp++;
		s+=*ptsp*(*ptsp);ptsp++;
		s+=*ptsp*(*ptsp);ptsp++;
		if(s>r) r=s;
	}
	return r;
}

/** Calculates the total edge distance of the Voronoi cell.
 * \return A floating point number holding the calculated distance. */
double voronoicell_base::total_edge_distance() {
	int i,j,k;
	double dis=0,dx,dy,dz;
	for(i=0;i<p-1;i++) for(j=0;j<nu[i];j++) {
		k=ed[i][j];
		if(k>i) {
			dx=pts[3*k]-pts[3*i];
			dy=pts[3*k+1]-pts[3*i+1];
			dz=pts[3*k+2]-pts[3*i+2];
			dis+=sqrt(dx*dx+dy*dy+dz*dz);
		}
	}
	return 0.5*dis;
}

inline bool voronoicell_base::search_edge(int l,int &m,int &k) {
	for(m=0;m<nu[l];m++) {
		k=ed[l][m];
		if(k>=0) return true;
	}
	return false;
}

/** Several routines in the class that gather cell-based statistics internally
 * track their progress by flipping edges to negative so that they know what
 * parts of the cell have already been tested. This function resets them back
 * to positive. When it is called, it assumes that every edge in the routine
 * should have already been flipped to negative, and it bails out with an
 * internal error if it encounters a positive edge. */
inline void voronoicell_base::reset_edges() {
	int i,j;
	for(i=0;i<p;i++) for(j=0;j<nu[i];j++) {
		if(ed[i][j]>=0) voro_fatal_error("Edge reset routine found a previously untested edge",VOROPP_INTERNAL_ERROR);
		ed[i][j]=-1-ed[i][j];
	}
}

/** Checks to see if a given vertex is inside, outside or within the test
 * plane. If the point is far away from the test plane, the routine immediately
 * returns whether it is inside or outside. If the routine is close the the
 * plane and within the specified tolerance, then the special check_marginal()
 * routine is called.
 * \param[in] n the vertex to test.
 * \param[out] ans the result of the scalar product used in evaluating the
 *                 location of the point.
 * \return -1 if the point is inside the plane, 1 if the point is outside the
 *         plane, or 0 if the point is within the plane. */
inline int voronoicell_base::m_test(int n,double &ans) {
	double *pp=pts+n+(size_t(n)<<1);
	ans=*(pp++)*px;
	ans+=*(pp++)*py;
	ans+=*pp*pz-prsq;
	if(ans<-tolerance2) {
		return -1;
	} else if(ans>tolerance2) {
		return 1;
	}
	return check_marginal(n,ans);
}

/** Checks to see if a given vertex is inside, outside or within the test
 * plane, for the case when the point has been detected to be very close to the
 * plane. The routine ensures that the returned results are always consistent
 * with previous tests, by keeping a table of any marginal results. The routine
 * first sees if the vertex is in the table, and if it finds a previously
 * computed result it uses that. Otherwise, it computes a result for this
 * vertex and adds it the table.
 * \param[in] n the vertex to test.
 * \param[in] ans the result of the scalar product used in evaluating
 *                the location of the point.
 * \return -1 if the point is inside the plane, 1 if the point is outside the
 *         plane, or 0 if the point is within the plane. */
int voronoicell_base::check_marginal(int n,double &ans) {
	int i;
	for(i=0;i<n_marg;i+=2) if(marg[i]==n) return marg[i+1];
	if(n_marg==current_marginal) {
		current_marginal<<=1;
		if(current_marginal>max_marginal)
			voro_fatal_error("Marginal case buffer allocation exceeded absolute maximum",VOROPP_MEMORY_ERROR);
#if VOROPP_VERBOSE >=2
		fprintf(stderr,"Marginal cases buffer scaled up to %d\n",i);
#endif
		int *pmarg=new int[current_marginal];
		for(int j=0;j<n_marg;j++) pmarg[j]=marg[j];
		delete [] marg;
		marg=pmarg;
	}
	marg[n_marg++]=n;
	marg[n_marg++]=ans>tolerance?1:(ans<-tolerance?-1:0);
	return marg[n_marg-1];
}

/** For each face of the Voronoi cell, this routine prints the out the normal
 * vector of the face, and scales it to the distance from the cell center to
 * that plane.
 * \param[out] v the vector to store the results in. */
void voronoicell_base::normals(std::vector<double> &v) {
	int i,j,k;
	v.clear();
	for(i=1;i<p;i++) for(j=0;j<nu[i];j++) {
		k=ed[i][j];
		if(k>=0) normals_search(v,i,j,k);
	}
	reset_edges();
}

/** This inline routine is called by normals(). It attempts to construct a
 * single normal vector that is associated with a particular face. It first
 * traces around the face, trying to find two vectors along the face edges
 * whose vector product is above the numerical tolerance. It then constructs
 * the normal vector using this product. If the face is too small, and none of
 * the vector products are large enough, the routine may return (0,0,0) as the
 * normal vector.
 * \param[in] v the vector to store the results in.
 * \param[in] i the initial vertex of the face to test.
 * \param[in] j the index of an edge of the vertex.
 * \param[in] k the neighboring vertex of i, set to ed[i][j]. */
inline void voronoicell_base::normals_search(std::vector<double> &v,int i,int j,int k) {
	ed[i][j]=-1-k;
	int l=cycle_up(ed[i][nu[i]+j],k),m;
	double ux,uy,uz,vx,vy,vz,wx,wy,wz,wmag;
	do {
		m=ed[k][l];ed[k][l]=-1-m;
		ux=pts[3*m]-pts[3*k];
		uy=pts[3*m+1]-pts[3*k+1];
		uz=pts[3*m+2]-pts[3*k+2];

		// Test to see if the length of this edge is above the tolerance
		if(ux*ux+uy*uy+uz*uz>tolerance_sq) {
			while(m!=i) {
				l=cycle_up(ed[k][nu[k]+l],m);
				k=m;m=ed[k][l];ed[k][l]=-1-m;
				vx=pts[3*m]-pts[3*k];
				vy=pts[3*m+1]-pts[3*k+1];
				vz=pts[3*m+2]-pts[3*k+2];

				// Construct the vector product of this edge with
				// the previous one
				wx=uz*vy-uy*vz;
				wy=ux*vz-uz*vx;
				wz=uy*vx-ux*vy;
				wmag=wx*wx+wy*wy+wz*wz;

				// Test to see if this vector product of the
				// two edges is above the tolerance
				if(wmag>tolerance_sq) {

					// Construct the normal vector and print it
					wmag=1/sqrt(wmag);
					v.push_back(wx*wmag);
					v.push_back(wy*wmag);
					v.push_back(wz*wmag);

					// Mark all of the remaining edges of this
					// face and exit
					while(m!=i) {
						l=cycle_up(ed[k][nu[k]+l],m);
						k=m;m=ed[k][l];ed[k][l]=-1-m;
					}
					return;
				}
			}
			v.push_back(0);
			v.push_back(0);
			v.push_back(0);
			return;
		}
		l=cycle_up(ed[k][nu[k]+l],m);
		k=m;
	} while (k!=i);
	v.push_back(0);
	v.push_back(0);
	v.push_back(0);
}


/** Returns the number of faces of a computed Voronoi cell.
 * \return The number of faces. */
int voronoicell_base::number_of_faces() {
	int i,j,k,l,m,s=0;
	for(i=1;i<p;i++) for(j=0;j<nu[i];j++) {
		k=ed[i][j];
		if(k>=0) {
			s++;
			ed[i][j]=-1-k;
			l=cycle_up(ed[i][nu[i]+j],k);
			do {
				m=ed[k][l];
				ed[k][l]=-1-m;
				l=cycle_up(ed[k][nu[k]+l],m);
				k=m;
			} while (k!=i);

		}
	}
	reset_edges();
	return s;
}

/** Returns a vector of the vertex orders.
 * \param[out] v the vector to store the results in. */
void voronoicell_base::vertex_orders(std::vector<int> &v) {
	v.resize(p);
	for(int i=0;i<p;i++) v[i]=nu[i];
}

/** Returns a vector of the vertex vectors using the local coordinate system.
 * \param[out] v the vector to store the results in. */
void voronoicell_base::vertices(std::vector<double> &v) {
	v.resize(3*p);
	double *ptsp=pts;
	for(int i=0;i<3*p;i+=3) {
		v[i]=*(ptsp++)*0.5;
		v[i+1]=*(ptsp++)*0.5;
		v[i+2]=*(ptsp++)*0.5;
	}
}


/** Returns a vector of the vertex vectors in the global coordinate system.
 * \param[out] v the vector to store the results in.
 * \param[in] (x,y,z) the position vector of the particle in the global
 *                    coordinate system. */
void voronoicell_base::vertices(double x,double y,double z,std::vector<double> &v) {
	v.resize(3*p);
	double *ptsp=pts;
	for(int i=0;i<3*p;i+=3) {
		v[i]=x+*(ptsp++)*0.5;
		v[i+1]=y+*(ptsp++)*0.5;
		v[i+2]=z+*(ptsp++)*0.5;
	}
}

/** This routine returns the perimeters of each face.
 * \param[out] v the vector to store the results in. */
void voronoicell_base::face_perimeters(std::vector<double> &v) {
	v.clear();
	int i,j,k,l,m;
	double dx,dy,dz,perim;
	for(i=1;i<p;i++) for(j=0;j<nu[i];j++) {
		k=ed[i][j];
		if(k>=0) {
			dx=pts[3*k]-pts[3*i];
			dy=pts[3*k+1]-pts[3*i+1];
			dz=pts[3*k+2]-pts[3*i+2];
			perim=sqrt(dx*dx+dy*dy+dz*dz);
			ed[i][j]=-1-k;
			l=cycle_up(ed[i][nu[i]+j],k);
			do {
				m=ed[k][l];
				dx=pts[3*m]-pts[3*k];
				dy=pts[3*m+1]-pts[3*k+1];
				dz=pts[3*m+2]-pts[3*k+2];
				perim+=sqrt(dx*dx+dy*dy+dz*dz);
				ed[k][l]=-1-m;
				l=cycle_up(ed[k][nu[k]+l],m);
				k=m;
			} while (k!=i);
			v.push_back(0.5*perim);
		}
	}
	reset_edges();
}

/** For each face, this routine outputs a bracketed sequence of numbers
 * containing a list of all the vertices that make up that face.
 * \param[out] v the vector to store the results in. */
void voronoicell_base::face_vertices(std::vector<int> &v) {
	int i,j,k,l,m,vp(0),vn;
	v.clear();
	for(i=1;i<p;i++) for(j=0;j<nu[i];j++) {
		k=ed[i][j];
		if(k>=0) {
			v.push_back(0);
			v.push_back(i);
			ed[i][j]=-1-k;
			l=cycle_up(ed[i][nu[i]+j],k);
			do {
				v.push_back(k);
				m=ed[k][l];
				ed[k][l]=-1-m;
				l=cycle_up(ed[k][nu[k]+l],m);
				k=m;
			} while (k!=i);
			vn=v.size();
			v[vp]=vn-vp-1;
			vp=vn;
		}
	}
	reset_edges();
}

/** Outputs a list of the number of edges in each face.
 * \param[out] v the vector to store the results in. */
void voronoicell_base::face_orders(std::vector<int> &v) {
	int i,j,k,l,m,q;
	v.clear();
	for(i=1;i<p;i++) for(j=0;j<nu[i];j++) {
		k=ed[i][j];
		if(k>=0) {
			q=1;
			ed[i][j]=-1-k;
			l=cycle_up(ed[i][nu[i]+j],k);
			do {
				q++;
				m=ed[k][l];
				ed[k][l]=-1-m;
				l=cycle_up(ed[k][nu[k]+l],m);
				k=m;
			} while (k!=i);
			v.push_back(q);;
		}
	}
	reset_edges();
}

/** Computes the number of edges that each face has and outputs a frequency
 * table of the results.
 * \param[out] v the vector to store the results in. */
void voronoicell_base::face_freq_table(std::vector<int> &v) {
	int i,j,k,l,m,q;
	v.clear();
	for(i=1;i<p;i++) for(j=0;j<nu[i];j++) {
		k=ed[i][j];
		if(k>=0) {
			q=1;
			ed[i][j]=-1-k;
			l=cycle_up(ed[i][nu[i]+j],k);
			do {
				q++;
				m=ed[k][l];
				ed[k][l]=-1-m;
				l=cycle_up(ed[k][nu[k]+l],m);
				k=m;
			} while (k!=i);
			if((unsigned int) q>=v.size()) v.resize(q+1,0);
			v[q]++;
		}
	}
	reset_edges();
}

/** This routine tests to see whether the cell intersects a plane by starting
 * from the guess point up. If up intersects, then it immediately returns true.
 * Otherwise, it calls the plane_intersects_track() routine.
 * \param[in] (x,y,z) the normal vector to the plane.
 * \param[in] rsq the distance along this vector of the plane.
 * \return False if the plane does not intersect the plane, true if it does. */
bool voronoicell_base::plane_intersects(double x,double y,double z,double rsq) {
	double g=x*pts[3*up]+y*pts[3*up+1]+z*pts[3*up+2];
	if(g<rsq) return plane_intersects_track(x,y,z,rsq,g);
	return true;
}

/** This routine tests to see if a cell intersects a plane. It first tests a
 * random sample of approximately sqrt(p)/4 points. If any of those are
 * intersect, then it immediately returns true. Otherwise, it takes the closest
 * point and passes that to plane_intersect_track() routine.
 * \param[in] (x,y,z) the normal vector to the plane.
 * \param[in] rsq the distance along this vector of the plane.
 * \return False if the plane does not intersect the plane, true if it does. */
bool voronoicell_base::plane_intersects_guess(double x,double y,double z,double rsq) {
	up=0;
	double g=x*pts[3*up]+y*pts[3*up+1]+z*pts[3*up+2];
	if(g<rsq) {
		int ca=1,cc=p>>3,mp=1;
		double m;
		while(ca<cc) {
			m=x*pts[3*mp]+y*pts[3*mp+1]+z*pts[3*mp+2];
			if(m>g) {
				if(m>rsq) return true;
				g=m;up=mp;
			}
			ca+=mp++;
		}
		return plane_intersects_track(x,y,z,rsq,g);
	}
	return true;
}

/* This routine tests to see if a cell intersects a plane, by tracing over the cell from
 * vertex to vertex, starting at up. It is meant to be called either by plane_intersects()
 * or plane_intersects_track(), when those routines cannot immediately resolve the case.
 * \param[in] (x,y,z) the normal vector to the plane.
 * \param[in] rsq the distance along this vector of the plane.
 * \param[in] g the distance of up from the plane.
 * \return False if the plane does not intersect the plane, true if it does. */
inline bool voronoicell_base::plane_intersects_track(double x,double y,double z,double rsq,double g) {
	int count=0,ls,usOuter,tp;
	double t;

	// The test point is outside of the cutting space
	for(usOuter=0;usOuter<nu[up];usOuter++) {
		tp=ed[up][usOuter];
		t=x*pts[3*tp]+y*pts[3*tp+1]+z*pts[3*tp+2];
		if(t>g) {
			ls=ed[up][nu[up]+usOuter];
			up=tp;
			while (t<rsq) {
				if(++count>=p) {
#if VOROPP_VERBOSE >=1
					fputs("Bailed out of convex calculation",stderr);
#endif
					for(tp=0;tp<p;tp++) if(x*pts[3*tp]+y*pts[3*tp+1]+z*pts[3*tp+2]>rsq) return true;
					return false;
				}

				// Test all the neighbors of the current point
				// and find the one which is closest to the
				// plane
				int usInner;
				for(usInner=0;usInner<ls;usInner++) {
					tp=ed[up][usInner];
					g=x*pts[3*tp]+y*pts[3*tp+1]+z*pts[3*tp+2];
					if(g>t) break;
				}
				if(usInner==ls) {
					usInner++;
					while(usInner<nu[up]) {
						tp=ed[up][usInner];
						g=x*pts[3*tp]+y*pts[3*tp+1]+z*pts[3*tp+2];
						if(g>t) break;
						usInner++;
					}
					if(usInner==nu[up]) return false;
				}
				ls=ed[up][nu[up]+usInner];up=tp;t=g;
			}
			return true;
		}
	}
	return false;
}

/** Counts the number of edges of the Voronoi cell.
 * \return the number of edges. */
int voronoicell_base::number_of_edges() {
	int edges=0,*nup=nu;
	while(nup<nu+p) edges+=*(nup++);
	return edges>>1;
}

/** This initializes the class to be a rectangular box. It calls the base class
 * initialization routine to set up the edge and vertex information, and then
 * sets up the neighbor information, with initial faces being assigned ID
 * numbers from -1 to -6.
 * \param[in] (xmin,xmax) the minimum and maximum x coordinates.
 * \param[in] (ymin,ymax) the minimum and maximum y coordinates.
 * \param[in] (zmin,zmax) the minimum and maximum z coordinates. */
void voronoicell_neighbor::init(double xmin,double xmax,double ymin,double ymax,double zmin,double zmax) {
	init_base(xmin,xmax,ymin,ymax,zmin,zmax);
	int *q=mne[3];
	*q=-5;q[1]=-3;q[2]=-1;
	q[3]=-5;q[4]=-2;q[5]=-3;
	q[6]=-5;q[7]=-1;q[8]=-4;
	q[9]=-5;q[10]=-4;q[11]=-2;
	q[12]=-6;q[13]=-1;q[14]=-3;
	q[15]=-6;q[16]=-3;q[17]=-2;
	q[18]=-6;q[19]=-4;q[20]=-1;
	q[21]=-6;q[22]=-2;q[23]=-4;
	*ne=q;ne[1]=q+3;ne[2]=q+6;ne[3]=q+9;
	ne[4]=q+12;ne[5]=q+15;ne[6]=q+18;ne[7]=q+21;
}

/** This initializes the class to be an octahedron. It calls the base class
 * initialization routine to set up the edge and vertex information, and then
 * sets up the neighbor information, with the initial faces being assigned ID
 * numbers from -1 to -8.
 * \param[in] l The distance from the octahedron center to a vertex. Six
 *              vertices are initialized at (-l,0,0), (l,0,0), (0,-l,0),
 *              (0,l,0), (0,0,-l), and (0,0,l). */
void voronoicell_neighbor::init_octahedron(double l) {
	init_octahedron_base(l);
	int *q=mne[4];
	*q=-5;q[1]=-6;q[2]=-7;q[3]=-8;
	q[4]=-1;q[5]=-2;q[6]=-3;q[7]=-4;
	q[8]=-6;q[9]=-5;q[10]=-2;q[11]=-1;
	q[12]=-8;q[13]=-7;q[14]=-4;q[15]=-3;
	q[16]=-5;q[17]=-8;q[18]=-3;q[19]=-2;
	q[20]=-7;q[21]=-6;q[22]=-1;q[23]=-4;
	*ne=q;ne[1]=q+4;ne[2]=q+8;ne[3]=q+12;ne[4]=q+16;ne[5]=q+20;
}

/** This initializes the class to be a tetrahedron. It calls the base class
 * initialization routine to set up the edge and vertex information, and then
 * sets up the neighbor information, with the initial faces being assigned ID
 * numbers from -1 to -4.
 * \param (x0,y0,z0) a position vector for the first vertex.
 * \param (x1,y1,z1) a position vector for the second vertex.
 * \param (x2,y2,z2) a position vector for the third vertex.
 * \param (x3,y3,z3) a position vector for the fourth vertex. */
void voronoicell_neighbor::init_tetrahedron(double x0,double y0,double z0,double x1,double y1,double z1,double x2,double y2,double z2,double x3,double y3,double z3) {
	init_tetrahedron_base(x0,y0,z0,x1,y1,z1,x2,y2,z2,x3,y3,z3);
	int *q=mne[3];
	*q=-4;q[1]=-3;q[2]=-2;
	q[3]=-3;q[4]=-4;q[5]=-1;
	q[6]=-4;q[7]=-2;q[8]=-1;
	q[9]=-2;q[10]=-3;q[11]=-1;
	*ne=q;ne[1]=q+3;ne[2]=q+6;ne[3]=q+9;
}

/** This routine checks to make sure the neighbor information of each face is
 * consistent. */
void voronoicell_neighbor::check_facets() {
	int i,j,k,l,m,q;
	for(i=1;i<p;i++) for(j=0;j<nu[i];j++) {
		k=ed[i][j];
		if(k>=0) {
			ed[i][j]=-1-k;
			q=ne[i][j];
			l=cycle_up(ed[i][nu[i]+j],k);
			do {
				m=ed[k][l];
				ed[k][l]=-1-m;
				if(ne[k][l]!=q) fprintf(stderr,"Facet error at (%d,%d)=%d, started from (%d,%d)=%d\n",k,l,ne[k][l],i,j,q);
				l=cycle_up(ed[k][nu[k]+l],m);
				k=m;
			} while (k!=i);
		}
	}
	reset_edges();
}

/** The class constructor allocates memory for storing neighbor information. */
voronoicell_neighbor::voronoicell_neighbor() {
	int i;
	check(current_vertex_order >= 4);
	mne=new int*[current_vertex_order];
	ne=new int*[current_vertices];
	for(i=0;i<3;i++) mne[i]=new int[init_n_vertices*i];
	mne[3]=new int[init_3_vertices*3];
	for(i=4;i<current_vertex_order;i++) mne[i]=new int[init_n_vertices*i];
}

/** The class destructor frees the dynamically allocated memory for storing
 * neighbor information. */
voronoicell_neighbor::~voronoicell_neighbor() {
	for(int i=current_vertex_order-1;i>=0;i--) if(mem[i]>0) delete [] mne[i];
	delete [] mne;
	delete [] ne;
}

/** Computes a vector list of neighbors. */
void voronoicell_neighbor::neighbors(std::vector<int> &v) {
	v.clear();
	int i,j,k,l,m;
	for(i=1;i<p;i++) for(j=0;j<nu[i];j++) {
		k=ed[i][j];
		if(k>=0) {
			v.push_back(ne[i][j]);
			ed[i][j]=-1-k;
			l=cycle_up(ed[i][nu[i]+j],k);
			do {
				m=ed[k][l];
				ed[k][l]=-1-m;
				l=cycle_up(ed[k][nu[k]+l],m);
				k=m;
			} while (k!=i);
		}
	}
	reset_edges();
}
/** Computes an Unreal TArray of neighbors. */
void voronoicell_neighbor::neighborsTArray(TArray<int> &nbrs, bool excludeBounds) {
	nbrs.Empty();
	int i, j, k, l, m;
	for (i = 1; i < p; i++) for (j = 0; j < nu[i]; j++) {
		k = ed[i][j];
		if (k >= 0) {
			if (!excludeBounds || ne[i][j] >= 0) {
				nbrs.Add(ne[i][j]);
			}
			ed[i][j] = -1 - k;
			l = cycle_up(ed[i][nu[i] + j], k);
			do {
				m = ed[k][l];
				ed[k][l] = -1 - m;
				l = cycle_up(ed[k][nu[k] + l], m);
				k = m;
			} while (k != i);
		}
	}
	reset_edges();
}
/** Computes Unreal TArrays of all cell info. */
void voronoicell_neighbor::extractCellInfo(const FVector& CellPosition, TArray<FVector> &Vertices, TArray<int32> &FaceVertexIndices, TArray<int32> &Nbrs, TArray<FVector> &Normals)
{
	Vertices.SetNumUninitialized(p);
	double *ptsp = pts;
	for (int i = 0; i < p; i++)
	{
		Vertices[i].X = CellPosition.X + *(ptsp++)*0.5;
		Vertices[i].Y = CellPosition.Y + *(ptsp++)*0.5;
		Vertices[i].Z = CellPosition.Z + *(ptsp++)*0.5;
	}
	
	
	FaceVertexIndices.Reset(); Nbrs.Reset(); Normals.Reset();

	// Vertex indices code (face_vertices)
	int i, j, k, l, m, vp(0), vn;
	for (i = 1; i < p; i++) for (j = 0; j < nu[i]; j++) {
		k = ed[i][j];
		if (k >= 0) { // on a new face
			Nbrs.Add(ne[i][j]);
			FaceVertexIndices.Add(0); // placeholder for face vertex count
			FaceVertexIndices.Add(i);
			FVector Normal(0,0,0);
			double foundEdgeMag = 0;
			double ux = 0, uy = 0, uz = 0, vx, vy, vz, wx, wy, wz, wmag = 0;

			ed[i][j] = -1 - k;
			l = cycle_up(ed[i][nu[i] + j], k);
			do {
				FaceVertexIndices.Add(k);
				m = ed[k][l];
				ed[k][l] = -1 - m;

				if (wmag <= tolerance_sq) // Haven't found a Normal yet
				{
					if (foundEdgeMag <= tolerance_sq) // Haven't found a first edge yet
					{
						ux = pts[3 * m] - pts[3 * k];
						uy = pts[3 * m + 1] - pts[3 * k + 1];
						uz = pts[3 * m + 2] - pts[3 * k + 2];
						foundEdgeMag = ux * ux + uy * uy + uz * uz;
					}
					else // Have a first edge; try constructing the Normal from each following edge
					{
						vx = pts[3 * m] - pts[3 * k];
						vy = pts[3 * m + 1] - pts[3 * k + 1];
						vz = pts[3 * m + 2] - pts[3 * k + 2];

						// Construct the vector product of this edge with
						// the previously-found one
						wx = uz * vy - uy * vz;
						wy = ux * vz - uz * vx;
						wz = uy * vx - ux * vy;
						wmag = wx * wx + wy * wy + wz * wz;

						// Test to see if this vector product of the
						// two edges is above the tolerance
						if (wmag > tolerance_sq) {
							// Set the normal vector
							double wmagInv = 1.0 / sqrt(wmag);
							Normal.X = wx * wmagInv;
							Normal.Y = wy * wmagInv;
							Normal.Z = wz * wmagInv;
						}
					}
				}

				l = cycle_up(ed[k][nu[k] + l], m);
				k = m;
			} while (k != i);

			Normals.Add(Normal);
			vn = FaceVertexIndices.Num();
			FaceVertexIndices[vp] = vn - vp - 1;
			vp = vn;
		}
	}
	reset_edges();
}
/** Computes Unreal TArrays of all cell info. */
void voronoicell::extractCellInfo(const FVector& CellPosition, TArray<FVector>& Vertices, TArray<int32>& FaceVertexIndices)
{
	Vertices.SetNumUninitialized(p);
	double* ptsp = pts;
	for (int i = 0; i < p; i++)
	{
		Vertices[i].X = CellPosition.X + *(ptsp++) * 0.5;
		Vertices[i].Y = CellPosition.Y + *(ptsp++) * 0.5;
		Vertices[i].Z = CellPosition.Z + *(ptsp++) * 0.5;
	}

	FaceVertexIndices.Reset();

	// Vertex indices code (face_vertices)
	int i, j, k, l, m, vp(0), vn;
	for (i = 1; i < p; i++) for (j = 0; j < nu[i]; j++) {
		k = ed[i][j];
		if (k >= 0) { // on a new face
			FaceVertexIndices.Add(0); // placeholder for face vertex count
			FaceVertexIndices.Add(i);

			ed[i][j] = -1 - k;
			l = cycle_up(ed[i][nu[i] + j], k);
			do {
				FaceVertexIndices.Add(k);
				m = ed[k][l];
				ed[k][l] = -1 - m;

				l = cycle_up(ed[k][nu[k] + l], m);
				k = m;
			} while (k != i);

			vn = FaceVertexIndices.Num();
			FaceVertexIndices[vp] = vn - vp - 1;
			vp = vn;
		}
	}
	reset_edges();
}

/** Prints the vertices, their edges, the relation table, and also notifies if
 * any memory errors are visible. */
void voronoicell_base::print_edges() {
	int j;
	double *ptsp=pts;
	for(int i=0;i<p;i++,ptsp+=3) {
		printf("%d %d  ",i,nu[i]);
		for(j=0;j<nu[i];j++) printf(" %d",ed[i][j]);
		printf("  ");
		while(j<(nu[i]<<1)) printf(" %d",ed[i][j]);
		printf("   %d",ed[i][j]);
		print_edges_neighbors(i);
		printf("  %g %g %g %p",*ptsp,ptsp[1],ptsp[2],(void*) ed[i]);
		if(ed[i]>=mep[nu[i]]+mec[nu[i]]*((nu[i]<<1)+1)) puts(" Memory error");
		else puts("");
	}
}

/** This prints out the neighbor information for vertex i. */
void voronoicell_neighbor::print_edges_neighbors(int i) {
	if(nu[i]>0) {
		int j=0;
		printf("     (");
		while(j<nu[i]-1) printf("%d,",ne[i][j++]);
		printf("%d)",ne[i][j]);
	} else printf("     ()");
}

// Explicit instantiation
template bool voronoicell_base::nplane(voronoicell&,double,double,double,double,int);
template bool voronoicell_base::nplane(voronoicell_neighbor&,double,double,double,double,int);
template void voronoicell_base::check_memory_for_copy(voronoicell&,voronoicell_base*);
template void voronoicell_base::check_memory_for_copy(voronoicell_neighbor&,voronoicell_base*);

}
PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS