// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file cell.hh
 * \brief Header file for the voronoicell and related classes. */

#ifndef VOROPP_CELL_HH
#define VOROPP_CELL_HH

#include <vector>
#include "CoreMinimal.h"

#include "config.hh"
#include "common.hh"

namespace voro {

/** \brief A class representing a single Voronoi cell.
 *
 * This class represents a single Voronoi cell, as a collection of vertices
 * that are connected by edges. The class contains routines for initializing
 * the Voronoi cell to be simple shapes such as a box, tetrahedron, or octahedron.
 * It the contains routines for recomputing the cell based on cutting it
 * by a plane, which forms the key routine for the Voronoi cell computation.
 * It contains numerous routine for computing statistics about the Voronoi cell,
 * and it can output the cell in several formats.
 *
 * This class is not intended for direct use, but forms the base of the
 * voronoicell and voronoicell_neighbor classes, which extend it based on
 * whether neighboring particle ID information needs to be tracked. */
class voronoicell_base {
	public:
		/** This holds the current size of the arrays ed and nu, which
		 * hold the vertex information. If more vertices are created
		 * than can fit in this array, then it is dynamically extended
		 * using the add_memory_vertices routine. */
		int current_vertices;
		/** This holds the current maximum allowed order of a vertex,
		 * which sets the size of the mem, mep, and mec arrays. If a
		 * vertex is created with more vertices than this, the arrays
		 * are dynamically extended using the add_memory_vorder routine.
		 */
		int current_vertex_order;
		/** This sets the size of the main delete stack. */
		int current_delete_size;
		/** This sets the size of the auxiliary delete stack. */
		int current_delete2_size;
		/** This sets the total number of vertices in the current cell.
		 */
		int p;
		/** This is the index of particular point in the cell, which is
		 * used to start the tracing routines for plane intersection
		 * and cutting. These routines will work starting from any
		 * point, but it's often most efficient to start from the last
		 * point considered, since in many cases, the cell construction
		 * algorithm may consider many planes with similar vectors
		 * concurrently. */
		int up;
		/** This is a two dimensional array that holds information
		 * about the edge connections of the vertices that make up the
		 * cell. The two dimensional array is not allocated in the
		 * usual method. To account for the fact the different vertices
		 * have different orders, and thus require different amounts of
		 * storage, the elements of ed[i] point to one-dimensional
		 * arrays in the mep[] array of different sizes.
		 *
		 * More specifically, if vertex i has order m, then ed[i]
		 * points to a one-dimensional array in mep[m] that has 2*m+1
		 * entries. The first m elements hold the neighboring edges, so
		 * that the jth edge of vertex i is held in ed[i][j]. The next
		 * m elements hold a table of relations which is redundant but
		 * helps speed up the computation. It satisfies the relation
		 * ed[ed[i][j]][ed[i][m+j]]=i. The final entry holds a back
		 * pointer, so that ed[i+2*m]=i. The back pointers are used
		 * when rearranging the memory. */
		int **ed;
		/** This array holds the order of the vertices in the Voronoi
		 * cell. This array is dynamically allocated, with its current
		 * size held by current_vertices. */
		int *nu;
		/** This in an array with size 3*current_vertices for holding
		 * the positions of the vertices. */
		double *pts;
		voronoicell_base();
		virtual ~voronoicell_base();
		void init_base(double xmin,double xmax,double ymin,double ymax,double zmin,double zmax);
		void init_octahedron_base(double l);
		void init_tetrahedron_base(double x0,double y0,double z0,double x1,double y1,double z1,double x2,double y2,double z2,double x3,double y3,double z3);
		void translate(double x,double y,double z);
		double volume();
		double max_radius_squared();
		double total_edge_distance();
		double surface_area();
		void centroid(double &cx,double &cy,double &cz);
		int number_of_faces();
		int number_of_edges();
		void vertex_orders(std::vector<int> &v);
		void vertices(std::vector<double> &v);
		void vertices(double x,double y,double z,std::vector<double> &v);
		void face_areas(std::vector<double> &v);
		void face_orders(std::vector<int> &v);
		void face_freq_table(std::vector<int> &v);
		void face_vertices(std::vector<int> &v);
		void face_perimeters(std::vector<double> &v);
		void normals(std::vector<double> &v);
		template<class vc_class>
		bool nplane(vc_class &vc,double x,double y,double z,double rsq,int p_id);
		bool plane_intersects(double x,double y,double z,double rsq);
		bool plane_intersects_guess(double x,double y,double z,double rsq);
		void construct_relations();
		void check_relations();
		void check_duplicates();
		void print_edges();
		/** Returns a list of IDs of neighboring particles
		 * corresponding to each face.
		 * \param[out] v a reference to a vector in which to return the
		 *               results. If no neighbor information is
		 *               available, a blank vector is returned. */
		virtual void neighbors(std::vector<int> &v) {v.clear();}
		/** This a virtual function that is overridden by a routine to
		 * print the neighboring particle IDs for a given vertex. By
		 * default, when no neighbor information is available, the
		 * routine does nothing.
		 * \param[in] i the vertex to consider. */
		virtual void print_edges_neighbors(int i) {};
		/** This is a simple inline function for picking out the index
		 * of the next edge counterclockwise at the current vertex.
		 * \param[in] a the index of an edge of the current vertex.
		 * \param[in] p the number of the vertex.
		 * \return 0 if a=nu[p]-1, or a+1 otherwise. */
		inline int cycle_up(int a,int vertexIndex) {return a==nu[vertexIndex]-1?0:a+1;}
		/** This is a simple inline function for picking out the index
		 * of the next edge clockwise from the current vertex.
		 * \param[in] a the index of an edge of the current vertex.
		 * \param[in] p the number of the vertex.
		 * \return nu[p]-1 if a=0, or a-1 otherwise. */
		inline int cycle_down(int a,int vertexIndex) {return a==0?nu[vertexIndex]-1:a-1;}
	protected:
		/** This a one dimensional array that holds the current sizes
		 * of the memory allocations for them mep array.*/
		int *mem;
		/** This is a one dimensional array that holds the current
		 * number of vertices of order p that are stored in the mep[p]
		 * array. */
		int *mec;
		/** This is a two dimensional array for holding the information
		 * about the edges of the Voronoi cell. mep[p] is a
		 * one-dimensional array for holding the edge information about
		 * all vertices of order p, with each vertex holding 2*p+1
		 * integers of information. The total number of vertices held
		 * on mep[p] is stored in mem[p]. If the space runs out, the
		 * code allocates more using the add_memory() routine. */
		int **mep;
		inline void reset_edges();
		template<class vc_class>
		void check_memory_for_copy(vc_class &vc,voronoicell_base* vb);
		void copy(voronoicell_base* vb);
	private:
		/** This is the delete stack, used to store the vertices which
		 * are going to be deleted during the plane cutting procedure.
		 */
		int *ds,*stacke;
		/** This is the auxiliary delete stack, which has size set by
		 * current_delete2_size. */
		int *ds2,*stacke2;
		/** This stores the current memory allocation for the marginal
		 * cases. */
		int current_marginal;
		/** This stores the total number of marginal points which are
		 * currently in the buffer. */
		int n_marg;
		/** This array contains a list of the marginal points, and also
		 * the outcomes of the marginal tests. */
		int *marg;
		/** The x coordinate of the normal vector to the test plane. */
		double px;
		/** The y coordinate of the normal vector to the test plane. */
		double py;
		/** The z coordinate of the normal vector to the test plane. */
		double pz;
		/** The magnitude of the normal vector to the test plane. */
		double prsq;
		template<class vc_class>
		void add_memory(vc_class &vc,int i,int *stackp2);
		template<class vc_class>
		void add_memory_vertices(vc_class &vc);
		template<class vc_class>
		void add_memory_vorder(vc_class &vc);
		void add_memory_ds(int *&stackp);
		void add_memory_ds2(int *&stackp2);
		template<class vc_class>
		inline bool collapse_order1(vc_class &vc);
		template<class vc_class>
		inline bool collapse_order2(vc_class &vc);
		template<class vc_class>
		inline bool delete_connection(vc_class &vc,int j,int k,bool hand);
		template<class vc_class>
		inline bool search_for_outside_edge(vc_class &vc,int &upRef);
		template<class vc_class>
		inline void add_to_stack(vc_class &vc,int lp,int *&stackp2);
		inline bool plane_intersects_track(double x,double y,double z,double rs,double g);
		inline void normals_search(std::vector<double> &v,int i,int j,int k);
		inline bool search_edge(int l,int &m,int &k);
		inline int m_test(int n,double &ans);
		int check_marginal(int n,double &ans);
		friend class voronoicell;
		friend class voronoicell_neighbor;
};

/** \brief Extension of the voronoicell_base class to represent a Voronoi
 * cell without neighbor information.
 *
 * This class is an extension of the voronoicell_base class, in cases when
 * is not necessary to track the IDs of neighboring particles associated
 * with each face of the Voronoi cell. */
class voronoicell : public voronoicell_base {
	public:
		using voronoicell_base::nplane;
		/** Copies the information from another voronoicell class into
		 * this class, extending memory allocation if necessary.
		 * \param[in] c the class to copy. */
		inline void operator=(voronoicell &c) {
			voronoicell_base* vb((voronoicell_base*) &c);
			check_memory_for_copy(*this,vb);copy(vb);
		}
		/** Cuts a Voronoi cell using by the plane corresponding to the
		 * perpendicular bisector of a particle.
		 * \param[in] (x,y,z) the position of the particle.
		 * \param[in] rsq the modulus squared of the vector.
		 * \param[in] p_id the plane ID, ignored for this case where no
		 *                 neighbor tracking is enabled.
		 * \return False if the plane cut deleted the cell entirely,
		 *         true otherwise. */
		inline bool nplane(double x,double y,double z,double rsq,int p_id) {
			return nplane(*this,x,y,z,rsq,0);
		}
		/** Cuts a Voronoi cell using by the plane corresponding to the
		 * perpendicular bisector of a particle.
		 * \param[in] (x,y,z) the position of the particle.
		 * \param[in] p_id the plane ID, ignored for this case where no
		 *                 neighbor tracking is enabled.
		 * \return False if the plane cut deleted the cell entirely,
		 *         true otherwise. */
		inline bool nplane(double x,double y,double z,int p_id) {
			double rsq=x*x+y*y+z*z;
			return nplane(*this,x,y,z,rsq,0);
		}
		/** Cuts a Voronoi cell using by the plane corresponding to the
		 * perpendicular bisector of a particle.
		 * \param[in] (x,y,z) the position of the particle.
		 * \param[in] rsq the modulus squared of the vector.
		 * \return False if the plane cut deleted the cell entirely,
		 *         true otherwise. */
		inline bool plane(double x,double y,double z,double rsq) {
			return nplane(*this,x,y,z,rsq,0);
		}
		/** Cuts a Voronoi cell using by the plane corresponding to the
		 * perpendicular bisector of a particle.
		 * \param[in] (x,y,z) the position of the particle.
		 * \return False if the plane cut deleted the cell entirely,
		 *         true otherwise. */
		inline bool plane(double x,double y,double z) {
			double rsq=x*x+y*y+z*z;
			return nplane(*this,x,y,z,rsq,0);
		}
		/** Initializes the Voronoi cell to be rectangular box with the
		 * given dimensions.
		 * \param[in] (xmin,xmax) the minimum and maximum x coordinates.
		 * \param[in] (ymin,ymax) the minimum and maximum y coordinates.
		 * \param[in] (zmin,zmax) the minimum and maximum z coordinates. */
		inline void init(double xmin,double xmax,double ymin,double ymax,double zmin,double zmax) {
			init_base(xmin,xmax,ymin,ymax,zmin,zmax);
		}
		/** Initializes the cell to be an octahedron with vertices at
		 * (l,0,0), (-l,0,0), (0,l,0), (0,-l,0), (0,0,l), and (0,0,-l).
		 * \param[in] l a parameter setting the size of the octahedron.
		 */
		inline void init_octahedron(double l) {
			init_octahedron_base(l);
		}
		/** Initializes the cell to be a tetrahedron.
		 * \param[in] (x0,y0,z0) the coordinates of the first vertex.
		 * \param[in] (x1,y1,z1) the coordinates of the second vertex.
		 * \param[in] (x2,y2,z2) the coordinates of the third vertex.
		 * \param[in] (x3,y3,z3) the coordinates of the fourth vertex.
		 */
		inline void init_tetrahedron(double x0,double y0,double z0,double x1,double y1,double z1,double x2,double y2,double z2,double x3,double y3,double z3) {
			init_tetrahedron_base(x0,y0,z0,x1,y1,z1,x2,y2,z2,x3,y3,z3);
		}
		// extract cell info in to Unreal arrays
		void extractCellInfo(const FVector& CellPosition, TArray<FVector>& Vertices, TArray<int32>& FaceVertexIndices);
	private:
		inline void n_allocate(int i,int m) {};
		inline void n_add_memory_vertices(int i) {};
		inline void n_add_memory_vorder(int i) {};
		inline void n_set_pointer(int neIndex,int n) {};
		inline void n_copy(int a,int b,int c,int d) {};
		inline void n_set(int a,int b,int c) {};
		inline void n_set_aux1(int k) {};
		inline void n_copy_aux1(int a,int b) {};
		inline void n_copy_aux1_shift(int a,int b) {};
		inline void n_set_aux2_copy(int a,int b) {};
		inline void n_copy_pointer(int a,int b) {};
		inline void n_set_to_aux1(int j) {};
		inline void n_set_to_aux2(int j) {};
		inline void n_allocate_aux1(int i) {};
		inline void n_switch_to_aux1(int i) {};
		inline void n_copy_to_aux1(int i,int m) {};
		inline void n_set_to_aux1_offset(int k,int m) {};
		inline void n_neighbors(std::vector<int> &v) {v.clear();};
		friend class voronoicell_base;
};

/** \brief Extension of the voronoicell_base class to represent a Voronoi cell
 * with neighbor information.
 *
 * This class is an extension of the voronoicell_base class, in cases when the
 * IDs of neighboring particles associated with each face of the Voronoi cell.
 * It contains additional data structures mne and ne for storing this
 * information. */
class voronoicell_neighbor : public voronoicell_base {
	public:
		using voronoicell_base::nplane;
		/** This two dimensional array holds the neighbor information
		 * associated with each vertex. mne[p] is a one dimensional
		 * array which holds all of the neighbor information for
		 * vertices of order p. */
		int **mne;
		/** This is a two dimensional array that holds the neighbor
		 * information associated with each vertex. ne[i] points to a
		 * one-dimensional array in mne[nu[i]]. ne[i][j] holds the
		 * neighbor information associated with the jth edge of vertex
		 * i. It is set to the ID number of the plane that made the
		 * face that is clockwise from the jth edge. */
		int **ne;
		voronoicell_neighbor();
		virtual ~voronoicell_neighbor();
		void operator=(voronoicell &c);
		void operator=(voronoicell_neighbor &c);
		/** Cuts the Voronoi cell by a particle whose center is at a
		 * separation of (x,y,z) from the cell center. The value of rsq
		 * should be initially set to \f$x^2+y^2+z^2\f$.
		 * \param[in] (x,y,z) the normal vector to the plane.
		 * \param[in] rsq the distance along this vector of the plane.
		 * \param[in] p_id the plane ID (for neighbor tracking only).
		 * \return False if the plane cut deleted the cell entirely,
		 * true otherwise. */
		inline bool nplane(double x,double y,double z,double rsq,int p_id) {
			return nplane(*this,x,y,z,rsq,p_id);
		}
		/** This routine calculates the modulus squared of the vector
		 * before passing it to the main nplane() routine with full
		 * arguments.
		 * \param[in] (x,y,z) the vector to cut the cell by.
		 * \param[in] p_id the plane ID (for neighbor tracking only).
		 * \return False if the plane cut deleted the cell entirely,
		 *         true otherwise. */
		inline bool nplane(double x,double y,double z,int p_id) {
			double rsq=x*x+y*y+z*z;
			return nplane(*this,x,y,z,rsq,p_id);
		}
		/** This version of the plane routine just makes up the plane
		 * ID to be zero. It will only be referenced if neighbor
		 * tracking is enabled.
		 * \param[in] (x,y,z) the vector to cut the cell by.
		 * \param[in] rsq the modulus squared of the vector.
		 * \return False if the plane cut deleted the cell entirely,
		 *         true otherwise. */
		inline bool plane(double x,double y,double z,double rsq) {
			return nplane(*this,x,y,z,rsq,0);
		}
		/** Cuts a Voronoi cell using the influence of a particle at
		 * (x,y,z), first calculating the modulus squared of this
		 * vector before passing it to the main nplane() routine. Zero
		 * is supplied as the plane ID, which will be ignored unless
		 * neighbor tracking is enabled.
		 * \param[in] (x,y,z) the vector to cut the cell by.
		 * \return False if the plane cut deleted the cell entirely,
		 *         true otherwise. */
		inline bool plane(double x,double y,double z) {
			double rsq=x*x+y*y+z*z;
			return nplane(*this,x,y,z,rsq,0);
		}
		void init(double xmin,double xmax,double ymin,double ymax,double zmin,double zmax);
		void init_octahedron(double l);
		void init_tetrahedron(double x0,double y0,double z0,double x1,double y1,double z1,double x2,double y2,double z2,double x3,double y3,double z3);
		void check_facets();
		virtual void neighbors(std::vector<int> &v);
		void neighborsTArray(TArray<int> &nbrs, bool excludeBounds);
		void extractCellInfo(const FVector& CellPosition, TArray<FVector> &Vertices, TArray<int32> &FaceVertexIndices, TArray<int32> &Nbrs, TArray<FVector> &Normals);
		virtual void print_edges_neighbors(int i);
	private:
		int *paux1;
		int *paux2;
		inline void n_allocate(int i,int m) {mne[i]=new int[m*i];}
		inline void n_add_memory_vertices(int i) {
			int **pp=new int*[i];
			check(current_vertices <= i);
			for(int j=0;j<current_vertices;j++) pp[j]=ne[j];
			delete [] ne;ne=pp;
		}
		inline void n_add_memory_vorder(int i) {
			int **p2=new int*[i];
			check(current_vertex_order <= i);
			for(int j=0;j<current_vertex_order;j++) p2[j]=mne[j];
			delete [] mne;mne=p2;
		}
		inline void n_set_pointer(int neIndex,int n) {
			ne[neIndex]=mne[n]+n*mec[n];
		}
		inline void n_copy(int a,int b,int c,int d) {ne[a][b]=ne[c][d];}
		inline void n_set(int a,int b,int c) {ne[a][b]=c;}
		inline void n_set_aux1(int k) {paux1=mne[k]+k*mec[k];}
		inline void n_copy_aux1(int a,int b) {paux1[b]=ne[a][b];}
		inline void n_copy_aux1_shift(int a,int b) {paux1[b]=ne[a][b+1];}
		inline void n_set_aux2_copy(int a,int b) {
			paux2=mne[b]+b*mec[b];
			for(int i=0;i<b;i++) ne[a][i]=paux2[i];
		}
		inline void n_copy_pointer(int a,int b) {ne[a]=ne[b];}
		inline void n_set_to_aux1(int j) {ne[j]=paux1;}
		inline void n_set_to_aux2(int j) {ne[j]=paux2;}
		inline void n_allocate_aux1(int i) {paux1=new int[i*mem[i]];}
		inline void n_switch_to_aux1(int i) {delete [] mne[i];mne[i]=paux1;}
		inline void n_copy_to_aux1(int i,int m) {paux1[m]=mne[i][m];}
		inline void n_set_to_aux1_offset(int k,int m) {ne[k]=paux1+m;}
		friend class voronoicell_base;
};

}

#endif
