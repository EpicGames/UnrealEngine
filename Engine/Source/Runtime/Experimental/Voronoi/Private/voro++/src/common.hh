// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file common.hh
 * \brief Header file for the small helper functions. */

#ifndef VOROPP_COMMON_HH
#define VOROPP_COMMON_HH

#include <cstdio>
#include <cstdlib>
#include <vector>

#include "config.hh"

#include "Containers/StringConv.h"

namespace voro {

/** \brief Function for printing fatal error messages and exiting.
 *
 * Function for printing fatal error messages and exiting.
 * \param[in] p a pointer to the message to print.
 * \param[in] status the status code to return with. */
inline void voro_fatal_error(const char *p,int status) {
	checkf(false, TEXT("voro++ failure: %s (status code: %d)"), ANSI_TO_TCHAR(p), status);
}

}

#endif
