// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file config.hh
 * \brief Master configuration file for setting various compile-time options. */

#ifndef VOROPP_CONFIG_HH
#define VOROPP_CONFIG_HH

namespace voro {

// These constants set the initial memory allocation for the Voronoi cell
/** The initial memory allocation for the number of vertices. */
const int init_vertices=256;
/** The initial memory allocation for the maximum vertex order. */
const int init_vertex_order=64;
/** The initial memory allocation for the number of regular vertices of order
 * 3. */
const int init_3_vertices=256;
/** The initial memory allocation for the number of vertices of higher order.
 */
const int init_n_vertices=8;
/** The initial buffer size for marginal cases used by the suretest class. */
const int init_marginal=64;
/** The initial size for the delete stack. */
const int init_delete_size=256;
/** The initial size for the auxiliary delete stack. */
const int init_delete2_size=256;
/** The initial size for the wall pointer array. */
const int init_wall_size=32;
/** The default initial size for the ordering class. */
const int init_ordering_size=4096;
/** The initial size of the pre_container chunk index. */
const int init_chunk_size=256;

// If the initial memory is too small, the program dynamically allocates more.
// However, if the limits below are reached, then the program bails out.
/** The maximum memory allocation for the number of vertices. */
const int max_vertices=16777216;
/** The maximum memory allocation for the maximum vertex order. */
const int max_vertex_order=16777216;
/** The maximum memory allocation for the any particular order of vertex. */
const int max_n_vertices=16777216;
/** The maximum buffer size for marginal cases used by the suretest class. */
const int max_marginal=16777216;
/** The maximum size for the delete stack. */
const int max_delete_size=16777216;
/** The maximum size for the auxiliary delete stack. */
const int max_delete2_size=16777216;
/** The maximum amount of particle memory allocated for a single region. */
const int max_particle_memory=16777216;
/** The maximum size for the wall pointer array. */
const int max_wall_size=2048;
/** The maximum size for the ordering class. */
const int max_ordering_size=67108864;
/** The maximum size for the pre_container chunk index. */
const int max_chunk_size=65536;

/** The chunk size in the pre_container classes. */
const int pre_container_chunk_size=1024;

#ifndef VOROPP_VERBOSE
/** Voro++ can print a number of different status and debugging messages to
 * notify the user of special behavior, and this macro sets the amount which
 * are displayed. At level 0, no messages are printed. At level 1, messages
 * about unusual cases during cell construction are printed, such as when the
 * plane routine bails out due to floating point problems. At level 2, general
 * messages about memory expansion are printed. At level 3, technical details
 * about memory management are printed. */
#define VOROPP_VERBOSE 0
#endif

/** If a point is within this distance of a cutting plane, then the code
 * assumes that point exactly lies on the plane. */
const double tolerance=1e-11;

/** If a point is within this distance of a cutting plane, then the code stores
 * whether this point is inside, outside, or exactly on the cutting plane in
 * the marginal cases buffer, to prevent the test giving a different result on
 * a subsequent evaluation due to floating point rounding errors. */
const double tolerance2=2e-11;

/** The square of the tolerance, used when deciding whether some squared
 * quantities are large enough to be used. */
const double tolerance_sq=tolerance*tolerance;

/** A large number that is used in the computation. */
const double large_number=1e30;

/** A radius to use as a placeholder when no other information is available. */
const double default_radius=0.5;

/** The maximum number of shells of periodic images to test over. */
const int max_unit_voro_shells=10;

/** A guess for the optimal number of particles per block, used to set up the
 * container grid. */
const double optimal_particles=5.6;

/** If this is set to 1, then the code reports any instances of particles being
 * put outside of the container geometry. */
#define VOROPP_REPORT_OUT_OF_BOUNDS 0

/** Voro++ returns this status code if there is a file-related error, such as
 * not being able to open file. */
#define VOROPP_FILE_ERROR 1

/** Voro++ returns this status code if there is a memory allocation error, if
 * one of the safe memory limits is exceeded. */
#define VOROPP_MEMORY_ERROR 2

/** Voro++ returns this status code if there is any type of internal error, if
 * it detects that representation of the Voronoi cell is inconsistent. This
 * status code will generally indicate a bug, and the developer should be
 * contacted. */
#define VOROPP_INTERNAL_ERROR 3

/** Voro++ returns this status code if it could not interpret the command line
 * arguments passed to the command line utility. */
#define VOROPP_CMD_LINE_ERROR 4

}

#endif
