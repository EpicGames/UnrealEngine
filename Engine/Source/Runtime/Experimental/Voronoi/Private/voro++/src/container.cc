// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file container.cc
 * \brief Function implementations for the container and related classes. */

#include "container.hh"

namespace voro {

/** The class constructor sets up the geometry of container, initializing the
 * minimum and maximum coordinates in each direction, and setting whether each
 * direction is periodic or not. It divides the container into a rectangular
 * grid of blocks, and allocates memory for each of these for storing particle
 * positions and IDs.
 * \param[in] (ax_,bx_) the minimum and maximum x coordinates.
 * \param[in] (ay_,by_) the minimum and maximum y coordinates.
 * \param[in] (az_,bz_) the minimum and maximum z coordinates.
 * \param[in] (nx_,ny_,nz_) the number of grid blocks in each of the three
 *			    coordinate directions.
 * \param[in] (xperiodic_,yperiodic_,zperiodic_) flags setting whether the
 *                                               container is periodic in each
 *                                               coordinate direction.
 * \param[in] init_mem the initial memory allocation for each block.
 * \param[in] ps_ the number of floating point entries to store for each
 *                particle. */
container_base::container_base(double ax_,double bx_,double ay_,double by_,double az_,double bz_,
		int nx_,int ny_,int nz_,bool xperiodic_,bool yperiodic_,bool zperiodic_,int init_mem,int ps_)
	: voro_base(nx_,ny_,nz_,(bx_-ax_)/nx_,(by_-ay_)/ny_,(bz_-az_)/nz_),
	ax(ax_), bx(bx_), ay(ay_), by(by_), az(az_), bz(bz_),
	xperiodic(xperiodic_), yperiodic(yperiodic_), zperiodic(zperiodic_),
	id(new int*[nxyz]), p(new double*[nxyz]), co(new int[nxyz]), mem(new int[nxyz]), ps(ps_) {
	int l;
	for(l=0;l<nxyz;l++) co[l]=0;
	for(l=0;l<nxyz;l++) mem[l]=init_mem;
	for(l=0;l<nxyz;l++) id[l]=new int[init_mem];
	for(l=0;l<nxyz;l++) p[l]=new double[ps*init_mem];
}

/** The container destructor frees the dynamically allocated memory. */
container_base::~container_base() {
	int l;
	if (p)
	{
		for (l = 0; l < nxyz; l++) delete[] p[l];
		delete[] p;
	}
	if (id)
	{
		for (l = 0; l < nxyz; l++) delete[] id[l];
		delete[] id;
	}
	delete [] co;
	delete [] mem;
}

voro_compute<container> container::make_compute() const
{
	return voro_compute<container>((container&)*this, xperiodic ? 2 * nx + 1 : nx, yperiodic ? 2 * ny + 1 : ny, zperiodic ? 2 * nz + 1 : nz);
}


voro_compute<container_poly> container_poly::make_compute() const
{
	return voro_compute<container_poly>((container_poly&)*this, xperiodic ? 2 * nx + 1 : nx, yperiodic ? 2 * ny + 1 : ny, zperiodic ? 2 * nz + 1 : nz);
}

/** The class constructor sets up the geometry of container.
 * \param[in] (ax_,bx_) the minimum and maximum x coordinates.
 * \param[in] (ay_,by_) the minimum and maximum y coordinates.
 * \param[in] (az_,bz_) the minimum and maximum z coordinates.
 * \param[in] (nx_,ny_,nz_) the number of grid blocks in each of the three
 *                       coordinate directions.
 * \param[in] (xperiodic_,yperiodic_,zperiodic_) flags setting whether the
 *                                               container is periodic in each
 *                                               coordinate direction.
 * \param[in] init_mem the initial memory allocation for each block. */
container::container(double ax_,double bx_,double ay_,double by_,double az_,double bz_,
	int nx_,int ny_,int nz_,bool xperiodic_,bool yperiodic_,bool zperiodic_,int init_mem)
	: container_base(ax_,bx_,ay_,by_,az_,bz_,nx_,ny_,nz_,xperiodic_,yperiodic_,zperiodic_,init_mem,3) {}

/** The class constructor sets up the geometry of container.
 * \param[in] (ax_,bx_) the minimum and maximum x coordinates.
 * \param[in] (ay_,by_) the minimum and maximum y coordinates.
 * \param[in] (az_,bz_) the minimum and maximum z coordinates.
 * \param[in] (nx_,ny_,nz_) the number of grid blocks in each of the three
 *                       coordinate directions.
 * \param[in] (xperiodic_,yperiodic_,zperiodic_) flags setting whether the
 *                                               container is periodic in each
 *                                               coordinate direction.
 * \param[in] init_mem the initial memory allocation for each block. */
container_poly::container_poly(double ax_,double bx_,double ay_,double by_,double az_,double bz_,
	int nx_,int ny_,int nz_,bool xperiodic_,bool yperiodic_,bool zperiodic_,int init_mem)
	: container_base(ax_,bx_,ay_,by_,az_,bz_,nx_,ny_,nz_,xperiodic_,yperiodic_,zperiodic_,init_mem,4) {ppr=p;}

/** Put a particle into the correct region of the container.
 * \param[in] n the numerical ID of the inserted particle.
 * \param[in] (x,y,z) the position vector of the inserted particle. */
void container::put(int n,double x,double y,double z) {
	int ijk;
	if(put_locate_block(ijk,x,y,z)) {
		id[ijk][co[ijk]]=n;
		double *pp=p[ijk]+3*co[ijk]++;
		*(pp++)=x;*(pp++)=y;*pp=z;
	}
}

/** Put a particle into the correct region of the container.
 * \param[in] n the numerical ID of the inserted particle.
 * \param[in] (x,y,z) the position vector of the inserted particle.
 * \param[in] r the radius of the particle. */
void container_poly::put(int n,double x,double y,double z,double r) {
	int ijk;
	if(put_locate_block(ijk,x,y,z)) {
		id[ijk][co[ijk]]=n;
		double *pp=p[ijk]+4*co[ijk]++;
		*(pp++)=x;*(pp++)=y;*(pp++)=z;*pp=r;
		if(max_radius<r) max_radius=r;
	}
}

/** Put a particle into the correct region of the container, also recording
 * into which region it was stored.
 * \param[in] vo the ordering class in which to record the region.
 * \param[in] n the numerical ID of the inserted particle.
 * \param[in] (x,y,z) the position vector of the inserted particle. */
void container::put(particle_order &vo,int n,double x,double y,double z) {
	int ijk;
	if(put_locate_block(ijk,x,y,z)) {
		id[ijk][co[ijk]]=n;
		vo.add(ijk,co[ijk]);
		double *pp=p[ijk]+3*co[ijk]++;
		*(pp++)=x;*(pp++)=y;*pp=z;
	}
}

/** Put a particle into the correct region of the container, also recording
 * into which region it was stored.
 * \param[in] vo the ordering class in which to record the region.
 * \param[in] n the numerical ID of the inserted particle.
 * \param[in] (x,y,z) the position vector of the inserted particle.
 * \param[in] r the radius of the particle. */
void container_poly::put(particle_order &vo,int n,double x,double y,double z,double r) {
	int ijk;
	if(put_locate_block(ijk,x,y,z)) {
		id[ijk][co[ijk]]=n;
		vo.add(ijk,co[ijk]);
		double *pp=p[ijk]+4*co[ijk]++;
		*(pp++)=x;*(pp++)=y;*(pp++)=z;*pp=r;
		if(max_radius<r) max_radius=r;
	}
}

/** This routine takes a particle position vector, tries to remap it into the
 * primary domain. If successful, it computes the region into which it can be
 * stored and checks that there is enough memory within this region to store
 * it.
 * \param[out] ijk the region index.
 * \param[in,out] (x,y,z) the particle position, remapped into the primary
 *                        domain if necessary.
 * \return True if the particle can be successfully placed into the container,
 * false otherwise. */
bool container_base::put_locate_block(int &ijk,double &x,double &y,double &z) {
	if(put_remap(ijk,x,y,z)) {
		if(co[ijk]==mem[ijk]) add_particle_memory(ijk);
		return true;
	}
#if VOROPP_REPORT_OUT_OF_BOUNDS ==1
	fprintf(stderr,"Out of bounds: (x,y,z)=(%g,%g,%g)\n",x,y,z);
#endif
	return false;
}

/** Takes a particle position vector and computes the region index into which
 * it should be stored. If the container is periodic, then the routine also
 * maps the particle position to ensure it is in the primary domain. If the
 * container is not periodic, the routine bails out.
 * \param[out] ijk the region index.
 * \param[in,out] (x,y,z) the particle position, remapped into the primary
 *                        domain if necessary.
 * \return True if the particle can be successfully placed into the container,
 * false otherwise. */
inline bool container_base::put_remap(int &ijk,double &x,double &y,double &z) const {
	int l;

	ijk=step_int((x-ax)*xsp);
	if(xperiodic) {l=step_mod(ijk,nx);x+=boxx*(l-ijk);ijk=l;}
	else if(ijk<0||ijk>=nx) return false;

	int j=step_int((y-ay)*ysp);
	if(yperiodic) {l=step_mod(j,ny);y+=boxy*(l-j);j=l;}
	else if(j<0||j>=ny) return false;

	int k=step_int((z-az)*zsp);
	if(zperiodic) {l=step_mod(k,nz);z+=boxz*(l-k);k=l;}
	else if(k<0||k>=nz) return false;

	ijk+=nx*j+nxy*k;
	return true;
}

/** Takes a position vector and attempts to remap it into the primary domain.
 * \param[out] (ai,aj,ak) the periodic image displacement that the vector is in,
 *                       with (0,0,0) corresponding to the primary domain.
 * \param[out] (ci,cj,ck) the index of the block that the position vector is
 *                        within, once it has been remapped.
 * \param[in,out] (x,y,z) the position vector to consider, which is remapped
 *                        into the primary domain during the routine.
 * \param[out] ijk the block index that the vector is within.
 * \return True if the particle is within the container or can be remapped into
 * it, false if it lies outside of the container bounds. */
inline bool container_base::remap(int &ai,int &aj,int &ak,int &ci,int &cj,int &ck,double &x,double &y,double &z,int &ijk) const {
	ci=step_int((x-ax)*xsp);
	if(ci<0||ci>=nx) {
		if(xperiodic) {ai=step_div(ci,nx);x-=ai*(bx-ax);ci-=ai*nx;}
		else return false;
	} else ai=0;

	cj=step_int((y-ay)*ysp);
	if(cj<0||cj>=ny) {
		if(yperiodic) {aj=step_div(cj,ny);y-=aj*(by-ay);cj-=aj*ny;}
		else return false;
	} else aj=0;

	ck=step_int((z-az)*zsp);
	if(ck<0||ck>=nz) {
		if(zperiodic) {ak=step_div(ck,nz);z-=ak*(bz-az);ck-=ak*nz;}
		else return false;
	} else ak=0;

	ijk=ci+nx*cj+nxy*ck;
	return true;
}

/** Takes a vector and finds the particle whose Voronoi cell contains that
 * vector. This is equivalent to finding the particle which is nearest to the
 * vector. Additional wall classes are not considered by this routine.
 * \param[in] (x,y,z) the vector to test.
 * \param[out] (rx,ry,rz) the position of the particle whose Voronoi cell
 *                        contains the vector. If the container is periodic,
 *                        this may point to a particle in a periodic image of
 *                        the primary domain.
 * \param[out] pid the ID of the particle.
 * \return True if a particle was found. If the container has no particles,
 * then the search will not find a Voronoi cell and false is returned. */
bool container::find_voronoi_cell(double x,double y,double z,double &rx,double &ry,double &rz,int &pid,voro_compute<container>& vc) const {
	int ai,aj,ak,ci,cj,ck,ijk;
	particle_record w;
	double mrs;

	// If the given vector lies outside the domain, but the container
	// is periodic, then remap it back into the domain
	if(!remap(ai,aj,ak,ci,cj,ck,x,y,z,ijk)) return false;
	vc.find_voronoi_cell(x,y,z,ci,cj,ck,ijk,w,mrs);

	if(w.ijk!=-1) {

		// Assemble the position vector of the particle to be returned,
		// applying a periodic remapping if necessary
		if(xperiodic) {ci+=w.di;if(ci<0||ci>=nx) ai+=step_div(ci,nx);}
		if(yperiodic) {cj+=w.dj;if(cj<0||cj>=ny) aj+=step_div(cj,ny);}
		if(zperiodic) {ck+=w.dk;if(ck<0||ck>=nz) ak+=step_div(ck,nz);}
		rx=p[w.ijk][3*w.l]+ai*(bx-ax);
		ry=p[w.ijk][3*w.l+1]+aj*(by-ay);
		rz=p[w.ijk][3*w.l+2]+ak*(bz-az);
		pid=id[w.ijk][w.l];
		return true;
	}

	// If no particle if found then just return false
	return false;
}

/** Takes a vector and finds the particle whose Voronoi cell contains that
 * vector. Additional wall classes are not considered by this routine.
 * \param[in] (x,y,z) the vector to test.
 * \param[out] (rx,ry,rz) the position of the particle whose Voronoi cell
 *                        contains the vector. If the container is periodic,
 *                        this may point to a particle in a periodic image of
 *                        the primary domain.
 * \param[out] pid the ID of the particle.
 * \return True if a particle was found. If the container has no particles,
 * then the search will not find a Voronoi cell and false is returned. */
bool container_poly::find_voronoi_cell(double x,double y,double z,double &rx,double &ry,double &rz,int &pid,voro_compute<container>& vc) const {
	int ai,aj,ak,ci,cj,ck,ijk;
	particle_record w;
	double mrs;

	// If the given vector lies outside the domain, but the container
	// is periodic, then remap it back into the domain
	if(!remap(ai,aj,ak,ci,cj,ck,x,y,z,ijk)) return false;
	vc.find_voronoi_cell(x,y,z,ci,cj,ck,ijk,w,mrs);

	if(w.ijk!=-1) {

		// Assemble the position vector of the particle to be returned,
		// applying a periodic remapping if necessary
		if(xperiodic) {ci+=w.di;if(ci<0||ci>=nx) ai+=step_div(ci,nx);}
		if(yperiodic) {cj+=w.dj;if(cj<0||cj>=ny) aj+=step_div(cj,ny);}
		if(zperiodic) {ck+=w.dk;if(ck<0||ck>=nz) ak+=step_div(ck,nz);}
		rx=p[w.ijk][4*w.l]+ai*(bx-ax);
		ry=p[w.ijk][4*w.l+1]+aj*(by-ay);
		rz=p[w.ijk][4*w.l+2]+ak*(bz-az);
		pid=id[w.ijk][w.l];
		return true;
	}

	// If no particle if found then just return false
	return false;
}

/** Increase memory for a particular region.
 * \param[in] i the index of the region to reallocate. */
void container_base::add_particle_memory(int i) {
	int l,nmem=mem[i]<<1;

	// Carry out a check on the memory allocation size, and
	// print a status message if requested
	if(nmem>max_particle_memory)
		voro_fatal_error("Absolute maximum memory allocation exceeded",VOROPP_MEMORY_ERROR);
#if VOROPP_VERBOSE >=3
	fprintf(stderr,"Particle memory in region %d scaled up to %d\n",i,nmem);
#endif

	// Allocate new memory and copy in the contents of the old arrays
	int *idp=new int[nmem];
	for(l=0;l<co[i];l++) idp[l]=id[i][l];
	double *pp=new double[ps*nmem];
	for(l=0;l<ps*co[i];l++) pp[l]=p[i][l];

	// Update pointers and delete old arrays
	mem[i]=nmem;
	delete [] id[i];id[i]=idp;
	delete [] p[i];p[i]=pp;
}

/** Outputs the a list of all the container regions along with the number of
 * particles stored within each. */
void container_base::region_count() const {
	int i,j,k,*cop=co;
	for(k=0;k<nz;k++) for(j=0;j<ny;j++) for(i=0;i<nx;i++)
		printf("Region (%d,%d,%d): %d particles\n",i,j,k,*(cop++));
}

/** Clears a container of particles. */
void container::clear() {
	for(int *cop=co;cop<co+nxyz;cop++) *cop=0;
}

/** Clears a container of particles, also clearing resetting the maximum radius
 * to zero. */
void container_poly::clear() {
	for(int *cop=co;cop<co+nxyz;cop++) *cop=0;
	max_radius=0;
}

/** Computes all of the Voronoi cells in the container, but does nothing
 * with the output. It is useful for measuring the pure computation time
 * of the Voronoi algorithm, without any additional calculations such as
 * volume evaluation or cell output. */
void container::compute_all_cells(voro_compute<container>& vc) {
	voronoicell c;
	c_loop_all vl(*this);
	if(vl.start()) do compute_cell(c,vl,vc);
	while(vl.inc());
}

/** Computes all of the Voronoi cells in the container, but does nothing
 * with the output. It is useful for measuring the pure computation time
 * of the Voronoi algorithm, without any additional calculations such as
 * volume evaluation or cell output. */
void container_poly::compute_all_cells(voro_compute<container>& vc) {
	voronoicell c;
	c_loop_all vl(*this);
	if(vl.start()) do compute_cell(c,vl,vc);while(vl.inc());
}

/** Calculates all of the Voronoi cells and sums their volumes. In most cases
 * without walls, the sum of the Voronoi cell volumes should equal the volume
 * of the container to numerical precision.
 * \return The sum of all of the computed Voronoi volumes. */
double container::sum_cell_volumes(voro_compute<container>& vc) {
	voronoicell c;
	double vol=0;
	c_loop_all vl(*this);
	if(vl.start()) do if(compute_cell(c,vl,vc)) vol+=c.volume();while(vl.inc());
	return vol;
}

/** Calculates all of the Voronoi cells and sums their volumes. In most cases
 * without walls, the sum of the Voronoi cell volumes should equal the volume
 * of the container to numerical precision.
 * \return The sum of all of the computed Voronoi volumes. */
double container_poly::sum_cell_volumes(voro_compute<container>& vc) {
	voronoicell c;
	double vol=0;
	c_loop_all vl(*this);
	if(vl.start()) do if(compute_cell(c,vl,vc)) vol+=c.volume();while(vl.inc());
	return vol;
}

/** This function tests to see if a given vector lies within the container
 * bounds and any walls.
 * \param[in] (x,y,z) the position vector to be tested.
 * \return True if the point is inside the container, false if the point is
 *         outside. */
bool container_base::point_inside(double x,double y,double z) const {
	if(x<ax||x>bx||y<ay||y>by||z<az||z>bz) return false;
	return point_inside_walls(x,y,z);
}

/** The wall_list constructor sets up an array of pointers to wall classes. */
wall_list::wall_list() : walls(new wall*[init_wall_size]), wep(walls), wel(walls+init_wall_size),
	current_wall_size(init_wall_size) {}

/** The wall_list destructor frees the array of pointers to the wall classes.
 */
wall_list::~wall_list() {
	delete [] walls;
}

/** Adds all of the walls on another wall_list to this class.
 * \param[in] wl a reference to the wall class. */
void wall_list::add_wall(wall_list &wl) {
	for(wall **wp=wl.walls;wp<wl.wep;wp++) add_wall(*wp);
}

/** Deallocates all of the wall classes pointed to by the wall_list. */
void wall_list::deallocate() {
	for(wall **wp=walls;wp<wep;wp++) delete *wp;
}

/** Increases the memory allocation for the walls array. */
void wall_list::increase_wall_memory() {
	current_wall_size<<=1;
	if(current_wall_size>max_wall_size)
		voro_fatal_error("Wall memory allocation exceeded absolute maximum",VOROPP_MEMORY_ERROR);
	wall **nwalls=new wall*[current_wall_size],**nwp=nwalls,**wp=walls;
	while(wp<wep) *(nwp++)=*(wp++);
	delete [] walls;
	walls=nwalls;wel=walls+current_wall_size;wep=nwp;
}

}
