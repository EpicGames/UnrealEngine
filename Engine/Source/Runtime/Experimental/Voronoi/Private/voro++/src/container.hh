// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file container.hh
 * \brief Header file for the container_base and related classes. */

#ifndef VOROPP_CONTAINER_HH
#define VOROPP_CONTAINER_HH

#include <cstdio>
#include <vector>

#include "config.hh"
#include "common.hh"
#include "v_base.hh"
#include "cell.hh"
#include "c_loops.hh"
#include "v_compute.hh"
#include "rad_option.hh"

namespace voro {

/** \brief Pure virtual class from which wall objects are derived.
 *
 * This is a pure virtual class for a generic wall object. A wall object
 * can be specified by deriving a new class from this and specifying the
 * functions.*/
class wall {
	public:
		virtual ~wall() {}
		/** A pure virtual function for testing whether a point is
		 * inside the wall object. */
		virtual bool point_inside(double x,double y,double z) const = 0;
		/** A pure virtual function for cutting a cell without
		 * neighbor-tracking with a wall. */
		virtual bool cut_cell(voronoicell &c,double x,double y,double z) const = 0;
		/** A pure virtual function for cutting a cell with
		 * neighbor-tracking enabled with a wall. */
		virtual bool cut_cell(voronoicell_neighbor &c,double x,double y,double z) const = 0;
};

/** \brief A class for storing a list of pointers to walls.
 *
 * This class stores a list of pointers to wall classes. It contains several
 * simple routines that make use of the wall classes (such as telling whether a
 * given position is inside all of the walls or not). It can be used by itself,
 * but also forms part of container_base, for associating walls with this
 * class. */
class wall_list {
	public:
		/** An array holding pointers to wall objects. */
		wall **walls;
		/** A pointer to the next free position to add a wall pointer.
		 */
		wall **wep;
		wall_list();
		wall_list(const wall_list& Other) = delete;
		wall_list& operator=(const wall_list& Other) = delete;
		wall_list(wall_list&& Other) noexcept : walls(Other.walls), wep(Other.wep), wel(Other.wel), current_wall_size(Other.current_wall_size)
		{
			Other.walls = nullptr;
		}
		wall_list& operator=(wall_list&& Other)
		{
			if (this != &Other)
			{
				walls = Other.walls;
				wep = Other.wep;
				wel = Other.wel;
				current_wall_size = Other.current_wall_size;
				Other.walls = nullptr;
			}
			return *this;
		}
		~wall_list();
		/** Adds a wall to the list.
		 * \param[in] w the wall to add. */
		inline void add_wall(wall *w) {
			if(wep==wel) increase_wall_memory();
			*(wep++)=w;
		}
		/** Adds a wall to the list.
		 * \param[in] w a reference to the wall to add. */
		inline void add_wall(wall &w) {add_wall(&w);}
		void add_wall(wall_list &wl);
		/** Determines whether a given position is inside all of the
		 * walls on the list.
		 * \param[in] (x,y,z) the position to test.
		 * \return True if it is inside, false if it is outside. */
		inline bool point_inside_walls(double x,double y,double z) const {
			for(wall **wp=walls;wp<wep;wp++) if(!((*wp)->point_inside(x,y,z))) return false;
			return true;
		}
		/** Cuts a Voronoi cell by all of the walls currently on
		 * the list.
		 * \param[in] c a reference to the Voronoi cell class.
		 * \param[in] (x,y,z) the position of the cell.
		 * \return True if the cell still exists, false if the cell is
		 * deleted. */
		template<class c_class>
		bool apply_walls(c_class &c,double x,double y,double z) const {
			for(wall **wp=walls;wp<wep;wp++) if(!((*wp)->cut_cell(c,x,y,z))) return false;
			return true;
		}
		void deallocate();
	protected:
		void increase_wall_memory();
		/** A pointer to the limit of the walls array, used to
		 * determine when array is full. */
		wall **wel;
		/** The current amount of memory allocated for walls. */
		int current_wall_size;
};

/** \brief Class for representing a particle system in a three-dimensional
 * rectangular box.
 *
 * This class represents a system of particles in a three-dimensional
 * rectangular box. Any combination of non-periodic and periodic coordinates
 * can be used in the three coordinate directions. The class is not intended
 * for direct use, but instead forms the base of the container and
 * container_poly classes that add specialized routines for computing the
 * regular and radical Voronoi tessellations respectively. It contains routines
 * that are commonly between these two classes, such as those for drawing the
 * domain, and placing particles within the internal data structure.
 *
 * The class is derived from the wall_list class, which encapsulates routines
 * for associating walls with the container, and the voro_base class, which
 * encapsulates routines about the underlying computational grid. */
class container_base : public voro_base, public wall_list {
	public:
		/** The minimum x coordinate of the container. */
		const double ax;
		/** The maximum x coordinate of the container. */
		const double bx;
		/** The minimum y coordinate of the container. */
		const double ay;
		/** The maximum y coordinate of the container. */
		const double by;
		/** The minimum z coordinate of the container. */
		const double az;
		/** The maximum z coordinate of the container. */
		const double bz;
		/** A boolean value that determines if the x coordinate in
		 * periodic or not. */
		const bool xperiodic;
		/** A boolean value that determines if the y coordinate in
		 * periodic or not. */
		const bool yperiodic;
		/** A boolean value that determines if the z coordinate in
		 * periodic or not. */
		const bool zperiodic;
		/** This array holds the numerical IDs of each particle in each
		 * computational box. */
		int **id;
		/** A two dimensional array holding particle positions. For the
		 * derived container_poly class, this also holds particle
		 * radii. */
		double **p;
		/** This array holds the number of particles within each
		 * computational box of the container. */
		int *co;
		/** This array holds the maximum amount of particle memory for
		 * each computational box of the container. If the number of
		 * particles in a particular box ever approaches this limit,
		 * more is allocated using the add_particle_memory() function.
		 */
		int *mem;
		/** The amount of memory in the array structure for each
		 * particle. This is set to 3 when the basic class is
		 * initialized, so that the array holds (x,y,z) positions. If
		 * the container class is initialized as part of the derived
		 * class container_poly, then this is set to 4, to also hold
		 * the particle radii. */
		const int ps;
		container_base(double ax_,double bx_,double ay_,double by_,double az_,double bz_,
				int nx_,int ny_,int nz_,bool xperiodic_,bool yperiodic_,bool zperiodic_,
				int init_mem,int ps_);
		container_base(container_base& Other) = delete;
		container_base& operator=(const container_base& Other) = delete;
		container_base(container_base&& Other) noexcept : voro_base(MoveTemp(Other)), wall_list(MoveTemp(Other)),
			ax(Other.ax), bx(Other.bx), ay(Other.ay), by(Other.by), az(Other.az), bz(Other.bz),
			xperiodic(Other.xperiodic), yperiodic(Other.yperiodic), zperiodic(Other.zperiodic),
			id(Other.id), p(Other.p), co(Other.co), mem(Other.mem), ps(Other.ps)
		{
			Other.id = nullptr;
			Other.p = nullptr;
			Other.co = nullptr;
			Other.mem = nullptr;
		}
		// Note: Move assignment not implemented due to const members on this class (and it has not been needed yet)
		container_base& operator=(container_base&& Other) = delete;

		~container_base();
		bool point_inside(double x,double y,double z) const;
		void region_count() const;
		/** Initializes the Voronoi cell prior to a compute_cell
		 * operation for a specific particle being carried out by a
		 * voro_compute class. The cell is initialized to fill the
		 * entire container. For non-periodic coordinates, this is set
		 * by the position of the walls. For periodic coordinates, the
		 * space is equally divided in either direction from the
		 * particle's initial position. Plane cuts made by any walls
		 * that have been added are then applied to the cell.
		 * \param[in,out] c a reference to a voronoicell object.
		 * \param[in] ijk the block that the particle is within.
		 * \param[in] q the index of the particle within its block.
		 * \param[in] (ci,cj,ck) the coordinates of the block in the
		 * 			 container coordinate system.
		 * \param[out] (i,j,k) the coordinates of the test block
		 * 		       relative to the voro_compute
		 * 		       coordinate system.
		 * \param[out] (x,y,z) the position of the particle.
		 * \param[out] disp a block displacement used internally by the
		 *		    compute_cell routine.
		 * \return False if the plane cuts applied by walls completely
		 * removed the cell, true otherwise. */
		template<class v_cell>
		inline bool initialize_voronoicell(v_cell &c,int ijk,int q,int ci,int cj,int ck,
				int &i,int &j,int &k,double &x,double &y,double &z,int &disp) const {
			double x1,x2,y1,y2,z1,z2,*pp=p[ijk]+ps*q;
			x=*(pp++);y=*(pp++);z=*pp;
			if(xperiodic) {x1=-(x2=0.5*(bx-ax));i=nx;} else {x1=ax-x;x2=bx-x;i=ci;}
			if(yperiodic) {y1=-(y2=0.5*(by-ay));j=ny;} else {y1=ay-y;y2=by-y;j=cj;}
			if(zperiodic) {z1=-(z2=0.5*(bz-az));k=nz;} else {z1=az-z;z2=bz-z;k=ck;}
			c.init(x1,x2,y1,y2,z1,z2);
			if(!apply_walls(c,x,y,z)) return false;
			disp=ijk-i-nx*(j+ny*k);
			return true;
		}
		/** Initializes parameters for a find_voronoi_cell call within
		 * the voro_compute template.
		 * \param[in] (ci,cj,ck) the coordinates of the test block in
		 * 			 the container coordinate system.
		 * \param[in] ijk the index of the test block
		 * \param[out] (i,j,k) the coordinates of the test block
		 * 		       relative to the voro_compute
		 * 		       coordinate system.
		 * \param[out] disp a block displacement used internally by the
		 *		    find_voronoi_cell routine. */
		inline void initialize_search(int ci,int cj,int ck,int ijk,int &i,int &j,int &k,int &disp) const {
			i=xperiodic?nx:ci;
			j=yperiodic?ny:cj;
			k=zperiodic?nz:ck;
			disp=ijk-i-nx*(j+ny*k);
		}
		/** Returns the position of a particle currently being computed
		 * relative to the computational block that it is within. It is
		 * used to select the optimal worklist entry to use.
		 * \param[in] (x,y,z) the position of the particle.
		 * \param[in] (ci,cj,ck) the block that the particle is within.
		 * \param[out] (fx,fy,fz) the position relative to the block.
		 */
		inline void frac_pos(double x,double y,double z,double ci,double cj,double ck,
				double &fx,double &fy,double &fz) const {
			fx=x-ax-boxx*ci;
			fy=y-ay-boxy*cj;
			fz=z-az-boxz*ck;
		}
		/** Calculates the index of block in the container structure
		 * corresponding to given coordinates.
		 * \param[in] (ci,cj,ck) the coordinates of the original block
		 * 			 in the current computation, relative
		 * 			 to the container coordinate system.
		 * \param[in] (ei,ej,ek) the displacement of the current block
		 * 			 from the original block.
		 * \param[in,out] (qx,qy,qz) the periodic displacement that
		 * 			     must be added to the particles
		 * 			     within the computed block.
		 * \param[in] disp a block displacement used internally by the
		 * 		    find_voronoi_cell and compute_cell routines.
		 * \return The block index. */
		inline int region_index(int ci,int cj,int ck,int ei,int ej,int ek,double &qx,double &qy,double &qz,int &disp) const {
			if(xperiodic) {if(ci+ei<nx) {ei+=nx;qx=-(bx-ax);} else if(ci+ei>=(nx<<1)) {ei-=nx;qx=bx-ax;} else qx=0;}
			if(yperiodic) {if(cj+ej<ny) {ej+=ny;qy=-(by-ay);} else if(cj+ej>=(ny<<1)) {ej-=ny;qy=by-ay;} else qy=0;}
			if(zperiodic) {if(ck+ek<nz) {ek+=nz;qz=-(bz-az);} else if(ck+ek>=(nz<<1)) {ek-=nz;qz=bz-az;} else qz=0;}
			return disp+ei+nx*(ej+ny*ek);
		}
		/** Sums up the total number of stored particles.
		 * \return The number of particles. */
		inline int total_particles() const {
			int tp=*co;
			for(int *cop=co+1;cop<co+nxyz;cop++) tp+=*cop;
			return tp;
		}
	protected:
		void add_particle_memory(int i);
		bool put_locate_block(int &ijk,double &x,double &y,double &z);
		inline bool put_remap(int &ijk,double &x,double &y,double &z) const;
		inline bool remap(int &ai,int &aj,int &ak,int &ci,int &cj,int &ck,double &x,double &y,double &z,int &ijk) const;
};

/** \brief Extension of the container_base class for computing regular Voronoi
 * tessellations.
 *
 * This class is an extension of the container_base class that has routines
 * specifically for computing the regular Voronoi tessellation with no
 * dependence on particle radii. */
class container : public container_base, public radius_mono {
	public:
		container(double ax_,double bx_,double ay_,double by_,double az_,double bz_,
				int nx_,int ny_,int nz_,bool xperiodic_,bool yperiodic_,bool zperiodic_,int init_mem);
		voro_compute<container> make_compute() const;

		void clear();
		void put(int n,double x,double y,double z);
		void put(particle_order &vo,int n,double x,double y,double z);
		void compute_all_cells(voro_compute<container>& vc);
		double sum_cell_volumes(voro_compute<container>& vc);
		bool find_voronoi_cell(double x,double y,double z,double &rx,double &ry,double &rz,int &pid,voro_compute<container>& vc) const;
		
		/** Computes the Voronoi cell for a particle currently being
		 * referenced by a loop class.
		 * \param[out] c a Voronoi cell class in which to store the
		 * 		 computed cell.
		 * \param[in] vl the loop class to use.
		 * \return True if the cell was computed. If the cell cannot be
		 * computed, if it is removed entirely by a wall or boundary
		 * condition, then the routine returns false. */
		template<class v_cell,class c_loop>
		inline bool compute_cell(v_cell &c,c_loop &vl,voro_compute<container>& vc) const {
			return vc.compute_cell(c,vl.ijk,vl.q,vl.i,vl.j,vl.k);
		}
		/** Computes the Voronoi cell for given particle.
		 * \param[out] c a Voronoi cell class in which to store the
		 * 		 computed cell.
		 * \param[in] ijk the block that the particle is within.
		 * \param[in] q the index of the particle within the block.
		 * \return True if the cell was computed. If the cell cannot be
		 * computed, if it is removed entirely by a wall or boundary
		 * condition, then the routine returns false. */
		template<class v_cell>
		inline bool compute_cell(v_cell &c,int ijk,int q,voro_compute<container>& vc) const {
			int k=ijk/nxy,ijkt=ijk-nxy*k,j=ijkt/nx,i=ijkt-j*nx;
			return vc.compute_cell(c,ijk,q,i,j,k);
		}
		/** Computes the Voronoi cell for a ghost particle at a given
		 * location. NOT thread-safe; temporarily places particle into the container!
		 * \param[out] c a Voronoi cell class in which to store the
		 * 		 computed cell.
		 * \param[in] (x,y,z) the location of the ghost particle.
		 * \return True if the cell was computed. If the cell cannot be
		 * computed, if it is removed entirely by a wall or boundary
		 * condition, then the routine returns false. */
		template<class v_cell>
		inline bool compute_ghost_cell(v_cell &c,double x,double y,double z,voro_compute<container>& vc) {
			int ijk;
			if(put_locate_block(ijk,x,y,z)) {
				double *pp=p[ijk]+3*co[ijk]++;
				*(pp++)=x;*(pp++)=y;*pp=z;
				bool q=compute_cell(c,ijk,co[ijk]-1,vc);
				co[ijk]--;
				return q;
			}
			return false;
		}
	private:
		friend class voro_compute<container>;
};

/** \brief Extension of the container_base class for computing radical Voronoi
 * tessellations.
 *
 * This class is an extension of container_base class that has routines
 * specifically for computing the radical Voronoi tessellation that depends on
 * the particle radii. */
class container_poly : public container_base, public radius_poly {
	public:
		container_poly(double ax_,double bx_,double ay_,double by_,double az_,double bz_,
				int nx_,int ny_,int nz_,bool xperiodic_,bool yperiodic_,bool zperiodic_,int init_mem);
		voro_compute<container_poly> make_compute() const;

		void clear();
		void put(int n,double x,double y,double z,double r);
		void put(particle_order &vo,int n,double x,double y,double z,double r);
		void compute_all_cells(voro_compute<container>& vc);
		double sum_cell_volumes(voro_compute<container>& vc);
		/** Computes the Voronoi cell for a particle currently being
		 * referenced by a loop class.
		 * \param[out] c a Voronoi cell class in which to store the
		 * 		 computed cell.
		 * \param[in] vl the loop class to use.
		 * \return True if the cell was computed. If the cell cannot be
		 * computed, if it is removed entirely by a wall or boundary
		 * condition, then the routine returns false. */
		template<class v_cell,class c_loop>
		inline bool compute_cell(v_cell &c,c_loop &vl,voro_compute<container>& vc) const {
			return vc.compute_cell(c,vl.ijk,vl.q,vl.i,vl.j,vl.k);
		}
		/** Computes the Voronoi cell for given particle.
		 * \param[out] c a Voronoi cell class in which to store the
		 * 		 computed cell.
		 * \param[in] ijk the block that the particle is within.
		 * \param[in] q the index of the particle within the block.
		 * \return True if the cell was computed. If the cell cannot be
		 * computed, if it is removed entirely by a wall or boundary
		 * condition, then the routine returns false. */
		template<class v_cell>
		inline bool compute_cell(v_cell &c,int ijk,int q,voro_compute<container>& vc) const {
			int k=ijk/nxy,ijkt=ijk-nxy*k,j=ijkt/nx,i=ijkt-j*nx;
			return vc.compute_cell(c,ijk,q,i,j,k);
		}
		/** Computes the Voronoi cell for a ghost particle at a given
		 * location.
		 * \param[out] c a Voronoi cell class in which to store the
		 * 		 computed cell. NOT thread-safe; temporarily places particle into the container!
		 * \param[in] (x,y,z) the location of the ghost particle.
		 * \param[in] r the radius of the ghost particle.
		 * \return True if the cell was computed. If the cell cannot be
		 * computed, if it is removed entirely by a wall or boundary
		 * condition, then the routine returns false. */
		template<class v_cell>
		inline bool compute_ghost_cell(v_cell &c,double x,double y,double z,double r,voro_compute<container>& vc) {
			int ijk;
			if(put_locate_block(ijk,x,y,z)) {
				double *pp=p[ijk]+4*co[ijk]++,tm=max_radius;
				*(pp++)=x;*(pp++)=y;*(pp++)=z;*pp=r;
				if(r>max_radius) max_radius=r;
				bool q=compute_cell(c,ijk,co[ijk]-1);
				co[ijk]--;max_radius=tm;
				return q;
			}
			return false;
		}
		bool find_voronoi_cell(double x,double y,double z,double &rx,double &ry,double &rz,int &pid,voro_compute<container>& vc) const;
	private:
		friend class voro_compute<container_poly>;
};

}

#endif
