// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file pre_container.cc
 * \brief Function implementations for the pre_container and related classes.
 */

#include <cmath>

#include "config.hh"
#include "pre_container.hh"

namespace voro {

// version of pre_container_base::guess_optimal that doesn't require actually building the pre_container
void guess_optimal(int siteCount, double sizeX, double sizeY, double sizeZ, int &nx, int &ny, int &nz) {
	double ilscale = pow(siteCount / (voro::optimal_particles*sizeX*sizeY*sizeZ), 1 / 3.0);
	nx = int(sizeX*ilscale + 1);
	ny = int(sizeY*ilscale + 1);
	nz = int(sizeZ*ilscale + 1);
}

/** The class constructor sets up the geometry of container, initializing the
 * minimum and maximum coordinates in each direction. It allocates an initial
 * chunk into which to store particle information.
 * \param[in] (ax_,bx_) the minimum and maximum x coordinates.
 * \param[in] (ay_,by_) the minimum and maximum y coordinates.
 * \param[in] (az_,bz_) the minimum and maximum z coordinates.
 * \param[in] (xperiodic_,yperiodic_,zperiodic_ ) flags setting whether the
 *                                                container is periodic in each
 *                                                coordinate direction.
 * \param[in] ps_ the number of floating point entries to store for each
 *                particle. */
pre_container_base::pre_container_base(double ax_,double bx_,double ay_,double by_,double az_,double bz_,
	bool xperiodic_,bool yperiodic_,bool zperiodic_,int ps_) :
	ax(ax_), bx(bx_), ay(ay_), by(by_), az(az_), bz(bz_),
	xperiodic(xperiodic_), yperiodic(yperiodic_), zperiodic(zperiodic_), ps(ps_),
	index_sz(init_chunk_size), pre_id(new int*[index_sz]), end_id(pre_id),
	pre_p(new double*[index_sz]), end_p(pre_p) {
		ch_id=*end_id=new int[pre_container_chunk_size];
		l_id=end_id+index_sz;e_id=ch_id+pre_container_chunk_size;
		ch_p=*end_p=new double[ps*pre_container_chunk_size];
}

/** The destructor frees the dynamically allocated memory. */
pre_container_base::~pre_container_base() {
	delete [] *end_p;
	delete [] *end_id;
	while (end_id!=pre_id) {
		end_p--;
		delete [] *end_p;
		end_id--;
		delete [] *end_id;
	}
	delete [] pre_p;
	delete [] pre_id;
}

/** Makes a guess at the optimal grid of blocks to use, computing in
 * a way that
 * \param[out] (nx,ny,nz) the number of blocks to use. */
void pre_container_base::guess_optimal(int &nx,int &ny,int &nz) {
	double dx=bx-ax,dy=by-ay,dz=bz-az;
	double ilscale=pow(total_particles()/(optimal_particles*dx*dy*dz),1/3.0);
	nx=int(dx*ilscale+1);
	ny=int(dy*ilscale+1);
	nz=int(dz*ilscale+1);
}

/** Stores a particle ID and position, allocating a new memory chunk if
 * necessary. For coordinate directions in which the container is not periodic,
 * the routine checks to make sure that the particle is within the container
 * bounds. If the particle is out of bounds, it is not stored.
 * \param[in] n the numerical ID of the inserted particle.
 * \param[in] (x,y,z) the position vector of the inserted particle. */
void pre_container::put(int n,double x,double y,double z) {
	if((xperiodic||(x>=ax&&x<=bx))&&(yperiodic||(y>=ay&&y<=by))&&(zperiodic||(z>=az&&z<=bz))) {
		if(ch_id==e_id) new_chunk();
		*(ch_id++)=n;
		*(ch_p++)=x;*(ch_p++)=y;*(ch_p++)=z;
	}
#if VOROPP_REPORT_OUT_OF_BOUNDS ==1
	else fprintf(stderr,"Out of bounds: (x,y,z)=(%g,%g,%g)\n",x,y,z);
#endif
}

/** Stores a particle ID and position, allocating a new memory chunk if necessary.
 * \param[in] n the numerical ID of the inserted particle.
 * \param[in] (x,y,z) the position vector of the inserted particle.
 * \param[in] r the radius of the particle. */
void pre_container_poly::put(int n,double x,double y,double z,double r) {
	if((xperiodic||(x>=ax&&x<=bx))&&(yperiodic||(y>=ay&&y<=by))&&(zperiodic||(z>=az&&z<=bz))) {
		if(ch_id==e_id) new_chunk();
		*(ch_id++)=n;
		*(ch_p++)=x;*(ch_p++)=y;*(ch_p++)=z;*(ch_p++)=r;
	}
#if VOROPP_REPORT_OUT_OF_BOUNDS ==1
	else fprintf(stderr,"Out of bounds: (x,y,z)=(%g,%g,%g)\n",x,y,z);
#endif
}

/** Transfers the particles stored within the class to a container class.
 * \param[in] con the container class to transfer to. */
void pre_container::setup(container &con) {
	int **c_id=pre_id,*idp,*ide,n;
	double **c_p=pre_p,*pp,x,y,z;
	while(c_id<end_id) {
		idp=*(c_id++);ide=idp+pre_container_chunk_size;
		pp=*(c_p++);
		while(idp<ide) {
			n=*(idp++);x=*(pp++);y=*(pp++);z=*(pp++);
			con.put(n,x,y,z);
		}
	}
	idp=*c_id;
	pp=*c_p;
	while(idp<ch_id) {
		n=*(idp++);x=*(pp++);y=*(pp++);z=*(pp++);
		con.put(n,x,y,z);
	}
}

/** Transfers the particles stored within the class to a container_poly class.
 * \param[in] con the container_poly class to transfer to. */
void pre_container_poly::setup(container_poly &con) {
	int **c_id=pre_id,*idp,*ide,n;
	double **c_p=pre_p,*pp,x,y,z,r;
	while(c_id<end_id) {
		idp=*(c_id++);ide=idp+pre_container_chunk_size;
		pp=*(c_p++);
		while(idp<ide) {
			n=*(idp++);x=*(pp++);y=*(pp++);z=*(pp++);r=*(pp++);
			con.put(n,x,y,z,r);
		}
	}
	idp=*c_id;
	pp=*c_p;
	while(idp<ch_id) {
		n=*(idp++);x=*(pp++);y=*(pp++);z=*(pp++);r=*(pp++);
		con.put(n,x,y,z,r);
	}
}

/** Transfers the particles stored within the class to a container class, also
 * recording the order in which particles were stored.
 * \param[in] vo the ordering class to use.
 * \param[in] con the container class to transfer to. */
void pre_container::setup(particle_order &vo,container &con) {
	int **c_id=pre_id,*idp,*ide,n;
	double **c_p=pre_p,*pp,x,y,z;
	while(c_id<end_id) {
		idp=*(c_id++);ide=idp+pre_container_chunk_size;
		pp=*(c_p++);
		while(idp<ide) {
			n=*(idp++);x=*(pp++);y=*(pp++);z=*(pp++);
			con.put(vo,n,x,y,z);
		}
	}
	idp=*c_id;
	pp=*c_p;
	while(idp<ch_id) {
		n=*(idp++);x=*(pp++);y=*(pp++);z=*(pp++);
		con.put(vo,n,x,y,z);
	}
}

/** Transfers the particles stored within the class to a container_poly class,
 * also recording the order in which particles were stored.
 * \param[in] vo the ordering class to use.
 * \param[in] con the container_poly class to transfer to. */
void pre_container_poly::setup(particle_order &vo,container_poly &con) {
	int **c_id=pre_id,*idp,*ide,n;
	double **c_p=pre_p,*pp,x,y,z,r;
	while(c_id<end_id) {
		idp=*(c_id++);ide=idp+pre_container_chunk_size;
		pp=*(c_p++);
		while(idp<ide) {
			n=*(idp++);x=*(pp++);y=*(pp++);z=*(pp++);r=*(pp++);
			con.put(vo,n,x,y,z,r);
		}
	}
	idp=*c_id;
	pp=*c_p;
	while(idp<ch_id) {
		n=*(idp++);x=*(pp++);y=*(pp++);z=*(pp++);r=*(pp++);
		con.put(vo,n,x,y,z,r);
	}
}

/** Allocates a new chunk of memory for storing particles. */
void pre_container_base::new_chunk() {
	end_id++;end_p++;
	if(end_id==l_id) extend_chunk_index();
	ch_id=*end_id=new int[pre_container_chunk_size];
	e_id=ch_id+pre_container_chunk_size;
	ch_p=*end_p=new double[ps*pre_container_chunk_size];
}

/** Extends the index of chunks. */
void pre_container_base::extend_chunk_index() {
	index_sz<<=1;
	if(index_sz>max_chunk_size)
		voro_fatal_error("Absolute memory limit on chunk index reached",VOROPP_MEMORY_ERROR);
#if VOROPP_VERBOSE >=2
	fprintf(stderr,"Pre-container chunk index scaled up to %d\n",index_sz);
#endif
	int **n_id=new int*[index_sz],**p_id=n_id,**c_id=pre_id;
	double **n_p=new double*[index_sz],**p_p=n_p,**c_p=pre_p;
	while(c_id<end_id) {
		*(p_id++)=*(c_id++);
		*(p_p++)=*(c_p++);
	}
	delete [] pre_id;pre_id=n_id;end_id=p_id;l_id=pre_id+index_sz;
	delete [] pre_p;pre_p=n_p;end_p=p_p;
}

}
