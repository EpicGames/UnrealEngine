// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file pre_container.hh
 * \brief Header file for the pre_container and related classes. */

#ifndef VOROPP_PRE_CONTAINER_HH
#define VOROPP_PRE_CONTAINER_HH

#include <cstdio>

#include "c_loops.hh"
#include "container.hh"

#include "HAL/PlatformMisc.h"

namespace voro {

void guess_optimal(int siteCount, double sizeX, double sizeY, double sizeZ, int &nx, int &ny, int &nz);

/** \brief A class for storing an arbitrary number of particles, prior to setting
 * up a container geometry.
 *
 * The pre_container_base class can dynamically import and store an arbitrary
 * number of particles. Once the particles have been read in, an appropriate
 * container class can be set up with the optimal grid size, and the particles
 * can be transferred.
 *
 * The pre_container_base class is not intended for direct use, but forms the
 * base of the pre_container and pre_container_poly classes, that add routines
 * depending on whether particle radii need to be tracked or not. */
class pre_container_base {
	public:
		/** The minimum x coordinate of the container. */
		const double ax;
		/** The maximum x coordinate of the container. */
		const double bx;
		/** The minimum y coordinate of the container. */
		const double ay;
		/** The maximum y coordinate of the container. */
		const double by;
		/** The minimum z coordinate of the container. */
		const double az;
		/** The maximum z coordinate of the container. */
		const double bz;
		/** A boolean value that determines if the x coordinate in
		 * periodic or not. */
		const bool xperiodic;
		/** A boolean value that determines if the y coordinate in
		 * periodic or not. */
		const bool yperiodic;
		/** A boolean value that determines if the z coordinate in
		 * periodic or not. */
		const bool zperiodic;
		void guess_optimal(int &nx,int &ny,int &nz);
		pre_container_base(double ax_,double bx_,double ay_,double by_,double az_,double bz_,bool xperiodic_,bool yperiodic_,bool zperiodic_,int ps_);
		~pre_container_base();
PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS
		/** Calculates and returns the total number of particles stored
		 * within the class.
		 * \return The number of particles. */
		inline int total_particles() {
			return (end_id-pre_id)*pre_container_chunk_size+(ch_id-*end_id);
		}
PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS
	protected:
		/** The number of doubles associated with a single particle
		 * (three for the standard container, four when radius
		 * information is stored). */
		const int ps;
		void new_chunk();
		void extend_chunk_index();
		/** The size of the chunk index. */
		int index_sz;
		/** A pointer to the chunk index to store the integer particle
		 * IDs. */
		int **pre_id;
		/** A pointer to the last allocated integer ID chunk. */
		int **end_id;
		/** A pointer to the end of the integer ID chunk index, used to
		 * determine when the chunk index is full. */
		int **l_id;
		/** A pointer to the next available slot on the current
		 * particle ID chunk. */
		int *ch_id;
		/** A pointer to the end of the current integer chunk. */
		int *e_id;
		/** A pointer to the chunk index to store the floating point
		 * information associated with particles. */
		double **pre_p;
		/** A pointer to the last allocated chunk of floating point
		 * information. */
		double **end_p;
		/** A pointer to the next available slot on the current
		 * floating point chunk. */
		double *ch_p;
};

/** \brief A class for storing an arbitrary number of particles without radius
 * information, prior to setting up a container geometry.
 *
 * The pre_container class is an extension of the pre_container_base class for
 * cases when no particle radius information is available. */
class pre_container : public pre_container_base {
	public:
		/** The class constructor sets up the geometry of container,
		 * initializing the minimum and maximum coordinates in each
		 * direction.
		 * \param[in] (ax_,bx_) the minimum and maximum x coordinates.
		 * \param[in] (ay_,by_) the minimum and maximum y coordinates.
		 * \param[in] (az_,bz_) the minimum and maximum z coordinates.
		 * \param[in] (xperiodic_,yperiodic_,zperiodic_ ) flags setting whether the
		 *                                                container is periodic in
		 *                                                each coordinate direction. */
		pre_container(double ax_,double bx_,double ay_,double by_,double az_,double bz_,
				bool xperiodic_,bool yperiodic_,bool zperiodic_)
			: pre_container_base(ax_,bx_,ay_,by_,az_,bz_,xperiodic_,yperiodic_,zperiodic_,3) {};
		void put(int n,double x,double y,double z);
		void setup(container &con);
		void setup(particle_order &vo,container &con);
};

/** \brief A class for storing an arbitrary number of particles with radius
 * information, prior to setting up a container geometry.
 *
 * The pre_container_poly class is an extension of the pre_container_base class
 * for cases when particle radius information is available. */
class pre_container_poly : public pre_container_base {
	public:
		/** The class constructor sets up the geometry of container,
		 * initializing the minimum and maximum coordinates in each
		 * direction.
		 * \param[in] (ax_,bx_) the minimum and maximum x coordinates.
		 * \param[in] (ay_,by_) the minimum and maximum y coordinates.
		 * \param[in] (az_,bz_) the minimum and maximum z coordinates.
		 * \param[in] (xperiodic_,yperiodic_,zperiodic_ ) flags setting whether the
		 *                                                container is periodic in
		 *                                                each coordinate direction. */
		pre_container_poly(double ax_,double bx_,double ay_,double by_,double az_,double bz_,
				bool xperiodic_,bool yperiodic_,bool zperiodic_)
			: pre_container_base(ax_,bx_,ay_,by_,az_,bz_,xperiodic_,yperiodic_,zperiodic_,4) {};
		void put(int n,double x,double y,double z,double r);
		void setup(container_poly &con);
		void setup(particle_order &vo,container_poly &con);
};

}

#endif
