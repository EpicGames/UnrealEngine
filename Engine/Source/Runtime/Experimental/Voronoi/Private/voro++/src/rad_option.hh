// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file rad_option.hh
 * \brief Header file for the classes encapsulating functionality for the
 * regular and radical Voronoi tessellations. */

#ifndef VOROPP_RAD_OPTION_HH
#define VOROPP_RAD_OPTION_HH

#include <cmath>

namespace voro {

/** \brief Class containing all of the routines that are specific to computing 
 * the regular Voronoi tessellation.
 *
 * The container and container_periodic classes are derived from this class,
 * and during the Voronoi cell computation, these routines are used to create
 * the regular Voronoi tessellation. */
class radius_mono {
	public:
		struct radius_info // intentionally empty; no special radius info needs to be passed in for radius_mono case
		{
		};
	protected:
		/** This is called prior to computing a Voronoi cell for a
		 * given particle to initialize any required constants.
		 * \param[in] ijk the block that the particle is within.
		 * \param[in] s the index of the particle within the block. */
		inline void r_init(int ijk,int s,radius_info& radius_out) const {}
		/** Sets a required constant to be used when carrying out a
		 * plane bounds check. */
		inline void r_prime(double rv,radius_info& radius_out) const {}
		/** Carries out a radius bounds check.
		 * \param[in] crs the radius squared to be tested.
		 * \param[in] mrs the current maximum distance to a Voronoi
		 *                vertex multiplied by two.
		 * \return True if particles at this radius could not possibly
		 * cut the cell, false otherwise. */
		inline bool r_ctest(double crs,double mrs,const radius_info& rad_info) const {return crs>mrs;}
		/** Scales a plane displacement during a plane bounds check.
		 * \param[in] lrs the plane displacement.
		 * \return The scaled value. */
		inline double r_cutoff(double lrs, const radius_info& rad_info) const {return lrs;}
		/** Adds the maximum radius squared to a given value.
		 * \param[in] rs the value to consider.
		 * \return The value with the radius squared added. */
		inline double r_max_add(double rs) const {return rs;}
		/** Subtracts the radius squared of a particle from a given 
		 * value.
		 * \param[in] rs the value to consider.
		 * \param[in] ijk the block that the particle is within.
		 * \param[in] q the index of the particle within the block. 
		 * \return The value with the radius squared subtracted. */
		inline double r_current_sub(double rs,int ijk,int q) const {return rs;}
		/** Scales a plane displacement prior to use in the plane cutting
		 * algorithm.
		 * \param[in] rs the initial plane displacement.
		 * \param[in] ijk the block that the particle is within.
		 * \param[in] q the index of the particle within the block.
		 * \return The scaled plane displacement. */ 
		inline double r_scale(double rs,int ijk,int q, const radius_info& rad_info) const {return rs;}
		/** Scales a plane displacement prior to use in the plane
		 * cutting algorithm, and also checks if it could possibly cut
		 * the cell.
		 * \param[in,out] rs the plane displacement to be scaled.
		 * \param[in] mrs the current maximum distance to a Voronoi
		 *                vertex multiplied by two.
		 * \param[in] ijk the block that the particle is within.
		 * \param[in] q the index of the particle within the block.
		 * \return True if the cell could possibly cut the cell, false
		 * otherwise. */		
		inline bool r_scale_check(double &rs,double mrs,int ijk,int q, const radius_info& rad_info) const {return rs<mrs;}
};

/**  \brief Class containing all of the routines that are specific to computing 
 * the radical Voronoi tessellation.
 *
 * The container_poly and container_periodic_poly classes are derived from this
 * class, and during the Voronoi cell computation, these routines are used to
 * create the radical Voronoi tessellation. */
class radius_poly {
	public:
		struct radius_info
		{
			double r_rad, r_mul, r_val;
		};
		/** A two-dimensional array holding particle positions and radii. */			
		double **ppr;
		/** The current maximum radius of any particle, used to
		 * determine when to cut off the radical Voronoi computation.
		 * */
		double max_radius;
		/** The class constructor sets the maximum particle radius to
		 * be zero. */
		radius_poly() : max_radius(0) {}
	protected:
		/** This is called prior to computing a Voronoi cell for a
		 * given particle to initialize any required constants.
		 * \param[in] ijk the block that the particle is within.
		 * \param[in] s the index of the particle within the block. */
		inline void r_init(int ijk, int s, radius_info& radius_out) const {
			radius_out.r_rad=ppr[ijk][4*s+3]*ppr[ijk][4*s+3];
			radius_out.r_mul= radius_out.r_rad-max_radius*max_radius;
		}
		/** Sets a required constant to be used when carrying out a
		 * plane bounds check. */
		inline void r_prime(double rv, radius_info& radius_out) const {
			radius_out.r_val = 1 + radius_out.r_mul / rv;
		}
		/** Carries out a radius bounds check.
		 * \param[in] crs the radius squared to be tested.
		 * \param[in] mrs the current maximum distance to a Voronoi
		 *                vertex multiplied by two.
		 * \return True if particles at this radius could not possibly
		 * cut the cell, false otherwise. */		
		inline bool r_ctest(double crs, double mrs, const radius_info& rad_info) const {return crs+ rad_info.r_mul>sqrt(mrs*crs);}
		/** Scales a plane displacement during a plane bounds check.
		 * \param[in] lrs the plane displacement.
		 * \return The scaled value. */		
		inline double r_cutoff(double lrs, const radius_info& rad_info) const {return lrs* rad_info.r_val;}
		/** Adds the maximum radius squared to a given value.
		 * \param[in] rs the value to consider.
		 * \return The value with the radius squared added. */		
		inline double r_max_add(double rs) const {return rs+max_radius*max_radius;}
		/** Subtracts the radius squared of a particle from a given 
		 * value.
		 * \param[in] rs the value to consider.
		 * \param[in] ijk the block that the particle is within.
		 * \param[in] q the index of the particle within the block. 
		 * \return The value with the radius squared subtracted. */
		inline double r_current_sub(double rs,int ijk,int q) const {
			return rs-ppr[ijk][4*q+3]*ppr[ijk][4*q+3];
		}
		/** Scales a plane displacement prior to use in the plane cutting
		 * algorithm.
		 * \param[in] rs the initial plane displacement.
		 * \param[in] ijk the block that the particle is within.
		 * \param[in] q the index of the particle within the block.
		 * \return The scaled plane displacement. */ 
		inline double r_scale(double rs,int ijk,int q, const radius_info& rad_info) const {
			return rs+ rad_info.r_rad-ppr[ijk][4*q+3]*ppr[ijk][4*q+3];
		}
		/** Scales a plane displacement prior to use in the plane
		 * cutting algorithm, and also checks if it could possibly cut
		 * the cell.
		 * \param[in,out] rs the plane displacement to be scaled.
		 * \param[in] mrs the current maximum distance to a Voronoi
		 *                vertex multiplied by two.
		 * \param[in] ijk the block that the particle is within.
		 * \param[in] q the index of the particle within the block.
		 * \return True if the cell could possibly cut the cell, false
		 * otherwise. */
		inline bool r_scale_check(double &rs,double mrs,int ijk,int q, const radius_info& rad_info) const {
			double trs=rs;
			rs+= rad_info.r_rad-ppr[ijk][4*q+3]*ppr[ijk][4*q+3];
			return rs<sqrt(mrs*trs);
		}
		
};

}
#endif
