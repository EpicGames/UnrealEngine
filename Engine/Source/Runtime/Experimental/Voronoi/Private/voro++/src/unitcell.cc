// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file unitcell.cc
 * \brief Function implementations for the unitcell class. */

#include <cmath>
#include <queue>

#include "unitcell.hh"
#include "cell.hh"

namespace voro {

/** Initializes the unit cell class for a particular non-orthogonal periodic
 * geometry, corresponding to a parallelepiped with sides given by three
 * vectors. The class constructs the unit Voronoi cell corresponding to this
 * geometry.
 * \param[in] (bx_) The x coordinate of the first unit vector.
 * \param[in] (bxy_,by_) The x and y coordinates of the second unit vector.
 * \param[in] (bxz_,byz_,bz_) The x, y, and z coordinates of the third unit
 *                            vector. */
unitcell::unitcell(double bx_,double bxy_,double by_,double bxz_,double byz_,double bz_)
	: bx(bx_), bxy(bxy_), by(by_), bxz(bxz_), byz(byz_), bz(bz_) {
	int i,j,l=1;

	// Initialize the Voronoi cell to be a very large rectangular box
	const double ucx=max_unit_voro_shells*bx,ucy=max_unit_voro_shells*by,ucz=max_unit_voro_shells*bz;
	unit_voro.init(-ucx,ucx,-ucy,ucy,-ucz,ucz);

	// Repeatedly cut the cell by shells of periodic image particles
	while(l<2*max_unit_voro_shells) {

		// Check to see if any of the planes from the current shell
		// will cut the cell
		if(unit_voro_intersect(l)) {

			// If they do, apply the plane cuts from the current
			// shell
			unit_voro_apply(l,0,0);
			for(i=1;i<l;i++) {
				unit_voro_apply(l,i,0);
				unit_voro_apply(-l,i,0);
			}
			for(i=-l;i<=l;i++) unit_voro_apply(i,l,0);
			for(i=1;i<l;i++) for(j=-l+1;j<=l;j++) {
				unit_voro_apply(l,j,i);
				unit_voro_apply(-j,l,i);
				unit_voro_apply(-l,-j,i);
				unit_voro_apply(j,-l,i);
			}
			for(i=-l;i<=l;i++) for(j=-l;j<=l;j++) unit_voro_apply(i,j,l);
		} else {

			// Calculate a bound on the maximum y and z coordinates
			// that could possibly cut the cell. This is based upon
			// a geometric result that particles with z>l can't cut
			// a cell lying within the paraboloid
			// z<=(l*l-x*x-y*y)/(2*l). It is always a tighter bound
			// than the one based on computing the maximum radius
			// of a Voronoi cell vertex.
			max_uv_y=max_uv_z=0;
			double y,z,q,*pts=unit_voro.pts,*pp=pts;
			while(pp<pts+3*unit_voro.p) {
				q=*(pp++);y=*(pp++);z=*(pp++);q=sqrt(q*q+y*y+z*z);
				if(y+q>max_uv_y) max_uv_y=y+q;
				if(z+q>max_uv_z) max_uv_z=z+q;
			}
			max_uv_z*=0.5;
			max_uv_y*=0.5;
			return;
		}
		l++;
	}

	// If the routine makes it here, then the unit cell still hasn't been
	// completely bounded by the plane cuts. Give the memory error code,
	// because this is mainly a case of hitting a safe limit, than any
	// inherent problem.
	voro_fatal_error("Periodic cell computation failed",VOROPP_MEMORY_ERROR);
}

/** Applies a pair of opposing plane cuts from a periodic image point
 * to the unit Voronoi cell.
 * \param[in] (i,j,k) the index of the periodic image to consider. */
inline void unitcell::unit_voro_apply(int i,int j,int k) {
	double x=i*bx+j*bxy+k*bxz,y=j*by+k*byz,z=k*bz;
	unit_voro.plane(x,y,z);
	unit_voro.plane(-x,-y,-z);
}

/** Calculates whether the unit Voronoi cell intersects a given periodic image
 * of the domain.
 * \param[in] (dx,dy,dz) the displacement of the periodic image.
 * \param[out] vol the proportion of the unit cell volume within this image,
 *                 only computed in the case that the two intersect.
 * \return True if they intersect, false otherwise. */
bool unitcell::intersects_image(double dx,double dy,double dz,double &vol) {
	const double bxinv=1/bx,byinv=1/by,bzinv=1/bz,ivol=bxinv*byinv*bzinv;
	voronoicell c;
	c=unit_voro;
	dx*=2;dy*=2;dz*=2;
	if(!c.plane(0,0,bzinv,dz+1)) return false;
	if(!c.plane(0,0,-bzinv,-dz+1)) return false;
	if(!c.plane(0,byinv,-byz*byinv*bzinv,dy+1)) return false;
	if(!c.plane(0,-byinv,byz*byinv*bzinv,-dy+1)) return false;
	if(!c.plane(bxinv,-bxy*bxinv*byinv,(bxy*byz-by*bxz)*ivol,dx+1)) return false;
	if(!c.plane(-bxinv,bxy*bxinv*byinv,(-bxy*byz+by*bxz)*ivol,-dx+1)) return false;
	vol=c.volume()*ivol;
	return true;
}

/** Computes a list of periodic domain images that intersect the unit Voronoi cell.
 * \param[out] vi a vector containing triplets (i,j,k) corresponding to domain
 *                images that intersect the unit Voronoi cell, when it is
 *                centered in the middle of the primary domain.
 * \param[out] vd a vector containing the fraction of the Voronoi cell volume
 *                within each corresponding image listed in vi. */
void unitcell::images(std::vector<int> &vi,std::vector<double> &vd) {
	const int ms2=max_unit_voro_shells*2+1,mss=ms2*ms2*ms2;
	bool *a=new bool[mss],*ac=a+max_unit_voro_shells*(1+ms2*(1+ms2)),*ap=a;
	int i,j,k;
	double vol;

	// Initialize mask
	while(ap<ac) *(ap++)=true;
	*(ap++)=false;
	while(ap<a+mss) *(ap++)=true;

	// Set up the queue and add (0,0,0) image to it
	std::queue<int> q;
	q.push(0);q.push(0);q.push(0);

	while(!q.empty()) {

		// Read the next entry on the queue
		i=q.front();q.pop();
		j=q.front();q.pop();
		k=q.front();q.pop();

		// Check intersection of this image
		if(intersects_image(i,j,k,vol)) {

			// Add this entry to the output vectors
			vi.push_back(i);
			vi.push_back(j);
			vi.push_back(k);
			vd.push_back(vol);

			// Add neighbors to the queue if they have not been
			// tested
			ap=ac+i+ms2*(j+ms2*k);
			if(k>-max_unit_voro_shells&&*(ap-ms2*ms2)) {q.push(i);q.push(j);q.push(k-1);*(ap-ms2*ms2)=false;}
			if(j>-max_unit_voro_shells&&*(ap-ms2)) {q.push(i);q.push(j-1);q.push(k);*(ap-ms2)=false;}
			if(i>-max_unit_voro_shells&&*(ap-1)) {q.push(i-1);q.push(j);q.push(k);*(ap-1)=false;}
			if(i<max_unit_voro_shells&&*(ap+1)) {q.push(i+1);q.push(j);q.push(k);*(ap+1)=false;}
			if(j<max_unit_voro_shells&&*(ap+ms2)) {q.push(i);q.push(j+1);q.push(k);*(ap+ms2)=false;}
			if(k<max_unit_voro_shells&&*(ap+ms2*ms2)) {q.push(i);q.push(j);q.push(k+1);*(ap+ms2*ms2)=false;}
		}
	}

	// Remove mask memory
	delete [] a;
}

/** Tests to see if a shell of periodic images could possibly cut the periodic
 * unit cell.
 * \param[in] l the index of the shell to consider.
 * \return True if a point in the shell cuts the cell, false otherwise. */
bool unitcell::unit_voro_intersect(int l) {
	int i,j;
	if(unit_voro_test(l,0,0)) return true;
	for(i=1;i<l;i++) {
		if(unit_voro_test(l,i,0)) return true;
		if(unit_voro_test(-l,i,0)) return true;
	}
	for(i=-l;i<=l;i++) if(unit_voro_test(i,l,0)) return true;
	for(i=1;i<l;i++) for(j=-l+1;j<=l;j++) {
		if(unit_voro_test(l,j,i)) return true;
		if(unit_voro_test(-j,l,i)) return true;
		if(unit_voro_test(-l,-j,i)) return true;
		if(unit_voro_test(j,-l,i)) return true;
	}
	for(i=-l;i<=l;i++) for(j=-l;j<=l;j++) if(unit_voro_test(i,j,l)) return true;
	return false;
}

/** Tests to see if a plane cut from a particular periodic image will cut th
 * unit Voronoi cell.
 * \param[in] (i,j,k) the index of the periodic image to consider.
 * \return True if the image cuts the cell, false otherwise. */
inline bool unitcell::unit_voro_test(int i,int j,int k) {
	double x=i*bx+j*bxy+k*bxz,y=j*by+k*byz,z=k*bz;
	double rsq=x*x+y*y+z*z;
	return unit_voro.plane_intersects(x,y,z,rsq);
}


}
