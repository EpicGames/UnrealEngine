// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file unitcell.hh
 * \brief Header file for the unitcell class. */

#ifndef VOROPP_UNITCELL_HH
#define VOROPP_UNITCELL_HH

#include <vector>

#include "config.hh"
#include "cell.hh"

namespace voro {

/** \brief Class for computation of the unit Voronoi cell associated with
 * a 3D non-rectangular periodic domain. */
class unitcell {
	public:
		/** The x coordinate of the first vector defining the periodic
		 * domain. */
		const double bx;
		/** The x coordinate of the second vector defining the periodic
		 * domain. */
		const double bxy;
		/** The y coordinate of the second vector defining the periodic
		 * domain. */
		const double by;
		/** The x coordinate of the third vector defining the periodic
		 * domain. */
		const double bxz;
		/** The y coordinate of the third vector defining the periodic
		 * domain. */
		const double byz;
		/** The z coordinate of the third vector defining the periodic
		 * domain. */
		const double bz;
		/** The computed unit Voronoi cell corresponding the given
		 * 3D non-rectangular periodic domain geometry. */
		voronoicell unit_voro;
		unitcell(double bx_,double bxy_,double by_,double bxz_,double byz_,double bz_);
		bool intersects_image(double dx,double dy,double dz,double &vol);
		void images(std::vector<int> &vi,std::vector<double> &vd);
	protected:
		/** The maximum y-coordinate that could possibly cut the
		 * computed unit Voronoi cell. */
		double max_uv_y;
		/** The maximum z-coordinate that could possibly cut the
		 * computed unit Voronoi cell. */
		double max_uv_z;
	private:
		inline void unit_voro_apply(int i,int j,int k);
		bool unit_voro_intersect(int l);
		inline bool unit_voro_test(int i,int j,int k);
};

}

#endif
