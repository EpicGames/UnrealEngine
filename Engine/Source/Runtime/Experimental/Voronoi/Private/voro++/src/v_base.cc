// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file v_base.cc
 * \brief Function implementations for the base Voronoi container class. */

#include "v_base.hh"
#include "config.hh"

namespace voro {

/** This function is called during container construction. The routine scans
 * all of the worklists in the wl[] array. For a given worklist of blocks
 * labeled \f$w_1\f$ to \f$w_n\f$, it computes a sequence \f$r_0\f$ to
 * \f$r_n\f$ so that $r_i$ is the minimum distance to all the blocks
 * \f$w_{j}\f$ where \f$j>i\f$ and all blocks outside the worklist. The values
 * of \f$r_n\f$ is calculated first, as the minimum distance to any block in
 * the shell surrounding the worklist. The \f$r_i\f$ are then computed in
 * reverse order by considering the distance to \f$w_{i+1}\f$. */
voro_base::voro_base(int nx_,int ny_,int nz_,double boxx_,double boxy_,double boxz_) :
	nx(nx_), ny(ny_), nz(nz_), nxy(nx_*ny_), nxyz(nxy*nz_), boxx(boxx_), boxy(boxy_), boxz(boxz_),
	xsp(1/boxx_), ysp(1/boxy_), zsp(1/boxz_), mrad(new double[wl_hgridcu*wl_seq_length]) {
	const unsigned int b1=1<<21,b2=1<<22,b3=1<<24,b4=1<<25,b5=1<<27,b6=1<<28;
	const double xstep=boxx/wl_fgrid,ystep=boxy/wl_fgrid,zstep=boxz/wl_fgrid;
	int i,j,k,lx,ly,lz,q;
	unsigned int f,*e=const_cast<unsigned int*> (wl);
	double xlo,ylo,zlo,xhi,yhi,zhi,minr,*radp=mrad;
	for(zlo=0,zhi=zstep,lz=0;lz<wl_hgrid;zlo=zhi,zhi+=zstep,lz++) {
		for(ylo=0,yhi=ystep,ly=0;ly<wl_hgrid;ylo=yhi,yhi+=ystep,ly++) {
			for(xlo=0,xhi=xstep,lx=0;lx<wl_hgrid;xlo=xhi,xhi+=xstep,lx++) {
				minr=large_number;
				for(q=e[0]+1;q<wl_seq_length;q++) {
					f=e[q];
					i=(f&127)-64;
					j=(f>>7&127)-64;
					k=(f>>14&127)-64;
					if((f&b2)==b2) {
						compute_minimum(minr,xlo,xhi,ylo,yhi,zlo,zhi,i-1,j,k);
						if((f&b1)==0) compute_minimum(minr,xlo,xhi,ylo,yhi,zlo,zhi,i+1,j,k);
					} else if((f&b1)==b1) compute_minimum(minr,xlo,xhi,ylo,yhi,zlo,zhi,i+1,j,k);
					if((f&b4)==b4) {
						compute_minimum(minr,xlo,xhi,ylo,yhi,zlo,zhi,i,j-1,k);
						if((f&b3)==0) compute_minimum(minr,xlo,xhi,ylo,yhi,zlo,zhi,i,j+1,k);
					} else if((f&b3)==b3) compute_minimum(minr,xlo,xhi,ylo,yhi,zlo,zhi,i,j+1,k);
					if((f&b6)==b6) {
						compute_minimum(minr,xlo,xhi,ylo,yhi,zlo,zhi,i,j,k-1);
						if((f&b5)==0) compute_minimum(minr,xlo,xhi,ylo,yhi,zlo,zhi,i,j,k+1);
					} else if((f&b5)==b5) compute_minimum(minr,xlo,xhi,ylo,yhi,zlo,zhi,i,j,k+1);
				}
				q--;
				while(q>0) {
					radp[q]=minr;
					f=e[q];
					i=(f&127)-64;
					j=(f>>7&127)-64;
					k=(f>>14&127)-64;
					compute_minimum(minr,xlo,xhi,ylo,yhi,zlo,zhi,i,j,k);
					q--;
				}
				*radp=minr;
				e+=wl_seq_length;
				radp+=wl_seq_length;
			}
		}
	}
}

/** Computes the minimum distance from a subregion to a given block. If this distance
 * is smaller than the value of minr, then it passes
 * \param[in,out] minr a pointer to the current minimum distance. If the distance
 *                     computed in this function is smaller, then this distance is
 *                     set to the new one.
 * \param[out] (xlo,ylo,zlo) the lower coordinates of the subregion being
 *                           considered.
 * \param[out] (xhi,yhi,zhi) the upper coordinates of the subregion being
 *                           considered.
 * \param[in] (ti,tj,tk) the coordinates of the block. */
void voro_base::compute_minimum(double &minr,double &xlo,double &xhi,double &ylo,double &yhi,double &zlo,double &zhi,int ti,int tj,int tk) {
	double radsq,temp;
	if(ti>0) {temp=boxx*ti-xhi;radsq=temp*temp;}
	else if(ti<0) {temp=xlo-boxx*(1+ti);radsq=temp*temp;}
	else radsq=0;

	if(tj>0) {temp=boxy*tj-yhi;radsq+=temp*temp;}
	else if(tj<0) {temp=ylo-boxy*(1+tj);radsq+=temp*temp;}

	if(tk>0) {temp=boxz*tk-zhi;radsq+=temp*temp;}
	else if(tk<0) {temp=zlo-boxz*(1+tk);radsq+=temp*temp;}

	if(radsq<minr) minr=radsq;
}

/** Checks to see whether "%n" appears in a format sequence to determine
 * whether neighbor information is required or not.
 * \param[in] format the format string to check.
 * \return True if a "%n" is found, false otherwise. */
bool voro_base::contains_neighbor(const char *format) const {
	char *fmp=(const_cast<char*>(format));

	// Check to see if "%n" appears in the format sequence
	while(*fmp!=0) {
		if(*fmp=='%') {
			fmp++;
			if(*fmp=='n') return true;
			else if(*fmp==0) return false;
		}
		fmp++;
	}

	return false;
}

#include "v_base_wl.inl"

}
