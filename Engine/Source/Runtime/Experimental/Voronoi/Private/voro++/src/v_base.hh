// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file v_base.hh
 * \brief Header file for the base Voronoi container class. */

#ifndef VOROPP_V_BASE_HH
#define VOROPP_V_BASE_HH

#include "worklist.hh"

namespace voro {

/** \brief Class containing data structures common across all particle container classes.
 *
 * This class contains constants and data structures that are common across all
 * particle container classes. It contains constants setting the size of the
 * underlying subgrid of blocks that forms the basis of the Voronoi cell
 * computations. It also constructs bound tables that are used in the Voronoi
 * cell computation, and contains a number of routines that are common across
 * all container classes. */
class voro_base {
	public:
		/** The number of blocks in the x direction. */
		const int nx;
		/** The number of blocks in the y direction. */
		const int ny;
		/** The number of blocks in the z direction. */
		const int nz;
		/** A constant, set to the value of nx multiplied by ny, which
		 * is used in the routines that step through blocks in
		 * sequence. */
		const int nxy;
		/** A constant, set to the value of nx*ny*nz, which is used in
		 * the routines that step through blocks in sequence. */
		const int nxyz;
		/** The size of a computational block in the x direction. */
		const double boxx;
		/** The size of a computational block in the y direction. */
		const double boxy;
		/** The size of a computational block in the z direction. */
		const double boxz;
		/** The inverse box length in the x direction. */
		const double xsp;
		/** The inverse box length in the y direction. */
		const double ysp;
		/** The inverse box length in the z direction. */
		const double zsp;
		/** An array to hold the minimum distances associated with the
		 * worklists. This array is initialized during container
		 * construction, by the initialize_radii() routine. */
		double *mrad;
		/** The pre-computed block worklists. */
		static const unsigned int wl[wl_seq_length*wl_hgridcu];
		bool contains_neighbor(const char* format) const;
		voro_base(int nx_,int ny_,int nz_,double boxx_,double boxy_,double boxz_);
		voro_base(const voro_base& Other) = delete;
		voro_base& operator=(const voro_base& Other) = delete;
		voro_base(voro_base&& Other) noexcept : nx(Other.nx), ny(Other.ny), nz(Other.nz), nxy(Other.nxy), nxyz(Other.nxyz), boxx(Other.boxx), boxy(Other.boxy), boxz(Other.boxz),
			xsp(Other.xsp), ysp(Other.ysp), zsp(Other.zsp), mrad(Other.mrad)
		{
			Other.mrad = nullptr;
		}
		// Note: Move assignment not implemented due to const members on this class (and it has not been needed yet)
		voro_base& operator=(voro_base&& Other) = delete;

		~voro_base() {delete [] mrad;}
	protected:
		/** A custom int function that returns consistent stepping
		 * for negative numbers, so that (-1.5, -0.5, 0.5, 1.5) maps
		 * to (-2,-1,0,1).
		 * \param[in] a the number to consider.
		 * \return The value of the custom int operation. */
		inline int step_int(double a) const {return a<0?int(a)-1:int(a);}
		/** A custom modulo function that returns consistent stepping
		 * for negative numbers. For example, (-2,-1,0,1,2) step_mod 2
		 * is (0,1,0,1,0).
		 * \param[in] (a,b) the input integers.
		 * \return The value of a modulo b, consistent for negative
		 * numbers. */
		inline int step_mod(int a,int b) const {return a>=0?a%b:b-1-(b-1-a)%b;}
		/** A custom integer division function that returns consistent
		 * stepping for negative numbers. For example, (-2,-1,0,1,2)
		 * step_div 2 is (-1,-1,0,0,1).
		 * \param[in] (a,b) the input integers.
		 * \return The value of a div b, consistent for negative
		 * numbers. */
		inline int step_div(int a,int b) const {return a>=0?a/b:-1+(a+1)/b;}
	private:
		void compute_minimum(double &minr,double &xlo,double &xhi,double &ylo,double &yhi,double &zlo,double &zhi,int ti,int tj,int tk);
};

}

#endif
