// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file v_compute.cc
 * \brief Function implementantions for the voro_compute template. */

#include "worklist.hh"
#include "v_compute.hh"
#include "rad_option.hh"
#include "container.hh"

namespace voro {

/** The class constructor initializes constants from the container class, and
 * sets up the mask and queue used for Voronoi computations.
 * \param[in] con_ a reference to the container class to use.
 * \param[in] (hx_,hy_,hz_) the size of the mask to use. */
template<class c_class>
voro_compute<c_class>::voro_compute(c_class &con_,int hx_,int hy_,int hz_) :
	con(con_), boxx(con_.boxx), boxy(con_.boxy), boxz(con_.boxz),
	xsp(con_.xsp), ysp(con_.ysp), zsp(con_.zsp),
	hx(hx_), hy(hy_), hz(hz_), hxy(hx_*hy_), hxyz(hxy*hz_), ps(con_.ps),
	id(con_.id), p(con_.p), co(con_.co), bxsq(boxx*boxx+boxy*boxy+boxz*boxz),
	mv(0), qu_size(3*(3+hxy+hz*(hx+hy))), wl(con_.wl), mrad(con_.mrad),
	mask(new unsigned int[hxyz]), qu(new int[qu_size]), qu_l(qu+qu_size) {
	reset_mask();
}

/** Scans all of the particles within a block to see if any of them have a
 * smaller distance to the given test vector. If one is found, the routine
 * updates the minimum distance and store information about this particle.
 * \param[in] ijk the index of the block.
 * \param[in] (x,y,z) the test vector to consider (which may have already had a
 *                    periodic displacement applied to it).
 * \param[in] (di,dj,dk) the coordinates of the current block, to store if the
 *			 particle record is updated.
 * \param[in,out] w a reference to a particle record in which to store
 *		    information about the particle whose Voronoi cell the
 *		    vector is within.
 * \param[in,out] mrs the current minimum distance, that may be updated if a
 * 		      closer particle is found. */
template<class c_class>
inline void voro_compute<c_class>::scan_all(int ijk,double x,double y,double z,int di,int dj,int dk,particle_record &w,double &mrs) {
	double x1,y1,z1,rs;bool in_block=false;
	for(int l=0;l<co[ijk];l++) {
		x1=p[ijk][ps*l]-x;
		y1=p[ijk][ps*l+1]-y;
		z1=p[ijk][ps*l+2]-z;
		rs=con.r_current_sub(x1*x1+y1*y1+z1*z1,ijk,l);
		if(rs<mrs) {mrs=rs;w.l=l;in_block=true;}
	}
	if(in_block) {w.ijk=ijk;w.di=di;w.dj=dj,w.dk=dk;}
}

/** Finds the Voronoi cell that given vector is within. For containers that are
 * not radially dependent, this corresponds to findig the particle that is
 * closest to the vector; for the radical tessellation containers, this
 * corresponds to a finding the minimum weighted distance.
 * \param[in] (x,y,z) the vector to consider.
 * \param[in] (ci,cj,ck) the coordinates of the block that the test particle is
 *                       in relative to the container data structure.
 * \param[in] ijk the index of the block that the test particle is in.
 * \param[out] w a reference to a particle record in which to store information
 * 		 about the particle whose Voronoi cell the vector is within.
 * \param[out] mrs the minimum computed distance. */
template<class c_class>
void voro_compute<c_class>::find_voronoi_cell(double x,double y,double z,int ci,int cj,int ck,int ijk,particle_record &w,double &mrs) {
	double qx=0,qy=0,qz=0,rs;
	int i,j,k,di,dj,dk,ei,ej,ek,f,g,disp;
	double fx,fy,fz,mxs,mys,mzs,*radp;
	unsigned int q,*e,*mijk;

	// Init setup for parameters to return
	w.ijk=-1;mrs=large_number;

	con.initialize_search(ci,cj,ck,ijk,i,j,k,disp);

	// Test all particles in the particle's local region first
	scan_all(ijk,x,y,z,0,0,0,w,mrs);

	// Now compute the fractional position of the particle within its
	// region and store it in (fx,fy,fz). We use this to compute an index
	// (di,dj,dk) of which subregion the particle is within.
	unsigned int m1,m2;
	con.frac_pos(x,y,z,ci,cj,ck,fx,fy,fz);
	di=int(fx*xsp*wl_fgrid);dj=int(fy*ysp*wl_fgrid);dk=int(fz*zsp*wl_fgrid);

	// The indices (di,dj,dk) tell us which worklist to use, to test the
	// blocks in the optimal order. But we only store worklists for the
	// eighth of the region where di, dj, and dk are all less than half the
	// full grid. The rest of the cases are handled by symmetry. In this
	// section, we detect for these cases, by reflecting high values of di,
	// dj, and dk. For these cases, a mask is constructed in m1 and m2
	// which is used to flip the worklist information when it is loaded.
	if(di>=wl_hgrid) {
		mxs=boxx-fx;
		m1=127+(3<<21);m2=1+(1<<21);di=wl_fgrid-1-di;if(di<0) di=0;
	} else {m1=m2=0;mxs=fx;}
	if(dj>=wl_hgrid) {
		mys=boxy-fy;
		m1|=(127<<7)+(3<<24);m2|=(1<<7)+(1<<24);dj=wl_fgrid-1-dj;if(dj<0) dj=0;
	} else mys=fy;
	if(dk>=wl_hgrid) {
		mzs=boxz-fz;
		m1|=(127<<14)+(3<<27);m2|=(1<<14)+(1<<27);dk=wl_fgrid-1-dk;if(dk<0) dk=0;
	} else mzs=fz;

	// Do a quick test to account for the case when the minimum radius is
	// small enought that no other blocks need to be considered
	rs=con.r_max_add(mrs);
	if(mxs*mxs>rs&&mys*mys>rs&&mzs*mzs>rs) return;

	// Now compute which worklist we are going to use, and set radp and e to
	// point at the right offsets
	ijk=di+wl_hgrid*(dj+wl_hgrid*dk);
	radp=mrad+ijk*wl_seq_length;
	e=(const_cast<unsigned int*> (wl))+ijk*wl_seq_length;

	// Read in how many items in the worklist can be tested without having to
	// worry about writing to the mask
	f=e[0];g=0;
	do {

		// If mrs is less than the minimum distance to any untested
		// block, then we are done
		if(con.r_max_add(mrs)<radp[g]) return;
		g++;

		// Load in a block off the worklist, permute it with the
		// symmetry mask, and decode its position. These are all
		// integer bit operations so they should run very fast.
		q=e[g];q^=m1;q+=m2;
		di=q&127;di-=64;
		dj=(q>>7)&127;dj-=64;
		dk=(q>>14)&127;dk-=64;

		// Check that the worklist position is in range
		ei=di+i;if(ei<0||ei>=hx) continue;
		ej=dj+j;if(ej<0||ej>=hy) continue;
		ek=dk+k;if(ek<0||ek>=hz) continue;

		// Call the compute_min_max_radius() function. This returns
		// true if the minimum distance to the block is bigger than the
		// current mrs, in which case we skip this block and move on.
		// Otherwise, it computes the maximum distance to the block and
		// returns it in crs.
		if(compute_min_radius(di,dj,dk,fx,fy,fz,mrs)) continue;

		// Now compute which region we are going to loop over, adding a
		// displacement for the periodic cases
		ijk=con.region_index(ci,cj,ck,ei,ej,ek,qx,qy,qz,disp);

		// If mrs is bigger than the maximum distance to the block,
		// then we have to test all particles in the block for
		// intersections. Otherwise, we do additional checks and skip
		// those particles which can't possibly intersect the block.
		scan_all(ijk,x-qx,y-qy,z-qz,di,dj,dk,w,mrs);
	} while(g<f);

	// Update mask value and initialize queue
	mv++;
	if(mv==0) {reset_mask();mv=1;}
	int *qu_s=qu,*qu_e=qu;

	while(g<wl_seq_length-1) {

		// If mrs is less than the minimum distance to any untested
		// block, then we are done
		if(con.r_max_add(mrs)<radp[g]) return;
		g++;

		// Load in a block off the worklist, permute it with the
		// symmetry mask, and decode its position. These are all
		// integer bit operations so they should run very fast.
		q=e[g];q^=m1;q+=m2;
		di=q&127;di-=64;
		dj=(q>>7)&127;dj-=64;
		dk=(q>>14)&127;dk-=64;

		// Compute the position in the mask of the current block. If
		// this lies outside the mask, then skip it. Otherwise, mark
		// it.
		ei=di+i;if(ei<0||ei>=hx) continue;
		ej=dj+j;if(ej<0||ej>=hy) continue;
		ek=dk+k;if(ek<0||ek>=hz) continue;
		mijk=mask+ei+hx*(ej+hy*ek);
		*mijk=mv;

		// Skip this block if it is further away than the current
		// minimum radius
		if(compute_min_radius(di,dj,dk,fx,fy,fz,mrs)) continue;

		// Now compute which region we are going to loop over, adding a
		// displacement for the periodic cases
		ijk=con.region_index(ci,cj,ck,ei,ej,ek,qx,qy,qz,disp);
		scan_all(ijk,x-qx,y-qy,z-qz,di,dj,dk,w,mrs);

		if(qu_e>qu_l-18) add_list_memory(qu_s,qu_e);
		scan_bits_mask_add(q,mijk,ei,ej,ek,qu_e);
	}

	// Do a check to see if we've reached the radius cutoff
	if(con.r_max_add(mrs)<radp[g]) return;

	// We were unable to completely compute the cell based on the blocks in
	// the worklist, so now we have to go block by block, reading in items
	// off the list
	while(qu_s!=qu_e) {

		// Read the next entry of the queue
		if(qu_s==qu_l) qu_s=qu;
		ei=*(qu_s++);ej=*(qu_s++);ek=*(qu_s++);
		di=ei-i;dj=ej-j;dk=ek-k;
		if(compute_min_radius(di,dj,dk,fx,fy,fz,mrs)) continue;

		ijk=con.region_index(ci,cj,ck,ei,ej,ek,qx,qy,qz,disp);
		scan_all(ijk,x-qx,y-qy,z-qz,di,dj,dk,w,mrs);

		// Test the neighbors of the current block, and add them to the
		// block list if they haven't already been tested
		if((qu_s<=qu_e?(qu_l-qu_e)+(qu_s-qu):qu_s-qu_e)<18) add_list_memory(qu_s,qu_e);
		add_to_mask(ei,ej,ek,qu_e);
	}
}

/** Scans the six orthogonal neighbors of a given block and adds them to the
 * queue if they haven't been considered already. It assumes that the queue
 * will definitely have enough memory to add six entries at the end.
 * \param[in] (ei,ej,ek) the block to consider.
 * \param[in,out] qu_e a pointer to the end of the queue. */
template<class c_class>
inline void voro_compute<c_class>::add_to_mask(int ei,int ej,int ek,int *&qu_e) {
	unsigned int *mijk=mask+ei+hx*(ej+hy*ek);
	if(ek>0) if(*(mijk-hxy)!=mv) {if(qu_e==qu_l) qu_e=qu;*(mijk-hxy)=mv;*(qu_e++)=ei;*(qu_e++)=ej;*(qu_e++)=ek-1;}
	if(ej>0) if(*(mijk-hx)!=mv) {if(qu_e==qu_l) qu_e=qu;*(mijk-hx)=mv;*(qu_e++)=ei;*(qu_e++)=ej-1;*(qu_e++)=ek;}
	if(ei>0) if(*(mijk-1)!=mv) {if(qu_e==qu_l) qu_e=qu;*(mijk-1)=mv;*(qu_e++)=ei-1;*(qu_e++)=ej;*(qu_e++)=ek;}
	if(ei<hx-1) if(*(mijk+1)!=mv) {if(qu_e==qu_l) qu_e=qu;*(mijk+1)=mv;*(qu_e++)=ei+1;*(qu_e++)=ej;*(qu_e++)=ek;}
	if(ej<hy-1) if(*(mijk+hx)!=mv) {if(qu_e==qu_l) qu_e=qu;*(mijk+hx)=mv;*(qu_e++)=ei;*(qu_e++)=ej+1;*(qu_e++)=ek;}
	if(ek<hz-1) if(*(mijk+hxy)!=mv) {if(qu_e==qu_l) qu_e=qu;*(mijk+hxy)=mv;*(qu_e++)=ei;*(qu_e++)=ej;*(qu_e++)=ek+1;}
}

/** Scans a worklist entry and adds any blocks to the queue
 * \param[in] (ei,ej,ek) the block to consider.
 * \param[in,out] qu_e a pointer to the end of the queue. */
template<class c_class>
inline void voro_compute<c_class>::scan_bits_mask_add(unsigned int q,unsigned int *mijk,int ei,int ej,int ek,int *&qu_e) {
	const unsigned int b1=1<<21,b2=1<<22,b3=1<<24,b4=1<<25,b5=1<<27,b6=1<<28;
	if((q&b2)==b2) {
		if(ei>0) {*(mijk-1)=mv;*(qu_e++)=ei-1;*(qu_e++)=ej;*(qu_e++)=ek;}
		if((q&b1)==0&&ei<hx-1) {*(mijk+1)=mv;*(qu_e++)=ei+1;*(qu_e++)=ej;*(qu_e++)=ek;}
	} else if((q&b1)==b1&&ei<hx-1) {*(mijk+1)=mv;*(qu_e++)=ei+1;*(qu_e++)=ej;*(qu_e++)=ek;}
	if((q&b4)==b4) {
		if(ej>0) {*(mijk-hx)=mv;*(qu_e++)=ei;*(qu_e++)=ej-1;*(qu_e++)=ek;}
		if((q&b3)==0&&ej<hy-1) {*(mijk+hx)=mv;*(qu_e++)=ei;*(qu_e++)=ej+1;*(qu_e++)=ek;}
	} else if((q&b3)==b3&&ej<hy-1) {*(mijk+hx)=mv;*(qu_e++)=ei;*(qu_e++)=ej+1;*(qu_e++)=ek;}
	if((q&b6)==b6) {
		if(ek>0) {*(mijk-hxy)=mv;*(qu_e++)=ei;*(qu_e++)=ej;*(qu_e++)=ek-1;}
		if((q&b5)==0&&ek<hz-1) {*(mijk+hxy)=mv;*(qu_e++)=ei;*(qu_e++)=ej;*(qu_e++)=ek+1;}
	} else if((q&b5)==b5&&ek<hz-1) {*(mijk+hxy)=mv;*(qu_e++)=ei;*(qu_e++)=ej;*(qu_e++)=ek+1;}
}

/** This routine computes a Voronoi cell for a single particle in the
 * container. It can be called by the user, but is also forms the core part of
 * several of the main functions, such as store_cell_volumes(), print_all(),
 * and the drawing routines. The algorithm constructs the cell by testing over
 * the neighbors of the particle, working outwards until it reaches those
 * particles which could not possibly intersect the cell. For maximum
 * efficiency, this algorithm is divided into three parts. In the first
 * section, the algorithm tests over the blocks which are in the immediate
 * vicinity of the particle, by making use of one of the precomputed worklists.
 * The code then continues to test blocks on the worklist, but also begins to
 * construct a list of neighboring blocks outside the worklist which may need
 * to be test. In the third section, the routine starts testing these
 * neighboring blocks, evaluating whether or not a particle in them could
 * possibly intersect the cell. For blocks that intersect the cell, it tests
 * the particles in that block, and then adds the block neighbors to the list
 * of potential places to consider.
 * \param[in,out] c a reference to a voronoicell object.
 * \param[in] ijk the index of the block that the test particle is in.
 * \param[in] s the index of the particle within the test block.
 * \param[in] (ci,cj,ck) the coordinates of the block that the test particle is
 *                       in relative to the container data structure.
 * \return False if the Voronoi cell was completely removed during the
 *         computation and has zero volume, true otherwise. */
template<class c_class>
template<class v_cell>
bool voro_compute<c_class>::compute_cell(v_cell &c,int ijk,int s,int ci,int cj,int ck) {
	static const int count_list[8]={7,11,15,19,26,35,45,59},*count_e=count_list+8;
	double x,y,z,x1,y1,z1,qx=0,qy=0,qz=0;
	double xlo,ylo,zlo,xhi,yhi,zhi,x2,y2,z2,rs;
	int i,j,k,di,dj,dk,ei,ej,ek,f,g,l,disp;
	double fx,fy,fz,gxs,gys,gzs,*radp;
	unsigned int q,*e,*mijk;

	if(!con.initialize_voronoicell(c,ijk,s,ci,cj,ck,i,j,k,x,y,z,disp)) return false;
	typename c_class::radius_info radi;
	con.r_init(ijk,s,radi);

	// Initialize the Voronoi cell to fill the entire container
	double crs,mrs;

	int next_count=3,*count_p=(const_cast<int*> (count_list));

	// Test all particles in the particle's local region first
	for(l=0;l<s;l++) {
		x1=p[ijk][ps*l]-x;
		y1=p[ijk][ps*l+1]-y;
		z1=p[ijk][ps*l+2]-z;
		rs=con.r_scale(x1*x1+y1*y1+z1*z1,ijk,l,radi);
		if(!c.nplane(x1,y1,z1,rs,id[ijk][l])) return false;
	}
	l++;
	while(l<co[ijk]) {
		x1=p[ijk][ps*l]-x;
		y1=p[ijk][ps*l+1]-y;
		z1=p[ijk][ps*l+2]-z;
		rs=con.r_scale(x1*x1+y1*y1+z1*z1,ijk,l,radi);
		if(!c.nplane(x1,y1,z1,rs,id[ijk][l])) return false;
		l++;
	}

	// Now compute the maximum distance squared from the cell center to a
	// vertex. This is used to cut off the calculation since we only need
	// to test out to twice this range.
	mrs=c.max_radius_squared();

	// Now compute the fractional position of the particle within its
	// region and store it in (fx,fy,fz). We use this to compute an index
	// (di,dj,dk) of which subregion the particle is within.
	unsigned int m1,m2;
	con.frac_pos(x,y,z,ci,cj,ck,fx,fy,fz);
	di=int(fx*xsp*wl_fgrid);dj=int(fy*ysp*wl_fgrid);dk=int(fz*zsp*wl_fgrid);

	// The indices (di,dj,dk) tell us which worklist to use, to test the
	// blocks in the optimal order. But we only store worklists for the
	// eighth of the region where di, dj, and dk are all less than half the
	// full grid. The rest of the cases are handled by symmetry. In this
	// section, we detect for these cases, by reflecting high values of di,
	// dj, and dk. For these cases, a mask is constructed in m1 and m2
	// which is used to flip the worklist information when it is loaded.
	if(di>=wl_hgrid) {
		gxs=fx;
		m1=127+(3<<21);m2=1+(1<<21);di=wl_fgrid-1-di;if(di<0) di=0;
	} else {m1=m2=0;gxs=boxx-fx;}
	if(dj>=wl_hgrid) {
		gys=fy;
		m1|=(127<<7)+(3<<24);m2|=(1<<7)+(1<<24);dj=wl_fgrid-1-dj;if(dj<0) dj=0;
	} else gys=boxy-fy;
	if(dk>=wl_hgrid) {
		gzs=fz;
		m1|=(127<<14)+(3<<27);m2|=(1<<14)+(1<<27);dk=wl_fgrid-1-dk;if(dk<0) dk=0;
	} else gzs=boxz-fz;
	gxs*=gxs;gys*=gys;gzs*=gzs;

	// Now compute which worklist we are going to use, and set radp and e to
	// point at the right offsets
	ijk=di+wl_hgrid*(dj+wl_hgrid*dk);
	radp=mrad+ijk*wl_seq_length;
	e=(const_cast<unsigned int*> (wl))+ijk*wl_seq_length;

	// Read in how many items in the worklist can be tested without having to
	// worry about writing to the mask
	f=e[0];g=0;
	do {

		// At the intervals specified by count_list, we recompute the
		// maximum radius squared
		if(g==next_count) {
			mrs=c.max_radius_squared();
			if(count_p!=count_e) next_count=*(count_p++);
		}

		// If mrs is less than the minimum distance to any untested
		// block, then we are done
		if(con.r_ctest(radp[g],mrs,radi)) return true;
		g++;

		// Load in a block off the worklist, permute it with the
		// symmetry mask, and decode its position. These are all
		// integer bit operations so they should run very fast.
		q=e[g];q^=m1;q+=m2;
		di=q&127;di-=64;
		dj=(q>>7)&127;dj-=64;
		dk=(q>>14)&127;dk-=64;

		// Check that the worklist position is in range
		ei=di+i;if(ei<0||ei>=hx) continue;
		ej=dj+j;if(ej<0||ej>=hy) continue;
		ek=dk+k;if(ek<0||ek>=hz) continue;

		// Call the compute_min_max_radius() function. This returns
		// true if the minimum distance to the block is bigger than the
		// current mrs, in which case we skip this block and move on.
		// Otherwise, it computes the maximum distance to the block and
		// returns it in crs.
		if(compute_min_max_radius(di,dj,dk,fx,fy,fz,gxs,gys,gzs,crs,mrs,radi)) continue;

		// Now compute which region we are going to loop over, adding a
		// displacement for the periodic cases
		ijk=con.region_index(ci,cj,ck,ei,ej,ek,qx,qy,qz,disp);

		// If mrs is bigger than the maximum distance to the block,
		// then we have to test all particles in the block for
		// intersections. Otherwise, we do additional checks and skip
		// those particles which can't possibly intersect the block.
		if(co[ijk]>0) {
			l=0;x2=x-qx;y2=y-qy;z2=z-qz;
			if(!con.r_ctest(crs,mrs,radi)) {
				do {
					x1=p[ijk][ps*l]-x2;
					y1=p[ijk][ps*l+1]-y2;
					z1=p[ijk][ps*l+2]-z2;
					rs=con.r_scale(x1*x1+y1*y1+z1*z1,ijk,l,radi);
					if(!c.nplane(x1,y1,z1,rs,id[ijk][l])) return false;
					l++;
				} while (l<co[ijk]);
			} else {
				do {
					x1=p[ijk][ps*l]-x2;
					y1=p[ijk][ps*l+1]-y2;
					z1=p[ijk][ps*l+2]-z2;
					rs=x1*x1+y1*y1+z1*z1;
					if(con.r_scale_check(rs,mrs,ijk,l,radi)&&!c.nplane(x1,y1,z1,rs,id[ijk][l])) return false;
					l++;
				} while (l<co[ijk]);
			}
		}
	} while(g<f);

	// If we reach here, we were unable to compute the entire cell using
	// the first part of the worklist. This section of the algorithm
	// continues the worklist, but it now starts preparing the mask that we
	// need if we end up going block by block. We do the same as before,
	// but we put a mark down on the mask for every block that's tested.
	// The worklist also contains information about which neighbors of each
	// block are not also on the worklist, and we start storing those
	// points in a list in case we have to go block by block. Update the
	// mask counter, and if it wraps around then reset the whole mask; that
	// will only happen once every 2^32 tries.
	mv++;
	if(mv==0) {reset_mask();mv=1;}

	// Set the queue pointers
	int *qu_s=qu,*qu_e=qu;

	while(g<wl_seq_length-1) {

		// At the intervals specified by count_list, we recompute the
		// maximum radius squared
		if(g==next_count) {
			mrs=c.max_radius_squared();
			if(count_p!=count_e) next_count=*(count_p++);
		}

		// If mrs is less than the minimum distance to any untested
		// block, then we are done
		if(con.r_ctest(radp[g],mrs,radi)) return true;
		g++;

		// Load in a block off the worklist, permute it with the
		// symmetry mask, and decode its position. These are all
		// integer bit operations so they should run very fast.
		q=e[g];q^=m1;q+=m2;
		di=q&127;di-=64;
		dj=(q>>7)&127;dj-=64;
		dk=(q>>14)&127;dk-=64;

		// Compute the position in the mask of the current block. If
		// this lies outside the mask, then skip it. Otherwise, mark
		// it.
		ei=di+i;if(ei<0||ei>=hx) continue;
		ej=dj+j;if(ej<0||ej>=hy) continue;
		ek=dk+k;if(ek<0||ek>=hz) continue;
		mijk=mask+ei+hx*(ej+hy*ek);
		*mijk=mv;

		// Call the compute_min_max_radius() function. This returns
		// true if the minimum distance to the block is bigger than the
		// current mrs, in which case we skip this block and move on.
		// Otherwise, it computes the maximum distance to the block and
		// returns it in crs.
		if(compute_min_max_radius(di,dj,dk,fx,fy,fz,gxs,gys,gzs,crs,mrs,radi)) continue;

		// Now compute which region we are going to loop over, adding a
		// displacement for the periodic cases
		ijk=con.region_index(ci,cj,ck,ei,ej,ek,qx,qy,qz,disp);

		// If mrs is bigger than the maximum distance to the block,
		// then we have to test all particles in the block for
		// intersections. Otherwise, we do additional checks and skip
		// those particles which can't possibly intersect the block.
		if(co[ijk]>0) {
			l=0;x2=x-qx;y2=y-qy;z2=z-qz;
			if(!con.r_ctest(crs,mrs,radi)) {
				do {
					x1=p[ijk][ps*l]-x2;
					y1=p[ijk][ps*l+1]-y2;
					z1=p[ijk][ps*l+2]-z2;
					rs=con.r_scale(x1*x1+y1*y1+z1*z1,ijk,l,radi);
					if(!c.nplane(x1,y1,z1,rs,id[ijk][l])) return false;
					l++;
				} while (l<co[ijk]);
			} else {
				do {
					x1=p[ijk][ps*l]-x2;
					y1=p[ijk][ps*l+1]-y2;
					z1=p[ijk][ps*l+2]-z2;
					rs=x1*x1+y1*y1+z1*z1;
					if(con.r_scale_check(rs,mrs,ijk,l,radi)&&!c.nplane(x1,y1,z1,rs,id[ijk][l])) return false;
					l++;
				} while (l<co[ijk]);
			}
		}

		// If there might not be enough memory on the list for these
		// additions, then add more
		if(qu_e>qu_l-18) add_list_memory(qu_s,qu_e);

		// Test the parts of the worklist element which tell us what
		// neighbors of this block are not on the worklist. Store them
		// on the block list, and mark the mask.
		scan_bits_mask_add(q,mijk,ei,ej,ek,qu_e);
	}

	// Do a check to see if we've reached the radius cutoff
	if(con.r_ctest(radp[g],mrs,radi)) return true;

	// We were unable to completely compute the cell based on the blocks in
	// the worklist, so now we have to go block by block, reading in items
	// off the list
	while(qu_s!=qu_e) {

		// If we reached the end of the list memory loop back to the
		// start
		if(qu_s==qu_l) qu_s=qu;

		// Read in a block off the list, and compute the upper and lower
		// coordinates in each of the three dimensions
		ei=*(qu_s++);ej=*(qu_s++);ek=*(qu_s++);
		xlo=(ei-i)*boxx-fx;xhi=xlo+boxx;
		ylo=(ej-j)*boxy-fy;yhi=ylo+boxy;
		zlo=(ek-k)*boxz-fz;zhi=zlo+boxz;

		// Carry out plane tests to see if any particle in this block
		// could possibly intersect the cell
		if(ei>i) {
			if(ej>j) {
				if(ek>k) {if(corner_test(c,xlo,ylo,zlo,xhi,yhi,zhi,radi)) continue;}
				else if(ek<k) {if(corner_test(c,xlo,ylo,zhi,xhi,yhi,zlo,radi)) continue;}
				else {if(edge_z_test(c,xlo,ylo,zlo,xhi,yhi,zhi,radi)) continue;}
			} else if(ej<j) {
				if(ek>k) {if(corner_test(c,xlo,yhi,zlo,xhi,ylo,zhi,radi)) continue;}
				else if(ek<k) {if(corner_test(c,xlo,yhi,zhi,xhi,ylo,zlo,radi)) continue;}
				else {if(edge_z_test(c,xlo,yhi,zlo,xhi,ylo,zhi,radi)) continue;}
			} else {
				if(ek>k) {if(edge_y_test(c,xlo,ylo,zlo,xhi,yhi,zhi,radi)) continue;}
				else if(ek<k) {if(edge_y_test(c,xlo,ylo,zhi,xhi,yhi,zlo,radi)) continue;}
				else {if(face_x_test(c,xlo,ylo,zlo,yhi,zhi,radi)) continue;}
			}
		} else if(ei<i) {
			if(ej>j) {
				if(ek>k) {if(corner_test(c,xhi,ylo,zlo,xlo,yhi,zhi,radi)) continue;}
				else if(ek<k) {if(corner_test(c,xhi,ylo,zhi,xlo,yhi,zlo,radi)) continue;}
				else {if(edge_z_test(c,xhi,ylo,zlo,xlo,yhi,zhi,radi)) continue;}
			} else if(ej<j) {
				if(ek>k) {if(corner_test(c,xhi,yhi,zlo,xlo,ylo,zhi,radi)) continue;}
				else if(ek<k) {if(corner_test(c,xhi,yhi,zhi,xlo,ylo,zlo,radi)) continue;}
				else {if(edge_z_test(c,xhi,yhi,zlo,xlo,ylo,zhi,radi)) continue;}
			} else {
				if(ek>k) {if(edge_y_test(c,xhi,ylo,zlo,xlo,yhi,zhi,radi)) continue;}
				else if(ek<k) {if(edge_y_test(c,xhi,ylo,zhi,xlo,yhi,zlo,radi)) continue;}
				else {if(face_x_test(c,xhi,ylo,zlo,yhi,zhi,radi)) continue;}
			}
		} else {
			if(ej>j) {
				if(ek>k) {if(edge_x_test(c,xlo,ylo,zlo,xhi,yhi,zhi,radi)) continue;}
				else if(ek<k) {if(edge_x_test(c,xlo,ylo,zhi,xhi,yhi,zlo,radi)) continue;}
				else {if(face_y_test(c,xlo,ylo,zlo,xhi,zhi,radi)) continue;}
			} else if(ej<j) {
				if(ek>k) {if(edge_x_test(c,xlo,yhi,zlo,xhi,ylo,zhi,radi)) continue;}
				else if(ek<k) {if(edge_x_test(c,xlo,yhi,zhi,xhi,ylo,zlo,radi)) continue;}
				else {if(face_y_test(c,xlo,yhi,zlo,xhi,zhi,radi)) continue;}
			} else {
				if(ek>k) {if(face_z_test(c,xlo,ylo,zlo,xhi,yhi,radi)) continue;}
				else if(ek<k) {if(face_z_test(c,xlo,ylo,zhi,xhi,yhi,radi)) continue;}
				else voro_fatal_error("Compute cell routine revisiting central block, which should never\nhappen.",VOROPP_INTERNAL_ERROR);
			}
		}

		// Now compute the region that we are going to test over, and
		// set a displacement vector for the periodic cases
		ijk=con.region_index(ci,cj,ck,ei,ej,ek,qx,qy,qz,disp);

		// Loop over all the elements in the block to test for cuts. It
		// would be possible to exclude some of these cases by testing
		// against mrs, but this will probably not save time.
		if(co[ijk]>0) {
			l=0;x2=x-qx;y2=y-qy;z2=z-qz;
			do {
				x1=p[ijk][ps*l]-x2;
				y1=p[ijk][ps*l+1]-y2;
				z1=p[ijk][ps*l+2]-z2;
				rs=con.r_scale(x1*x1+y1*y1+z1*z1,ijk,l,radi);
				if(!c.nplane(x1,y1,z1,rs,id[ijk][l])) return false;
				l++;
			} while (l<co[ijk]);
		}

		// If there's not much memory on the block list then add more
		if((qu_s<=qu_e?(qu_l-qu_e)+(qu_s-qu):qu_s-qu_e)<18) add_list_memory(qu_s,qu_e);

		// Test the neighbors of the current block, and add them to the
		// block list if they haven't already been tested
		add_to_mask(ei,ej,ek,qu_e);
	}

	return true;
}

/** This function checks to see whether a particular block can possibly have
 * any intersection with a Voronoi cell, for the case when the closest point
 * from the cell center to the block is at a corner.
 * \param[in,out] c a reference to a Voronoi cell.
 * \param[in] (xl,yl,zl) the relative coordinates of the corner of the block
 *                       closest to the cell center.
 * \param[in] (xh,yh,zh) the relative coordinates of the corner of the block
 *                       furthest away from the cell center.
 * \return False if the block may intersect, true if does not. */
template<class c_class>
template<class v_cell>
bool voro_compute<c_class>::corner_test(v_cell &c,double xl,double yl,double zl,double xh,double yh,double zh,typename c_class::radius_info& radi) const {
	con.r_prime(xl*xl+yl*yl+zl*zl,radi);
	if(c.plane_intersects_guess(xh,yl,zl,con.r_cutoff(xl*xh+yl*yl+zl*zl,radi))) return false;
	if(c.plane_intersects(xh,yh,zl,con.r_cutoff(xl*xh+yl*yh+zl*zl,radi))) return false;
	if(c.plane_intersects(xl,yh,zl,con.r_cutoff(xl*xl+yl*yh+zl*zl,radi))) return false;
	if(c.plane_intersects(xl,yh,zh,con.r_cutoff(xl*xl+yl*yh+zl*zh,radi))) return false;
	if(c.plane_intersects(xl,yl,zh,con.r_cutoff(xl*xl+yl*yl+zl*zh,radi))) return false;
	if(c.plane_intersects(xh,yl,zh,con.r_cutoff(xl*xh+yl*yl+zl*zh,radi))) return false;
	return true;
}

/** This function checks to see whether a particular block can possibly have
 * any intersection with a Voronoi cell, for the case when the closest point
 * from the cell center to the block is on an edge which points along the x
 * direction.
 * \param[in,out] c a reference to a Voronoi cell.
 * \param[in] (x0,x1) the minimum and maximum relative x coordinates of the
 *                    block.
 * \param[in] (yl,zl) the relative y and z coordinates of the corner of the
 *                    block closest to the cell center.
 * \param[in] (yh,zh) the relative y and z coordinates of the corner of the
 *                    block furthest away from the cell center.
 * \return False if the block may intersect, true if does not. */
template<class c_class>
template<class v_cell>
inline bool voro_compute<c_class>::edge_x_test(v_cell &c,double x0,double yl,double zl,double x1,double yh,double zh,typename c_class::radius_info& radi) const {
	con.r_prime(yl*yl+zl*zl,radi);
	if(c.plane_intersects_guess(x0,yl,zh,con.r_cutoff(yl*yl+zl*zh,radi))) return false;
	if(c.plane_intersects(x1,yl,zh,con.r_cutoff(yl*yl+zl*zh,radi))) return false;
	if(c.plane_intersects(x1,yl,zl,con.r_cutoff(yl*yl+zl*zl,radi))) return false;
	if(c.plane_intersects(x0,yl,zl,con.r_cutoff(yl*yl+zl*zl,radi))) return false;
	if(c.plane_intersects(x0,yh,zl,con.r_cutoff(yl*yh+zl*zl,radi))) return false;
	if(c.plane_intersects(x1,yh,zl,con.r_cutoff(yl*yh+zl*zl,radi))) return false;
	return true;
}

/** This function checks to see whether a particular block can possibly have
 * any intersection with a Voronoi cell, for the case when the closest point
 * from the cell center to the block is on an edge which points along the y
 * direction.
 * \param[in,out] c a reference to a Voronoi cell.
 * \param[in] (y0,y1) the minimum and maximum relative y coordinates of the
 *                    block.
 * \param[in] (xl,zl) the relative x and z coordinates of the corner of the
 *                    block closest to the cell center.
 * \param[in] (xh,zh) the relative x and z coordinates of the corner of the
 *                    block furthest away from the cell center.
 * \return False if the block may intersect, true if does not. */
template<class c_class>
template<class v_cell>
inline bool voro_compute<c_class>::edge_y_test(v_cell &c,double xl,double y0,double zl,double xh,double y1,double zh,typename c_class::radius_info& radi) const {
	con.r_prime(xl*xl+zl*zl,radi);
	if(c.plane_intersects_guess(xl,y0,zh,con.r_cutoff(xl*xl+zl*zh,radi))) return false;
	if(c.plane_intersects(xl,y1,zh,con.r_cutoff(xl*xl+zl*zh,radi))) return false;
	if(c.plane_intersects(xl,y1,zl,con.r_cutoff(xl*xl+zl*zl,radi))) return false;
	if(c.plane_intersects(xl,y0,zl,con.r_cutoff(xl*xl+zl*zl,radi))) return false;
	if(c.plane_intersects(xh,y0,zl,con.r_cutoff(xl*xh+zl*zl,radi))) return false;
	if(c.plane_intersects(xh,y1,zl,con.r_cutoff(xl*xh+zl*zl,radi))) return false;
	return true;
}

/** This function checks to see whether a particular block can possibly have
 * any intersection with a Voronoi cell, for the case when the closest point
 * from the cell center to the block is on an edge which points along the z
 * direction.
 * \param[in,out] c a reference to a Voronoi cell.
 * \param[in] (z0,z1) the minimum and maximum relative z coordinates of the block.
 * \param[in] (xl,yl) the relative x and y coordinates of the corner of the
 *                    block closest to the cell center.
 * \param[in] (xh,yh) the relative x and y coordinates of the corner of the
 *                    block furthest away from the cell center.
 * \return False if the block may intersect, true if does not. */
template<class c_class>
template<class v_cell>
inline bool voro_compute<c_class>::edge_z_test(v_cell &c,double xl,double yl,double z0,double xh,double yh,double z1,typename c_class::radius_info& radi) const {
	con.r_prime(xl*xl+yl*yl,radi);
	if(c.plane_intersects_guess(xl,yh,z0,con.r_cutoff(xl*xl+yl*yh,radi))) return false;
	if(c.plane_intersects(xl,yh,z1,con.r_cutoff(xl*xl+yl*yh,radi))) return false;
	if(c.plane_intersects(xl,yl,z1,con.r_cutoff(xl*xl+yl*yl,radi))) return false;
	if(c.plane_intersects(xl,yl,z0,con.r_cutoff(xl*xl+yl*yl,radi))) return false;
	if(c.plane_intersects(xh,yl,z0,con.r_cutoff(xl*xh+yl*yl,radi))) return false;
	if(c.plane_intersects(xh,yl,z1,con.r_cutoff(xl*xh+yl*yl,radi))) return false;
	return true;
}

/** This function checks to see whether a particular block can possibly have
 * any intersection with a Voronoi cell, for the case when the closest point
 * from the cell center to the block is on a face aligned with the x direction.
 * \param[in,out] c a reference to a Voronoi cell.
 * \param[in] xl the minimum distance from the cell center to the face.
 * \param[in] (y0,y1) the minimum and maximum relative y coordinates of the
 *                    block.
 * \param[in] (z0,z1) the minimum and maximum relative z coordinates of the
 *                    block.
 * \return False if the block may intersect, true if does not. */
template<class c_class>
template<class v_cell>
inline bool voro_compute<c_class>::face_x_test(v_cell &c,double xl,double y0,double z0,double y1,double z1,typename c_class::radius_info& radi) const {
	con.r_prime(xl*xl,radi);
	if(c.plane_intersects_guess(xl,y0,z0,con.r_cutoff(xl*xl,radi))) return false;
	if(c.plane_intersects(xl,y0,z1,con.r_cutoff(xl*xl,radi))) return false;
	if(c.plane_intersects(xl,y1,z1,con.r_cutoff(xl*xl,radi))) return false;
	if(c.plane_intersects(xl,y1,z0,con.r_cutoff(xl*xl,radi))) return false;
	return true;
}

/** This function checks to see whether a particular block can possibly have
 * any intersection with a Voronoi cell, for the case when the closest point
 * from the cell center to the block is on a face aligned with the y direction.
 * \param[in,out] c a reference to a Voronoi cell.
 * \param[in] yl the minimum distance from the cell center to the face.
 * \param[in] (x0,x1) the minimum and maximum relative x coordinates of the
 *                    block.
 * \param[in] (z0,z1) the minimum and maximum relative z coordinates of the
 *                    block.
 * \return False if the block may intersect, true if does not. */
template<class c_class>
template<class v_cell>
inline bool voro_compute<c_class>::face_y_test(v_cell &c,double x0,double yl,double z0,double x1,double z1,typename c_class::radius_info& radi) const {
	con.r_prime(yl*yl,radi);
	if(c.plane_intersects_guess(x0,yl,z0,con.r_cutoff(yl*yl,radi))) return false;
	if(c.plane_intersects(x0,yl,z1,con.r_cutoff(yl*yl,radi))) return false;
	if(c.plane_intersects(x1,yl,z1,con.r_cutoff(yl*yl,radi))) return false;
	if(c.plane_intersects(x1,yl,z0,con.r_cutoff(yl*yl,radi))) return false;
	return true;
}

/** This function checks to see whether a particular block can possibly have
 * any intersection with a Voronoi cell, for the case when the closest point
 * from the cell center to the block is on a face aligned with the z direction.
 * \param[in,out] c a reference to a Voronoi cell.
 * \param[in] zl the minimum distance from the cell center to the face.
 * \param[in] (x0,x1) the minimum and maximum relative x coordinates of the
 *                    block.
 * \param[in] (y0,y1) the minimum and maximum relative y coordinates of the
 *                    block.
 * \return False if the block may intersect, true if does not. */
template<class c_class>
template<class v_cell>
inline bool voro_compute<c_class>::face_z_test(v_cell &c,double x0,double y0,double zl,double x1,double y1,typename c_class::radius_info& radi) const {
	con.r_prime(zl*zl,radi);
	if(c.plane_intersects_guess(x0,y0,zl,con.r_cutoff(zl*zl,radi))) return false;
	if(c.plane_intersects(x0,y1,zl,con.r_cutoff(zl*zl,radi))) return false;
	if(c.plane_intersects(x1,y1,zl,con.r_cutoff(zl*zl,radi))) return false;
	if(c.plane_intersects(x1,y0,zl,con.r_cutoff(zl*zl,radi))) return false;
	return true;
}


/** This routine checks to see whether a point is within a particular distance
 * of a nearby region. If the point is within the distance of the region, then
 * the routine returns true, and computes the maximum distance from the point
 * to the region. Otherwise, the routine returns false.
 * \param[in] (di,dj,dk) the position of the nearby region to be tested,
 *                       relative to the region that the point is in.
 * \param[in] (fx,fy,fz) the displacement of the point within its region.
 * \param[in] (gxs,gys,gzs) the maximum squared distances from the point to the
 *                          sides of its region.
 * \param[out] crs a reference in which to return the maximum distance to the
 *                 region (only computed if the routine returns false).
 * \param[in] mrs the distance to be tested.
 * \return True if the region is further away than mrs, false if the region in
 *         within mrs. */
template<class c_class>
bool voro_compute<c_class>::compute_min_max_radius(int di,int dj,int dk,double fx,double fy,double fz,double gxs,double gys,double gzs,double &crs,double mrs,const typename c_class::radius_info& radi) const {
	double xlo,ylo,zlo;
	if(di>0) {
		xlo=di*boxx-fx;
		crs=xlo*xlo;
		if(dj>0) {
			ylo=dj*boxy-fy;
			crs+=ylo*ylo;
			if(dk>0) {
				zlo=dk*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=bxsq+2*(boxx*xlo+boxy*ylo+boxz*zlo);
			} else if(dk<0) {
				zlo=(dk+1)*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=bxsq+2*(boxx*xlo+boxy*ylo-boxz*zlo);
			} else {
				if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxx*(2*xlo+boxx)+boxy*(2*ylo+boxy)+gzs;
			}
		} else if(dj<0) {
			ylo=(dj+1)*boxy-fy;
			crs+=ylo*ylo;
			if(dk>0) {
				zlo=dk*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=bxsq+2*(boxx*xlo-boxy*ylo+boxz*zlo);
			} else if(dk<0) {
				zlo=(dk+1)*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=bxsq+2*(boxx*xlo-boxy*ylo-boxz*zlo);
			} else {
				if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxx*(2*xlo+boxx)+boxy*(-2*ylo+boxy)+gzs;
			}
		} else {
			if(dk>0) {
				zlo=dk*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxz*(2*zlo+boxz);
			} else if(dk<0) {
				zlo=(dk+1)*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxz*(-2*zlo+boxz);
			} else {
				if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=gzs;
			}
			crs+=gys+boxx*(2*xlo+boxx);
		}
	} else if(di<0) {
		xlo=(di+1)*boxx-fx;
		crs=xlo*xlo;
		if(dj>0) {
			ylo=dj*boxy-fy;
			crs+=ylo*ylo;
			if(dk>0) {
				zlo=dk*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=bxsq+2*(-boxx*xlo+boxy*ylo+boxz*zlo);
			} else if(dk<0) {
				zlo=(dk+1)*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=bxsq+2*(-boxx*xlo+boxy*ylo-boxz*zlo);
			} else {
				if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxx*(-2*xlo+boxx)+boxy*(2*ylo+boxy)+gzs;
			}
		} else if(dj<0) {
			ylo=(dj+1)*boxy-fy;
			crs+=ylo*ylo;
			if(dk>0) {
				zlo=dk*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=bxsq+2*(-boxx*xlo-boxy*ylo+boxz*zlo);
			} else if(dk<0) {
				zlo=(dk+1)*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=bxsq+2*(-boxx*xlo-boxy*ylo-boxz*zlo);
			} else {
				if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxx*(-2*xlo+boxx)+boxy*(-2*ylo+boxy)+gzs;
			}
		} else {
			if(dk>0) {
				zlo=dk*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxz*(2*zlo+boxz);
			} else if(dk<0) {
				zlo=(dk+1)*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxz*(-2*zlo+boxz);
			} else {
				if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=gzs;
			}
			crs+=gys+boxx*(-2*xlo+boxx);
		}
	} else {
		if(dj>0) {
			ylo=dj*boxy-fy;
			crs=ylo*ylo;
			if(dk>0) {
				zlo=dk*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxz*(2*zlo+boxz);
			} else if(dk<0) {
				zlo=(dk+1)*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxz*(-2*zlo+boxz);
			} else {
				if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=gzs;
			}
			crs+=boxy*(2*ylo+boxy);
		} else if(dj<0) {
			ylo=(dj+1)*boxy-fy;
			crs=ylo*ylo;
			if(dk>0) {
				zlo=dk*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxz*(2*zlo+boxz);
			} else if(dk<0) {
				zlo=(dk+1)*boxz-fz;
				crs+=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxz*(-2*zlo+boxz);
			} else {
				if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=gzs;
			}
			crs+=boxy*(-2*ylo+boxy);
		} else {
			if(dk>0) {
				zlo=dk*boxz-fz;crs=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxz*(2*zlo+boxz);
			} else if(dk<0) {
				zlo=(dk+1)*boxz-fz;crs=zlo*zlo;if(con.r_ctest(crs,mrs,radi)) return true;
				crs+=boxz*(-2*zlo+boxz);
			} else {
				crs=0;
				voro_fatal_error("Min/max radius function called for central block, which should never\nhappen.",VOROPP_INTERNAL_ERROR);
			}
			crs+=gys;
		}
		crs+=gxs;
	}
	return false;
}

template<class c_class>
bool voro_compute<c_class>::compute_min_radius(int di,int dj,int dk,double fx,double fy,double fz,double mrs) const {
	double t,crs;

	if(di>0) {t=di*boxx-fx;crs=t*t;}
	else if(di<0) {t=(di+1)*boxx-fx;crs=t*t;}
	else crs=0;

	if(dj>0) {t=dj*boxy-fy;crs+=t*t;}
	else if(dj<0) {t=(dj+1)*boxy-fy;crs+=t*t;}

	if(dk>0) {t=dk*boxz-fz;crs+=t*t;}
	else if(dk<0) {t=(dk+1)*boxz-fz;crs+=t*t;}

	return crs>con.r_max_add(mrs);
}

/** Adds memory to the queue.
 * \param[in,out] qu_s a reference to the queue start pointer.
 * \param[in,out] qu_e a reference to the queue end pointer. */
template<class c_class>
inline void voro_compute<c_class>::add_list_memory(int*& qu_s,int*& qu_e) {
	qu_size<<=1;
	int *qu_n=new int[qu_size],*qu_c=qu_n;
#if VOROPP_VERBOSE >=2
	fprintf(stderr,"List memory scaled up to %d\n",qu_size);
#endif
	if(qu_s<=qu_e) {
		while(qu_s<qu_e) *(qu_c++)=*(qu_s++);
	} else {
		while(qu_s<qu_l) *(qu_c++)=*(qu_s++);qu_s=qu;
		while(qu_s<qu_e) *(qu_c++)=*(qu_s++);
	}
	delete [] qu;
	qu_s=qu=qu_n;
	qu_l=qu+qu_size;
	qu_e=qu_c;
}

// Explicit template instantiation
template voro_compute<container>::voro_compute(container&,int,int,int);
template voro_compute<container_poly>::voro_compute(container_poly&,int,int,int);
template bool voro_compute<container>::compute_cell(voronoicell&,int,int,int,int,int);
template bool voro_compute<container>::compute_cell(voronoicell_neighbor&,int,int,int,int,int);
template void voro_compute<container>::find_voronoi_cell(double,double,double,int,int,int,int,particle_record&,double&);
template bool voro_compute<container_poly>::compute_cell(voronoicell&,int,int,int,int,int);
template bool voro_compute<container_poly>::compute_cell(voronoicell_neighbor&,int,int,int,int,int);
template void voro_compute<container_poly>::find_voronoi_cell(double,double,double,int,int,int,int,particle_record&,double&);

}
