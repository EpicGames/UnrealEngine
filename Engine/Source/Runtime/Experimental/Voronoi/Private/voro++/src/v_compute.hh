// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file v_compute.hh
 * \brief Header file for the voro_compute template and related classes. */

#ifndef VOROPP_V_COMPUTE_HH
#define VOROPP_V_COMPUTE_HH

#include "config.hh"
#include "worklist.hh"
#include "cell.hh"

namespace voro {

/** \brief Structure for holding information about a particle.
 *
 * This small structure holds information about a single particle, and is used
 * by several of the routines in the voro_compute template for passing
 * information by reference between functions. */
struct particle_record {
	/** The index of the block that the particle is within. */
	int ijk;
	/** The number of particle within its block. */
	int l;
	/** The x-index of the block. */
	int di;
	/** The y-index of the block. */
	int dj;
	/** The z-index of the block. */
	int dk;
};

/** \brief Template for carrying out Voronoi cell computations. */
template <class c_class>
class voro_compute {
	public:
		/** A reference to the container class on which to carry out*/
		const c_class &con;
		/** The size of an internal computational block in the x
		 * direction. */
		const double boxx;
		/** The size of an internal computational block in the y
		 * direction. */
		const double boxy;
		/** The size of an internal computational block in the z
		 * direction. */
		const double boxz;
		/** The inverse box length in the x direction, set to
		 * nx/(bx-ax). */
		const double xsp;
		/** The inverse box length in the y direction, set to
		 * ny/(by-ay). */
		const double ysp;
		/** The inverse box length in the z direction, set to
		 * nz/(bz-az). */
		const double zsp;
		/** The number of boxes in the x direction for the searching mask. */
		const int hx;
		/** The number of boxes in the y direction for the searching mask. */
		const int hy;
		/** The number of boxes in the z direction for the searching mask. */
		const int hz;
		/** A constant, set to the value of hx multiplied by hy, which
		 * is used in the routines which step through mask boxes in
		 * sequence. */
		const int hxy;
		/** A constant, set to the value of hx*hy*hz, which is used in
		 * the routines which step through mask boxes in sequence. */
		const int hxyz;
		/** The number of floating point entries to store for each
		 * particle. */
		const int ps;

		// TODO: these are all things we could just access via the container ref con, above? no reason for them to live on this class?
		/** This array holds the numerical IDs of each particle in each
		 * computational box. */
		int **id;
		/** A two dimensional array holding particle positions. For the
		 * derived container_poly class, this also holds particle
		 * radii. */
		double **p;
		/** An array holding the number of particles within each
		 * computational box of the container. */
		int *co;

		voro_compute(c_class &con_,int hx_,int hy_,int hz_);
		/** The class destructor frees the dynamically allocated memory
		 * for the mask and queue. */
		~voro_compute() {
			delete [] qu;
			delete [] mask;
		}
		voro_compute(const voro_compute<c_class>& Other) = delete;
		voro_compute<c_class>& operator=(const voro_compute<c_class>& Other) = delete;
		voro_compute(voro_compute<c_class>&& Other) noexcept : con(Other.con), boxx(Other.boxx), boxy(Other.boxy), boxz(Other.boxz),
			xsp(Other.xsp), ysp(Other.ysp), zsp(Other.zsp),
			hx(Other.hx), hy(Other.hy), hz(Other.hz), hxy(Other.hxy), hxyz(Other.hxyz), ps(Other.ps),
			id(Other.id), p(Other.p), co(Other.co), bxsq(Other.bxsq),
			mv(Other.mv), qu_size(Other.qu_size), wl(Other.wl), mrad(Other.mrad),
			mask(Other.mask), qu(Other.qu), qu_l(Other.qu_l)
		{
			Other.qu = nullptr;
			Other.mask = nullptr;
		}
		// Note: Move assignment not implemented due to const members on this class (and it has not been needed yet)
		voro_compute<c_class>& operator=(voro_compute<c_class>&& Other) = delete;

		template<class v_cell>
		bool compute_cell(v_cell &c,int ijk,int s,int ci,int cj,int ck);
		void find_voronoi_cell(double x,double y,double z,int ci,int cj,int ck,int ijk,particle_record &w,double &mrs);
	private:
		/** A constant set to boxx*boxx+boxy*boxy+boxz*boxz, which is
		 * frequently used in the computation. */
		const double bxsq;
		/** This sets the current value being used to mark tested blocks
		 * in the mask. */
		unsigned int mv;
		/** The current size of the search list. */
		int qu_size;

		// TODO: can get via 'con' -- no reason for wl to live on this class?
		/** A pointer to the array of worklists. */
		const unsigned int *wl;
		/** An pointer to the array holding the minimum distances
		 * associated with the worklists. */
		double *mrad;
		/** This array is used during the cell computation to determine
		 * which blocks have been considered. */
		unsigned int *mask;
		/** An array is used to store the queue of blocks to test
		 * during the Voronoi cell computation. */
		int *qu;
		/** A pointer to the end of the queue array, used to determine
		 * when the queue is full. */
		int *qu_l;
		template<class v_cell>
		bool corner_test(v_cell &c,double xl,double yl,double zl,double xh,double yh,double zh,typename c_class::radius_info& radi) const;
		template<class v_cell>
		inline bool edge_x_test(v_cell &c,double x0,double yl,double zl,double x1,double yh,double zh,typename c_class::radius_info& radi) const;
		template<class v_cell>
		inline bool edge_y_test(v_cell &c,double xl,double y0,double zl,double xh,double y1,double zh,typename c_class::radius_info& radi) const;
		template<class v_cell>
		inline bool edge_z_test(v_cell &c,double xl,double yl,double z0,double xh,double yh,double z1,typename c_class::radius_info& radi) const;
		template<class v_cell>
		inline bool face_x_test(v_cell &c,double xl,double y0,double z0,double y1,double z1,typename c_class::radius_info& radi) const;
		template<class v_cell>
		inline bool face_y_test(v_cell &c,double x0,double yl,double z0,double x1,double z1,typename c_class::radius_info& radi) const;
		template<class v_cell>
		inline bool face_z_test(v_cell &c,double x0,double y0,double zl,double x1,double y1,typename c_class::radius_info& radi) const;
		bool compute_min_max_radius(int di,int dj,int dk,double fx,double fy,double fz,double gx,double gy,double gz,double& crs,double mrs,const typename c_class::radius_info& radi) const;
		bool compute_min_radius(int di,int dj,int dk,double fx,double fy,double fz,double mrs) const;
		inline void add_to_mask(int ei,int ej,int ek,int *&qu_e);
		inline void scan_bits_mask_add(unsigned int q,unsigned int *mijk,int ei,int ej,int ek,int *&qu_e);
		inline void scan_all(int ijk,double x,double y,double z,int di,int dj,int dk,particle_record &w,double &mrs);
		void add_list_memory(int*& qu_s,int*& qu_e);
		/** Resets the mask in cases where the mask counter wraps
		 * around. */
		inline void reset_mask() {
			for(unsigned int *mp(mask);mp<mask+hxyz;mp++) *mp=0;
		}
};

}

#endif
