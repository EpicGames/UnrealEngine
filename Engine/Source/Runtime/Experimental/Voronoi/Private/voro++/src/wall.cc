// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file wall.cc
 * \brief Function implementations for the derived wall classes. */

#include "wall.hh"

namespace voro {

/** Tests to see whether a point is inside the sphere wall object.
 * \param[in,out] (x,y,z) the vector to test.
 * \return True if the point is inside, false if the point is outside. */
bool wall_sphere::point_inside(double x,double y,double z) const {
	return (x-xc)*(x-xc)+(y-yc)*(y-yc)+(z-zc)*(z-zc)<rc*rc;
}

/** Cuts a cell by the sphere wall object. The spherical wall is approximated by
 * a single plane applied at the point on the sphere which is closest to the center
 * of the cell. This works well for particle arrangements that are packed against
 * the wall, but loses accuracy for sparse particle distributions.
 * \param[in,out] c the Voronoi cell to be cut.
 * \param[in] (x,y,z) the location of the Voronoi cell.
 * \return True if the cell still exists, false if the cell is deleted. */
template<class v_cell>
bool wall_sphere::cut_cell_base(v_cell &c,double x,double y,double z) const {
	double xd=x-xc,yd=y-yc,zd=z-zc,dq=xd*xd+yd*yd+zd*zd;
	if (dq>1e-5) {
		dq=2*(sqrt(dq)*rc-dq);
		return c.nplane(xd,yd,zd,dq,w_id);
	}
	return true;
}

/** Tests to see whether a point is inside the plane wall object.
 * \param[in] (x,y,z) the vector to test.
 * \return True if the point is inside, false if the point is outside. */
bool wall_plane::point_inside(double x,double y,double z) const {
	return x*xc+y*yc+z*zc<ac;
}

/** Cuts a cell by the plane wall object.
 * \param[in,out] c the Voronoi cell to be cut.
 * \param[in] (x,y,z) the location of the Voronoi cell.
 * \return True if the cell still exists, false if the cell is deleted. */
template<class v_cell>
bool wall_plane::cut_cell_base(v_cell &c,double x,double y,double z) const {
	double dq=2*(ac-x*xc-y*yc-z*zc);
	return c.nplane(xc,yc,zc,dq,w_id);
}

/** Tests to see whether a point is inside the cylindrical wall object.
 * \param[in] (x,y,z) the vector to test.
 * \return True if the point is inside, false if the point is outside. */
bool wall_cylinder::point_inside(double x,double y,double z) const {
	double xd=x-xc,yd=y-yc,zd=z-zc;
	double pa=(xd*xa+yd*ya+zd*za)*asi;
	xd-=xa*pa;yd-=ya*pa;zd-=za*pa;
	return xd*xd+yd*yd+zd*zd<rc*rc;
}

/** Cuts a cell by the cylindrical wall object. The cylindrical wall is
 * approximated by a single plane applied at the point on the cylinder which is
 * closest to the center of the cell. This works well for particle arrangements
 * that are packed against the wall, but loses accuracy for sparse particle
 * distributions.
 * \param[in,out] c the Voronoi cell to be cut.
 * \param[in] (x,y,z) the location of the Voronoi cell.
 * \return True if the cell still exists, false if the cell is deleted. */
template<class v_cell>
bool wall_cylinder::cut_cell_base(v_cell &c,double x,double y,double z) const {
	double xd=x-xc,yd=y-yc,zd=z-zc,pa=(xd*xa+yd*ya+zd*za)*asi;
	xd-=xa*pa;yd-=ya*pa;zd-=za*pa;
	pa=xd*xd+yd*yd+zd*zd;
	if(pa>1e-5) {
		pa=2*(sqrt(pa)*rc-pa);
		return c.nplane(xd,yd,zd,pa,w_id);
	}
	return true;
}

/** Tests to see whether a point is inside the cone wall object.
 * \param[in] (x,y,z) the vector to test.
 * \return True if the point is inside, false if the point is outside. */
bool wall_cone::point_inside(double x,double y,double z) const {
	double xd=x-xc,yd=y-yc,zd=z-zc,pa=(xd*xa+yd*ya+zd*za)*asi;
	xd-=xa*pa;yd-=ya*pa;zd-=za*pa;
	pa*=gra;
	if (pa<0) return false;
	pa*=pa;
	return xd*xd+yd*yd+zd*zd<pa;
}

/** Cuts a cell by the cone wall object. The conical wall is
 * approximated by a single plane applied at the point on the cone which is
 * closest to the center of the cell. This works well for particle arrangements
 * that are packed against the wall, but loses accuracy for sparse particle
 * distributions.
 * \param[in,out] c the Voronoi cell to be cut.
 * \param[in] (x,y,z) the location of the Voronoi cell.
 * \return True if the cell still exists, false if the cell is deleted. */
template<class v_cell>
bool wall_cone::cut_cell_base(v_cell &c,double x,double y,double z) const {
	double xd=x-xc,yd=y-yc,zd=z-zc,xf,yf,zf,q,pa=(xd*xa+yd*ya+zd*za)*asi;
	xd-=xa*pa;yd-=ya*pa;zd-=za*pa;
	pa=xd*xd+yd*yd+zd*zd;
	if(pa>1e-5) {
		pa=1/sqrt(pa);
		q=sqrt(asi);
		xf=-sang*q*xa+cang*pa*xd;
		yf=-sang*q*ya+cang*pa*yd;
		zf=-sang*q*za+cang*pa*zd;
		pa=2*(xf*(xc-x)+yf*(yc-y)+zf*(zc-z));
		return c.nplane(xf,yf,zf,pa,w_id);
	}
	return true;
}

// Explicit instantiation
template bool wall_sphere::cut_cell_base(voronoicell&,double,double,double) const;
template bool wall_sphere::cut_cell_base(voronoicell_neighbor&,double,double,double) const;
template bool wall_plane::cut_cell_base(voronoicell&,double,double,double) const;
template bool wall_plane::cut_cell_base(voronoicell_neighbor&,double,double,double) const;
template bool wall_cylinder::cut_cell_base(voronoicell&,double,double,double) const;
template bool wall_cylinder::cut_cell_base(voronoicell_neighbor&,double,double,double) const;
template bool wall_cone::cut_cell_base(voronoicell&,double,double,double) const;
template bool wall_cone::cut_cell_base(voronoicell_neighbor&,double,double,double) const;

}
