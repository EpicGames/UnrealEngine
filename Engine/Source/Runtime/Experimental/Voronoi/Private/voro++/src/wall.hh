// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file wall.hh
 * \brief Header file for the derived wall classes. */

#ifndef VOROPP_WALL_HH
#define VOROPP_WALL_HH

#include "cell.hh"
#include "container.hh"

namespace voro {

/** \brief A class representing a spherical wall object.
 *
 * This class represents a spherical wall object. */
struct wall_sphere : public wall {
	public:
		/** Constructs a spherical wall object.
		 * \param[in] w_id_ an ID number to associate with the wall for
		 *		    neighbor tracking.
		 * \param[in] (xc_,yc_,zc_) a position vector for the sphere's
		 * 			    center.
		 * \param[in] rc_ the radius of the sphere. */
		wall_sphere(double xc_,double yc_,double zc_,double rc_,int w_id_=-99)
			: w_id(w_id_), xc(xc_), yc(yc_), zc(zc_), rc(rc_) {}
		bool point_inside(double x,double y,double z) const;
		template<class v_cell>
		bool cut_cell_base(v_cell &c,double x,double y,double z) const;
		bool cut_cell(voronoicell &c,double x,double y,double z) const {return cut_cell_base(c,x,y,z);}
		bool cut_cell(voronoicell_neighbor &c,double x,double y,double z) const {return cut_cell_base(c,x,y,z);}
	private:
		const int w_id;
		const double xc,yc,zc,rc;
};

/** \brief A class representing a plane wall object.
 *
 * This class represents a single plane wall object. */
struct wall_plane : public wall {
	public:
		/** Constructs a plane wall object.
		 * \param[in] (xc_,yc_,zc_) a normal vector to the plane.
		 * \param[in] ac_ a displacement along the normal vector.
		 * \param[in] w_id_ an ID number to associate with the wall for
		 *		    neighbor tracking. */
		wall_plane(double xc_,double yc_,double zc_,double ac_,int w_id_=-99)
			: w_id(w_id_), xc(xc_), yc(yc_), zc(zc_), ac(ac_) {}
		bool point_inside(double x,double y,double z) const;
		template<class v_cell>
		bool cut_cell_base(v_cell &c,double x,double y,double z) const;
		bool cut_cell(voronoicell &c,double x,double y,double z) const {return cut_cell_base(c,x,y,z);}
		bool cut_cell(voronoicell_neighbor &c,double x,double y,double z) const {return cut_cell_base(c,x,y,z);}
	private:
		const int w_id;
		const double xc,yc,zc,ac;
};

/** \brief A class representing a cylindrical wall object.
 *
 * This class represents a open cylinder wall object. */
struct wall_cylinder : public wall {
	public:
		/** Constructs a cylinder wall object.
		 * \param[in] (xc_,yc_,zc_) a point on the axis of the
		 *			    cylinder.
		 * \param[in] (xa_,ya_,za_) a vector pointing along the
		 *			    direction of the cylinder.
		 * \param[in] rc_ the radius of the cylinder
		 * \param[in] w_id_ an ID number to associate with the wall for
		 *		    neighbor tracking. */
		wall_cylinder(double xc_,double yc_,double zc_,double xa_,double ya_,double za_,double rc_,int w_id_=-99)
			: w_id(w_id_), xc(xc_), yc(yc_), zc(zc_), xa(xa_), ya(ya_), za(za_),
			asi(1/(xa_*xa_+ya_*ya_+za_*za_)), rc(rc_) {}
		bool point_inside(double x,double y,double z) const;
		template<class v_cell>
		bool cut_cell_base(v_cell &c,double x,double y,double z) const;
		bool cut_cell(voronoicell &c,double x,double y,double z) const {return cut_cell_base(c,x,y,z);}
		bool cut_cell(voronoicell_neighbor &c,double x,double y,double z) const {return cut_cell_base(c,x,y,z);}
	private:
		const int w_id;
		const double xc,yc,zc,xa,ya,za,asi,rc;
};


/** \brief A class representing a conical wall object.
 *
 * This class represents a cone wall object. */
struct wall_cone : public wall {
	public:
		/** Constructs a cone wall object.
		 * \param[in] (xc_,yc_,zc_) the apex of the cone.
		 * \param[in] (xa_,ya_,za_) a vector pointing along the axis of
		 *			    the cone.
		 * \param[in] ang the angle (in radians) of the cone, measured
		 *		  from the axis.
		 * \param[in] w_id_ an ID number to associate with the wall for
		 *		    neighbor tracking. */
		wall_cone(double xc_,double yc_,double zc_,double xa_,double ya_,double za_,double ang,int w_id_=-99)
			: w_id(w_id_), xc(xc_), yc(yc_), zc(zc_), xa(xa_), ya(ya_), za(za_),
			asi(1/(xa_*xa_+ya_*ya_+za_*za_)),
			gra(tan(ang)), sang(sin(ang)), cang(cos(ang)) {}
		bool point_inside(double x,double y,double z) const;
		template<class v_cell>
		bool cut_cell_base(v_cell &c,double x,double y,double z) const;
		bool cut_cell(voronoicell &c,double x,double y,double z) const {return cut_cell_base(c,x,y,z);}
		bool cut_cell(voronoicell_neighbor &c,double x,double y,double z) const {return cut_cell_base(c,x,y,z);}
	private:
		const int w_id;
		const double xc,yc,zc,xa,ya,za,asi,gra,sang,cang;
};

}

#endif
