// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file worklist.hh
 * \brief Header file for setting constants used in the block worklists that are
 * used during cell computation.
 *
 * This file is automatically generated by worklist_gen.pl and it is not
 * intended to be edited by hand. */

#ifndef VOROPP_WORKLIST_HH
#define VOROPP_WORKLIST_HH

namespace voro {

/** Each region is divided into a grid of subregions, and a worklist is
# constructed for each. This parameter sets is set to half the number of
# subregions that the block is divided into. */
const int wl_hgrid=4;
/** The number of subregions that a block is subdivided into, which is twice
the value of hgrid. */
const int wl_fgrid=8;
/** The total number of worklists, set to the cube of hgrid. */
const int wl_hgridcu=64;
/** The number of elements in each worklist. */
const int wl_seq_length=64;

}
#endif
