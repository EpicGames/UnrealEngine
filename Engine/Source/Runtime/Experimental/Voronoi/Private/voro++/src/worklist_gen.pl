#!/usr/bin/perl
# Voro++, a 3D cell-based Voronoi library
#
# Author   : Chris H. Rycroft (LBL / UC Berkeley)
# Email    : chr@alum.mit.edu
# Date     : August 30th 2011
#
# worklist_gen.pl - this perl script is used to automatically generate the
# worklists of blocks that are stored in worklist.cc, that are used by the
# compute_cell() routine to ensure maximum efficiency

# Each region is divided into a grid of subregions, and a worklist is
# constructed for each. This parameter sets is set to half the number of
# subregions that the block is divided into.
$hr=4;

# This parameter is automatically set to the the number of subregions that the
# block is divided into
$r=$hr*2;

# This parameter sets the total number of block entries in each worklist
$ls=63;

# When the worklists are being constructed, a mask array is made use of. To
# prevent the creation of array elements with negative indices, this parameter
# sets an arbitrary displacement.
$dis=8;

# Constants used mask indexing
$d=2*$dis+1;$dd=$d*$d;$d0=(1+$d+$dd)*$dis;

use Math::Trig;

# Construct the worklist header file, based on the parameters above
open W,">worklist.hh";
print W <<EOF;
// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr\@alum.mit.edu
// Date     : August 30th 2011

/** \\file worklist.hh
 * \\brief Header file for setting constants used in the block worklists that are
 * used during cell computation.
 *
 * This file is automatically generated by worklist_gen.pl and it is not
 * intended to be edited by hand. */

#ifndef VOROPP_WORKLIST_HH
#define VOROPP_WORKLIST_HH

namespace voro {

/** Each region is divided into a grid of subregions, and a worklist is
# constructed for each. This parameter sets is set to half the number of
# subregions that the block is divided into. */
EOF
print W "const int wl_hgrid=$hr;\n";
print W <<EOF;
/** The number of subregions that a block is subdivided into, which is twice
the value of hgrid. */
EOF
print W "const int wl_fgrid=$r;\n";
print W <<EOF;
/** The total number of worklists, set to the cube of hgrid. */
EOF
printf W "const int wl_hgridcu=%d;\n",$hr*$hr*$hr;
print W <<EOF;
/** The number of elements in each worklist. */
EOF
printf W "const int wl_seq_length=%d;\n",$ls+1;
print W <<EOF;

}
#endif
EOF
close W;

# Construct the preamble to the worklist.cc file
open W,">v_base_wl.cc";
print W <<EOF;
// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr\@alum.mit.edu
// Date     : August 30th 2011

/** \\file v_base_wl.cc
 * \\brief The table of block worklists that are used during the cell
 * computation, which is part of the voro_base class.
 *
 * This file is automatically generated by worklist_gen.pl and it is not
 * intended to be edited by hand. */

EOF
printf W "const unsigned int voro_base::wl[wl_seq_length*wl_hgridcu]={\n";

# Now create a worklist for each subregion
for($kk=0;$kk<$hr;$kk++) {
	for($jj=0;$jj<$hr;$jj++) {
		for($ii=0;$ii<$hr;$ii++) {
			worklist($ii,$jj,$kk);
		}
	}
}

# Finish the file and close it
print W "};\n";
close W;

# A subroutine
sub worklist {
#	print "@_[0] @_[1] @_[2]\n";
	$ind=@_[0]+$hr*(@_[1]+$hr*@_[2]);
	$ac=0;$v+=2;
	$xp=$yp=$zp=0;
	$x=(@_[0]+0.5)/$r;
	$y=(@_[1]+0.5)/$r;
	$z=(@_[2]+0.5)/$r;
	$m[$d0]=$v;
	add(1,0,0);add(0,1,0);add(0,0,1);
	add(-1,0,0);add(0,-1,0);add(0,0,-1);
	foreach $l (1..$ls) {
		$minwei=1e9;
		foreach (0..$ac-1) {
			$xt=@a[3*$_];$yt=@a[3*$_+1];$zt=@a[3*$_+2];
#			$wei=dis($x,$y,$z,$xt,$yt,$zt)+1*acos(($xt*$xp+$yt*$yp+$zt*$zp)/($xt*$xt+$yt*$yt+$zt*$zt)*($xp*$xp+$yp*$yp+$zp*$zp));
			$wei=adis($x,$y,$z,$xt,$yt,$zt)+0.02*sqrt(($xt-$xp)**2+($yt-$yp)**2+($zt-$zp)**2);
			$nx=$_,$minwei=$wei if $wei<$minwei;
		}
		$xp=@a[3*$nx];$yp=@a[3*$nx+1];$zp=@a[3*$nx+2];
		add($xp+1,$yp,$zp);add($xp,$yp+1,$zp);add($xp,$yp,$zp+1);
		add($xp-1,$yp,$zp);add($xp,$yp-1,$zp);add($xp,$yp,$zp-1);
#		print "=> $l $xp $yp $zp\n" if $l<4;
		push @b,(splice @a,3*$nx,3);$ac--;
	}

	# Mark all blocks that are on this worklist entry
	$m[$d0]=++$v;
	for($i=0;$i<$#b;$i+=3) {
		$xt=$b[$i];$yt=$b[$i+1];$zt=$b[$i+2];
		$m[$d0+$xt+$d*$yt+$dd*$zt]=$v;
	}

	# Find which neighboring outside blocks need to be marked when
	# considering this block, being as conservative as possible by
	# overwriting the marks, so that the last possible entry that can reach
	# a block is used
	for($i=$j=0;$i<$#b;$i+=3,$j++) {
		$xt=$b[$i];$yt=$b[$i+1];$zt=$b[$i+2];
		$k=$d0+$xt+$yt*$d+$zt*$dd;
		$la[$k+1]=$j, $m[$k+1]=$v+1 if $xt>=0 && $m[$k+1]!=$v;
		$la[$k+$d]=$j, $m[$k+$d]=$v+1 if $yt>=0 && $m[$k+$d]!=$v;
		$la[$k+$dd]=$j, $m[$k+$dd]=$v+1 if $zt>=0 && $m[$k+$dd]!=$v;
		$la[$k-1]=$j, $m[$k-1]=$v+1 if $xt<=0 && $m[$k-1]!=$v;
		$la[$k-$d]=$j, $m[$k-$d]=$v+1 if $yt<=0 && $m[$k-$d]!=$v;
		$la[$k-$dd]=$j, $m[$k-$dd]=$v+1 if $zt<=0 && $m[$k-$dd]!=$v;
	}

	# Scan to ensure that no neighboring blocks have been missed by the
	# outwards-looking logic in the above section
	for($i=0;$i<$#b;$i+=3) {
		wl_check($d0+$b[$i]+$b[$i+1]*$d+$b[$i+2]*$dd);
	}

	# Compute the number of entries where outside blocks do not need to be
	# consider
	for($i=$j=0;$i<$#b;$i+=3,$j++) {
		$k=$d0+$b[$i]+$b[$i+1]*$d+$b[$i+2]*$dd;
		last if $m[$k+1]!=$v;
		last if $m[$k+$d]!=$v;
		last if $m[$k+$dd]!=$v;
		last if $m[$k-1]!=$v;
		last if $m[$k-$d]!=$v;
		last if $m[$k-$dd]!=$v;
	}
	print W "\t$j";

	# Create worklist entry and save to file
	$j=0;
	while ($#b>0) {
		$xt=shift @b;$yt=shift @b;$zt=shift @b;
		$k=$d0+$xt+$yt*$d+$zt*$dd;
		$o=0;
		$o|=1 if $m[$k+1]!=$v && $la[$k+1]==$j;
		$o^=3 if $m[$k-1]!=$v && $la[$k-1]==$j;
		$o|=8 if $m[$k+$d]!=$v && $la[$k+$d]==$j;
		$o^=24 if $m[$k-$d]!=$v && $la[$k-$d]==$j;
		$o|=64 if $m[$k+$dd]!=$v && $la[$k+$dd]==$j;
		$o^=192 if $m[$k-$dd]!=$v && $la[$k-$dd]==$j;
		printf W ",%#x",(($xt+64)|($yt+64)<<7|($zt+64)<<14|$o<<21);
		$j++;
	}
	print W "," unless $ind==$hr*$hr*$hr-1;
	print W "\n";

	# Remove list memory
	undef @a;
	undef @b;
}

sub add {
	if ($m[$d0+@_[0]+$d*@_[1]+$dd*@_[2]]!=$v) {
		$ac++;
		push @a,@_[0],@_[1],@_[2];
		$m[$d0+@_[0]+$d*@_[1]+$dd*@_[2]]=$v;
	}
}

sub dis {
	$xl=@_[3]+0.3-@_[0];$xh=@_[3]+0.7-@_[0];
	$yl=@_[4]+0.3-@_[1];$yh=@_[4]+0.7-@_[1];
	$zl=@_[5]+0.3-@_[2];$zh=@_[5]+0.7-@_[2];
	$dis=(abs($xl)<abs($xh)?$xl:$xh)**2
		+(abs($yl)<abs($yh)?$yl:$yh)**2
		+(abs($zl)<abs($zh)?$zl:$zh)**2;
	return sqrt $dis;
}

sub adis {
	$xco=$yco=$zco=0;
	$xco=@_[0]-@_[3] if @_[3]>0;
	$xco=@_[0]-@_[3]-1 if @_[3]<0;
	$yco=@_[1]-@_[4] if @_[4]>0;
	$yco=@_[1]-@_[4]-1 if @_[4]<0;
	$zco=@_[2]-@_[5] if @_[5]>0;
	$zco=@_[2]-@_[5]-1 if @_[5]<0;
	return sqrt $xco*$xco+$yco*$yco+$zco*$zco;
}

sub wl_check {
	die "Failure in worklist construction\n" if $m[@_[0]+1]<$v||$m[@_[0]-1]<$v
						  ||$m[@_[0]+$d]<$v||$m[@_[0]-$d]<$v
						  ||$m[@_[0]+$dd]<$v||$m[@_[0]-$dd]<$v;
}
