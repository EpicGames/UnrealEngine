// Copyright Epic Games, Inc. All Rights Reserved.
// Modified version of Voro++'s source file

// Voro++, a 3D cell-based Voronoi library
//
// Author   : Chris H. Rycroft (LBL / UC Berkeley)
// Email    : chr@alum.mit.edu
// Date     : August 30th 2011

/** \file voro++.hh
 * \brief A file that loads all of the Voro++ header files. */

/** \mainpage Voro++ class reference manual
 * \section intro Introduction
 * Voro++ is a software library for carrying out three-dimensional computations
 * of the Voronoi tessellation. A distinguishing feature of the Voro++ library
 * is that it carries out cell-based calculations, computing the Voronoi cell
 * for each particle individually, rather than computing the Voronoi
 * tessellation as a global network of vertices and edges. It is particularly
 * well-suited for applications that rely on cell-based statistics, where
 * features of Voronoi cells (eg. volume, centroid, number of faces) can be
 * used to analyze a system of particles.
 *
 * Voro++ is written in C++ and can be built as a static library that can be
 * linked to. This manual provides a reference for every function in the class
 * structure. For a general overview of the program, see the Voro++ website at
 * http://math.lbl.gov/voro++/ and in particular the example programs at
 * http://math.lbl.gov/voro++/examples/ that demonstrate many of the library's
 * features.
 *
 * \section class C++ class structure
 * The code is structured around several C++ classes. The voronoicell_base
 * class contains all of the routines for constructing a single Voronoi cell.
 * It represents the cell as a collection of vertices that are connected by
 * edges, and there are routines for initializing, making, and outputting the
 * cell. The voronoicell_base class form the base of the voronoicell and
 * voronoicell_neighbor classes, which add specialized routines depending on
 * whether neighboring particle ID information for each face must be tracked or
 * not. Collectively, these classes are referred to as "voronoicell classes"
 * within the documentation.
 *
 * There is a hierarchy of classes that represent three-dimensional particle
 * systems. All of these are derived from the voro_base class, which contains
 * constants that divide a three-dimensional system into a rectangular grid of
 * equally-sized rectangular blocks; this grid is used for computational
 * efficiency during the Voronoi calculations.
 *
 * The container_base, container, and container_poly are then derived from the
 * voro_base class to represent a particle system in a specific
 * three-dimensional rectangular box using both periodic and non-periodic
 * boundary conditions. In addition, the container_periodic_base,
 * container_periodic, and container_periodic_poly classes represent
 * a particle system in a three-dimensional non-orthogonal periodic domain,
 * defined by three periodicity vectors that represent a parallelepiped.
 * Collectively, these classes are referred to as "container classes" within
 * the documentation.
 *
 * The voro_compute template encapsulates all of the routines for computing
 * Voronoi cells. Each container class has a voro_compute template within
 * it, that accesses the container's particle system, and computes the Voronoi
 * cells.
 *
 * There are several wall classes that can be used to apply certain boundary
 * conditions using additional plane cuts during the Voronoi cell compution.
 * The code also contains a number of small loop classes, c_loop_all,
 * c_loop_subset, c_loop_all_periodic, and c_loop_order that can be used to
 * iterate over a certain subset of particles in a container. The latter class
 * makes use of a special particle_order class that stores a specific order of
 * particles within the container. The library also contains the classes
 * pre_container_base, pre_container, and pre_container_poly, that can be used
 * as temporary storage when importing data of unknown size.
 *
 * \section voronoicell The voronoicell classes
 * The voronoicell class represents a single Voronoi cell as a convex
 * polyhedron, with a set of vertices that are connected by edges. The class
 * contains a variety of functions that can be used to compute and output the
 * Voronoi cell corresponding to a particular particle. The command init()
 * can be used to initialize a cell as a large rectangular box. The Voronoi cell
 * can then be computed by repeatedly cutting it with planes that correspond to
 * the perpendicular bisectors between that particle and its neighbors.
 *
 * This is achieved by using the plane() routine, which will recompute the
 * cell's vertices and edges after cutting it with a single plane. This is the
 * key routine in voronoicell class. It begins by exploiting the convexity
 * of the underlying cell, tracing between edges to work out if the cell
 * intersects the cutting plane. If it does not intersect, then the routine
 * immediately exits. Otherwise, it finds an edge or vertex that intersects
 * the plane, and from there, traces out a new face on the cell, recomputing
 * the edge and vertex structure accordingly.
 *
 * Once the cell is computed, there are many routines for computing features of
 * the the Voronoi cell, such as its volume, surface area, or centroid. There
 * are also many routines for outputting features of the Voronoi cell, or
 * writing its shape in formats that can be read by Gnuplot or POV-Ray.
 *
 * \subsection internal Internal data representation
 * The voronoicell class has a public member p representing the
 * number of vertices. The polyhedral structure of the cell is stored
 * in the following arrays:
 *
 * - pts: a one-dimensional array of floating point numbers, that represent the
 *   position vectors x_0, x_1, ..., x_{p-1} of the polyhedron vertices.
 * - nu: the order of each vertex n_0, n_1, ..., n_{p-1}, corresponding to
 *   the number of other vertices to which each is connected.
 * - ed: a two-dimensional table of edges and relations. For the ith vertex,
 *   ed[i] has 2n_i+1 elements. The first n_i elements are the edges e(j,i),
 *   where e(j,i) is the jth neighbor of vertex i. The edges are ordered
 *   according to a right-hand rule with respect to an outward-pointing normal.
 *   The next n_i elements are the relations l(j,i) which satisfy the property
 *   e(l(j,i),e(j,i)) = i. The final element of the ed[i] list is a back
 *   pointer used in memory allocation.
 *
 * In a very large number of cases, the values of n_i will be 3. This is because
 * the only way that a higher-order vertex can be created in the plane()
 * routine is if the cutting plane perfectly intersects an existing vertex. For
 * random particle arrangements with position vectors specified to double
 * precision this should happen very rarely. A preliminary version of this code
 * was quite successful with only making use of vertices of order 3. However,
 * when calculating millions of cells, it was found that this approach is not
 * robust, since a single floating point error can invalidate the computation.
 * This can also be a problem for cases featuring crystalline arrangements of
 * particles where the corresponding Voronoi cells may have high-order vertices
 * by construction.
 *
 * Because of this, Voro++ takes the approach that it if an existing vertex is
 * within a small numerical tolerance of the cutting plane, it is treated as
 * being exactly on the plane, and the polyhedral topology is recomputed
 * accordingly. However, while this improves robustness, it also adds the
 * complexity that n_i may no longer always be 3. This causes memory management
 * to be significantly more complicated, as different vertices require a
 * different number of elements in the ed[][] array. To accommodate this, the
 * voronoicell class allocated edge memory in a different array called mep[][],
 * in such a way that all vertices of order k are held in mep[k]. If vertex
 * i has order k, then ed[i] points to memory within mep[k]. The array ed[][]
 * is never directly initialized as a two-dimensional array itself, but points
 * at allocations within mep[][]. To the user, it appears as though each row of
 * ed[][] has a different number of elements. When vertices are added or
 * deleted, care must be taken to reorder and reassign elements in these
 * arrays.
 *
 * During the plane() routine, the code traces around the vertices of the cell,
 * and adds new vertices along edges which intersect the cutting plane to
 * create a new face. The values of l(j,i) are used in this computation, as
 * when the code is traversing from one vertex on the cell to another, this
 * information allows the code to immediately work out which edge of a vertex
 * points back to the one it came from. As new vertices are created, the l(j,i)
 * are also updated to ensure consistency. To ensure robustness, the plane
 * cutting algorithm should work with any possible combination of vertices
 * which are inside, outside, or exactly on the cutting plane.
 *
 * Vertices exactly on the cutting plane create some additional computational
 * difficulties. If there are two marginal vertices connected by an existing
 * edge, then it would be possible for duplicate edges to be created between
 * those two vertices, if the plane routine traces along both sides of this
 * edge while constructing the new face. The code recognizes these cases and
 * prevents the double edge from being formed. Another possibility is the
 * formation of vertices of order two or one. At the end of the plane cutting
 * routine, the code checks to see if any of these are present, removing the
 * order one vertices by just deleting them, and removing the order two
 * vertices by connecting the two neighbors of each vertex together. It is
 * possible that the removal of a single low-order vertex could result in the
 * creation of additional low-order vertices, so the process is applied
 * recursively until no more are left.
 *
 * \section container The container classes
 * There are four container classes available for general usage: container,
 * container_poly, container_periodic, and container_periodic_poly. Each of
 * these represent a system of particles in a specific three-dimensional
 * geometry. They contain routines for importing particles from a text file,
 * and adding particles individually. They also contain a large number of
 * analyzing and outputting the particle system. Internally, the routines that
 * compute Voronoi cells do so by making use of the voro_compute template.
 * Each container class contains routines that tell the voro_compute template
 * about the specific geometry of this container.
 *
 * \section voro_compute The voro_compute template
 * The voro_compute template encapsulates the routines for carrying out the
 * Voronoi cell computations. It contains data structures suchs as a mask and a
 * queue that are used in the computations. The voro_compute template is
 * associated with a specific container class, and during the computation, it
 * calls routines in the container class to access the particle positions that
 * are stored there.
 *
 * The key routine in this class is compute_cell(), which makes use of a
 * voronoicell class to construct a Voronoi cell for a specific particle in the
 * container. The basic approach that this function takes is to repeatedly cut
 * the Voronoi cell by planes corresponding neighboring particles, and stop
 * when it recognizes that all the remaining particles in the container are too
 * far away to possibly influence cell's shape. The code makes use of two
 * possible methods for working out when a cell computation is complete:
 *
 * - Radius test: if the maximum distance of a Voronoi cell
 *   vertex from the cell center is R, then no particles more than a distance
 *   2R away can possibly influence the cell. This a very fast computation to
 *   do, but it has no directionality: if the cell extends a long way in one
 *   direction then particles a long distance in other directions will still
 *   need to be tested.
 * - Region test: it is possible to test whether a specific region can
 *   possibly influence the cell by applying a series of plane tests at the
 *   point on the region which is closest to the Voronoi cell center. This is a
 *   slower computation to do, but it has directionality.
 *
 * Another useful observation is that the regions that need to be tested are
 * simply connected, meaning that if a particular region does not need to be
 * tested, then neighboring regions which are further away do not need to be
 * tested.
 *
 * For maximum efficiency, it was found that a hybrid approach making use of
 * both of the above tests worked well in practice. Radius tests work well for
 * the first few blocks, but switching to region tests after then prevent the
 * code from becoming extremely slow, due to testing over very large spherical
 * shells of particles. The compute_cell() routine therefore takes the
 * following approach:
 *
 * - Initialize the voronoicell class to fill the entire computational domain.
 * - Cut the cell by any wall objects that have been added to the container.
 * - Apply plane cuts to the cell corresponding to the other particles which
 *   are within the current particle's region.
 * - Test over a pre-computed worklist of neighboring regions, that have been
 *   ordered according to the minimum distance away from the particle's
 *   position. Apply radius tests after every few regions to see if the
 *   calculation can terminate.
 * - If the code reaches the end of the worklist, add all the neighboring
 *   regions to a new list.
 * - Carry out a region test on the first item of the list. If the region needs
 *   to be tested, apply the plane() routine for all of its particles, and then
 *   add any neighboring regions to the end of the list that need to be tested.
 *   Continue until the list has no elements left.
 *
 * The compute_cell() routine forms the basis of many other routines, such as
 * store_cell_volumes() and draw_cells_gnuplot() that can be used to calculate
 * and draw the cells in a container.
 *
 * \section walls Wall computation
 * Wall computations are handled by making use of a pure virtual wall class.
 * Specific wall types are derived from this class, and require the
 * specification of two routines: point_inside() that tests to see if a point
 * is inside a wall or not, and cut_cell() that cuts a cell according to the
 * wall's position. The walls can be added to the container using the
 * add_wall() command, and these are called each time a compute_cell() command
 * is carried out. At present, wall types for planes, spheres, cylinders, and
 * cones are provided, although custom walls can be added by creating new
 * classes derived from the pure virtual class. Currently all wall types
 * approximate the wall surface with a single plane, which produces some small
 * errors, but generally gives good results for dense particle packings in
 * direct contact with a wall surface. It would be possible to create more
 * accurate walls by making cut_cell() routines that approximate the curved
 * surface with multiple plane cuts.
 *
 * The wall objects can used for periodic calculations, although to obtain
 * valid results, the walls should also be periodic as well. For example, in a
 * domain that is periodic in the x direction, a cylinder aligned along the x
 * axis could be added. At present, the interior of all wall objects are convex
 * domains, and consequently any superposition of them will be a convex domain
 * also. Carrying out computations in non-convex domains poses some problems,
 * since this could theoretically lead to non-convex Voronoi cells, which the
 * internal data representation of the voronoicell class does not support. For
 * non-convex cases where the wall surfaces feature just a small amount of
 * negative curvature (eg. a torus) approximating the curved surface with a
 * single plane cut may give an acceptable level of accuracy. For non-convex
 * cases that feature internal angles, the best strategy may be to decompose
 * the domain into several convex subdomains, carry out a calculation in each,
 * and then add the results together. The voronoicell class cannot be easily
 * modified to handle non-convex cells as this would fundamentally alter the
 * algorithms that it uses, and cases could arise where a single plane cut
 * could create several new faces as opposed to just one.
 *
 * \section loops Loop classes
 * The container classes have a number of simple routines for calculating
 * Voronoi cells for all particles within them. However, in some situations it
 * is desirable to iterate over a specific subset of particles. This can be
 * achieved with the c_loop classes that are all derived from the c_loop_base
 * class. Each class can iterate over a specific subset of particles in a
 * container. There are three loop classes for use with the container and
 * container_poly classes:
 *
 * - c_loop_all will loop over all of the particles in a container.
 * - c_loop_subset will loop over a subset of particles in a container that lie
 *   within some geometrical region. It can loop over particles in a
 *   rectangular box, particles in a sphere, or particles that lie within
 *   specific internal computational blocks.
 * - c_loop_order will loop over a specific list of particles that were
 *   previously stored in a particle_order class.
 *
 * Several of the key routines within the container classes (such as
 * draw_cells_gnuplot and print_custom) have versions where they can be passed
 * a loop class to use. Loop classes can also be used directly and there are
 * some examples on the library website that demonstrate this. It is also
 * possible to write custom loop classes.
 *
 * In addition to the loop classes mentioned above, there is also a
 * c_loop_all_periodic class, that is specifically for use with the
 * container_periodic and container_periodic_poly classes. Since the data
 * structures of these containers differ considerably, it requires a different
 * loop class that is not interoperable with the others.
 *
 * \section pre_container The pre_container classes
 * Voro++ makes use of internal computational grid of blocks that are used to
 * configure the code for maximum efficiency. As discussed on the library
 * website, the best performance is achieved for around 5 particles per block,
 * with anything in the range from 3 to 12 giving good performance. Usually
 * the size of the grid can be chosen by ensuring that the number of blocks is
 * equal to the number of particles divided by 5.
 *
 * However, this can be difficult to choose in cases when the number of
 * particles is not known a priori, and in thes cases the pre_container classes
 * can be used. They can import an arbitrary number of particle positions from
 * a file, dynamically allocating memory in chunks as necessary. Once particles
 * are imported, they can guess an optimal block arrangement to use for the
 * container class, and then transfer the particles to the container. By
 * default, this procedure is used by the command-line utility to enable it to
 * work well with arbitrary sizes of input data.
 *
 * The pre_container class can be used when no particle radius information is
 * available, and the pre_container_poly class can be used when radius
 * information is available. At present, the pre_container classes can only be
 * used with the container and container_poly classes. They do not support
 * the container_periodic and container_periodic_poly classes. */

#ifndef VOROPP_HH
#define VOROPP_HH

#include "src/config.hh"
#include "src/common.hh"
#include "src/cell.hh"
#include "src/v_base.hh"
#include "src/rad_option.hh"
#include "src/container.hh"
#include "src/unitcell.hh"
#include "src/pre_container.hh"
#include "src/v_compute.hh"
#include "src/c_loops.hh"
#include "src/wall.hh"

#endif
