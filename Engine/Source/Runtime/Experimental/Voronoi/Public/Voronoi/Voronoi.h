// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Math/Box.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Templates/PimplPtr.h"
#include "Templates/Tuple.h"
#include "Templates/UnrealTemplate.h"

bool VORONOI_API VoronoiNeighbors(const TArrayView<const FVector> &Sites, TArray<TArray<int>> &Neighbors, bool bExcludeBounds = true, double SquaredDistSkipPtThreshold = UE_KINDA_SMALL_NUMBER);
bool VORONOI_API GetVoronoiEdges(const TArrayView<const FVector> &Sites, const FBox& Bounds, TArray<TTuple<FVector, FVector>> &Edges, TArray<int32> &CellMember, double SquaredDistSkipPtThreshold = UE_KINDA_SMALL_NUMBER);

// All the info you would typically want about a single cell in the Voronoi diagram, in the format that is easiest to compute
struct FVoronoiCellInfo
{
	TArray<FVector> Vertices;
	TArray<int32> Faces;
	TArray<int32> Neighbors;
	TArray<FVector> Normals;
};

// third party library voro++'s container class; stores voronoi sites in a uniform grid accel structure
namespace voro {
	class container;
	template <class ContainerType> class voro_compute;
}

class FVoronoiDiagram;

// The Voronoi Compute Helper caches information to help compute cells quickly
// To safely parallelize voronoi diagram construction, create a separate compute helper per-thread
class FVoronoiComputeHelper
{
	TPimplPtr<voro::voro_compute<voro::container>> Compute;
public:

	FVoronoiComputeHelper() {}
	FVoronoiComputeHelper(const FVoronoiComputeHelper& Other) = delete;
	FVoronoiComputeHelper& operator=(const FVoronoiComputeHelper& Other) = delete;
	FVoronoiComputeHelper(FVoronoiComputeHelper&& Other) = default;
	FVoronoiComputeHelper& operator=(FVoronoiComputeHelper&& Other) = default;

	VORONOI_API void Init();

	friend class FVoronoiDiagram;
};

class FVoronoiDiagram
{
	int32 NumSites;
	TPimplPtr<voro::container> Container;
	FBox Bounds;

public:
	VORONOI_API const static int MinDefaultSitesPerThread;

	/**
	 * Create a Voronoi diagram with the given sites, in a box bounding containing those sites
	 * 
	 * @param Sites							Voronoi sites for the diagram
	 * @param ExtraBoundingSpace			Voronoi diagram will be computed within the bounding box of the sites + this amount of extra space in each dimension
	 * @param SquaredDistSkipPtThreshold	A safety threshold to avoid creating invalid cells: sites that are within this distance of an already-added site will not be added
	 *										(If you know there will be no duplicate sites, can set to zero for faster perf.)
	 */
	VORONOI_API FVoronoiDiagram(const TArrayView<const FVector>& Sites, double ExtraBoundingSpace, double SquaredDistSkipPtThreshold = UE_KINDA_SMALL_NUMBER);
	/**
	 * Create a Voronoi diagram with the given sites, in a given bounding box
	 * 
	 * @param Sites							Voronoi sites for the diagram
	 * @param Bounds						Bounding box within which to compute the Voronoi diagram
	 * @param ExtraBoundingSpace			Voronoi diagram will be computed within the input Bounds + this amount of extra space in each dimension
	 * @param SquaredDistSkipPtThreshold	A safety threshold to avoid creating invalid cells: sites that are within this distance of an already-added site will not be added.
	 *										(If you know there will be no duplicate sites, can set to zero for faster perf.)
	 */
	VORONOI_API FVoronoiDiagram(const TArrayView<const FVector>& Sites, const FBox &Bounds, double ExtraBoundingSpace, double SquaredDistSkipPtThreshold = UE_KINDA_SMALL_NUMBER);

	/**
	 * Make a container for a Voronoi diagram without any points.  Call AddSites() after to finish creating the diagram.
	 * 
	 * @param ExpectedNumSites		How many sites you expect to add in total
	 * @param Bounds				Bounding box within which to compute the Voronoi diagram
	 * @param ExtraBoundingSpace	Amount to expand the bounding box before computing the Voronoi diagram
	 */
	VORONOI_API FVoronoiDiagram(int32 ExpectedNumSites, const FBox& Bounds, double ExtraBoundingSpace);

	/**
	 * Make an empty diagram, to be filled in with a subsequent call to "Initialize()"
	 */
	FVoronoiDiagram() : NumSites(0), Container(nullptr), Bounds(EForceInit::ForceInit)
	{}

	FVoronoiDiagram(const FVoronoiDiagram& Other) = delete;
	FVoronoiDiagram& operator=(const FVoronoiDiagram& Other) = delete;
	FVoronoiDiagram(FVoronoiDiagram&& Other) = default;
	FVoronoiDiagram& operator=(FVoronoiDiagram&& Other) = default;

	/**
	 * Create a Voronoi diagram with the given sites, in a given bounding box.  Any previous data is discarded.
	 *
	 * @param Sites							Voronoi sites for the diagram
	 * @param Bounds						Bounding box within which to compute the Voronoi diagram
	 * @param ExtraBoundingSpace			Voronoi diagram will be computed within the input Bounds + this amount of extra space in each dimension
	 * @param SquaredDistSkipPtThreshold	A safety threshold to avoid creating invalid cells: sites that are within this distance of an already-added site will not be added.
	 *										(If you know there will be no duplicate sites, can set to zero for faster perf.)
	 */
	VORONOI_API void Initialize(const TArrayView<const FVector>& Sites, const FBox& Bounds, double ExtraBoundingSpace, double SquaredDistSkipPtThreshold = UE_KINDA_SMALL_NUMBER);

	VORONOI_API ~FVoronoiDiagram();

	static VORONOI_API FBox GetBounds(const TArrayView<const FVector>& Sites, double ExtraBoundingSpace = UE_DOUBLE_KINDA_SMALL_NUMBER);

	int32 Num() const
	{
		return NumSites;
	}

	VORONOI_API void AddSites(const TArrayView<const FVector>& Sites, double SquaredDistSkipPtThreshold = 0.0f);

	VORONOI_API void ComputeAllCellsSerial(TArray<FVoronoiCellInfo>& AllCells);
	VORONOI_API void ComputeAllCells(TArray<FVoronoiCellInfo>& AllCells, int32 ApproxSitesPerThread = -1);

	VORONOI_API void ComputeAllNeighbors(TArray<TArray<int32>>& AllNeighbors, bool bExcludeBounds = true, int32 ApproxSitesPerThread = -1);

	VORONOI_API void ComputeCellEdgesSerial(TArray<TTuple<FVector, FVector>>& Edges, TArray<int32>& CellMember);
	VORONOI_API void ComputeCellEdges(TArray<TTuple<FVector, FVector>>& Edges, TArray<int32>& CellMember, int32 ApproxSitesPerThread = -1);

	/**
	 * Find the closest Voronoi cell to a given position
	 * 
	 * @param Pos				Position to query for closest point
	 * @param ComputeHelper		A valid helper as returned by GetComputeHelper() for this voronoi diagram
	 * @param OutFoundSite		The position of the closest Voronoi site, if any
	 * @return					The id of the Voronoi cell containing the given position, or -1 if position is outside diagram
	 */
	VORONOI_API int32 FindCell(const FVector& Pos, FVoronoiComputeHelper& ComputeHelper, FVector& OutFoundSite) const;

	int32 FindCell(const FVector& Pos, FVoronoiComputeHelper& ComputeHelper) const
	{
		FVector OutFoundSite;
		return FindCell(Pos, ComputeHelper, OutFoundSite);
	}

	VORONOI_API FVoronoiComputeHelper GetComputeHelper() const;

	int32 ApproxSitesPerThreadWithDefault(int32 ApproxSitesPerThreadIn)
	{
		if (ApproxSitesPerThreadIn < 0)
		{
			return FMath::Max(MinDefaultSitesPerThread, NumSites / 64);
		}
		return ApproxSitesPerThreadIn;
	}

private:
	VORONOI_API TArray<int32> GetParallelBlockRanges(int32 ApproxSitesPerThread);
};

/**
 * Use a Voronoi diagram to support faster querying of Voronoi-like data from arbitrary sample points:
 *  - Distance to closest point
 *  - ID of closest point
 *  - IDs of closest two points
 *  - Distance to closest Voronoi cell boundary (not considering bounding box walls)
 * 
 * Note: Queries are only thread safe if each thread has its own FVoronoiComputeHelper
 */
class FVoronoiDiagramField
{
	TArray<FVector> Sites;
	TArray<TArray<int32>> Neighbors;
	FVoronoiDiagram Diagram;

public:
	FVoronoiDiagramField() = default;

	FVoronoiDiagramField(const TArray<FVector>& SitesIn, const FBox& Bounds, double SquaredDistSkipPtThreshold = DBL_EPSILON)
	{
		Initialize(SitesIn, Bounds, SquaredDistSkipPtThreshold);
	}

	void Initialize(const TArray<FVector>& SitesIn, const FBox& Bounds, double SquaredDistSkipPtThreshold = DBL_EPSILON)
	{
		Sites = SitesIn;
		Diagram.Initialize(Sites, Bounds, 0, SquaredDistSkipPtThreshold);
		Diagram.ComputeAllNeighbors(Neighbors, true);
	}

	FVoronoiDiagramField(const FVoronoiDiagramField& Other) = delete;
	FVoronoiDiagramField& operator=(const FVoronoiDiagramField& Other) = delete;
	FVoronoiDiagramField(FVoronoiDiagramField&& Other) = default;
	FVoronoiDiagramField& operator=(FVoronoiDiagramField&& Other) = default;

	inline FVoronoiComputeHelper GetComputeHelper() const
	{
		return Diagram.GetComputeHelper();
	}

	// @return Distance to the nearest Voronoi cell boundary (ignoring bounding box walls), or InvalidValue if outside of bounds
	double DistanceToCellWall(const FVector& Sample, FVoronoiComputeHelper& ComputeHelper, double InvalidValue = TNumericLimits<double>::Lowest())
	{
		FVector CloseSite;
		int32 Cell = Diagram.FindCell(Sample, ComputeHelper, CloseSite);
		if (Cell < 0)
		{
			return InvalidValue;
		}
		double BestDistSq = DBL_MAX;
		int32 BestNbr = -1;
		for (int32 NbrCell : Neighbors[Cell])
		{
			double DistSq = FVector::DistSquared(Sample, Sites[NbrCell]);
			if (DistSq < BestDistSq)
			{
				BestNbr = NbrCell;
				BestDistSq = DistSq;
			}
		}
		if (BestNbr == -1)
		{
			return InvalidValue;
		}
		FVector NbrPos = Sites[BestNbr];
		FVector Mid = (NbrPos + CloseSite) * .5;
		FVector Normal = NbrPos - CloseSite;
		bool bNormalizeSuccess = Normal.Normalize();
		checkSlow(bNormalizeSuccess); // expect this can't happen because the Voronoi diagram can't be constructed w/ duplicate points
		// Note: Returns a signed plane distance, but should always be positive because we know Sample is closer to CloseSite
		return (Mid - Sample).Dot(Normal);
	}

	TPair<int32, int32> ClosestTwoIDs(const FVector& Sample, FVoronoiComputeHelper& ComputeHelper)
	{
		TPair<int32, int32> ToRet(-1, -1);
		FVector CloseSite;
		ToRet.Key = Diagram.FindCell(Sample, ComputeHelper, CloseSite);
		if (ToRet.Key < 0)
		{
			return ToRet;
		}
		double BestDistSq = DBL_MAX;
		ToRet.Value = -1;
		for (int32 NbrCell : Neighbors[ToRet.Key])
		{
			double DistSq = FVector::DistSquared(Sample, Sites[NbrCell]);
			if (DistSq < BestDistSq)
			{
				ToRet.Value = NbrCell;
				BestDistSq = DistSq;
			}
		}
		return ToRet;
	}

	// @return Distance to the nearest Voronoi site, or InvalidValue if outside of bounds
	inline double DistanceToClosest(const FVector& Sample, FVoronoiComputeHelper& ComputeHelper, double InvalidValue = TNumericLimits<double>::Lowest())
	{
		FVector CloseSite;
		int32 Cell = Diagram.FindCell(Sample, ComputeHelper, CloseSite);
		if (Cell < 0)
		{
			return InvalidValue;
		}
		return FVector::Distance(Sample, CloseSite);
	}

	inline int32 ClosestID(const FVector& Sample, FVoronoiComputeHelper& ComputeHelper)
	{
		return Diagram.FindCell(Sample, ComputeHelper);
	}
};
