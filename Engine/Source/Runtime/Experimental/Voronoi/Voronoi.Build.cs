// Copyright Epic Games, Inc. All Rights Reserved.
using System;
namespace UnrealBuildTool.Rules
{
	public class Voronoi : ModuleRules
	{
        public Voronoi(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject"
                }
                );

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

			// Disable all static analysis checkers for this module ( third party code ) 
			bDisableStaticAnalysis = true;
		}
	}
}
